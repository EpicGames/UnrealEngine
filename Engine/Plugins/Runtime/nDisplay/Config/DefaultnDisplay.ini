[CoreRedirects]
+StructRedirects=(OldName="DisplayClusterConfigurationMasterNode",NewName="DisplayClusterConfigurationPrimaryNode")
+StructRedirects=(OldName="DisplayClusterConfigurationMasterNodePorts",NewName="DisplayClusterConfigurationPrimaryNodePorts")

+PropertyRedirects=(OldName="DisplayClusterConfigurationCluster.MasterNode",NewName="DisplayClusterConfigurationCluster.PrimaryNode")

+ClassRedirects=(OldName="/Script/DisplayClusterMedia.SharedMemoryMediaOutput", NewName="/Script/SharedMemoryMedia.SharedMemoryMediaOutput")
+ClassRedirects=(OldName="/Script/DisplayClusterMedia.SharedMemoryMediaCapture", NewName="/Script/SharedMemoryMedia.SharedMemoryMediaCapture")
+ClassRedirects=(OldName="/Script/DisplayClusterMedia.SharedMemoryMediaSource", NewName="/Script/SharedMemoryMedia.SharedMemoryMediaSource")
+ClassRedirects=(OldName="/Script/DisplayClusterMedia.SharedMemoryMediaPlayer", NewName="/Script/SharedMemoryMedia.SharedMemoryMediaPlayer")

; 5.4
+StructRedirects=(OldName="/Script/DisplayClusterConfiguration.DisplayClusterConfigurationMedia", NewName="/Script/DisplayClusterConfiguration.DisplayClusterConfigurationMediaViewport")
