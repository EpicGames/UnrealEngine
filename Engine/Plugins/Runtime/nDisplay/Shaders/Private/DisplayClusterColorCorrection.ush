// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"
#include "/Engine/Private/GammaCorrectionCommon.ush"

#ifndef REFACTORING_ALLOWED
// Empty definition when REFACTORING_ALLOWED flag is not defined,
// per specification: https://learn.microsoft.com/en-us/windows/win32/direct3dhlsl/precise
#define precise
#endif

// This function copied from the `\\Engine\Plugins\Compositing\OpenColorIO\Shaders\Private\OpenColorIOShader.usf`
// Note: For our (zero-tolerance) tests to succeed against a no-op, precise is required with the pow function.
float3 DisplayGammaCorrection(precise float3 InColor, precise float InGamma)
{
	return sign(InColor) * pow(abs(InColor), InGamma);
}

float SafeRcp(float x)
{
	return x > 0.0 ? rcp(x) : 0.0;
}

float4 Unpremultiply(float4 InColor)
{
	return float4(InColor.xyz * SafeRcp(InColor.w), InColor.w);
}

float4 Premultiply(float4 InColor)
{
	return float4(InColor.xyz * InColor.w, InColor.w);
}

float4 InvertedUnpremultiply(float4 InColor)
{
	return float4(InColor.xyz * SafeRcp(1.0f-InColor.w), InColor.w);
}

float4 InvertedPremultiply(float4 InColor)
{
	return float4(InColor.xyz * (1.0f-InColor.w), InColor.w);
}
