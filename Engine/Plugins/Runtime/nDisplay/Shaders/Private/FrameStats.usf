// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"
#include "/Engine/Private/MiniFontCommon.ush"

uint FrameCount;
uint Timecode;
static const uint Semicolon = 58;
static const float4 FontColor = float4(0.1f, 0.9f, 0.1f, 0.0f);

// Function taken from MiniFontCommon.ush, but with alpha support
void PrintCharacterA(int2 PixelPos, inout float4 OutColor, float4 FontColor, inout int2 LeftTop, int CharacterID)
{
	uint2 Rel = (uint2)(PixelPos - LeftTop);

	// Background: black character as shadow
	FLATTEN if (Rel.x < (NATIVE_CHARACTER_RES + 1) && Rel.y < (NATIVE_CHARACTER_RES + 1))
	{
		OutColor = lerp(OutColor, float4(0.0f, 0.0f, 0.0f, 0.0f), SampleMiniFont(CharacterID, Rel - uint2(1, 1)));
	}
	// Foreground: colored character
	FLATTEN if (Rel.x < (NATIVE_CHARACTER_RES) && Rel.y < (NATIVE_CHARACTER_RES))
	{
		OutColor = lerp(OutColor, FontColor, SampleMiniFont(CharacterID, Rel));
	}

	LeftTop.x += NATIVE_CHARACTER_RES;
}

void PrintTwoDigitUintA(int2 PixelPos, inout float4 OutColor, float4 FontColor, inout int2 Cursor, uint Number)
{
	float FloatNumber = float(Number) + 0.05f;

	PrintCharacterA(PixelPos, OutColor, FontColor, Cursor, ExtractDigitFromFloat(FloatNumber, 10));
	PrintCharacterA(PixelPos, OutColor, FontColor, Cursor, ExtractDigitFromFloat(FloatNumber, 1));
}

void PrintLargeUintA(int2 PixelPos, inout float4 OutColor, float4 FontColor, inout int2 Cursor, uint Number)
{
	float FloatNumber = float(Number) + 0.05f;

	// before period
	PrintCharacterA(PixelPos, OutColor, FontColor, Cursor, ExtractDigitFromFloat(FloatNumber, 1000000));
	PrintCharacterA(PixelPos, OutColor, FontColor, Cursor, ExtractDigitFromFloat(FloatNumber, 100000));
	PrintCharacterA(PixelPos, OutColor, FontColor, Cursor, ExtractDigitFromFloat(FloatNumber, 10000));
	PrintCharacterA(PixelPos, OutColor, FontColor, Cursor, ExtractDigitFromFloat(FloatNumber, 1000));
	PrintCharacterA(PixelPos, OutColor, FontColor, Cursor, ExtractDigitFromFloat(FloatNumber, 100));
	PrintCharacterA(PixelPos, OutColor, FontColor, Cursor, ExtractDigitFromFloat(FloatNumber, 10));
	PrintCharacterA(PixelPos, OutColor, FontColor, Cursor, ExtractDigitFromFloat(FloatNumber, 1));
}

void MainPS(in noperspective float4 InUVAndScreenPos : TEXCOORD0,
	float4 SvPosition : SV_POSITION,
	out float4 OutColor : SV_Target0)
{
	const uint HoursTC   = (Timecode >> 24) & 0xFF;
	const uint MinutesTC = (Timecode >> 16) & 0xFF;
	const uint SecondsTC = (Timecode >> 8)  & 0xFF;
	const uint FramesTC  = (Timecode >> 0)  & 0xFF;
	const int2 PixelPos = int2(SvPosition.xy);
	
	// Inverted-alpha to match the translucency of scene color
	OutColor = float4(0.0f, 0.0f, 0.0f, 1.0f);
	
	int2 Cursor = 1;
	const int StartX = Cursor.x;

	PrintTwoDigitUintA(PixelPos, OutColor, FontColor, Cursor, HoursTC);
	PrintCharacterA(PixelPos, OutColor, FontColor, Cursor, Semicolon);
	PrintTwoDigitUintA(PixelPos, OutColor, FontColor, Cursor, MinutesTC);
	PrintCharacterA(PixelPos, OutColor, FontColor, Cursor, Semicolon);
	PrintTwoDigitUintA(PixelPos, OutColor, FontColor, Cursor, SecondsTC);
	PrintCharacterA(PixelPos, OutColor, FontColor, Cursor, Semicolon);
	PrintTwoDigitUintA(PixelPos, OutColor, FontColor, Cursor, FramesTC);

	Cursor.x = StartX;
	Cursor.y += 12;
	
	PrintLargeUintA(PixelPos, OutColor, FontColor, Cursor, FrameCount);
}

Texture2D		InputTexture;
SamplerState	InputTextureSampler;
Texture2D		FrameStatsTexture;
SamplerState	FrameStatsTextureSampler;

void MainOutputPS(in noperspective float4 InUVAndScreenPos : TEXCOORD0,
	float4 SvPosition : SV_POSITION,
	out float4 OutColor : SV_Target0)
{
	uint2 PhysicalDimension;
	FrameStatsTexture.GetDimensions(PhysicalDimension.x, PhysicalDimension.y);
	
	float4 SceneColor = InputTexture.Sample(InputTextureSampler, InUVAndScreenPos.xy);
	float4 FrameStats = FrameStatsTexture.Sample(FrameStatsTextureSampler, 0.5f * SvPosition.xy/float2(PhysicalDimension));

	OutColor = FrameStats + SceneColor * FrameStats.a;
}
