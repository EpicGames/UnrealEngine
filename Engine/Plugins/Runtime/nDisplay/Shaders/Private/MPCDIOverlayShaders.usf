// Copyright Epic Games, Inc. All Rights Reserved.
#include "MPCDIUtils.ush"

static const float DefaultChromakeyTileMultiplier = 10.0;

OutputVS IcvfxWarpVS(InputVS IN)
{
	OutputVS OUT;

#if MESH_WARP
	OUT.PFMPosition = mul(float4(IN.Position.xyz,1.f), MeshToStageProjectionMatrix).xyz;
	OUT.UV_Chromakey = float4(IN.UV_Chromakey, 0.f, 1.f);
	float4 ScreenPosition = float4(IN.UV.xy, 0, 1);
#else
	float4 ScreenPosition = IN.Position;	
#endif

	DrawRectangle(ScreenPosition, IN.UV, OUT.Position, OUT.UV.xy);
	OUT.Position.zw = float2(0.f, 1.f);
	OUT.UV = float4(IN.UV, 0.f, 1.f);
	return OUT;
}

OutputPS Passthrough_PS(OutputVS IN)
{
	OutputPS OUT;

	float4 ViewportUV = mul(IN.UV, ViewportTextureProjectionMatrix);	
	float4 ViewportColor = InputTexture.Sample(InputSampler, ViewportUV.xy);

	OUT.Color = ViewportColor;
	return OUT;
}

float3 ComposeOverlayColor(float4 InColor, float4 InOverlayColor)
{
	return InOverlayColor.xyz + InColor.xyz * InOverlayColor.w;
}

/** Apply final gamma for LightCard color. */
float4 GetFinalLightcardColor(float4 InColor)
{
	// ICVFX expects all lightcards in final color space (with OCIO applied)
	// Gamma should not be converted after OCIO.
	return InColor;
}

// Additive render pass with 2 overlays
// a = in\out, b,c = ordered overlays (a + b + c)
// c.xyz + (b.xyz + a.xyz * b.w)*c.w = a.xyz * (b.w *c.w) + b.xyz*c.w + c.xyz
float4 ComposeRenderPassTwoOverlays(float4 InOverlay1, float4 InOverlay2)
{
	return float4(ComposeOverlayColor(InOverlay1, InOverlay2), InOverlay1.w * InOverlay2.w);
}

////////////////////////////////////////////////////////////////////////////////
// LightCard
////////////////////////////////////////////////////////////////////////////////
float4 GetLightCardUnderTextureColor(float2 InUV)
{
	return LightCardUnderTexture.Sample(LightCardUnderSampler, InUV);
}

float4 GetLightCardOverTextureColor(float2 InUV)
{
	return LightCardOverTexture.Sample(LightCardOverSampler, InUV);
}

float4 GetUVLightCardUnderTextureColor(float2 InUV)
{
	float4 UVLightCardUnderColor = UVLightCardUnderTexture.Sample(UVLightCardUnderSampler, InUV);
	
	if (LightCardMode & UVLIGHTCARD_MERGE)
	{
		// UVLIGHTCARD_MERGE - special rendering mode, for Outer viewports that are forced to render LightCards only in "over" or "under" mode.
		// both UV lightcard textures are rendered under or over the inner frustum.

		float4 UVLightCardOverColor= UVLightCardOverTexture.Sample(UVLightCardOverSampler, InUV);

		return ComposeRenderPassTwoOverlays(UVLightCardUnderColor, UVLightCardOverColor);
	}

	return UVLightCardUnderColor;
}

float4 GetUVLightCardOverTextureColor(float2 InUV)
{
	float4 UVLightCardOverColor = UVLightCardOverTexture.Sample(UVLightCardOverSampler, InUV);

	if (LightCardMode & UVLIGHTCARD_MERGE)
	{
		// UVLIGHTCARD_MERGE - special rendering mode, for Outer viewports that are forced to render LightCards only in "over" or "under" mode.
		// both UV lightcard textures are rendered under or over the inner frustum.

		float4 UVLightCardUnderColor = UVLightCardUnderTexture.Sample(UVLightCardUnderSampler, InUV);

		return ComposeRenderPassTwoOverlays(UVLightCardUnderColor, UVLightCardOverColor);
	}

	return UVLightCardOverColor;
}

// This function expects that (LightCardMode & (LIGHTCARD_UNDER | UVLIGHTCARD_UNDER)) != 0
float4 GetLightCardUnderColor(OutputVS IN, float2 OverlayUV)
{
// Only the mesh has a UV_Chromakey parameter.
#if MESH_WARP
	if( (LightCardMode & (LIGHTCARD_UNDER | UVLIGHTCARD_UNDER)) == (LIGHTCARD_UNDER | UVLIGHTCARD_UNDER))
	{
		return ComposeRenderPassTwoOverlays(GetLightCardUnderTextureColor(OverlayUV), GetUVLightCardUnderTextureColor(IN.UV_Chromakey.xy));
	}

	if(LightCardMode & UVLIGHTCARD_UNDER)
	{
		return GetUVLightCardUnderTextureColor(IN.UV_Chromakey.xy);
	}
#endif

	return GetLightCardUnderTextureColor(OverlayUV);
}


// This function expects that (LightCardMode & (LIGHTCARD_OVER | UVLIGHTCARD_OVER)) != 0
float4 GetLightCardOverColor(OutputVS IN, float2 OverlayUV)
{
// Only the mesh has a UV_Chromakey parameter.
#if MESH_WARP
	if( (LightCardMode & (LIGHTCARD_OVER | UVLIGHTCARD_OVER)) == (LIGHTCARD_OVER | UVLIGHTCARD_OVER))
	{
		return ComposeRenderPassTwoOverlays(GetLightCardOverTextureColor(OverlayUV), GetUVLightCardOverTextureColor(IN.UV_Chromakey.xy));
	}

	if(LightCardMode & UVLIGHTCARD_OVER)
	{
		return GetUVLightCardOverTextureColor(IN.UV_Chromakey.xy);
	}
#endif

	return GetLightCardOverTextureColor(OverlayUV);
}

float3 ComposeLightCardUnderColor(float4 InColor, OutputVS IN, float2 OverlayUV)
{
	if( LightCardMode & (LIGHTCARD_UNDER | UVLIGHTCARD_UNDER))
	{
		float4 LightCardUnderColor = GetLightCardUnderColor(IN, OverlayUV);

		// Convert final LC color from linear to the output color space
		LightCardUnderColor = GetFinalLightcardColor(LightCardUnderColor);

		return ComposeOverlayColor(InColor, LightCardUnderColor);
	}
	
	return InColor.xyz;
}

float3 ComposeLightCardOverColor(float4 InColor, OutputVS IN, float2 OverlayUV)
{
	if( LightCardMode & (LIGHTCARD_OVER | UVLIGHTCARD_OVER))
	{
		float4 LightCardOverColor = GetLightCardOverColor(IN, OverlayUV);

		// Convert final LC color from linear to the output color space
		LightCardOverColor = GetFinalLightcardColor(LightCardOverColor);

		return ComposeOverlayColor(InColor, LightCardOverColor);
	}
	
	return InColor.xyz;
}

////////////////////////////////////////////////////////////////////////////////
float2 TileUV(float2 InUV, float InScale)
{
	return frac(InUV * InScale);
}

float GetFeatherAlpha(float2 EdgeOffset, float Feather)
{
	float Alpha = 1;

	// do not change the alpha within an in-camera frame
	if (EdgeOffset.x < 1)
	{
		Alpha *= smoothstep(0, Feather, EdgeOffset.x); // horizontal component
	}
	
	// do not change the alpha within an in-camera frame
	if (EdgeOffset.y < 1)
	{
		Alpha *= smoothstep(0, Feather, EdgeOffset.y); // vertical component
	}

	return Alpha;
}

float GetInnerCameraSoftEdgeAlpha(float2 CameraUV, float4 InInnerCameraSoftEdge)
{
	float2 EdgeOffset = float2(1, 1);

	// InInnerCameraSoftEdge.x values are used to pass left/right soft edge values
	if(CameraUV.x < InInnerCameraSoftEdge.x) // left 
	{
		float CameraTextureAlphaLeft = Pow2(saturate(abs(CameraUV.x) / InInnerCameraSoftEdge.x));
		EdgeOffset.x = CameraTextureAlphaLeft;
	}
	else
	if(CameraUV.x > (1 - InInnerCameraSoftEdge.x)) // right
	{
		float CameraTextureAlphaRight = Pow2(saturate(abs(1 - CameraUV.x) / InInnerCameraSoftEdge.x));
		EdgeOffset.x = CameraTextureAlphaRight;
	}

	// InInnerCameraSoftEdge.y values are used to pass top/bottom soft edge values
	if(CameraUV.y < (InInnerCameraSoftEdge.y)) // top
	{
		float CameraTextureAlphaTop = Pow2(saturate(abs(CameraUV.y) / InInnerCameraSoftEdge.y));
		EdgeOffset.y = CameraTextureAlphaTop;
	}
	else
	if(CameraUV.y > (1 - InInnerCameraSoftEdge.y)) // bottom
	{
		float CameraTextureAlphaBottom = Pow2(saturate(abs(1 - CameraUV.y) / InInnerCameraSoftEdge.y));
		EdgeOffset.y = CameraTextureAlphaBottom;
	}

	// InInnerCameraSoftEdge.z are used to pass Feather soft edge values
	float Feather = InInnerCameraSoftEdge.z;

	return GetFeatherAlpha(EdgeOffset, Feather);
}

float4 GetChromakeyMarkerColor(float2 InMarkersUV)
{
	// Tile scale is inversely proprotional to the distance
	float TileScale = DefaultChromakeyTileMultiplier / ChromakeyMarkerDistance;
	
	// Invert vertical axis for more intuitive controls
	float2 InvertedChromakeyMarkerOffset = ChromakeyMarkerOffset;
	InvertedChromakeyMarkerOffset.y = -ChromakeyMarkerOffset.y;

	float2 ChromakeyMarkerUV = TileUV(InMarkersUV - InvertedChromakeyMarkerOffset/TileScale, TileScale);

	// Scale individual tile from its center, and independently from the distance
	float2 HalfTile = float2(0.5, 0.5);
	ChromakeyMarkerUV -= HalfTile;
	ChromakeyMarkerUV *= ChromakeyMarkerDistance / ChromakeyMarkerScale;
	float4 ChromakeyMarkerTextureColor = ChromakeyMarkerTexture.Sample(ChromakeyMarkerSampler, ChromakeyMarkerUV + HalfTile);

	// Anything outside scaled UV bounds is untextured
	if(any(abs(ChromakeyMarkerUV) > HalfTile))
	{
		return float4(ChromakeyMarkerColor.xyz, 0);
	}

	return float4(ChromakeyMarkerColor.xyz, ChromakeyMarkerTextureColor.w);
}

float4 GetInnerCameraColor(float4 WarpedUV, OutputVS IN)
{
	// Transform WarpedUV to Camera ScreenSpaceUV
	float4 CameraUVW = mul(WarpedUV, InnerCameraProjectionMatrix);
	float2 CameraUV = CameraUVW.xy / CameraUVW.w;

#if CHROMAKEYFRAMECOLOR
	float4 CameraColor = float4(0,0,0,0);
#else
	// ** Incamera Frame **
	float3 CameraBaseColor = InnerCameraTexture.Sample(InnerCameraSampler, CameraUV).rgb;
	float4 CameraColor = float4(CameraBaseColor, 0);
#endif

	// ********* Compose camera alpha **********
	if (CameraUVW.w > 0) // clip back plane
	{
		//Defined texel:
		float2 ToEdge = (CameraUV.xy * 2) - 1.0f; // -1..1
		float Weight = 1 - max(abs(ToEdge.x), abs(ToEdge.y));

		// Near clip Plane tests 
		if (Weight >= 0)
		{
			// *** InCamera Soft Edges ***
			CameraColor.w = GetInnerCameraSoftEdgeAlpha(CameraUV, InnerCameraSoftEdge);

			// **** disable incamera render inside overlapped areas
			#if ENABLE_INNER_CAMERA_OVERLAP && !INNER_CAMERA_OVERLAP
			if(OverlappedInnerCamerasNum > 0 && CameraColor.w < 1)
			{
				for(uint OverlappedCameraIndex = 0; OverlappedCameraIndex < OverlappedInnerCamerasNum; OverlappedCameraIndex++)
				{
					float4x4 InnerCameraProjectionMatrix2 = OverlappedInnerCamerasProjectionMatrices[OverlappedCameraIndex];

					// Transform WarpedUV to Camera ScreenSpaceUV
					float4 CameraUVW2 = mul(WarpedUV, InnerCameraProjectionMatrix2);
					if (CameraUVW2.w > 0) // clip back plane
					{
						float2 CameraUV2 = CameraUVW2.xy / CameraUVW2.w;
						float2 ToEdge2 = (CameraUV2.xy * 2) - 1.0f; // -1..1
						float Weight2 = 1 - max(abs(ToEdge2.x), abs(ToEdge2.y));
						if (Weight2 >= 0)
						{
							float UnderCameraAlpha = GetInnerCameraSoftEdgeAlpha(CameraUV2, OverlappedInnerCameraSoftEdges[OverlappedCameraIndex]);

							//!
							if(UnderCameraAlpha > 0.1)
							{
								// This pixel was overlapped. Do not render
								CameraColor.w = 0;
							}
						}
					}
				}
			}
			#endif

			// ********* Compose camera colors **********
			#if CHROMAKEYFRAMECOLOR || CHROMAKEY
				// Use frame color by default
				float4 OverlayChromakeyColor = float4(ChromakeyColor.xyz,0);

				#if CHROMAKEY
					float4 ChromakeyCameraTextureColor = ChromakeyCameraTexture.Sample(ChromakeyCameraSampler, CameraUV);

					// Get alpha from chromakey rtt frame
					OverlayChromakeyColor.w = ChromakeyCameraTextureColor.w;
				#endif

				#if CHROMAKEY_MARKER
					float4 ChromakeyMarkerTextureColor = GetChromakeyMarkerColor(IN.UV_Chromakey.xy);

					// Blend marker color with chromakey
					OverlayChromakeyColor.xyz = lerp(OverlayChromakeyColor.xyz, ChromakeyMarkerTextureColor.xyz, ChromakeyMarkerTextureColor.w);
				#endif

				// ********* When camera overlap we render chromakey **********
				#if ENABLE_INNER_CAMERA_OVERLAP && INNER_CAMERA_OVERLAP
					if(CameraColor.w >= 0)
					{
						OverlayChromakeyColor.w = 0;

						// disable chromakey
						float OverlayAlphaMult = 0;

						for(uint OverlappedCameraIndex = 0; OverlappedCameraIndex < OverlappedInnerCamerasNum; OverlappedCameraIndex++)
						{
							float4x4 InnerCameraProjectionMatrix2 = OverlappedInnerCamerasProjectionMatrices[OverlappedCameraIndex];

							// Transform WarpedUV to Camera ScreenSpaceUV
							float4 CameraUVW2 = mul(WarpedUV, InnerCameraProjectionMatrix2);
							if (CameraUVW2.w > 0) // clip back plane
							{
								float2 CameraUV2 = CameraUVW2.xy / CameraUVW2.w;
								float2 ToEdge2 = (CameraUV2.xy * 2) - 1.0f; // -1..1
								float Weight2 = 1 - max(abs(ToEdge2.x), abs(ToEdge2.y));
								if (Weight2 >= 0)
								{
									// This pixel was overlapped
									// force to render Chromakey
									OverlayAlphaMult = max(OverlayAlphaMult, GetInnerCameraSoftEdgeAlpha(CameraUV2, OverlappedInnerCameraSoftEdges[OverlappedCameraIndex]));
								}
							}
						}

						CameraColor.w = CameraColor.w * OverlayAlphaMult;
					}
				#endif

				CameraColor.xyz = lerp(CameraColor.xyz, OverlayChromakeyColor.xyz, 1-OverlayChromakeyColor.w);

			#endif

			// *** InCamera Border ***
			if(InnerCameraBorderThickness>0)
			{
				if((ToEdge.x < (InnerCameraBorderThickness / InnerCameraFrameAspectRatio - 1)) || (ToEdge.x > (1 - InnerCameraBorderThickness / InnerCameraFrameAspectRatio)) ||
					(ToEdge.y < (InnerCameraBorderThickness - 1)) || (ToEdge.y > (1 - InnerCameraBorderThickness)))
				{
					CameraColor.rgb = InnerCameraBorderColor.xyz;
					CameraColor.w = 1.0f;
				}
			}
		}
	}

	return CameraColor;
}

OutputPS IcvfxWarpPS(OutputVS IN)
{
	OutputPS OUT;

	// Load warped UV
#if MESH_WARP
	float4 WarpedUV = float4(IN.PFMPosition, 1.f);
#else
	float4 WarpedUV = WarpMapTexture.Sample(WarpMapSampler, IN.UV.xy);
#endif
	
	// Transform WarpedUV to ScreenSpaceUV
	float4 ViewportUVW = mul(WarpedUV, ViewportTextureProjectionMatrix);
	float2 ViewportUV = ViewportUVW.xy / ViewportUVW.w;


	float4 OverlayUVW = mul(WarpedUV, OverlayProjectionMatrix);
	float2 OverlayUV = OverlayUVW.xy / OverlayUVW.w;

	// Compose all:
#if VIEWPORT_INPUT

	// First and single pass renders:

	#if VIEWPORT_INPUT_ALPHA
		float4 OutColor = float4(InputTexture.Sample(InputSampler, ViewportUV));
	#else
		float4 OutColor = float4(InputTexture.Sample(InputSampler, ViewportUV).rgb, SCENE_ALPHA_OPAQUE);
	#endif

	
	// Compose a "Under" lightcard, if used.
	OutColor.xyz = ComposeLightCardUnderColor(OutColor, IN, OverlayUV);
	
	#if INNER_CAMERA
		float4 CameraColor = GetInnerCameraColor(WarpedUV, IN);
		OutColor.xyz = lerp(OutColor.xyz, CameraColor.xyz, CameraColor.w);
	#endif


	// Compose a "Over" lightcard, if used.
	OutColor.xyz = ComposeLightCardOverColor(OutColor,IN, OverlayUV);

#else 
	float4 OutColor;

	// MultiCam or Final renderpass (only cam or overlay per pass))
	#if INNER_CAMERA
		OutColor = GetInnerCameraColor(WarpedUV, IN);
	#else
		if( LightCardMode & (LIGHTCARD_OVER | UVLIGHTCARD_OVER))
		{
			OutColor = GetLightCardOverColor(IN, OverlayUV);

			// Convert final LC color from linear to the output color space
			OutColor = GetFinalLightcardColor(OutColor);
		}
		else
		{
			OutColor = float4(0,0,0,0);
		}
	#endif

#endif

	//@todo: add LUT here

	// Apply final mpcdi color blending
	OUT.Color = ApplyBlending(OutColor.xyz, IN, OutColor.w);
	return OUT;
}
