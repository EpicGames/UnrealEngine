// Copyright Epic Games, Inc. All Rights Reserved.
#include "MPCDIUtils.ush"

OutputVS WarpVS(InputVS IN)
{
	OutputVS OUT;

#if MESH_WARP
	OUT.PFMPosition = mul(float4(IN.Position.xyz, 1.f), MeshToStageProjectionMatrix).xyz;
	OUT.UV_Chromakey = float4(IN.UV_Chromakey, 0.f, 1.f);
	float4 ScreenPosition = float4(IN.UV.xy, 0, 1);
#else
	float4 ScreenPosition = IN.Position;
#endif

	DrawRectangle(ScreenPosition, IN.UV, OUT.Position, OUT.UV.xy);
	OUT.Position.zw = float2(0.f, 1.f);
	OUT.UV = float4(IN.UV, 0.f, 1.f);
	return OUT;
}

OutputPS Passthrough_PS(OutputVS IN)
{
	OutputPS OUT;

	float4 ViewportUV = mul(IN.UV, ViewportTextureProjectionMatrix);
	float4 ViewportColor = InputTexture.Sample(InputSampler, ViewportUV.xy);

	OUT.Color = ViewportColor;
	return OUT;
}

OutputPS WarpPS(OutputVS IN)
{
	OutputPS OUT;

	// Load warped UV
#if MESH_WARP
	float4 WarpedUV = float4(IN.PFMPosition, 1.f);
#else
	float4 WarpedUV = WarpMapTexture.Sample(WarpMapSampler, IN.UV.xy);
#endif

	// Transform WarpedUV to ScreenSpaceUV
	float4 ViewportUVW = mul(WarpedUV, ViewportTextureProjectionMatrix);
	float2 ViewportUV = ViewportUVW.xy / ViewportUVW.w;

#if VIEWPORT_INPUT_ALPHA
	float4 OutColor = float4(InputTexture.Sample(InputSampler, ViewportUV));
#else
	float4 OutColor = float4(InputTexture.Sample(InputSampler, ViewportUV).rgb, SCENE_ALPHA_OPAQUE);
#endif

	//@todo: add LUT here

	// Apply final mpcdi color blending
	OUT.Color = ApplyBlending(OutColor.xyz, IN, OutColor.w);
	return OUT;
}
