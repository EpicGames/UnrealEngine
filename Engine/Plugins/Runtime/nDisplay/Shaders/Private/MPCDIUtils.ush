// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"
#include "DisplayClusterColorCorrection.ush"

//VS Parameters
float4 DrawRectanglePosScaleBias;
float4 DrawRectangleInvTargetSizeAndTextureSize;
float4 DrawRectangleUVScaleBias;

// PS parameters
Texture2D InputTexture;
Texture2D WarpMapTexture;

// AlphaMap and BetaMap textures.
Texture2D AlphaMapTexture;
Texture2D BetaMapTexture;

// Indicates the number of components within the alpha or beta map. Valid values are 1 or 3.
int AlphaMapComponentDepth;
int BetaMapComponentDepth;

// This parameter contains the 'gammaEmbedded' tag value, which is the gamma value for the AlphaMap texture.
// Note: The MPCDI 2.0 standard does not define an 'EmbeddedGamma' tag value for the BetaMap tag.
// However, it does require gamma correction for BetaMap.
// Therefore, this parameter is also used for the BetaMap texture.
float AlphaMapGammaEmbedded;

Texture2D InnerCameraTexture;
Texture2D ChromakeyCameraTexture;
Texture2D ChromakeyMarkerTexture;

Texture2D LightCardUnderTexture;
Texture2D LightCardOverTexture;
Texture2D UVLightCardUnderTexture;
Texture2D UVLightCardOverTexture;

SamplerState InputSampler;
SamplerState WarpMapSampler;
SamplerState AlphaMapSampler;
SamplerState BetaMapSampler;

SamplerState InnerCameraSampler;
SamplerState ChromakeyCameraSampler;
SamplerState ChromakeyMarkerSampler;

SamplerState LightCardUnderSampler;
SamplerState LightCardOverSampler;
SamplerState UVLightCardUnderSampler;
SamplerState UVLightCardOverSampler;

float4x4 MeshToStageProjectionMatrix;
float4x4 ViewportTextureProjectionMatrix;
float4x4 OverlayProjectionMatrix;
float4x4 InnerCameraProjectionMatrix;

// The total number of InCameras, which are rendered below.
int OverlappedInnerCamerasNum;

#ifdef MAX_INNER_CAMERAS_AMOUNT
#define ENABLE_INNER_CAMERA_OVERLAP 1

// Matrices and SoftEdges for InCameras, which are rendered below.
float4x4 OverlappedInnerCamerasProjectionMatrices[MAX_INNER_CAMERAS_AMOUNT];
float4 OverlappedInnerCameraSoftEdges[MAX_INNER_CAMERAS_AMOUNT];
#else
#define ENABLE_INNER_CAMERA_OVERLAP 0
#endif

float LightCardGamma;

// These are the flags for the 'LightCardMode' parameter:

// LIGHTCARD_UNDER - LightCard is rendered under InCamera. The image from the "LightCardUnderTexture" texture is used.
#define LIGHTCARD_UNDER   (1 << 0)

// LIGHTCARD_OVER - LightCard is rendered over InCamera. The image from the "LightCardOverTexture" texture is used.
#define LIGHTCARD_OVER    (1 << 1)

// UVLIGHTCARD_UNDER - UV LightCard is rendered under InCamera. The image from the "UVLightCardUnderTexture" texture is used.
#define UVLIGHTCARD_UNDER (1 << 2)

// UVLIGHTCARD_OVER - UV LightCard is rendered over InCamera. The image from the "UVLightCardOverTexture" texture is used.
#define UVLIGHTCARD_OVER  (1 << 3)

// UVLIGHTCARD_MERGE - special rendering mode, for Outer viewports that are forced to render LightCards only in "over" or "under" mode.
// both UV lightcard textures are rendered under or over the inner frustum.
// Images from the "UVLightCardOverTexture" and "UVLightCardOverTexture" textures are used.
// Can only be combined in two ways: (UVLIGHTCARD_MERGE|UVLIGHTCARD_UNDER) or (UVLIGHTCARD_MERGE|UVLIGHTCARD_OVER)
#define UVLIGHTCARD_MERGE (1 << 4)

// The comment from PostProcessMaterialShaders.usf:327-331:
// In Unreal, an alpha channel value of 0.0f means that the pixel is opaque and a value of 1.0f means that there is translucency.
// This is the opposite of how the alpha channel is interpreted in formats like PNG. For this reason, rendered images that are exported
// from the engine will have their alpha channel inverted to match the PNG convention. However, for consistency inside the rendering chain,
// we need to stick to 0.0f as the default value.
#define SCENE_ALPHA_OPAQUE 0.0f

// Several flags in this parameter: LIGHTCARD_UNDER, LIGHTCARD_OVER, UVLIGHTCARD_UNDER, UVLIGHTCARD_OVER, UVLIGHTCARD_MERGE
int LightCardMode;

float4 InnerCameraSoftEdge;

float4 InnerCameraBorderColor;
float InnerCameraBorderThickness;
float InnerCameraFrameAspectRatio;

float4 ChromakeyColor;
float4 ChromakeyMarkerColor;

float ChromakeyMarkerScale;
float ChromakeyMarkerDistance;
float2 ChromakeyMarkerOffset;

struct InputVS
{
	float4 Position : ATTRIBUTE0;
	float2 UV : ATTRIBUTE1;
#if MESH_WARP
	float2 UV_Chromakey : ATTRIBUTE2;
#endif
};

struct OutputVS
{
	float4	Position : SV_POSITION;
	float4	UV : TEXCOORD0;
#if MESH_WARP
	float4	UV_Chromakey : TEXCOORD1;
	float3	PFMPosition : NORMAL;
#endif
};

struct OutputPS
{
	float4 Color : SV_Target0;
};

float Pow2(float X)
{
	return X * X;
}

float MPCDI_sRGBToLinear(float srgb)
{
	if (srgb <= 0.04045)
		return srgb / 12.92;
	else
		return pow((srgb + 0.055) / 1.055, 2.4);
}

float MPCDI_LinarTosRGB(float lin)
{
	if (lin <= 0.0031308)
		return lin * 12.92;
	else
		return 1.055 * pow(lin, 1.0 / 2.4) - 0.055;
}

float3 MPCDI_sRGBToLinear(float3 srgb)
{
	return float3(MPCDI_sRGBToLinear(srgb.r),
				  MPCDI_sRGBToLinear(srgb.g),
				  MPCDI_sRGBToLinear(srgb.b));
}

float3 MPCDI_LinearTosRGB(float3 lin)
{
	return float3(MPCDI_LinarTosRGB(lin.r),
				  MPCDI_LinarTosRGB(lin.g),
				  MPCDI_LinarTosRGB(lin.b));
}

/** Used for calculating vertex positions and UVs when drawing with DrawRectangle */
void DrawRectangle(in float4 InPosition, in float2 InTexCoord, out float4 OutPosition, out float2 OutTexCoord)
{
	OutPosition = InPosition;
	OutPosition.xy = -1.0f + 2.0f * (DrawRectanglePosScaleBias.zw + (InPosition.xy * DrawRectanglePosScaleBias.xy)) * DrawRectangleInvTargetSizeAndTextureSize.xy;
	OutPosition.xy *= float2(1, -1);
	OutTexCoord.xy = (DrawRectangleUVScaleBias.zw + (InTexCoord.xy * DrawRectangleUVScaleBias.xy)) * DrawRectangleInvTargetSizeAndTextureSize.zw;
}


float4 ApplyBlending(float3 InColor_sRGB, OutputVS IN, float ColorAlphaValue)
{
	// 2.4 Apply Alpha and Beta mapping [MPCDI 2.0]
	//
	// Intensity blending and black level adjustments in projector overlap regions are expressed in alpha
	// and beta maps respectively. These are expressed with the following equation.
	//
	//  Pf = (Pi * a * (1 - b)) + b
	//
	// where:
	//   Pi - is the initial color
	//   Pf - is final pixel color
	//    a - is the gamma-corrected intensity mapping color
	//    b - is the gamma-corrected black level offset color
	
	// BetaMap can be used by itself, without AlphaMap.
#if ALPHAMAP_BLENDING || BETAMAP_BLENDING
	float3 LinearColor = MPCDI_sRGBToLinear(InColor_sRGB);

	// To convert a texture between gamma and linear color space, simply use these formulas:
	// LinearColor = pow(sRGBColor, Gamma)
	//   sRGBColor = pow(LinearColor, 1.0 / Gamma)
	
	// Note: AlphaMapGammaEmbedded is used for both AlphaMap and BetaMap.
	// (The MPCDI 2.0 standard does not define an 'EmbeddedGamma' tag value for the BetaMap tag. However, it does require gamma correction for BetaMap.)
	
#if ALPHAMAP_BLENDING
	if(AlphaMapComponentDepth < 3)
	{
		// Grayscale AlphaMap: get a single-channel value and convert its color space to linear.
		float LinearAlphaMap = pow(AlphaMapTexture.Sample(AlphaMapSampler, IN.UV.xy).r, AlphaMapGammaEmbedded);
		LinearColor = LinearColor * LinearAlphaMap;
	}
	else
	{
		// RGB AlphaMap: get the values of the 3 texture channels and convert their color space to linear.
		float3 LinearAlphaMap = pow(AlphaMapTexture.Sample(AlphaMapSampler, IN.UV.xy).rgb, AlphaMapGammaEmbedded);
		LinearColor = LinearColor * LinearAlphaMap;
	}
#endif

#if BETAMAP_BLENDING
	if(BetaMapComponentDepth < 3)
	{
		// Grayscale BetaMap: get a single-channel value and convert its color space to linear.
		float LinearBetaMap = pow(BetaMapTexture.Sample(BetaMapSampler, IN.UV.xy).r, AlphaMapGammaEmbedded);
		LinearColor = LinearColor * ( 1 - LinearBetaMap) + LinearBetaMap;
	}
	else
	{
		// RGB BetaMap: get the values of the 3 texture channels and convert their color space to linear.
		float3 LinearBetaMap = pow(BetaMapTexture.Sample(BetaMapSampler, IN.UV.xy).rgb, AlphaMapGammaEmbedded);
		LinearColor = LinearColor * ( 1 - LinearBetaMap) + LinearBetaMap;
	}
#endif // BETAMAP_BLENDING

	return float4(MPCDI_LinearTosRGB(LinearColor), ColorAlphaValue);

#else // #if ALPHAMAP_BLENDING || BETAMAP_BLENDING
	
	return float4(InColor_sRGB, ColorAlphaValue);
	
#endif // #if ALPHAMAP_BLENDING || BETAMAP_BLENDING
}
