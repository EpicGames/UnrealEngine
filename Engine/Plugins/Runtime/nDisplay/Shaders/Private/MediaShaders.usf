// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/GammaCorrectionCommon.ush"

Texture2D InputTexture;
SamplerState InputSampler;


// Pixel shader for Linear-PQ conversion
float4 LinearToPQ_PS(float2 uv : TEXCOORD) : SV_Target
{
	float4 colorLinear = InputTexture.Sample(InputSampler, uv);
	float3 colorPQ = LinearToST2084(colorLinear.rgb);
	colorPQ = saturate(colorPQ);

	return float4(colorPQ, colorLinear.a);
}


// Pixel shader for PQ-Linear conversion
float4 PQToLinear_PS(float2 uv : TEXCOORD) : SV_Target
{
	float4 colorPQ = InputTexture.Sample(InputSampler, uv);
	float3 colorLinear = ST2084ToLinear(colorPQ.rgb);

	return float4(colorLinear, colorPQ.a);
}
