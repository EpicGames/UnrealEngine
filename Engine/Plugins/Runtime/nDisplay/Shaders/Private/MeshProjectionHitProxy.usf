// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "/Engine/Generated/Material.ush"
#include "/Engine/Generated/VertexFactory.ush"

float4 HitProxyId;

void Main(FVertexFactoryInterpolantsVSToPS FactoryInterpolants, in INPUT_POSITION_QUALIFIERS float4 SvPosition : SV_Position OPTIONAL_IsFrontFace, out float4 OutColor : SV_Target0)
{
	FMaterialPixelParameters MaterialParameters = GetMaterialPixelParameters(FactoryInterpolants, SvPosition);
	FPixelMaterialInputs PixelMaterialInputs;
	CalcMaterialParameters(MaterialParameters, PixelMaterialInputs, SvPosition, bIsFrontFace);

	GetMaterialCoverageAndClipping(MaterialParameters, PixelMaterialInputs);

	OutColor = HitProxyId;
}