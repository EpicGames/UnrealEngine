// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/ScreenPass.ush"

SCREEN_PASS_TEXTURE_VIEWPORT(Input)

Texture2D SceneColor;
Texture2D SceneDepth;
Texture2D<uint2> SceneStencil;

RWTexture2D<float4> RWColorIn;
RWTexture2D<float4> RWColorOut;
RWTexture2D<float> RWDepthIn;
RWTexture2D<float> RWDepthOut;
RWTexture2D<uint> RWStencilIn;
RWTexture2D<uint> RWStencilOut;
RWTexture2D<uint> RWBlurStencilIn;
RWTexture2D<uint> RWBlurStencilOut;

float4x4 NormalCorrectionMatrix;

#define FILTER_KERNEL_SIZE 25

DECLARE_SCALAR_ARRAY(float, SpatialKernel, FILTER_KERNEL_SIZE);
DECLARE_SCALAR_ARRAY(float, InteriorSpatialKernel, FILTER_KERNEL_SIZE);

float2 SampleDirection;
float2 SampleOffsetScale;
float InvSigma;

/** Determines if the contents of the geometry stencil buffer should be copied to the blur stencil buffer during the dilation pass */
uint bCopyToBlurStencil;

/** Returns true if the current pixel is beyond the circular azimuthal bounds and should be clipped */
bool ClipToAzimuthalBounds(uint2 PixelPos)
{
	const float2 UV = (PixelPos - float2(0.5, 0.5)) * View.ViewSizeAndInvSize.zw;
	const float Radius = length(UV - float2(0.5, 0.5));

	return Radius >= 0.5;
}

/** Applies an offset to the specified pixel position, ensuring that the resulting offset position is within the azimuthal bounds */
uint2 AdjustPixelPos(uint2 PixelPos, int2 PixelOffset)
{
	uint2 OffsetPos = PixelPos + PixelOffset;

	// If the sample position would be outside the circular bounds of the azimuthal projection,
	// return the original pixel position instead
	uint2 SamplePos = select(ClipToAzimuthalBounds(OffsetPos), PixelPos, OffsetPos);
	
	return uint2(clamp(SamplePos.x, 0.0, Input_Extent.x - 1), clamp(SamplePos.y, 0.0, Input_Extent.y - 1));
}

/** Converts a vector in normal space to color space */
float3 NormalToColorSpace(float3 Normal)
{
	return Normal * 0.5 + 0.5;
}

/** Converts a color from color space to normal space */
float3 NormalFromColorSpace(float3 Color)
{
	return 2.0 * Color - 1.0;
}

/** Transforms the specified world vector into the radial space of the specified vector,
  * oriented so that the x axis points in the direction of the radial vector */
float3 TransformToRadialBasis(float3 Vector, float3 RadialVector)
{
	const float KINDA_SMALL_NUMBER = 0.0001;
	float3 Up = abs(RadialVector.z) < (1.0 - KINDA_SMALL_NUMBER) ? float3(0, 0, 1) : float3(1, 0, 0);
	
	float3 AzimuthalVector = normalize(cross(Up, RadialVector));
	float3 InclinationVector = cross(RadialVector, AzimuthalVector);
	
	return float3(dot(Vector, RadialVector), dot(Vector, AzimuthalVector), dot(Vector, InclinationVector));
}

/** A compute pass that copies the scene's color, depth, and stencil buffers into RW textures, as well as converting all normals from world space to radial space */
[numthreads(8, 8, 1)]
void CreateRWTexturesCS(uint2 DispatchThreadId : SV_DispatchThreadID)
{
	// In order to compute the radial space vector to convert the normals to radial space with, compute the world position of the current pixel
	// This assumes the scene was rendered using an azimuthal projection
	const uint2 PixelPos = DispatchThreadId;
	const float2 UV = PixelPos * View.ViewSizeAndInvSize.zw;
	const float2 ScreenPos = ViewportUVToScreenPos(UV);
	
	float3 ProjectedViewPos = mul(float4(ScreenPos.x * View.NearPlane, ScreenPos.y * View.NearPlane, 0.0, View.NearPlane), View.ClipToView).xyz;
	
	float Rho = length(ProjectedViewPos);
	float3 UnitViewPos = normalize(ProjectedViewPos);
	float3 PlanePos = UnitViewPos / UnitViewPos.z;
	float2 PolarCoords = float2(sqrt(PlanePos.x * PlanePos.x + PlanePos.y * PlanePos.y), atan2(PlanePos.y, PlanePos.x));
	
	float3 ViewPos = float3(sin(PolarCoords.x) * cos(PolarCoords.y), sin(PolarCoords.x) * sin(PolarCoords.y), cos(PolarCoords.x)) * Rho;
	float3 WorldPos = mul(float4(ViewPos, 0), View.ViewToTranslatedWorld).xyz;
	
	float3 RadialVector = normalize(mul(WorldPos, (float3x3) NormalCorrectionMatrix));
	
	float4 Color = SceneColor.Load(uint3(PixelPos, 0));
	float Depth = SceneDepth.Load(uint3(PixelPos, 0)).r;
	uint Stencil = SceneStencil.Load(uint3(PixelPos, 0)) STENCIL_COMPONENT_SWIZZLE;
	
	// If the current pixel is geometry, as indicated by the stencil, use the scene's color as the normal; otherwise,
	// use the negative radial vector for the empty space normal
	float3 Normal = lerp(-RadialVector, NormalFromColorSpace(Color.rgb), (float)Stencil);
	float3 LocalNormal = TransformToRadialBasis(Normal, RadialVector);
	
	// Sphere-ize the empty space depth buffer. Due to using a perspective projection matrix, a uniform depth value does
	// not create a sphere around the stage, which can cause artifacts during blurring around stage geometry edges. To
	// compute the correct depth, compute an azimuthal projection for a maximum radius point (pulled from the projection's
	// far plane value), and extract its depth value
	float MaxRho = View.NearPlane * (1.0 - 1.0 / View.ViewToClip[2][2]);
	float4 ClipSpacePos = mul(float4(UnitViewPos * MaxRho, 1.0), View.ViewToClip);
	float MaxDepth = ClipSpacePos.z / ClipSpacePos.w;

	RWColorOut[PixelPos] = float4(NormalToColorSpace(LocalNormal), 1.0);
	RWDepthOut[PixelPos] = lerp(MaxDepth, Depth, (float)Stencil);
	RWStencilOut[PixelPos] = Stencil;
	RWBlurStencilOut[PixelPos] = Stencil;
}

/** A compute pass that dilates the depth buffer of overlapping geometry, allowing nearer geometry depth to dilate into further geometry depth */
[numthreads(8, 8, 1)]
void DepthDilationCS(uint2 DispatchThreadId : SV_DispatchThreadID)
{
	const uint2 PixelPos = DispatchThreadId;
	
	const int kRadius = (FILTER_KERNEL_SIZE - 1) / 2;

	float InitialDepth = RWDepthIn[PixelPos];
	uint InitialStencil = RWStencilIn[PixelPos];

	float FinalDepth = 0.0;
	float W = 0.0;
	
	RWDepthOut[PixelPos] = RWDepthIn[PixelPos];

	if (ClipToAzimuthalBounds(PixelPos))
	{
		return;
	}
	
	// Only dilate onto geometry; if the current pixel's stencil is zero, the dilation can be skipped
	if (InitialStencil > 0)
	{
		UNROLL
		for (int kIndex = 0; kIndex < FILTER_KERNEL_SIZE; ++kIndex)
		{
			int2 PixelOffset = int2(kIndex - kRadius, kIndex - kRadius) * SampleOffsetScale * SampleDirection;
			uint2 SamplePos = AdjustPixelPos(PixelPos, PixelOffset);
			
			float Depth = RWDepthIn[SamplePos];
			uint Stencil = RWStencilIn[SamplePos];
			
			// Use a spatial factor here to ensure the depth closest to the edge of the geometry is dilated outwards
			float SpatialFactor = GET_SCALAR_ARRAY_ELEMENT(InteriorSpatialKernel, kIndex);
			
			// Only dilate overlapping geometry outwards if it is nearer to the view origin than the current pixel's depth
			float DepthFactor = max(Depth - InitialDepth, 0) * Stencil * SpatialFactor;
			
			FinalDepth += Depth * DepthFactor;
			W += DepthFactor;
		}
		
		if (W > 0.0)
		{
			RWDepthOut[PixelPos] = FinalDepth / W;
		}
	}
}

/** A compute pass that dilates the normal, depth, and stencil buffer of geometry into empty space */
[numthreads(8, 8, 1)]
void DilationCS(uint2 DispatchThreadId : SV_DispatchThreadID)
{
	const uint2 PixelPos = DispatchThreadId;
	
	const int kRadius = (FILTER_KERNEL_SIZE - 1) / 2;
	
	uint InitialStencil = RWStencilIn[PixelPos];
	
	float3 FinalColor = float3(0.0, 0.0, 0.0);
	float FinalDepth = 0.0;
	uint FinalStencil = 0;
	
	float W = 0.0;

	RWColorOut[PixelPos] = RWColorIn[PixelPos];
	RWDepthOut[PixelPos] = RWDepthIn[PixelPos];
	RWStencilOut[PixelPos] = RWStencilIn[PixelPos];
	RWBlurStencilOut[PixelPos] = RWBlurStencilIn[PixelPos];

	if (ClipToAzimuthalBounds(PixelPos))
	{
		return;
	}
	
	// Only dilate into empty space; if the current pixel's stencil is non-zero, the dilation can be skipped
	if (InitialStencil < 1)
	{
		UNROLL
		for (int kIndex = 0; kIndex < FILTER_KERNEL_SIZE; ++kIndex)
		{
			int2 PixelOffset = int2(kIndex - kRadius, kIndex - kRadius) * SampleOffsetScale * SampleDirection;
			uint2 SamplePos = AdjustPixelPos(PixelPos, PixelOffset);
		
			float3 Color = RWColorIn[SamplePos].rgb;
			float Depth = RWDepthIn[SamplePos];
			uint Stencil = RWStencilIn[SamplePos];
			
			// Use a spatial factor here to ensure the depth closest to the edge of the geometry is dilated outwards
			float SpatialFactor = GET_SCALAR_ARRAY_ELEMENT(SpatialKernel, kIndex);
			
			// Only dilate geometry into empty space
			float StencilFactor = Stencil * SpatialFactor;

			FinalColor += Color * StencilFactor;
			FinalDepth += Depth * StencilFactor;
			FinalStencil += Stencil;
			
			W += StencilFactor;
		}
	
		if (W > 0.0)
		{
			RWColorOut[PixelPos] = float4(FinalColor / W, 1.0);
			RWDepthOut[PixelPos] = FinalDepth / W;
		}

		RWStencilOut[PixelPos] = clamp(FinalStencil, 0, 1);

		if (bCopyToBlurStencil)
		{
			RWBlurStencilOut[PixelPos] = clamp(FinalStencil, 0, 1);
		}
	}
}

/** A compute pass that performs a standard Gaussian blur of the normal and depth buffers. Uses a different spatial deviation for blurring empty space or geometry */
[numthreads(8, 8, 1)]
void BlurCS(uint2 DispatchThreadId : SV_DispatchThreadID)
{
	const uint2 PixelPos = DispatchThreadId;
	
	const int kRadius = (FILTER_KERNEL_SIZE - 1) / 2;
	
	uint Stencil = RWBlurStencilIn[PixelPos];
	
	float3 FinalNormal = float3(0.0, 0.0, 0.0);
	float FinalDepth = 0.0;
	float W = 0.0;

	// Don't blur any pixels that are beyond the bounds of the azimuthal projection (which are a circle inscribed within the view texture)
	if (ClipToAzimuthalBounds(PixelPos))
	{
		RWColorOut[PixelPos] = RWColorIn[PixelPos];
		RWDepthOut[PixelPos] = RWDepthIn[PixelPos];
		return;
	}
	
	UNROLL
	for (int kIndex = 0; kIndex < FILTER_KERNEL_SIZE; ++kIndex)
	{
		int2 PixelOffset = int2(kIndex - kRadius, kIndex - kRadius) * SampleOffsetScale * SampleDirection;
		uint2 SamplePos = AdjustPixelPos(PixelPos, PixelOffset);
		
		float3 Normal = NormalFromColorSpace(RWColorIn[SamplePos].rgb);
		float Depth = RWDepthIn[SamplePos];

		// Get the correct spatial kernel depending on if we are blurring a screen or empty space
		float Factor = lerp(GET_SCALAR_ARRAY_ELEMENT(SpatialKernel, kIndex), GET_SCALAR_ARRAY_ELEMENT(InteriorSpatialKernel, kIndex), Stencil);

		FinalNormal += Normal * Factor;
		FinalDepth += Depth * Factor;
		W += Factor;
	}
	
	FinalNormal /= W;
	FinalDepth /= W;
	
	RWColorOut[PixelPos] = float4(NormalToColorSpace(FinalNormal), 1.0);
	RWDepthOut[PixelPos] = FinalDepth;
}

/** A pixel shader that renders the results of any compute shader passes to an output render target, packing the normal and depth values into the correct RGBA channels */
void OutputNormalMapPS(noperspective float4 UVAndScreenPos : TEXCOORD0, out float4 OutColor : SV_Target0)
{
	const float2 UV = UVAndScreenPos.xy;
	const uint2 PixelPos = int2(UV * Input_Extent);
	
	float3 NormalColor = RWColorIn[PixelPos].rgb;
	float Depth = RWDepthIn[PixelPos];
	
	// Normal vector is placed into the RGB channel, while depth is placed into the A channel
	OutColor = float4(NormalColor, Depth);
}