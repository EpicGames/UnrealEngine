// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/ScreenPass.ush"

// Note: opengl compiler doesn't like Texture2D<float>
Texture2D EditorPrimitivesDepth;
Texture2D<uint2> EditorPrimitivesStencil;

Texture2D ColorTexture;
SamplerState ColorSampler;

Texture2D DepthTexture;
SamplerState DepthSampler;

SCREEN_PASS_TEXTURE_VIEWPORT(Color)
SCREEN_PASS_TEXTURE_VIEWPORT(Depth)

float3 OutlineColor;
float SelectionHighlightIntensity;

struct FPixelInfo
{
	// if there is an selected object
	bool bObjectMask;

	int ObjectId;
	// if the current pixel is not occluded by some other object in front of it
	float fVisible;
	// hard version of fVisible
	bool bVisible;
};

// @param DeviceZPlane plane in screenspace .x: ddx(DeviceZ), y: ddy(DeviceZ) z:DeviceZ 
float ReconstructDeviceZ(float3 DeviceZPlane, float2 PixelOffset)
{
	return dot(DeviceZPlane, float3(PixelOffset.xy, 1));
}

// @param DeviceZPlane plane in screenspace .x: ddx(DeviceZ), y: ddy(DeviceZ) z:DeviceZ
FPixelInfo GetPixelInfo(int2 PixelPos, int OffsetX, int OffsetY, float3 DeviceZPlane, float2 DeviceZMinMax)
{
	FPixelInfo Ret;

	PixelPos += int2(OffsetX, OffsetY);

	// more stable on the silhouette
	float2 ReconstructionOffset = 0.5f - View.TemporalAAParams.zw;

	float DeviceZ = EditorPrimitivesDepth.Load(int3(PixelPos, 0)).r;
	int Stencil = EditorPrimitivesStencil.Load(int3(PixelPos, 0)) STENCIL_COMPONENT_SWIZZLE;

	float SceneDeviceZ = ReconstructDeviceZ(DeviceZPlane, ReconstructionOffset);

	// clamp SceneDeviceZ in (DeviceZMinMax.x .. DeviceZMinMax.z)
	// this avoids flicking artifacts on the silhouette by limiting the depth reconstruction error
	SceneDeviceZ = max(SceneDeviceZ, DeviceZMinMax.x);
	SceneDeviceZ = min(SceneDeviceZ, DeviceZMinMax.y);

	// test against far plane
	Ret.bObjectMask = DeviceZ != 0.0f;

	// outline even between multiple selected objects (best usability)
	Ret.ObjectId = Stencil;

#if (COMPILER_GLSL == 1 && FEATURE_LEVEL < FEATURE_LEVEL_SM4) || (COMPILER_METAL && FEATURE_LEVEL < FEATURE_LEVEL_SM4)
	// Stencil read in opengl is not supported on older versions
	Ret.ObjectId = Ret.bObjectMask ? 2 : 0;
#endif

	// Soft Bias with DeviceZ for best quality (larger bias than usual because SceneDeviceZ is only approximated)
	const float DeviceDepthFade = 0.00005f;

	// 2 to always bias over the current plane
	Ret.fVisible = saturate(2.0f - (SceneDeviceZ - DeviceZ) / DeviceDepthFade);

	Ret.bVisible = Ret.fVisible >= 0.5f;

	return Ret;
}

// Computes min and max at once.
void MinMax(inout int2 Var, int Value)
{
	Var.x = min(Var.x, Value);
	Var.y = max(Var.y, Value);
}

float4 GetOutline(int2 PixelPos, FPixelInfo CenterPixelInfo, float3 DeviceZPlane, float2 DeviceZMinMax)
{
	float4 Color = 0;

	// [0]:center, [1..4]:borders
	FPixelInfo PixelInfo[5];
	PixelInfo[0] = CenterPixelInfo;

	// Diagonal cross is thicker than vertical/horizontal cross.
	PixelInfo[1] = GetPixelInfo(PixelPos,  1,  1, DeviceZPlane, DeviceZMinMax);
	PixelInfo[2] = GetPixelInfo(PixelPos, -1, -1, DeviceZPlane, DeviceZMinMax);
	PixelInfo[3] = GetPixelInfo(PixelPos,  1, -1, DeviceZPlane, DeviceZMinMax);
	PixelInfo[4] = GetPixelInfo(PixelPos, -1,  1, DeviceZPlane, DeviceZMinMax);

	// With (.x != .y) we can detect a border around and between each object.
	int2 BorderMinMax = int2(255,0);
	{
		MinMax(BorderMinMax, PixelInfo[1].ObjectId);
		MinMax(BorderMinMax, PixelInfo[2].ObjectId);
		MinMax(BorderMinMax, PixelInfo[3].ObjectId);
		MinMax(BorderMinMax, PixelInfo[4].ObjectId);
	}

	int2 VisibleBorderMinMax = int2(255,0);
	{
		FLATTEN	if (PixelInfo[1].bVisible) MinMax(VisibleBorderMinMax, PixelInfo[1].ObjectId);
		FLATTEN	if (PixelInfo[2].bVisible) MinMax(VisibleBorderMinMax, PixelInfo[2].ObjectId);
		FLATTEN	if (PixelInfo[3].bVisible) MinMax(VisibleBorderMinMax, PixelInfo[3].ObjectId);
		FLATTEN	if (PixelInfo[4].bVisible) MinMax(VisibleBorderMinMax, PixelInfo[4].ObjectId);
	}

	// this border around the object
	bool bVisibleBorder = VisibleBorderMinMax.y != 0;
	bool bBorder = BorderMinMax.x != BorderMinMax.y;
	bool bInnerBorder = BorderMinMax.y == 4;

	// moving diagonal lines
	float PatternMask = ((PixelPos.x/2 + PixelPos.y/2) % 2) * 0.6f;

	// the contants express the two opacity values we see when the primitive is hidden
	float LowContrastPatternMask = lerp(0.2, 1.0f, PatternMask);

	float3 SelectionColor = OutlineColor.rgb;
	FLATTEN if (bBorder)
	{
		FLATTEN if (bVisibleBorder)
		{
			// unoccluded border
			Color = float4(SelectionColor, 1);
		}
		else
		{
			// occluded border
			Color = lerp(float4(0, 0, 0, 0), float4(SelectionColor, 1), LowContrastPatternMask);
		}
	}

	FLATTEN if (bInnerBorder)
	{
		// even occluded object is filled
		// occluded object is rendered differently(flickers with TemporalAA)
		float VisibleMask = lerp(PatternMask, 1.0f, PixelInfo[0].fVisible);

		// darken occluded parts
		Color = lerp(Color, float4(SelectionColor * VisibleMask, 1), SelectionHighlightIntensity);
	}

	// highlight inner part of the object
	if (PixelInfo[0].bObjectMask)
	{
		float VisibleMask = lerp(PatternMask, 1.0f, PixelInfo[0].fVisible);
		float InnerHighlightAmount = SelectionHighlightIntensity;
		Color = lerp(Color, float4(SelectionColor, 1), VisibleMask * InnerHighlightAmount);
	}

	return Color;
}

void Main(
	noperspective float4 UVAndScreenPos : TEXCOORD0,
	float4 SvPosition : SV_POSITION,
	out float4 OutColor : SV_Target0)
{
	const float2 UV = UVAndScreenPos.xy;
	const int2 Pixel = int2(UV * Color_Extent);

	// Scout outwards from the center pixel to determine if any neighboring pixels are selected
	const int ScoutStep = 2;
	
	const int4 StencilScout = int4(
		EditorPrimitivesStencil.Load(int3(Pixel + int2( ScoutStep,  0), 0)) STENCIL_COMPONENT_SWIZZLE,
		EditorPrimitivesStencil.Load(int3(Pixel + int2(-ScoutStep,  0), 0)) STENCIL_COMPONENT_SWIZZLE,
		EditorPrimitivesStencil.Load(int3(Pixel + int2( 0,  ScoutStep), 0)) STENCIL_COMPONENT_SWIZZLE,
		EditorPrimitivesStencil.Load(int3(Pixel + int2( 0, -ScoutStep), 0)) STENCIL_COMPONENT_SWIZZLE
	);

	const int ScoutSum = dot(saturate(StencilScout), 1);

	// If this sum is zero, none of our neighbors are selected pixels and we can skip all the heavy processing.
	if (ScoutSum > 0)
	{
		float Center = Texture2DSampleLevel(DepthTexture, DepthSampler, UV, 0).r;
		float Left   = Texture2DSampleLevel(DepthTexture, DepthSampler, UV + float2(-1,  0) * Depth_ExtentInverse, 0).r;
		float Right  = Texture2DSampleLevel(DepthTexture, DepthSampler, UV + float2( 1,  0) * Depth_ExtentInverse, 0).r;
		float Top    = Texture2DSampleLevel(DepthTexture, DepthSampler, UV + float2( 0, -1) * Depth_ExtentInverse, 0).r;
		float Bottom = Texture2DSampleLevel(DepthTexture, DepthSampler, UV + float2( 0,  1) * Depth_ExtentInverse, 0).r;

		// This allows to reconstruct depth with a small pixel offset without many texture lookups (4xMSAA * 5 neighbors -> 20 samples)
		// It's an approximation assuming the surface is a plane.
		float3 DeviceZPlane;
		DeviceZPlane.x = (Right - Left) / 2;
		DeviceZPlane.y = (Bottom - Top) / 2;
		DeviceZPlane.z = Center;

		float2 DeviceZMinMax;
		DeviceZMinMax.x = min(Center, min(min(Left, Right), min(Top, Bottom)));
		DeviceZMinMax.y = max(Center, max(max(Left, Right), max(Top, Bottom)));

		FPixelInfo CenterPixelInfo = GetPixelInfo(Pixel, 0, 0, DeviceZPlane, DeviceZMinMax);
		float4 Outline = GetOutline(Pixel, CenterPixelInfo, DeviceZPlane, DeviceZMinMax);

		OutColor = Texture2DSample(ColorTexture, ColorSampler, UV);

		// Scene color has gamma applied. Need to remove and re-apply after linear operation.
		OutColor.rgb = pow(OutColor.rgb, 2.2f );
		OutColor.rgb = lerp(OutColor.rgb, Outline.rgb, Outline.a);
		OutColor.rgb = pow(OutColor.rgb, 1.0f / 2.2f);
	}
	else
	{
		OutColor = Texture2DSample(ColorTexture, ColorSampler, UV);
	}
}
