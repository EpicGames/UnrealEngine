// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "/Engine/Generated/Material.ush"
#include "/Engine/Generated/VertexFactory.ush"

#include "SubstrateUtils.ush"


struct FMeshProjectionVSToPS
{
	FVertexFactoryInterpolantsVSToPS FactoryInterpolants;
	float4 Position : SV_POSITION;
	float4 SavedWorldPosition : POSITION1;
	float Azimuth : CUSTOM_AZIMUTH;
};

void AzimuthalVS(FVertexFactoryInput Input, out FMeshProjectionVSToPS Output)
{
	ResolvedView = ResolveView();

	const FVertexFactoryIntermediates VFIntermediates = GetVertexFactoryIntermediates(Input);
	Output.SavedWorldPosition = VertexFactoryGetWorldPosition(Input, VFIntermediates);
	const float3x3 TangentToLocal = VertexFactoryGetTangentToLocal(Input, VFIntermediates);

	const FMaterialVertexParameters VertexParameters = GetMaterialVertexParameters(Input, VFIntermediates, Output.SavedWorldPosition.xyz, TangentToLocal);
	Output.SavedWorldPosition.xyz += GetMaterialWorldPositionOffset(VertexParameters);

	Output.FactoryInterpolants = VertexFactoryGetInterpolantsVSToPS(Input, VFIntermediates, VertexParameters);

	// Project the vertex view position onto a unit sphere
	const float4 ViewPosition = mul(Output.SavedWorldPosition, ResolvedView.TranslatedWorldToView);
	const float Rho = length(ViewPosition.xyz);
	const float3 UnitPosition = ViewPosition.xyz / Rho;
	
	// Compute the position in view space of the point on the projection plane using azimthual equidistance projection
	
	const float Azimuth = acos(UnitPosition.z); // 0 <= azimuth <= pi
	const float Theta = atan2(UnitPosition.y, UnitPosition.x); // -pi <= theta <= pi
		
	const float2 PolarCoords = float2(Azimuth, Theta);
	const float3 PlanePosition = float3(PolarCoords.x * cos(PolarCoords.y), PolarCoords.x * sin(PolarCoords.y), 1);
		
	// Project the plane position into view space
	const float4 ProjectedPosition = float4(PlanePosition * Rho / length(PlanePosition), 1);
	
	Output.Azimuth = Azimuth;
	Output.Position = mul(ProjectedPosition, ResolvedView.ViewToClip);
}

void PerspectiveVS(FVertexFactoryInput Input, out FMeshProjectionVSToPS Output)
{
	ResolvedView = ResolveView();

	const FVertexFactoryIntermediates VFIntermediates = GetVertexFactoryIntermediates(Input);
	Output.SavedWorldPosition = VertexFactoryGetWorldPosition(Input, VFIntermediates);
	const float3x3 TangentToLocal = VertexFactoryGetTangentToLocal(Input, VFIntermediates);

	const FMaterialVertexParameters VertexParameters = GetMaterialVertexParameters(Input, VFIntermediates, Output.SavedWorldPosition.xyz, TangentToLocal);
	Output.SavedWorldPosition.xyz += GetMaterialWorldPositionOffset(VertexParameters);

	Output.FactoryInterpolants = VertexFactoryGetInterpolantsVSToPS(Input, VFIntermediates, VertexParameters);
	
	Output.Azimuth = 0;
	Output.Position = mul(Output.SavedWorldPosition, ResolvedView.TranslatedWorldToClip);
}

uint ProjectionParameters_UVProjectionIndex;
float ProjectionParameters_UVProjectionPlaneSize;
float ProjectionParameters_UVProjectionPlaneDistance;
float3 ProjectionParameters_UVProjectionPlaneOffset;

void UVProjectionVS(FVertexFactoryInput Input, out FMeshProjectionVSToPS Output)
{
	ResolvedView = ResolveView();

	const FVertexFactoryIntermediates VFIntermediates = GetVertexFactoryIntermediates(Input);
	Output.SavedWorldPosition = VertexFactoryGetWorldPosition(Input, VFIntermediates);
	const float3x3 TangentToLocal = VertexFactoryGetTangentToLocal(Input, VFIntermediates);

	const FMaterialVertexParameters VertexParameters = GetMaterialVertexParameters(Input, VFIntermediates, Output.SavedWorldPosition.xyz, TangentToLocal);
	Output.SavedWorldPosition.xyz += GetMaterialWorldPositionOffset(VertexParameters);

	Output.FactoryInterpolants = VertexFactoryGetInterpolantsVSToPS(Input, VFIntermediates, VertexParameters);

	float2 UV = float2(0, 0);
	
#if NUM_TEX_COORD_INTERPOLATORS
	int UVIndex = min(ProjectionParameters_UVProjectionIndex, NUM_TEX_COORD_INTERPOLATORS - 1);
	
	UV = GetUV(Output.FactoryInterpolants, UVIndex);
	
	const float3 PlanePosition = ProjectionParameters_UVProjectionPlaneOffset + 
		float3(ProjectionParameters_UVProjectionPlaneSize * (UV.x - 0.5), ProjectionParameters_UVProjectionPlaneSize * (0.5 - UV.y), ProjectionParameters_UVProjectionPlaneDistance);
	
	Output.Azimuth = UVIndex;
	Output.Position = mul(float4(PlanePosition, 1.0), ResolvedView.ViewToClip);
#else
	Output.Azimuth = 0;
	Output.Position = mul(Output.SavedWorldPosition, ResolvedView.TranslatedWorldToClip);
#endif
}

/** Gets a scalar value that indicates how much the current pixel diverges from where it should be from the projected vertices */
float GetProjectionDivergence(FMeshProjectionVSToPS Input)
{
	// Compute the "actual" azimuth of the pixel based on its screen space position
	// This will be subtracted from the expected azimuth interpolated from the vertices of the primitive
	// to determine the divergence magnitude
	
	float4 ScreenPos = SvPositionToResolvedScreenPosition(Input.Position);
	float3 ViewPos = mul(ScreenPos, ResolvedView.ClipToView).xyz;
	float3 UnitPos = normalize(ViewPos);
	float ActualAzimuth = length(UnitPos.xy) / UnitPos.z;
	
	return abs(ActualAzimuth - Input.Azimuth);
}

/** Clips any pixel whose projection divergence exceeds the specified threshold */
void ClipDivergentPixel(FMeshProjectionVSToPS Input, float DivergenceThreshold)
{
	float Divergence = GetProjectionDivergence(Input);
	clip(DivergenceThreshold - Divergence);
}

void MainPS(FMeshProjectionVSToPS Input OPTIONAL_IsFrontFace, out float4 OutputColor : SV_Target0)
{
	FMaterialPixelParameters MaterialParameters = GetMaterialPixelParameters(Input.FactoryInterpolants, Input.Position);
	FPixelMaterialInputs PixelMaterialInputs;
	CalcMaterialParameters(MaterialParameters, PixelMaterialInputs, Input.Position, bIsFrontFace);

	GetMaterialCoverageAndClipping(MaterialParameters, PixelMaterialInputs);

	OutputColor = GetMaterialProperties(PixelMaterialInputs, MaterialParameters);
	
	static const float AzimuthFeatherStart = radians(180 - 45);
	static const float AzimuthFeatherEnd   = radians(180 - 30);
	
	const float AlphaBlendWeight = saturate((Input.Azimuth - AzimuthFeatherStart) / (AzimuthFeatherEnd - AzimuthFeatherStart));
	OutputColor.a = lerp(OutputColor.a, 0, AlphaBlendWeight);
}

float4x4 NormalCorrectionMatrix;

void NormalPS(FMeshProjectionVSToPS Input OPTIONAL_IsFrontFace, out float4 OutputColor : SV_Target0)
{
	ResolvedView = ResolveView();
	
	const float ClipThreshold = 0.25;
	ClipDivergentPixel(Input, ClipThreshold);
	
	FMaterialPixelParameters MaterialParameters = GetMaterialPixelParameters(Input.FactoryInterpolants, Input.Position);
	FPixelMaterialInputs PixelMaterialInputs;
	CalcMaterialParameters(MaterialParameters, PixelMaterialInputs, Input.Position, bIsFrontFace);

	GetMaterialCoverageAndClipping(MaterialParameters, PixelMaterialInputs);
	
	float3 Normal = mul(TransformTangentNormalToWorld(MaterialParameters.TangentToWorld, float3(0, 0, 1)), (float3x3) NormalCorrectionMatrix);
	
	OutputColor = float4(Normal * 0.5 + 0.5, 1.0f);
}