// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"

Texture2D PostprocessInput0;
SamplerState PostprocessInput0Sampler;


struct InputVS
{
	float4 Position : ATTRIBUTE0;
	float2 UV : ATTRIBUTE1;
};

struct OutputVS
{
	float4	Position : SV_POSITION;
	float4	UV : TEXCOORD0;
};

struct OutputPS
{
	float4 Color : SV_Target0;
};


OutputVS OutputRemap_VS(InputVS IN)
{
	float2 Pos = IN.Position.xy;

	Pos.xy = Pos.xy * 2 - 1.0f;	

	OutputVS Out;
	Out.Position = float4(Pos, 0, 1);
	Out.UV = float4(IN.UV, 0.f, 1.f);
	return Out;
}


OutputPS OutputRemap_PS(OutputVS IN)
{
	OutputPS Out;

	Out.Color = PostprocessInput0.Sample(PostprocessInput0Sampler, IN.UV.xy);

	return Out;
}
