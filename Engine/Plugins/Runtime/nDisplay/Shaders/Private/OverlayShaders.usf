// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"

Texture2D TextureOverlay;
SamplerState SamplerOverlay;

Texture2D TextureBase;
SamplerState SamplerBase;

float4 DrawOverlay_PS(float2 uv : TEXCOORD) : SV_Target
{
	float4 srcColor = TextureOverlay.Sample(SamplerOverlay, uv);
	float4 dstColor = TextureBase.Sample(SamplerBase, uv);

	// Source-over blending (source color is already alpha-premultiplied)
	float3 outColor = dstColor.rgb * (1 - srcColor.a) + srcColor.rgb;

	return float4(outColor, 1);
}
