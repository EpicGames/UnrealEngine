// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"

struct InputVS
{
	float4 Position : ATTRIBUTE0;
	float2 UV : ATTRIBUTE1;
};
struct OutputVS
{
	float4	Position : SV_POSITION;
	float4	UV : TEXCOORD0;
};
struct OutputPS
{
	float4 Color : SV_Target0;
};


OutputVS DirectProjectionVS(InputVS IN)
{
	float2 Pos = IN.Position.xy;

	Pos.xy = Pos.xy*2 - 1.0f;
	Pos.xy *= float2(1, -1);

	OutputVS Out;
	Out.Position = float4(Pos, 0, 1);
	Out.UV = float4(IN.UV, 0.f, 1.f);
	return Out;
}


Texture2D SrcTexture;
float2 SampleOffset;
int KernelRadius;
SamplerState BilinearClampTextureSampler;


OutputPS DirectComposePS(OutputVS IN)
{
	float4 Color = SrcTexture.Sample(BilinearClampTextureSampler, IN.UV.xy);
	Color.w = 1.0 - Color.w;

	OutputPS Out;
	Out.Color = Color;
	return Out;
}

float GaussianWeight1d(float u, float KernelRadius)
{
	float TwoKernelRadiusSq = 2 * KernelRadius * KernelRadius;
	return exp(-u * u / TwoKernelRadiusSq) / sqrt(3.14 * TwoKernelRadiusSq);
}


OutputPS BlurPostProcessPS(OutputVS IN)
{
	float2 UV = IN.UV.xy;
	float4 OutColor;

#if BLUR_DILATE
	OutColor = SrcTexture.Sample(BilinearClampTextureSampler, UV);

	for (float SampleIndex = -KernelRadius; SampleIndex <= KernelRadius; ++SampleIndex)
	{
		float2 SampleUV = UV + SampleIndex * SampleOffset;
		float4 SampleColor = SrcTexture.Sample(BilinearClampTextureSampler, SampleUV);

		if (SampleColor.w < OutColor.w)
		{
			OutColor = SampleColor;
		}
	}


#else
	OutColor = float4(0.0f, 0.0f, 0.0f, 0.0f);
	float WeightSum = 0.0f;

	for (float SampleIndex = -KernelRadius; SampleIndex <= KernelRadius; ++SampleIndex)
	{
		float2 SampleUV = UV + SampleIndex * SampleOffset;
		float SampleWeight = GaussianWeight1d(SampleIndex, KernelRadius);

		OutColor += SrcTexture.Sample(BilinearClampTextureSampler, SampleUV) * SampleWeight;
		WeightSum += SampleWeight;
	}

	OutColor /= max(WeightSum, 0.01f);
#endif

	OutputPS Out;

	Out.Color = OutColor;

	return Out;
}