// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "DisplayClusterColorCorrection.ush"

// Input texture and sampler
Texture2D InputTexture;
SamplerState InputTextureSampler;

// Definition for the color encoding and alpha overrides
// Encodings.X = Input Encoding
// Encodings.Y = Output Encoding
// Encodings.Z = Override Alpha

#define ESourceEncoding_Linear  0
#define ESourceEncoding_Gamma   1
#define ESourceEncoding_sRGB    2
#define ESourceEncoding_MediaPQ 3

#define EOverrideAlpha_None   0
#define EOverrideAlpha_Invert 1
#define EOverrideAlpha_One    2
#define EOverrideAlpha_Zero   3

// Color encoding parameters:
uint3 Encodings;
float3 DisplayGamma;

// Definition for the ColorPremultiply
// ColorPremultiply.X = Input texture Color premultiply
// ColorPremultiply.Y = Output texture Color premultiply
// ColorPremultiply.Z = SideFeather

#define EColorPremultiply_None          0
#define EColorPremultiply_Alpha         1
#define EColorPremultiply_InvertedAlpha 2

#define ESideFeather_None      0
#define ESideFeather_Linear    1
#define ESideFeather_Smooth    2

// Color modifier
uint3 ColorPremultiply;

// Feather margins
//
//  Outer LT              RT
//  +---------------------+
//  |                     |
//  |  Inner LT      RT   |
//  |     +----------+    |
//  |     |          |    |
//  |     +----------+    |
//  |     LB         RB   |
//  +---------------------+
//  LB                    RB
//
// L,R,T,B = Left, Right, Top, Bottom
//
// InnerFeatherMargins = <InnerL, 1-InnerR, InnerT, 1-InnerB>  // Fully opaque region
// OuterFeatherMargins = <OuterL, 1-OuterR, OuterT, 1-OuterB>  // Fully transparent region
float4 InnerFeatherMargins;
float4 OuterFeatherMargins;

/**
 * Returns a linearly feathered alpha based on UV position and side margins.
 * The feathering fades from fully transparent (Max margins) to fully opaque (Min margins).
 */
float GetSideFeatherLinear(float2 UV)
{
	float2 InvUV = 1.0 - UV;
	float4 uvMargins = float4(UV.x, InvUV.x, UV.y, InvUV.y);
	float4 FeatherRange = InnerFeatherMargins - OuterFeatherMargins;

    // Prevent division by zero by replacing 0 range with 1.0
	float4 SafeFeatherRange = max(FeatherRange, 1e-5);

    // Compute normalized feather
	float4 NormalizedFeather = saturate((uvMargins - OuterFeatherMargins) / SafeFeatherRange);
	
	// Small threshold for treating near-zero floats as zero, to suppress noise.
	const float NearZeroThreshold = 1e-5f;
	
    // Mask: if FeatherRange is 0, use 1.0 (i.e. fully opaque / no feather)
	float4 RangeValidMask = step(NearZeroThreshold, FeatherRange);
	
	// RangeValidMask = 0 if < 1e-5, 1 if >= 1e-5
	NormalizedFeather = lerp(1.0, NormalizedFeather, RangeValidMask);

	// Return combined alpha (multiplicative blend)
	return NormalizedFeather.x * NormalizedFeather.y * NormalizedFeather.z * NormalizedFeather.w;
}

/**
 * Computes a feathered alpha value based on side-specific margins using smoothstep transitions.
 *
 * Logic:
 * - Inside the inner margins (`InnerFeatherMargins`): alpha = 1 (fully opaque)
 * - Outside the outer margins (`OuterFeatherMargins`): alpha = 0 (fully transparent)
 * - Between Min and Max: alpha fades smoothly using smoothstep

 * @param UV   UV coordinates (float2), expected in [0, 1] range
 * @return       Alpha value in [0, 1] with smooth feathering on each side
 */
float GetSideFeatherSmooth(float2 UV)
{
	// Remap UVs for mirrored sides (Right and Bottom)
	float2 InvUV = 1.0 - UV;

    // Combine UVs into one vector: <Left, Right, Top, Bottom>
	float4 uvMargins = float4(UV.x, InvUV.x, UV.y, InvUV.y);

    // Compute smooth feather for all sides: Left, Right, Top, Bottom
	float4 Feather = smoothstep(OuterFeatherMargins, InnerFeatherMargins, uvMargins);

    // Return combined alpha (multiplicative blend)
	return Feather.x * Feather.y * Feather.z * Feather.w;
}

/** Implements color encoding. */
	float4 DisplayClusterEncodeColorAndAlpha(
	FScreenVertexOutput Input,
	float4 InColorArg)
{
	float4 OutColor = InColorArg;

#if COLOR_PREMULTIPLY	
	// Unpremultiply color
	BRANCH
    if(ColorPremultiply.x == EColorPremultiply_Alpha) // Unpremultiply
    {
        OutColor = Unpremultiply(OutColor);
	}
	else if(ColorPremultiply.x == EColorPremultiply_InvertedAlpha) // Unpremultiply inverted alpha
	{
        OutColor = InvertedUnpremultiply(OutColor);
	}
#endif
	
#if OVERRIDE_ALPHA
	// Override alpha
	BRANCH
    if(Encodings.z == EOverrideAlpha_Invert)
    {
        OutColor.a = 1.0f - OutColor.a;
    }
    else if(Encodings.z == EOverrideAlpha_One)
    {
        OutColor.a = 1.0f;
    }
    else if(Encodings.z == EOverrideAlpha_Zero)
    {
        OutColor.a = 0.0f;
    }
	
	// Add Side Feather color
	BRANCH
    if(ColorPremultiply.z == ESideFeather_Linear)
    {
	    float Feather = GetSideFeatherLinear(Input.UV.xy);
	
		// Premultiply color and alpha
		OutColor *= Feather;
    }
	else if(ColorPremultiply.z == ESideFeather_Smooth)
	{  
		float Feather = GetSideFeatherSmooth(Input.UV.xy);
	
		// Premultiply color and alpha
		OutColor *= Feather;
	}
#endif

#if ENCODE_INPUT || ENCODE_OUTPUT

#if ENCODE_INPUT
	// Convert Input color to linear
	BRANCH
    if(Encodings.x == ESourceEncoding_Gamma) // Gamma -> Linear
    {
        OutColor.rgb = DisplayGammaCorrection(OutColor.rgb, DisplayGamma.x);
    }
    else if(Encodings.x == ESourceEncoding_sRGB) // sRGB -> Linear
    {
        OutColor.rgb = sRGBToLinear(OutColor.rgb);
	}
	else if(Encodings.x == ESourceEncoding_MediaPQ) // MediaPQ -> Linear
    {
		// Pixel shader for PQ-Linear conversion
		OutColor.rgb = ST2084ToLinear(OutColor.rgb);
	}

#endif // ENCODE_INPUT

#if ENCODE_OUTPUT
	// Convert the output color to a output color space
	BRANCH
    if(Encodings.y == ESourceEncoding_Gamma) // Linear -> Gamma
    {
        OutColor.rgb = DisplayGammaCorrection(OutColor.rgb, DisplayGamma.y);
    }
    else if(Encodings.y == ESourceEncoding_sRGB) // Linear -> sRGB
    {
        OutColor.rgb = LinearToSrgb(OutColor.rgb);
    }
	else if(Encodings.y == ESourceEncoding_MediaPQ) // Linear -> MediaPQ
    {
		// Pixel shader for Linear-PQ conversion
		float3 ColorPQ = LinearToST2084(OutColor.rgb);
		OutColor.rgb = saturate(ColorPQ);
    }
#endif // ENCODE_OUTPUT

#endif // ENCODE_INPUT || ENCODE_OUTPUT

#if COLOR_PREMULTIPLY
	// Premultiply color
	BRANCH
    if(ColorPremultiply.y == EColorPremultiply_Alpha) // Premultiply
    {
        OutColor = Premultiply(OutColor);
	}
	else if(ColorPremultiply.y == EColorPremultiply_InvertedAlpha) // Premultiply inverted alpha
	{
        OutColor = InvertedPremultiply(OutColor);
	}
#endif

	return OutColor;
}

void Main(
	FScreenVertexOutput Input,
	out float4 OutColor : SV_Target0)
{
	float4 InColor = Texture2DSample(InputTexture, InputTextureSampler, Input.UV);

	OutColor = DisplayClusterEncodeColorAndAlpha(Input, InColor);
}