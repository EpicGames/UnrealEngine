// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


float4 GetMaterialProperties(in FPixelMaterialInputs PixelMaterialInputs, in FMaterialPixelParameters MaterialParameters)
{
	float3 OutEmissive = 0;
	float OutOpacity = 0;
#if TEMPLATE_USES_SUBSTRATE
	// Initialise a Substrate header with normal in registers
	FSubstrateData SubstrateData = PixelMaterialInputs.GetFrontSubstrateData();
	FSubstratePixelHeader SubstratePixelHeader = MaterialParameters.GetFrontSubstrateHeader();
	SubstratePixelHeader.IrradianceAO.MaterialAO = GetMaterialAmbientOcclusion(PixelMaterialInputs);

	float TotalCoverage = 1.f;
	if (SubstratePixelHeader.SubstrateTree.BSDFCount > 0)
	{
		const float3 V = MaterialParameters.CameraVector;
		const FSubstrateIntegrationSettings Settings = InitSubstrateIntegrationSettings(false /*bForceFullyRough*/, true /*bRoughDiffuseEnabled*/, 0 /*PeelLayersAboveDepth*/, false/*bRoughnessTracking*/);
		float3 TotalTransmittancePreCoverage = 0;
		SubstratePixelHeader.SubstrateUpdateTree(SubstrateData, V, Settings, TotalCoverage, TotalTransmittancePreCoverage);
		OutOpacity = TotalCoverage;

		// Clip pixels that will never receive any lumen front material reflections.
		// We cannot generalise and use GetMaterialClippingVelocity for that because this function clips with a 1/255 threshold...that can result in NaN refelctions if some pixels are not genrated correctly.
		clip(TotalCoverage <= 0.0f ? -1.0f : 1.0f);

		// Accumulate emissive contribution
		float TopLayerTotalWeight = 0.0f;
		Substrate_for_unroll(int BSDFIdx = 0, BSDFIdx < SubstratePixelHeader.SubstrateTree.BSDFCount, ++BSDFIdx)
		{
			#define CurrentBSDF SubstratePixelHeader.SubstrateTree.BSDFs[BSDFIdx]
			if (SubstrateIsBSDFVisible(CurrentBSDF))
			{
				OutEmissive += CurrentBSDF.LuminanceWeightV * BSDF_GETEMISSIVE(CurrentBSDF);
			}
			#undef CurrentBSDF
		}
	}
#else
	OutEmissive = GetMaterialEmissive(PixelMaterialInputs);
	OutOpacity = GetMaterialOpacity(PixelMaterialInputs);
#endif

	return float4(OutEmissive, OutOpacity);
}
