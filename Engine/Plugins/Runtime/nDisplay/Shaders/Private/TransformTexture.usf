// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"

Texture2D Texture;
SamplerState Sampler;
float4 TransformMatrix;

float4 TransformTexture_PS(float2 UV : TEXCOORD) : SV_Target
{
    // Unpack transformation matrix
    float2x2 TM = float2x2(
        TransformMatrix.x, TransformMatrix.y, // row 0
        TransformMatrix.z, TransformMatrix.w  // row 1
    );

    // Find UV according to the trasformation
    float2 centered = UV - 0.5f;
    float2 transformed = mul(centered, TM);
    float2 sourceUV = transformed + 0.5f;

	// Sample with the new UV
    return Texture.Sample(Sampler, sourceUV);
}
