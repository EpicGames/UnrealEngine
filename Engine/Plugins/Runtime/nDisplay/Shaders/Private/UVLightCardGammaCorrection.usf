// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "DisplayClusterColorCorrection.ush"

Texture2D InputTexture;
SamplerState InputSampler;


float4 GammaCorrectionPS(noperspective float4 UVAndScreenPos : TEXCOORD0) : SV_Target0
{
	float4 OutColor = Texture2DSample(InputTexture, InputSampler, UVAndScreenPos.xy);

	// Always render UVLC in linear color space.
	
	// Invert alpha.
	return float4(OutColor.rgb, 1.0 - OutColor.a);
}