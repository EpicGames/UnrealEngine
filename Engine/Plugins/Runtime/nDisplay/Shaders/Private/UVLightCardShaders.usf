// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "/Engine/Generated/Material.ush"
#include "/Engine/Generated/VertexFactory.ush"

#include "SubstrateUtils.ush"


struct FUVLightCardVSToPS
{
	FVertexFactoryInterpolantsVSToPS FactoryInterpolants;
	float4 Position : SV_POSITION;
	float4 SavedWorldPosition : POSITION1;
};

struct OutputPS
{
	float4 Color : SV_Target0;
};


void MainVS(FVertexFactoryInput Input, out FUVLightCardVSToPS Output)
{
	ResolvedView = ResolveView();

	const FVertexFactoryIntermediates VFIntermediates = GetVertexFactoryIntermediates(Input);
	Output.SavedWorldPosition = VertexFactoryGetWorldPosition(Input, VFIntermediates);
	const float3x3 TangentToLocal = VertexFactoryGetTangentToLocal(Input, VFIntermediates);

	const FMaterialVertexParameters VertexParameters = GetMaterialVertexParameters(Input, VFIntermediates, Output.SavedWorldPosition.xyz, TangentToLocal);
	Output.SavedWorldPosition.xyz += GetMaterialWorldPositionOffset(VertexParameters);

	Output.FactoryInterpolants = VertexFactoryGetInterpolantsVSToPS(Input, VFIntermediates, VertexParameters);
	
	Output.Position = mul(Output.SavedWorldPosition, ResolvedView.TranslatedWorldToClip);
}

void MainPS(FUVLightCardVSToPS Input OPTIONAL_IsFrontFace, out float4 OutputColor : SV_Target0)
{
	FMaterialPixelParameters MaterialParameters = GetMaterialPixelParameters(Input.FactoryInterpolants, Input.Position);
	FPixelMaterialInputs PixelMaterialInputs;
	CalcMaterialParameters(MaterialParameters, PixelMaterialInputs, Input.Position, bIsFrontFace);

	GetMaterialCoverageAndClipping(MaterialParameters, PixelMaterialInputs);
	
	OutputColor = GetMaterialProperties(PixelMaterialInputs, MaterialParameters);
}
