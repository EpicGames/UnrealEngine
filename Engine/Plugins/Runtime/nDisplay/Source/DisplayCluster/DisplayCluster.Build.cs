// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class DisplayCluster : ModuleRules
{
	public DisplayCluster(ReadOnlyTargetRules ROTargetRules) : base(ROTargetRules)
	{
		// The DisplayCluster plugin is distributed with engine hot fixes and thus isn't tied to binary
		// compatibility between hotfixes by only using Public/ interface of the renderer, but also Internal/ ones.
		bTreatAsEngineModule = true;

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"CinematicCamera",
				"Core",
				"CoreUObject",
				"DisplayClusterConfiguration",
				"DisplayClusterLightCardEditorShaders",
				"DisplayClusterLightCardExtender",
				"Engine",
				"EngineSettings",
				"EnhancedInput",
				"VirtualProduction",
			});

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"HeadMountedDisplay",
				"CompositeCore",
				"InputCore",
				"Json",
				"JsonUtilities",
				"MediaAssets",
				"MediaIOCore",
				"Networking",
				"OpenColorIO",
				"OpenCV",
				"OpenCVHelper",
				"ProceduralMeshComponent",
				"Renderer",
				"RenderCore",
				"RHI",
				"SharedMemoryMedia",
				"Slate",
				"SlateCore",
				"Sockets",
				"UMG"
			});

		if (Target.bBuildEditor == true)
		{
			PublicIncludePathModuleNames.Add("DisplayClusterConfigurator");

			PrivateDependencyModuleNames.Add("UnrealEd");
			PrivateDependencyModuleNames.Add("EditorWidgets");
			PrivateDependencyModuleNames.Add("LevelEditor");
		}

		if (Target.Type == TargetType.Editor)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"ConcertSyncClient"
				});
		}

		if (Target.Platform == UnrealTargetPlatform.Win64)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"D3D11RHI",
					"D3D12RHI",
			});

			AddEngineThirdPartyPrivateStaticDependencies(Target, "DX11", "DX12");
		}

		if(Target.Platform == UnrealTargetPlatform.Win64 || Target.Platform == UnrealTargetPlatform.Linux)
		{
			AddEngineThirdPartyPrivateStaticDependencies(Target, "NVAPI");
		}
	}
}
