// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDisplayCluster.h"
#include "IPDisplayClusterManager.h"

class IPDisplayClusterRenderManager;
class IPDisplayClusterClusterManager;
class IPDisplayClusterConfigManager;
class IPDisplayClusterGameManager;


/**
 * Private module interface
 */
class IPDisplayCluster
	: public IDisplayCluster
	, public IPDisplayClusterManager
{
public:

	virtual ~IPDisplayCluster() = default;

public:

	/** Returns private render manager interface */
	virtual IPDisplayClusterRenderManager* GetPrivateRenderMgr() const = 0;

	/** Returns private cluster manager interface */
	virtual IPDisplayClusterClusterManager* GetPrivateClusterMgr() const = 0;

	/** Returns private config manager interface */
	virtual IPDisplayClusterConfigManager* GetPrivateConfigMgr() const = 0;

	/** Returns private game manager interface */
	virtual IPDisplayClusterGameManager* GetPrivateGameMgr() const = 0;
};
