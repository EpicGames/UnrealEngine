// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DisplayClusterEnums.h"

class UDisplayClusterConfigurationData;


/**
 * Private manager interface
 */
class IPDisplayClusterManager
{
public:
	virtual ~IPDisplayClusterManager() = default;

public:
	// Called at start to initialize internals
	virtual bool Init(EDisplayClusterOperationMode OperationMode)
	{ return true; }

	// Called before application/Editor exit to release internals
	virtual void Release()
	{ }

	// Called on each session start before first level start (before the first tick)
	virtual bool StartSession(UDisplayClusterConfigurationData* InConfigData, const FString& InNodeId)
	{ return true; }

	// Called on each session end at early step before exit (before UGameEngine::Preexit)
	virtual void EndSession()
	{ }

	// Called each time a new game level starts
	virtual bool StartScene(UWorld* InWorld)
	{ return true; }

	// Called when current level is going to be closed (i.e. when loading new map)
	virtual void EndScene()
	{ }

	// Called before start every frame
	virtual void StartFrame(uint64 FrameNum)
	{ }

	// Called before start every frame
	virtual void EndFrame(uint64 FrameNum)
	{ }

	// Called every frame before world Tick
	virtual void PreTick(float DeltaSeconds)
	{ }

	// Called every frame during world Tick
	virtual void Tick(float DeltaSeconds)
	{ }

	// Called every frame during world Tick
	virtual void PostTick(float DeltaSeconds)
	{ }
};
