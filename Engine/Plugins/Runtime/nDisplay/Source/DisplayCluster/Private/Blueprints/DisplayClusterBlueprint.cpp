// Copyright Epic Games, Inc. All Rights Reserved.

#include "Blueprints/DisplayClusterBlueprint.h"
#include "Blueprints/DisplayClusterBlueprintGeneratedClass.h"
#include "DisplayClusterRootActor.h"
#include "Components/DisplayClusterCameraComponent.h"
#include "Components/DisplayClusterICVFXCameraComponent.h"
#include "Components/DisplayClusterScreenComponent.h"

#include "IDisplayClusterConfiguration.h"

#include "EngineAnalytics.h"
#include "Engine/SCS_Node.h"
#include "Containers/Set.h"

#include "Misc/DisplayClusterLog.h"
#include "UObject/ObjectSaveContext.h"


UDisplayClusterBlueprint::UDisplayClusterBlueprint()
	: ConfigData(nullptr), AssetVersion(0)
{
	BlueprintType = BPTYPE_Normal;
#if WITH_EDITORONLY_DATA
	bRunConstructionScriptOnInteractiveChange = false;
#endif
}

#if WITH_EDITOR

UClass* UDisplayClusterBlueprint::GetBlueprintClass() const
{
	return UDisplayClusterBlueprintGeneratedClass::StaticClass();
}

void UDisplayClusterBlueprint::GetReparentingRules(TSet<const UClass*>& AllowedChildrenOfClasses,
	TSet<const UClass*>& DisallowedChildrenOfClasses) const
{
	AllowedChildrenOfClasses.Add(ADisplayClusterRootActor::StaticClass());
}
#endif

void UDisplayClusterBlueprint::UpdateConfigExportProperty()
{
	bool bConfigExported = false;

	if (UDisplayClusterConfigurationData* Config = GetOrLoadConfig())
	{
		PrepareConfigForExport();
		
		FString PrettyConfig;

		bConfigExported = IDisplayClusterConfiguration::Get().ConfigAsString(Config, PrettyConfig);

		if (bConfigExported)
		{
			// We cache a somewhat minified version of the config so that the context view of the asset registry data is less bloated.

			ConfigExport.Empty(PrettyConfig.Len());

			for (auto CharIt = PrettyConfig.CreateConstIterator(); CharIt; ++CharIt)
			{
				const TCHAR Char = *CharIt;

				// Remove tabs, carriage returns and newlines.
				if ((Char == TCHAR('\t')) || (Char == TCHAR('\r')) || (Char == TCHAR('\n')))
				{
					continue;
				}

				ConfigExport.AppendChar(Char);
			}
		}
	}

	if (!bConfigExported)
	{
		ConfigExport = TEXT("");
	}
}


void UDisplayClusterBlueprint::UpdateSummaryProperty()
{
	if (!ConfigData || !ConfigData->Cluster)
	{
		Summary = TEXT("No data!");
		return;
	}

	TArray<FString> Lines;

	// Description

	if (ConfigData->Info.Description.Len())
	{
		Lines.Add(ConfigData->Info.Description);
		Lines.Add(TEXT(""));
	}

	// Settings

	Lines.Add(TEXT("Settings:"));
	Lines.Add(TEXT("--------"));
	Lines.Add(TEXT(""));

	Lines.Add(FString::Printf(TEXT("Sync Policy: %s"), *ConfigData->Cluster->Sync.RenderSyncPolicy.Type));
	Lines.Add(FString::Printf(TEXT("Follow Local Player Camera: %s"), ConfigData->bFollowLocalPlayerCamera ? TEXT("Yes") : TEXT("No")));
	Lines.Add(FString::Printf(TEXT("Viewports Screen %% Multiplier: %.2f"), ConfigData->RenderFrameSettings.ClusterICVFXOuterViewportBufferRatioMult));

	Lines.Add(TEXT(""));

	// Cluster

	Lines.Add(TEXT("Cluster:"));
	Lines.Add(TEXT("-------"));
	Lines.Add(TEXT(""));

	TSet<FString> Hosts;
	int32 NumNodes = 0;
	int32 NumHeadlessNodes = 0;
	int32 NumFullscreenNodes = 0;
	int32 NumViewports = 0;

	TSet<FString> ViewportMedias;
	TSet<FString> NodeMedias;
	TSet<FString> IcvfxCameraMedias;

	for (const TPair<FString, TObjectPtr<UDisplayClusterConfigurationClusterNode>>& NodePair : ConfigData->Cluster->Nodes)
	{
		const FString& NodeId = NodePair.Key;
		const TObjectPtr<UDisplayClusterConfigurationClusterNode> Node = NodePair.Value;

		if (!ensure(Node))
		{
			continue;
		}

		Hosts.Add(Node->Host);
		NumNodes++;

		if (Node->MediaSettings.bEnable)
		{
			for (const FDisplayClusterConfigurationMediaOutput& MediaOutput : Node->MediaSettings.MediaOutputs)
			{
				if (MediaOutput.MediaOutput)
				{
					FString MediaName = MediaOutput.MediaOutput.GetClass()->GetName();
					MediaName.RemoveFromEnd(TEXT("Output"));
					NodeMedias.Add(MediaName);
				}
			}
		}

		if (Node->bRenderHeadless)
		{
			NumHeadlessNodes++;
		}
		else
		{
			if (Node->bIsFullscreen)
			{
				NumFullscreenNodes++;
			}
		}

		for (const TPair<FString, TObjectPtr<UDisplayClusterConfigurationViewport>>& ViewportPair : Node->Viewports)
		{
			TObjectPtr<UDisplayClusterConfigurationViewport> Viewport = ViewportPair.Value;

			if (!ensure(Viewport))
			{
				continue;
			}

			NumViewports++;

			if (Viewport->RenderSettings.Media.bEnable)
			{
				if (Viewport->RenderSettings.Media.MediaInput.MediaSource)
				{
					FString MediaName = Viewport->RenderSettings.Media.MediaInput.MediaSource->GetClass()->GetName();
					MediaName.RemoveFromEnd(TEXT("Source"));
					ViewportMedias.Add(MediaName);
				}

				for (const FDisplayClusterConfigurationMediaOutput& MediaOutput : Viewport->RenderSettings.Media.MediaOutputs)
				{
					if (!MediaOutput.MediaOutput)
					{
						continue;
					}

					FString MediaName = MediaOutput.MediaOutput->GetClass()->GetName();
					MediaName.RemoveFromEnd(TEXT("Output"));
					ViewportMedias.Add(MediaName);
				}
			}
		}
	}

	Lines.Add(FString::Printf(TEXT("Hosts: %d"), Hosts.Num()));
	Lines.Add(FString::Printf(TEXT("Nodes: %d (%d Headless, %d Fullscreen)"), NumNodes, NumHeadlessNodes, NumFullscreenNodes));
	Lines.Add(FString::Printf(TEXT("Viewports: %d"), NumViewports));

	// Here we find the icvfx camera templates in the blueprint, using  the SimpleConstructionScript.

	if (SimpleConstructionScript)
	{
		TMap<FString, int32> CamerasByMediaOrSplit;

		for (USCS_Node* Node : SimpleConstructionScript->GetAllNodes())
		{
			if (UDisplayClusterICVFXCameraComponent* IcvfxCamera = Cast<UDisplayClusterICVFXCameraComponent>(Node->ComponentTemplate))
			{
				// We're intentionally including disabled cameras in the count, but not disabled Media in them.

				const FDisplayClusterConfigurationMediaICVFX& MediaSettings = IcvfxCamera->GetCameraSettingsICVFX().RenderSettings.Media;

				if (MediaSettings.bEnable)
				{
					switch (MediaSettings.SplitType)
					{
					case EDisplayClusterConfigurationMediaSplitType::FullFrame:
					{
						int32& FullFrameCameras = CamerasByMediaOrSplit.FindOrAdd(TEXT("Full Frame"), 0);
						FullFrameCameras++;

						// Gather the media types used.

						for (const FDisplayClusterConfigurationMediaOutputGroup& OutputGroup : MediaSettings.MediaOutputGroups)
						{
							if (!OutputGroup.MediaOutput)
							{
								continue;
							}

							FString MediaName = OutputGroup.MediaOutput->GetClass()->GetName();
							MediaName.RemoveFromEnd(TEXT("Output"));
							IcvfxCameraMedias.Add(MediaName);
						}

						for (const FDisplayClusterConfigurationMediaInputGroup& InputGroup : MediaSettings.MediaInputGroups)
						{
							if (!InputGroup.MediaSource)
							{
								continue;
							}

							FString MediaName = InputGroup.MediaSource->GetClass()->GetName();
							MediaName.RemoveFromEnd(TEXT("Source"));
							IcvfxCameraMedias.Add(MediaName);
						}

						break;
					}
					case EDisplayClusterConfigurationMediaSplitType::UniformTiles:
					{
						const FString TiledString = FString::Printf(TEXT("Tiled %dx%d"), MediaSettings.TiledSplitLayout.X, MediaSettings.TiledSplitLayout.Y);
						int32& TiledCameras = CamerasByMediaOrSplit.FindOrAdd(TiledString, 0);
						TiledCameras++;

						// Gather the media types used.

						for (const FDisplayClusterConfigurationMediaTiledInputGroup& InputGroup : MediaSettings.TiledMediaInputGroups)
						{
							for (const FDisplayClusterConfigurationMediaUniformTileInput& Tile : InputGroup.Tiles)
							{
								if (!Tile.MediaSource)
								{
									continue;
								}

								FString MediaName = Tile.MediaSource->GetClass()->GetName();
								MediaName.RemoveFromEnd(TEXT("Source"));
								IcvfxCameraMedias.Add(MediaName);
							}
						}

						for (const FDisplayClusterConfigurationMediaTiledOutputGroup& OutputGroup : MediaSettings.TiledMediaOutputGroups)
						{
							for (const FDisplayClusterConfigurationMediaUniformTileOutput& Tile : OutputGroup.Tiles)
							{
								if (!Tile.MediaOutput)
								{
									continue;
								}

								FString MediaName = Tile.MediaOutput->GetClass()->GetName();
								MediaName.RemoveFromEnd(TEXT("Output"));
								IcvfxCameraMedias.Add(MediaName);
							}
						}

						break;
					}
					default:
						checkNoEntry();
					}
				}
				else
				{
					int32& NoMediaCameras = CamerasByMediaOrSplit.FindOrAdd(TEXT("No Media"), 0);
					NoMediaCameras++;
				}
			}
		}

		FString IcvfxCamerasLine = TEXT("ICVFX Cameras: ");

		if (CamerasByMediaOrSplit.Num())
		{
			TArray<FString> TypeValues;

			for (const TPair<FString, int32>& IcvfxCameraPair : CamerasByMediaOrSplit)
			{
				TypeValues.Add(FString::Printf(TEXT("%d (%s)"), IcvfxCameraPair.Value, *IcvfxCameraPair.Key));
			}

			IcvfxCamerasLine += FString::Join(TypeValues, TEXT(", "));
		}
		else
		{
			IcvfxCamerasLine += TEXT("None");
		}

		Lines.Add(IcvfxCamerasLine);
	}

	Lines.Add(TEXT(""));

	// Media

	if (IcvfxCameraMedias.Num() || ViewportMedias.Num() || NodeMedias.Num())
	{
		Lines.Add(TEXT("Media:"));
		Lines.Add(TEXT("------"));
		Lines.Add(TEXT(""));

		if (NodeMedias.Num())
		{
			Lines.Add(TEXT("Node Media: ") + FString::Join(NodeMedias, TEXT(", ")));
		}

		if (ViewportMedias.Num())
		{
			Lines.Add(TEXT("Viewport Media: ") + FString::Join(ViewportMedias, TEXT(", ")));
		}

		if (IcvfxCameraMedias.Num())
		{
			Lines.Add(TEXT("ICVFX Camera Media: ") + FString::Join(IcvfxCameraMedias, TEXT(", ")));
		}
	}

	Summary = FString::Join(Lines, TEXT("\n"));
}


namespace DisplayClusterBlueprint
{
	void SendAnalytics(const FString& EventName, const UDisplayClusterConfigurationData* const ConfigData)
	{
		if (!FEngineAnalytics::IsAvailable())
		{
			return;
		}

		// Gather attributes related to this config
		TArray<FAnalyticsEventAttribute> EventAttributes;

		if (ConfigData)
		{
			if (ConfigData->Cluster)
			{
				// Number of Nodes
				EventAttributes.Add(FAnalyticsEventAttribute(TEXT("NumNodes"), ConfigData->Cluster->Nodes.Num()));

				// Number of Viewports
				TSet<FString> UniquelyNamedViewports;

				for (auto NodesIt = ConfigData->Cluster->Nodes.CreateConstIterator(); NodesIt; ++NodesIt)
				{
					for (auto ViewportsIt = ConfigData->Cluster->Nodes.CreateConstIterator(); ViewportsIt; ++ViewportsIt)
					{
						UniquelyNamedViewports.Add(ViewportsIt->Key);
					}
				}

				// Number of uniquely named viewports
				EventAttributes.Add(FAnalyticsEventAttribute(TEXT("NumUniquelyNamedViewports"), UniquelyNamedViewports.Num()));
			}
		}

		FEngineAnalytics::GetProvider().RecordEvent(EventName, EventAttributes);
	}
}

void UDisplayClusterBlueprint::PreSave(FObjectPreSaveContext SaveContext)
{
	Super::PreSave(SaveContext);

	UpdateConfigExportProperty();
	UpdateSummaryProperty();
	DisplayClusterBlueprint::SendAnalytics(TEXT("Usage.nDisplay.ConfigSaved"), ConfigData);

#if WITH_EDITOR

	// Child blueprints need to re-generate their config export property as well
	// Note: Using GetDerivedClasses will only get loaded classes, which is the normal case,
	// and the rest will be caught when they get loaded as an out of date exported config
	// will be detected.
	if (GIsEditor)
	{
		TArray<UClass*> ChildClasses;
		GetDerivedClasses(GeneratedClass, ChildClasses);

		for (UClass* ChildClass : ChildClasses)
		{
			// CLASS_NewerVersionExists suggests there is a newer class that will update the asset, so we skip it.
			if (ChildClass->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists))
			{
				continue;
			}

			UDisplayClusterBlueprint* ChildDCBP = Cast<UDisplayClusterBlueprint>(ChildClass->ClassGeneratedBy);
			check(ChildDCBP); // We already know it is a derived class

			// Only mark as dirty if the config needs updating, to avoid unnecessary re-saves.

			const FString OriginalChildConfigExport = ChildDCBP->ConfigExport;
			ChildDCBP->UpdateConfigExportProperty();

			if (!ChildDCBP->ConfigExport.Equals(OriginalChildConfigExport))
			{
				if (ChildDCBP->MarkPackageDirty())
				{
					UE_LOGF(LogDisplayClusterBlueprint, Display,
						"ConfigExport of the child nDisplay blueprint actor '%ls' is not up to date in the asset, so the package was marked as dirty and should be re-saved.",
						*ChildDCBP->GetOutermost()->GetName() // If MarkPackageDirty succeeded then GetOutermost() must exist.
					);
				}
			}
		}
	}

#endif // WITH_EDITOR
}

void UDisplayClusterBlueprint::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR

	// If the exported config is out of date, mark the package as dirty for the user
	// to re-save. This may happen, for example, when the parent blueprint is updated, 
	// or when the config export logic has changed.

	if (GIsEditor)
	{
		const FString LoadedConfigExport = ConfigExport;
		UpdateConfigExportProperty();
		UpdateSummaryProperty(); // Note: No need to mark the asset dirty if the generated Summary has changed since it is not being used externally.

		if (!ConfigExport.Equals(LoadedConfigExport))
		{
			if (MarkPackageDirty())
			{			
				UE_LOGF(LogDisplayClusterBlueprint, Display,
					"ConfigExport of the nDisplay actor '%ls' was not up to date in the asset, so the package was marked as dirty and should be re-saved.",
					*GetOutermost()->GetName() // If MarkPackageDirty succeeded then GetOutermost() must exist.
				);
			}
		}
	}
#endif // WITH_EDITOR
}

UDisplayClusterBlueprintGeneratedClass* UDisplayClusterBlueprint::GetGeneratedClass() const
{
	return Cast<UDisplayClusterBlueprintGeneratedClass>(*GeneratedClass);
}

UDisplayClusterConfigurationData* UDisplayClusterBlueprint::GetOrLoadConfig()
{
	if (GeneratedClass)
	{
		if (ADisplayClusterRootActor* CDO = Cast<ADisplayClusterRootActor>(GeneratedClass->GetDefaultObject(false)))
		{
			ConfigData = CDO->GetConfigData();
		}
	}
	
	return ConfigData;
}

void UDisplayClusterBlueprint::SetConfigData(UDisplayClusterConfigurationData* InConfigData, bool bForceRecreate)
{
#if WITH_EDITOR
	Modify();
#endif

	if (GeneratedClass)
	{
		if (ADisplayClusterRootActor* CDO = Cast<ADisplayClusterRootActor>(GeneratedClass->GetDefaultObject(false)))
		{
			CDO->UpdateConfigDataInstance(InConfigData, bForceRecreate);
			GetOrLoadConfig();
		}
	}
	
#if WITH_EDITORONLY_DATA
	if(InConfigData)
	{
		InConfigData->SaveConfig();
	}
#endif
}

const FString& UDisplayClusterBlueprint::GetConfigPath() const
{
	static FString EmptyString;
#if WITH_EDITORONLY_DATA
	return ConfigData ? ConfigData->PathToConfig : EmptyString;
#else
	return EmptyString;
#endif
}

void UDisplayClusterBlueprint::SetConfigPath(const FString& InPath)
{
#if WITH_EDITORONLY_DATA
	if(UDisplayClusterConfigurationData* LoadedConfigData = GetOrLoadConfig())
	{
		LoadedConfigData->PathToConfig = InPath;
		LoadedConfigData->SaveConfig();
	}
#endif
}

void UDisplayClusterBlueprint::PrepareConfigForExport()
{
	if (!ensure(GeneratedClass))
	{
		return;
	}
		
	ADisplayClusterRootActor* CDO = CastChecked<ADisplayClusterRootActor>(GeneratedClass->GetDefaultObject(false));

	UDisplayClusterConfigurationData* Data = GetOrLoadConfig();
	check(Data);
	
	// Components to export
	TArray<UDisplayClusterCameraComponent*> CameraComponents;
	TArray<UDisplayClusterScreenComponent*> ScreenComponents;
	TArray<USceneComponent*>  XformComponents;
	// Auxiliary map for building hierarchy
	TMap<UActorComponent*, FString> ParentComponentsMap;

	// Make sure the 'Scene' object is there. Otherwise instantiate it.
	// Could be null on assets used during 4.27 development, before scene was added back in.
	const EObjectFlags CommonFlags = RF_Public | RF_Transactional;
	if (Data->Scene == nullptr)
	{
		Data->Scene = NewObject<UDisplayClusterConfigurationScene>(
			this,
			UDisplayClusterConfigurationScene::StaticClass(),
			NAME_None,
			IsTemplate() ? RF_ArchetypeObject | CommonFlags : CommonFlags);
	}
	
	// Extract CDO cameras (no screens in the CDO)
	// Get list of cameras
	CDO->GetComponents(CameraComponents);

	// Extract BP components

	const TArray<USCS_Node*>& Nodes = SimpleConstructionScript->GetAllNodes();
	for (const USCS_Node* const Node : Nodes)
	{
		// Fill ID info for all descendants
		GatherParentComponentsInfo(Node, ParentComponentsMap);

		// Cameras
		if (Node->ComponentClass->IsChildOf(UDisplayClusterCameraComponent::StaticClass()))
		{
			UDisplayClusterCameraComponent* ComponentTemplate = CastChecked<UDisplayClusterCameraComponent>(Node->GetActualComponentTemplate(CastChecked<UBlueprintGeneratedClass>(GeneratedClass)));
			CameraComponents.Add(ComponentTemplate);
		}
		// Screens
		else if (Node->ComponentClass->IsChildOf(UDisplayClusterScreenComponent::StaticClass()))
		{
			UDisplayClusterScreenComponent* ComponentTemplate = CastChecked<UDisplayClusterScreenComponent>(Node->GetActualComponentTemplate(CastChecked<UBlueprintGeneratedClass>(GeneratedClass)));
			ScreenComponents.Add(ComponentTemplate);
		}
		// All other components will be exported as Xforms
		else if (Node->ComponentClass->IsChildOf(USceneComponent::StaticClass()))
		{
			USceneComponent* ComponentTemplate = CastChecked<USceneComponent>(Node->GetActualComponentTemplate(CastChecked<UBlueprintGeneratedClass>(GeneratedClass)));
			XformComponents.Add(ComponentTemplate);
		}
	}

	// Save asset path
	Data->Info.AssetPath = GetPathName();

	// Prepare the target containers
	Data->Scene->Cameras.Empty(CameraComponents.Num());
	Data->Scene->Screens.Empty(ScreenComponents.Num());
	Data->Scene->Xforms.Empty(XformComponents.Num());

	// Export cameras
	for (const UDisplayClusterCameraComponent* const CfgComp : CameraComponents)
	{
		UDisplayClusterConfigurationSceneComponentCamera* SceneComp = NewObject<UDisplayClusterConfigurationSceneComponentCamera>(Data->Scene, CfgComp->GetFName(), RF_Public);

		// Save the properties
		SceneComp->bSwapEyes = CfgComp->GetSwapEyes();
		SceneComp->InterpupillaryDistance = CfgComp->GetInterpupillaryDistance();
		// Safe to cast -- values match.
		SceneComp->StereoOffset = (EDisplayClusterConfigurationEyeStereoOffset)CfgComp->GetStereoOffset();

		FString* ParentId = ParentComponentsMap.Find(CfgComp);
		SceneComp->ParentId = (ParentId ? *ParentId : FString());
		SceneComp->Location = CfgComp->GetRelativeLocation();
		SceneComp->Rotation = CfgComp->GetRelativeRotation();

		// Store the object
		Data->Scene->Cameras.Emplace(GetObjectNameFromSCSNode(SceneComp), SceneComp);
	}

	// Export screens
	for (const UDisplayClusterScreenComponent* const CfgComp : ScreenComponents)
	{
		UDisplayClusterConfigurationSceneComponentScreen* SceneComp = NewObject<UDisplayClusterConfigurationSceneComponentScreen>(Data->Scene, CfgComp->GetFName());

		// Save the properties
		FString* ParentId = ParentComponentsMap.Find(CfgComp);
		SceneComp->ParentId = (ParentId ? *ParentId : FString());
		SceneComp->Location = CfgComp->GetRelativeLocation();
		SceneComp->Rotation = CfgComp->GetRelativeRotation();

		const FVector RelativeCompScale = CfgComp->GetRelativeScale3D();
		SceneComp->Size = FVector2D(RelativeCompScale.Y, RelativeCompScale.Z);

		// Store the object
		Data->Scene->Screens.Emplace(GetObjectNameFromSCSNode(SceneComp), SceneComp);
	}

	// Export xforms
	for (const USceneComponent* const CfgComp : XformComponents)
	{
		UDisplayClusterConfigurationSceneComponentXform* SceneComp = NewObject<UDisplayClusterConfigurationSceneComponentXform>(Data->Scene, CfgComp->GetFName());

		// Save the properties
		FString* ParentId = ParentComponentsMap.Find(CfgComp);
		SceneComp->ParentId = (ParentId ? *ParentId : FString());
		SceneComp->Location = CfgComp->GetRelativeLocation();
		SceneComp->Rotation = CfgComp->GetRelativeRotation();

		// Store the object
		Data->Scene->Xforms.Emplace(GetObjectNameFromSCSNode(SceneComp), SceneComp);
	}

	// Avoid empty string keys in the config data maps
	CleanupConfigMaps(Data);
}

FString UDisplayClusterBlueprint::GetObjectNameFromSCSNode(const UObject* const Object) const
{
	FString OutCompName;

	if (Object)
	{
		OutCompName = Object->GetName();
		OutCompName.RemoveFromEnd(TEXT("_GEN_VARIABLE"));
	}

	return OutCompName;
}

void UDisplayClusterBlueprint::GatherParentComponentsInfo(const USCS_Node* const InNode,
	TMap<UActorComponent*, FString>& OutParentsMap) const
{
	if (InNode && InNode->ComponentClass->IsChildOf(UActorComponent::StaticClass()))
	{
		// Save current node to the map
		UActorComponent* ParentNode = InNode->GetActualComponentTemplate(CastChecked<UBlueprintGeneratedClass>(GeneratedClass));
		if (!OutParentsMap.Contains(ParentNode))
		{
			OutParentsMap.Emplace(ParentNode);
		}

		// Now iterate through the children nodes
		for (USCS_Node* ChildNode : InNode->ChildNodes)
		{
			UActorComponent* ChildComponentTemplate = CastChecked<UActorComponent>(ChildNode->GetActualComponentTemplate(CastChecked<UBlueprintGeneratedClass>(GeneratedClass)));
			if (ChildComponentTemplate)
			{
				OutParentsMap.Emplace(ChildComponentTemplate, GetObjectNameFromSCSNode(InNode->ComponentTemplate));
			}

			GatherParentComponentsInfo(ChildNode, OutParentsMap);
		}
	}
}

void UDisplayClusterBlueprint::CleanupConfigMaps(UDisplayClusterConfigurationData* Data) const
{
	check(Data && Data->Cluster);

	static const FString InvalidKey = FString();

	// Set of the maps we're going to process
	TSet<TMap<FString, FString>*> MapsToProcess;
	// Pre-allocate some memory. Not a precise amount of elements, but should be enough in most cases
	MapsToProcess.Reserve(3 + 4 * Data->Cluster->Nodes.Num());

	// Add single instance maps
	MapsToProcess.Add(&Data->CustomParameters);
	MapsToProcess.Add(&Data->Cluster->Sync.InputSyncPolicy.Parameters);
	MapsToProcess.Add(&Data->Cluster->Sync.RenderSyncPolicy.Parameters);

	// Add per-node and per-viewport maps
	Data->Cluster->Nodes.Remove(InvalidKey);
	for (TPair<FString, UDisplayClusterConfigurationClusterNode*> Node : Data->Cluster->Nodes)
	{
		check(Node.Value);

		// Per-node maps
		Node.Value->Postprocess.Remove(InvalidKey);
		for (TPair<FString, FDisplayClusterConfigurationPostprocess> PostOpIt : Node.Value->Postprocess)
		{
			MapsToProcess.Add(&PostOpIt.Value.Parameters);
		}

		// Per-viewport maps
		Node.Value->Viewports.Remove(InvalidKey);
		for (TPair<FString, UDisplayClusterConfigurationViewport*> ViewportIt : Node.Value->Viewports)
		{
			check(ViewportIt.Value);
			MapsToProcess.Add(&ViewportIt.Value->ProjectionPolicy.Parameters);
		}
	}

	// Finally, remove all the pairs with empty keys
	for (TMap<FString, FString>* Map : MapsToProcess)
	{
		Map->Remove(InvalidKey);
	}
}


void UDisplayClusterBlueprint::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);

	// Add ConfigExport to the tags so that it is asset searchable.
	Context.AddTag(FAssetRegistryTag(TEXT("ConfigExport"), ConfigExport, FAssetRegistryTag::TT_Hidden));
}

