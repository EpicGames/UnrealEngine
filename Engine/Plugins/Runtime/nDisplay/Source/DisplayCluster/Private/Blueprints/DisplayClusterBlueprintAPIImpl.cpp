// Copyright Epic Games, Inc. All Rights Reserved.

#include "Blueprints/DisplayClusterBlueprintAPIImpl.h"
#include "Blueprints/DisplayClusterBlueprintLib.h"


// Cluster runtime API

EDisplayClusterOperationMode UDisplayClusterBlueprintAPIImpl::GetOperationMode() const
{
	return UDisplayClusterBlueprintLib::GetOperationMode();
}

ADisplayClusterRootActor* UDisplayClusterBlueprintAPIImpl::GetRootActor() const
{
	return UDisplayClusterBlueprintLib::GetRootActor();
}

// Local node runtime API

FString UDisplayClusterBlueprintAPIImpl::GetNodeId() const
{
	return UDisplayClusterBlueprintLib::GetNodeId();
}

void UDisplayClusterBlueprintAPIImpl::GetActiveNodeIds(TArray<FString>& OutNodeIds) const
{
	UDisplayClusterBlueprintLib::GetActiveNodeIds(OutNodeIds);
}

int32 UDisplayClusterBlueprintAPIImpl::GetActiveNodesAmount() const
{
	return UDisplayClusterBlueprintLib::GetActiveNodesAmount();
}

bool UDisplayClusterBlueprintAPIImpl::IsPrimary() const
{
	return UDisplayClusterBlueprintLib::IsPrimary();
}

bool UDisplayClusterBlueprintAPIImpl::IsSecondary() const
{
	return UDisplayClusterBlueprintLib::IsSecondary();
}

bool UDisplayClusterBlueprintAPIImpl::IsBackup() const
{
	return UDisplayClusterBlueprintLib::IsBackup();
}

EDisplayClusterNodeRole UDisplayClusterBlueprintAPIImpl::GetClusterRole() const
{
	return UDisplayClusterBlueprintLib::GetClusterRole();
}

// Cluster events API

void UDisplayClusterBlueprintAPIImpl::AddClusterEventListener(TScriptInterface<IDisplayClusterClusterEventListener> Listener)
{
	UDisplayClusterBlueprintLib::AddClusterEventListener(Listener);
}

void UDisplayClusterBlueprintAPIImpl::RemoveClusterEventListener(TScriptInterface<IDisplayClusterClusterEventListener> Listener)
{
	UDisplayClusterBlueprintLib::RemoveClusterEventListener(Listener);
}

void UDisplayClusterBlueprintAPIImpl::EmitClusterEventJson(const FDisplayClusterClusterEventJson& Event, bool bPrimaryOnly)
{
	UDisplayClusterBlueprintLib::EmitClusterEventJson(Event, bPrimaryOnly);
}

void UDisplayClusterBlueprintAPIImpl::EmitClusterEventBinary(const FDisplayClusterClusterEventBinary& Event, bool bPrimaryOnly)
{
	UDisplayClusterBlueprintLib::EmitClusterEventBinary(Event, bPrimaryOnly);
}

void UDisplayClusterBlueprintAPIImpl::SendClusterEventJsonTo(const FString& Address, const int32 Port, const FDisplayClusterClusterEventJson& Event, bool bPrimaryOnly)
{
	UDisplayClusterBlueprintLib::SendClusterEventJsonTo(Address, Port, Event, bPrimaryOnly);
}

void UDisplayClusterBlueprintAPIImpl::SendClusterEventBinaryTo(const FString& Address, const int32 Port, const FDisplayClusterClusterEventBinary& Event, bool bPrimaryOnly)
{
	UDisplayClusterBlueprintLib::SendClusterEventBinaryTo(Address, Port, Event, bPrimaryOnly);
}
