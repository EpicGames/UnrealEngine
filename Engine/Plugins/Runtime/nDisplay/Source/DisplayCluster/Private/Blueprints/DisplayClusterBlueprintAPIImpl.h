// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Blueprints/IDisplayClusterBlueprintAPI.h"
#include "DisplayClusterBlueprintAPIImpl.generated.h"


/**
 * Blueprint API interface implementation
 */
UCLASS()
class DISPLAYCLUSTER_API UDisplayClusterBlueprintAPIImpl
	: public UObject
	, public IDisplayClusterBlueprintAPI
{
	GENERATED_BODY()

public:
	// DisplayCluster module API
	UE_DEPRECATED(5.4, "This function has been deprecated and will be removed soon.")
	UFUNCTION(BlueprintCallable, meta = (DeprecatedFunction, DeprecationMessage = "This function has been deprecated and will be removed soon."))
	virtual bool IsModuleInitialized() const override
	{
		return true;
	}

public:
	// Runtime Cluster API
	UE_DEPRECATED(5.4, "This function has been moved to UDisplayClusterBlueprintLib.")
	UFUNCTION(BlueprintCallable, meta = (DeprecatedFunction, DeprecationMessage = "This function is now available in the main blueprint functions list under 'nDisplay' section."))
	virtual EDisplayClusterOperationMode GetOperationMode() const override;

	UE_DEPRECATED(5.4, "This function has been moved to UDisplayClusterBlueprintLib.")
	UFUNCTION(BlueprintCallable, meta = (DeprecatedFunction, DeprecationMessage = "This function is now available in the main blueprint functions list under 'nDisplay' section."))
	virtual ADisplayClusterRootActor* GetRootActor() const override;

	// Runtime local node API
	UE_DEPRECATED(5.4, "This function has been moved to UDisplayClusterBlueprintLib.")
	UFUNCTION(BlueprintCallable, meta = (DeprecatedFunction, DeprecationMessage = "This function is now available in the main blueprint functions list under 'nDisplay' section."))
	virtual FString GetNodeId() const override;
	
	/** Returns List of the active nodes in the runtime cluster node in a cluster. */
	UE_DEPRECATED(5.4, "This function has been moved to UDisplayClusterBlueprintLib.")
	UFUNCTION(BlueprintCallable, meta = (DeprecatedFunction, DeprecationMessage = "This function is now available in the main blueprint functions list under 'nDisplay' section."))
	virtual void GetActiveNodeIds(TArray<FString>& OutNodeIds) const override;
	

	/** Returns amount of active nodes in a cluster. */
	UE_DEPRECATED(5.4, "This function has been moved to UDisplayClusterBlueprintLib.")
	UFUNCTION(BlueprintCallable, meta = (DeprecatedFunction, DeprecationMessage = "This function is now available in the main blueprint functions list under 'nDisplay' section."))
	virtual int32 GetActiveNodesAmount() const override;

	UE_DEPRECATED(5.4, "This function has been moved to UDisplayClusterBlueprintLib.")
	UFUNCTION(BlueprintCallable, meta = (DeprecatedFunction, DeprecationMessage = "This function is now available in the main blueprint functions list under 'nDisplay' section."))
	virtual EDisplayClusterNodeRole GetClusterRole() const override;

	UE_DEPRECATED(5.4, "This function has been moved to UDisplayClusterBlueprintLib.")
	UFUNCTION(BlueprintCallable, meta = (DeprecatedFunction, DeprecationMessage = "This function is now available in the main blueprint functions list under 'nDisplay' section."))
	virtual bool IsPrimary() const override;
	
	UE_DEPRECATED(5.4, "This function has been moved to UDisplayClusterBlueprintLib.")
	UFUNCTION(BlueprintCallable, meta = (DeprecatedFunction, DeprecationMessage = "This function is now available in the main blueprint functions list under 'nDisplay' section."))
	virtual bool IsSecondary() const override;

	/** Returns true if current node is a backup node in a cluster. */
	UE_DEPRECATED(5.4, "This function has been moved to UDisplayClusterBlueprintLib.")
	UFUNCTION(BlueprintCallable, meta = (DeprecatedFunction, DeprecationMessage = "This function is now available in the main blueprint functions list under 'nDisplay' section."))
	virtual bool IsBackup() const override;

	// Cluster events API
	UE_DEPRECATED(5.4, "This function has been moved to UDisplayClusterBlueprintLib.")
	UFUNCTION(BlueprintCallable, meta = (DeprecatedFunction, DeprecationMessage = "This function is now available in the main blueprint functions list under 'nDisplay' section."))
	virtual void AddClusterEventListener(TScriptInterface<IDisplayClusterClusterEventListener> Listener) override;

	UE_DEPRECATED(5.4, "This function has been moved to UDisplayClusterBlueprintLib.")
	UFUNCTION(BlueprintCallable, meta = (DeprecatedFunction, DeprecationMessage = "This function is now available in the main blueprint functions list under 'nDisplay' section."))
	virtual void RemoveClusterEventListener(TScriptInterface<IDisplayClusterClusterEventListener> Listener) override;

	UE_DEPRECATED(5.4, "This function has been moved to UDisplayClusterBlueprintLib.")
	UFUNCTION(BlueprintCallable, meta = (DeprecatedFunction, DeprecationMessage = "This function is now available in the main blueprint functions list under 'nDisplay' section."))
	virtual void EmitClusterEventJson(const FDisplayClusterClusterEventJson& Event, bool bPrimaryOnly) override;

	UE_DEPRECATED(5.4, "This function has been moved to UDisplayClusterBlueprintLib.")
	UFUNCTION(BlueprintCallable, meta = (DeprecatedFunction, DeprecationMessage = "This function is now available in the main blueprint functions list under 'nDisplay' section."))
	virtual void EmitClusterEventBinary(const FDisplayClusterClusterEventBinary& Event, bool bPrimaryOnly) override;

	UE_DEPRECATED(5.4, "This function has been moved to UDisplayClusterBlueprintLib.")
	UFUNCTION(BlueprintCallable, meta = (DeprecatedFunction, DeprecationMessage = "This function is now available in the main blueprint functions list under 'nDisplay' section."))
	virtual void SendClusterEventJsonTo(const FString& Address, const int32 Port, const FDisplayClusterClusterEventJson& Event, bool bPrimaryOnly) override;

	UE_DEPRECATED(5.4, "This function has been moved to UDisplayClusterBlueprintLib.")
	UFUNCTION(BlueprintCallable, meta = (DeprecatedFunction, DeprecationMessage = "This function is now available in the main blueprint functions list under 'nDisplay' section."))
	virtual void SendClusterEventBinaryTo(const FString& Address, const int32 Port, const FDisplayClusterClusterEventBinary& Event, bool bPrimaryOnly) override;
};
