// Copyright Epic Games, Inc. All Rights Reserved.

#include "Blueprints/DisplayClusterBlueprintLib.h"
#include "Blueprints/DisplayClusterBlueprintAPIImpl.h"

#include "Cluster/IDisplayClusterClusterManager.h"
#include "Components/DisplayClusterICVFXCameraComponent.h"
#include "Game/IDisplayClusterGameManager.h"
#include "Misc/DisplayClusterLog.h"

#include "IDisplayCluster.h"

#include "DisplayClusterChromakeyCardActor.h"
#include "DisplayClusterLightCardActor.h"
#include "DisplayClusterRootActor.h"
#include "DisplayClusterConfigurationTypes.h"

#include "UObject/Package.h"

#if WITH_EDITOR
#include "ScopedTransaction.h"
#endif

#define LOCTEXT_NAMESPACE "UDisplayClusterBlueprintLib"


// [DEPRECATED]
void UDisplayClusterBlueprintLib::GetAPI(TScriptInterface<IDisplayClusterBlueprintAPI>& OutAPI)
{
	static UDisplayClusterBlueprintAPIImpl* Obj = NewObject<UDisplayClusterBlueprintAPIImpl>(GetTransientPackage(), NAME_None, RF_MarkAsRootSet);
	OutAPI = Obj;
}

EDisplayClusterOperationMode UDisplayClusterBlueprintLib::GetOperationMode()
{
	return IDisplayCluster::Get().GetOperationMode();
}

ADisplayClusterRootActor* UDisplayClusterBlueprintLib::GetRootActor()
{
	return IDisplayCluster::Get().GetGameMgr()->GetRootActor();
}

FString UDisplayClusterBlueprintLib::GetNodeId()
{
	return IDisplayCluster::Get().GetClusterMgr()->GetNodeId();
}

void UDisplayClusterBlueprintLib::GetActiveNodeIds(TArray<FString>& OutNodeIds)
{
	IDisplayCluster::Get().GetClusterMgr()->GetNodeIds(OutNodeIds);
}

int32 UDisplayClusterBlueprintLib::GetActiveNodesAmount()
{
	return IDisplayCluster::Get().GetClusterMgr()->GetNodesAmount();
}

bool UDisplayClusterBlueprintLib::IsPrimary()
{
	return IDisplayCluster::Get().GetClusterMgr()->IsPrimary();
}

bool UDisplayClusterBlueprintLib::IsSecondary()
{
	return IDisplayCluster::Get().GetClusterMgr()->IsSecondary();
}

bool UDisplayClusterBlueprintLib::IsBackup()
{
	return IDisplayCluster::Get().GetClusterMgr()->IsBackup();
}

bool UDisplayClusterBlueprintLib::HasClusterRole(EDisplayClusterNodeRole Role)
{
	return IDisplayCluster::Get().GetClusterMgr()->HasClusterRole(Role);
}

EDisplayClusterNodeRole UDisplayClusterBlueprintLib::GetClusterRole()
{
	return IDisplayCluster::Get().GetClusterMgr()->GetClusterRole();
}

void UDisplayClusterBlueprintLib::AddClusterEventListener(TScriptInterface<IDisplayClusterClusterEventListener> Listener)
{
	IDisplayCluster::Get().GetClusterMgr()->AddClusterEventListener(Listener);
}

void UDisplayClusterBlueprintLib::RemoveClusterEventListener(TScriptInterface<IDisplayClusterClusterEventListener> Listener)
{
	IDisplayCluster::Get().GetClusterMgr()->RemoveClusterEventListener(Listener);
}

void UDisplayClusterBlueprintLib::EmitClusterEventJson(const FDisplayClusterClusterEventJson& Event, bool bPrimaryOnly)
{
	IDisplayCluster::Get().GetClusterMgr()->EmitClusterEventJson(Event, bPrimaryOnly);
}

void UDisplayClusterBlueprintLib::EmitClusterEventBinary(const FDisplayClusterClusterEventBinary& Event, bool bPrimaryOnly)
{
	IDisplayCluster::Get().GetClusterMgr()->EmitClusterEventBinary(Event, bPrimaryOnly);
}

void UDisplayClusterBlueprintLib::SendClusterEventJsonTo(const FString& Address, const int32 Port, const FDisplayClusterClusterEventJson& Event, bool bPrimaryOnly)
{
	if (Port >= 0 && Port <= 0xffff)
	{
		const uint16 PortNumber = static_cast<uint16>(Port);
		IDisplayCluster::Get().GetClusterMgr()->SendClusterEventTo(Address, PortNumber, Event, bPrimaryOnly);
	}
	else
	{
		UE_LOGF(LogDisplayClusterBlueprint, Warning, "Wrong port number: %d", Port);
	}
}

void UDisplayClusterBlueprintLib::SendClusterEventBinaryTo(const FString& Address, const int32 Port, const FDisplayClusterClusterEventBinary& Event, bool bPrimaryOnly)
{
	if (Port >= 0 && Port <= 0xffff)
	{
		const uint16 PortNumber = static_cast<uint16>(Port);
		IDisplayCluster::Get().GetClusterMgr()->SendClusterEventTo(Address, PortNumber, Event, bPrimaryOnly);
	}
	else
	{
		UE_LOGF(LogDisplayClusterBlueprint, Warning, "Wrong port number: %d", Port);
	}
}

ADisplayClusterLightCardActor* UDisplayClusterBlueprintLib::CreateLightCard(ADisplayClusterRootActor* RootActor)
{
	if (!RootActor)
	{
		return nullptr;
	}

	// Create the light card
#if WITH_EDITOR
	FScopedTransaction Transaction(LOCTEXT("CreateLightCard", "Create Light Card"));
#endif

	const FVector SpawnLocation = RootActor->GetDefaultCamera()->GetComponentLocation();
	FRotator SpawnRotation = RootActor->GetDefaultCamera()->GetComponentRotation();
	SpawnRotation.Yaw -= 180.f;

	FActorSpawnParameters SpawnParameters;
	SpawnParameters.bNoFail = true;
	SpawnParameters.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AdjustIfPossibleButAlwaysSpawn;
	SpawnParameters.Name = TEXT("LightCard");
	SpawnParameters.NameMode = FActorSpawnParameters::ESpawnActorNameMode::Requested;
	SpawnParameters.OverrideLevel = RootActor->GetLevel();

	ADisplayClusterLightCardActor* NewActor = CastChecked<ADisplayClusterLightCardActor>(
		RootActor->GetWorld()->SpawnActor(ADisplayClusterLightCardActor::StaticClass(),
			&SpawnLocation, &SpawnRotation, MoveTemp(SpawnParameters)));

	NewActor->Latitude = 30.0;
	NewActor->Longitude = 180.0;
	NewActor->Color = FLinearColor::Gray;

#if WITH_EDITOR
	NewActor->SetActorLabel(NewActor->GetName());
#endif

	FAttachmentTransformRules AttachmentRules(EAttachmentRule::KeepWorld, false);
	NewActor->AttachToActor(RootActor, AttachmentRules);

	// Add it to the root actor
	NewActor->AddToRootActor(RootActor);

#if WITH_EDITOR
	// Required so operator panel updates
	RootActor->PostEditChange();
#endif
	
	return NewActor;
}

void UDisplayClusterBlueprintLib::DuplicateLightCards(TArray<ADisplayClusterLightCardActor*> OriginalLightcards, TArray<ADisplayClusterLightCardActor*>& OutNewLightCards)
{
#if WITH_EDITOR
	TMap<UWorld*, FCachedActorLabels> ActorLabelsByWorld;
	FScopedTransaction Transaction(LOCTEXT("DuplicateLightCard", "Duplicate Light Cards"));
#endif

	for (ADisplayClusterLightCardActor* OriginalLightcard : OriginalLightcards)
	{
		if (!OriginalLightcard)
		{
			continue;
		}

		ADisplayClusterRootActor* RootActor = OriginalLightcard->GetRootActorOwner();
		if (!RootActor)
		{
			continue;
		}

		UWorld* World = RootActor->GetWorld();
		ULevel* Level = World ? World->GetCurrentLevel() : nullptr;
		if (!Level)
		{
			continue;
		}

		const FName UniqueName = MakeUniqueObjectName(Level, OriginalLightcard->GetClass());
		ADisplayClusterLightCardActor* NewLightCard = CastChecked<ADisplayClusterLightCardActor>(StaticDuplicateObject(OriginalLightcard, Level, UniqueName));

#if WITH_EDITOR
		Level->AddLoadedActor(NewLightCard);

		FCachedActorLabels* ActorLabels = ActorLabelsByWorld.Find(World);
		if (!ActorLabels)
		{
			ActorLabels = &ActorLabelsByWorld.Emplace(World, World);
		}

		FActorLabelUtilities::SetActorLabelUnique(NewLightCard, NewLightCard->GetActorLabel(), ActorLabels);
		ActorLabels->Add(NewLightCard->GetActorLabel());
#endif

		FAttachmentTransformRules AttachmentRules(EAttachmentRule::KeepWorld, false);
		NewLightCard->AttachToActor(RootActor, AttachmentRules);

		// Add it to the root actor
		NewLightCard->AddToRootActor(RootActor);

#if WITH_EDITOR
		// Required so operator panel updates
		RootActor->PostEditChange();
#endif

#if WITH_EDITOR
		if (GIsEditor)
		{
			GEditor->BroadcastLevelActorAdded(NewLightCard);
		}
#endif

		OutNewLightCards.Add(NewLightCard);
	}
}

void UDisplayClusterBlueprintLib::FindLightCardsForRootActor(const ADisplayClusterRootActor* RootActor, TSet<ADisplayClusterLightCardActor*>& OutLightCards)
{
	if (!RootActor)
	{
		return;
	}

	FDisplayClusterConfigurationICVFX_VisibilityList& RootActorLightCards = RootActor->GetConfigData()->StageSettings.Lightcard.ShowOnlyList;

	for (const TSoftObjectPtr<AActor>& LightCardActor : RootActorLightCards.Actors)
	{
		if (!LightCardActor.IsValid() || !LightCardActor->IsA<ADisplayClusterLightCardActor>())
		{
			continue;
		}

		OutLightCards.Add(Cast<ADisplayClusterLightCardActor>(LightCardActor.Get()));
	}

	if (const UWorld* World = RootActor->GetWorld())
	{
		for (TActorIterator<ADisplayClusterLightCardActor> ActorIt(World); ActorIt; ++ActorIt)
		{
			if (!IsValid(*ActorIt))
			{
				continue;
			}

			if (ActorIt->GetRootActorOwner() == RootActor)
			{
				OutLightCards.Add(*ActorIt);
			}
			else
			{
				// If there are any layers that are specified as light card layers, iterate over all actors in the world and 
				// add any that are members of any of the light card layers to the list. Only add an actor once, even if it is
				// in multiple layers
				for (const FActorLayer& ActorLayer : RootActorLightCards.ActorLayers)
				{
					if (ActorIt->Layers.Contains(ActorLayer.Name))
					{
						OutLightCards.Add(*ActorIt);
						break;
					}
				}
			}
		}
	}
}

void UDisplayClusterBlueprintLib::FindChromakeyCardsForRootActor(const ADisplayClusterRootActor* RootActor,
                                                                 TSet<ADisplayClusterChromakeyCardActor*>& OutChromakeyCards)
{
	if (!RootActor)
	{
		return;
	}

	TArray<UDisplayClusterICVFXCameraComponent*> CameraComponents;
	RootActor->GetComponents(CameraComponents);

	for (const UDisplayClusterICVFXCameraComponent* Camera : CameraComponents)
	{
		const FDisplayClusterConfigurationICVFX_ChromakeyRenderSettings* CameraChromakeyRenderSettings = Camera->GetCameraSettingsICVFX().Chromakey.GetChromakeyRenderSettings(RootActor->GetStageSettings());
		const FDisplayClusterConfigurationICVFX_VisibilityList* CameraChromakeyShowOnlyList = CameraChromakeyRenderSettings ? &CameraChromakeyRenderSettings->ShowOnlyList : nullptr;

		if (CameraChromakeyShowOnlyList)
		{
			for (const TSoftObjectPtr<AActor>& ChromakeyCardActor : CameraChromakeyShowOnlyList->Actors)
			{
				if (!ChromakeyCardActor.IsValid() || !ChromakeyCardActor->IsA<ADisplayClusterChromakeyCardActor>())
				{
					continue;
				}

				OutChromakeyCards.Add(Cast<ADisplayClusterChromakeyCardActor>(ChromakeyCardActor.Get()));
			}
		}

		if (const UWorld* World = RootActor->GetWorld())
		{
			for (TActorIterator<ADisplayClusterChromakeyCardActor> ActorIt(World); ActorIt; ++ActorIt)
			{
				if (!IsValid(*ActorIt))
				{
					continue;
				}

				if (ActorIt->GetRootActorOwner() == RootActor)
				{
					OutChromakeyCards.Add(*ActorIt);
				}
				else if(CameraChromakeyShowOnlyList)
				{
					for (const FActorLayer& ActorLayer : CameraChromakeyShowOnlyList->ActorLayers)
					{
						if (ActorIt->Layers.Contains(ActorLayer.Name))
						{
							OutChromakeyCards.Add(*ActorIt);
							break;
						}
					}
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
