// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cluster/Controller/DisplayClusterClusterNodeCtrlBase.h"
#include "Cluster/IPDisplayClusterClusterManager.h"

#include "Misc/DisplayClusterGlobals.h"
#include "Misc/DisplayClusterLog.h"

#include "Network/IDisplayClusterServer.h"
#include "Network/IDisplayClusterClient.h"
#include "Network/Configuration/DisplayClusterNetworkConfiguration.h"
#include "Network/Listener/DisplayClusterTcpListener.h"
#include "Network/Service/ClusterEventsJson/DisplayClusterClusterEventsJsonClient.h"
#include "Network/Service/ClusterEventsBinary/DisplayClusterClusterEventsBinaryClient.h"


FDisplayClusterClusterNodeCtrlBase::FDisplayClusterClusterNodeCtrlBase(const FString& InControllerName, const FString& InClusterNodeId)
	: ClusterNodeId(InClusterNodeId)
	, ControllerName(InControllerName)
{
}

FDisplayClusterClusterNodeCtrlBase::~FDisplayClusterClusterNodeCtrlBase()
{
}

TWeakPtr<FDisplayClusterService> FDisplayClusterClusterNodeCtrlBase::GetService(const FName& ServiceName) const
{
	FScopeLock Lock(&RegisteredServicesCS);

	const TSharedPtr<FDisplayClusterService>* const ServiceFound = RegisteredServices.Find(ServiceName);
	if (!ServiceFound)
	{
		return nullptr;
	}

	return ServiceFound->ToWeakPtr();
}

void FDisplayClusterClusterNodeCtrlBase::SendClusterEventTo(const FString& Address, const uint16 Port, const FDisplayClusterClusterEventJson& Event, bool bPrimaryOnly)
{
	const bool bIsPrimary  = GDisplayCluster->GetClusterMgr()->IsPrimary();
	const bool bShouldSend = (bPrimaryOnly ? bIsPrimary : true);

	if (bShouldSend)
	{
		if (TUniquePtr<FDisplayClusterClusterEventsJsonClient> Client = MakeUnique<FDisplayClusterClusterEventsJsonClient>(UE::nDisplay::Network::Configuration::JsonEventsExternalClientName, false))
		{
			if (Client->Connect(Address, Port, 1, 0.f))
			{
				Client->EmitClusterEventJson(Event);
				Client->Disconnect();
			}
		}
	}
}

void FDisplayClusterClusterNodeCtrlBase::SendClusterEventTo(const FString& Address, const uint16 Port, const FDisplayClusterClusterEventBinary& Event, bool bPrimaryOnly)
{
	const bool bIsPrimary  = GDisplayCluster->GetClusterMgr()->IsPrimary();
	const bool bShouldSend = (bPrimaryOnly ? bIsPrimary : true);

	if (bShouldSend)
	{
		if (TUniquePtr<FDisplayClusterClusterEventsBinaryClient> Client = MakeUnique<FDisplayClusterClusterEventsBinaryClient>(UE::nDisplay::Network::Configuration::BinaryEventsExternalClientName, false))
		{
			if (Client->Connect(Address, Port, 1, 0.f))
			{
				Client->EmitClusterEventBinary(Event);
				Client->Disconnect();
			}
		}
	}
}

TMap<FName, TSharedPtr<FDisplayClusterService>> FDisplayClusterClusterNodeCtrlBase::GetRegisteredServices()
{
	FScopeLock Lock(&RegisteredServicesCS);
	return RegisteredServices;
}

bool FDisplayClusterClusterNodeCtrlBase::StartServerWithLogs(const TSharedPtr<IDisplayClusterServer>& Server, const FString& Address, const uint16 Port) const
{
	if (!Server)
	{
		UE_LOGF(LogDisplayClusterCluster, Error, "%ls - Invalid server instance (nullptr)", *GetControllerName());
		return false;
	}

	const bool bResult = Server->Start(Address, Port);

	if (bResult)
	{
		UE_LOGF(LogDisplayClusterCluster, Log, "%ls - Server '%ls' started at [%ls:%u]", *GetControllerName(), *Server->GetName(), *Address, Port);
	}
	else
	{
		UE_LOGF(LogDisplayClusterCluster, Error, "%ls - Server '%ls' failed to start at [%ls:%u]", *GetControllerName(), *Server->GetName(), *Address, Port);
	}

	return bResult;
}

bool FDisplayClusterClusterNodeCtrlBase::StartServerWithLogs(const TSharedPtr<IDisplayClusterServer>& Server, TSharedRef<FDisplayClusterTcpListener>& TcpListener)
{
	if (!Server)
	{
		UE_LOGF(LogDisplayClusterCluster, Error, "%ls - Invalid server instance (nullptr)", *GetControllerName());
		return false;
	}

	const bool bResult = Server->Start(TcpListener);

	if (bResult)
	{
		if (TcpListener->IsListening())
		{
			UE_LOGF(LogDisplayClusterCluster, Log, "%ls - Server '%ls' started at [%ls:%u]", *GetControllerName(), *Server->GetName(), *TcpListener->GetListeningHost(), TcpListener->GetListeningPort());
		}
		else
		{
			UE_LOGF(LogDisplayClusterCluster, Log, "%ls - Server '%ls' is awaiting for the listener to start", *GetControllerName(), *Server->GetName());
		}
	}
	else
	{
		UE_LOGF(LogDisplayClusterCluster, Error, "%ls - Server '%ls' failed to start at [%ls:%u]", *GetControllerName(), *Server->GetName(), *TcpListener->GetListeningHost(), TcpListener->GetListeningPort());
	}

	return bResult;
}

bool FDisplayClusterClusterNodeCtrlBase::StartClientWithLogs(IDisplayClusterClient* Client, const FString& Address, const uint16 Port, const uint32 ClientConnTriesAmount, const uint32 ClientConnRetryDelay)
{
	if (!Client)
	{
		UE_LOGF(LogDisplayClusterCluster, Error, "%ls - Invalid client instance (nullptr)", *GetControllerName());
		return false;
	}

	const bool bResult = Client->Connect(Address, Port, ClientConnTriesAmount, ClientConnRetryDelay);

	if (bResult)
	{
		UE_LOGF(LogDisplayClusterCluster, Log, "%ls - client '%ls' connected to [%ls:%u]", *GetControllerName(), *Client->GetName(), *Address, Port);
	}
	else
	{
		UE_LOGF(LogDisplayClusterCluster, Error, "%ls - client '%ls' couldn't connect to [%ls:%u]", *GetControllerName(), *Client->GetName(), *Address, Port);
	}

	return bResult;
}

void FDisplayClusterClusterNodeCtrlBase::RegisterLocalService(const FName& ServiceName, const TSharedRef<FDisplayClusterService>& Service)
{
	FScopeLock Lock(&RegisteredServicesCS);

	checkSlow(!RegisteredServices.Contains(ServiceName));
	RegisteredServices.Emplace(ServiceName, Service);
}
