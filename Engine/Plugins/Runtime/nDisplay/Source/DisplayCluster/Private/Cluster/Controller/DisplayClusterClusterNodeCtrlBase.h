// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Cluster/Controller/IDisplayClusterClusterNodeController.h"
#include "Network/IDisplayClusterClient.h"

class IDisplayClusterServer;
class FDisplayClusterTcpListener;
class FDisplayClusterClusterEventsBinaryClient;
class FDisplayClusterClusterEventsJsonClient;
struct FDisplayClusterClusterEventBinary;
struct FDisplayClusterClusterEventJson;


/**
 * Base node controller
 *
 * Encapsulates some common controller logic and data
 */
class FDisplayClusterClusterNodeCtrlBase
	: public IDisplayClusterClusterNodeController
{
public:

	FDisplayClusterClusterNodeCtrlBase(const FString& InControllerName, const FString& InClusterNodeId);

	virtual ~FDisplayClusterClusterNodeCtrlBase();

public:

	//~ Begin IDisplayClusterClusterNodeController
	virtual FString GetNodeId() const override final
	{
		return ClusterNodeId;
	}

	virtual FString GetControllerName() const override final
	{
		return ControllerName;
	}

	virtual TWeakPtr<FDisplayClusterService> GetService(const FName& ServiceName) const override;

	virtual void SendClusterEventTo(const FString& Address, const uint16 Port, const FDisplayClusterClusterEventJson& Event, bool bPrimaryOnly) override;
	virtual void SendClusterEventTo(const FString& Address, const uint16 Port, const FDisplayClusterClusterEventBinary& Event, bool bPrimaryOnly) override;
	//~ End IDisplayClusterClusterNodeController

protected:

	/** Provides accesss to the service register for children */
	TMap<FName, TSharedPtr<FDisplayClusterService>> GetRegisteredServices();

	/** A helper function to start a server with its own connection listener */
	bool StartServerWithLogs(const TSharedPtr<IDisplayClusterServer>& Server, const FString& Address, const uint16 Port) const;

	/** A helper function to start a connection listener provided */
	bool StartServerWithLogs(const TSharedPtr<IDisplayClusterServer>& Server, TSharedRef<FDisplayClusterTcpListener>& TcpListener);

	/** A helper function to initialize (connect) a specific client to a specific address */
	bool StartClientWithLogs(IDisplayClusterClient* Client, const FString& Address, const uint16 Port, const uint32 ClientConnTriesAmount, const uint32 ClientConnRetryDelay);

	/** Register local server so it can be reached via API */
	void RegisterLocalService(const FName& ServiceName, const TSharedRef<FDisplayClusterService>& Service);

private:

	/** Cluster node ID */
	const FString ClusterNodeId;

	/** Controller name/ID */
	const FString ControllerName;

private:

	/** Services that have been registered, and accessible from the outside via API */
	TMap<FName, TSharedPtr<FDisplayClusterService>> RegisteredServices;

	/** A critical section to control access to the container of registered services */
	mutable FCriticalSection RegisteredServicesCS;
};
