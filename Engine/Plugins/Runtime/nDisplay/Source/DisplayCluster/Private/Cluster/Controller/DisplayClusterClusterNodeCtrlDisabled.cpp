// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cluster/Controller/DisplayClusterClusterNodeCtrlDisabled.h"


FDisplayClusterClusterNodeCtrlDisabled::FDisplayClusterClusterNodeCtrlDisabled()
	: ControllerName(TEXT("CTRL_OFF"))
{
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterClusterNodeController
//////////////////////////////////////////////////////////////////////////////////////////////
bool FDisplayClusterClusterNodeCtrlDisabled::Initialize()
{
	return true;
}

void FDisplayClusterClusterNodeCtrlDisabled::Shutdown()
{
	// Nothing to do
}

FString FDisplayClusterClusterNodeCtrlDisabled::GetNodeId() const
{
	return FString();
}

FString FDisplayClusterClusterNodeCtrlDisabled::GetControllerName() const
{
	return ControllerName;
}

void FDisplayClusterClusterNodeCtrlDisabled::SendClusterEventTo(const FString& Address, const uint16 Port, const FDisplayClusterClusterEventBinary& Event, bool bPrimaryOnly)
{
	// Nothing to do
}

void FDisplayClusterClusterNodeCtrlDisabled::SendClusterEventTo(const FString& Address, const uint16 Port, const FDisplayClusterClusterEventJson& Event, bool bPrimaryOnly)
{
	// Nothing to do
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProtocolClusterSync
//////////////////////////////////////////////////////////////////////////////////////////////
EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlDisabled::WaitForGameStart()
{
	return EDisplayClusterCommResult::NotAllowed;
}

EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlDisabled::WaitForFrameStart()
{
	return EDisplayClusterCommResult::NotAllowed;
}

EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlDisabled::WaitForFrameEnd()
{
	return EDisplayClusterCommResult::NotAllowed;
}

EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlDisabled::GetTimeData(double& OutDeltaTime, double& OutGameTime, TOptional<FQualifiedFrameTime>& OutFrameTime)
{
	return EDisplayClusterCommResult::NotAllowed;
}

EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlDisabled::GetObjectsData(EDisplayClusterSyncGroup InSyncGroup, TMap<FString, TArray<uint8>>& OutObjectsData)
{
	return EDisplayClusterCommResult::NotAllowed;
}

EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlDisabled::GetEventsData(
	TArray<TSharedPtr<FDisplayClusterClusterEventJson>>& OutJsonEvents,
	TArray<TSharedPtr<FDisplayClusterClusterEventBinary>>& OutBinaryEvents)
{
	return EDisplayClusterCommResult::NotAllowed;
}

EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlDisabled::GetNativeInputData(TMap<FString, TArray<uint8>>& OutNativeInputData)
{
	return EDisplayClusterCommResult::NotAllowed;
}

EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlDisabled::PropagateStatesData(const TArray<uint8>& InLocalStatesData, TArray<uint8>& OutClusterStatesData)
{
	return EDisplayClusterCommResult::NotAllowed;
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProtocolRenderSync
//////////////////////////////////////////////////////////////////////////////////////////////
EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlDisabled::SynchronizeOnBarrier()
{
	return EDisplayClusterCommResult::NotAllowed;
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProtocolEventsJson
//////////////////////////////////////////////////////////////////////////////////////////////
EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlDisabled::EmitClusterEventJson(const FDisplayClusterClusterEventJson& Event)
{
	return EDisplayClusterCommResult::NotAllowed;
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProtocolEventsBinary
//////////////////////////////////////////////////////////////////////////////////////////////
EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlDisabled::EmitClusterEventBinary(const FDisplayClusterClusterEventBinary& Event)
{
	return EDisplayClusterCommResult::NotAllowed;
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProtocolGenericBarrier
//////////////////////////////////////////////////////////////////////////////////////////////
EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlDisabled::CreateBarrier(const FString& BarrierId, const TMap<FString, TSet<FString>>& NodeToSyncCallers, uint32 Timeout, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result)
{
	return EDisplayClusterCommResult::NotAllowed;
}

EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlDisabled::WaitUntilBarrierIsCreated(const FString& BarrierId, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result)
{
	return EDisplayClusterCommResult::NotAllowed;
}

EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlDisabled::IsBarrierAvailable(const FString& BarrierId, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result)
{
	return EDisplayClusterCommResult::NotAllowed;
}

EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlDisabled::ReleaseBarrier(const FString& BarrierId, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result)
{
	return EDisplayClusterCommResult::NotAllowed;
}

EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlDisabled::SyncOnBarrier(const FString& BarrierId, const FString& CallerId, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result)
{
	return EDisplayClusterCommResult::NotAllowed;
}

EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlDisabled::SyncOnBarrierWithData(const FString& BarrierId, const FString& CallerId, const TArray<uint8>& RequestData, TArray<uint8>& OutResponseData, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result)
{
	return EDisplayClusterCommResult::NotAllowed;
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProtocolInternalComm
//////////////////////////////////////////////////////////////////////////////////////////////
EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlDisabled::GatherServicesHostingInfo(const FNodeServicesHostingInfo& ThisNodeInfo, FClusterServicesHostingInfo& OutHostingInfo)
{
	return EDisplayClusterCommResult::NotAllowed;
}

EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlDisabled::PostFailureNegotiate(TArray<uint8>& InOutRecoveryData)
{
	return EDisplayClusterCommResult::NotAllowed;
}

EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlDisabled::RequestNodeDrop(const FString& NodeId, uint8 DropReason)
{
	return EDisplayClusterCommResult::NotAllowed;
}
