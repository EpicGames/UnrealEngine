// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cluster/Controller/DisplayClusterClusterNodeCtrlEditor.h"

#include "Cluster/IPDisplayClusterClusterManager.h"
#include "Config/IPDisplayClusterConfigManager.h"

#include "Misc/DisplayClusterGlobals.h"
#include "Misc/DisplayClusterLog.h"

#include "Network/Configuration/DisplayClusterNetworkConfiguration.h"
#include "Network/Service/ClusterEventsJson/DisplayClusterClusterEventsJsonClient.h"
#include "Network/Service/ClusterEventsBinary/DisplayClusterClusterEventsBinaryService.h"
#include "Network/Service/ClusterEventsBinary/DisplayClusterClusterEventsBinaryClient.h"
#include "Network/Service/ClusterEventsJson/DisplayClusterClusterEventsJsonService.h"

#include "DisplayClusterConfigurationTypes.h"


FDisplayClusterClusterNodeCtrlEditor::FDisplayClusterClusterNodeCtrlEditor()
	: FDisplayClusterClusterNodeCtrlBase(TEXT("CTRL_ED"), TEXT("Node_Editor"))
{
}

FDisplayClusterClusterNodeCtrlEditor::~FDisplayClusterClusterNodeCtrlEditor()
{
	// In case Shutdown() has not been called before deleting this controller, we have to stop
	// all the clients and servers this controller owns. We do it safely (non-virtual shutdown).
	ShutdownImpl();
}

bool FDisplayClusterClusterNodeCtrlEditor::Initialize()
{
	const bool bBaseInitialized = FDisplayClusterClusterNodeCtrlBase::Initialize();
	if (!bBaseInitialized)
	{
		UE_LOGF(LogDisplayClusterCluster, Warning, "%ls - Couldn't initialize base controller internals", *GetControllerName());
		// Don't return false as it's not critical for PIE
	}

	if (!InitializeServers())
	{
		UE_LOGF(LogDisplayClusterCluster, Warning, "%ls - Couldn't initialize internal servers", *GetControllerName());
		// Don't return false as it's not critical for PIE
	}

	if (!StartServers())
	{
		UE_LOGF(LogDisplayClusterCluster, Warning, "%ls - An error occurred while starting internal servers", *GetControllerName());
		// Don't return false as it's not critical for PIE
	}

	return true;
}

void FDisplayClusterClusterNodeCtrlEditor::Shutdown()
{
	ShutdownImpl();
	FDisplayClusterClusterNodeCtrlBase::Shutdown();
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProtocolClusterSync
//////////////////////////////////////////////////////////////////////////////////////////////
EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlEditor::WaitForGameStart()
{
	// PIE stub
	return EDisplayClusterCommResult::Ok;
}

EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlEditor::WaitForFrameStart()
{
	// PIE stub
	return EDisplayClusterCommResult::Ok;
}

EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlEditor::WaitForFrameEnd()
{
	// PIE stub
	return EDisplayClusterCommResult::Ok;
}

EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlEditor::GetTimeData(double& OutDeltaTime, double& OutGameTime, TOptional<FQualifiedFrameTime>& OutFrameTime)
{
	IPDisplayClusterClusterManager* const ClusterMgr = GDisplayCluster->GetPrivateClusterMgr();
	if (!ClusterMgr)
	{
		UE_LOGF(LogDisplayClusterCluster, Warning, "%ls - wrong cluster manager object", *GetControllerName());
		return EDisplayClusterCommResult::InternalError;
	}

	ClusterMgr->CacheTimeData();
	ClusterMgr->ExportTimeData(OutDeltaTime, OutGameTime, OutFrameTime);

	return EDisplayClusterCommResult::Ok;
}

EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlEditor::GetObjectsData(EDisplayClusterSyncGroup InSyncGroup, TMap<FString, TArray<uint8>>& OutObjectsData)
{
	// PIE stub
	return EDisplayClusterCommResult::Ok;
}

EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlEditor::GetEventsData(TArray<TSharedPtr<FDisplayClusterClusterEventJson>>& OutJsonEvents, TArray<TSharedPtr<FDisplayClusterClusterEventBinary>>& OutBinaryEvents)
{
	if (IPDisplayClusterClusterManager* const ClusterMgr = GDisplayCluster->GetPrivateClusterMgr())
	{
		ClusterMgr->CacheEvents();
		ClusterMgr->ExportEventsData(OutJsonEvents, OutBinaryEvents);
		return EDisplayClusterCommResult::Ok;
	}

	return EDisplayClusterCommResult::InternalError;
}

EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlEditor::GetNativeInputData(TMap<FString, TArray<uint8>>& OutNativeInputData)
{
	// PIE stub
	return EDisplayClusterCommResult::Ok;
}

EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlEditor::PropagateStatesData(const TArray<uint8>& InLocalStatesData, TArray<uint8>& OutClusterStatesData)
{
	// PIE stub
	return EDisplayClusterCommResult::Ok;
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProtocolRenderSync
//////////////////////////////////////////////////////////////////////////////////////////////
EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlEditor::SynchronizeOnBarrier()
{
	// PIE stub
	return EDisplayClusterCommResult::Ok;
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProtocolEventsJson
//////////////////////////////////////////////////////////////////////////////////////////////
EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlEditor::EmitClusterEventJson(const FDisplayClusterClusterEventJson& Event)
{
	// PIE stub
	return EDisplayClusterCommResult::Ok;
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProtocolEventsBinary
//////////////////////////////////////////////////////////////////////////////////////////////
EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlEditor::EmitClusterEventBinary(const FDisplayClusterClusterEventBinary& Event)
{
	// PIE stub
	return EDisplayClusterCommResult::Ok;
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProtocolGenericBarrier
//////////////////////////////////////////////////////////////////////////////////////////////
EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlEditor::CreateBarrier(const FString& BarrierId, const TMap<FString, TSet<FString>>& NodeToSyncCallers, uint32 Timeout, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result)
{
	// PIE stub
	return EDisplayClusterCommResult::Ok;
}

EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlEditor::WaitUntilBarrierIsCreated(const FString& BarrierId, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result)
{
	// PIE stub
	return EDisplayClusterCommResult::Ok;
}

EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlEditor::IsBarrierAvailable(const FString& BarrierId, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result)
{
	// PIE stub
	return EDisplayClusterCommResult::Ok;
}

EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlEditor::ReleaseBarrier(const FString& BarrierId, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result)
{
	// PIE stub
	return EDisplayClusterCommResult::Ok;
}

EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlEditor::SyncOnBarrier(const FString& BarrierId, const FString& CallerId, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result)
{
	// PIE stub
	return EDisplayClusterCommResult::Ok;
}

EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlEditor::SyncOnBarrierWithData(const FString& BarrierId, const FString& CallerId, const TArray<uint8>& RequestData, TArray<uint8>& OutResponseData, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result)
{
	// PIE stub
	return EDisplayClusterCommResult::Ok;
}

//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProtocolInternalComm
//////////////////////////////////////////////////////////////////////////////////////////////
EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlEditor::GatherServicesHostingInfo(const FNodeServicesHostingInfo& ThisNodeInfo, FClusterServicesHostingInfo& OutHostingInfo)
{
	// PIE stub
	return EDisplayClusterCommResult::Ok;
}

EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlEditor::PostFailureNegotiate(TArray<uint8>& InOutRecoveryData)
{
	// PIE stub
	return EDisplayClusterCommResult::Ok;
}

EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlEditor::RequestNodeDrop(const FString& NodeId, uint8 DropReason)
{
	// PIE stub
	return EDisplayClusterCommResult::Ok;
}


//////////////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterClusterNodeCtrlEditor
//////////////////////////////////////////////////////////////////////////////////////////////
bool FDisplayClusterClusterNodeCtrlEditor::InitializeServers()
{
	UE_LOGF(LogDisplayClusterCluster, Log, "%ls - Initializing servers...", *GetControllerName());

	// Instantiate public external servers                                                             
	ClusterEventsJsonServer   = MakeShared<FDisplayClusterClusterEventsJsonService>  (UE::nDisplay::Network::Configuration::JsonEventsExternalServerName);
	ClusterEventsBinaryServer = MakeShared<FDisplayClusterClusterEventsBinaryService>(UE::nDisplay::Network::Configuration::BinaryEventsExternalServerName);

	return true;
}

bool FDisplayClusterClusterNodeCtrlEditor::StartServers()
{
	UE_LOGF(LogDisplayClusterCluster, Log, "%ls - Starting servers...", *GetControllerName());

	// Get config data
	const UDisplayClusterConfigurationData* ConfigData = GDisplayCluster->GetPrivateConfigMgr()->GetConfig();
	if (!ConfigData)
	{
		UE_LOGF(LogDisplayClusterCluster, Warning, "%ls - Couldn't get configuration data", *GetControllerName());
		return false;
	}

	// Primary node configuration
	const UDisplayClusterConfigurationClusterNode* const PrimaryNodeCfg = ConfigData->GetPrimaryNode();
	if (!PrimaryNodeCfg)
	{
		UE_LOGF(LogDisplayClusterCluster, Warning, "%ls - No P-node configuration was found", *GetControllerName());
		return false;
	}

	// Always use localhost for PIE because there might be some other host specified in the configuration data
	const FString HostForPie("127.0.0.1");
	const FDisplayClusterConfigurationPrimaryNodePorts& Ports = ConfigData->Cluster->PrimaryNode.Ports;

	UE_LOGF(LogDisplayClusterCluster, Log, "%ls - Listening at `%ls` to port_cej=%u, port_ceb=%u",
		*GetControllerName(), *HostForPie, Ports.ClusterEventsJson, Ports.ClusterEventsBinary);

	// Start the servers
	const bool bCEJServerStarted = StartServerWithLogs(ClusterEventsJsonServer,   HostForPie, Ports.ClusterEventsJson);
	const bool bCEBServerStarted = StartServerWithLogs(ClusterEventsBinaryServer, HostForPie, Ports.ClusterEventsBinary);

	return bCEJServerStarted && bCEBServerStarted;
}

void FDisplayClusterClusterNodeCtrlEditor::StopServers()
{
	ClusterEventsJsonServer->Shutdown();
	ClusterEventsBinaryServer->Shutdown();
}

void FDisplayClusterClusterNodeCtrlEditor::ShutdownImpl()
{
	UE_LOGF(LogDisplayClusterCluster, Log, "%ls - Shutting down...", *GetControllerName());

	StopServers();
}
