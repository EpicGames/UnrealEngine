// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Cluster/Controller/DisplayClusterClusterNodeCtrlBase.h"

class FDisplayClusterClusterEventsJsonService;
class FDisplayClusterClusterEventsBinaryService;


/**
 * Node controller for 'Editor' operation mode.
 *
 * This controller is used in PIE only, therefore it has a very limited functionality.
 * So far, the following features are supported:
 *  - JSON and binary events processsing. This is useful for debugging event based logic in PIE.
 *  - JSON and binary events sending outside (inheriated from base controller).
 */
class FDisplayClusterClusterNodeCtrlEditor
	: public FDisplayClusterClusterNodeCtrlBase
{
public:

	FDisplayClusterClusterNodeCtrlEditor();
	virtual ~FDisplayClusterClusterNodeCtrlEditor();

public:

	//~ Begin IDisplayClusterClusterNodeController
	virtual bool Initialize() override;
	virtual void Shutdown() override;
	//~ End IDisplayClusterClusterNodeController

public:

	//~ Begin IDisplayClusterProtocolClusterSync
	virtual EDisplayClusterCommResult WaitForGameStart() override;
	virtual EDisplayClusterCommResult WaitForFrameStart() override;
	virtual EDisplayClusterCommResult WaitForFrameEnd() override;
	virtual EDisplayClusterCommResult GetTimeData(double& OutDeltaTime, double& OutGameTime, TOptional<FQualifiedFrameTime>& OutFrameTime) override;
	virtual EDisplayClusterCommResult GetObjectsData(EDisplayClusterSyncGroup InSyncGroup, TMap<FString, TArray<uint8>>& OutObjectsData) override;
	virtual EDisplayClusterCommResult GetEventsData(TArray<TSharedPtr<FDisplayClusterClusterEventJson>>& OutJsonEvents, TArray<TSharedPtr<FDisplayClusterClusterEventBinary>>& OutBinaryEvents) override;
	virtual EDisplayClusterCommResult GetNativeInputData(TMap<FString, TArray<uint8>>& OutNativeInputData) override;
	virtual EDisplayClusterCommResult PropagateStatesData(const TArray<uint8>& InLocalStatesData, TArray<uint8>& OutClusterStatesData) override;
	//~ End IDisplayClusterProtocolClusterSync

public:

	//~ Begin IDisplayClusterProtocolRenderSync
	virtual EDisplayClusterCommResult SynchronizeOnBarrier() override;
	//~ End IDisplayClusterProtocolRenderSync

public:

	//~ Begin IDisplayClusterProtocolEventsJson
	virtual EDisplayClusterCommResult EmitClusterEventJson(const FDisplayClusterClusterEventJson& Event) override;
	//~ End IDisplayClusterProtocolEventsJson

public:

	//~ Begin IDisplayClusterProtocolEventsBinary
	virtual EDisplayClusterCommResult EmitClusterEventBinary(const FDisplayClusterClusterEventBinary& Event) override;
	//~ End IDisplayClusterProtocolEventsBinary

public:

	//~ Begin IDisplayClusterProtocolGenericBarrier
	virtual EDisplayClusterCommResult CreateBarrier(const FString& BarrierId, const TMap<FString, TSet<FString>>& NodeToSyncCallers, uint32 Timeout, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result) override;
	virtual EDisplayClusterCommResult WaitUntilBarrierIsCreated(const FString& BarrierId, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result) override;
	virtual EDisplayClusterCommResult IsBarrierAvailable(const FString& BarrierId, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result) override;
	virtual EDisplayClusterCommResult ReleaseBarrier(const FString& BarrierId, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result) override;
	virtual EDisplayClusterCommResult SyncOnBarrier(const FString& BarrierId, const FString& CallerId, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result) override;
	virtual EDisplayClusterCommResult SyncOnBarrierWithData(const FString& BarrierId, const FString& CallerId, const TArray<uint8>& RequestData, TArray<uint8>& OutResponseData, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result) override;
	//~ End IDisplayClusterProtocolGenericBarrier

public:

	//~ Begin IDisplayClusterProtocolInternalComm
	virtual EDisplayClusterCommResult GatherServicesHostingInfo(const FNodeServicesHostingInfo& ThisNodeInfo, FClusterServicesHostingInfo& OutHostingInfo) override;
	virtual EDisplayClusterCommResult PostFailureNegotiate(TArray<uint8>& InOutRecoveryData) override;
	virtual EDisplayClusterCommResult RequestNodeDrop(const FString& NodeId, uint8 DropReason) override;
	//~ End IDisplayClusterProtocolInternalComm

private:

	/** Auxiliary function that initializes all the internal servers */
	bool InitializeServers();

	/** Auxiliary function that starts all the internal servers */
	bool StartServers();

	/** Auxiliary function that stops all the internal servers */
	void StopServers();

	/** Non-virtual implementation of 'Shutdown' to prevent any potential issues when called from destrcutor */
	void ShutdownImpl();

private: // servers

	/** JSON events server */
	TSharedPtr<FDisplayClusterClusterEventsJsonService> ClusterEventsJsonServer;

	/** Binary events server */
	TSharedPtr<FDisplayClusterClusterEventsBinaryService> ClusterEventsBinaryServer;
};
