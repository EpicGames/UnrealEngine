// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cluster/Controller/DisplayClusterClusterNodeCtrlMain.h"

#include "Async/Async.h"

#include "Cluster/IPDisplayClusterClusterManager.h"
#include "Cluster/Controller/DisplayClusterCtrlContext.h"
#include "Cluster/Failover/DisplayClusterCommDataCache.h"
#include "Cluster/RequestHandler/DisplayClusterCommRequestHandlerLocal.h"
#include "Config/IPDisplayClusterConfigManager.h"

#include "Misc/AssertionMacros.h"
#include "Misc/DisplayClusterGlobals.h"
#include "Misc/DisplayClusterLog.h"

#include "Network/Configuration/DisplayClusterNetworkConfiguration.h"
#include "Network/Service/ClusterSync/DisplayClusterClusterSyncClient.h"
#include "Network/Service/ClusterSync/DisplayClusterClusterSyncService.h"
#include "Network/Service/ClusterEventsBinary/DisplayClusterClusterEventsBinaryClient.h"
#include "Network/Service/ClusterEventsBinary/DisplayClusterClusterEventsBinaryService.h"
#include "Network/Service/ClusterEventsJson/DisplayClusterClusterEventsJsonClient.h"
#include "Network/Service/ClusterEventsJson/DisplayClusterClusterEventsJsonService.h"
#include "Network/Service/GenericBarrier/DisplayClusterGenericBarrierClient.h"
#include "Network/Service/GenericBarrier/DisplayClusterGenericBarrierService.h"
#include "Network/Service/InternalComm/DisplayClusterInternalCommClient.h"
#include "Network/Service/InternalComm/DisplayClusterInternalCommService.h"
#include "Network/Service/RenderSync/DisplayClusterRenderSyncClient.h"
#include "Network/Service/RenderSync/DisplayClusterRenderSyncService.h"
#include "Network/Listener/DisplayClusterTcpListener.h"
#include "Network/DisplayClusterNetworkTypes.h"

#include "DisplayClusterCallbacks.h"
#include "DisplayClusterConfigurationTypes.h"


FDisplayClusterClusterNodeCtrlMain::FDisplayClusterClusterNodeCtrlMain(const FString& InClusterNodeId)
	: FDisplayClusterClusterNodeCtrlBase(TEXT("CTRL_CLSTR"), InClusterNodeId)
	, TcpListener(MakeShared<FDisplayClusterTcpListener>(true, FString("nDisplay-TCP-listener")))
	, InternalServiceNames {
		UE::nDisplay::Network::Configuration::ClusterSyncServerName,
		UE::nDisplay::Network::Configuration::RenderSyncServerName,
		UE::nDisplay::Network::Configuration::GenericBarrierServerName,
		UE::nDisplay::Network::Configuration::JsonEventsServerName,
		UE::nDisplay::Network::Configuration::BinaryEventsServerName,
		UE::nDisplay::Network::Configuration::InternalCommServerName }
	, ExternalServiceNames {
		UE::nDisplay::Network::Configuration::BinaryEventsExternalServerName,
		UE::nDisplay::Network::Configuration::JsonEventsExternalServerName }
{
}

FDisplayClusterClusterNodeCtrlMain::~FDisplayClusterClusterNodeCtrlMain()
{
	// In case Shutdown() wasn't call before deleting this controller, we have to stop
	// all the clients and servers that this controller owns. We do it safely (non-virtual shutdown).
	ShutdownImpl();
}

bool FDisplayClusterClusterNodeCtrlMain::Initialize()
{
	// Base initialization
	if (!FDisplayClusterClusterNodeCtrlBase::Initialize())
	{
		UE_LOGF(LogDisplayClusterCluster, Error, "%ls - Couldn't initialize base controller internals", *GetControllerName());
		return false;
	}

	// Initialize servers
	if (!InitializeServers())
	{
		UE_LOGF(LogDisplayClusterCluster, Error, "%ls - Couldn't initialize node servers", *GetControllerName());
		return false;
	}

	// Initialize clients
	if (!InitializeClients())
	{
		UE_LOGF(LogDisplayClusterCluster, Error, "%ls - Couldn't initialize node clients", *GetControllerName());
		return false;
	}

	// Config manager
	const IPDisplayClusterConfigManager* const ConfigMgr = GDisplayCluster->GetPrivateConfigMgr();
	if (!ConfigMgr)
	{
		UE_LOGF(LogDisplayClusterCluster, Error, "%ls - Couldn't get configuration manager", *GetControllerName());
		return false;
	}

	// Get config data
	const UDisplayClusterConfigurationData* const ConfigData = ConfigMgr->GetConfig();
	if (!ConfigData)
	{
		UE_LOGF(LogDisplayClusterCluster, Error, "%ls - Couldn't get configuration data", *GetControllerName());
		return false;
	}

	// This node ID
	const FString NodeId = GetNodeId();
	if (NodeId.IsEmpty())
	{
		UE_LOGF(LogDisplayClusterCluster, Error, "%ls - Invalid cluster node ID", *GetControllerName());
		return false;
	}

	// Get configuration of this node
	const UDisplayClusterConfigurationClusterNode* const ConfigNode = ConfigData->GetNode(NodeId);
	if (!ConfigNode)
	{
		UE_LOGF(LogDisplayClusterCluster, Error, "%ls - Couldn't obtain any configuration for node '%ls'", *GetControllerName(), *NodeId);
		return false;
	}

	// Start internal servers (all nodes)
	if (!StartServersInternal(ConfigData, ConfigNode))
	{
		UE_LOGF(LogDisplayClusterCluster, Error, "%ls - An error occurred while starting internal servers", *GetControllerName());
		return false;
	}

	// Start external servers on p-node only
	const bool bIsPrimary = GDisplayCluster->GetPrivateClusterMgr()->IsPrimary();
	if (bIsPrimary)
	{
		if (!StartServersExternal(ConfigData, ConfigNode))
		{
			UE_LOGF(LogDisplayClusterCluster, Error, "%ls - An error occurred while starting external servers", *GetControllerName());
			return false;
		}
	}

	// Connect to the servers (all nodes)
	const bool bClientsStarted = StartClients();
	if (!bClientsStarted)
	{
		UE_LOGF(LogDisplayClusterCluster, Error, "%ls - An error occurred during clients start", *GetControllerName());
		return false;
	}

	// Here we need to synchronize all the nodes on a barrier. This is required to ensure the p-node has started
	// its external servers, and there won't be any race conditions with other nodes running on the same machine.
	// Technically, we can re-use WaitForGameStart barrier here as we'd need the same timeout settings.
	WaitForGameStart();

	// Start external services (non-primary nodes only)
	if (!bIsPrimary)
	{
		// Try to start external services. If primary node is running on the same machine, it has occupied the ext server ports already.
		// So we wouldn't start here properly. In this case, postpone ext servers initialization until this node becomes primary.
		bExternalServersRunning = StartServersExternal(ConfigData, ConfigNode);
		if (!bExternalServersRunning)
		{
			// Stop all external servers in case some of them have not started
			StopServersExternal();
		}
	}

	UE_LOGF(LogDisplayClusterCluster, Log, "%ls - In-cluster connections have been established", *GetControllerName());

	// Subscribe
	SubscribeToEvents();

	return true;
}

void FDisplayClusterClusterNodeCtrlMain::Shutdown()
{
	ShutdownImpl();
	FDisplayClusterClusterNodeCtrlBase::Shutdown();
}

TSet<FName> FDisplayClusterClusterNodeCtrlMain::GetInternalServiceNames() const
{
	return InternalServiceNames;
}

int32 FDisplayClusterClusterNodeCtrlMain::InitializeGeneralPurposeBarrierClients()
{
	FScopeLock Lock(&GPBClientsCS);

	// Instantiate and connet all the clients
	TSharedRef<FNodeGeneralPurposeBarrierClientSet> NewSet = MakeShared<FNodeGeneralPurposeBarrierClientSet>();
	if (!NewSet->Initialize(HostingInfo))
	{
		return INDEX_NONE;
	}

	// Generate unique set ID
	const int32 NewClientSetId = GPBClientSetCounter++;

	// Associate this new set with the ID
	GPBClients.Emplace(NewClientSetId, MoveTemp(NewSet));

	UE_LOGF(LogDisplayClusterCluster, Log, "%ls - Allocated GPB clients set '%d'", *GetControllerName(), NewClientSetId);

	// And return associated ID
	return NewClientSetId;
}

void FDisplayClusterClusterNodeCtrlMain::ReleaseGeneralPurposeBarrierClients(int32 ClientSetId)
{
	if (ClientSetId != INDEX_NONE)
	{
		FScopeLock Lock(&GPBClientsCS);
		UE_LOGF(LogDisplayClusterCluster, Log, "%ls - Releasing GPB clients set '%d'", *GetControllerName(), ClientSetId);
		GPBClients.Remove(ClientSetId);
	}
}

bool FDisplayClusterClusterNodeCtrlMain::DropClusterNode(const FString& NodeId)
{
	// Kill all sessions of the requested node
	for (const FName& ServiceName : InternalServiceNames)
	{
		if (TSharedPtr<FDisplayClusterService> Service = GetService(ServiceName).Pin())
		{
			Service->KillSession(NodeId);
		}
		else
		{
			UE_LOGF(LogDisplayClusterCluster, Warning, "%ls - Service '%ls' is not running, couldn't find the server", *GetControllerName(), *ServiceName.ToString());
		}
	}

	// Release all the clients associated with this node
	{
		FScopeLock Lock(&ClientsCS);
		Clients.Remove(*NodeId);
	}

	return true;
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProtocolClusterSync
//////////////////////////////////////////////////////////////////////////////////////////////
EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlMain::WaitForGameStart()
{
	if (const TSharedPtr<FNodeClientSet>& ClientSet = GetActiveClientSet())
	{
		return ClientSet->ClusterSyncClient->WaitForGameStart();
	}

	return EDisplayClusterCommResult::InternalError;
}

EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlMain::WaitForFrameStart()
{
	if (const TSharedPtr<FNodeClientSet>& ClientSet = GetActiveClientSet())
	{
		return ClientSet->ClusterSyncClient->WaitForFrameStart();
	}

	return EDisplayClusterCommResult::InternalError;
}

EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlMain::WaitForFrameEnd()
{
	if (const TSharedPtr<FNodeClientSet>& ClientSet = GetActiveClientSet())
	{
		return ClientSet->ClusterSyncClient->WaitForFrameEnd();
	}

	return EDisplayClusterCommResult::InternalError;
}

EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlMain::GetTimeData(double& OutDeltaTime, double& OutGameTime, TOptional<FQualifiedFrameTime>& OutFrameTime)
{
	IPDisplayClusterClusterManager* const ClusterMgr = GDisplayCluster->GetPrivateClusterMgr();

	if (ClusterMgr->IsPrimary())
	{
		// Loopback optimization for Primary-to-Primary communication. No need to serialize data
		// and send over network, just pass it to the handler directly.
		return FDisplayClusterCommRequestHandlerLocal::Get().GetTimeData(OutDeltaTime, OutGameTime, OutFrameTime);
	}
	else
	{
		if (const TSharedPtr<FNodeClientSet>& ClientSet = GetActiveClientSet())
		{
			return ClientSet->ClusterSyncClient->GetTimeData(OutDeltaTime, OutGameTime, OutFrameTime);
		}
	}

	return EDisplayClusterCommResult::InternalError;
}

EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlMain::GetObjectsData(EDisplayClusterSyncGroup InSyncGroup, TMap<FString, TArray<uint8>>& OutObjectsData)
{
	IPDisplayClusterClusterManager* const ClusterMgr = GDisplayCluster->GetPrivateClusterMgr();

	if (ClusterMgr->IsPrimary())
	{
		// Loopback optimization for Primary-to-Primary communication. No need to serialize data
		// and send over network, just pass it to the handler directly.
		return FDisplayClusterCommRequestHandlerLocal::Get().GetObjectsData(InSyncGroup, OutObjectsData);
	}
	else
	{
		if (const TSharedPtr<FNodeClientSet>& ClientSet = GetActiveClientSet())
		{
			return ClientSet->ClusterSyncClient->GetObjectsData(InSyncGroup, OutObjectsData);
		}
	}

	return EDisplayClusterCommResult::InternalError;
}

EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlMain::GetEventsData(TArray<TSharedPtr<FDisplayClusterClusterEventJson>>& OutJsonEvents, TArray<TSharedPtr<FDisplayClusterClusterEventBinary>>& OutBinaryEvents)
{
	IPDisplayClusterClusterManager* const ClusterMgr = GDisplayCluster->GetPrivateClusterMgr();

	if (ClusterMgr->IsPrimary())
	{
		// Loopback optimization for Primary-to-Primary communication. No need to serialize data
		// and send over network, just pass it to the handler directly.
		return FDisplayClusterCommRequestHandlerLocal::Get().GetEventsData(OutJsonEvents, OutBinaryEvents);
	}
	else
	{
		if (const TSharedPtr<FNodeClientSet>& ClientSet = GetActiveClientSet())
		{
			return ClientSet->ClusterSyncClient->GetEventsData(OutJsonEvents, OutBinaryEvents);
		}
	}

	return EDisplayClusterCommResult::InternalError;
}

EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlMain::GetNativeInputData(TMap<FString, TArray<uint8>>& OutNativeInputData)
{
	IPDisplayClusterClusterManager* const ClusterMgr = GDisplayCluster->GetPrivateClusterMgr();

	if (ClusterMgr->IsPrimary())
	{
		// Loopback optimization for Primary-to-Primary communication. No need to serialize data
		// and send over network, just pass it to the handler directly.
		return FDisplayClusterCommRequestHandlerLocal::Get().GetNativeInputData(OutNativeInputData);
	}
	else
	{
		if (const TSharedPtr<FNodeClientSet>& ClientSet = GetActiveClientSet())
		{
			return ClientSet->ClusterSyncClient->GetNativeInputData(OutNativeInputData);
		}
	}

	return EDisplayClusterCommResult::InternalError;
}

EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlMain::PropagateStatesData(const TArray<uint8>& InLocalStatesData, TArray<uint8>& OutClusterStatesData)
{
	if (const TSharedPtr<FNodeClientSet>& ClientSet = GetActiveClientSet())
	{
		return ClientSet->ClusterSyncClient->PropagateStatesData(InLocalStatesData, OutClusterStatesData);
	}

	return EDisplayClusterCommResult::InternalError;
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProtocolRenderSync
//////////////////////////////////////////////////////////////////////////////////////////////
EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlMain::SynchronizeOnBarrier()
{
	if (const TSharedPtr<FNodeClientSet>& ClientSet = GetActiveClientSet())
	{
		return ClientSet->RenderSyncClient->SynchronizeOnBarrier();
	}

	return EDisplayClusterCommResult::InternalError;
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProtocolEventsJson
//////////////////////////////////////////////////////////////////////////////////////////////
EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlMain::EmitClusterEventJson(const FDisplayClusterClusterEventJson& Event)
{
	if (const TSharedPtr<FNodeClientSet>& ClientSet = GetActiveClientSet())
	{
		return ClientSet->ClusterEventsJsonClient->EmitClusterEventJson(Event);
	}

	return EDisplayClusterCommResult::InternalError;
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProtocolEventsBinary
//////////////////////////////////////////////////////////////////////////////////////////////
EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlMain::EmitClusterEventBinary(const FDisplayClusterClusterEventBinary& Event)
{
	if (const TSharedPtr<FNodeClientSet>& ClientSet = GetActiveClientSet())
	{
		return ClientSet->ClusterEventsBinaryClient->EmitClusterEventBinary(Event);
	}

	return EDisplayClusterCommResult::InternalError;
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProtocolGenericBarrier
//////////////////////////////////////////////////////////////////////////////////////////////
EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlMain::CreateBarrier(const FString& BarrierId, const TMap<FString, TSet<FString>>& NodeToSyncCallers, uint32 Timeout, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result)
{
	if (TSharedPtr<FDisplayClusterGenericBarrierClient> Client = GetBarrierClientFromContext())
	{
		return Client->CreateBarrier(BarrierId, NodeToSyncCallers, Timeout, Result);
	}

	return EDisplayClusterCommResult::InternalError;
}

EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlMain::WaitUntilBarrierIsCreated(const FString& BarrierId, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result)
{
	if (TSharedPtr<FDisplayClusterGenericBarrierClient> Client = GetBarrierClientFromContext())
	{
		return Client->WaitUntilBarrierIsCreated(BarrierId, Result);
	}

	return EDisplayClusterCommResult::InternalError;
}

EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlMain::IsBarrierAvailable(const FString& BarrierId, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result)
{
	if (TSharedPtr<FDisplayClusterGenericBarrierClient> Client = GetBarrierClientFromContext())
	{
		return Client->IsBarrierAvailable(BarrierId, Result);
	}

	return EDisplayClusterCommResult::InternalError;
}

EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlMain::ReleaseBarrier(const FString& BarrierId, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result)
{
	if (TSharedPtr<FDisplayClusterGenericBarrierClient> Client = GetBarrierClientFromContext())
	{
		return Client->ReleaseBarrier(BarrierId, Result);
	}

	return EDisplayClusterCommResult::InternalError;
}

EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlMain::SyncOnBarrier(const FString& BarrierId, const FString& CallerId, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result)
{
	if (TSharedPtr<FDisplayClusterGenericBarrierClient> Client = GetBarrierClientFromContext())
	{
		return Client->SyncOnBarrier(BarrierId, CallerId, Result);
	}

	return EDisplayClusterCommResult::InternalError;
}

EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlMain::SyncOnBarrierWithData(const FString& BarrierId, const FString& CallerId, const TArray<uint8>& RequestData, TArray<uint8>& OutResponseData, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result)
{
	if (TSharedPtr<FDisplayClusterGenericBarrierClient> Client = GetBarrierClientFromContext())
	{
		return Client->SyncOnBarrierWithData(BarrierId, CallerId, RequestData, OutResponseData, Result);
	}

	return EDisplayClusterCommResult::InternalError;
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProtocolInternalComm
//////////////////////////////////////////////////////////////////////////////////////////////
EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlMain::GatherServicesHostingInfo(const FNodeServicesHostingInfo& ThisNodeInfo, FClusterServicesHostingInfo& OutHostingInfo)
{
	if (const TSharedPtr<FNodeClientSet>& ClientSet = GetActiveClientSet())
	{
		return ClientSet->InternalCommClient->GatherServicesHostingInfo(ThisNodeInfo, OutHostingInfo);
	}

	return EDisplayClusterCommResult::InternalError;
}

EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlMain::PostFailureNegotiate(TArray<uint8>& InOutRecoveryData)
{
	if (const TSharedPtr<FNodeClientSet>& ClientSet = GetActiveClientSet())
	{
		return ClientSet->InternalCommClient->PostFailureNegotiate(InOutRecoveryData);
	}

	return EDisplayClusterCommResult::InternalError;
}

EDisplayClusterCommResult FDisplayClusterClusterNodeCtrlMain::RequestNodeDrop(const FString& NodeId, uint8 DropReason)
{
	if (const TSharedPtr<FNodeClientSet>& ClientSet = GetActiveClientSet())
	{
		return ClientSet->InternalCommClient->RequestNodeDrop(NodeId, DropReason);
	}

	return EDisplayClusterCommResult::InternalError;
}


//////////////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterClusterNodeCtrlMain
//////////////////////////////////////////////////////////////////////////////////////////////
bool FDisplayClusterClusterNodeCtrlMain::InitializeServers()
{
	UE_LOGF(LogDisplayClusterCluster, Log, "%ls - Initializing local servers...", *GetControllerName());

	// CS - ClusterSync
	{
		const FName& ServerName = UE::nDisplay::Network::Configuration::ClusterSyncServerName;
		UE_LOGF(LogDisplayClusterCluster, Log, "%ls - Initializing %ls...", *GetControllerName(), *ServerName.ToString());
		RegisterLocalService(ServerName, MakeShared<FDisplayClusterClusterSyncService>(ServerName));
		check(InternalServiceNames.Contains(ServerName));
	}

	// RS - RenderSync
	{
		const FName& ServerName = UE::nDisplay::Network::Configuration::RenderSyncServerName;
		UE_LOGF(LogDisplayClusterCluster, Log, "%ls - Initializing %ls...", *GetControllerName(), *ServerName.ToString());
		RegisterLocalService(ServerName, MakeShared<FDisplayClusterRenderSyncService>(ServerName));
		check(InternalServiceNames.Contains(ServerName));
	}

	// GB - GenericBarrier
	{
		const FName& ServerName = UE::nDisplay::Network::Configuration::GenericBarrierServerName;
		UE_LOGF(LogDisplayClusterCluster, Log, "%ls - Initializing %ls...", *GetControllerName(), *ServerName.ToString());
		RegisterLocalService(ServerName, MakeShared<FDisplayClusterGenericBarrierService>(ServerName));
		check(InternalServiceNames.Contains(ServerName));
	}

	// CEJ - JSON events
	{
		const FName& ServerName = UE::nDisplay::Network::Configuration::JsonEventsServerName;
		UE_LOGF(LogDisplayClusterCluster, Log, "%ls - Initializing %ls...", *GetControllerName(), *ServerName.ToString());
		RegisterLocalService(ServerName, MakeShared<FDisplayClusterClusterEventsJsonService>(ServerName));
		check(InternalServiceNames.Contains(ServerName));
	}

	// CEB - Binary events
	{
		const FName& ServerName = UE::nDisplay::Network::Configuration::BinaryEventsServerName;
		UE_LOGF(LogDisplayClusterCluster, Log, "%ls - Initializing %ls...", *GetControllerName(), *ServerName.ToString());
		RegisterLocalService(ServerName, MakeShared<FDisplayClusterClusterEventsBinaryService>(ServerName));
		check(InternalServiceNames.Contains(ServerName));
	}

	// IC - InternalComm
	{
		const FName& ServerName = UE::nDisplay::Network::Configuration::InternalCommServerName;
		UE_LOGF(LogDisplayClusterCluster, Log, "%ls - Initializing %ls...", *GetControllerName(), *ServerName.ToString());
		RegisterLocalService(ServerName, MakeShared<FDisplayClusterInternalCommService>(ServerName));
		check(InternalServiceNames.Contains(ServerName));
	}

	// CEJ_Ext - External JSON events
	{
		const FName& ServerName = UE::nDisplay::Network::Configuration::JsonEventsExternalServerName;
		UE_LOGF(LogDisplayClusterCluster, Log, "%ls - Initializing %ls...", *GetControllerName(), *ServerName.ToString());
		RegisterLocalService(ServerName, MakeShared<FDisplayClusterClusterEventsJsonService>(ServerName));
	}

	// CEB_Ext - External Binary events
	{
		const FName& ServerName = UE::nDisplay::Network::Configuration::BinaryEventsExternalServerName;
		UE_LOGF(LogDisplayClusterCluster, Log, "%ls - Initializing %ls...", *GetControllerName(), *ServerName.ToString());
		RegisterLocalService(ServerName, MakeShared<FDisplayClusterClusterEventsBinaryService>(ServerName));
	}

	return true;
}

bool FDisplayClusterClusterNodeCtrlMain::StartServersInternal(const UDisplayClusterConfigurationData* const InConfigData, const UDisplayClusterConfigurationClusterNode* const InConfigNode)
{
	UE_LOGF(LogDisplayClusterCluster, Log, "%ls - Starting internal servers...", *GetControllerName());

	// Connection validation lambda. Only cluster nodes allowed to connect.
	auto IsConnectionAllowedFunc = [](const FDisplayClusterSessionInfo& SessionInfo)
		{
			// Here we make sure the node belongs to the cluster
			TArray<FString> NodeIds;
			GDisplayCluster->GetPrivateClusterMgr()->GetNodeIds(NodeIds);
			return NodeIds.ContainsByPredicate([SessionInfo](const FString& Item)
				{
					return Item.Equals(SessionInfo.NodeId.Get(FString()), ESearchCase::IgnoreCase);
				});
		};

	bool bAllInternalServersStarted = true;

	// Start all internal servers with the same TCP listener.
	for (const FName& ServiceName : InternalServiceNames)
	{
		if (TSharedPtr<FDisplayClusterService> Service = GetService(ServiceName).Pin())
		{
			// Set connection validation for internal sync servers. Only cluster nodes allowed.
			Service->OnIsConnectionAllowed().BindLambda(IsConnectionAllowedFunc);

			// Listen for node failure notifications
			Service->OnNodeFailed().AddRaw(this, &FDisplayClusterClusterNodeCtrlMain::HandleNodeFailed);

			// Start server
			bAllInternalServersStarted &= StartServerWithLogs(Service, TcpListener);
		}
		else
		{
			UE_LOGF(LogDisplayClusterCluster, Warning, "%ls - Couldn't find server '%ls'", *GetControllerName(), *ServiceName.ToString());
			bAllInternalServersStarted = false;
		}
	}

	// Start listening for incoming connections.
	//  - P-node uses the port number specified in the configuration data
	//  - All other nodes start listening any available port
	const bool bIsPrimaryNode = GDisplayCluster->GetPrivateClusterMgr()->IsPrimary();
	const FDisplayClusterConfigurationPrimaryNodePorts& PrimaryPorts = InConfigData->Cluster->PrimaryNode.Ports;
	const uint16 RequestedPortNum = (bIsPrimaryNode ? PrimaryPorts.ClusterSync : 0);

	// Finally, start listening for incoming connections
	const bool bConnectionListenerStarted = TcpListener->StartListening(InConfigNode->Host, RequestedPortNum);
	if (!bConnectionListenerStarted)
	{
		UE_LOGF(LogDisplayClusterCluster, Error, "%ls - Internal TCP listener was not able to start at [%ls:%u]", *GetControllerName(), *InConfigNode->Host, RequestedPortNum);
	}

	return bConnectionListenerStarted;
}

bool FDisplayClusterClusterNodeCtrlMain::StartServersExternal()
{
	if (const UDisplayClusterConfigurationData* const ConfigData = GDisplayCluster->GetPrivateConfigMgr()->GetConfig())
	{
		if (const UDisplayClusterConfigurationClusterNode* const ConfigNode = ConfigData->GetNode(GetNodeId()))
		{
			return StartServersExternal(ConfigData, ConfigNode);
		}
	}

	return false;
}

bool FDisplayClusterClusterNodeCtrlMain::StartServersExternal(const UDisplayClusterConfigurationData* const InConfigData, const UDisplayClusterConfigurationClusterNode* const InConfigNode)
{
	UE_LOGF(LogDisplayClusterCluster, Log, "%ls - Starting external servers...", *GetControllerName());

	bool bAllExternalServersStarted = true;

	// CEJ_Ext
	if (TSharedPtr<FDisplayClusterService> Service = GetService(UE::nDisplay::Network::Configuration::JsonEventsExternalServerName).Pin())
	{
		const uint16 PortNum = InConfigData->Cluster->PrimaryNode.Ports.ClusterEventsJson;
		UE_LOGF(LogDisplayClusterCluster, Log, "%ls - Public JSON events server starting at [%ls:%u]", *GetControllerName(), *InConfigNode->Host, PortNum);
		bAllExternalServersStarted &= StartServerWithLogs(Service, *InConfigNode->Host, PortNum);
	}

	// CEB_Ext
	if (TSharedPtr<FDisplayClusterService> Service = GetService(UE::nDisplay::Network::Configuration::BinaryEventsExternalServerName).Pin())
	{
		const uint16 PortNum = InConfigData->Cluster->PrimaryNode.Ports.ClusterEventsBinary;
		UE_LOGF(LogDisplayClusterCluster, Log, "%ls - Public binary events server starting at [%ls:%u]", *GetControllerName(), *InConfigNode->Host, PortNum);
		bAllExternalServersStarted &= StartServerWithLogs(Service, *InConfigNode->Host, PortNum);
	}

	return bAllExternalServersStarted;
}

void FDisplayClusterClusterNodeCtrlMain::RunBackgroundServersExternalStartTask()
{
	// Should we start extrenal servers?
	if (bExternalServersRunning)
	{
		return;
	}

	// Start a background task that would try to restart external servers every N seconds until succeeded
	AsyncTask(ENamedThreads::AnyBackgroundThreadNormalTask, [this, NodeCtrlWeakPtr = AsWeak()]()
		{
			while (NodeCtrlWeakPtr.IsValid() && !IsEngineExitRequested())
			{
				if (TSharedPtr<FDisplayClusterClusterNodeCtrlMain> NodeCtrl = NodeCtrlWeakPtr.Pin())
				{
					bExternalServersRunning = NodeCtrl->StartServersExternal();
					if (!bExternalServersRunning)
					{
						// Stop servers, and retry later

						StopServersExternal();

						// Delay before next attempt
						constexpr int32 RetryDelay = 5;
						UE_LOGF(LogDisplayClusterCluster, Warning, "%ls - (Delayed re-start) Couldn't start external servers. Retry in %d seconds.", *GetControllerName(), RetryDelay);

						FPlatformProcess::SleepNoStats(RetryDelay);
					}
					else
					{
						// Leave the cycle, and stop finish this task
						UE_LOGF(LogDisplayClusterCluster, Log, "%ls - (Delayed re-start) External servers started.", *GetControllerName());
						break;
					}
				}
			}
		});
}

void FDisplayClusterClusterNodeCtrlMain::StopServers()
{
	StopServersExternal();
	StopServersInternal();
}

void FDisplayClusterClusterNodeCtrlMain::StopServersInternal()
{
	UE_LOGF(LogDisplayClusterCluster, Log, "%ls - Stopping internal servers...", *GetControllerName());

	// Stop listening for incoming connections
	TcpListener->StopListening(true);

	StopServersImpl(InternalServiceNames);
}

void FDisplayClusterClusterNodeCtrlMain::StopServersExternal()
{
	UE_LOGF(LogDisplayClusterCluster, Log, "%ls - Stopping external servers...", *GetControllerName());

	StopServersImpl(ExternalServiceNames);
}

void FDisplayClusterClusterNodeCtrlMain::StopServersImpl(const TSet<FName>& ServiceNames)
{
	// Stop all internal services
	const TMap<FName, TSharedPtr<FDisplayClusterService>> Services = GetRegisteredServices();
	for (const TPair<FName, TSharedPtr<FDisplayClusterService>>& Service : Services)
	{
		if (ServiceNames.Contains(Service.Key) && Service.Value)
		{
			Service.Value->Shutdown();
		}
	}
}

bool FDisplayClusterClusterNodeCtrlMain::InitializeClients()
{
	UE_LOGF(LogDisplayClusterCluster, Log, "%ls - Initializing internal clients...", *GetControllerName());

	// Cluster manager
	const IPDisplayClusterClusterManager* const ClusterMgr = GDisplayCluster->GetPrivateClusterMgr();
	if (!ClusterMgr)
	{
		UE_LOGF(LogDisplayClusterCluster, Error, "%ls - Couldn't access cluster manager", *GetControllerName());
		return false;
	}

	// Get node IDs of the cluster
	TSet<FString> NodeIds;
	ClusterMgr->GetNodeIds(NodeIds);
	if (NodeIds.IsEmpty())
	{
		UE_LOGF(LogDisplayClusterCluster, Error, "%ls - No cluster nodes found", *GetControllerName());
		return false;
	}

	// Instantiate clients
	{
		FScopeLock Lock(&ClientsCS);

		// Reset existing clients if there are any
		Clients.Empty(NodeIds.Num());

		// For each node, create its own set of clients
		for (const FString& NodeId : NodeIds)
		{
			// Create new client set
			TSharedRef<FNodeClientSet>& NodeClients = Clients.Emplace(*NodeId, MakeShared<FNodeClientSet>());

			FName ClientName;

			// CS
			ClientName = *FString::Printf(TEXT("%s[%s]"), *UE::nDisplay::Network::Configuration::ClusterSyncClientName.ToString(), *NodeId);
			NodeClients->ClusterSyncClient = MakeUnique<FDisplayClusterClusterSyncClient>(ClientName);

			// RS
			ClientName = *FString::Printf(TEXT("%s[%s]"), *UE::nDisplay::Network::Configuration::RenderSyncClientName.ToString(), *NodeId);
			NodeClients->RenderSyncClient = MakeUnique<FDisplayClusterRenderSyncClient>(ClientName);

			// CEJ
			ClientName = *FString::Printf(TEXT("%s[%s]"), *UE::nDisplay::Network::Configuration::JsonEventsClientName.ToString(), *NodeId);
			NodeClients->ClusterEventsJsonClient = MakeUnique<FDisplayClusterClusterEventsJsonClient>(ClientName);

			// CEB
			ClientName = *FString::Printf(TEXT("%s[%s]"), *UE::nDisplay::Network::Configuration::BinaryEventsClientName.ToString(), *NodeId);
			NodeClients->ClusterEventsBinaryClient = MakeUnique<FDisplayClusterClusterEventsBinaryClient>(ClientName);

			// IC
			ClientName = *FString::Printf(TEXT("%s[%s]"), *UE::nDisplay::Network::Configuration::InternalCommClientName.ToString(), *NodeId);
			NodeClients->InternalCommClient = MakeUnique<FDisplayClusterInternalCommClient>(ClientName);
		}
	}

	return true;
}

bool FDisplayClusterClusterNodeCtrlMain::StartClients()
{
	UE_LOGF(LogDisplayClusterCluster, Log, "%ls - Starting internal clients...", *GetControllerName());

	// Config manager
	const IPDisplayClusterConfigManager* const ConfigMgr = GDisplayCluster->GetPrivateConfigMgr();
	if (!ConfigMgr)
	{
		UE_LOGF(LogDisplayClusterCluster, Error, "%ls - Couldn't get configuration manager", *GetControllerName());
		return false;
	}

	// Get config data
	const UDisplayClusterConfigurationData* const ConfigData = ConfigMgr->GetConfig();
	if (!ConfigData || !ConfigData->Cluster)
	{
		UE_LOGF(LogDisplayClusterCluster, Error, "%ls - Couldn't get configuration data", *GetControllerName());
		return false;
	}

	// Connect to the primary node. We need this to be done first because the P-node
	// will then provide us with all the necessary information on how to connect to other nodes
	if (!StartClientsConnectPrimary(ConfigData))
	{
		UE_LOGF(LogDisplayClusterCluster, Error, "%ls - Couldn't connect to the primary node", *GetControllerName());
		return false;
	}

	// Now connect to the remaining nodes
	if (!StartClientsConnectRemaining(ConfigData))
	{
		UE_LOGF(LogDisplayClusterCluster, Error, "%ls - Couldn't connect to the remaining nodes", *GetControllerName());
		return false;
	}

	return true;
}

bool FDisplayClusterClusterNodeCtrlMain::StartClientsConnectPrimary(const UDisplayClusterConfigurationData* const InConfigData)
{
	check(InConfigData);

	const FDisplayClusterConfigurationPrimaryNode& InfoPrimaryNode = InConfigData->Cluster->PrimaryNode;

	FScopeLock Lock(&ClientsCS);

	// Find a set of P-node clients
	TSharedRef<FNodeClientSet>* PrimaryNodeClients = Clients.Find(*InfoPrimaryNode.Id);
	if (!PrimaryNodeClients)
	{
		UE_LOGF(LogDisplayClusterCluster, Error, "%ls - Couldn't find a set of '%ls' P-node clients", *GetControllerName(), *InfoPrimaryNode.Id);
		return false;
	}

	// Find a P-node configuration data
	UDisplayClusterConfigurationClusterNode* CfgPrimaryNode = InConfigData->GetNode(InfoPrimaryNode.Id);
	if (!IsValid(CfgPrimaryNode))
	{
		UE_LOGF(LogDisplayClusterCluster, Error, "%ls - Couldn't find a configuration data of '%ls' P-node", *GetControllerName(), *InfoPrimaryNode.Id);
		return false;
	}

	// Network settings
	const FDisplayClusterConfigurationNetworkSettings& NetCfg = InConfigData->Cluster->Network;

	bool bAllClientsConnected = true;

	// Connect all the clients
	bAllClientsConnected &= StartClientWithLogs((**PrimaryNodeClients).ClusterSyncClient.Get(),         CfgPrimaryNode->Host, InfoPrimaryNode.Ports.ClusterSync, NetCfg.ConnectRetriesAmount, NetCfg.ConnectRetryDelay);
	bAllClientsConnected &= StartClientWithLogs((**PrimaryNodeClients).RenderSyncClient.Get(),          CfgPrimaryNode->Host, InfoPrimaryNode.Ports.ClusterSync, NetCfg.ConnectRetriesAmount, NetCfg.ConnectRetryDelay);
	bAllClientsConnected &= StartClientWithLogs((**PrimaryNodeClients).ClusterEventsJsonClient.Get(),   CfgPrimaryNode->Host, InfoPrimaryNode.Ports.ClusterSync, NetCfg.ConnectRetriesAmount, NetCfg.ConnectRetryDelay);
	bAllClientsConnected &= StartClientWithLogs((**PrimaryNodeClients).ClusterEventsBinaryClient.Get(), CfgPrimaryNode->Host, InfoPrimaryNode.Ports.ClusterSync, NetCfg.ConnectRetriesAmount, NetCfg.ConnectRetryDelay);
	bAllClientsConnected &= StartClientWithLogs((**PrimaryNodeClients).InternalCommClient.Get(),        CfgPrimaryNode->Host, InfoPrimaryNode.Ports.ClusterSync, NetCfg.ConnectRetriesAmount, NetCfg.ConnectRetryDelay);

	return bAllClientsConnected;
}

bool FDisplayClusterClusterNodeCtrlMain::StartClientsConnectRemaining(const UDisplayClusterConfigurationData* const InConfigData)
{
	check(InConfigData);

	// Primary node Id
	const FName PrimaryNodeId = *InConfigData->Cluster->PrimaryNode.Id;
	check(Clients.Contains(PrimaryNodeId));

	FScopeLock Lock(&ClientsCS);

	// P-node clients set
	TSharedRef<FNodeClientSet>* FoundPrimaryClnSet = Clients.Find(PrimaryNodeId);
	if (!FoundPrimaryClnSet)
	{
		UE_LOGF(LogDisplayClusterCluster, Error, "%ls - Couldn't obtain '%ls' P-node client set", *GetControllerName(), *InConfigData->Cluster->PrimaryNode.Id);
		return false;
	}

	// Prepare info for this node
	IDisplayClusterProtocolInternalComm::FNodeServicesHostingInfo ThisNodeInfo;
	FillThisNodeHostingInfo(ThisNodeInfo);

	// Get whole cluster services hosting info. At this point, cluster initialization remains unfinished
	// as we are not yet connected to the non-primary nodes. That's exactly what we're doing here. That is
	// the reason why we call GatherServicesHostingInfo() directly via node controller, and not via
	// NetAPI facade. Once all connections are established, NetAPI is the only place for all networking requests.
	const EDisplayClusterCommResult CommResult = (*FoundPrimaryClnSet)->InternalCommClient->GatherServicesHostingInfo(ThisNodeInfo, HostingInfo);
	if (CommResult != EDisplayClusterCommResult::Ok)
	{
		UE_LOGF(LogDisplayClusterCluster, Error, "%ls - Couldn't gather any service hosting info", *GetControllerName());
		return false;
	}

	// Log connection information
	{
		UE_LOGF(LogDisplayClusterCluster, Log, "%ls - Exported connection info: port_cs=%u, port_eb=%u, port_ej=%u",
			*GetControllerName(), ThisNodeInfo.ClusterSyncPort, ThisNodeInfo.BinaryEventsPort, ThisNodeInfo.JsonEventsPort);

		for (const TPair<FName, IDisplayClusterProtocolInternalComm::FNodeServicesHostingInfo>& NodeInfo : HostingInfo.ClusterHostingInfo)
		{
			UE_LOGF(LogDisplayClusterCluster, Log, "%ls - Imported connection info: node=%ls port_cs=%u, port_eb=%u, port_ej=%u",
				*GetControllerName(), *NodeInfo.Key.ToString(), NodeInfo.Value.ClusterSyncPort, NodeInfo.Value.BinaryEventsPort, NodeInfo.Value.JsonEventsPort);
		}
	}

	check(Clients.Num() == HostingInfo.ClusterHostingInfo.Num());

	bool bAllClientsConnected = true;

	// Establish connections with all non-primary nodes
	for (const TPair<FName, TSharedRef<FNodeClientSet>>& NodeClients : Clients)
	{
		// Skip P-node as we have connected to it already
		if (NodeClients.Key == PrimaryNodeId)
		{
			continue;
		}

		// Services info of the cluster node
		IDisplayClusterProtocolInternalComm::FNodeServicesHostingInfo* NodeServices = HostingInfo.ClusterHostingInfo.Find(NodeClients.Key);
		if (!NodeServices)
		{
			UE_LOGF(LogDisplayClusterCluster, Error, "%ls - The cluster hosting information doesn't contain node '%ls'", *GetControllerName(), *NodeClients.Key.ToString());
			return false;
		}

		const TSharedRef<FNodeClientSet>& ConnectingToClnSet = NodeClients.Value;
		const FString ConnectingToNodeId = NodeClients.Key.ToString();

		// Get node configuration
		const UDisplayClusterConfigurationClusterNode* const ConnectingToNodeConfig = InConfigData->GetNode(ConnectingToNodeId);
		if (!ConnectingToNodeConfig)
		{
			UE_LOGF(LogDisplayClusterCluster, Error, "%ls - Couldn't obtain configuration data for node '%ls'", *GetControllerName(), *ConnectingToNodeId);
			return false;
		}

		const FDisplayClusterConfigurationNetworkSettings& NetCfg = InConfigData->Cluster->Network;
		const FString& ConnectingToHost = ConnectingToNodeConfig->Host;
		const uint16   ConnectingToPort = NodeServices->ClusterSyncPort;

		check(ConnectingToPort > 0);

		// Start clients
		bAllClientsConnected &= StartClientWithLogs(ConnectingToClnSet->ClusterSyncClient.Get(),         ConnectingToHost, ConnectingToPort, NetCfg.ConnectRetriesAmount, NetCfg.ConnectRetryDelay);
		bAllClientsConnected &= StartClientWithLogs(ConnectingToClnSet->RenderSyncClient.Get(),          ConnectingToHost, ConnectingToPort, NetCfg.ConnectRetriesAmount, NetCfg.ConnectRetryDelay);
		bAllClientsConnected &= StartClientWithLogs(ConnectingToClnSet->ClusterEventsJsonClient.Get(),   ConnectingToHost, ConnectingToPort, NetCfg.ConnectRetriesAmount, NetCfg.ConnectRetryDelay);
		bAllClientsConnected &= StartClientWithLogs(ConnectingToClnSet->ClusterEventsBinaryClient.Get(), ConnectingToHost, ConnectingToPort, NetCfg.ConnectRetriesAmount, NetCfg.ConnectRetryDelay);
		bAllClientsConnected &= StartClientWithLogs(ConnectingToClnSet->InternalCommClient.Get(),        ConnectingToHost, ConnectingToPort, NetCfg.ConnectRetriesAmount, NetCfg.ConnectRetryDelay);
	}

	return bAllClientsConnected;
}

void FDisplayClusterClusterNodeCtrlMain::StopClients()
{
	UE_LOGF(LogDisplayClusterCluster, Log, "%ls - Stopping internal clients...", *GetControllerName());

	FScopeLock Lock(&ClientsCS);

	// Disconnect internal clients
	for (TPair<FName, TSharedRef<FNodeClientSet>>& NodeClient : Clients)
	{
		TSharedRef<FNodeClientSet>& ClnSet = NodeClient.Value;

		ClnSet->ClusterEventsJsonClient->Disconnect();
		ClnSet->ClusterEventsBinaryClient->Disconnect();
		ClnSet->ClusterSyncClient->Disconnect();
		ClnSet->RenderSyncClient->Disconnect();
		ClnSet->InternalCommClient->Disconnect();
	}

	// Disconnect barrier clients
	for (TPair<int32, TSharedRef<FNodeGeneralPurposeBarrierClientSet>>& ClientSet : GPBClients)
	{
		for (TPair<FName, TSharedRef<FDisplayClusterGenericBarrierClient>>& NodeClient : ClientSet.Value->Clients)
		{
			NodeClient.Value->Disconnect();
		}
	}
}

void FDisplayClusterClusterNodeCtrlMain::SubscribeToEvents()
{
	GDisplayCluster->GetCallbacks().OnDisplayClusterFailoverPrimaryNodeChanged().AddRaw(this, &FDisplayClusterClusterNodeCtrlMain::HandlePrimaryNodeChanged);
}

void FDisplayClusterClusterNodeCtrlMain::UnsubscribeFromEvents()
{
	GDisplayCluster->GetCallbacks().OnDisplayClusterFailoverPrimaryNodeChanged().RemoveAll(this);
}

void FDisplayClusterClusterNodeCtrlMain::FillThisNodeHostingInfo(FNodeServicesHostingInfo& OutHostingInfo)
{
	// Internal services are tied to the same TCP listener
	OutHostingInfo.ClusterSyncPort = TcpListener->GetListeningPort();

	// External JSON events
	{
		TSharedPtr<FDisplayClusterService> Service = GetService(UE::nDisplay::Network::Configuration::JsonEventsExternalServerName).Pin();
		OutHostingInfo.JsonEventsPort = (Service ? Service->GetPort() : 0);
	}

	// External binary events
	{
		TSharedPtr<FDisplayClusterService> Service = GetService(UE::nDisplay::Network::Configuration::BinaryEventsExternalServerName).Pin();
		OutHostingInfo.BinaryEventsPort = (Service ? Service->GetPort() : 0);
	}
}

void FDisplayClusterClusterNodeCtrlMain::ShutdownImpl()
{
	UE_LOGF(LogDisplayClusterCluster, Log, "%ls - Shutting down...", *GetControllerName());

	bIsTerminating = true;

	UnsubscribeFromEvents();

	StopClients();
	StopServers();
}

void FDisplayClusterClusterNodeCtrlMain::HandleNodeFailed(const FDisplayClusterServiceFailureEvent& FailureInfo)
{
	if (FailureInfo.NodeFailed.IsSet() && !IsEngineExitRequested() && !bIsTerminating)
	{
		const FString NodeFailed = FailureInfo.NodeFailed.Get(FString());
		GDisplayCluster->GetPrivateClusterMgr()->DropNode(NodeFailed, IPDisplayClusterClusterManager::ENodeDropReason::Failed);
	}
}

void FDisplayClusterClusterNodeCtrlMain::HandlePrimaryNodeChanged(const FString& NewPrimaryId)
{
	// Did this node just become primary?
	if (NewPrimaryId.Equals(GetNodeId(), ESearchCase::IgnoreCase))
	{
		// Start external servers in background if not started yet
		RunBackgroundServersExternalStartTask();
	}
}

const TSharedPtr<FDisplayClusterClusterNodeCtrlMain::FNodeClientSet> FDisplayClusterClusterNodeCtrlMain::GetActiveClientSet()
{
	const FName TargetNodeId = FDisplayClusterCtrlContext::Get().TargetNodeId.Get(NAME_None);

	if(TargetNodeId != NAME_None)
	{
		FScopeLock Lock(&ClientsCS);

		if (TSharedRef<FNodeClientSet>* FoundClientSet = Clients.Find(TargetNodeId))
		{
			return *FoundClientSet;
		}
	}

	return nullptr;
}

const TSharedPtr<FDisplayClusterGenericBarrierClient> FDisplayClusterClusterNodeCtrlMain::GetBarrierClientFromContext()
{
	TSharedPtr<FDisplayClusterGenericBarrierClient> OutcomeClient;

	// TLS context
	const FDisplayClusterCtrlContext& RequestContext = FDisplayClusterCtrlContext::Get();

	// Client set ID is provided by the actual barrier user
	const int32 ClientSetId = RequestContext.GPBClientId.Get(INDEX_NONE);
	if (ClientSetId != INDEX_NONE)
	{
		// Target node ID is set by the transaction/failover controller
		const FName TargetNodeId = RequestContext.TargetNodeId.Get(NAME_None);
		if(TargetNodeId != NAME_None)
		{
			FScopeLock Lock(&GPBClientsCS);

			// Find set by the set ID
			if (TSharedRef<FNodeGeneralPurposeBarrierClientSet>* ClientSet = GPBClients.Find(ClientSetId))
			{
				// Find client by the node ID
				if (TSharedRef<FDisplayClusterGenericBarrierClient>* Client = (*ClientSet)->Clients.Find(TargetNodeId))
				{
					OutcomeClient = Client->ToSharedPtr();
				}
			}
		}
	}

	return OutcomeClient;
}

bool FDisplayClusterClusterNodeCtrlMain::FNodeGeneralPurposeBarrierClientSet::Initialize(const IDisplayClusterProtocolInternalComm::FClusterServicesHostingInfo& InHostingInfo)
{
	// Get all active nodes
	TSet<FString> ActiveNodes;
	if (IPDisplayClusterClusterManager* const ClusterMgr = GDisplayCluster->GetPrivateClusterMgr())
	{
		ClusterMgr->GetNodeIds(ActiveNodes);

		// For each node, we need to establish a new connection
		if (const IPDisplayClusterConfigManager* const ConfigMgr = GDisplayCluster->GetPrivateConfigMgr())
		{
			if (const UDisplayClusterConfigurationData* const ConfigData = ConfigMgr->GetConfig())
			{
				for (const FString& NodeId : ActiveNodes)
				{
					// Find this node configuration
					const UDisplayClusterConfigurationClusterNode* const NodeCfg = ConfigData->GetNode(NodeId);
					if (!IsValid(NodeCfg))
					{
						UE_LOGF(LogDisplayClusterCluster, Warning, "Couldn't initialize GPB client for node '%ls'. No corresponding config data found.", *NodeId);
						return false;
					}

					// Instantiate new client
					const FString ClientName = *FString::Printf(TEXT("%s[%s]"), *UE::nDisplay::Network::Configuration::GenericBarrierClientName.ToString(), *NodeId);
					TSharedRef<FDisplayClusterGenericBarrierClient>& NewClient = Clients.Emplace(*NodeId, MakeShared<FDisplayClusterGenericBarrierClient>(*ClientName));

					// Find hosting info of this node
					const IDisplayClusterProtocolInternalComm::FNodeServicesHostingInfo* NodeInfo = InHostingInfo.ClusterHostingInfo.Find(*NodeId);
					if (!NodeInfo)
					{
						UE_LOGF(LogDisplayClusterCluster, Warning, "Couldn't initialize GPB client for node '%ls'. No corresponding config data found.", *NodeId);
						return false;
					}

					// Establish connection
					if (!NewClient->Connect(NodeCfg->Host, NodeInfo->ClusterSyncPort))
					{
						UE_LOGF(LogDisplayClusterCluster, Warning, "Couldn't establish connection for GPB client '%ls'.", *NewClient->GetName());
						ClusterMgr->DropNode(NodeId, IPDisplayClusterClusterManager::ENodeDropReason::Failed);
						continue;
					}
				}
			}
		}
	}

	return true;
}
