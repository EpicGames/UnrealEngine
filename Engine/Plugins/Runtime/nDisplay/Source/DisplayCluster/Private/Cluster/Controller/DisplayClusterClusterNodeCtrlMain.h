// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Cluster/Controller/DisplayClusterClusterNodeCtrlBase.h"
#include "Network/Service/DisplayClusterService.h"

class FDisplayClusterClusterEventsBinaryClient;
class FDisplayClusterClusterEventsJsonClient;
class FDisplayClusterClusterSyncClient;
class FDisplayClusterGenericBarrierClient;
class FDisplayClusterInternalCommClient;
class FDisplayClusterRenderSyncClient;
class FDisplayClusterTcpListener;
class UDisplayClusterConfigurationClusterNode;
class UDisplayClusterConfigurationData;


/**
 * Node controller for 'Cluster' operation mode.
 *
 * Provides full set of features required in 'Cluster' operation mode.
 */
class FDisplayClusterClusterNodeCtrlMain
	: public FDisplayClusterClusterNodeCtrlBase
	, public TSharedFromThis<FDisplayClusterClusterNodeCtrlMain>
{
public:

	FDisplayClusterClusterNodeCtrlMain(const FString& InClusterNodeId);
	virtual ~FDisplayClusterClusterNodeCtrlMain();

public:

	//~ Begin IDisplayClusterClusterNodeController
	virtual bool Initialize() override;
	virtual void Shutdown() override;
	virtual TSet<FName> GetInternalServiceNames() const override;
	int32 InitializeGeneralPurposeBarrierClients() override;
	virtual void ReleaseGeneralPurposeBarrierClients(int32 ClientSetId) override;
	virtual bool DropClusterNode(const FString& NodeId) override;
	//~ End IDisplayClusterClusterNodeController

public:

	//~ Begin IDisplayClusterProtocolClusterSync
	virtual EDisplayClusterCommResult WaitForGameStart() override;
	virtual EDisplayClusterCommResult WaitForFrameStart() override;
	virtual EDisplayClusterCommResult WaitForFrameEnd() override;
	virtual EDisplayClusterCommResult GetTimeData(double& OutDeltaTime, double& OutGameTime, TOptional<FQualifiedFrameTime>& OutFrameTime) override;
	virtual EDisplayClusterCommResult GetObjectsData(EDisplayClusterSyncGroup InSyncGroup, TMap<FString, TArray<uint8>>& OutObjectsData) override;
	virtual EDisplayClusterCommResult GetEventsData(TArray<TSharedPtr<FDisplayClusterClusterEventJson>>& OutJsonEvents, TArray<TSharedPtr<FDisplayClusterClusterEventBinary>>& OutBinaryEvents) override;
	virtual EDisplayClusterCommResult GetNativeInputData(TMap<FString, TArray<uint8>>& OutNativeInputData) override;
	virtual EDisplayClusterCommResult PropagateStatesData(const TArray<uint8>& InLocalStatesData, TArray<uint8>& OutClusterStatesData) override;
	//~ End IDisplayClusterProtocolClusterSync

public:

	//~ Begin IDisplayClusterProtocolRenderSync
	virtual EDisplayClusterCommResult SynchronizeOnBarrier() override;
	//~ End IDisplayClusterProtocolRenderSync

public:

	//~ Begin IDisplayClusterProtocolEventsJson
	virtual EDisplayClusterCommResult EmitClusterEventJson(const FDisplayClusterClusterEventJson& Event) override;
	//~ End IDisplayClusterProtocolEventsJson

public:

	//~ Begin IDisplayClusterProtocolEventsBinary
	virtual EDisplayClusterCommResult EmitClusterEventBinary(const FDisplayClusterClusterEventBinary& Event) override;
	//~ End IDisplayClusterProtocolEventsBinary

public:

	//~ Begin IDisplayClusterProtocolGenericBarrier
	virtual EDisplayClusterCommResult CreateBarrier(const FString& BarrierId, const TMap<FString, TSet<FString>>& NodeToSyncCallers, uint32 Timeout, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result) override;
	virtual EDisplayClusterCommResult WaitUntilBarrierIsCreated(const FString& BarrierId, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result) override;
	virtual EDisplayClusterCommResult IsBarrierAvailable(const FString& BarrierId, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result) override;
	virtual EDisplayClusterCommResult ReleaseBarrier(const FString& BarrierId, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result) override;
	virtual EDisplayClusterCommResult SyncOnBarrier(const FString& BarrierId, const FString& CallerId, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result) override;
	virtual EDisplayClusterCommResult SyncOnBarrierWithData(const FString& BarrierId, const FString& CallerId, const TArray<uint8>& RequestData, TArray<uint8>& OutResponseData, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result) override;
	//~ End IDisplayClusterProtocolGenericBarrier

public:

	//~ Begin IDisplayClusterProtocolInternalComm
	virtual EDisplayClusterCommResult GatherServicesHostingInfo(const FNodeServicesHostingInfo& ThisNodeInfo, FClusterServicesHostingInfo& OutHostingInfo) override;
	virtual EDisplayClusterCommResult PostFailureNegotiate(TArray<uint8>& InOutRecoveryData) override;
	virtual EDisplayClusterCommResult RequestNodeDrop(const FString& NodeId, uint8 DropReason) override;
	//~ End IDisplayClusterProtocolInternalComm

private:

	/** Initialize node servers */
	bool InitializeServers();

	/** Start internal servers */
	bool StartServersInternal(const UDisplayClusterConfigurationData* const InConfigData, const UDisplayClusterConfigurationClusterNode* const InConfigNode);

	/** Start external servers (no params) */
	bool StartServersExternal();

	/** Start external servers */
	bool StartServersExternal(const UDisplayClusterConfigurationData* const InConfigData, const UDisplayClusterConfigurationClusterNode* const InConfigNode);

	/** Start background async task that periodically tries to start the external servers */
	void RunBackgroundServersExternalStartTask();

	/** Stop internal servers */
	void StopServers();

	/** Stop internal servers */
	void StopServersInternal();

	/** Stop external servers */
	void StopServersExternal();

	/** Stop servers */
	void StopServersImpl(const TSet<FName>& ServiceNames);

	/** Initialize internal clients */
	bool InitializeClients();

	/** Initialize and connect internal clients (enty point for the client start process) */
	bool StartClients();

	/** Connect internal clients to the primary node only */
	bool StartClientsConnectPrimary(const UDisplayClusterConfigurationData* const ConfigData);

	/** Connect internal clients to the remaining nodes */
	bool StartClientsConnectRemaining(const UDisplayClusterConfigurationData* const ConfigData);

	/** Stop internal clients */
	void StopClients();

private:

	// Forward declaration
	struct FNodeClientSet;

	/** Subscribe to external events */
	void SubscribeToEvents();

	/** Unsubscribe from external events */
	void UnsubscribeFromEvents();

	/** Outputs services hosting info of this cluster node */
	void FillThisNodeHostingInfo(FNodeServicesHostingInfo& OutHostingInfo);

	/** Non-virtual implementation of 'shutdown' to prevent any potential issues when called from virtual destructor */
	void ShutdownImpl();

	/** Handle node failure callbacks */
	void HandleNodeFailed(const FDisplayClusterServiceFailureEvent& FailureInfo);

	/** Handles primary node change events */
	void HandlePrimaryNodeChanged(const FString& NewPrimaryId);

	/** Returns a set of clients that is currently active. */
	const TSharedPtr<FNodeClientSet> GetActiveClientSet();

	/** Returns GPB client requested for a transaction. */
	const TSharedPtr<FDisplayClusterGenericBarrierClient> GetBarrierClientFromContext();

private: // clients

	/**
	 * Auxiliary structure that contains local net clients
	 */
	struct FNodeClientSet
	{
		/** ClusterSync client (used on 'Main' thread to synchronize world simulation) */
		TUniquePtr<FDisplayClusterClusterSyncClient> ClusterSyncClient;

		/** RenderSync client (used on 'RHI' thread to synchronize presentation) */
		TUniquePtr<FDisplayClusterRenderSyncClient> RenderSyncClient;

		/** JSON cluster events client (used on 'Any' thread to send JSON events to a P-node) */
		TUniquePtr<FDisplayClusterClusterEventsJsonClient> ClusterEventsJsonClient;

		/** Binary cluster events client (used on 'Any' thread to send binary events to a P-node) */
		TUniquePtr<FDisplayClusterClusterEventsBinaryClient> ClusterEventsBinaryClient;

		/** InternalComm client (used on 'Any' thread for in-cluster communication) */
		TUniquePtr<FDisplayClusterInternalCommClient> InternalCommClient;
	};

	/** Per-node clients. This node has a bunch of clients connected to each cluster node, including itself. */
	TMap<FName, TSharedRef<FNodeClientSet>> Clients;

	/** A critical section to access the set of currently active clients. */
	mutable FCriticalSection ClientsCS;

private: // GP barrier clients

	/**
	 * Auxiliary structure that contains local GP barrier clients
	 */
	struct FNodeGeneralPurposeBarrierClientSet
	{
		/** Instantiates the client set, and establishes all neccessary connections. Returns true if everything is Ok. */
		bool Initialize(const IDisplayClusterProtocolInternalComm::FClusterServicesHostingInfo& InHostingInfo);

		/** Per-node clients (used on 'Any' thread) */
		TMap<FName, TSharedRef<FDisplayClusterGenericBarrierClient>> Clients;
	};

	/** Client ID to client set map*/
	TMap<int32, TSharedRef<FNodeGeneralPurposeBarrierClientSet>> GPBClients;

	/** GPB client set counter to keep client ID unique. */
	int32 GPBClientSetCounter = 0;

	/** A critical section to access the set GP barrier clients. */
	mutable FCriticalSection GPBClientsCS;

private: // servers

	/** Shared TCP connection listener for all internal services */
	TSharedRef<FDisplayClusterTcpListener> TcpListener;

	/** Holds internal service names */
	const TSet<FName> InternalServiceNames;

	/** Holds external service names */
	const TSet<FName> ExternalServiceNames;

	/** Keeps connection information of every node in the cluster. */
	FClusterServicesHostingInfo HostingInfo;

	/** Whether external servers have started and running. Used for deferred ext servers start. */
	bool bExternalServersRunning = false;

	/** Used to ignore any session termination callbacks. */
	bool bIsTerminating = false;
};
