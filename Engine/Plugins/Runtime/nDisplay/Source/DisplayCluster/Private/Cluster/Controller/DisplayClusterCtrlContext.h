// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/ThreadSingleton.h"
#include "Misc/Optional.h"
#include "UObject/NameTypes.h"


/**
 * Per-thread communication context
 *
 * This container is used to pass any data from the failover controller to the cluster controller.
 */
struct FDisplayClusterCtrlContext
	: public TThreadSingleton<FDisplayClusterCtrlContext>
{
	/** If set, this node must be addressed during the current request. */
	TOptional<FName> TargetNodeId;

	/** Holds ID of a client set to use in the GP barrier sync transactions. */
	TOptional<int32> GPBClientId;
};
