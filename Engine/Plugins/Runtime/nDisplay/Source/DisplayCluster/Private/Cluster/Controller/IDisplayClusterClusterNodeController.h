// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Network/Protocol/IDisplayClusterProtocolClusterSync.h"
#include "Network/Protocol/IDisplayClusterProtocolEventsBinary.h"
#include "Network/Protocol/IDisplayClusterProtocolEventsJson.h"
#include "Network/Protocol/IDisplayClusterProtocolGenericBarrier.h"
#include "Network/Protocol/IDisplayClusterProtocolInternalComm.h"
#include "Network/Protocol/IDisplayClusterProtocolRenderSync.h"

#include "DisplayClusterEnums.h"

class FDisplayClusterService;


/**
 * Node controller interface
 */
class IDisplayClusterClusterNodeController
	: public IDisplayClusterProtocolClusterSync
	, public IDisplayClusterProtocolEventsBinary
	, public IDisplayClusterProtocolEventsJson
	, public IDisplayClusterProtocolGenericBarrier
	, public IDisplayClusterProtocolInternalComm
	, public IDisplayClusterProtocolRenderSync
{
public:

	virtual ~IDisplayClusterClusterNodeController() = default;

public:
	/** Initialize controller instance */
	virtual bool Initialize()
	{
		return true;
	}

	/** Stop  clients/servers/etc */
	virtual void Shutdown()
	{ }

public:

	/** Return node ID */
	virtual FString GetNodeId() const = 0;

	/** Return controller name */
	virtual FString GetControllerName() const = 0;

	/** Returns a set of internal service names */
	virtual TSet<FName> GetInternalServiceNames() const
	{
		return { };
	}

	/** Access to a specific service */
	virtual TWeakPtr<FDisplayClusterService> GetService(const FName& ServiceName) const
	{
		return nullptr;
	}

	/** Initializes internal set of GB clients. Returns set ID for external referencing. */
	virtual int32 InitializeGeneralPurposeBarrierClients()
	{
		return INDEX_NONE;
	}

	/** Releases requested clients set. */
	virtual void ReleaseGeneralPurposeBarrierClients(int32 ClientSetId)
	{ }

	/** Drop specific cluster node */
	virtual bool DropClusterNode(const FString& NodeId)
	{
		return false;
	}

	/** Send binary event to a specific target outside of the cluster */
	virtual void SendClusterEventTo(const FString& Address, const uint16 Port, const FDisplayClusterClusterEventBinary& Event, bool bPrimaryOnly)
	{ }

	/** Send JSON event to a specific target outside of the cluster */
	virtual void SendClusterEventTo(const FString& Address, const uint16 Port, const FDisplayClusterClusterEventJson& Event, bool bPrimaryOnly)
	{ }
};
