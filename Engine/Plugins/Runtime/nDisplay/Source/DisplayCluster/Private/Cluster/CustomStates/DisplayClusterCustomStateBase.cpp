// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cluster/CustomStates/DisplayClusterCustomStateBase.h"
#include "RenderingThread.h"


namespace UE::nDisplay::Private
{
	FCustomStateBase::FCustomStateBase(const FName& InUniqueName, const FName& InClusterNodeId)
		: UniqueName(InUniqueName)
		, ClusterNodeId(InClusterNodeId)
		, StateCS(MakeShared<FCriticalSection>())
	{
		if (IDisplayCluster::IsAvailable())
		{
			ClusterMgr = IDisplayCluster::Get().GetClusterMgr();
		}
	}

	FName FCustomStateBase::GetName() const
	{
		return UniqueName;
	}

	void FCustomStateBase::Lock() const
	{
		StateCS->Lock();
	}

	void FCustomStateBase::Unlock() const
	{
		StateCS->Unlock();
	}

	void FCustomStateBase::ExecuteOnRenderThread(TUniqueFunction<void()> InFunc)
	{
		if (InFunc.IsSet())
		{
			// Just schedule update on the render thread
			ENQUEUE_RENDER_COMMAND(CustomState_ExecuteOnRenderThread)(
				[Func = MoveTemp(InFunc)](FRHICommandListImmediate& RHICmdList)
				{
					Func();
				});
		}
	}
}
