// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cluster/CustomStates/DisplayClusterCustomStateHelpers.h"
#include "Cluster/CustomStates/IDisplayClusterCustomState.h"

#include "Serialization/Archive.h"
#include "Serialization/MemoryWriter.h"


namespace UE::nDisplay::Cluster::Private
{
	FCustomStateRequestBlob::FCustomStateRequestBlob(const TSharedPtr<IDisplayClusterCustomState>& InState)
		: StateName(InState->GetName())
		, bCustomUpstream(InState->HasCustomUpstreamConfiguration())
		, UpstreamNodes(bCustomUpstream ? InState->GetUpstreams() : TSet<FName>{})
		, bExposesData(InState->ShouldPropagate())
		, StateType(InState->GetType())
	{
		if (bExposesData)
		{
			FMemoryWriter DataWriter(StateData);
			InState->Serialize(DataWriter);
		}
	}

	void FCustomStateRequestBlob::Serialize(FArchive& Ar)
	{
		Ar << StateName;
		Ar << StateType;

		// Conditional upstream configuration
		Ar << bCustomUpstream;
		if (bCustomUpstream)
		{
			Ar << UpstreamNodes;
		}
		
		// Conditional state exposure
		Ar << bExposesData;
		if (bExposesData)
		{
			Ar << StateData;
		}
	}

	void FCustomStateResponseBlob::Serialize(FArchive& Ar)
	{
		Ar << StateName;

		if (Ar.IsSaving())
		{
			// Nodes num
			int32 NodesNum = NodeStates.Num();
			Ar << NodesNum;

			for (TPair<FName, TPair<FName, TArray<uint8>>>& NodeState : NodeStates)
			{
				// Node ID
				Ar << NodeState.Key;
				// Type ID
				Ar << NodeState.Value.Key;
				// State data
				Ar << NodeState.Value.Value;
			}
		}
		else
		{
			// Nodes num
			int32 NodesNum = 0;
			Ar << NodesNum;

			for (int32 NodeIdx = 0; NodeIdx < NodesNum; ++NodeIdx)
			{
				// Node ID
				FName NodeId;
				Ar << NodeId;

				// Type ID
				FName TypeId;
				Ar << TypeId;

				// State data
				TArray<uint8> StateData;
				Ar << StateData;

				// Add to the map
				NodeStates.Emplace(NodeId, MakeTuple(TypeId, MoveTemp(StateData)));
			}
		}
	}
}
