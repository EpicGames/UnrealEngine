// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FArchive;
class IDisplayClusterCustomState;


namespace UE::nDisplay::Cluster::Private
{
	/**
	 * Auxiliary class to pass custom state synchronization requests in binary
	 */
	class FCustomStateRequestBlob
	{
	public:

		FCustomStateRequestBlob() = default;
		FCustomStateRequestBlob(const TSharedPtr<IDisplayClusterCustomState>& InState);
		~FCustomStateRequestBlob() = default;

	public:

		/** Serialization implementation */
		void Serialize(FArchive& Ar);

	public:

		/** State name */
		FName StateName;

		/** Whether a state has custom upstream nodes configuration */
		bool bCustomUpstream = false;

		/**
		 * When custom upstream configuration is used, this set contains
		 * node IDs that the state instance wants to get updates from.
		 */
		TSet<FName> UpstreamNodes;

		/** Whether this state exposes any data outside during synchronization */
		bool bExposesData = false;

		/** Type ID of this state */
		FName StateType;

		/** Serialized state data */
		TArray<uint8> StateData;
	};


	/**
	 * Auxiliary class to pass custom state synchronization responses in binary
	 */
	class FCustomStateResponseBlob
	{
	public:

		FCustomStateResponseBlob() = default;

		FCustomStateResponseBlob(const FName& InStateName)
			: StateName(InStateName)
		{ }

	public:

		/** Serialization implementation */
		void Serialize(FArchive& Ar);

	public:

		/** State name */
		FName StateName;

		/**
		 * Contains serialized state data in a form of <NodeId-to-{ StateType, StateData }> map.
		 * So any state may have multiple cluster nodes that it comes from, each one would
		 * provide its own serialized data.
		 */
		TMap<FName, TPair<FName, TArray<uint8>>> NodeStates;
	};
}
