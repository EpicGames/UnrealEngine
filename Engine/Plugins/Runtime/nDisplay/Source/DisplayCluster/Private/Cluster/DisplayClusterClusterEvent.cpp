// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cluster/DisplayClusterClusterEvent.h"

#include "Misc/DisplayClusterLog.h"
#include "Misc/DisplayClusterStrings.h"
#include "Misc/DisplayClusterTypesConverter.h"

#include "Serialization/Archive.h"


//////////////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterClusterEventBase
//////////////////////////////////////////////////////////////////////////////////////////////
void FDisplayClusterClusterEventBase::Serialize(FArchive& Ar)
{
	Ar << bIsSystemEvent;
	Ar << bShouldDiscardOnRepeat;
}


//////////////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterClusterEventJson
//////////////////////////////////////////////////////////////////////////////////////////////
void FDisplayClusterClusterEventJson::Serialize(FArchive& Ar)
{
	FDisplayClusterClusterEventBase::Serialize(Ar);

	Ar << Category;
	Ar << Type;
	Ar << Name;
	Ar << Parameters;
}

FString FDisplayClusterClusterEventJson::ToString(bool bWithParams) const
{
	if (bWithParams)
	{
		FString ParamText;
		for (const TPair<FString, FString>& Parameter : Parameters)
		{
			ParamText += FString::Printf(TEXT("%s%s%s;"), *Parameter.Key, DisplayClusterStrings::common::KeyValSeparator, *Parameter.Value);
		}

		return FString::Printf(TEXT("%s:%s:%s:%d:%d:%s"),
			*Category,
			*Type,
			*Name,
			bIsSystemEvent ? 1 : 0,
			bShouldDiscardOnRepeat ? 1 : 0,
			*ParamText);
	}
	else
	{
		return FString::Printf(TEXT("%s:%s:%s:%d:%d"),
			*Category,
			*Type,
			*Name,
			bIsSystemEvent ? 1 : 0,
			bShouldDiscardOnRepeat ? 1 : 0);
	}
}


//////////////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterClusterEventBinary
//////////////////////////////////////////////////////////////////////////////////////////////
void FDisplayClusterClusterEventBinary::SerializeToByteArray(TArray<uint8>& Arch) const
{
	// Allocate buffer memory
	const uint32 BufferSize = sizeof(EventId) + sizeof(bIsSystemEvent) + sizeof(bShouldDiscardOnRepeat) + EventData.Num();
	Arch.SetNumUninitialized(BufferSize);

	uint32 WriteOffset = 0;

	// EventId
	FMemory::Memcpy(Arch.GetData() + WriteOffset, &EventId, sizeof(EventId));
	WriteOffset += sizeof(EventId);

	// bIsSystemEvent
	FMemory::Memcpy(Arch.GetData() + WriteOffset, &bIsSystemEvent, sizeof(bIsSystemEvent));
	WriteOffset += sizeof(bIsSystemEvent);

	// bShouldDiscardOnRepeat
	FMemory::Memcpy(Arch.GetData() + WriteOffset, &bShouldDiscardOnRepeat, sizeof(bShouldDiscardOnRepeat));
	WriteOffset += sizeof(bShouldDiscardOnRepeat);

	// EventData
	FMemory::Memcpy(Arch.GetData() + WriteOffset, EventData.GetData(), EventData.Num());
}

bool FDisplayClusterClusterEventBinary::DeserializeFromByteArray(const TArray<uint8>& Arch)
{
	static const int32 MinBufferSize = sizeof(EventId) + sizeof(bIsSystemEvent) + sizeof(bShouldDiscardOnRepeat);

	if (Arch.Num() < MinBufferSize)
	{
		return false;
	}

	uint32 ReadOffset = 0;

	// EventId
	FMemory::Memcpy(&EventId, Arch.GetData() + ReadOffset, sizeof(EventId));
	ReadOffset += sizeof(EventId);

	// bIsSystemEvent
	FMemory::Memcpy(&bIsSystemEvent, Arch.GetData() + ReadOffset, sizeof(bIsSystemEvent));
	ReadOffset += sizeof(bIsSystemEvent);

	// bShouldDiscardOnRepeat
	FMemory::Memcpy(&bShouldDiscardOnRepeat, Arch.GetData() + ReadOffset, sizeof(bShouldDiscardOnRepeat));
	ReadOffset += sizeof(bShouldDiscardOnRepeat);

	// EventData
	const uint32 BinaryDataLength = Arch.Num() - ReadOffset;
	EventData.AddUninitialized(BinaryDataLength);
	FMemory::Memcpy(EventData.GetData(), Arch.GetData() + ReadOffset, BinaryDataLength);

	return true;
}

void FDisplayClusterClusterEventBinary::Serialize(FArchive& Ar)
{
	FDisplayClusterClusterEventBase::Serialize(Ar);

	Ar << EventId;
	Ar << EventData;
}
