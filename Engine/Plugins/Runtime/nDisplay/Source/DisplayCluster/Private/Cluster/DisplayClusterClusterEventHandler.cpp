// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cluster/DisplayClusterClusterEventHandler.h"

#include "Misc/DisplayClusterAppExit.h"
#include "Misc/DisplayClusterConsoleExec.h"
#include "Misc/DisplayClusterStrings.h"


namespace DisplayClusterStrings {
	namespace cluster_events {
		static constexpr auto EvtConsoleExecName = TEXT("console exec");
	}
}


FDisplayClusterClusterEventHandler::FDisplayClusterClusterEventHandler()
{
	ListenerDelegate = FOnClusterEventJsonListener::CreateRaw(this, &FDisplayClusterClusterEventHandler::HandleClusterEvent);
}

void FDisplayClusterClusterEventHandler::HandleClusterEvent(const FDisplayClusterClusterEventJson& InEvent)
{
	// Filter system events only
	if (InEvent.bIsSystemEvent)
	{
		// Filter those events we're interested in
		if (InEvent.Category.Equals(DisplayClusterStrings::cluster_events::EventCategory, ESearchCase::IgnoreCase) &&
			InEvent.Type.Equals(DisplayClusterStrings::cluster_events::EventType, ESearchCase::IgnoreCase))
		{
			if (InEvent.Name.Equals(DisplayClusterStrings::cluster_events::EvtQuitName, ESearchCase::IgnoreCase))
			{
				// QUIT event
				FDisplayClusterAppExit::ExitApplication(FString("QUIT requested on a system cluster event"));
			}
			else if (InEvent.Name.Equals(DisplayClusterStrings::cluster_events::EvtConsoleExecName, ESearchCase::IgnoreCase))
			{
				// Console command event
				FDisplayClusterConsoleExec::Exec(InEvent);
			}
		}
	}
}
