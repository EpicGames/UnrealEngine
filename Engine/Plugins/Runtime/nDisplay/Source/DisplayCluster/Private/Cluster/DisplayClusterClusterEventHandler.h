// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Cluster/IDisplayClusterClusterEventListener.h"
#include "Cluster/IDisplayClusterClusterManager.h"


/**
 * Internal handler for the system cluster events
 */
class FDisplayClusterClusterEventHandler
	: public IDisplayClusterClusterEventListener
{
protected:

	FDisplayClusterClusterEventHandler();

public:

	/** Singleton access */
	static FDisplayClusterClusterEventHandler& Get()
	{
		static FDisplayClusterClusterEventHandler Instance;
		return Instance;
	}

	/** Returns JSON event handler delegate */
	FOnClusterEventJsonListener& GetJsonListenerDelegate()
	{
		return ListenerDelegate;
	}

protected:

	/** Cluster event processing function */
	void HandleClusterEvent(const FDisplayClusterClusterEventJson& InEvent);

	/** Cluster event listener delegate */
	FOnClusterEventJsonListener ListenerDelegate;
};
