// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cluster/DisplayClusterClusterManager.h"

#include "Cluster/DisplayClusterClusterEventHandler.h"
#include "Cluster/DisplayClusterGenericBarrierAPI.h"

#include "Cluster/IDisplayClusterClusterSyncObject.h"
#include "Cluster/IDisplayClusterClusterEventListener.h"

#include "Cluster/Controller/DisplayClusterClusterNodeCtrlDisabled.h"
#include "Cluster/Controller/DisplayClusterClusterNodeCtrlEditor.h"
#include "Cluster/Controller/DisplayClusterClusterNodeCtrlMain.h"

#include "Cluster/CustomStates/IDisplayClusterCustomState.h"

#include "Cluster/Failover/DisplayClusterFailoverNodeCtrlDisabled.h"
#include "Cluster/Failover/DisplayClusterFailoverNodeCtrlEditor.h"
#include "Cluster/Failover/DisplayClusterFailoverNodeCtrlMain.h"

#include "Cluster/NetAPI/DisplayClusterNetApiFacade.h"

#include "Config/IPDisplayClusterConfigManager.h"

#include "Dom/JsonObject.h"

#include "Misc/App.h"
#include "Misc/DisplayClusterAppExit.h"
#include "Misc/DisplayClusterGlobals.h"
#include "Misc/DisplayClusterHelpers.h"
#include "Misc/DisplayClusterLog.h"
#include "Misc/DisplayClusterStrings.h"
#include "Misc/DisplayClusterTypesConverter.h"
#include "Misc/ScopeLock.h"

#include "Serialization/Archive.h"
#include "UObject/Interface.h"

#include "DisplayClusterConfigurationTypes.h"
#include "IDisplayClusterCallbacks.h"


namespace UE::nDisplay::DisplayCluster::Private
{
	static bool GIsCustomStatesPropagationEnabled = true;
	static FAutoConsoleVariableRef CVarDisplayClusterOptimizeResourcesOnOffscreenNodes(
		TEXT("nDisplay.Cluster.CustomStatesPropagation.Enabled"),
		GIsCustomStatesPropagationEnabled,
		TEXT("Whether states synchronization is required (consumes a bit of traffic)\n")
		TEXT("0 : Disabled\n")
		TEXT("1 : Enabled\n"),
		ECVF_Default
	);
}


FDisplayClusterClusterManager::FDisplayClusterClusterManager()
	: NodeCtrl(MakeShared<FDisplayClusterClusterNodeCtrlDisabled>())
	, FailoverCtrl(MakeShared<FDisplayClusterFailoverNodeCtrlDisabled>(NodeCtrl))
	, NetApi(new FDisplayClusterNetApiFacade(FailoverCtrl))
{
	// Sync objects
	ObjectsToSync.Emplace(EDisplayClusterSyncGroup::PreTick).Reserve(64);
	ObjectsToSync.Emplace(EDisplayClusterSyncGroup::Tick).Reserve(64);
	ObjectsToSync.Emplace(EDisplayClusterSyncGroup::PostTick).Reserve(64);

	// Sync objects - replication
	ObjectsToSyncCache.Emplace(EDisplayClusterSyncGroup::PreTick);
	ObjectsToSyncCache.Emplace(EDisplayClusterSyncGroup::Tick);
	ObjectsToSyncCache.Emplace(EDisplayClusterSyncGroup::PostTick);

	ObjectsToSyncCacheReadySignals.Emplace(EDisplayClusterSyncGroup::PreTick,  FPlatformProcess::GetSynchEventFromPool(true));
	ObjectsToSyncCacheReadySignals.Emplace(EDisplayClusterSyncGroup::Tick,     FPlatformProcess::GetSynchEventFromPool(true));
	ObjectsToSyncCacheReadySignals.Emplace(EDisplayClusterSyncGroup::PostTick, FPlatformProcess::GetSynchEventFromPool(true));

	// Set cluster event handlers. These are the entry points for any incoming cluster events.
	OnClusterEventJson.AddRaw(this,   &FDisplayClusterClusterManager::OnClusterEventJsonHandler);
	OnClusterEventBinary.AddRaw(this, &FDisplayClusterClusterManager::OnClusterEventBinaryHandler);

	// Set internal system events handler
	OnClusterEventJson.Add(FDisplayClusterClusterEventHandler::Get().GetJsonListenerDelegate());
}

FDisplayClusterClusterManager::~FDisplayClusterClusterManager()
{
	UE_LOGF(LogDisplayClusterCluster, Log, "Releasing cluster manager...");

	// Trigger all data cache availability events to prevent client session threads to be deadlocked.
	SetInternalSyncObjectsReleaseState(true);

	// Stop networking in case it hasn't been stopped yet
	ReleaseNetworking();

	// Return sync event objects to the pool
	for (TPair<EDisplayClusterSyncGroup, FEvent*>& GroupEventIt : ObjectsToSyncCacheReadySignals)
	{
		FPlatformProcess::ReturnSynchEventToPool(GroupEventIt.Value);
	}
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IPDisplayClusterManager
//////////////////////////////////////////////////////////////////////////////////////////////
bool FDisplayClusterClusterManager::Init(EDisplayClusterOperationMode OperationMode)
{
	CurrentOperationMode = OperationMode;
	return true;
}

void FDisplayClusterClusterManager::Release()
{
	CurrentOperationMode = EDisplayClusterOperationMode::Disabled;
}

bool FDisplayClusterClusterManager::StartSession(UDisplayClusterConfigurationData* InConfigData, const FString& InNodeId)
{
	ClusterNodeId = InNodeId;

	UE_LOGF(LogDisplayClusterCluster, Log, "Node ID: %ls", *ClusterNodeId);

	// Node name must be valid
	if (ClusterNodeId.IsEmpty())
	{
		UE_LOGF(LogDisplayClusterCluster, Error, "Node ID was not specified");
		return false;
	}

	// Get configuration data
	const UDisplayClusterConfigurationData* const ConfigData = GDisplayCluster->GetPrivateConfigMgr()->GetConfig();
	if (!ConfigData)
	{
		UE_LOGF(LogDisplayClusterCluster, Error, "Couldn't get configuration data");
		return false;
	}

	// Does it exist in the cluster configuration?
	if (!ConfigData->Cluster->Nodes.Contains(ClusterNodeId))
	{
		UE_LOGF(LogDisplayClusterCluster, Error, "Node '%ls' not found in the configuration data", *ClusterNodeId);
		return false;
	}

	// Subscribe for events
	GDisplayCluster->GetCallbacks().OnDisplayClusterFailoverPrimaryNodeChanged().AddRaw(this, &FDisplayClusterClusterManager::OnPrimaryNodeChangedHandler);
	GDisplayCluster->GetCallbacks().OnDisplayClusterFailoverNodeDown().AddRaw(this, &FDisplayClusterClusterManager::OnClusterNodeFailed);

	// Reset all internal sync objects
	SetInternalSyncObjectsReleaseState(false);

	// Save initial list of cluster nodes
	ConfigData->Cluster->Nodes.GetKeys(InitialClusterNodeIds);

	// Also, initialize the active nodes list
	{
		FScopeLock Lock(&ActiveClusterNodeIdsCS);
		ActiveClusterNodeIds = InitialClusterNodeIds;
	}

	// Determine cluster role for this instance
	InitializeClusterRole(InNodeId, ConfigData);

	// Set primary node
	SetPrimaryNode(ConfigData->Cluster->PrimaryNode.Id);

	// Initialize networking internals
	const bool bNetworkingInternalsInitialized = InitializeNetworking(ConfigData);
	if (!bNetworkingInternalsInitialized)
	{
		UE_LOGF(LogDisplayClusterCluster, Error, "Node '%ls' could not initialize networking subsystems", *ClusterNodeId);
		return false;
	}

	return true;
}

void FDisplayClusterClusterManager::EndSession()
{
	// Unsubscribe from the session events
	GDisplayCluster->GetCallbacks().OnDisplayClusterFailoverPrimaryNodeChanged().RemoveAll(this);
	GDisplayCluster->GetCallbacks().OnDisplayClusterFailoverNodeDown().RemoveAll(this);

	// Trigger all data cache availability events to prevent
	// client session threads to be deadlocked.
	SetInternalSyncObjectsReleaseState(true);

	// Stop networking
	ReleaseNetworking();

	{
		FScopeLock Lock(&ActiveClusterNodeIdsCS);
		ActiveClusterNodeIds.Reset();
	}

	InitialClusterNodeIds.Reset();
	ClusterNodeId.Empty();
}

void FDisplayClusterClusterManager::EndScene()
{
	{
		FScopeLock Lock(&ObjectsToSyncCS);
		for (auto& SyncGroupPair : ObjectsToSync)
		{
			SyncGroupPair.Value.Reset();
		}
	}

	{
		FScopeLock Lock(&ClusterEventListenersCS);
		ClusterEventListeners.Reset();
	}

	NativeInputCache.Reset();
}

void FDisplayClusterClusterManager::StartFrame(uint64 FrameNum)
{
	// Even though this signal gets reset on EndFrame, it's still possible a client
	// will try to synchronize time data before the primary node finishes EndFrame
	// processing. Since time data replication step and EndFrame call don't have
	// any barriers between each other, it's theoretically possible a client will
	// get outdated time information which will break determinism. As a simple
	// solution that requires minimum resources, we do safe signal reset right
	// after WaitForFrameStart barrier, which is called after time data
	// synchronization. As a result, we're 100% sure the clients will always get
	// actual time data.
	TimeDataCacheReadySignal->Reset();

	// The following code is a fix/workaround for exactly the same problem described
	// above. With new failover code, there is one more data item (TimeData cache)
	// that must be safely reset before any other nodes call GetTimeData().
	// 
	// Consider this fix temporary. In the future, a more robust solution should be
	// implemented. The game thread simulation pipeline should use a single barrier
	// that synchronizes frame start and time data in one call.
	GetDataCache()->TempWorkaround_ResetTimeDataCache();
}

void FDisplayClusterClusterManager::EndFrame(uint64 FrameNum)
{
	// Reset all the synchronization objects
	SetInternalSyncObjectsReleaseState(false);

	// Reset cache containers
	JsonEventsCache.Reset();
	BinaryEventsCache.Reset();
	NativeInputCache.Reset();

	// Reset objects sync cache for all sync groups
	for (TPair<EDisplayClusterSyncGroup, TMap<FString, TArray<uint8>>>& It : ObjectsToSyncCache)
	{
		It.Value.Reset();
	}
}

void FDisplayClusterClusterManager::PreTick(float DeltaSeconds)
{
	// Sync custom states
	SyncStates();

	// Sync cluster objects (PreTick)
	SyncObjects(EDisplayClusterSyncGroup::PreTick);

	// Sync cluster events
	SyncEvents();
}

void FDisplayClusterClusterManager::Tick(float DeltaSeconds)
{
	// Sync cluster objects (Tick)
	SyncObjects(EDisplayClusterSyncGroup::Tick);
}

void FDisplayClusterClusterManager::PostTick(float DeltaSeconds)
{
	// Sync cluster objects (PostTick)
	SyncObjects(EDisplayClusterSyncGroup::PostTick);
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterClusterManager
//////////////////////////////////////////////////////////////////////////////////////////////
bool FDisplayClusterClusterManager::IsPrimary() const
{
	return HasClusterRole(EDisplayClusterNodeRole::Primary);
}

bool FDisplayClusterClusterManager::IsSecondary() const
{
	return HasClusterRole(EDisplayClusterNodeRole::Secondary);
}

bool FDisplayClusterClusterManager::IsBackup() const
{
	return HasClusterRole(EDisplayClusterNodeRole::Backup);
}

bool FDisplayClusterClusterManager::HasClusterRole(EDisplayClusterNodeRole Role) const
{
	return GetClusterRole() == Role;
}

EDisplayClusterNodeRole FDisplayClusterClusterManager::GetClusterRole() const
{
	FScopeLock Lock(&CurrentNodeRoleCS);
	return CurrentNodeRole;
}

FString FDisplayClusterClusterManager::GetPrimaryNodeId() const
{
	FScopeLock Lock(&PrimaryNodeIdCS);
	return PrimaryNodeId;
}

FString FDisplayClusterClusterManager::GetNodeId() const
{
	return ClusterNodeId;
}

uint32 FDisplayClusterClusterManager::GetNodesAmount() const
{
	FScopeLock Lock(&ActiveClusterNodeIdsCS);
	const int32 NodesNum = ActiveClusterNodeIds.Num();
	return static_cast<uint32>(NodesNum <= 0 ? 0 : NodesNum);
}

void FDisplayClusterClusterManager::GetNodeIds(TArray<FString>& OutNodeIds) const
{
	FScopeLock Lock(&ActiveClusterNodeIdsCS);
	OutNodeIds = ActiveClusterNodeIds.Array();
}

void FDisplayClusterClusterManager::GetNodeIds(TSet<FString>& OutNodeIds) const
{
	FScopeLock Lock(&ActiveClusterNodeIdsCS);
	OutNodeIds = ActiveClusterNodeIds;
}

bool FDisplayClusterClusterManager::DropClusterNode(const FString& NodeId)
{
	if (!IsPrimary())
	{
		UE_LOGF(LogDisplayClusterCluster, Warning, "Node drop is allowed on P-nodes only");
		return false;
	}

	return DropNode(NodeId, IPDisplayClusterClusterManager::ENodeDropReason::UserRequest);
}

void FDisplayClusterClusterManager::RegisterSyncObject(IDisplayClusterClusterSyncObject* SyncObj, EDisplayClusterSyncGroup SyncGroup)
{
	if (SyncObj)
	{
		FScopeLock Lock(&ObjectsToSyncCS);
		ObjectsToSync[SyncGroup].Add(SyncObj);
		UE_LOGF(LogDisplayClusterCluster, Log, "Registered sync object: %ls", *SyncObj->GetSyncId());
	}
}

void FDisplayClusterClusterManager::UnregisterSyncObject(IDisplayClusterClusterSyncObject* SyncObj)
{
	if (SyncObj)
	{
		FScopeLock Lock(&ObjectsToSyncCS);

		for (auto& GroupPair : ObjectsToSync)
		{
			GroupPair.Value.Remove(SyncObj);
		}

		UE_LOGF(LogDisplayClusterCluster, Log, "Unregistered sync object: %ls", *SyncObj->GetSyncId());
	}
}

bool FDisplayClusterClusterManager::RegisterCustomState(const FName& UniqueStateName, TSharedPtr<IDisplayClusterCustomState>& StateVariable)
{
	if (!StateVariable.IsValid())
	{
		return false;
	}

	// Name duplication is not allowed
	if (IsCustomStateRegistered(UniqueStateName))
	{
		return false;
	}

	// Store for future replication
	{
		FScopeLock Lock(&CustomStatesCS);
		CustomStates.Emplace(UniqueStateName, StateVariable);
	}

	return true;
}

bool FDisplayClusterClusterManager::UnregisterCustomState(const FName& UniqueStateName)
{
	FScopeLock Lock(&CustomStatesCS);
	const int32 NumRemovedPairs = CustomStates.Remove(UniqueStateName);
	return NumRemovedPairs > 0;
}

TSet<FName> FDisplayClusterClusterManager::GetRegisteredCustomStateNames() const
{
	FScopeLock Lock(&CustomStatesCS);
	TSet<FName> Output;
	CustomStates.GetKeys(Output);
	return Output;
}

bool FDisplayClusterClusterManager::IsCustomStateRegistered(const FName& UniqueStateName) const
{
	FScopeLock Lock(&CustomStatesCS);
	return CustomStates.Contains(UniqueStateName);
}

TSharedRef<IDisplayClusterGenericBarriersClient> FDisplayClusterClusterManager::CreateGenericBarriersClient()
{
	return MakeShared<FDisplayClusterGenericBarrierAPI>();
}

void FDisplayClusterClusterManager::AddClusterEventListener(TScriptInterface<IDisplayClusterClusterEventListener> Listener)
{
	FScopeLock Lock(&ClusterEventListenersCS);
	if (Listener.GetObject() && IsValidChecked(Listener.GetObject()) && !Listener.GetObject()->IsUnreachable())
	{
		ClusterEventListeners.Add(Listener);
	}
}

void FDisplayClusterClusterManager::RemoveClusterEventListener(TScriptInterface<IDisplayClusterClusterEventListener> Listener)
{
	FScopeLock Lock(&ClusterEventListenersCS);
	if (ClusterEventListeners.Contains(Listener))
	{
		ClusterEventListeners.Remove(Listener);
		UE_LOGF(LogDisplayClusterCluster, Verbose, "Cluster event listeners left: %d", ClusterEventListeners.Num());
	}
}

void FDisplayClusterClusterManager::AddClusterEventJsonListener(const FOnClusterEventJsonListener& Listener)
{
	FScopeLock Lock(&ClusterEventListenersCS);
	OnClusterEventJson.Add(Listener);
}

void FDisplayClusterClusterManager::RemoveClusterEventJsonListener(const FOnClusterEventJsonListener& Listener)
{
	FScopeLock Lock(&ClusterEventListenersCS);
	OnClusterEventJson.Remove(Listener.GetHandle());
}

void FDisplayClusterClusterManager::AddClusterEventBinaryListener(const FOnClusterEventBinaryListener& Listener)
{
	FScopeLock Lock(&ClusterEventListenersCS);
	OnClusterEventBinary.Add(Listener);
}

void FDisplayClusterClusterManager::RemoveClusterEventBinaryListener(const FOnClusterEventBinaryListener& Listener)
{
	FScopeLock Lock(&ClusterEventListenersCS);
	OnClusterEventBinary.Remove(Listener.GetHandle());
}

void FDisplayClusterClusterManager::EmitClusterEventJson(const FDisplayClusterClusterEventJson& Event, bool bPrimaryOnly)
{
	UE_LOGF(LogDisplayClusterCluster, Verbose, "JSON event emission request: %ls", *Event.ToString());

	FScopeLock Lock(&ClusterEventsJsonCS);

	if (CurrentOperationMode == EDisplayClusterOperationMode::Cluster || CurrentOperationMode == EDisplayClusterOperationMode::Editor)
	{
		// [Primary] Since we receive cluster events asynchronously, we push it to a primary events pool
		if (IsPrimary())
		{
			// Generate event ID
			const FString EventId = FString::Printf(TEXT("%s-%s-%s"), *Event.Category, *Event.Type, *Event.Name);
			// Make it shared ptr
			TSharedPtr<FDisplayClusterClusterEventJson> EventPtr = MakeShared<FDisplayClusterClusterEventJson>(Event);
			// Store event object
			if (EventPtr->bShouldDiscardOnRepeat)
			{
				ClusterEventsJson.FindOrAdd(EventPtr->bIsSystemEvent).Emplace(EventId, EventPtr);
			}
			else
			{
				ClusterEventsJsonNonDiscarded.Add(EventPtr);
			}
		}
		// [Secondary] Send event to the primary node
		else
		{
			// An event will be emitted from a secondary node if it's explicitly specified by bPrimaryOnly=false
			if (!bPrimaryOnly)
			{
				FailoverCtrl->EmitClusterEventJson(Event);
			}
		}
	}
}

void FDisplayClusterClusterManager::EmitClusterEventBinary(const FDisplayClusterClusterEventBinary& Event, bool bPrimaryOnly)
{
	UE_LOGF(LogDisplayClusterCluster, Verbose, "BIN event emission request: %d", Event.EventId);

	FScopeLock Lock(&ClusterEventsBinaryCS);

	if (CurrentOperationMode == EDisplayClusterOperationMode::Cluster || CurrentOperationMode == EDisplayClusterOperationMode::Editor)
	{
		// [Primary] Since we receive cluster events asynchronously, we push it to a primary events pool
		if (IsPrimary())
		{
			// Make it shared ptr
			TSharedPtr<FDisplayClusterClusterEventBinary> EventPtr = MakeShared<FDisplayClusterClusterEventBinary>(Event);

			if (EventPtr->bShouldDiscardOnRepeat)
			{
				ClusterEventsBinary.FindOrAdd(EventPtr->bIsSystemEvent).Emplace(EventPtr->EventId, EventPtr);
			}
			else
			{
				ClusterEventsBinaryNonDiscarded.Add(EventPtr);
			}
		}
		// [Secondary] Send event to the primary node
		else
		{
			// An event will be emitted from a secondary node if it's explicitly specified by bPrimaryOnly=false
			if (!bPrimaryOnly)
			{
				FailoverCtrl->EmitClusterEventBinary(Event);
			}
		}
	}
}

void FDisplayClusterClusterManager::SendClusterEventTo(const FString& Address, const uint16 Port, const FDisplayClusterClusterEventJson& Event, bool bPrimaryOnly)
{
	if (CurrentOperationMode == EDisplayClusterOperationMode::Cluster || CurrentOperationMode == EDisplayClusterOperationMode::Editor)
	{
		if (IsPrimary() || !bPrimaryOnly)
		{
			UE_LOGF(LogDisplayClusterCluster, Verbose, "JSON event emission request: recipient=%ls:%u, event=%ls:%ls:%ls", *Address, Port, *Event.Category, *Event.Type, *Event.Name);
			NodeCtrl->SendClusterEventTo(Address, Port, Event, bPrimaryOnly);
		}
	}
}

void FDisplayClusterClusterManager::SendClusterEventTo(const FString& Address, const uint16 Port, const FDisplayClusterClusterEventBinary& Event, bool bPrimaryOnly)
{
	if (CurrentOperationMode == EDisplayClusterOperationMode::Cluster || CurrentOperationMode == EDisplayClusterOperationMode::Editor)
	{
		if (IsPrimary() || !bPrimaryOnly)
		{
			UE_LOGF(LogDisplayClusterCluster, Verbose, "BIN event emission request: recipient=%ls:%u, event=%d", *Address, Port, Event.EventId);
			NodeCtrl->SendClusterEventTo(Address, Port, Event, bPrimaryOnly);
		}
	}
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IPDisplayClusterClusterManager
//////////////////////////////////////////////////////////////////////////////////////////////
FDisplayClusterNetApiFacade& FDisplayClusterClusterManager::GetNetApi()
{
	return *NetApi.Get();
}

TSharedRef<IDisplayClusterClusterNodeController> FDisplayClusterClusterManager::GetNodeController()
{
	return NodeCtrl;
}

TSharedRef<FDisplayClusterCommDataCache> FDisplayClusterClusterManager::GetDataCache()
{
	return FailoverCtrl->GetDataCache();
}

TWeakPtr<FDisplayClusterService> FDisplayClusterClusterManager::GetNodeService(const FName& ServiceName)
{
	return NodeCtrl->GetService(ServiceName);
}

bool FDisplayClusterClusterManager::DropNode(const FString& NodeId, IPDisplayClusterClusterManager::ENodeDropReason DropReason)
{
	// Ignore invalid requests
	{
		FScopeLock Lock(&ActiveClusterNodeIdsCS);

		if (!ActiveClusterNodeIds.Contains(NodeId))
		{
			return false;
		}
	}

	UE_LOGF(LogDisplayClusterCluster, Log, "Requested node '%ls' drop, reason=%u", *NodeId, EnumToUnderlyingType(DropReason));

	// User requests are sent to the desired nodes as "exit" like commands
	if (DropReason == IPDisplayClusterClusterManager::ENodeDropReason::UserRequest)
	{
		FailoverCtrl->RequestNodeDrop(NodeId, static_cast<uint8>(DropReason));
	}
	// Other requests should go though failover pipeline
	else if (DropReason == IPDisplayClusterClusterManager::ENodeDropReason::Failed)
	{
		HandleNodeDrop(NodeId);
	}

	return true;
}

void FDisplayClusterClusterManager::CacheTimeData()
{
	// Cache data so it will be the same for all requests within current frame
	DeltaTimeCache = FApp::GetDeltaTime();
	GameTimeCache  = FApp::GetGameTime();
	FrameTimeCache = FApp::GetCurrentFrameTime();

	UE_LOGF(LogDisplayClusterCluster, Verbose, "Time data cache: Delta=%lf, Game=%lf, Frame=%lf",
		DeltaTimeCache, GameTimeCache, FrameTimeCache.IsSet() ? FrameTimeCache.GetValue().AsSeconds() : 0.f);

	TimeDataCacheReadySignal->Trigger();
}

void FDisplayClusterClusterManager::SyncTimeData()
{
	double DeltaTime = 0.0f;
	double GameTime  = 0.0f;
	TOptional<FQualifiedFrameTime> FrameTime;

	UE_LOGF(LogDisplayClusterCluster, Verbose, "Downloading synchronization data (time)...");

	GetNetApi().GetClusterSyncAPI()->GetTimeData(DeltaTime, GameTime, FrameTime);

	UE_LOGF(LogDisplayClusterCluster, Verbose, "Downloading finished. Delta=%lf, Game=%lf, Frame=%lf",
		DeltaTime, GameTime, FrameTime.IsSet() ? FrameTime.GetValue().AsSeconds() : 0.f);

	// Apply new time data (including primary node)
	ImportTimeData(DeltaTime, GameTime, FrameTime);
}

void FDisplayClusterClusterManager::ExportTimeData(double& OutDeltaTime, double& OutGameTime, TOptional<FQualifiedFrameTime>& OutFrameTime)
{
	// Wait until data is available
	TimeDataCacheReadySignal->Wait();

	// Return cached values
	OutDeltaTime = DeltaTimeCache;
	OutGameTime  = GameTimeCache;
	OutFrameTime = FrameTimeCache;
}

void FDisplayClusterClusterManager::ImportTimeData(const double& InDeltaTime, const double& InGameTime, const TOptional<FQualifiedFrameTime>& InFrameTime)
{
	// Compute new 'current' and 'last' time on the local platform timeline
	const double NewCurrentTime = FPlatformTime::Seconds();
	const double NewLastTime  = NewCurrentTime - InDeltaTime;

	// Store new data
	FApp::SetCurrentTime(NewLastTime);
	FApp::UpdateLastTime();
	FApp::SetCurrentTime(NewCurrentTime);
	FApp::SetDeltaTime(InDeltaTime);
	FApp::SetGameTime(InGameTime);
	FApp::SetIdleTime(0);
	FApp::SetIdleTimeOvershoot(0);

	if (InFrameTime.IsSet())
	{
		FApp::SetCurrentFrameTime(InFrameTime.GetValue());
		UE_LOGF(LogDisplayClusterCluster, Verbose, "DisplayCluster timecode: %ls | %ls", *FTimecode::FromFrameNumber(InFrameTime->Time.GetFrame(), InFrameTime->Rate).ToString(), *InFrameTime->Rate.ToPrettyText().ToString());
	}
	else
	{
		FApp::InvalidateCurrentFrameTime();
		UE_LOGF(LogDisplayClusterCluster, Verbose, "DisplayCluster timecode: Invalid");
	}
}

void FDisplayClusterClusterManager::SyncStates()
{
	// Skip states syncrhonization if disabled by a cvar
	if (!UE::nDisplay::DisplayCluster::Private::GIsCustomStatesPropagationEnabled)
	{
		return;
	}

	FScopeLock Lock(&CustomStatesCS);

	// Update frame bound data before propagating local updates
	AdvanceFrameOnCustomStates();

	// Prepare request data. All local states will be serialized into a request buffer.
	TArray<uint8> StatesRequestData;
	GenerateLocalStatesPropagationData(StatesRequestData);

	// Propagate our local states, and receive the states of entire cluster
	TArray<uint8> StatesResponse;
	UE_LOGF(LogDisplayClusterCluster, Verbose, "Downloading cluster states data (custom states)...");
	GetNetApi().GetClusterSyncAPI()->PropagateStatesData(StatesRequestData, StatesResponse);
	UE_LOGF(LogDisplayClusterCluster, Verbose, "Downloading finished. %d bytes received (custom states).", StatesResponse.Num());

	// Deserialize the response, and propagate updates among local state instances
	ProcessClusterStatesData(StatesResponse);
}

void FDisplayClusterClusterManager::SyncObjects(EDisplayClusterSyncGroup InSyncGroup)
{
	TMap<FString, TArray<uint8>> ObjectsData;

	UE_LOGF(LogDisplayClusterCluster, Verbose, "Downloading synchronization data (objects)...");
	GetNetApi().GetClusterSyncAPI()->GetObjectsData(InSyncGroup, ObjectsData);
	UE_LOGF(LogDisplayClusterCluster, Verbose, "Downloading finished. Available %d records (objects).", ObjectsData.Num());

	// Perform data load (objects state update)
	ImportObjectsData(InSyncGroup, ObjectsData);
}

void FDisplayClusterClusterManager::CacheObjects(EDisplayClusterSyncGroup SyncGroup)
{
	FScopeLock Lock(&ObjectsToSyncCS);

	// Cache data for requested sync group
	if (TMap<FString, TArray<uint8>>* GroupCache = ObjectsToSyncCache.Find(SyncGroup))
	{
		UE_LOGF(LogDisplayClusterCluster, Verbose, "Exporting sync data for sync group: %u, items to sync: %d", (uint8)SyncGroup, GroupCache->Num());

		for (IDisplayClusterClusterSyncObject* SyncObj : ObjectsToSync[SyncGroup])
		{
			if (SyncObj && SyncObj->IsActive() && SyncObj->IsDirty())
			{
				// Get ID of the object being serialized
				FString SyncId = SyncObj->GetSyncId();
				UE_LOGF(LogDisplayClusterCluster, Verbose, "Adding object to sync: %ls", *SyncId);

				// Initialize output buffer
				TArray<uint8> SyncData;
				constexpr int32 SyncDataReservedBufferSize = 256;
				SyncData.Reserve(SyncDataReservedBufferSize);

				// Serialize the object
				FMemoryWriter Writer(SyncData);
				SyncObj->SerializeDC(Writer);

				UE_LOGF(LogDisplayClusterCluster, VeryVerbose, "Sync object: %ls serialized into %d bytes", *SyncId, SyncData.Num());

				// Cache the object
				GroupCache->Emplace(MoveTemp(SyncId), MoveTemp(SyncData));

				SyncObj->ClearDirty();
			}
		}
	}

	UE_LOGF(LogDisplayClusterCluster, Verbose, "Objects data cache contains %d records", ObjectsToSyncCache[SyncGroup].Num());

	// Notify data is available
	ObjectsToSyncCacheReadySignals[SyncGroup]->Trigger();
}

void FDisplayClusterClusterManager::ExportObjectsData(const EDisplayClusterSyncGroup InSyncGroup, TMap<FString, TArray<uint8>>& OutObjectsData)
{
	// Wait until primary node provides data
	ObjectsToSyncCacheReadySignals[InSyncGroup]->Wait();
	// Return cached value
	OutObjectsData = ObjectsToSyncCache[InSyncGroup];
}

void FDisplayClusterClusterManager::ImportObjectsData(const EDisplayClusterSyncGroup InSyncGroup, const TMap<FString, TArray<uint8>>& InObjectsData)
{
	if (InObjectsData.Num() > 0)
	{
		for (auto It = InObjectsData.CreateConstIterator(); It; ++It)
		{
			UE_LOGF(LogDisplayClusterCluster, VeryVerbose, "sync-data: %ls has %d bytes", *It->Key, It->Value.Num());
		}

		FScopeLock Lock(&ObjectsToSyncCS);

		for (IDisplayClusterClusterSyncObject* SyncObj : ObjectsToSync[InSyncGroup])
		{
			if (SyncObj && SyncObj->IsActive())
			{
				const FString SyncId = SyncObj->GetSyncId();
				const TArray<uint8>* const FoundSerializedData = InObjectsData.Find(SyncId);

				// Determine whether any data exists for this object
				if (!FoundSerializedData)
				{
					UE_LOGF(LogDisplayClusterCluster, VeryVerbose, "%ls has nothing to update", *SyncId);
					continue;
				}

				FMemoryReader Reader(*FoundSerializedData);
				SyncObj->SerializeDC(Reader);
			}
		}
	}
}

void FDisplayClusterClusterManager::SyncEvents()
{
	TArray<TSharedPtr<FDisplayClusterClusterEventJson>>   JsonEvents;
	TArray<TSharedPtr<FDisplayClusterClusterEventBinary>> BinaryEvents;

	// Get events data from a provider
	UE_LOGF(LogDisplayClusterCluster, Verbose, "Downloading synchronization data (events)...");
	GetNetApi().GetClusterSyncAPI()->GetEventsData(JsonEvents, BinaryEvents);
	UE_LOGF(LogDisplayClusterCluster, Verbose, "Downloading finished. Available events: json=%d binary=%d", JsonEvents.Num(), BinaryEvents.Num());

	// Import and process them
	ImportEventsData(JsonEvents, BinaryEvents);
}

void FDisplayClusterClusterManager::CacheEvents()
{
	// Export JSON events
	{
		FScopeLock Lock(&ClusterEventsJsonCS);

		// Export all system and non-system json events that have 'discard on repeat' flag
		for (const TPair<bool, TMap<FString, TSharedPtr<FDisplayClusterClusterEventJson>>>& It : ClusterEventsJson)
		{
			TArray<TSharedPtr<FDisplayClusterClusterEventJson>> JsonEventsToExport;
			It.Value.GenerateValueArray(JsonEventsToExport);
			JsonEventsCache.Append(MoveTemp(JsonEventsToExport));
		}

		// Clear original containers
		ClusterEventsJson.Reset();

		// Export all json events that don't have 'discard on repeat' flag
		JsonEventsCache.Append(MoveTemp(ClusterEventsJsonNonDiscarded));
	}

	// Export binary events
	{
		FScopeLock Lock(&ClusterEventsBinaryCS);

		// Export all binary events that have 'discard on repeat' flag
		for (const TPair<bool, TMap<int32, TSharedPtr<FDisplayClusterClusterEventBinary>>>& It : ClusterEventsBinary)
		{
			TArray<TSharedPtr<FDisplayClusterClusterEventBinary>> BinaryEventsToExport;
			It.Value.GenerateValueArray(BinaryEventsToExport);
			BinaryEventsCache.Append(MoveTemp(BinaryEventsToExport));
		}

		// Clear original containers
		ClusterEventsBinary.Reset();

		// Export all binary events that don't have 'discard on repeat' flag
		BinaryEventsCache.Append(MoveTemp(ClusterEventsBinaryNonDiscarded));
	}

	UE_LOGF(LogDisplayClusterCluster, Verbose, "Cluster events data cache contains: json=%d, binary=%d", JsonEventsCache.Num(), BinaryEventsCache.Num());

	// Notify data is available
	CachedEventsDataSignal->Trigger();
}

void FDisplayClusterClusterManager::ExportEventsData(TArray<TSharedPtr<FDisplayClusterClusterEventJson>>& OutJsonEvents, TArray<TSharedPtr<FDisplayClusterClusterEventBinary>>& OutBinaryEvents)
{
	// Wait until data is available
	CachedEventsDataSignal->Wait();

	// Return cached value
	OutJsonEvents   = JsonEventsCache;
	OutBinaryEvents = BinaryEventsCache;
}

void FDisplayClusterClusterManager::ImportEventsData(const TArray<TSharedPtr<FDisplayClusterClusterEventJson>>& InJsonEvents, const TArray<TSharedPtr<FDisplayClusterClusterEventBinary>>& InBinaryEvents)
{
	// Process and fire all JSON events
	if (InJsonEvents.Num() > 0)
	{
		FScopeLock LockListeners(&ClusterEventListenersCS);

		for (const TSharedPtr<FDisplayClusterClusterEventJson>& Event : InJsonEvents)
		{
			UE_LOGF(LogDisplayClusterCluster, Verbose, "Processing json event %ls|%ls|%ls|s%d|d%d...", *Event->Category, *Event->Type, *Event->Name, Event->bIsSystemEvent ? 1 : 0, Event->bShouldDiscardOnRepeat ? 1 : 0);
			// Fire event
			OnClusterEventJson.Broadcast(*Event);
		}
	}

	// Process and fire all binary events
	if (InBinaryEvents.Num() > 0)
	{
		FScopeLock LockListeners(&ClusterEventListenersCS);

		for (const TSharedPtr<FDisplayClusterClusterEventBinary>& Event : InBinaryEvents)
		{
			UE_LOGF(LogDisplayClusterCluster, Verbose, "Processing binary event %d...", Event->EventId);
			// Fire event
			OnClusterEventBinary.Broadcast(*Event);
		}
	}
}

void FDisplayClusterClusterManager::ImportNativeInputData(TMap<FString, TArray<uint8>>& InNativeInputData)
{
	// Cache input data
	NativeInputCache = MoveTemp(InNativeInputData);

	UE_LOGF(LogDisplayClusterCluster, VeryVerbose, "Native input data cache: %d items", NativeInputCache.Num());

	// Notify the data is available
	NativeInputCacheReadySignal->Trigger();
}

void FDisplayClusterClusterManager::ExportNativeInputData(TMap<FString, TArray<uint8>>& OutNativeInputData)
{
	// Wait for data cache to be ready
	NativeInputCacheReadySignal->Wait();
	// Export data from cache
	OutNativeInputData = NativeInputCache;
}


//////////////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterClusterManager
//////////////////////////////////////////////////////////////////////////////////////////////
bool FDisplayClusterClusterManager::InitializeNetworking(const UDisplayClusterConfigurationData* ConfigData)
{
	// Instantiate cluster node controller
	NodeCtrl = CreateClusterNodeController();

	// Initialize the controller
	if (!NodeCtrl->Initialize())
	{
		UE_LOGF(LogDisplayClusterCluster, Error, "Couldn't initialize the networking controller.");
		return false;
	}

	// Instantiate failover controller
	FailoverCtrl = CreateFailoverController(NodeCtrl);

	// Initialize the controller
	if (!FailoverCtrl->Initialize(ConfigData))
	{
		UE_LOGF(LogDisplayClusterCluster, Error, "Couldn't initialize the failover controller.");
		return false;
	}

	// Finally, setup API
	NetApi = MakeUnique<FDisplayClusterNetApiFacade>(FailoverCtrl);

	UE_LOGF(LogDisplayClusterCluster, Log, "Networking internals have been successfully initialized.");

	return true;
}

void FDisplayClusterClusterManager::ReleaseNetworking()
{
	// Stop local clients/servers
	NodeCtrl->Shutdown();

	// Reset controllers to their 'Disabled' state
	NodeCtrl = MakeShared<FDisplayClusterClusterNodeCtrlDisabled>();
	FailoverCtrl = MakeShared<FDisplayClusterFailoverNodeCtrlDisabled>(NodeCtrl);
	NetApi = MakeUnique<FDisplayClusterNetApiFacade>(FailoverCtrl);
}

TSharedRef<IDisplayClusterClusterNodeController> FDisplayClusterClusterManager::CreateClusterNodeController() const
{
	// Instantiate appropriate controller depending on the operation mode
	if (CurrentOperationMode == EDisplayClusterOperationMode::Cluster)
	{
		UE_LOGF(LogDisplayClusterCluster, Log, "Instantiating 'Main' node controller...");
		return MakeShared<FDisplayClusterClusterNodeCtrlMain>(ClusterNodeId);
	}
	else if (CurrentOperationMode == EDisplayClusterOperationMode::Editor)
	{
		UE_LOGF(LogDisplayClusterCluster, Log, "Instantiating 'Editor' node controller...");
		return MakeShared<FDisplayClusterClusterNodeCtrlEditor>();
	}

	// Otherwise 'Disabled'
	UE_LOGF(LogDisplayClusterCluster, Log, "Instantiating 'Disabled' node controller...");
	return MakeShared<FDisplayClusterClusterNodeCtrlDisabled>();
}

TSharedRef<IDisplayClusterFailoverNodeController> FDisplayClusterClusterManager::CreateFailoverController(TSharedRef<IDisplayClusterClusterNodeController>& ClusterCtrl) const
{
	// Instantiate appropriate controller depending on the operation mode
	if (CurrentOperationMode == EDisplayClusterOperationMode::Cluster)
	{
		UE_LOGF(LogDisplayClusterCluster, Log, "Instantiating 'Main' failover controller...");
		return MakeShared<FDisplayClusterFailoverNodeCtrlMain>(ClusterCtrl);
	}
	else if (CurrentOperationMode == EDisplayClusterOperationMode::Editor)
	{
		UE_LOGF(LogDisplayClusterCluster, Log, "Instantiating 'Editor' failover controller...");
		return MakeShared<FDisplayClusterFailoverNodeCtrlEditor>(ClusterCtrl);
	}

	// Otherwise 'Disabled'
	UE_LOGF(LogDisplayClusterCluster, Log, "Instantiating 'Disabled' failover controller...");
	return MakeShared<FDisplayClusterFailoverNodeCtrlDisabled>(ClusterCtrl);
}

void FDisplayClusterClusterManager::InitializeClusterRole(const FString& NodeId, const UDisplayClusterConfigurationData* ConfigData)
{
	checkSlow(ConfigData);

	const bool bIsPrimary = NodeId.Equals(ConfigData->Cluster->PrimaryNode.Id, ESearchCase::IgnoreCase);
	if (bIsPrimary)
	{
		SetClusterRole(EDisplayClusterNodeRole::Primary);
		return;
	}
	
	// Currently we don't completely support the backup nodes concept. So this
	// part remains @todo. If it was supported, we would need to determine
	// either it's 'secondary' or 'backup'.
	SetClusterRole(EDisplayClusterNodeRole::Secondary);
}

void FDisplayClusterClusterManager::SetPrimaryNode(const FString& NewPrimaryNodeId)
{
	UE_LOGF(LogDisplayClusterCluster, Verbose, "Requested new primary node: '%ls'", *NewPrimaryNodeId);

	{
		FScopeLock LockPrimary(&PrimaryNodeIdCS);

		// Nothing to do if already set
		if (PrimaryNodeId.Equals(NewPrimaryNodeId, ESearchCase::IgnoreCase))
		{
			UE_LOGF(LogDisplayClusterCluster, VeryVerbose, "'%ls' is primary already", *NewPrimaryNodeId);
			return;
		}

		// Check if new node is valid
		{
			FScopeLock LockActive(&ActiveClusterNodeIdsCS);
			if (!ActiveClusterNodeIds.Contains(NewPrimaryNodeId))
			{
				UE_LOGF(LogDisplayClusterCluster, VeryVerbose, "'%ls' was not found in the list of active nodes", *NewPrimaryNodeId);
				return;
			}
		}

		// Update current primary
		PrimaryNodeId = NewPrimaryNodeId;

		UE_LOGF(LogDisplayClusterCluster, Log, "New primary node (P-node): '%ls'", *NewPrimaryNodeId);

		// Update the role if we're the new primary.
		const bool bThisNodeIsNowPrimary = NewPrimaryNodeId.Equals(ClusterNodeId, ESearchCase::IgnoreCase);
		if (bThisNodeIsNowPrimary)
		{
			SetClusterRole(EDisplayClusterNodeRole::Primary);
		}
	}
}

void FDisplayClusterClusterManager::SetClusterRole(EDisplayClusterNodeRole NewRole)
{
	FScopeLock Lock(&CurrentNodeRoleCS);
	UE_LOGF(LogDisplayClusterCluster, Log, "New cluster role: '%u'", static_cast<uint8>(NewRole));
	CurrentNodeRole = NewRole;
}

void FDisplayClusterClusterManager::HandleNodeDrop(const FString& NodeId)
{
	// Remove this node from the list of active nodes
	{
		FScopeLock Lock(&ActiveClusterNodeIdsCS);
		if (ActiveClusterNodeIds.Remove(NodeId) <= 0)
		{
			// This node has been processed already so nothing to do
			return;
		}
	}

	// Just exit if this node has failed
	if (NodeId.Equals(GetNodeId(), ESearchCase::IgnoreCase))
	{
		FDisplayClusterAppExit::ExitApplication(TEXT("This node has failed. Requesting exit."));
		return;
	}

	// Let the node controller drop it
	NodeCtrl->DropClusterNode(NodeId);

	// Let the failover controller process this
	if (!FailoverCtrl->HandleFailure(NodeId))
	{
		FDisplayClusterAppExit::ExitApplication(TEXT("Failover controller was unable to handle a failure. Requesting exit."));
	}

	// Finally, broadcast node failed event
	GDisplayCluster->GetCallbacks().OnDisplayClusterFailoverNodeDown().Broadcast(NodeId);
}

void FDisplayClusterClusterManager::OnClusterEventJsonHandler(const FDisplayClusterClusterEventJson& Event)
{
	FScopeLock Lock(&ClusterEventListenersCS);

	decltype(ClusterEventListeners) InvalidListeners;

	for (auto Listener : ClusterEventListeners)
	{
		if (!Listener.GetObject() || !IsValidChecked(Listener.GetObject()) || Listener.GetObject()->IsUnreachable()) // Note: .GetInterface() is always returning null when intefrace is added to class in the Blueprint.
		{
			UE_LOGF(LogDisplayClusterCluster, Warning, "Will remove invalid cluster event listener");
			InvalidListeners.Add(Listener);
			continue;
		}
		Listener->Execute_OnClusterEventJson(Listener.GetObject(), Event);
	}

	for (auto& InvalidListener : InvalidListeners)
	{
		ClusterEventListeners.Remove(InvalidListener);
	}
}

void FDisplayClusterClusterManager::OnClusterEventBinaryHandler(const FDisplayClusterClusterEventBinary& Event)
{
	FScopeLock Lock(&ClusterEventListenersCS);

	decltype(ClusterEventListeners) InvalidListeners;

	for (auto Listener : ClusterEventListeners)
	{
		if (!Listener.GetObject() || !IsValidChecked(Listener.GetObject()) || Listener.GetObject()->IsUnreachable()) // Note: .GetInterface() is always returning null when intefrace is added to class in the Blueprint.
		{
			UE_LOGF(LogDisplayClusterCluster, Warning, "Will remove invalid cluster event listener");
			InvalidListeners.Add(Listener);
			continue;
		}

		Listener->Execute_OnClusterEventBinary(Listener.GetObject(), Event);
	}

	for (auto& InvalidListener : InvalidListeners)
	{
		ClusterEventListeners.Remove(InvalidListener);
	}
}

void FDisplayClusterClusterManager::SetInternalSyncObjectsReleaseState(bool bRelease)
{
	if (bRelease)
	{
		// Set all events signaled
		TimeDataCacheReadySignal->Trigger();
		CachedEventsDataSignal->Trigger();
		NativeInputCacheReadySignal->Trigger();

		for (TPair<EDisplayClusterSyncGroup, FEvent*>& It : ObjectsToSyncCacheReadySignals)
		{
			It.Value->Trigger();
		}
	}
	else
	{
		// Reset all cache events
		TimeDataCacheReadySignal->Reset();
		CachedEventsDataSignal->Reset();
		NativeInputCacheReadySignal->Reset();

		// Reset events for all sync groups
		for (TPair<EDisplayClusterSyncGroup, FEvent*>& It : ObjectsToSyncCacheReadySignals)
		{
			It.Value->Reset();
		}
	}
}

void FDisplayClusterClusterManager::OnPrimaryNodeChangedHandler(const FString& NewPrimaryId)
{
	SetPrimaryNode(NewPrimaryId);
}

void FDisplayClusterClusterManager::OnClusterNodeFailed(const FString& FailedNodeId)
{
	// Remove it from the active nodes list
	{
		FScopeLock Lock(&ActiveClusterNodeIdsCS);
		ActiveClusterNodeIds.Remove(FailedNodeId);
	}
}

void FDisplayClusterClusterManager::AdvanceFrameOnCustomStates()
{
	FScopeLock Lock(&CustomStatesCS);

	for (const TPair<FName, TSharedPtr<IDisplayClusterCustomState>>& CustomState : CustomStates)
	{
		CustomState.Value->AdvanceFrame();
	}
}

void FDisplayClusterClusterManager::GenerateLocalStatesPropagationData(TArray<uint8>& OutLocalStatesData)
{
	FScopeLock Lock(&CustomStatesCS);

	OutLocalStatesData.Reset();
	FMemoryWriter DataWriter(OutLocalStatesData);

	// States num
	{
		int32 StatesNum = CustomStates.Num();
		DataWriter << StatesNum;
	}

	// States data
	{
		for (TPair<FName, TSharedPtr<IDisplayClusterCustomState>>& CustomState : CustomStates)
		{
			// Serialize the sate instance into a blob
			CustomState.Value->Lock();
			UE::nDisplay::Cluster::Private::FCustomStateRequestBlob StateBlob(CustomState.Value);
			CustomState.Value->Unlock();

			// Write this blob to the buffer
			StateBlob.Serialize(DataWriter);
		}
	}
}

void FDisplayClusterClusterManager::ProcessClusterStatesData(const TArray<uint8>& ClusterStatesData)
{
	// Just skip if there is no input data
	if (ClusterStatesData.Num() <= 0)
	{
		return;
	}

	FMemoryReader DataReader(ClusterStatesData);

	// Read states num
	int32 StatesNum = 0;
	DataReader << StatesNum;
	if (StatesNum <= 0)
	{
		checkSlow(StatesNum == 0);
		return;
	}

	// Clear data & reserve memory
	TArray<FClusterStateBlob> ClusterStatesDataMap;
	ClusterStatesDataMap.Reserve(StatesNum);

	// Read states
	for (int32 StateIdx = 0; StateIdx < StatesNum; ++StateIdx)
	{
		FClusterStateBlob StateBlob;
		StateBlob.Serialize(DataReader);
		ClusterStatesDataMap.Add(MoveTemp(StateBlob));
	}

	// Propagate the updates locally
	{
		FScopeLock Lock(&CustomStatesCS);

		// Iterate through all cluster states
		for (FClusterStateBlob& ClusterState : ClusterStatesDataMap)
		{
			// If this state exists locally, let it update itself
			if (TSharedPtr<IDisplayClusterCustomState>* FoundState = CustomStates.Find(ClusterState.StateName))
			{
				// Final data to pass to the local state instance
				TMap<FName, TArray<uint8>> CompatibleStates;

				// The type of local state data
				const FName LocalStateType = (*FoundState)->GetType();

				// Iterate through all the states we have received
				for (TPair<FName, TPair<FName, TArray<uint8>>>& NodeState : ClusterState.NodeStates)
				{
					// If types match
					if (LocalStateType == NodeState.Value.Key)
					{
						CompatibleStates.Emplace(NodeState.Key, MoveTemp(NodeState.Value.Value));
					}
				}

				// Finally, let the state instance update itself
				(*FoundState)->Update(CompatibleStates);
			}
		}
	}
}
