// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Cluster/IPDisplayClusterClusterManager.h"
#include "Cluster/DisplayClusterClusterEvent.h"
#include "Cluster/CustomStates/DisplayClusterCustomStateHelpers.h"
#include "HAL/Event.h"
#include "Misc/Optional.h"
#include "Misc/QualifiedFrameTime.h"

class IDisplayClusterClusterNodeController;
class IDisplayClusterFailoverNodeController;
class FDisplayClusterNetApiFacade;
class FDisplayClusterService;
class UDisplayClusterConfigurationData;


/**
 * Cluster manager. Responsible for network communication and data replication.
 */
class FDisplayClusterClusterManager
	: public IPDisplayClusterClusterManager
{
public:

	FDisplayClusterClusterManager();
	virtual ~FDisplayClusterClusterManager();

public:

	//~ Begin IPDisplayClusterManager
	virtual bool Init(EDisplayClusterOperationMode OperationMode) override;
	virtual void Release() override;
	virtual bool StartSession(UDisplayClusterConfigurationData* InConfigData, const FString& NodeID) override;
	virtual void EndSession() override;
	virtual void EndScene() override;
	virtual void StartFrame(uint64 FrameNum) override;
	virtual void EndFrame(uint64 FrameNum) override;
	virtual void PreTick(float DeltaSeconds) override;
	virtual void Tick(float DeltaSeconds) override;
	virtual void PostTick(float DeltaSeconds) override;
	//~ End IPDisplayClusterManager

public:

	//~ Begin IDisplayClusterClusterManager
	virtual bool IsPrimary()   const override;
	virtual bool IsSecondary() const override;
	virtual bool IsBackup()    const override;
	virtual EDisplayClusterNodeRole GetClusterRole() const override;
	virtual bool HasClusterRole(EDisplayClusterNodeRole Role) const override;

	virtual FString GetPrimaryNodeId() const override;

	virtual FString GetNodeId() const override;
	virtual uint32 GetNodesAmount() const override;
	virtual void GetNodeIds(TArray<FString>& OutNodeIds) const override;
	virtual void GetNodeIds(TSet<FString>& OutNodeIds) const override;

	virtual bool DropClusterNode(const FString& NodeId) override;

	virtual void RegisterSyncObject(IDisplayClusterClusterSyncObject* SyncObj, EDisplayClusterSyncGroup SyncGroup) override;
	virtual void UnregisterSyncObject(IDisplayClusterClusterSyncObject* SyncObj) override;

	virtual bool RegisterCustomState(const FName& UniqueStateName, TSharedPtr<IDisplayClusterCustomState>& StateVariable) override;
	virtual bool UnregisterCustomState(const FName& UniqueStateName) override;
	virtual TSet<FName> GetRegisteredCustomStateNames() const override;
	virtual bool IsCustomStateRegistered(const FName& UniqueStateName) const override;

	virtual TSharedRef<IDisplayClusterGenericBarriersClient> CreateGenericBarriersClient() override;

	virtual void AddClusterEventListener(TScriptInterface<IDisplayClusterClusterEventListener>) override;
	virtual void RemoveClusterEventListener(TScriptInterface<IDisplayClusterClusterEventListener>) override;

	virtual void AddClusterEventJsonListener(const FOnClusterEventJsonListener& Listener) override;
	virtual void RemoveClusterEventJsonListener(const FOnClusterEventJsonListener& Listener) override;

	virtual void AddClusterEventBinaryListener(const FOnClusterEventBinaryListener& Listener) override;
	virtual void RemoveClusterEventBinaryListener(const FOnClusterEventBinaryListener& Listener) override;

	virtual void EmitClusterEventJson(const FDisplayClusterClusterEventJson& Event, bool bPrimaryOnly) override;
	virtual void EmitClusterEventBinary(const FDisplayClusterClusterEventBinary& Event, bool bPrimaryOnly) override;

	virtual void SendClusterEventTo(const FString& Address, const uint16 Port, const FDisplayClusterClusterEventJson&   Event, bool bPrimaryOnly) override;
	virtual void SendClusterEventTo(const FString& Address, const uint16 Port, const FDisplayClusterClusterEventBinary& Event, bool bPrimaryOnly) override;
	//~ End IDisplayClusterClusterManager

public:

	//~ Begin IPDisplayClusterClusterManager

	virtual FDisplayClusterNetApiFacade& GetNetApi() override;
	virtual TSharedRef<IDisplayClusterClusterNodeController> GetNodeController() override;
	virtual TSharedRef<FDisplayClusterCommDataCache> GetDataCache() override;
	virtual TWeakPtr<FDisplayClusterService> GetNodeService(const FName& ServiceName) override;

	virtual bool DropNode(const FString& NodeId, ENodeDropReason DropReason) override;

	// Sync time data
	virtual void SyncTimeData() override;
	virtual void CacheTimeData() override;
	virtual void ExportTimeData(      double& OutDeltaTime,      double& OutGameTime,      TOptional<FQualifiedFrameTime>& OutFrameTime) override;
	virtual void ImportTimeData(const double& InDeltaTime, const double& InGameTime, const TOptional<FQualifiedFrameTime>& InFrameTime) override;

	// Sync states
	virtual void SyncStates() override;

	// Sync objects
	virtual void SyncObjects(EDisplayClusterSyncGroup SyncGroup) override;
	virtual void CacheObjects(EDisplayClusterSyncGroup SyncGroup) override;
	virtual void ExportObjectsData(const EDisplayClusterSyncGroup InSyncGroup,       TMap<FString, TArray<uint8>>& OutObjectsData) override;
	virtual void ImportObjectsData(const EDisplayClusterSyncGroup InSyncGroup, const TMap<FString, TArray<uint8>>& InObjectsData) override;

	// Sync cluster events
	virtual void SyncEvents()  override;
	virtual void CacheEvents() override;
	virtual void ExportEventsData(      TArray<TSharedPtr<FDisplayClusterClusterEventJson>>& OutJsonEvents,      TArray<TSharedPtr<FDisplayClusterClusterEventBinary>>& OutBinaryEvents) override;
	virtual void ImportEventsData(const TArray<TSharedPtr<FDisplayClusterClusterEventJson>>& InJsonEvents, const TArray<TSharedPtr<FDisplayClusterClusterEventBinary>>& InBinaryEvents) override;

	// Sync native input
	virtual void ExportNativeInputData(TMap<FString, TArray<uint8>>& OutNativeInputData) override;
	virtual void ImportNativeInputData(TMap<FString, TArray<uint8>>& InNativeInputData) override;

	//~ End IPDisplayClusterClusterManager

private:

	/** Performs initialization of networking subsystems */
	bool InitializeNetworking(const UDisplayClusterConfigurationData* ConfigData);

	/** Release networking subsystems */
	void ReleaseNetworking();

	/** Factory method to instantiate a networking controller */
	TSharedRef<IDisplayClusterClusterNodeController> CreateClusterNodeController() const;

	/** Factory method to instantiate a failover controller */
	TSharedRef<IDisplayClusterFailoverNodeController> CreateFailoverController(TSharedRef<IDisplayClusterClusterNodeController>& ClusterCtrl) const;

	/** Auxiliary method for group set/reset of internal signals */
	void SetInternalSyncObjectsReleaseState(bool bRelease);

	/** Determines cluster role on session start. */
	void InitializeClusterRole(const FString& NodeId, const UDisplayClusterConfigurationData* ConfigData);

	/** Changes current primary node ID. It doesn't do any actual role transition, just the ID variable. */
	void SetPrimaryNode(const FString& NewPrimaryNodeId);

	/** Changes current node role. */
	void SetClusterRole(EDisplayClusterNodeRole NewRole);

	/** Performs node drop internal cleaning. */
	void HandleNodeDrop(const FString& NodeId);

private:

	/** JSON cluster event handler */
	void OnClusterEventJsonHandler(const FDisplayClusterClusterEventJson& Event);

	/** Binary cluster event handler */
	void OnClusterEventBinaryHandler(const FDisplayClusterClusterEventBinary& Event);

	/** Handles primary node change events */
	void OnPrimaryNodeChangedHandler(const FString& NewPrimaryId);

	/** Handles node failure events */
	void OnClusterNodeFailed(const FString& FailedNodeId);

private:

	using FClusterStateBlob = UE::nDisplay::Cluster::Private::FCustomStateResponseBlob;

	/** Updates frame bound data on the state instances */
	void AdvanceFrameOnCustomStates();

	/**
	 * Serializes local cluster states before propagation
	 * 
	 * @param OutLocalStatesData - [out] Serialized state of all local state instanfces
	 */
	void GenerateLocalStatesPropagationData(TArray<uint8>& OutLocalStatesData);

	/**
	 * Deserializes response data, and propagate the updates among local state instances
	 * 
	 * @param ClusterStatesData - Serialized state of all states from every node
	 */
	void ProcessClusterStatesData(const TArray<uint8>& ClusterStatesData);

private:

	/** Networking controller */
	TSharedRef<IDisplayClusterClusterNodeController> NodeCtrl;

	/** Failover controller */
	TSharedRef<IDisplayClusterFailoverNodeController> FailoverCtrl;

	/** Networking API */
	TUniquePtr<FDisplayClusterNetApiFacade> NetApi;

private:

	/** Current operation mode */
	EDisplayClusterOperationMode CurrentOperationMode = EDisplayClusterOperationMode::Disabled;


	/** Current primary node. It may change in runtime after failure handling. */
	FString PrimaryNodeId;

	/** Safe access to the primary node ID */
	mutable FCriticalSection PrimaryNodeIdCS;


	/** Current role in the cluster */
	EDisplayClusterNodeRole CurrentNodeRole = EDisplayClusterNodeRole::None;

	/** Critical section to guard current role access */
	mutable FCriticalSection CurrentNodeRoleCS;


	/** This node ID */
	FString ClusterNodeId;

	/** A full set of cluster node IDs used on cluster start. */
	TSet<FString> InitialClusterNodeIds;

	/** A subset of cluster node IDs that are currently active */
	TSet<FString> ActiveClusterNodeIds;

	/** A critical section to guard access to the set of active nodes */
	mutable FCriticalSection ActiveClusterNodeIdsCS;


	FEventRef TimeDataCacheReadySignal{ EEventMode::ManualReset };
	double DeltaTimeCache = 0.f;
	double GameTimeCache  = 0.f;
	TOptional<FQualifiedFrameTime> FrameTimeCache;

	// Sync objects
	TMap<EDisplayClusterSyncGroup, TSet<IDisplayClusterClusterSyncObject*>> ObjectsToSync;
	mutable FCriticalSection ObjectsToSyncCS;
	// Sync objects - replication
	TMap<EDisplayClusterSyncGroup, FEvent*> ObjectsToSyncCacheReadySignals;
	TMap<EDisplayClusterSyncGroup, TMap<FString, TArray<uint8>>> ObjectsToSyncCache;

	// Native input - replication
	FEventRef NativeInputCacheReadySignal{ EEventMode::ManualReset };
	TMap<FString, TArray<uint8>> NativeInputCache;

	// JSON events
	TMap<bool, TMap<FString, TSharedPtr<FDisplayClusterClusterEventJson>>> ClusterEventsJson;
	TArray<TSharedPtr<FDisplayClusterClusterEventJson>> ClusterEventsJsonNonDiscarded;
	mutable FCriticalSection ClusterEventsJsonCS;
	FOnClusterEventJson OnClusterEventJson;
	
	// Binary events
	TMap<bool, TMap<int32, TSharedPtr<FDisplayClusterClusterEventBinary>>> ClusterEventsBinary;
	TArray<TSharedPtr<FDisplayClusterClusterEventBinary>> ClusterEventsBinaryNonDiscarded;
	mutable FCriticalSection ClusterEventsBinaryCS;
	FOnClusterEventBinary OnClusterEventBinary;

	// JSON/Binary events - replication
	FEventRef CachedEventsDataSignal{ EEventMode::ManualReset };
	TArray<TSharedPtr<FDisplayClusterClusterEventJson>>   JsonEventsCache;
	TArray<TSharedPtr<FDisplayClusterClusterEventBinary>> BinaryEventsCache;

	// Cluster event listeners
	mutable FCriticalSection ClusterEventListenersCS;
	TArray<TScriptInterface<IDisplayClusterClusterEventListener>> ClusterEventListeners;

	// Custom states
	mutable FCriticalSection CustomStatesCS;
	TMap<FName, TSharedPtr<IDisplayClusterCustomState>> CustomStates;
};
