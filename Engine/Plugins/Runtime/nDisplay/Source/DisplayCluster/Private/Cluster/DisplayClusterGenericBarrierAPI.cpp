// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cluster/DisplayClusterGenericBarrierAPI.h"

#include "Cluster/IPDisplayClusterClusterManager.h"
#include "Cluster/Controller/DisplayClusterCtrlContext.h"
#include "Cluster/Controller/IDisplayClusterClusterNodeController.h"
#include "Cluster/NetAPI/DisplayClusterNetApiFacade.h"

#include "Misc/DisplayClusterGlobals.h"
#include "Misc/DisplayClusterLog.h"

#include "Network/Barrier/IDisplayClusterBarrier.h"
#include "Network/Configuration/DisplayClusterNetworkConfiguration.h"
#include "Network/Service/GenericBarrier/DisplayClusterGenericBarrierService.h"

#include "IDisplayCluster.h"


FDisplayClusterGenericBarrierAPI::FDisplayClusterGenericBarrierAPI()
{
	ClientSetId = GDisplayCluster->GetPrivateClusterMgr()->GetNodeController()->InitializeGeneralPurposeBarrierClients();
	UE_LOGF(LogDisplayClusterBarrierGP, Log, "GP barrier client '%d' instantiated", ClientSetId)
}

FDisplayClusterGenericBarrierAPI::~FDisplayClusterGenericBarrierAPI()
{
	if (IDisplayCluster::IsAvailable())
	{
		GDisplayCluster->GetPrivateClusterMgr()->GetNodeController()->ReleaseGeneralPurposeBarrierClients(ClientSetId);
		UE_LOGF(LogDisplayClusterBarrierGP, Log, "GP barrier client '%d' released", ClientSetId)
	}
}

bool FDisplayClusterGenericBarrierAPI::CreateBarrier(const FString& BarrierId, const TMap<FString, TSet<FString>>& NodeToSyncCallers, uint32 Timeout)
{
	using EBarrierCtrlResult = IDisplayClusterProtocolGenericBarrier::EBarrierControlResult;

	// Pass client Id
	FDisplayClusterCtrlContext::Get().GPBClientId = ClientSetId;

	// Process request
	EBarrierCtrlResult CtrlResult = EBarrierCtrlResult::UnknownError;
	const EDisplayClusterCommResult CommResult = GDisplayCluster->GetPrivateClusterMgr()->GetNetApi().GetGenericBarrierAPI()->CreateBarrier(BarrierId, NodeToSyncCallers, Timeout, CtrlResult);

	const bool bCtrlOk = (CtrlResult == EBarrierCtrlResult::CreatedSuccessfully || CtrlResult == EBarrierCtrlResult::AlreadyExists);
	const bool bCommOk = (CommResult == EDisplayClusterCommResult::Ok);
	const bool bSucceeded = bCtrlOk && bCommOk;

	if (bSucceeded)
	{
		// Setup sync delegate
		ConfigureBarrierSyncDelegate(BarrierId, true);
	}

	return bSucceeded;
}

bool FDisplayClusterGenericBarrierAPI::WaitUntilBarrierIsCreated(const FString& BarrierId)
{
	using EBarrierCtrlResult = IDisplayClusterProtocolGenericBarrier::EBarrierControlResult;

	// Pass client Id
	FDisplayClusterCtrlContext::Get().GPBClientId = ClientSetId;

	// Process request
	EBarrierCtrlResult CtrlResult = EBarrierCtrlResult::UnknownError;
	const EDisplayClusterCommResult CommResult = GDisplayCluster->GetPrivateClusterMgr()->GetNetApi().GetGenericBarrierAPI()->WaitUntilBarrierIsCreated(BarrierId, CtrlResult);

	const bool bCtrlOk = (CtrlResult == EBarrierCtrlResult::AlreadyExists);
	const bool bCommOk = (CommResult == EDisplayClusterCommResult::Ok);
	const bool bSucceeded = bCtrlOk && bCommOk;

	return bSucceeded;
}

bool FDisplayClusterGenericBarrierAPI::IsBarrierAvailable(const FString& BarrierId)
{
	using EBarrierCtrlResult = IDisplayClusterProtocolGenericBarrier::EBarrierControlResult;

	// Pass client Id
	FDisplayClusterCtrlContext::Get().GPBClientId = ClientSetId;

	// Process request
	EBarrierCtrlResult CtrlResult = EBarrierCtrlResult::UnknownError;
	const EDisplayClusterCommResult CommResult = GDisplayCluster->GetPrivateClusterMgr()->GetNetApi().GetGenericBarrierAPI()->IsBarrierAvailable(BarrierId, CtrlResult);

	const bool bCtrlOk = (CtrlResult == EBarrierCtrlResult::AlreadyExists);
	const bool bCommOk = (CommResult == EDisplayClusterCommResult::Ok);
	const bool bBarrierAvailable = bCtrlOk && bCommOk;

	return bBarrierAvailable;
}

bool FDisplayClusterGenericBarrierAPI::ReleaseBarrier(const FString& BarrierId)
{
	using EBarrierCtrlResult = IDisplayClusterProtocolGenericBarrier::EBarrierControlResult;

	// Pass client Id
	FDisplayClusterCtrlContext::Get().GPBClientId = ClientSetId;

	// Release sync delegate
	ConfigureBarrierSyncDelegate(BarrierId, false);

	// Process request
	EBarrierCtrlResult CtrlResult = EBarrierCtrlResult::UnknownError;
	const EDisplayClusterCommResult CommResult = GDisplayCluster->GetPrivateClusterMgr()->GetNetApi().GetGenericBarrierAPI()->ReleaseBarrier(BarrierId, CtrlResult);

	const bool bCtrlOk = (CtrlResult == EBarrierCtrlResult::ReleasedSuccessfully);
	const bool bCommOk = (CommResult == EDisplayClusterCommResult::Ok);
	const bool bBarrierReleased = bCtrlOk && bCommOk;

	return bBarrierReleased;
}

bool FDisplayClusterGenericBarrierAPI::Synchronize(const FString& BarrierId, const FString& CallerId)
{
	using EBarrierCtrlResult = IDisplayClusterProtocolGenericBarrier::EBarrierControlResult;

	// Pass client Id
	FDisplayClusterCtrlContext::Get().GPBClientId = ClientSetId;

	// Process request
	EBarrierCtrlResult CtrlResult = EBarrierCtrlResult::UnknownError;
	const EDisplayClusterCommResult CommResult = GDisplayCluster->GetPrivateClusterMgr()->GetNetApi().GetGenericBarrierAPI()->SyncOnBarrier(BarrierId, CallerId, CtrlResult);

	const bool bCtrlOk = (CtrlResult == EBarrierCtrlResult::SynchronizedSuccessfully);
	const bool bCommOk = (CommResult == EDisplayClusterCommResult::Ok);
	const bool bSynchronized = bCtrlOk && bCommOk;

	return bSynchronized;
}

bool FDisplayClusterGenericBarrierAPI::Synchronize(const FString& BarrierId, const FString& CallerId, const TArray<uint8>& RequestData, TArray<uint8>& OutResponseData)
{
	using EBarrierCtrlResult = IDisplayClusterProtocolGenericBarrier::EBarrierControlResult;

	// Pass client Id
	FDisplayClusterCtrlContext::Get().GPBClientId = ClientSetId;

	// Process request
	EBarrierCtrlResult CtrlResult = EBarrierCtrlResult::UnknownError;
	const EDisplayClusterCommResult CommResult = GDisplayCluster->GetPrivateClusterMgr()->GetNetApi().GetGenericBarrierAPI()->SyncOnBarrierWithData(BarrierId, CallerId, RequestData, OutResponseData, CtrlResult);

	const bool bCtrlOk = (CtrlResult == EBarrierCtrlResult::SynchronizedSuccessfully);
	const bool bCommOk = (CommResult == EDisplayClusterCommResult::Ok);
	const bool bSynchronized = bCtrlOk && bCommOk;

	return bSynchronized;
}

IDisplayClusterGenericBarriersClient::FOnGenericBarrierSynchronizationDelegate* FDisplayClusterGenericBarrierAPI::GetBarrierSyncDelegate(const FString& BarrierId)
{
	FBarrierCallbacksHolder* BarrierCallbacksHolder = BarrierCallbacksMap.Find(BarrierId);
	return BarrierCallbacksHolder ? &BarrierCallbacksHolder->OnGenericBarrierSynchronizationDelegate : nullptr;
}

IDisplayClusterGenericBarriersClient::FOnGenericBarrierTimeoutDelegate* FDisplayClusterGenericBarrierAPI::GetBarrierTimeoutDelegate(const FString& BarrierId)
{
	FBarrierCallbacksHolder* BarrierCallbacksHolder = BarrierCallbacksMap.Find(BarrierId);
	return BarrierCallbacksHolder ? &BarrierCallbacksHolder->OnGenericBarrierTimeoutDelegate : nullptr;
}

TSharedPtr<FDisplayClusterGenericBarrierService> FDisplayClusterGenericBarrierAPI::GetGenericBarrierService() const
{
	TSharedPtr<FDisplayClusterService> Service = GDisplayCluster->GetPrivateClusterMgr()->GetNodeService(UE::nDisplay::Network::Configuration::GenericBarrierServerName).Pin();
	return StaticCastSharedPtr<FDisplayClusterGenericBarrierService>(Service);
}

bool FDisplayClusterGenericBarrierAPI::ConfigureBarrierSyncDelegate(const FString& BarrierId, bool bSetup)
{
	// Once a barrier is created, we can explicitly set custom sync handler to that specific barrier.
	// We need to pick GB service on the p-node, and set the synchronization delegate so it's
	// called when all the barrier clients have arrived.
	if (GDisplayCluster->GetOperationMode() == EDisplayClusterOperationMode::Cluster)
	{
		if (TSharedPtr<FDisplayClusterGenericBarrierService> GBService = GetGenericBarrierService())
		{
			if (TSharedPtr<IDisplayClusterBarrier> Barrier = GBService->GetBarrier(BarrierId))
			{
				// Setup
				if (bSetup)
				{
					if (!BarrierCallbacksMap.Contains(BarrierId))
					{
						BarrierCallbacksMap.Emplace(BarrierId);
						Barrier->GetPreSyncEndDelegate().BindRaw(this, &FDisplayClusterGenericBarrierAPI::OnBarrierSync);
						Barrier->OnBarrierTimeout().AddRaw(this, &FDisplayClusterGenericBarrierAPI::OnBarrierTimeout);
					}
				}
				// Release
				else
				{
					// Unsubscribe first
					Barrier->GetPreSyncEndDelegate().Unbind();
					Barrier->OnBarrierTimeout().RemoveAll(this);

					// Then release the locals
					BarrierCallbacksMap.Remove(BarrierId);
				}

				return true;
			}
		}
	}

	return false;
}

void FDisplayClusterGenericBarrierAPI::OnBarrierSync(FDisplayClusterBarrierPreSyncEndDelegateData& SyncData)
{
	// Process sync callbacks on the primary node only
	const bool bIsPrimary = GDisplayCluster->GetPrivateClusterMgr()->IsPrimary();
	if (!bIsPrimary)
	{
		return;
	}

	// Make sure the delegate is set
	FBarrierCallbacksHolder* const BarrierCallbacks = BarrierCallbacksMap.Find(SyncData.BarrierId);
	if (!BarrierCallbacks || !BarrierCallbacks->OnGenericBarrierSynchronizationDelegate.IsBound())
	{
		return;
	}

	// Access GB server
	TSharedPtr<FDisplayClusterGenericBarrierService> GBService = GetGenericBarrierService();
	checkSlow(GBService);
	if (!GBService)
	{
		return;
	}

	// Get barrier info
	const TSharedPtr<const FDisplayClusterGenericBarrierService::FBarrierInfo> BarrierInfo = GBService->GetBarrierInfo(SyncData.BarrierId);
	checkSlow(BarrierInfo.IsValid());
	if (!BarrierInfo.IsValid())
	{
		return;
	}

	// Now forward data to the handler
	FGenericBarrierSynchronizationDelegateData CallbackData{ SyncData.BarrierId, BarrierInfo->ThreadToNodeMapping, SyncData.RequestData, SyncData.ResponseData };
	BarrierCallbacks->OnGenericBarrierSynchronizationDelegate.Execute(CallbackData);
}

void FDisplayClusterGenericBarrierAPI::OnBarrierTimeout(const FString& BarrierId, const TSet<FString>& NodesTimedOut)
{
	// Process sync callbacks on the primary node only
	const bool bIsPrimary = GDisplayCluster->GetPrivateClusterMgr()->IsPrimary();
	if (!bIsPrimary)
	{
		return;
	}

	// Make sure the delegate is set
	FBarrierCallbacksHolder* const BarrierCallbacks = BarrierCallbacksMap.Find(BarrierId);
	if (!BarrierCallbacks)
	{
		return;
	}

	// Call the delegate
	BarrierCallbacks->OnGenericBarrierTimeoutDelegate.ExecuteIfBound(NodesTimedOut);
}
