// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Cluster/IDisplayClusterGenericBarriersClient.h"

class FDisplayClusterGenericBarrierService;
struct FDisplayClusterBarrierPreSyncEndDelegateData;


/**
 * Generic barriers API
 */
class FDisplayClusterGenericBarrierAPI
	: public IDisplayClusterGenericBarriersClient
{
public:

	FDisplayClusterGenericBarrierAPI();
	~FDisplayClusterGenericBarrierAPI();

public:

	//~ Begin IDisplayClusterGenericBarriersClient
	virtual bool CreateBarrier(const FString& BarrierId, const TMap<FString, TSet<FString>>& NodeToSyncCallers, uint32 Timeout) override;
	virtual bool WaitUntilBarrierIsCreated(const FString& BarrierId) override;
	virtual bool IsBarrierAvailable(const FString& BarrierId) override;
	virtual bool ReleaseBarrier(const FString& BarrierId) override;
	virtual bool Synchronize(const FString& BarrierId, const FString& CallerId) override;
	virtual bool Synchronize(const FString& BarrierId, const FString& CallerId, const TArray<uint8>& RequestData, TArray<uint8>& OutResponseData) override;
	virtual FOnGenericBarrierSynchronizationDelegate* GetBarrierSyncDelegate(const FString& BarrierId) override;
	virtual FOnGenericBarrierTimeoutDelegate* GetBarrierTimeoutDelegate(const FString& BarrierId) override;
	//~ End IDisplayClusterGenericBarriersClient

private:

	/** Provides access to the GB service */
	TSharedPtr<FDisplayClusterGenericBarrierService> GetGenericBarrierService() const;

	/** Setup/release sync delegate for a specific barrier */
	bool ConfigureBarrierSyncDelegate(const FString& BarrierId, bool bSetup);

	/** Callback on barrier sync phase end */
	void OnBarrierSync(FDisplayClusterBarrierPreSyncEndDelegateData& SyncData);

	/** Callback on barrier timeout */
	void OnBarrierTimeout(const FString& BarrierId, const TSet<FString>& NodesTimedOut);

private:

	/**
	 * Aux structure to keep all the delegates/events of a barrier
	 */
	struct FBarrierCallbacksHolder
	{
		/** Synchronization delegate */
		FOnGenericBarrierSynchronizationDelegate OnGenericBarrierSynchronizationDelegate;

		/** Timeout delegate */
		FOnGenericBarrierTimeoutDelegate OnGenericBarrierTimeoutDelegate;
	};

	/** Holds per-barrier delegates/callbacks */
	TMap<FString, FBarrierCallbacksHolder> BarrierCallbacksMap;

private:

	/** Holds client set ID allocated in the cluster controller, and bound to this GPB client. */
	int32 ClientSetId = INDEX_NONE;
};
