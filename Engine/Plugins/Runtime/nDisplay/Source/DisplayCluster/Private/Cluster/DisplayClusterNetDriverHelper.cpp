// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cluster/DisplayClusterNetDriverHelper.h"
#include "Config/IDisplayClusterConfigManager.h"
#include "DisplayClusterConfigurationTypes.h"

#include "Misc/DisplayClusterGlobals.h"
#include "Misc/DisplayClusterStrings.h"

#include "Network/Service/ClusterEventsBinary/DisplayClusterClusterEventsBinaryClient.h"

int32 FDisplayClusterNetDriverHelper::GetNumberOfClusterNodes() const
{
	if (GDisplayCluster == nullptr)
	{
		return 0;
	}

	IDisplayClusterConfigManager* DisplayClusterConfigManager = GDisplayCluster->GetConfigMgr();
	if (DisplayClusterConfigManager == nullptr)
	{
		return 0;
	}

	UDisplayClusterConfigurationData* DisplayClusterConfigurationData = DisplayClusterConfigManager->GetConfig();
	if (DisplayClusterConfigurationData == nullptr)
	{
		return 0;
	}

	if (!IsValid(DisplayClusterConfigurationData->Cluster))
	{
		return 0;
	}

	return DisplayClusterConfigurationData->Cluster->Nodes.Num();
}

FString FDisplayClusterNetDriverHelper::GetPrimaryNodeAddress()
{
	FString NoAddress;

	if (GDisplayCluster == nullptr)
	{
		return NoAddress;
	}

	IDisplayClusterConfigManager* DisplayClusterConfigManager = GDisplayCluster->GetConfigMgr();

	if (DisplayClusterConfigManager == nullptr)
	{
		return NoAddress;
	}

	UDisplayClusterConfigurationData* DisplayClusterConfigurationData = DisplayClusterConfigManager->GetConfig();

	if (DisplayClusterConfigurationData == nullptr)
	{
		return NoAddress;
	}

	if (!IsValid(DisplayClusterConfigurationData->Cluster))
	{
		return NoAddress;
	}

	FString PrimaryNodeId = DisplayClusterConfigurationData->Cluster->PrimaryNode.Id;

	return DisplayClusterConfigurationData->Cluster->GetNode(PrimaryNodeId)->Host;
}

bool FDisplayClusterNetDriverHelper::RegisterClusterEventsBinaryClient(uint32 ClusterId, const FString& ClientAddress, uint16 ClientPort)
{	
	TSharedPtr<FDisplayClusterClusterEventsBinaryClient>* FoundClient = ClusterEventsBinaryClients.Find(ClusterId);

	if (FoundClient != nullptr)
	{
		UE_LOGF(LogDisplayClusterNetwork, Error, "Couldn't not register event binary client for %ls:%d. It already exists.", *ClientAddress, ClientPort);
		return false;
	}
	
	TSharedPtr<FDisplayClusterClusterEventsBinaryClient> Client = MakeShared<FDisplayClusterClusterEventsBinaryClient>(TEXT("CLN_CEB_NetDrv"));
	
	if (!Client->Connect(ClientAddress, ClientPort, 1, 0.f))
	{
		UE_LOGF(LogDisplayClusterNetwork, Error, "Couldn't not connect to %ls:%d", *ClientAddress, ClientPort);

		return false;
	}

	ClusterEventsBinaryClients.Add(ClusterId, Client);

	return true;
}

bool FDisplayClusterNetDriverHelper::RemoveClusterEventsBinaryClient(uint32 СlusterId)
{
	TSharedPtr<FDisplayClusterClusterEventsBinaryClient>* FoundClient = ClusterEventsBinaryClients.Find(СlusterId);

	if (FoundClient == nullptr)
	{
		return false;
	}

	const int32 RemovedEntries = ClusterEventsBinaryClients.Remove(СlusterId);

	return RemovedEntries > 0;
}

bool FDisplayClusterNetDriverHelper::HasClient(uint32 ClusterId)
{
	return ClusterEventsBinaryClients.Contains(ClusterId);
}

bool FDisplayClusterNetDriverHelper::GetRequiredArguments(const FURL& URL, const TCHAR*& OutClusterId, const TCHAR*& OutPrimaryNodeId, const TCHAR*& OutPrimaryNodePort, const TCHAR*& OutClusterNodesNum)
{
	if (!URL.HasOption(DisplayClusterStrings::uri_args::ClusterId) ||
		!URL.HasOption(DisplayClusterStrings::uri_args::PrimaryNodeId) ||
		!URL.HasOption(DisplayClusterStrings::uri_args::PrimaryNodePort) ||
		!URL.HasOption(DisplayClusterStrings::uri_args::ClusterNodesNum))
	{
		return false;
	}

	OutClusterId = URL.GetOption(DisplayClusterStrings::uri_args::ClusterId, nullptr);
	OutPrimaryNodeId = URL.GetOption(DisplayClusterStrings::uri_args::PrimaryNodeId, nullptr);
	OutPrimaryNodePort = URL.GetOption(DisplayClusterStrings::uri_args::PrimaryNodePort, nullptr);
	OutClusterNodesNum = URL.GetOption(DisplayClusterStrings::uri_args::ClusterNodesNum, nullptr);

	return true;
}

bool FDisplayClusterNetDriverHelper::SendCommandToCluster(uint32 ClusterId, const FDisplayClusterClusterEventBinary& Event)
{
	TSharedPtr<FDisplayClusterClusterEventsBinaryClient>* ClientFound = ClusterEventsBinaryClients.Find(ClusterId);

	if (ClientFound == nullptr)
	{
		UE_LOGF(LogDisplayClusterNetwork, Error, "Event binary client not found for ClusterId %d", ClusterId);
		return false;
	}

	TSharedPtr<FDisplayClusterClusterEventsBinaryClient> Client = *ClientFound;

	const EDisplayClusterCommResult CommResult = Client->EmitClusterEventBinary(Event);

	return (CommResult == EDisplayClusterCommResult::Ok);
}

void FDisplayClusterNetDriverHelper::SendCommandToAllClusters(const FDisplayClusterClusterEventBinary& Event)
{
	for (const TPair<uint32, TSharedPtr<FDisplayClusterClusterEventsBinaryClient>>& Client : ClusterEventsBinaryClients)
	{
		Client.Value->EmitClusterEventBinary(Event);
	}
}
