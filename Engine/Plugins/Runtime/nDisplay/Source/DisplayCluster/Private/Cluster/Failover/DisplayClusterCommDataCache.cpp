// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cluster/Failover/DisplayClusterCommDataCache.h"

#include "Cluster/IPDisplayClusterClusterManager.h"

#include "Misc/DisplayClusterGlobals.h"
#include "Misc/DisplayClusterLog.h"

#include "Network/Barrier/IDisplayClusterBarrier.h"
#include "Network/Configuration/DisplayClusterNetworkConfiguration.h"
#include "Network/Service/InternalComm/DisplayClusterInternalCommService.h"

#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"

#include "IDisplayClusterCallbacks.h"


namespace UE::nDisplay::Failover::Private
{
	const FName Name_SlotTime = TEXT("Time");

	const FName Name_SlotObjects_PreTick  = TEXT("Objects_PreTick");
	const FName Name_SlotObjects_Tick     = TEXT("Objects_Tick");
	const FName Name_SlotObjects_PostTick = TEXT("Objects_PostTick");

	const FName Name_SlotEvents      = TEXT("Events");
	const FName Name_SlotNativeInput = TEXT("NativeInput");
}


FDisplayClusterCommDataCache::FDisplayClusterCommDataCache()
{
	// Initialize internal static data
	Initialize_Barriers();
	Initialize_GetTimeData();
	Initialize_GetObjectsData();
	Initialize_GetEventsData();
	Initialize_GetNativeInputData();

	// Set up callbacks
	SubscribeToCallbacks();
}

FDisplayClusterCommDataCache::~FDisplayClusterCommDataCache()
{
	// Unsubscribe from external callbacks
	UnsubscribeFromCallbacks();
}

void FDisplayClusterCommDataCache::Initialize_Barriers()
{
	OpGetBarrierOpen = [this](const FName& BarrierName, const FName& SyncCallerName) -> bool
		{
			// Local counter
			uint64 LocalSyncCount = 0;
			{
				FScopeLock LockNode(&LocalDataCache.BarrierSyncStatesCS);
				LocalSyncCount = LocalDataCache.BarrierSyncStates.FindOrAdd(BarrierName).FindOrAdd(SyncCallerName, 0);
			}

			// Cluster counter
			uint64 ClusterSyncCount = 0;
			{
				FScopeLock LockNode(&ClusterDataCache.BarrierSyncStatesCS);
				ClusterSyncCount = ClusterDataCache.BarrierSyncStates.FindOrAdd(BarrierName).FindOrAdd(SyncCallerName, 0);
			}

			return ClusterSyncCount > LocalSyncCount;
		};

	OpAdvanceBarrierCounter = [this](const FName& BarrierName, const FName& SyncCallerName)
		{
			FScopeLock LockNode(&LocalDataCache.BarrierSyncStatesCS);
			LocalDataCache.BarrierSyncStates.FindOrAdd(BarrierName).FindOrAdd(SyncCallerName, 0)++;
		};
}

void FDisplayClusterCommDataCache::Initialize_GetTimeData()
{
	GetTimeData_OpLoad = [this](double& OutDeltaTime, double& OutGameTime, TOptional<FQualifiedFrameTime>& OutFrameTime)
		{
			OpLoadImpl<FCache_GetTimeData>(UE::nDisplay::Failover::Private::Name_SlotTime, OutDeltaTime, OutGameTime, OutFrameTime);
		};

	GetTimeData_OpSave = [this](double& InDeltaTime, double& InGameTime, TOptional<FQualifiedFrameTime>& InFrameTime)
		{
			OpSaveImpl<FCache_GetTimeData>(UE::nDisplay::Failover::Private::Name_SlotTime, InDeltaTime, InGameTime, InFrameTime);
		};

	GetTimeData_OpIsCached = [this]() -> bool
		{
			return OpIsCachedImpl(UE::nDisplay::Failover::Private::Name_SlotTime);
		};

	// Instantiate the corresponding storage in advance
	LocalDataCache.GameThreadDataCache.Emplace(UE::nDisplay::Failover::Private::Name_SlotTime, MakeUnique<FCache_GetTimeData>());
}

void FDisplayClusterCommDataCache::Initialize_GetObjectsData()
{
	GetObjectsData_OpLoad = [this](const EDisplayClusterSyncGroup InSyncGroup, TMap<FString, TArray<uint8>>& OutObjectsData)
		{
			OpLoadImpl<FCache_GetObjectsData>(GetObjectsDataSlotName(InSyncGroup), OutObjectsData);
		};

	GetObjectsData_OpSave = [this](const EDisplayClusterSyncGroup InSyncGroup, TMap<FString, TArray<uint8>>& InObjectsData)
		{
			OpSaveImpl<FCache_GetObjectsData>(GetObjectsDataSlotName(InSyncGroup), InObjectsData);
		};

	GetObjectsData_OpIsCached = [this](const EDisplayClusterSyncGroup InSyncGroup) -> FOpIsCached
		{
			return [this, SyncGroup = InSyncGroup]() -> bool { return OpIsCachedImpl(GetObjectsDataSlotName(SyncGroup)); };
		};

	// Instantiate the corresponding storages in advance (each sync group has its own slot)
	LocalDataCache.GameThreadDataCache.Emplace(GetObjectsDataSlotName(EDisplayClusterSyncGroup::PreTick),  MakeUnique<FCache_GetObjectsData>());
	LocalDataCache.GameThreadDataCache.Emplace(GetObjectsDataSlotName(EDisplayClusterSyncGroup::Tick),     MakeUnique<FCache_GetObjectsData>());
	LocalDataCache.GameThreadDataCache.Emplace(GetObjectsDataSlotName(EDisplayClusterSyncGroup::PostTick), MakeUnique<FCache_GetObjectsData>());
}

void FDisplayClusterCommDataCache::Initialize_GetEventsData()
{
	GetEventsData_OpLoad = [this](TArray<TSharedPtr<FDisplayClusterClusterEventJson>>& OutJsonEvents, TArray<TSharedPtr<FDisplayClusterClusterEventBinary>>& OutBinaryEvents)
		{
			OpLoadImpl<FCache_GetEventsData>(UE::nDisplay::Failover::Private::Name_SlotEvents, OutJsonEvents, OutBinaryEvents);
		};

	GetEventsData_OpSave = [this](TArray<TSharedPtr<FDisplayClusterClusterEventJson>>& InJsonEvents, TArray<TSharedPtr<FDisplayClusterClusterEventBinary>>& InBinaryEvents)
		{
			OpSaveImpl<FCache_GetEventsData>(UE::nDisplay::Failover::Private::Name_SlotEvents, InJsonEvents, InBinaryEvents);
		};

	GetEventsData_OpIsCached = [this]() -> bool
		{
			return OpIsCachedImpl(UE::nDisplay::Failover::Private::Name_SlotEvents);
		};

	// Instantiate the corresponding storage in advance
	LocalDataCache.GameThreadDataCache.Emplace(UE::nDisplay::Failover::Private::Name_SlotEvents, MakeUnique<FCache_GetEventsData>());
}

void FDisplayClusterCommDataCache::Initialize_GetNativeInputData()
{
	GetNativeInputData_OpLoad = [this](TMap<FString, TArray<uint8>>& OutNativeInputData)
		{
			OpLoadImpl<FCache_GetNativeInputData>(UE::nDisplay::Failover::Private::Name_SlotNativeInput, OutNativeInputData);
		};

	GetNativeInputData_OpSave = [this](TMap<FString, TArray<uint8>>& InNativeInputData)
		{
			OpSaveImpl<FCache_GetNativeInputData>(UE::nDisplay::Failover::Private::Name_SlotNativeInput, InNativeInputData);
		};

	GetNativeInputData_OpIsCached = [this]() -> bool
		{
			return OpIsCachedImpl(UE::nDisplay::Failover::Private::Name_SlotNativeInput);
		};

	// Instantiate the corresponding storage in advance
	LocalDataCache.GameThreadDataCache.Emplace(UE::nDisplay::Failover::Private::Name_SlotNativeInput, MakeUnique<FCache_GetNativeInputData>());
}

FName FDisplayClusterCommDataCache::GetObjectsDataSlotName(const EDisplayClusterSyncGroup InSyncGroup) const
{
	switch (InSyncGroup)
	{
	case EDisplayClusterSyncGroup::PreTick:
		return UE::nDisplay::Failover::Private::Name_SlotObjects_PreTick;

	case EDisplayClusterSyncGroup::Tick:
		return UE::nDisplay::Failover::Private::Name_SlotObjects_Tick;

	case EDisplayClusterSyncGroup::PostTick:
		return UE::nDisplay::Failover::Private::Name_SlotObjects_PostTick;

	default:
		unimplemented();
	}

	return NAME_None;
}

bool FDisplayClusterCommDataCache::OpIsCachedImpl(const FName& SlotName) const
{
	// Check local cache first as the most used
	{
		FScopeLock Lock(&LocalDataCache.GameThreadDataCacheCS);

		if (const FCyclicDataCacheBase* const LocalData = LocalDataCache.GetGTSlotData(SlotName))
		{
			if (LocalData->bCached)
			{
				return true;
			}
		}
		else
		{
			UE_LOGF(LogDisplayClusterFailover, Warning, "Local cache doesn't have a '%ls' slot", *SlotName.ToString());
			checkNoEntry();
		}
	}

	// If not found in local, let's see if there is anything in cluster
	{
		FScopeLock Lock(&ClusterDataCache.GameThreadDataCacheCS);

		if (const FCyclicDataCacheBase* const ClusterData = ClusterDataCache.GetGTSlotData(SlotName))
		{
			if (ClusterData->bCached)
			{
				return true;
			}
		}
	}

	return false;
}

template <typename TSlotDataType, typename... Args>
void FDisplayClusterCommDataCache::OpLoadImpl(const FName& SlotName, Args&... SlotData)
{
	// Check cluster cache first as higher priority
	{
		FScopeLock Lock(&ClusterDataCache.GameThreadDataCacheCS);

		if (FCyclicDataCacheBase* const ClusterData = ClusterDataCache.GetGTSlotData(SlotName))
		{
			if (ClusterData->bCached)
			{
				static_cast<TSlotDataType*>(ClusterData)->CopyData(true, SlotData...);
				return;
			}
		}
	}

	// Otherwise, load from local cache
	{
		FScopeLock Lock(&LocalDataCache.GameThreadDataCacheCS);

		if (FCyclicDataCacheBase* const LocalData = LocalDataCache.GetGTSlotData(SlotName))
		{
			if (LocalData->bCached)
			{
				static_cast<TSlotDataType*>(LocalData)->CopyData(true, SlotData...);
				return;
			}
		}
	}

	UE_LOGF(LogDisplayClusterFailover, Warning, "No cached data found for '%ls'", *SlotName.ToString());
	checkNoEntry();
}

template <typename TSlotDataType, typename... Args>
void FDisplayClusterCommDataCache::OpSaveImpl(const FName& SlotName, Args&... SlotData)
{
	// Always store data to the local cache
	{
		FScopeLock Lock(&LocalDataCache.GameThreadDataCacheCS);

		if (FCyclicDataCacheBase* const LocalData = LocalDataCache.GetGTSlotData(SlotName))
		{
			TSlotDataType* const LocalDataCasted = static_cast<TSlotDataType*>(LocalData);
			LocalDataCasted->CopyData(false, SlotData...);
			LocalDataCasted->bCached = true;
			return;
		}

		ensureMsgf(false, TEXT("Local cache doesn't have a '%s' slot"), *SlotName.ToString());
	}
}

void FDisplayClusterCommDataCache::SubscribeToCallbacks()
{
	if (!GDisplayCluster)
	{
		return;
	}

	if (IDisplayCluster::IsAvailable())
	{
		// DCEndFrame is used to invalidate per-frame cache
		IDisplayCluster::Get().GetCallbacks().OnDisplayClusterEndFrame().AddRaw(this, &FDisplayClusterCommDataCache::ProcessDCEndFrame);
	}

	// Set up a post-failure negotiation delegate
	{
		bool bSubscribedToNegotiationSync = false;

		// Get InternalComm service
		TSharedPtr<FDisplayClusterService> Service = GDisplayCluster->GetPrivateClusterMgr()->GetNodeService(UE::nDisplay::Network::Configuration::InternalCommServerName).Pin();
		if (TSharedPtr<FDisplayClusterInternalCommService> ICService = StaticCastSharedPtr<FDisplayClusterInternalCommService>(Service))
		{
			// Get corresponding barrier
			if (TSharedPtr<IDisplayClusterBarrier> Barrier = ICService->GetPostFailureNegotiationBarrier())
			{
				// Set up a delegate
				Barrier->GetPreSyncEndDelegate().BindRaw(this, &FDisplayClusterCommDataCache::OnPostFailureBarrierSync);
				bSubscribedToNegotiationSync = true;
			}
		}

		if (!bSubscribedToNegotiationSync)
		{
			UE_LOGF(LogDisplayClusterFailover, Warning, "Couldn't set up a post-failure negotiation delegate");
		}
	}
}

void FDisplayClusterCommDataCache::UnsubscribeFromCallbacks()
{
	if (!GDisplayCluster)
	{
		return;
	}

	if (IDisplayCluster::IsAvailable())
	{
		// Unsubscribe from DCEndFrame
		IDisplayCluster::Get().GetCallbacks().OnDisplayClusterEndFrame().RemoveAll(this);
	}

	// Clear post-failure negotiation delegate
	bool bSubscribedToNegotiationSync = false;
	TSharedPtr<FDisplayClusterService> Service = GDisplayCluster->GetPrivateClusterMgr()->GetNodeService(UE::nDisplay::Network::Configuration::InternalCommServerName).Pin();
	if (TSharedPtr<FDisplayClusterInternalCommService> ICService = StaticCastSharedPtr<FDisplayClusterInternalCommService>(Service))
	{
		if (TSharedPtr<IDisplayClusterBarrier> Barrier = ICService->GetPostFailureNegotiationBarrier())
		{
			Barrier->GetPreSyncEndDelegate().Unbind();
		}
	}
}

void FDisplayClusterCommDataCache::ProcessDCEndFrame(uint64 FrameNum)
{
	// Invalidate game thread cache (cache ready flags only)
	LocalDataCache.InvalidateGameThreadData();

	// Invalidate cluster cache (full reset to optimize the cache operations)
	ClusterDataCache.InvalidateGameThreadData(true);
}

void FDisplayClusterCommDataCache::OnPostFailureBarrierSync(FDisplayClusterBarrierPreSyncEndDelegateData& SyncData)
{
	UE_LOGF(LogDisplayClusterFailover, Log, "Post-failure recovery. Building actual sync state...");

	// Build cluster sync state based on the sync states of all nodes
	BuildClusterSyncState(SyncData.RequestData, SyncData.ResponseData);
}

void FDisplayClusterCommDataCache::TempWorkaround_ResetTimeDataCache()
{
	// Manually reset TimeData slot only
	FScopeLock Lock(&LocalDataCache.GameThreadDataCacheCS);
	if (FDisplayClusterCommDataCache::FCyclicDataCacheBase* const TimeDataCache = LocalDataCache.GetGTSlotData(UE::nDisplay::Failover::Private::Name_SlotTime))
	{
		TimeDataCache->Reset();
	}
}

void FDisplayClusterCommDataCache::GenerateNodeSyncState(TArray<uint8>& OutNodeSyncState)
{
	OutNodeSyncState.Empty(4096);

	FScopeLock Lock(&LocalDataCache.GameThreadDataCacheCS);

	UE_LOGF(LogDisplayClusterFailover, Verbose, "Post-failure recovery. Local sync state:\n%ls", *LocalDataCache.ToLogString());

	// Serialize local state
	FMemoryWriter MemoryWriter(OutNodeSyncState);
	LocalDataCache.Serialize(MemoryWriter);
}

void FDisplayClusterCommDataCache::UpdateClusterSyncState(const TArray<uint8>& InClusterSyncState)
{
	// Just deserialize into cluster state holder
	FMemoryReader MemoryReader(InClusterSyncState);
	ClusterDataCache.Serialize(MemoryReader);

	UE_LOGF(LogDisplayClusterFailover, Verbose, "Post-failure recovery. Got cluster sync state:\n%ls", *ClusterDataCache.ToLogString());
}

void FDisplayClusterCommDataCache::BuildClusterSyncState(const TMap<FString, TArray<uint8>>& RequestData, TMap<FString, TArray<uint8>>& ResponseData)
{
	// Deserialize all node states
	UE_LOGF(LogDisplayClusterFailover, Log, "\n Post-failure synchronization \n\nInput states: \n");
	TMap<FString, FDataCacheHolder> NodeSyncStates;
	for (const TPair<FString, TArray<uint8>>& NodeRequest : RequestData)
	{
		FMemoryReader MemoryReader(NodeRequest.Value);
		FDataCacheHolder& NodeSyncState = NodeSyncStates.Emplace(NodeRequest.Key, 0);
		NodeSyncState.Serialize(MemoryReader);
		UE_LOGF(LogDisplayClusterFailover, Log, " * %ls: %ls\n", *NodeRequest.Key, *NodeSyncState.ToLogString());
	}

	// Cluster sync state. This one gonna be the post-failure negotiation outcome.
	FDataCacheHolder ClusterSyncData;

	// Holds maximum synchronization count for every barrier
	TMap<FName, uint64> MaxBarrierSyncCount;

	// Iterate over each node state
	for (TPair<FString, FDataCacheHolder>& NodeSyncState : NodeSyncStates)
	{
		//////////////////////////////////////////////////////////////////////////////////
		// Process game thread data and build GT cluster sync state. Basically, we union
		// multiple sync data sets to get the most recent and actual state.

		// Iterate over data slots
		for (TPair<FName, TUniquePtr<FCyclicDataCacheBase>>& DataSlot : NodeSyncState.Value.GameThreadDataCache)
		{
			TUniquePtr<FCyclicDataCacheBase>* const FoundData = ClusterSyncData.GameThreadDataCache.Find(DataSlot.Key);

			// Update output slot if:
			// - output slot is not exists
			// - output slot has non-cached data, input slot has cached data
			if (!FoundData || (!FoundData->Get()->bCached && DataSlot.Value->bCached))
			{
				// Save it to the final outcome
				ClusterSyncData.GameThreadDataCache.Emplace(DataSlot.Key, MoveTemp(DataSlot.Value));
			}
		}

		//////////////////////////////////////////////////////////////////////////////////
		// Process barrier sync states as well

		// Now we unite callers of each barrier into the output map, and find max sync count of every barrier.
		// Iterate over every barrier
		for (const TPair<FName, TMap<FName, uint64>>& BarrierSyncState : NodeSyncState.Value.BarrierSyncStates)
		{
			// Iterate over every sync caller of this particular barrier
			TMap<FName, uint64>& OutputBarrierSyncState = ClusterSyncData.BarrierSyncStates.FindOrAdd(BarrierSyncState.Key);
			uint64& MaxSyncCount = MaxBarrierSyncCount.FindOrAdd(BarrierSyncState.Key, 0);

			// Go through every sync caller, and find the final sync count that will be an outcome
			for (const TPair<FName, uint64>& CallerSyncState : BarrierSyncState.Value)
			{
				// Store maximum sync count value
				MaxSyncCount = FMath::Max(MaxSyncCount, CallerSyncState.Value);
				OutputBarrierSyncState.FindOrAdd(CallerSyncState.Key, 0);
			}
		}
	}

	// Now set maximum sync count for every caller
	for (TPair<FName, TMap<FName, uint64>>& OutputBarrierSyncState: ClusterSyncData.BarrierSyncStates)
	{
		const uint64& MaxCount = MaxBarrierSyncCount.FindOrAdd(OutputBarrierSyncState.Key, 0);
		for (TPair<FName, uint64>& CallerSyncState : OutputBarrierSyncState.Value)
		{
			CallerSyncState.Value = MaxCount;
		}
	}


	UE_LOGF(LogDisplayClusterFailover, Log, "\nOutput state: \n [%ls]", *ClusterSyncData.ToLogString());

	// Serialize the final cluster state
	TArray<uint8> GeneratedResponseData;
	FMemoryWriter MemoryWriter(GeneratedResponseData);
	ClusterSyncData.Serialize(MemoryWriter);

	// Fill per-node response
	ResponseData.Empty(RequestData.Num());
	for (const TPair<FString, TArray<uint8>>& NodeRequest : RequestData)
	{
		ResponseData.Emplace(NodeRequest.Key, CopyTemp(GeneratedResponseData));
	}
}

void FDisplayClusterCommDataCache::FCyclicDataCacheBase::Serialize(FArchive& Ar)
{
	Ar << bCached;
}

void FDisplayClusterCommDataCache::FCyclicDataCacheBase::Reset()
{
	bCached = false;
}

FString FDisplayClusterCommDataCache::FCache_GetTimeData::ToLogString() const
{
	const FQualifiedFrameTime QFT = FrameTime.Get({});
	return FString::Printf(TEXT("TD[%s]: DeltaTime=%f, GameTime=%f, FrameTime=[Rate=%d/%d, Time=%d@%f}"),
		bCached ? TEXT("+") : TEXT("-"),
		DeltaTime, GameTime,
		QFT.Rate.Numerator, QFT.Rate.Denominator,
		QFT.Time.FrameNumber.Value, QFT.Time.MaxSubframe);
}

void FDisplayClusterCommDataCache::FCache_GetTimeData::Serialize(FArchive& Ar)
{
	FCyclicDataCacheBase::Serialize(Ar);

	Ar << DeltaTime;
	Ar << GameTime;

	bool bFrameTimeSet = false;
	
	if (Ar.IsSaving())
	{
		bFrameTimeSet = FrameTime.IsSet();
		Ar << bFrameTimeSet;

		if (bFrameTimeSet)
		{
			Ar << FrameTime.GetValue().Time;
			Ar << FrameTime.GetValue().Rate;
		}
	}
	else
	{
		Ar << bFrameTimeSet;

		if (bFrameTimeSet)
		{
			FFrameTime Time;
			Ar << Time;

			FFrameRate Rate;
			Ar << Rate;

			FrameTime = FQualifiedFrameTime(Time, Rate);
		}
	}
}

void FDisplayClusterCommDataCache::FCache_GetTimeData::Reset()
{
	FCyclicDataCacheBase::Reset();

	DeltaTime = 0;
	GameTime  = 0;
	FrameTime.Reset();
}

FString FDisplayClusterCommDataCache::FCache_GetObjectsData::ToLogString() const
{
	FString Output;
	Output.Reserve(1024);

	Output = FString::Printf(TEXT("OD[%s]: "), bCached ? TEXT("+") : TEXT("-"));
	int32 Idx = 0;
	for (const TPair<FString, TArray<uint8>>& DataIt : ObjData)
	{
		Output += FString::Printf(TEXT("<%d : %s=[%d bytes]> "), Idx++, *DataIt.Key, DataIt.Value.Num());
	}

	return Output;
}

void FDisplayClusterCommDataCache::FCache_GetObjectsData::Serialize(FArchive& Ar)
{
	FCyclicDataCacheBase::Serialize(Ar);

	Ar << ObjData;
}

void FDisplayClusterCommDataCache::FCache_GetObjectsData::Reset()
{
	FCyclicDataCacheBase::Reset();
	ObjData.Reset();
}

FString FDisplayClusterCommDataCache::FCache_GetEventsData::ToLogString() const
{
	return FString::Printf(TEXT("ED[%s]: json_num=%d, bin_num=%d"), bCached ? TEXT("+") : TEXT("-"), JsonEvents.Num(), BinaryEvents.Num());
}

void FDisplayClusterCommDataCache::FCache_GetEventsData::Serialize(FArchive& Ar)
{
	FCyclicDataCacheBase::Serialize(Ar);

	int32 JsonEventsNum = JsonEvents.Num();
	Ar << JsonEventsNum;

	JsonEvents.Reserve(JsonEventsNum);

	for (int32 Idx = 0; Idx < JsonEventsNum; ++Idx)
	{
		if (Ar.IsSaving())
		{
			JsonEvents[Idx]->Serialize(Ar);
		}
		else
		{
			TSharedPtr<FDisplayClusterClusterEventJson> JsonEvent = MakeShared<FDisplayClusterClusterEventJson>();
			JsonEvent->Serialize(Ar);
			JsonEvents.Add(JsonEvent);
		}
	}

	int32 BinaryEventsNum = BinaryEvents.Num();
	Ar << BinaryEventsNum;

	BinaryEvents.Reserve(BinaryEventsNum);

	for (int32 Idx = 0; Idx < BinaryEventsNum; ++Idx)
	{
		if (Ar.IsSaving())
		{
			BinaryEvents[Idx]->Serialize(Ar);
		}
		else
		{
			TSharedPtr<FDisplayClusterClusterEventBinary> BinaryEvent = MakeShared<FDisplayClusterClusterEventBinary>();
			BinaryEvent->Serialize(Ar);
			BinaryEvents.Add(BinaryEvent);
		}
	}
}

void FDisplayClusterCommDataCache::FCache_GetEventsData::Reset()
{
	FCyclicDataCacheBase::Reset();
	JsonEvents.Reset();
	BinaryEvents.Reset();
}

FString FDisplayClusterCommDataCache::FCache_GetNativeInputData::ToLogString() const
{
	FString Output;
	Output.Reserve(1024);

	Output = FString::Printf(TEXT("ID[%s]: "), bCached ? TEXT("+") : TEXT("-"));
	int32 Idx = 0;
	for (const TPair<FString, TArray<uint8>>& DataIt : NativeInputData)
	{
		Output += FString::Printf(TEXT("<%d : %s -> %d bytes> "), Idx++, *DataIt.Key, DataIt.Value.Num());
	}

	return Output;
}

void FDisplayClusterCommDataCache::FCache_GetNativeInputData::Serialize(FArchive& Ar)
{
	FCyclicDataCacheBase::Serialize(Ar);

	Ar << NativeInputData;
}

void FDisplayClusterCommDataCache::FCache_GetNativeInputData::Reset()
{
	FCyclicDataCacheBase::Reset();
	NativeInputData.Reset();
}

void FDisplayClusterCommDataCache::FDataCacheHolder::InvalidateGameThreadData(bool bReset)
{
	FScopeLock Lock(&GameThreadDataCacheCS);

	// Full invalidation by releasing data
	if (bReset)
	{
		GameThreadDataCache.Reset();
	}
	// Reset data slots only
	else
	{
		for (TPair<FName, TUniquePtr<FCyclicDataCacheBase>>& CacheSlot : GameThreadDataCache)
		{
			if (CacheSlot.Value)
			{
				CacheSlot.Value->Reset();
			}
		}
	}
}

FDisplayClusterCommDataCache::FCyclicDataCacheBase* FDisplayClusterCommDataCache::FDataCacheHolder::GetGTSlotData(const FName& SlotName)
{
	const TUniquePtr<FDisplayClusterCommDataCache::FCyclicDataCacheBase>* const SlotDataPtr = GameThreadDataCache.Find(SlotName);
	return SlotDataPtr ? SlotDataPtr->Get() : nullptr;
}

const FDisplayClusterCommDataCache::FCyclicDataCacheBase* FDisplayClusterCommDataCache::FDataCacheHolder::GetGTSlotData(const FName& SlotName) const
{
	const TUniquePtr<FDisplayClusterCommDataCache::FCyclicDataCacheBase>* const SlotDataPtr = GameThreadDataCache.Find(SlotName);
	return SlotDataPtr ? SlotDataPtr->Get() : nullptr;
}

FString FDisplayClusterCommDataCache::FDataCacheHolder::ToLogString() const
{
	FString LogStr;
	LogStr.Reserve(1024);


	LogStr = TEXT("\n\tGameThread data:\n");
	{
		const FName NamesInCallOrder[] =
		{
			UE::nDisplay::Failover::Private::Name_SlotTime,
			UE::nDisplay::Failover::Private::Name_SlotObjects_PreTick,
			UE::nDisplay::Failover::Private::Name_SlotEvents,
			UE::nDisplay::Failover::Private::Name_SlotNativeInput,
			UE::nDisplay::Failover::Private::Name_SlotObjects_Tick,
			UE::nDisplay::Failover::Private::Name_SlotObjects_PostTick
		};

		FScopeLock Lock(&GameThreadDataCacheCS);

		for (const FName& SlotName : NamesInCallOrder)
		{
			const TUniquePtr<FCyclicDataCacheBase>* SlotData = GameThreadDataCache.Find(SlotName);

			if (SlotData && SlotData->IsValid())
			{
				LogStr += FString::Printf(TEXT("\t\t%s\n"), *SlotData->Get()->ToLogString());
			}
			else
			{
				checkfSlow(false, TEXT("Found an uninitialized slot '%s'"), *SlotName.ToString());
			}
		}
	}

	LogStr += TEXT("\n\tBarrier sync states:\n");
	{
		FScopeLock Lock(&BarrierSyncStatesCS);

		for (const TPair<FName, TMap<FName, uint64>>& BarrierSlot : BarrierSyncStates)
		{
			LogStr += FString::Printf(TEXT("\t\t> %s - %d callers\n"), *BarrierSlot.Key.ToString(), BarrierSlot.Value.Num());
			for (const TPair<FName, uint64>& BarrierCaller : BarrierSlot.Value)
			{
				LogStr += FString::Printf(TEXT("\t\t\t%s=%llu\n"), *BarrierCaller.Key.ToString(), BarrierCaller.Value);
			}
		}
	}

	return LogStr;
}

void FDisplayClusterCommDataCache::FDataCacheHolder::Serialize(FArchive& Ar)
{
	FScopeLock Lock1(&GameThreadDataCacheCS);
	FScopeLock Lock2(&BarrierSyncStatesCS);

	// GetTimeData
	{
		FCyclicDataCacheBase* const BaseData = GameThreadDataCache.FindOrAdd(UE::nDisplay::Failover::Private::Name_SlotTime, MakeUnique<FCache_GetTimeData>()).Get();
		FCache_GetTimeData* const Data = static_cast<FCache_GetTimeData*>(BaseData);
		checkSlow(Data);
		Data->Serialize(Ar);
	}

	// GetObjectsData - PreTick
	{
		FCyclicDataCacheBase* const BaseData = GameThreadDataCache.FindOrAdd(UE::nDisplay::Failover::Private::Name_SlotObjects_PreTick, MakeUnique<FCache_GetObjectsData>()).Get();
		FCache_GetObjectsData* const Data = static_cast<FCache_GetObjectsData*>(BaseData);
		checkSlow(Data);
		Data->Serialize(Ar);
	}

	// GetObjectsData - Tick
	{
		FCyclicDataCacheBase* const BaseData = GameThreadDataCache.FindOrAdd(UE::nDisplay::Failover::Private::Name_SlotObjects_Tick, MakeUnique<FCache_GetObjectsData>()).Get();
		FCache_GetObjectsData* const Data = static_cast<FCache_GetObjectsData*>(BaseData);
		checkSlow(Data);
		Data->Serialize(Ar);
	}

	// GetObjectsData - PostTick
	{
		FCyclicDataCacheBase* const BaseData = GameThreadDataCache.FindOrAdd(UE::nDisplay::Failover::Private::Name_SlotObjects_PostTick, MakeUnique<FCache_GetObjectsData>()).Get();
		FCache_GetObjectsData* const Data = static_cast<FCache_GetObjectsData*>(BaseData);
		checkSlow(Data);
		Data->Serialize(Ar);
	}

	// GetEventsData
	{
		FCyclicDataCacheBase* const BaseData = GameThreadDataCache.FindOrAdd(UE::nDisplay::Failover::Private::Name_SlotEvents, MakeUnique<FCache_GetEventsData>()).Get();
		FCache_GetEventsData* const Data = static_cast<FCache_GetEventsData*>(BaseData);
		checkSlow(Data);
		Data->Serialize(Ar);
	}

	// GetNativeInputData
	{
		FCyclicDataCacheBase* const BaseData = GameThreadDataCache.FindOrAdd(UE::nDisplay::Failover::Private::Name_SlotNativeInput, MakeUnique<FCache_GetNativeInputData>()).Get();
		FCache_GetNativeInputData* const Data = static_cast<FCache_GetNativeInputData*>(BaseData);
		checkSlow(Data);
		Data->Serialize(Ar);
	}

	// Barriers state
	{
		Ar << BarrierSyncStates;
	}
}
