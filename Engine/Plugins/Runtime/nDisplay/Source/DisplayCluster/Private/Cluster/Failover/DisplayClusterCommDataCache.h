// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Cluster/DisplayClusterClusterEvent.h"
#include "HAL/Event.h"

#include "Misc/Optional.h"
#include "Misc/QualifiedFrameTime.h"

#include "Templates/UniquePtr.h"

#include "DisplayClusterEnums.h"

struct FDisplayClusterBarrierPreSyncEndDelegateData;


/**
 * Communication data cache
 * 
 * This class is a mediator in communication between cluster nodes. The main purpose of it:
 *  - To help the failover controller to deal with transactions
 *  - To cache data that will be exported to other nodes
 *  - To catch up missing data during failover
 */
class FDisplayClusterCommDataCache
{
public:

	FDisplayClusterCommDataCache();
	virtual ~FDisplayClusterCommDataCache();

public:

	/** Common types */
	using FOpIsCached = TFunction<bool(void)>;

public:

	/** Returns true if time data has been cached for current frame */
	FOpIsCached GetTimeData_OpIsCached;

	/** Loads time data from cache */
	TFunction<void(double& OutDeltaTime, double& OutGameTime, TOptional<FQualifiedFrameTime>& OutFrameTime)> GetTimeData_OpLoad;

	/** Caches time data */
	TFunction<void(double& InDeltaTime, double& InGameTime, TOptional<FQualifiedFrameTime>& InFrameTime)> GetTimeData_OpSave;

public:

	/** Returns true if sync objects data has been cached for current frame */
	TFunction<FOpIsCached(const EDisplayClusterSyncGroup InSyncGroup)> GetObjectsData_OpIsCached;

	/** Loads sync objects data from cache */
	TFunction<void(const EDisplayClusterSyncGroup InSyncGroup, TMap<FString, TArray<uint8>>& OutObjectsData)> GetObjectsData_OpLoad;

	/** Caches sync objects data */
	TFunction<void(const EDisplayClusterSyncGroup InSyncGroup, TMap<FString, TArray<uint8>>& InObjectsData)> GetObjectsData_OpSave;

public:

	/** Returns true if events data has been cached for current frame */
	FOpIsCached GetEventsData_OpIsCached;

	/** Loads events data from cache */
	TFunction<void(TArray<TSharedPtr<FDisplayClusterClusterEventJson>>& OutJsonEvents, TArray<TSharedPtr<FDisplayClusterClusterEventBinary>>& OutBinaryEvents)> GetEventsData_OpLoad;

	/** Caches events data */
	TFunction<void(TArray<TSharedPtr<FDisplayClusterClusterEventJson>>& InJsonEvents, TArray<TSharedPtr<FDisplayClusterClusterEventBinary>>& InBinaryEvents)> GetEventsData_OpSave;

public:

	/** Returns true if input data has been cached for current frame */
	FOpIsCached GetNativeInputData_OpIsCached;

	/** Loads input data from cache */
	TFunction<void(TMap<FString, TArray<uint8>>& OutNativeInputData)> GetNativeInputData_OpLoad;

	/** Caches input data */
	TFunction<void(TMap<FString, TArray<uint8>>& InNativeInputData)> GetNativeInputData_OpSave;

public:

	/** Returns true if specified barrier is considered open for a specific caller from the cluster POV */
	TFunction<bool(const FName& BarrierName, const FName& SyncCallerName)> OpGetBarrierOpen;

	/** Incrmements barrier sync counter */
	TFunction<void(const FName& BarrierName, const FName& SyncCallerName)> OpAdvanceBarrierCounter;

public:

	/**
	 * [Temporary Workaround]
	 * 
	 * Reset time data cache manually
	 */
	void TempWorkaround_ResetTimeDataCache();

protected:

	/** Generates and exports the state of synchronization of this node */
	void GenerateNodeSyncState(TArray<uint8>& OutNodeSyncState);

	/** Update cluster synchronization state after post-failure negotiation */
	void UpdateClusterSyncState(const TArray<uint8>& ClusterSyncState);

	/** Summarizes abstract synchrnization state of the cluster based on the sync states of every node */
	void BuildClusterSyncState(const TMap<FString, TArray<uint8>>& RequestData, TMap<FString, TArray<uint8>>& ResponseData);

private:

	/** Initializes internal barrier cache */
	void Initialize_Barriers();

	/** Initializes internal GetTimeData function cache */
	void Initialize_GetTimeData();

	/** Initializes internal GetObjectsData function cache */
	void Initialize_GetObjectsData();

	/** Initializes internal GetEventsData function cache */
	void Initialize_GetEventsData();

	/** Initializes internal GetNativeInputData function cache */
	void Initialize_GetNativeInputData();

private:

	/** Returns GetObjectsData slot name based on sync group (each group is stored separately) */
	FName GetObjectsDataSlotName(const EDisplayClusterSyncGroup InSyncGroup) const;

private:

	/** This is a generic implementation of "Is slot cached" algorithm */
	bool OpIsCachedImpl(const FName& SlotName) const;

	/** This is a generic implementation of "Load data from a slot" algorithm */
	template <typename TSlotDataType, typename... Args>
	void OpLoadImpl(const FName& SlotName, Args&...);

	/** This is a generic implementation of "Save data to a slot" algorithm */
	template <typename TSlotDataType, typename... Args>
	void OpSaveImpl(const FName& SlotName, Args&...);

private:

	/** Subscribes to external callbacks */
	void SubscribeToCallbacks();

	/** Unsubscribes from external callbacks */
	void UnsubscribeFromCallbacks();

	/** EndFrame callback handler to invalidate game thread bound cache */
	void ProcessDCEndFrame(uint64 FrameNum);

	/** PostFailure negotiation sync callback. Called from the corresponding barrier on P-nodes only. */
	void OnPostFailureBarrierSync(FDisplayClusterBarrierPreSyncEndDelegateData& SyncData);

private:

	/**
	 * Base class for thread cyclic cache data types
	 */
	struct FCyclicDataCacheBase
	{
	public:

		/** Virtual destructor for proper resource cleaning */
		virtual ~FCyclicDataCacheBase() = default;

	public:

		/** Generates log string */
		virtual FString ToLogString() const = 0;

		/** Serialization */
		virtual void Serialize(FArchive& Ar);

		/** Release internals and/or reset to default */
		virtual void Reset();

	public:

		/** Cache state flag (true => cached) */
		bool bCached = false;
	};

private: // GetTimeData

	/**
	 * GetTimeData cache
	 * 
	 * Caches time data for current frame
	 */
	struct FCache_GetTimeData : public FCyclicDataCacheBase
	{
	public:

		/** GetTimeData parameter cache */
		double DeltaTime = 0;

		/** GetTimeData parameter cache */
		double GameTime  = 0;

		/** GetTimeData parameter cache */
		TOptional<FQualifiedFrameTime> FrameTime;

	public:

		/** Copies data TO or FROM */
		void CopyData(bool bCopyOutside, double& DeltaTimeRef, double& GameTimeRef, TOptional<FQualifiedFrameTime>& FrameTimeRef)
		{
			if (bCopyOutside)
			{
				DeltaTimeRef = DeltaTime;
				GameTimeRef  = GameTime;
				FrameTimeRef = FrameTime;
			}
			else
			{
				DeltaTime = DeltaTimeRef;
				GameTime  = GameTimeRef;
				FrameTime = FrameTimeRef;
			}
		}

		//~ Begin FCyclicDataCacheBase
		virtual FString ToLogString() const override;
		virtual void Serialize(FArchive& Ar) override;
		virtual void Reset() override;
		//~ End FCyclicDataCacheBase

		/** Serialization */
		friend FArchive& operator<<(FArchive& Ar, FCache_GetTimeData& Instance)
		{
			Instance.Serialize(Ar);
			return Ar;
		}
	};

private: // GetObjectsData

	/**
	 * GetObjectsData cache
	 * 
	 * Caches custom objects data of a specified sync group for current frame
	 */
	struct FCache_GetObjectsData : public FCyclicDataCacheBase
	{
	public:

		/** Object data of a dedicated synchronization group */
		TMap<FString, TArray<uint8>> ObjData;

	public:

		/** Copies data TO or FROM */
		void CopyData(bool bCopyOutside, TMap<FString, TArray<uint8>>& ObjDataRef)
		{
			if (bCopyOutside)
			{
				ObjDataRef = ObjData;
			}
			else
			{
				ObjData = ObjDataRef;
			}
		}

		//~ Begin FCyclicDataCacheBase
		virtual FString ToLogString() const override;
		virtual void Serialize(FArchive& Ar) override;
		virtual void Reset() override;
		//~ End FCyclicDataCacheBase

		/** Serialization */
		friend FArchive& operator<<(FArchive& Ar, FCache_GetObjectsData& Instance)
		{
			Instance.Serialize(Ar);
			return Ar;
		}
	};

private: // GetEventsData

	/**
	 * GetEventsData cache
	 * 
	 * Caches cluster events data for current frame
	 */
	struct FCache_GetEventsData : public FCyclicDataCacheBase
	{
	public:

		/** JSON events cached */
		TArray<TSharedPtr<FDisplayClusterClusterEventJson>>   JsonEvents;

		/** Binary events cached */
		TArray<TSharedPtr<FDisplayClusterClusterEventBinary>> BinaryEvents;

	public:

		/** Copies data TO or FROM */
		void CopyData(bool bCopyOutside, TArray<TSharedPtr<FDisplayClusterClusterEventJson>>& JsonEventsRef, TArray<TSharedPtr<FDisplayClusterClusterEventBinary>>& BinaryEventsRef)
		{
			if (bCopyOutside)
			{
				JsonEventsRef   = JsonEvents;
				BinaryEventsRef = BinaryEvents;
			}
			else
			{
				JsonEvents   = JsonEventsRef;
				BinaryEvents = BinaryEventsRef;
			}
		}

		//~ Begin FCyclicDataCacheBase
		virtual FString ToLogString() const override;
		virtual void Serialize(FArchive& Ar) override;
		virtual void Reset() override;
		//~ End FCyclicDataCacheBase

		/** Serialization */
		friend FArchive& operator<<(FArchive& Ar, FCache_GetEventsData& Instance)
		{
			Instance.Serialize(Ar);
			return Ar;
		}
	};

private: // GetNativeInputData

	/**
	 * GetNativeInputData cache
	 *
	 * Caches native input data for current frame
	 */
	struct FCache_GetNativeInputData : public FCyclicDataCacheBase
	{
	public:

		/** Native input data cached for current cycle (frame) */
		TMap<FString, TArray<uint8>> NativeInputData;

	public:

		/** Copies data TO or FROM */
		void CopyData(bool bCopyOutside, TMap<FString, TArray<uint8>>& NativeInputDataRef)
		{
			if (bCopyOutside)
			{
				NativeInputDataRef = NativeInputData;
			}
			else
			{
				NativeInputData = NativeInputDataRef;
			}
		}

		//~ Begin FCyclicDataCacheBase
		virtual FString ToLogString() const override;
		virtual void Serialize(FArchive& Ar) override;
		virtual void Reset() override;
		//~ End FCyclicDataCacheBase

		/** Serialization */
		friend FArchive& operator<<(FArchive& Ar, FCache_GetNativeInputData& Instance)
		{
			Instance.Serialize(Ar);
			return Ar;
		}
	};

private:

	/**
	 * Data cache holder
	 *
	 * This is an auxiliary wrapper for data cache to simplify serialization during recovery
	 */
	struct FDataCacheHolder
	{
	public:

		FDataCacheHolder() = default;

		/** Workaround to allow the usage of TMap::Emplace for uncopyable TUniquePtr members */
		FDataCacheHolder(int)
		{ }

	public:

		/**
		 * Game thread data cache.
		 *
		 * Contains the game thread data received from a P-node during the current frame.
		 * This data is stored only by the receiving side. If the P-node fails, other nodes
		 * can use this data to catch up with the simulation. Valid only until the next frame.
		 */
		TMap<FName, TUniquePtr<FCyclicDataCacheBase>> GameThreadDataCache;

		/** Critical section for safe access */
		mutable FCriticalSection GameThreadDataCacheCS;

		/**
		 * Barrier synchronization state.
		 *
		 * Tracks the usage of the node barriers. Barrier-based synchronization requires all cluster
		 * nodes to maintain the same state between synchronization cycles. This is used to recover
		 * from a P-node failure.
		 */
		TMap<FName, TMap<FName, uint64>> BarrierSyncStates;

		/** Critical section for safe access */
		mutable FCriticalSection BarrierSyncStatesCS;

	public:

		/** Invalidate game thread data */
		void InvalidateGameThreadData(bool bReset = false);

		/** Access to the game thread data slots */
		FCyclicDataCacheBase* GetGTSlotData(const FName& SlotName);

		/** Access to the game thread data slots (const version) */
		const FCyclicDataCacheBase* GetGTSlotData(const FName& SlotName) const;

	public:

		/** Generates log string */
		FString ToLogString() const;

		/** Serialization */
		void Serialize(FArchive& Ar);

		/** Serialization */
		friend FArchive& operator<<(FArchive& Ar, FDataCacheHolder& Instance)
		{
			Instance.Serialize(Ar);
			return Ar;
		}
	};

	/** Represents the synchronization state of this node */
	FDataCacheHolder LocalDataCache;

	/** Represents the synchronization state of the whole cluster. It's generated during post-failure recovery. */
	FDataCacheHolder ClusterDataCache;
};
