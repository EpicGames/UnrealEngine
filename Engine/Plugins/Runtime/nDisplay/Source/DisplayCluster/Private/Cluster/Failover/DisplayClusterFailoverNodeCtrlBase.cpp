// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cluster/Failover/DisplayClusterFailoverNodeCtrlBase.h"


FDisplayClusterFailoverNodeCtrlBase::FDisplayClusterFailoverNodeCtrlBase(TSharedRef<IDisplayClusterClusterNodeController>& InNodeController)
	: NodeController(InNodeController)
{
}
