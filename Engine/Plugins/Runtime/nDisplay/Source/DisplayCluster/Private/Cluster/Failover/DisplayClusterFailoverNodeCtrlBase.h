// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Cluster/Failover/IDisplayClusterFailoverNodeController.h"
#include "DisplayClusterConfigurationTypes_Enums.h"

class IDisplayClusterClusterNodeController;
struct FDisplayClusterSessionInfo;


/**
 * Base failover node controller class
 */
class FDisplayClusterFailoverNodeCtrlBase
	: public IDisplayClusterFailoverNodeController
{
public:

	FDisplayClusterFailoverNodeCtrlBase(TSharedRef<IDisplayClusterClusterNodeController>& InNodeController);

protected:

	/** Access to the active node controller */
	const TSharedRef<IDisplayClusterClusterNodeController>& GetNodeController() const
	{
		return NodeController;
	}

private:

	/** Active node controller */
	TSharedRef<IDisplayClusterClusterNodeController> NodeController;
};
