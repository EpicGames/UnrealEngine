// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cluster/Failover/DisplayClusterFailoverNodeCtrlDisabled.h"
#include "Cluster/Failover/DisplayClusterCommDataCache.h"


////////////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterFailoverNodeController
////////////////////////////////////////////////////////////////////////////////////////////////////
bool FDisplayClusterFailoverNodeCtrlDisabled::Initialize(const UDisplayClusterConfigurationData* ConfigData)
{
	// Always return true, as 'Disabled' is a valid mode
	return true;
}

TSharedRef<FDisplayClusterCommDataCache> FDisplayClusterFailoverNodeCtrlDisabled::GetDataCache()
{
	static TSharedRef<FDisplayClusterCommDataCache> DataBroker = MakeShared<FDisplayClusterCommDataCache>();
	return DataBroker;
}

bool FDisplayClusterFailoverNodeCtrlDisabled::HandleFailure(const FString& FailedNodeId)
{
	return true;
}


////////////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProtocolClusterSync
////////////////////////////////////////////////////////////////////////////////////////////////////
EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlDisabled::WaitForGameStart()
{
	return EDisplayClusterCommResult::NotAllowed;
}

EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlDisabled::WaitForFrameStart()
{
	return EDisplayClusterCommResult::NotAllowed;
}

EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlDisabled::WaitForFrameEnd()
{
	return EDisplayClusterCommResult::NotAllowed;
}

EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlDisabled::GetTimeData(double& OutDeltaTime, double& OutGameTime, TOptional<FQualifiedFrameTime>& OutFrameTime)
{
	return EDisplayClusterCommResult::NotAllowed;
}

EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlDisabled::GetObjectsData(EDisplayClusterSyncGroup InSyncGroup, TMap<FString, TArray<uint8>>& OutObjectsData)
{
	return EDisplayClusterCommResult::NotAllowed;
}

EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlDisabled::GetEventsData(TArray<TSharedPtr<FDisplayClusterClusterEventJson>>& OutJsonEvents, TArray<TSharedPtr<FDisplayClusterClusterEventBinary>>& OutBinaryEvents)
{
	return EDisplayClusterCommResult::NotAllowed;
}

EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlDisabled::GetNativeInputData(TMap<FString, TArray<uint8>>& OutNativeInputData)
{
	return EDisplayClusterCommResult::NotAllowed;
}

EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlDisabled::PropagateStatesData(const TArray<uint8>& InLocalStatesData, TArray<uint8>& OutClusterStatesData)
{
	return EDisplayClusterCommResult::NotAllowed;
}


////////////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProtocolRenderSync
////////////////////////////////////////////////////////////////////////////////////////////////////
EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlDisabled::SynchronizeOnBarrier()
{
	return EDisplayClusterCommResult::NotAllowed;
}


////////////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProtocolEventsJson
////////////////////////////////////////////////////////////////////////////////////////////////////
EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlDisabled::EmitClusterEventJson(const FDisplayClusterClusterEventJson& Event)
{
	return EDisplayClusterCommResult::NotAllowed;
}


////////////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProtocolEventsBinary
////////////////////////////////////////////////////////////////////////////////////////////////////
EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlDisabled::EmitClusterEventBinary(const FDisplayClusterClusterEventBinary& Event)
{
	return EDisplayClusterCommResult::NotAllowed;
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProtocolGenericBarrier
//////////////////////////////////////////////////////////////////////////////////////////////
EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlDisabled::CreateBarrier(const FString& BarrierId, const TMap<FString, TSet<FString>>& NodeToSyncCallers, uint32 Timeout, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result)
{
	return EDisplayClusterCommResult::NotAllowed;
}

EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlDisabled::WaitUntilBarrierIsCreated(const FString& BarrierId, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result)
{
	return EDisplayClusterCommResult::NotAllowed;
}

EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlDisabled::IsBarrierAvailable(const FString& BarrierId, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result)
{
	return EDisplayClusterCommResult::NotAllowed;
}

EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlDisabled::ReleaseBarrier(const FString& BarrierId, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result)
{
	return EDisplayClusterCommResult::NotAllowed;
}

EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlDisabled::SyncOnBarrier(const FString& BarrierId, const FString& CallerId, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result)
{
	return EDisplayClusterCommResult::NotAllowed;
}

EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlDisabled::SyncOnBarrierWithData(const FString& BarrierId, const FString& CallerId, const TArray<uint8>& RequestData, TArray<uint8>& OutResponseData, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result)
{
	return EDisplayClusterCommResult::NotAllowed;
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProtocolInternalComm
//////////////////////////////////////////////////////////////////////////////////////////////
EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlDisabled::GatherServicesHostingInfo(const FNodeServicesHostingInfo& ThisNodeInfo, FClusterServicesHostingInfo& OutHostingInfo)
{
	return EDisplayClusterCommResult::NotAllowed;
}

EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlDisabled::PostFailureNegotiate(TArray<uint8>& InOutRecoveryData)
{
	return EDisplayClusterCommResult::NotAllowed;
}

EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlDisabled::RequestNodeDrop(const FString& NodeId, uint8 DropReason)
{
	return EDisplayClusterCommResult::NotAllowed;
}
