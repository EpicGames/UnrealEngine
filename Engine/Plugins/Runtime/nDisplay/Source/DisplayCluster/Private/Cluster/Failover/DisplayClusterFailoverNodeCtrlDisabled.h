// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Cluster/Failover/DisplayClusterFailoverNodeCtrlBase.h"


/**
 * Failover controller for 'Disabled' operation mode.
 *
 * It's mostly a stub for API calls.
*/
class FDisplayClusterFailoverNodeCtrlDisabled
	: public FDisplayClusterFailoverNodeCtrlBase
{
public:

	FDisplayClusterFailoverNodeCtrlDisabled(TSharedRef<IDisplayClusterClusterNodeController>& InNodeController)
		: FDisplayClusterFailoverNodeCtrlBase(InNodeController)
	{ }

public:

	//~ Begin IDisplayClusterFailoverNodeController
	virtual bool Initialize(const UDisplayClusterConfigurationData* ConfigData) override;
	virtual TSharedRef<FDisplayClusterCommDataCache> GetDataCache() override;
	virtual bool HandleFailure(const FString& FailedNodeId) override;
	//~ End IDisplayClusterFailoverNodeController

public:

	//~ Begin IDisplayClusterProtocolClusterSync
	virtual EDisplayClusterCommResult WaitForGameStart() override;
	virtual EDisplayClusterCommResult WaitForFrameStart() override;
	virtual EDisplayClusterCommResult WaitForFrameEnd() override;
	virtual EDisplayClusterCommResult GetTimeData(double& OutDeltaTime, double& OutGameTime, TOptional<FQualifiedFrameTime>& OutFrameTime) override;
	virtual EDisplayClusterCommResult GetObjectsData(EDisplayClusterSyncGroup InSyncGroup, TMap<FString, TArray<uint8>>& OutObjectsData) override;
	virtual EDisplayClusterCommResult GetEventsData(TArray<TSharedPtr<FDisplayClusterClusterEventJson>>& OutJsonEvents, TArray<TSharedPtr<FDisplayClusterClusterEventBinary>>& OutBinaryEvents) override;
	virtual EDisplayClusterCommResult GetNativeInputData(TMap<FString, TArray<uint8>>& OutNativeInputData) override;
	virtual EDisplayClusterCommResult PropagateStatesData(const TArray<uint8>& InLocalStatesData, TArray<uint8>& OutClusterStatesData) override;
	//~ End IDisplayClusterProtocolClusterSync

public:

	//~ Begin IDisplayClusterProtocolRenderSync
	virtual EDisplayClusterCommResult SynchronizeOnBarrier() override;
	//~ End IDisplayClusterProtocolRenderSync

public:

	//~ Begin IDisplayClusterProtocolEventsJson
	virtual EDisplayClusterCommResult EmitClusterEventJson(const FDisplayClusterClusterEventJson& Event) override;
	//~ End IDisplayClusterProtocolEventsJson

public:

	//~ Begin IDisplayClusterProtocolEventsBinary
	virtual EDisplayClusterCommResult EmitClusterEventBinary(const FDisplayClusterClusterEventBinary& Event) override;
	//~ End IDisplayClusterProtocolEventsBinary

public:

	//~ Begin IDisplayClusterProtocolGenericBarrier
	virtual EDisplayClusterCommResult CreateBarrier(const FString& BarrierId, const TMap<FString, TSet<FString>>& NodeToSyncCallers, uint32 Timeout, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result) override;
	virtual EDisplayClusterCommResult WaitUntilBarrierIsCreated(const FString& BarrierId, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result) override;
	virtual EDisplayClusterCommResult IsBarrierAvailable(const FString& BarrierId, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result) override;
	virtual EDisplayClusterCommResult ReleaseBarrier(const FString& BarrierId, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result) override;
	virtual EDisplayClusterCommResult SyncOnBarrier(const FString& BarrierId, const FString& CallerId, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result) override;
	virtual EDisplayClusterCommResult SyncOnBarrierWithData(const FString& BarrierId, const FString& CallerId, const TArray<uint8>& RequestData, TArray<uint8>& OutResponseData, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result) override;
	//~ End IDisplayClusterProtocolGenericBarrier

public:

	//~ Begin IDisplayClusterProtocolInternalComm
	virtual EDisplayClusterCommResult GatherServicesHostingInfo(const FNodeServicesHostingInfo& ThisNodeInfo, FClusterServicesHostingInfo& OutHostingInfo) override;
	virtual EDisplayClusterCommResult PostFailureNegotiate(TArray<uint8>& InOutRecoveryData) override;
	virtual EDisplayClusterCommResult RequestNodeDrop(const FString& NodeId, uint8 DropReason) override;
	//~ End IDisplayClusterProtocolInternalComm
};
