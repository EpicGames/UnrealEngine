// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cluster/Failover/DisplayClusterFailoverNodeCtrlEditor.h"

#include "Cluster/Controller/IDisplayClusterClusterNodeController.h"

#include "Misc/DisplayClusterAppExit.h"
#include "Misc/DisplayClusterLog.h"
#include "Misc/DisplayClusterTypesConverterPrivate.h"

#include "Network/DisplayClusterNetworkTypes.h"


////////////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterFailoverNodeController
////////////////////////////////////////////////////////////////////////////////////////////////////
bool FDisplayClusterFailoverNodeCtrlEditor::Initialize(const UDisplayClusterConfigurationData* ConfigData)
{
	return true;
}

TSharedRef<FDisplayClusterCommDataCache> FDisplayClusterFailoverNodeCtrlEditor::GetDataCache()
{
	return DataCache;
}

bool FDisplayClusterFailoverNodeCtrlEditor::HandleFailure(const FString& FailedNodeId)
{
	return true;
}


////////////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProtocolClusterSync
////////////////////////////////////////////////////////////////////////////////////////////////////
EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlEditor::WaitForGameStart()
{
	// `Editor` failover controller doesn't perform any validations, just forward the call
	return GetNodeController()->WaitForGameStart();
}

EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlEditor::WaitForFrameStart()
{
	// `Editor` failover controller doesn't perform any validations, just forward the call
	return GetNodeController()->WaitForFrameStart();
}

EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlEditor::WaitForFrameEnd()
{
	// `Editor` failover controller doesn't perform any validations, just forward the call
	return GetNodeController()->WaitForFrameEnd();
}

EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlEditor::GetTimeData(double& OutDeltaTime, double& OutGameTime, TOptional<FQualifiedFrameTime>& OutFrameTime)
{
	// `Editor` failover controller doesn't perform any validations, just forward the call
	return GetNodeController()->GetTimeData(OutDeltaTime, OutGameTime, OutFrameTime);
}

EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlEditor::GetObjectsData(EDisplayClusterSyncGroup InSyncGroup, TMap<FString, TArray<uint8>>& OutObjectsData)
{
	// `Editor` failover controller doesn't perform any validations, just forward the call
	return GetNodeController()->GetObjectsData(InSyncGroup, OutObjectsData);
}

EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlEditor::GetEventsData(TArray<TSharedPtr<FDisplayClusterClusterEventJson>>& OutJsonEvents, TArray<TSharedPtr<FDisplayClusterClusterEventBinary>>& OutBinaryEvents)
{
	// `Editor` failover controller doesn't perform any validations, just forward the call
	return GetNodeController()->GetEventsData(OutJsonEvents, OutBinaryEvents);
}

EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlEditor::GetNativeInputData(TMap<FString, TArray<uint8>>& OutNativeInputData)
{
	// `Editor` failover controller doesn't perform any validations, just forward the call
	return GetNodeController()->GetNativeInputData(OutNativeInputData);
}

EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlEditor::PropagateStatesData(const TArray<uint8>& InLocalStatesData, TArray<uint8>& OutClusterStatesData)
{
	// `Editor` failover controller doesn't perform any validations, just forward the call
	return GetNodeController()->PropagateStatesData(InLocalStatesData, OutClusterStatesData);
}


////////////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProtocolRenderSync
////////////////////////////////////////////////////////////////////////////////////////////////////
EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlEditor::SynchronizeOnBarrier()
{
	// `Editor` failover controller doesn't perform any validations, just forward the call
	return GetNodeController()->SynchronizeOnBarrier();
}


////////////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProtocolEventsJson
////////////////////////////////////////////////////////////////////////////////////////////////////
EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlEditor::EmitClusterEventJson(const FDisplayClusterClusterEventJson& Event)
{
	// `Editor` failover controller doesn't perform any validations, just forward the call
	return GetNodeController()->EmitClusterEventJson(Event);
}


////////////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProtocolEventsBinary
////////////////////////////////////////////////////////////////////////////////////////////////////
EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlEditor::EmitClusterEventBinary(const FDisplayClusterClusterEventBinary& Event)
{
	// `Editor` failover controller doesn't perform any validations, just forward the call
	return GetNodeController()->EmitClusterEventBinary(Event);
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProtocolGenericBarrier
//////////////////////////////////////////////////////////////////////////////////////////////
EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlEditor::CreateBarrier(const FString& BarrierId, const TMap<FString, TSet<FString>>& NodeToSyncCallers, uint32 Timeout, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result)
{
	// `Editor` failover controller doesn't perform any validations, just forward the call
	return GetNodeController()->CreateBarrier(BarrierId, NodeToSyncCallers, Timeout, Result);
}

EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlEditor::WaitUntilBarrierIsCreated(const FString& BarrierId, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result)
{
	// `Editor` failover controller doesn't perform any validations, just forward the call
	return GetNodeController()->WaitUntilBarrierIsCreated(BarrierId, Result);
}

EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlEditor::IsBarrierAvailable(const FString& BarrierId, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result)
{
	// `Editor` failover controller doesn't perform any validations, just forward the call
	return GetNodeController()->IsBarrierAvailable(BarrierId, Result);
}

EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlEditor::ReleaseBarrier(const FString& BarrierId, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result)
{
	// `Editor` failover controller doesn't perform any validations, just forward the call
	return GetNodeController()->ReleaseBarrier(BarrierId, Result);
}

EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlEditor::SyncOnBarrier(const FString& BarrierId, const FString& CallerId, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result)
{
	// `Editor` failover controller doesn't perform any validations, just forward the call
	return static_cast<IDisplayClusterProtocolGenericBarrier&>(GetNodeController().Get()).SyncOnBarrier(BarrierId, CallerId, Result);
}

EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlEditor::SyncOnBarrierWithData(const FString& BarrierId, const FString& CallerId, const TArray<uint8>& RequestData, TArray<uint8>& OutResponseData, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result)
{
	// `Editor` failover controller doesn't perform any validations, just forward the call
	return GetNodeController()->SyncOnBarrierWithData(BarrierId, CallerId, RequestData, OutResponseData, Result);
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProtocolInternalComm
//////////////////////////////////////////////////////////////////////////////////////////////
EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlEditor::GatherServicesHostingInfo(const FNodeServicesHostingInfo& ThisNodeInfo, FClusterServicesHostingInfo& OutHostingInfo)
{
	// Not expected in 'Editor'
	return EDisplayClusterCommResult::NotImplemented;
}

EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlEditor::PostFailureNegotiate(TArray<uint8>& InOutRecoveryData)
{
	// Not expected in 'Editor'
	return EDisplayClusterCommResult::NotImplemented;
}

EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlEditor::RequestNodeDrop(const FString& NodeId, uint8 DropReason)
{
	// Not expected in 'Editor'
	return EDisplayClusterCommResult::NotImplemented;
}
