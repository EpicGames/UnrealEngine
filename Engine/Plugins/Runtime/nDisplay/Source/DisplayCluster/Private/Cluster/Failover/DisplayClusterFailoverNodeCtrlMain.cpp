// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cluster/Failover/DisplayClusterFailoverNodeCtrlMain.h"

#include "Async/Async.h"

#include "Cluster/IPDisplayClusterClusterManager.h"
#include "Cluster/Controller/DisplayClusterCtrlContext.h"
#include "Cluster/Controller/IDisplayClusterClusterNodeController.h"

#include "Misc/DisplayClusterGlobals.h"
#include "Misc/DisplayClusterLog.h"
#include "Misc/ScopeRWLock.h"

#include "Network/Service/ClusterSync/DisplayClusterClusterSyncService.h"
#include "Network/Service/ClusterEventsBinary/DisplayClusterClusterEventsBinaryService.h"
#include "Network/Service/ClusterEventsJson/DisplayClusterClusterEventsJsonService.h"
#include "Network/Service/GenericBarrier/DisplayClusterGenericBarrierService.h"
#include "Network/Service/InternalComm/DisplayClusterInternalCommService.h"
#include "Network/Service/RenderSync/DisplayClusterRenderSyncService.h"

#include "CoreGlobals.h"
#include "DisplayClusterConfigurationTypes.h"
#include "IDisplayClusterCallbacks.h"


namespace UE::nDisplay::Failover::Private
{
	// Returns the full set of cluster nodes. Returns by value (RVO).
	static TSet<FString> GetAllNodes()
	{
		TSet<FString> AllNodes;
		GDisplayCluster->GetPrivateClusterMgr()->GetNodeIds(AllNodes);

		return AllNodes;
	}

	// Returns the full set of cluster nodes excluding the P-node. Returns by value (RVO).
	static TSet<FString> GetAllNodesNoPrimary()
	{
		const IPDisplayClusterClusterManager* const ClusterMgr = GDisplayCluster->GetPrivateClusterMgr();
		checkSlow(ClusterMgr);

		TSet<FString> AllNodes;

		ClusterMgr->GetNodeIds(AllNodes);
		AllNodes.Remove(ClusterMgr->GetPrimaryNodeId());
		return AllNodes;
	}

	// Returns the full set of cluster nodes excluding the custom ones. Returns by value (RVO).
	static TSet<FString> GetAllNodesExceptFor(const TSet<FString>& ExcludeSet)
	{
		const IPDisplayClusterClusterManager* const ClusterMgr = GDisplayCluster->GetPrivateClusterMgr();
		checkSlow(ClusterMgr);

		TSet<FString> AllNodes;

		ClusterMgr->GetNodeIds(AllNodes);
		AllNodes = AllNodes.Difference(ExcludeSet);
		return AllNodes;
	}
}


FDisplayClusterFailoverNodeCtrlMain::FDisplayClusterFailoverNodeCtrlMain(TSharedRef<IDisplayClusterClusterNodeController>& InNodeController)
	: FDisplayClusterFailoverNodeCtrlBase(InNodeController)
	, ThisNodeId(GDisplayCluster->GetPrivateClusterMgr()->GetNodeId())
	, ThisNodeIdName(*ThisNodeId)
	, FailoverSettings(MakeUnique<FDisplayClusterConfigurationFailoverSettings>())
{
}


////////////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterFailoverNodeController
////////////////////////////////////////////////////////////////////////////////////////////////////
bool FDisplayClusterFailoverNodeCtrlMain::Initialize(const UDisplayClusterConfigurationData* ConfigData)
{
	// Save failover configuration
	const bool bConfigSetup = SetupFailoverConfiguration(ConfigData);
	if (!bConfigSetup)
	{
		UE_LOGF(LogDisplayClusterFailover, Warning, "Couldn't setup failover configuration");
		return false;
	}

	// Perform other internal initialization
	SetupInternals();

	return true;
}

TSharedRef<FDisplayClusterCommDataCache> FDisplayClusterFailoverNodeCtrlMain::GetDataCache()
{
	return AsShared();
}

bool FDisplayClusterFailoverNodeCtrlMain::HandleFailure(const FString& FailedNodeId)
{
	FScopeLock SingleThreadOnlyLock(&FailoverCS);

	if (FailedNodesProcessedAlready.Contains(FailedNodeId))
	{
		// This node has been processed already, ignore this request
		return true;
	}
	else
	{
		// Remember this node so we won't run failover procedure for it again
		FailedNodesProcessedAlready.Add(FailedNodeId);

		// Remove it from the backup list
		FailoverSettings->PrimaryBackups.ItemNames.Remove(FailedNodeId);
	}

	UE_LOGF(LogDisplayClusterFailover, Log, "Reported node '%ls' failure.", *FailedNodeId);

	// If failover is disabled, always terminate
	if (!FailoverSettings->bEnabled)
	{
		bTerminateTransactionProcessingLoop = true;
		UE_LOGF(LogDisplayClusterFailover, Log, "Failover subsystem is disabled. No failures allowed.");
		return false;
	}

	// If this node has failed, always terminate
	if (ThisNodeId.Equals(FailedNodeId, ESearchCase::IgnoreCase))
	{
		bTerminateTransactionProcessingLoop = true;
		UE_LOGF(LogDisplayClusterFailover, Log, "This node just failed. Terminating...");
		return false;
	}

	// If critical node failed, always terminate
	if (FailoverSettings->CriticalNodes.ItemNames.Contains(FailedNodeId))
	{
		bTerminateTransactionProcessingLoop = true;
		UE_LOGF(LogDisplayClusterFailover, Log, "Critical node '%ls' has failed. Cluster termination is required.", *FailedNodeId);
		return false;
	}

	// If it's not primary, there is no need to run the recovery procedure
	const FString CurrentPrimaryNodeId = GDisplayCluster->GetPrivateClusterMgr()->GetPrimaryNodeId();
	if (!FailedNodeId.Equals(CurrentPrimaryNodeId, ESearchCase::IgnoreCase))
	{
		return true;
	}

	// Holds the ID of a node that we're currently processing
	FString NodeProcessing = FailedNodeId;

	// Write locking in SWMR. A single recovery process (writer/failover) is allowed.
	UE::TRWScopeLock TransactionLock(RecoveryLock, SLT_Write);

	// Process until succeeded, or game termination is requested. If new p-node is unresponsive
	// the NodeProcessing variable will be updated, and we'll go on another cycle.
	while (!NodeProcessing.IsEmpty())
	{
		// Remove it from the backup list
		FailoverSettings->PrimaryBackups.ItemNames.Remove(NodeProcessing);

		// If critical node failed, always terminate
		if (FailoverSettings->CriticalNodes.ItemNames.Contains(NodeProcessing))
		{
			bTerminateTransactionProcessingLoop = true;
			UE_LOGF(LogDisplayClusterFailover, Log, "Critical node '%ls' has failed. Cluster termination is required.", *NodeProcessing);
			return false;
		}

		// If this node failed, always terminate
		if (ThisNodeId.Equals(NodeProcessing, ESearchCase::IgnoreCase))
		{
			bTerminateTransactionProcessingLoop = true;
			UE_LOGF(LogDisplayClusterFailover, Log, "This node just failed. Terminating...");
			return false;
		}

		// Being here means the primary node has failed. Let's elect a new boss.
		const FString NewPrimaryId = ElectNewPrimaryNode();
		if (NewPrimaryId.IsEmpty())
		{
			bTerminateTransactionProcessingLoop = true;
			UE_LOGF(LogDisplayClusterFailover, Log, "No P-node candidates left.");
			return false;
		}

		UE_LOGF(LogDisplayClusterFailover, Log, "Elected new P-node: '%ls'.", *NewPrimaryId);

		// Notify everybody about new P-node
		GDisplayCluster->GetCallbacks().OnDisplayClusterFailoverPrimaryNodeChanged().Broadcast(NewPrimaryId);

		// Now recovery and re-sync all the remaining nodes
		if (!ProcessRecovery())
		{
			// If recovery failed, we need to drop this new primary as well
			NodeProcessing = NewPrimaryId;

			UE_LOGF(LogDisplayClusterFailover, Warning, "Couldn't switch to the new P-node '%ls'. Restarting recovery cycle.", *NewPrimaryId);
		}
		else
		{
			// Everything is fine, leave the failure processing loop
			break;
		}
	}

	return true;
}


////////////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProtocolClusterSync
////////////////////////////////////////////////////////////////////////////////////////////////////
EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlMain::WaitForGameStart()
{
	static const FName TxnName     = TEXT("WaitForGameStart");
	static const FName BarrierName = TEXT("GameStartBarrier");

	const EDisplayClusterCommResult CommResult = ProcessTransactionSYNC(
		TxnName,
		BarrierName,
		ThisNodeIdName,
		[&]() -> EDisplayClusterCommResult // OpSendReq
		{
			return GetNodeController()->WaitForGameStart();
		}
	);

	return CommResult;
}

EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlMain::WaitForFrameStart()
{
	static const FName TxnName     = TEXT("WaitForFrameStart");
	static const FName BarrierName = TEXT("FrameStartBarrier");

	const EDisplayClusterCommResult CommResult = ProcessTransactionSYNC(
		TxnName,
		BarrierName,
		ThisNodeIdName,
		[&]() -> EDisplayClusterCommResult // OpSendReq
		{
			return GetNodeController()->WaitForFrameStart();
		}
	);

	return CommResult;
}

EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlMain::WaitForFrameEnd()
{
	static const FName TxnName     = TEXT("WaitForFrameEnd");
	static const FName BarrierName = TEXT("FrameEndBarrier");

	const EDisplayClusterCommResult CommResult = ProcessTransactionSYNC(
		TxnName,
		BarrierName,
		ThisNodeIdName,
		[&]() -> EDisplayClusterCommResult // OpSendReq
		{
			return GetNodeController()->WaitForFrameEnd();
		}
	);

	return CommResult;
}

EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlMain::GetTimeData(double& OutDeltaTime, double& OutGameTime, TOptional<FQualifiedFrameTime>& OutFrameTime)
{
	static const FName TxnName = TEXT("GetTimeData");

	const EDisplayClusterCommResult CommResult = ProcessTransactionGET(
		TxnName,
		GetTimeData_OpIsCached,
		[&]() // OpLoad
		{
			GetTimeData_OpLoad(OutDeltaTime, OutGameTime, OutFrameTime);
		},
		[&]() // OpSave
		{
			GetTimeData_OpSave(OutDeltaTime, OutGameTime, OutFrameTime);
		},
		[&]() -> EDisplayClusterCommResult // OpSendReq
		{
			return GetNodeController()->GetTimeData(OutDeltaTime, OutGameTime, OutFrameTime);
		}
	);

	return CommResult;
}

EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlMain::GetObjectsData(EDisplayClusterSyncGroup InSyncGroup, TMap<FString, TArray<uint8>>& OutObjectsData)
{
	static const TMap<EDisplayClusterSyncGroup, FName> TxnNames =
	{
		{ EDisplayClusterSyncGroup::PreTick,  TEXT("GetObjectsData_PreTick") },
		{ EDisplayClusterSyncGroup::Tick,     TEXT("GetObjectsData_Tick") },
		{ EDisplayClusterSyncGroup::PostTick, TEXT("GetObjectsData_PostTick") }
	};

	const FName TxnName = TxnNames[InSyncGroup];

	const EDisplayClusterCommResult CommResult = ProcessTransactionGET(
		TxnName,
		GetObjectsData_OpIsCached(InSyncGroup), // OpIsCached
		[&]() // OpLoad
		{
			GetObjectsData_OpLoad(InSyncGroup, OutObjectsData);
		},
		[&]() // OpSave
		{
			GetObjectsData_OpSave(InSyncGroup, OutObjectsData);
		},
		[&]() -> EDisplayClusterCommResult // OpSendReq
		{
			return GetNodeController()->GetObjectsData(InSyncGroup, OutObjectsData);
		}
	);

	return CommResult;
}

EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlMain::GetEventsData(
	TArray<TSharedPtr<FDisplayClusterClusterEventJson>>&   OutJsonEvents,
	TArray<TSharedPtr<FDisplayClusterClusterEventBinary>>& OutBinaryEvents)
{
	static const FName TxnName = TEXT("GetEventsData");

	const EDisplayClusterCommResult CommResult = ProcessTransactionGET(
		TxnName,
		GetEventsData_OpIsCached,
		[&]() // OpLoad
		{
			GetEventsData_OpLoad(OutJsonEvents, OutBinaryEvents);
		},
		[&]() // OpSave
		{
			GetEventsData_OpSave(OutJsonEvents, OutBinaryEvents);
		},
		[&]() -> EDisplayClusterCommResult // OpSendReq
		{
			return GetNodeController()->GetEventsData(OutJsonEvents, OutBinaryEvents);
		}
	);

	return CommResult;
}

EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlMain::GetNativeInputData(TMap<FString, TArray<uint8>>& OutNativeInputData)
{
	static const FName TxnName = TEXT("GetNativeInputData");

	const EDisplayClusterCommResult CommResult = ProcessTransactionGET(
		TxnName,
		GetNativeInputData_OpIsCached,
		[&]() // OpLoad
		{
			GetNativeInputData_OpLoad(OutNativeInputData);
		},
		[&]() // OpSave
		{
			GetNativeInputData_OpSave(OutNativeInputData);
		},
		[&]() -> EDisplayClusterCommResult // OpSendReq
		{
			return GetNodeController()->GetNativeInputData(OutNativeInputData);
		}
	);

	return CommResult;
}

EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlMain::PropagateStatesData(const TArray<uint8>& InLocalStatesData, TArray<uint8>& OutClusterStatesData)
{
	static const FName TxnName     = TEXT("PropagateStatesData");
	static const FName BarrierName = TEXT("SyncStatesBarrier");

	const EDisplayClusterCommResult CommResult = ProcessTransactionSYNC(
		TxnName,
		BarrierName,
		ThisNodeIdName,
		[&]() -> EDisplayClusterCommResult // OpSendReq
		{
			return GetNodeController()->PropagateStatesData(InLocalStatesData, OutClusterStatesData);
		});

	return CommResult;
}


////////////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProtocolRenderSync
////////////////////////////////////////////////////////////////////////////////////////////////////
EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlMain::SynchronizeOnBarrier()
{
	static const FName TxnName     = TEXT("WaitForPresent");
	static const FName BarrierName = TEXT("PresentBarrier");

	const EDisplayClusterCommResult CommResult = ProcessTransactionSYNC(
		TxnName,
		BarrierName,
		ThisNodeIdName,
		[&]() -> EDisplayClusterCommResult // OpSendReq
		{
			return GetNodeController()->SynchronizeOnBarrier();
		}
	);

	return CommResult;
}


////////////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProtocolEventsJson
////////////////////////////////////////////////////////////////////////////////////////////////////
EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlMain::EmitClusterEventJson(const FDisplayClusterClusterEventJson& Event)
{
	static const FName TxnName = TEXT("EmitClusterEventJson");

	const EDisplayClusterCommResult CommResult = ProcessTransactionPUSH(
		TxnName,
		[&]() -> EDisplayClusterCommResult // OpSendReq
		{
			return GetNodeController()->EmitClusterEventJson(Event);
		});

	return CommResult;
}


////////////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProtocolEventsBinary
////////////////////////////////////////////////////////////////////////////////////////////////////
EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlMain::EmitClusterEventBinary(const FDisplayClusterClusterEventBinary& Event)
{
	static const FName TxnName = TEXT("EmitClusterEventBinary");

	const EDisplayClusterCommResult CommResult = ProcessTransactionPUSH(
		TxnName,
		[&]() -> EDisplayClusterCommResult // OpSendReq
		{
			return GetNodeController()->EmitClusterEventBinary(Event);
		});

	return CommResult;
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProtocolGenericBarrier
//////////////////////////////////////////////////////////////////////////////////////////////
EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlMain::CreateBarrier(const FString& BarrierId, const TMap<FString, TSet<FString>>& NodeToSyncCallers, uint32 Timeout, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result)
{
	static const FName TxnName = TEXT("CreateBarrier");

	const EDisplayClusterCommResult CommResult = ProcessTransactionMCAST(
		TxnName,
		[&]() -> EDisplayClusterCommResult // OpSendReq
		{
			return GetNodeController()->CreateBarrier(BarrierId, NodeToSyncCallers, Timeout, Result);
		},
		UE::nDisplay::Failover::Private::GetAllNodes());

	return CommResult;
}

EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlMain::WaitUntilBarrierIsCreated(const FString& BarrierId, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result)
{
	static const FName TxnName = TEXT("WaitUntilBarrierIsCreated");

	const EDisplayClusterCommResult CommResult = ProcessTransactionPUSH(
		TxnName,
		[&]() -> EDisplayClusterCommResult // OpSendReq
		{
			return GetNodeController()->WaitUntilBarrierIsCreated(BarrierId, Result);
		});

	return CommResult;
}

EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlMain::IsBarrierAvailable(const FString& BarrierId, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result)
{
	static const FName TxnName = TEXT("IsBarrierAvailable");

	const EDisplayClusterCommResult CommResult = ProcessTransactionPUSH(
		TxnName,
		[&]() -> EDisplayClusterCommResult // OpSendReq
		{
			return GetNodeController()->IsBarrierAvailable(BarrierId, Result);
		});

	return CommResult;
}

EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlMain::ReleaseBarrier(const FString& BarrierId, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result)
{
	static const FName TxnName = TEXT("ReleaseBarrier");

	const EDisplayClusterCommResult CommResult = ProcessTransactionMCAST(
		TxnName,
		[&]() -> EDisplayClusterCommResult // OpSendReq
		{
			return GetNodeController()->ReleaseBarrier(BarrierId, Result);
		},
		UE::nDisplay::Failover::Private::GetAllNodes());

	return CommResult;
}

EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlMain::SyncOnBarrier(const FString& BarrierId, const FString& CallerId, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result)
{
	const FName TxnName = *FString::Printf(TEXT("GPBSync::%s::%s"), *BarrierId, *CallerId);

	const EDisplayClusterCommResult CommResult = ProcessTransactionSYNC(
		TxnName,
		*BarrierId,
		*CallerId,
		[&]() -> EDisplayClusterCommResult // OpSendReq
		{
			return static_cast<IDisplayClusterProtocolGenericBarrier&>(GetNodeController().Get()).SyncOnBarrier(BarrierId, CallerId, Result);
		});

	return CommResult;
}

EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlMain::SyncOnBarrierWithData(const FString& BarrierId, const FString& CallerId, const TArray<uint8>& RequestData, TArray<uint8>& OutResponseData, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result)
{
	const FName TxnName = *FString::Printf(TEXT("GPBSync::%s::%s"), *BarrierId, *CallerId);

	const EDisplayClusterCommResult CommResult = ProcessTransactionSYNC(
		TxnName,
		*BarrierId,
		*CallerId,
		[&]() -> EDisplayClusterCommResult // OpSendReq
		{
			return GetNodeController()->SyncOnBarrierWithData(BarrierId, CallerId, RequestData, OutResponseData, Result);
		});

	return CommResult;
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProtocolInternalComm
//////////////////////////////////////////////////////////////////////////////////////////////
EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlMain::GatherServicesHostingInfo(const FNodeServicesHostingInfo& ThisNodeInfo, FClusterServicesHostingInfo& OutHostingInfo)
{
	// GatherServicesHostingInfo is called once during startup directly by the node controller.
	// At this point, the networking, cluster, and failover subsystems are not fully initialized.
	// Once the cluster is running, this function is no longer needed and should not be called again.
	return EDisplayClusterCommResult::NotAllowed;
}

EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlMain::PostFailureNegotiate(TArray<uint8>& InOutRecoveryData)
{
	static const FName TxnName = TEXT("PostFailureNegotiate");

	const EDisplayClusterCommResult CommResult = ProcessTransactionRECOVERY(
		TxnName,
		[&]() -> EDisplayClusterCommResult // OpSendReq
		{
			return GetNodeController()->PostFailureNegotiate(InOutRecoveryData);
		});

	return CommResult;
}

EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlMain::RequestNodeDrop(const FString& NodeId, uint8 DropReason)
{
	static const FName TxnName = TEXT("RequestNodeDrop");

	const EDisplayClusterCommResult CommResult = ProcessTransactionMCAST(
		TxnName,
		[&]() -> EDisplayClusterCommResult // OpSendReq
		{
			return GetNodeController()->RequestNodeDrop(NodeId, DropReason);
		},
		// Send to this node only
		{ NodeId });

	return CommResult;
}


//////////////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterFailoverNodeCtrlMain
//////////////////////////////////////////////////////////////////////////////////////////////
bool FDisplayClusterFailoverNodeCtrlMain::SetupFailoverConfiguration(const UDisplayClusterConfigurationData* ConfigData)
{
	checkSlow(ConfigData && ConfigData->Cluster);

	if (!ConfigData || !ConfigData->Cluster)
	{
		UE_LOGF(LogDisplayClusterFailover, Error, "Invalid configuration data");
		return false;
	}

	// Copy original failover configuration
	*FailoverSettings = ConfigData->Cluster->Failover;

	LogFailoverConfiguration(*FailoverSettings);

	return true;
}

void FDisplayClusterFailoverNodeCtrlMain::LogFailoverConfiguration(const FDisplayClusterConfigurationFailoverSettings& FailoverCfg)
{
	UE_LOGF(LogDisplayClusterFailover, Log, "Failover config: Failover Enabled = %d", FailoverCfg.bEnabled ? 1 : 0);

	const int32 PrioritizedBackupsNum = FailoverCfg.PrimaryBackups.ItemNames.Num();
	UE_LOGF(LogDisplayClusterFailover, Log, "Failover config: Prioritized primary backups (%d items):", PrioritizedBackupsNum);
	for (int32 Idx = 0; Idx < PrioritizedBackupsNum; ++Idx)
	{
		UE_LOGF(LogDisplayClusterFailover, Log, "Failover config: Prio-backup [%2d]: %ls", Idx, *FailoverCfg.PrimaryBackups.ItemNames[Idx]);
	}

	const int32 CriticalNodesNum = FailoverCfg.CriticalNodes.ItemNames.Num();
	UE_LOGF(LogDisplayClusterFailover, Log, "Failover config: Critical nodes (%d items):", CriticalNodesNum);
	for (int32 Idx = 0; Idx < CriticalNodesNum; ++Idx)
	{
		UE_LOGF(LogDisplayClusterFailover, Log, "Failover config: Critical node [%2d]: %ls", Idx, *FailoverCfg.CriticalNodes.ItemNames[Idx]);
	}
}

void FDisplayClusterFailoverNodeCtrlMain::SetupInternals()
{
	// Save full node list
	TSet<FString> NodeIds;
	GDisplayCluster->GetPrivateClusterMgr()->GetNodeIds(NodeIds);

	// Remove invalid backup nodes
	const TSet<FString> OrigBackupNodes(FailoverSettings->PrimaryBackups.ItemNames);
	const TSet<FString> ValidBackupNodes   = OrigBackupNodes.Intersect(NodeIds);
	const TSet<FString> InvalidBackupNodes = OrigBackupNodes.Difference(ValidBackupNodes);

	// Put some logs
	for (const FString& InvalidBackupNode : InvalidBackupNodes)
	{
		UE_LOGF(LogDisplayClusterFailover, Log, "Found invalid backup node '%ls'. Removing it from the backup list.", *InvalidBackupNode);
	}

	// Update backup nodes list
	FailoverSettings->PrimaryBackups.ItemNames = ValidBackupNodes.Array();
}

bool FDisplayClusterFailoverNodeCtrlMain::ProcessRecovery()
{
	// Generate current node synchronization state
	TArray<uint8> InOutRecoveryData;
	GenerateNodeSyncState(InOutRecoveryData);

	// And send it to the p-node. If everything is fine, we'll get actual cluster
	// syncrhonization state in response.
	if (PostFailureNegotiate(InOutRecoveryData) != EDisplayClusterCommResult::Ok)
	{
		UE_LOGF(LogDisplayClusterFailover, Warning, "Failed to process recovery syncrhonization");
		return false;
	}

	// Update cluster sync state internally
	UpdateClusterSyncState(InOutRecoveryData);

	return true;
}

FString FDisplayClusterFailoverNodeCtrlMain::ElectNewPrimaryNode()
{
	FString NewPrimaryId;

	// Failover disabled
	if (!FailoverSettings->bEnabled)
	{
		return NewPrimaryId;
	}

	// The amount of prioritized backup nodes
	const int32 PrioritizedBackupsNum = FailoverSettings->PrimaryBackups.ItemNames.Num();

	// If there are any prioritized backups, use the top most one (the first in order)
	if (PrioritizedBackupsNum > 0)
	{
		NewPrimaryId = FailoverSettings->PrimaryBackups.ItemNames[0];
	}
	// If no prioritized left, let's pick one from remainings
	else
	{
		// Get current node IDs
		TArray<FString> NodeIds;
		GDisplayCluster->GetPrivateClusterMgr()->GetNodeIds(NodeIds);

		// Return the first one in alphabetical order
		if (NodeIds.Num() > 0)
		{
			NodeIds.Sort([](const FString& LHS, const FString& RHS) { return LHS < RHS; });
			NewPrimaryId = NodeIds[0];
		}
	}

	return NewPrimaryId;
}

uint64 FDisplayClusterFailoverNodeCtrlMain::GetTransactionCounter(const FName& TransactionName)
{
	FScopeLock Lock(&TransactionCoutnerCS);
	return TransactionCoutner.FindOrAdd(TransactionName, 0);
}

uint64 FDisplayClusterFailoverNodeCtrlMain::GetAndIncrementTransactionCounter(const FName& TransactionName)
{
	FScopeLock Lock(&TransactionCoutnerCS);
	return TransactionCoutner.FindOrAdd(TransactionName, 0)++;
}

void FDisplayClusterFailoverNodeCtrlMain::IncrementTransactionCounter(const FName& TransactionName)
{
	FScopeLock Lock(&TransactionCoutnerCS);
	TransactionCoutner.FindOrAdd(TransactionName, 0)++;
}

EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlMain::ProcessTransactionGET(
	const FName&           TransactionName,
	const FOpIsCached&     OpIsCached,
	const FOpCacheWrapper& OpCacheLoad,
	const FOpCacheWrapper& OpCacheSave,
	const FOpSendReq&      OpSendReq)
{
	EDisplayClusterCommResult CommResult = EDisplayClusterCommResult::InternalError;

	const uint64 CurrentTransactionNum = GetTransactionCounter(TransactionName);

	UE_LOGF(LogDisplayClusterFailover, Verbose, "Txn GET beg: %ls [%llu]",
		*TransactionName.ToString(), CurrentTransactionNum);

	// We need to run until the transaction is done, or exit requested. Every
	// loop pass basically means an attempt to perform a transaction to a
	// specific node that is currently primary.
	while (!bTerminateTransactionProcessingLoop && !IsEngineExitRequested())
	{
		FString TxnTargetNode;

		{
			// Read locking in SWMR. Multiple transactions (readers) can run in parallel.
			UE::TRWScopeLock Lock(RecoveryLock, SLT_ReadOnly);

			// If data is available in cache, use it
			if (::Invoke(OpIsCached))
			{
				UE_LOGF(LogDisplayClusterFailover, Verbose, "Txn GET@%ls: %ls [%llu] - using cached data",
					*TxnTargetNode, *TransactionName.ToString(), CurrentTransactionNum);

				::Invoke(OpCacheLoad);

				CommResult = EDisplayClusterCommResult::Ok;
				break;
			}
			// Otherwise, ask the primary node
			else
			{
				// Remember target node during the transaction
				TxnTargetNode = GDisplayCluster->GetPrivateClusterMgr()->GetPrimaryNodeId();

				// Update context so this node will be used as destination
				FDisplayClusterCtrlContext::Get().TargetNodeId = *TxnTargetNode;

				UE_LOGF(LogDisplayClusterFailover, Verbose, "Txn GET@%ls: %ls [%llu] - sending request",
					*TxnTargetNode, *TransactionName.ToString(), CurrentTransactionNum);

				// Perform transaction
				CommResult = ::Invoke(OpSendReq);

				// If everything is Ok, remember success and leave the loop to finish transaction
				if (CommResult == EDisplayClusterCommResult::Ok)
				{
					::Invoke(OpCacheSave);
					break;
				}
			}
		}

		UE_LOGF(LogDisplayClusterFailover, Verbose, "Txn GET@%ls: %ls [%llu] - request failed",
			*TxnTargetNode, *TransactionName.ToString(), CurrentTransactionNum);

		// Being here means the primary node has failed. Trigger the drop procedure.
		GDisplayCluster->GetPrivateClusterMgr()->DropNode(TxnTargetNode, IPDisplayClusterClusterManager::ENodeDropReason::Failed);
	}

	UE_LOGF(LogDisplayClusterFailover, Verbose, "Txn GET end: %ls [%llu]", *TransactionName.ToString(), CurrentTransactionNum);

	IncrementTransactionCounter(TransactionName);

	return CommResult;
}

EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlMain::ProcessTransactionPUSH(
	const FName&      TransactionName,
	const FOpSendReq& OpSendReq)
{
	EDisplayClusterCommResult CommResult = EDisplayClusterCommResult::InternalError;

	const uint64 CurrentTransactionNum = GetTransactionCounter(TransactionName);

	UE_LOGF(LogDisplayClusterFailover, Verbose, "Txn PUSH beg: %ls [%llu]",
		*TransactionName.ToString(), CurrentTransactionNum);

	// We need to run until the transaction is done, or exit requested. Every
	// loop pass basically means an attempt to perform a transaction to a
	// specific node that is currently primary.
	while (!bTerminateTransactionProcessingLoop && !IsEngineExitRequested())
	{
		FString TxnTargetNode;

		{
			// Read locking in SWMR. Multiple transactions (readers) can run in parallel.
			UE::TRWScopeLock Lock(RecoveryLock, SLT_ReadOnly);

			// Remember target node during the transaction
			TxnTargetNode = GDisplayCluster->GetPrivateClusterMgr()->GetPrimaryNodeId();

			// Update context so this node will be used as destination
			FDisplayClusterCtrlContext::Get().TargetNodeId = *TxnTargetNode;

			UE_LOGF(LogDisplayClusterFailover, VeryVerbose, "Txn PUSH@%ls: %ls [%llu] - sending request",
				*TxnTargetNode, *TransactionName.ToString(), CurrentTransactionNum);

			// Perform transaction
			CommResult = ::Invoke(OpSendReq);

			// If everything is Ok, remember this barrier as open and finish transaction
			if (CommResult == EDisplayClusterCommResult::Ok)
			{
				break;
			}
		}

		UE_LOGF(LogDisplayClusterFailover, Log, "Txn PUSH@%ls: %ls [%llu] - request failed",
			*TxnTargetNode, *TransactionName.ToString(), CurrentTransactionNum);

		// Being here means the primary node has failed. Trigger the drop procedure.
		GDisplayCluster->GetPrivateClusterMgr()->DropNode(TxnTargetNode, IPDisplayClusterClusterManager::ENodeDropReason::Failed);
	}

	UE_LOGF(LogDisplayClusterFailover, Verbose, "Txn PUSH end: %ls [%llu]", *TransactionName.ToString(), CurrentTransactionNum);

	IncrementTransactionCounter(TransactionName);

	return CommResult;
}

EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlMain::ProcessTransactionSYNC(
	const FName& TransactionName,
	const FName& BarrierId,
	const FName& CallerId,
	const FOpSendReq& OpSendReq)
{
	EDisplayClusterCommResult CommResult = EDisplayClusterCommResult::InternalError;

	const uint64 CurrentTransactionNum = GetTransactionCounter(TransactionName);

	UE_LOGF(LogDisplayClusterFailover, Verbose, "Txn SYNC beg: %ls [%llu]",
		*TransactionName.ToString(), CurrentTransactionNum);

	// We need to run until the transaction is done, or exit requested. Every
	// loop pass basically means an attempt to perform a transaction to a
	// specific node that is currently primary.
	while (!bTerminateTransactionProcessingLoop && !IsEngineExitRequested())
	{
		FString TxnTargetNode;

		{
			// Read locking in SWMR. Multiple transactions (readers) can run in parallel.
			UE::TRWScopeLock Lock(RecoveryLock, SLT_ReadOnly);

			// Check if this barrier has been opened. If so, we should skip synchronization on it.
			if (::Invoke(OpGetBarrierOpen, BarrierId, CallerId))
			{
				UE_LOGF(LogDisplayClusterFailover, VeryVerbose, "Txn SYNC@%ls: %ls [%llu] - using cached data",
					*TxnTargetNode, *TransactionName.ToString(), CurrentTransactionNum);

				// Update local barrier history
				::Invoke(OpAdvanceBarrierCounter, BarrierId, CallerId);

				CommResult = EDisplayClusterCommResult::Ok;
				break;
			}
			// Otherwise, synchronize on the barrier
			else
			{
				// Remember target node during the transaction
				TxnTargetNode = GDisplayCluster->GetPrivateClusterMgr()->GetPrimaryNodeId();

				// Update context so this node will be used as destination
				FDisplayClusterCtrlContext::Get().TargetNodeId = *TxnTargetNode;

				UE_LOGF(LogDisplayClusterFailover, VeryVerbose, "Txn SYNC@%ls: %ls [%llu] - sending request",
					*TxnTargetNode, *TransactionName.ToString(), CurrentTransactionNum);

				// Perform transaction
				CommResult = ::Invoke(OpSendReq);

				// If everything is Ok, remember this barrier as open and finish transaction
				if (CommResult == EDisplayClusterCommResult::Ok)
				{
					// Also cache the response data
					::Invoke(OpAdvanceBarrierCounter, BarrierId, CallerId);
					break;
				}
			}
		}

		UE_LOGF(LogDisplayClusterFailover, Log, "Txn SYNC@%ls: %ls [%llu] - request failed",
			*TxnTargetNode, *TransactionName.ToString(), CurrentTransactionNum);

		// Being here means the primary node has failed. Trigger the drop procedure.
		GDisplayCluster->GetPrivateClusterMgr()->DropNode(TxnTargetNode, IPDisplayClusterClusterManager::ENodeDropReason::Failed);
	}

	UE_LOGF(LogDisplayClusterFailover, Verbose, "Txn SYNC end: %ls [%llu]", *TransactionName.ToString(), CurrentTransactionNum);

	IncrementTransactionCounter(TransactionName);

	return CommResult;
}

EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlMain::ProcessTransactionMCAST(
	const FName&         TransactionName,
	const FOpSendReq&    OpSendReq,
	const TSet<FString>& TargetNodes)
{
	EDisplayClusterCommResult CommResult = EDisplayClusterCommResult::InternalError;

	const uint64 CurrentTransactionNum = GetTransactionCounter(TransactionName);

	UE_LOGF(LogDisplayClusterFailover, Verbose, "Txn MCAST beg: %ls [%llu]",
		*TransactionName.ToString(), CurrentTransactionNum);

	// For each target there will be a separate transaction
	for (const FString& TxnTargetNode : TargetNodes)
	{
		// Leave if loop termination requested
		if (bTerminateTransactionProcessingLoop || IsEngineExitRequested())
		{
			break;
		}

		{
			// Read locking in SWMR. Multiple transactions (readers) can run in parallel.
			UE::TRWScopeLock Lock(RecoveryLock, SLT_ReadOnly);

			// Update context so this node will be used as destination
			FDisplayClusterCtrlContext::Get().TargetNodeId = *TxnTargetNode;

			UE_LOGF(LogDisplayClusterFailover, VeryVerbose, "Txn MCAST@%ls: %ls [%llu] - sending request",
				*TxnTargetNode, *TransactionName.ToString(), CurrentTransactionNum);

			// Perform transaction
			CommResult = ::Invoke(OpSendReq);

			// If everything is Ok, got to the next node
			if (CommResult == EDisplayClusterCommResult::Ok)
			{
				continue;
			}
		}

		UE_LOGF(LogDisplayClusterFailover, Log, "Txn MCAST@%ls: %ls [%llu] - request failed",
			*TxnTargetNode, *TransactionName.ToString(), CurrentTransactionNum);

		// Being here means the primary node has failed. Trigger the drop procedure.
		GDisplayCluster->GetPrivateClusterMgr()->DropNode(TxnTargetNode, IPDisplayClusterClusterManager::ENodeDropReason::Failed);
	}

	UE_LOGF(LogDisplayClusterFailover, Verbose, "Txn MCAST end: %ls [%llu]", *TransactionName.ToString(), CurrentTransactionNum);

	IncrementTransactionCounter(TransactionName);

	return CommResult;
}

EDisplayClusterCommResult FDisplayClusterFailoverNodeCtrlMain::ProcessTransactionRECOVERY(
	const FName&      TransactionName,
	const FOpSendReq& OpSendReq)
{
	// Remember target node during the transaction
	const FString TxnTargetNode = GDisplayCluster->GetPrivateClusterMgr()->GetPrimaryNodeId();

	const uint64 CurrentTransactionNum = GetTransactionCounter(TransactionName);

	UE_LOGF(LogDisplayClusterFailover, Verbose, "RECOVERY beg: %ls [%llu] - requesting data from '%ls'",
		*TransactionName.ToString(), CurrentTransactionNum, *TxnTargetNode);

	// Update context so this node will be used as destination
	FDisplayClusterCtrlContext::Get().TargetNodeId = *TxnTargetNode;

	// Perform transaction
	const EDisplayClusterCommResult CommResult = ::Invoke(OpSendReq);

	UE_LOGF(LogDisplayClusterFailover, Verbose, "RECOVERY end: %ls [%llu]",
		*TransactionName.ToString(), CurrentTransactionNum);

	IncrementTransactionCounter(TransactionName);

	return CommResult;
}
