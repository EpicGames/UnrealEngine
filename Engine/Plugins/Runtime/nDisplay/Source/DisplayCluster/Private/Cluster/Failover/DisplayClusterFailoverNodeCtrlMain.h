// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Cluster/Failover/DisplayClusterCommDataCache.h"
#include "Cluster/Failover/DisplayClusterFailoverNodeCtrlBase.h"

#include "Misc/OptionalFwd.h"
#include "Misc/QualifiedFrameTime.h"
#include "Misc/TransactionallySafeRWLock.h"

#include <atomic>

class FDisplayClusterService;
class UDisplayClusterConfigurationData;
struct FDisplayClusterConfigurationFailoverSettings;
struct FDisplayClusterServiceFailureEvent;


/**
 * Failover controller for 'Cluster' operation mode.
 *
 * Provides full set of features required in 'Cluster' operation mode.
 */
class FDisplayClusterFailoverNodeCtrlMain
	: public FDisplayClusterFailoverNodeCtrlBase
	, public FDisplayClusterCommDataCache
	, public TSharedFromThis<FDisplayClusterFailoverNodeCtrlMain>
{
public:

	FDisplayClusterFailoverNodeCtrlMain(TSharedRef<IDisplayClusterClusterNodeController>& InNodeController);
	virtual ~FDisplayClusterFailoverNodeCtrlMain() = default;

public:

	//~ Begin IDisplayClusterFailoverNodeController
	virtual bool Initialize(const UDisplayClusterConfigurationData* ConfigData) override;
	virtual TSharedRef<FDisplayClusterCommDataCache> GetDataCache() override;
	virtual bool HandleFailure(const FString& FailedNodeId) override;
	//~ End IDisplayClusterFailoverNodeController

public:

	//~ Begin IDisplayClusterProtocolClusterSync
	virtual EDisplayClusterCommResult WaitForGameStart() override;
	virtual EDisplayClusterCommResult WaitForFrameStart() override;
	virtual EDisplayClusterCommResult WaitForFrameEnd() override;
	virtual EDisplayClusterCommResult GetTimeData(double& OutDeltaTime, double& OutGameTime, TOptional<FQualifiedFrameTime>& OutFrameTime) override;
	virtual EDisplayClusterCommResult GetObjectsData(EDisplayClusterSyncGroup InSyncGroup, TMap<FString, TArray<uint8>>& OutObjectsData) override;
	virtual EDisplayClusterCommResult GetEventsData(TArray<TSharedPtr<FDisplayClusterClusterEventJson>>& OutJsonEvents, TArray<TSharedPtr<FDisplayClusterClusterEventBinary>>& OutBinaryEvents) override;
	virtual EDisplayClusterCommResult GetNativeInputData(TMap<FString, TArray<uint8>>& OutNativeInputData) override;
	virtual EDisplayClusterCommResult PropagateStatesData(const TArray<uint8>& InLocalStatesData, TArray<uint8>& OutClusterStatesData) override;
	//~ End IDisplayClusterProtocolClusterSync

public:

	//~ Begin IDisplayClusterProtocolRenderSync
	virtual EDisplayClusterCommResult SynchronizeOnBarrier() override;
	//~ End IDisplayClusterProtocolRenderSync

public:

	//~ Begin IDisplayClusterProtocolEventsJson
	virtual EDisplayClusterCommResult EmitClusterEventJson(const FDisplayClusterClusterEventJson& Event) override;
	//~ End IDisplayClusterProtocolEventsJson

public:

	//~ Begin IDisplayClusterProtocolEventsBinary
	virtual EDisplayClusterCommResult EmitClusterEventBinary(const FDisplayClusterClusterEventBinary& Event) override;
	//~ End IDisplayClusterProtocolEventsBinary

public:

	//~ Begin IDisplayClusterProtocolGenericBarrier
	virtual EDisplayClusterCommResult CreateBarrier(const FString& BarrierId, const TMap<FString, TSet<FString>>& NodeToSyncCallers, uint32 Timeout, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result) override;
	virtual EDisplayClusterCommResult WaitUntilBarrierIsCreated(const FString& BarrierId, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result) override;
	virtual EDisplayClusterCommResult IsBarrierAvailable(const FString& BarrierId, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result) override;
	virtual EDisplayClusterCommResult ReleaseBarrier(const FString& BarrierId, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result) override;
	virtual EDisplayClusterCommResult SyncOnBarrier(const FString& BarrierId, const FString& CallerId, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result) override;
	virtual EDisplayClusterCommResult SyncOnBarrierWithData(const FString& BarrierId, const FString& CallerId, const TArray<uint8>& RequestData, TArray<uint8>& OutResponseData, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result) override;
	//~ End IDisplayClusterProtocolGenericBarrier

public:

	//~ Begin IDisplayClusterProtocolInternalComm
	virtual EDisplayClusterCommResult GatherServicesHostingInfo(const FNodeServicesHostingInfo& ThisNodeInfo, FClusterServicesHostingInfo& OutHostingInfo) override;
	virtual EDisplayClusterCommResult PostFailureNegotiate(TArray<uint8>& InOutRecoveryData) override;
	virtual EDisplayClusterCommResult RequestNodeDrop(const FString& NodeId, uint8 DropReason) override;
	//~ End IDisplayClusterProtocolInternalComm

private:

	/** An auxiliary initialization function to process failover configuration setup */
	bool SetupFailoverConfiguration(const UDisplayClusterConfigurationData* ConfigData);

	/** Reports configuration to the log stream */
	void LogFailoverConfiguration(const FDisplayClusterConfigurationFailoverSettings& FailoverCfg);

	/** Initialize other internals */
	void SetupInternals();

	/** Implements cluster recovery after primary failure */
	bool ProcessRecovery();

	/** Elects new primary node if possible */
	FString ElectNewPrimaryNode();

	/** Returns current transaction count */
	uint64 GetTransactionCounter(const FName& TransactionName);

	/** Returns current transaction count, and increments it */
	uint64 GetAndIncrementTransactionCounter(const FName& TransactionName);

	/** Increments transaction counter */
	void IncrementTransactionCounter(const FName& TransactionName);

private:

	/** Basic transaction operation for remote data request */
	using FOpSendReq      = TFunction<EDisplayClusterCommResult()>;

	/** Cache operation provider */
	using FOpCacheWrapper = TFunction<void()>;

	/**
	 * GET type of transactions
	 *
	 * Retrieves data from the current P-node or the cache. Updates the local cache
	 * upon successfully receiving data from the P-node. Initiates the failure
	 * handling procedure if the P-node fails.
	 */
	EDisplayClusterCommResult ProcessTransactionGET(
		const FName&           TransactionName,
		const FOpIsCached&     OpIsCached,
		const FOpCacheWrapper& OpCacheLoad,
		const FOpCacheWrapper& OpCacheSave,
		const FOpSendReq&      OpSendReq
	);

	/**
	 * PUSH type of transactions
	 *
	 * This type of transaction is used to send data to the P-node. It is also used
	 * for trivial communication messages that neither send any data nor expect
	 * to receive any in return.
	 */
	EDisplayClusterCommResult ProcessTransactionPUSH(
		const FName&      TransactionName,
		const FOpSendReq& OpSendReq
	);

	/**
	 * SYNC type of transactions
	 *
	 * Used for barrier synchronzation. Remembers successfull synchronization attempts.
	 * It also uses the cluster synchronization state to decide whether synchronization
	 * on a specified barrier is necessary at the moment.
	 */
	EDisplayClusterCommResult ProcessTransactionSYNC(
		const FName& TransactionName,
		const FName& BarrierId,
		const FName& CallerId,
		const FOpSendReq& OpSendReq
	);

	/**
	 * MCAST type of transactions
	 *
	 * Used for multicast calls. Each node call is performed as a separate transaction.
	 */
	EDisplayClusterCommResult ProcessTransactionMCAST(
		const FName&         TransactionName,
		const FOpSendReq&    OpSendReq,
		const TSet<FString>& TargetNodes
	);

	/**
	 * RECOVERY type of transactions
	 *
	 * The transactions of this type are used during failure handling procedure. Unlike any other
	 * transaction type, this one is performed in exclusive mode (Write Lock), therefore no other
	 * transaction allowed to run simultaneously.
	 */
	EDisplayClusterCommResult ProcessTransactionRECOVERY(
		const FName&      TransactionName,
		const FOpSendReq& OpSendReq
	);

private:

	/** This cluster node Id */
	const FString ThisNodeId;

	/** This cluster node Id as FName*/
	const FName ThisNodeIdName;

	/** Failover settings from cluster configuration */
	TUniquePtr<FDisplayClusterConfigurationFailoverSettings> FailoverSettings;

	/** Prevents endless transaction working cycle */
	std::atomic<bool> bTerminateTransactionProcessingLoop = false;

	/** Tracks nodes that have previously failed and have already undergone failure handling */
	TSet<FString> FailedNodesProcessedAlready;

	/**
	  Safety lock to block any transactions during recovery. Here the SWMR concept "Single Writer Multiple Readers"
	  is turned into "Single Recovery Multiple Transactions". We allow to run multiple transactions in parallel,
	  but none of them allowed during recovery. Also, it's not allowed to run multiple recovery processes simultaneously.
	*/
	FTransactionallySafeRWLock RecoveryLock;

	/**
	  This one is used to safely operate on some failover internals before starting 
	  the actual post-failure recovery procedure which requires RecoveryLock in W-mode.
	*/
	mutable FCriticalSection FailoverCS;

	/** Keeps all the counters for transactions */
	TMap<FName, uint64> TransactionCoutner;

	/** Safe access to the transaction counters */
	mutable FCriticalSection TransactionCoutnerCS;
};
