// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Network/Protocol/IDisplayClusterProtocolClusterSync.h"
#include "Network/Protocol/IDisplayClusterProtocolEventsBinary.h"
#include "Network/Protocol/IDisplayClusterProtocolEventsJson.h"
#include "Network/Protocol/IDisplayClusterProtocolGenericBarrier.h"
#include "Network/Protocol/IDisplayClusterProtocolInternalComm.h"
#include "Network/Protocol/IDisplayClusterProtocolRenderSync.h"

class FDisplayClusterCommDataCache;
class UDisplayClusterConfigurationData;


/**
 * Failover controller interface
 */
class IDisplayClusterFailoverNodeController
	: public IDisplayClusterProtocolClusterSync
	, public IDisplayClusterProtocolEventsBinary
	, public IDisplayClusterProtocolEventsJson
	, public IDisplayClusterProtocolGenericBarrier
	, public IDisplayClusterProtocolInternalComm
	, public IDisplayClusterProtocolRenderSync
{
public:

	virtual ~IDisplayClusterFailoverNodeController() = default;

public:

	/** Initialize the controller */
	virtual bool Initialize(const UDisplayClusterConfigurationData* ConfigData) = 0;

	/** Provides access to the communication data cache */
	virtual TSharedRef<FDisplayClusterCommDataCache> GetDataCache() = 0;

	/** Process node failure */
	virtual bool HandleFailure(const FString& FailedNodeId) = 0;
};
