// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Cluster/IDisplayClusterClusterManager.h"
#include "IPDisplayClusterManager.h"

#include "Misc/FrameRate.h"
#include "Misc/Timecode.h"
#include "Misc/QualifiedFrameTime.h"

class FDisplayClusterCommDataCache;
class FDisplayClusterNetApiFacade;
class FDisplayClusterService;
class IDisplayClusterClusterNodeController;

/**
 * Cluster manager private interface
 */
class IPDisplayClusterClusterManager :
	public IDisplayClusterClusterManager,
	public IPDisplayClusterManager
{
public:

	virtual ~IPDisplayClusterClusterManager() = default;

public:

	/** A list of reasons why a node may leave the cluster */
	enum class ENodeDropReason : uint8
	{
		UserRequest,
		Failed,
	};

public:

	/** Access to the networking API */
	virtual FDisplayClusterNetApiFacade& GetNetApi() = 0;

	/** Access to the node controller */
	virtual TSharedRef<IDisplayClusterClusterNodeController> GetNodeController() = 0;

	/** Access to the communication data cache */
	virtual TSharedRef<FDisplayClusterCommDataCache> GetDataCache() = 0;

	/** Access to the node services */
	virtual TWeakPtr<FDisplayClusterService> GetNodeService(const FName& ServiceName) = 0;

	/** Drop cluster node for a reason. This is the entry point for drop requests. */
	virtual bool DropNode(const FString& NodeId, ENodeDropReason DropReason) = 0;

public: // Time data sync

	/** Time data synchronization (procedure entry point) */
	virtual void SyncTimeData() = 0;

	/** Cache current time data */
	virtual void CacheTimeData() = 0;

	/** Export current time data */
	virtual void ExportTimeData(      double& OutDeltaTime,      double& OutGameTime,      TOptional<FQualifiedFrameTime>& OutFrameTime) = 0;

	/** Import time data from external source */
	virtual void ImportTimeData(const double& InDeltaTime, const double& InGameTime, const TOptional<FQualifiedFrameTime>& InFrameTime) = 0;

public: // Custom states

	/** Custom states synchronization (procedure entry point) */
	virtual void SyncStates() = 0;

public: // Objects sync

	/** Custom objects synchronization (procedure entry point) */
	virtual void SyncObjects(EDisplayClusterSyncGroup SyncGroup) = 0;

	/** Caches objects data */
	virtual void CacheObjects(EDisplayClusterSyncGroup SyncGroup) = 0;

	/** Exports objects data from this node */
	virtual void ExportObjectsData(const EDisplayClusterSyncGroup InSyncGroup,       TMap<FString, TArray<uint8>>& OutObjectsData) = 0;

	/** Imports objects data from the primary node */
	virtual void ImportObjectsData(const EDisplayClusterSyncGroup InSyncGroup, const TMap<FString, TArray<uint8>>& InObjectsData) = 0;

public: // Cluster events sync

	/** Cluster events synchronization (procedure entry point) */
	virtual void SyncEvents() = 0;

	/** Cache events */
	virtual void CacheEvents() = 0;

	/** Exports events data from the current node (primary) */
	virtual void ExportEventsData(      TArray<TSharedPtr<FDisplayClusterClusterEventJson>>& OutJsonEvents,      TArray<TSharedPtr<FDisplayClusterClusterEventBinary>>& OutBinaryEvents) = 0;

	/** Imports and processes the cluster events (come from the primary node) */
	virtual void ImportEventsData(const TArray<TSharedPtr<FDisplayClusterClusterEventJson>>& InJsonEvents, const TArray<TSharedPtr<FDisplayClusterClusterEventBinary>>& InBinaryEvents) = 0;

public: // Native input sync

	/** Exports native input data from the local PlayerInput on the primary node */
	virtual void ExportNativeInputData(TMap<FString, TArray<uint8>>& OutNativeInputData) = 0;

	/** Imports and applies the native input data from the the primary node */
	virtual void ImportNativeInputData(TMap<FString, TArray<uint8>>& InNativeInputData) = 0;
};
