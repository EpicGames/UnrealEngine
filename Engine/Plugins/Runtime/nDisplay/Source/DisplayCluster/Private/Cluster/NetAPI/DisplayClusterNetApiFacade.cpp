// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cluster/NetAPI/DisplayClusterNetApiFacade.h"

#include "Cluster/Failover/IDisplayClusterFailoverNodeController.h"


/**
 * Base class to encapsulate common functionality of the concrete API facades
 */
class FClientAPIBase
{
public:

	/** Constructor */
	FClientAPIBase(TSharedRef<IDisplayClusterFailoverNodeController>& InFailoverController)
		: FailoverController(InFailoverController)
	{ }

protected:

	/** Returns failover controller */
	TSharedRef<IDisplayClusterFailoverNodeController>& GetFailoverController()
	{
		return FailoverController;
	}

private:

	/** Currently active failover controller */
	TSharedRef<IDisplayClusterFailoverNodeController> FailoverController;
};


/**
 * ClusterSync API facade. Wraps the lower level ClusterSync protocol calls into failover transactions.
 */
class FClusterSyncAPI
	: public FClientAPIBase
	, public IDisplayClusterProtocolClusterSync
{
public:

	/** Constructor */
	FClusterSyncAPI(TSharedRef<IDisplayClusterFailoverNodeController>& InFailoverController)
		: FClientAPIBase(InFailoverController)
	{ }

public:

	//~ Begin IDisplayClusterProtocolClusterSync

	/** Failover transaction for WaitForGameStart */
	virtual EDisplayClusterCommResult WaitForGameStart() override
	{
		checkSlow(IsInGameThread());
		return GetFailoverController()->WaitForGameStart();
	}

	/** Failover transaction for WaitForFrameStart */
	virtual EDisplayClusterCommResult WaitForFrameStart() override
	{
		checkSlow(IsInGameThread());
		return GetFailoverController()->WaitForFrameStart();
	}

	/** Failover transaction for WaitForFrameEnd */
	virtual EDisplayClusterCommResult WaitForFrameEnd() override
	{
		checkSlow(IsInGameThread());
		return GetFailoverController()->WaitForFrameEnd();
	}

	/** Failover transaction for GetTimeData */
	virtual EDisplayClusterCommResult GetTimeData(double& OutDeltaTime, double& OutGameTime, TOptional<FQualifiedFrameTime>& OutFrameTime) override
	{
		checkSlow(IsInGameThread());
		return GetFailoverController()->GetTimeData(OutDeltaTime, OutGameTime, OutFrameTime);
	}

	/** Failover transaction for GetObjectsData */
	virtual EDisplayClusterCommResult GetObjectsData(EDisplayClusterSyncGroup InSyncGroup, TMap<FString, TArray<uint8>>& OutObjectsData) override
	{
		checkSlow(IsInGameThread());
		return GetFailoverController()->GetObjectsData(InSyncGroup, OutObjectsData);
	}

	/** Failover transaction for GetEventsData */
	virtual EDisplayClusterCommResult GetEventsData(
		TArray<TSharedPtr<FDisplayClusterClusterEventJson>>& OutJsonEvents,
		TArray<TSharedPtr<FDisplayClusterClusterEventBinary>>& OutBinaryEvents) override
	{
		checkSlow(IsInGameThread());
		return GetFailoverController()->GetEventsData(OutJsonEvents, OutBinaryEvents);
	}

	/** Failover transaction for GetNativeInputData */
	virtual EDisplayClusterCommResult GetNativeInputData(TMap<FString, TArray<uint8>>& OutNativeInputData) override
	{
		checkSlow(IsInGameThread());
		return GetFailoverController()->GetNativeInputData(OutNativeInputData);
	}

	/** Failover transaction for PropagateStatesData */
	virtual EDisplayClusterCommResult PropagateStatesData(const TArray<uint8>& InLocalStatesData, TArray<uint8>& OutClusterStatesData) override
	{
		checkSlow(IsInGameThread());
		return GetFailoverController()->PropagateStatesData(InLocalStatesData, OutClusterStatesData);
	}

	//~ End IDisplayClusterProtocolClusterSync
};


/**
 * RenderSync API facade. Wraps the lower level RenderSync protocol calls into failover transactions.
 */
class FRenderSyncAPI
	: public FClientAPIBase
	, public IDisplayClusterProtocolRenderSync
{
public:

	/** Constructor */
	FRenderSyncAPI(TSharedRef<IDisplayClusterFailoverNodeController>& InFailoverController)
		: FClientAPIBase(InFailoverController)
	{ }

public:

	//~ Begin IDisplayClusterProtocolRenderSync

	/** Failover transaction for SynchronizeOnBarrier */
	virtual EDisplayClusterCommResult SynchronizeOnBarrier() override
	{
		return static_cast<IDisplayClusterProtocolRenderSync&>(GetFailoverController().Get()).SynchronizeOnBarrier();
	}

	//~ End IDisplayClusterProtocolRenderSync
};


/**
 * Binary events API facade. Wraps the lower level BinaryEvents protocol calls into failover transactions.
 */
class FBinaryEventsAPI
	: public FClientAPIBase
	, public IDisplayClusterProtocolEventsBinary
{
public:

	/** Constructor */
	FBinaryEventsAPI(TSharedRef<IDisplayClusterFailoverNodeController>& InFailoverController)
		: FClientAPIBase(InFailoverController)
	{ }

public:

	//~ Begin IDisplayClusterProtocolEventsBinary

	/** Failover transaction for EmitClusterEventBinary */
	virtual EDisplayClusterCommResult EmitClusterEventBinary(const FDisplayClusterClusterEventBinary& Event) override
	{
		return GetFailoverController()->EmitClusterEventBinary(Event);
	}

	//~ End IDisplayClusterProtocolEventsBinary
};


/**
 * JSON events API facade. Wraps the lower level JsonEvents protocol calls into failover transactions.
 */
class FJsonEventsAPI
	: public FClientAPIBase
	, public IDisplayClusterProtocolEventsJson
{
public:

	/** Constructor */
	FJsonEventsAPI(TSharedRef<IDisplayClusterFailoverNodeController>& InFailoverController)
		: FClientAPIBase(InFailoverController)
	{ }

public:

	//~ Begin IDisplayClusterProtocolEventsJson

	/** Failover transaction for EmitClusterEventJson */
	virtual EDisplayClusterCommResult EmitClusterEventJson(const FDisplayClusterClusterEventJson& Event) override
	{
		return GetFailoverController()->EmitClusterEventJson(Event);
	}

	//~ End IDisplayClusterProtocolEventsJson
};


/**
 * Generic barrier API facade. Wraps the lower level GenericBarrier protocol calls into failover transactions.
 */
class FGenericBarrierAPI
	: public FClientAPIBase
	, public IDisplayClusterProtocolGenericBarrier
{
public:

	/** Constructor */
	FGenericBarrierAPI(TSharedRef<IDisplayClusterFailoverNodeController>& InFailoverController)
		: FClientAPIBase(InFailoverController)
	{ }

public:

	//~ Begin IDisplayClusterProtocolGenericBarrier

	/** Failover transaction for CreateBarrier */
	virtual EDisplayClusterCommResult CreateBarrier(const FString& BarrierId, const TMap<FString, TSet<FString>>& NodeToSyncCallers, uint32 Timeout, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result) override
	{
		return GetFailoverController()->CreateBarrier(BarrierId, NodeToSyncCallers, Timeout, Result);
	}

	/** Failover transaction for WaitUntilBarrierIsCreated */
	virtual EDisplayClusterCommResult WaitUntilBarrierIsCreated(const FString& BarrierId, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result) override
	{
		return GetFailoverController()->WaitUntilBarrierIsCreated(BarrierId, Result);
	}

	/** Failover transaction for IsBarrierAvailable */
	virtual EDisplayClusterCommResult IsBarrierAvailable(const FString& BarrierId, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result) override
	{
		return GetFailoverController()->IsBarrierAvailable(BarrierId, Result);
	}

	/** Failover transaction for ReleaseBarrier */
	virtual EDisplayClusterCommResult ReleaseBarrier(const FString& BarrierId, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result) override
	{
		return GetFailoverController()->ReleaseBarrier(BarrierId, Result);
	}

	/** Failover transaction for SyncOnBarrier */
	virtual EDisplayClusterCommResult SyncOnBarrier(const FString& BarrierId, const FString& CallerId, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result) override
	{
		return static_cast<IDisplayClusterProtocolGenericBarrier&>(GetFailoverController().Get()).SyncOnBarrier(BarrierId, CallerId, Result);
	}

	/** Failover transaction for SyncOnBarrierWithData */
	virtual EDisplayClusterCommResult SyncOnBarrierWithData(const FString& BarrierId, const FString& CallerId, const TArray<uint8>& RequestData, TArray<uint8>& OutResponseData, IDisplayClusterProtocolGenericBarrier::EBarrierControlResult& Result) override
	{
		return GetFailoverController()->SyncOnBarrierWithData(BarrierId, CallerId, RequestData, OutResponseData, Result);
	}

	//~ End IDisplayClusterProtocolGenericBarrier
};

FDisplayClusterNetApiFacade::FDisplayClusterNetApiFacade(TSharedRef<IDisplayClusterFailoverNodeController>& InFailoverController)
	: ClusterSyncAPI(MakeShared<FClusterSyncAPI>(InFailoverController))
	, RenderSyncAPI(MakeShared<FRenderSyncAPI>(InFailoverController))
	, BinaryEventsAPI(MakeShared<FBinaryEventsAPI>(InFailoverController))
	, JsonEventsAPI(MakeShared<FJsonEventsAPI>(InFailoverController))
	, GenericBarrierAPI(MakeShared<FGenericBarrierAPI>(InFailoverController))
{
}
