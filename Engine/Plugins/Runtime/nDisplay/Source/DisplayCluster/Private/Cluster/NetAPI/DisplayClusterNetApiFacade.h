// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Network/Protocol/IDisplayClusterProtocolClusterSync.h"
#include "Network/Protocol/IDisplayClusterProtocolEventsBinary.h"
#include "Network/Protocol/IDisplayClusterProtocolEventsJson.h"
#include "Network/Protocol/IDisplayClusterProtocolGenericBarrier.h"
#include "Network/Protocol/IDisplayClusterProtocolRenderSync.h"

class IDisplayClusterFailoverNodeController;


/**
 * An auxiliary facade class that provides the networking API without any direct access
 * to the networking internals. It encapsulates the in-cluster role based behavior,
 * failover and networking subsystems, keeping the API transparent to the clients.
 *
 * This is the entry point for all networking requests. At this step, we route all
 * the net calls to the current failover controller, which is the next step in the pipeline.
 */
class FDisplayClusterNetApiFacade
{
public:

	FDisplayClusterNetApiFacade(TSharedRef<IDisplayClusterFailoverNodeController>& InFailoverController);

public:

	/** Access to the cluster sync API (Game thread only) */
	TSharedRef<IDisplayClusterProtocolClusterSync> GetClusterSyncAPI()
	{
		check(IsInGameThread());
		return ClusterSyncAPI;
	}

	/** Access to the render sync API (RHI thread only) */
	TSharedRef<IDisplayClusterProtocolRenderSync> GetRenderSyncAPI()
	{
		return RenderSyncAPI;
	}

	/** Access to binary events API (ANY thread) */
	TSharedRef<IDisplayClusterProtocolEventsBinary> GetBinaryEventsAPI()
	{
		return BinaryEventsAPI;
	}

	/** Access to JSON events API (ANY thread) */
	TSharedRef<IDisplayClusterProtocolEventsJson> GetJsonEventsAPI()
	{
		return JsonEventsAPI;
	}

protected:

	/** GenericBarrier API is exposed to FDisplayClusterGenericBarrierAPI only */
	friend class FDisplayClusterGenericBarrierAPI;

	/** Access to generic barrier API (ANY thread) */
	TSharedRef<IDisplayClusterProtocolGenericBarrier> GetGenericBarrierAPI()
	{
		return GenericBarrierAPI;
	}

private:

	/** ClusterSync API */
	TSharedRef<IDisplayClusterProtocolClusterSync> ClusterSyncAPI;

	/** RenderSync API */
	TSharedRef<IDisplayClusterProtocolRenderSync> RenderSyncAPI;

	/** Binary events API */
	TSharedRef<IDisplayClusterProtocolEventsBinary> BinaryEventsAPI;

	/** JSON events API */
	TSharedRef<IDisplayClusterProtocolEventsJson> JsonEventsAPI;

	/** Generic barrier API */
	TSharedRef<IDisplayClusterProtocolGenericBarrier> GenericBarrierAPI;
};
