// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Network/Protocol/IDisplayClusterProtocolClusterSync.h"


/**
 * Base class for comm requests handling
 */
class FDisplayClusterCommRequestHandlerBase
	: public IDisplayClusterProtocolClusterSync
{
public:

	FDisplayClusterCommRequestHandlerBase() = default;
	virtual ~FDisplayClusterCommRequestHandlerBase() = default;
};
