// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cluster/RequestHandler/DisplayClusterCommRequestHandlerLocal.h"

#include "Cluster/IPDisplayClusterClusterManager.h"
#include "Cluster/Failover/DisplayClusterCommDataCache.h"
#include "Misc/DisplayClusterGlobals.h"


FDisplayClusterCommRequestHandlerLocal& FDisplayClusterCommRequestHandlerLocal::Get()
{
	static FDisplayClusterCommRequestHandlerLocal Instance;
	return Instance;
}

EDisplayClusterCommResult FDisplayClusterCommRequestHandlerLocal::WaitForGameStart()
{
	// Not used currently
	return EDisplayClusterCommResult::NotImplemented;
}

EDisplayClusterCommResult FDisplayClusterCommRequestHandlerLocal::WaitForFrameStart()
{
	// Not used currently
	return EDisplayClusterCommResult::NotImplemented;
}

EDisplayClusterCommResult FDisplayClusterCommRequestHandlerLocal::WaitForFrameEnd()
{
	// Not used currently
	return EDisplayClusterCommResult::NotImplemented;
}

EDisplayClusterCommResult FDisplayClusterCommRequestHandlerLocal::GetTimeData(double& OutDeltaTime, double& OutGameTime, TOptional<FQualifiedFrameTime>& OutFrameTime)
{
	const TSharedRef<FDisplayClusterCommDataCache> DataCache = GDisplayCluster->GetPrivateClusterMgr()->GetDataCache();

	if (DataCache->GetTimeData_OpIsCached())
	{
		DataCache->GetTimeData_OpLoad(OutDeltaTime, OutGameTime, OutFrameTime);
	}
	else
	{
		IPDisplayClusterClusterManager* const ClusterMgr = GDisplayCluster->GetPrivateClusterMgr();
		ClusterMgr->CacheTimeData();
		ClusterMgr->ExportTimeData(OutDeltaTime, OutGameTime, OutFrameTime);
	}

	return EDisplayClusterCommResult::Ok;
}

EDisplayClusterCommResult FDisplayClusterCommRequestHandlerLocal::GetObjectsData(EDisplayClusterSyncGroup InSyncGroup, TMap<FString, TArray<uint8>>& OutObjectsData)
{
	const TSharedRef<FDisplayClusterCommDataCache> DataCache = GDisplayCluster->GetPrivateClusterMgr()->GetDataCache();

	if (DataCache->GetObjectsData_OpIsCached(InSyncGroup)())
	{
		DataCache->GetObjectsData_OpLoad(InSyncGroup, OutObjectsData);
	}
	else
	{
		IPDisplayClusterClusterManager* const ClusterMgr = GDisplayCluster->GetPrivateClusterMgr();
		ClusterMgr->CacheObjects(InSyncGroup);
		ClusterMgr->ExportObjectsData(InSyncGroup, OutObjectsData);
	}

	return EDisplayClusterCommResult::Ok;
}

EDisplayClusterCommResult FDisplayClusterCommRequestHandlerLocal::GetEventsData(TArray<TSharedPtr<FDisplayClusterClusterEventJson>>& OutJsonEvents, TArray<TSharedPtr<FDisplayClusterClusterEventBinary>>& OutBinaryEvents)
{
	const TSharedRef<FDisplayClusterCommDataCache> DataCache = GDisplayCluster->GetPrivateClusterMgr()->GetDataCache();

	if (DataCache->GetEventsData_OpIsCached())
	{
		DataCache->GetEventsData_OpLoad(OutJsonEvents, OutBinaryEvents);
	}
	else
	{
		IPDisplayClusterClusterManager* const ClusterMgr = GDisplayCluster->GetPrivateClusterMgr();
		ClusterMgr->CacheEvents();
		ClusterMgr->ExportEventsData(OutJsonEvents, OutBinaryEvents);
	}

	return EDisplayClusterCommResult::Ok;
}

EDisplayClusterCommResult FDisplayClusterCommRequestHandlerLocal::GetNativeInputData(TMap<FString, TArray<uint8>>& OutNativeInputData)
{
	const TSharedRef<FDisplayClusterCommDataCache> DataCache = GDisplayCluster->GetPrivateClusterMgr()->GetDataCache();

	if (DataCache->GetNativeInputData_OpIsCached())
	{
		DataCache->GetNativeInputData_OpLoad(OutNativeInputData);
	}
	else
	{
		IPDisplayClusterClusterManager* const ClusterMgr = GDisplayCluster->GetPrivateClusterMgr();
		ClusterMgr->ExportNativeInputData(OutNativeInputData);
	}

	return EDisplayClusterCommResult::Ok;
}

EDisplayClusterCommResult FDisplayClusterCommRequestHandlerLocal::PropagateStatesData(const TArray<uint8>& InLocalStatesData, TArray<uint8>& OutClusterStatesData)
{
	// Not used currently
	return EDisplayClusterCommResult::NotImplemented;
}
