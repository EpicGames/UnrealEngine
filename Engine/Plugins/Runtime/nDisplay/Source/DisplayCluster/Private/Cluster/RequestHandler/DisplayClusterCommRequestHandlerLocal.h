// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Cluster/RequestHandler/DisplayClusterCommRequestHandlerBase.h"
#include "Misc/CoreMiscDefines.h"
#include "Network/DisplayClusterNetworkTypes.h"


/**
 * Local request handler
 * 
 * Handles comm requests that were sent by this node. Also encapsulates
 * interaction with cached synchronization data (failover/recovery).
 */
class FDisplayClusterCommRequestHandlerLocal
	: public FDisplayClusterCommRequestHandlerBase
{
public:

	UE_NONCOPYABLE(FDisplayClusterCommRequestHandlerLocal);

public:

	/** Singleton access */
	static FDisplayClusterCommRequestHandlerLocal& Get();

private:

	FDisplayClusterCommRequestHandlerLocal() = default;
	virtual ~FDisplayClusterCommRequestHandlerLocal() = default;

public:

	//~ Begin IDisplayClusterProtocolClusterSync
	virtual EDisplayClusterCommResult WaitForGameStart() override;
	virtual EDisplayClusterCommResult WaitForFrameStart() override;
	virtual EDisplayClusterCommResult WaitForFrameEnd() override;
	virtual EDisplayClusterCommResult GetTimeData(double& OutDeltaTime, double& OutGameTime, TOptional<FQualifiedFrameTime>& OutFrameTime) override;
	virtual EDisplayClusterCommResult GetObjectsData(EDisplayClusterSyncGroup InSyncGroup, TMap<FString, TArray<uint8>>& OutObjectsData) override;
	virtual EDisplayClusterCommResult GetEventsData(TArray<TSharedPtr<FDisplayClusterClusterEventJson>>& OutJsonEvents, TArray<TSharedPtr<FDisplayClusterClusterEventBinary>>& OutBinaryEvents) override;
	virtual EDisplayClusterCommResult GetNativeInputData(TMap<FString, TArray<uint8>>& OutNativeInputData) override;
	virtual EDisplayClusterCommResult PropagateStatesData(const TArray<uint8>& InLocalStatesData, TArray<uint8>& OutClusterStatesData) override;
	//~ End IDisplayClusterProtocolClusterSync
};
