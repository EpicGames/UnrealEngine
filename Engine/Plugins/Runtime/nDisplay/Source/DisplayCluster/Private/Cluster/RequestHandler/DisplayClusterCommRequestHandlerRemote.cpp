// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cluster/RequestHandler/DisplayClusterCommRequestHandlerRemote.h"

#include "Cluster/IPDisplayClusterClusterManager.h"
#include "Cluster/Failover/DisplayClusterCommDataCache.h"
#include "Misc/DisplayClusterGlobals.h"


FDisplayClusterCommRequestHandlerRemote& FDisplayClusterCommRequestHandlerRemote::Get()
{
	static FDisplayClusterCommRequestHandlerRemote Instance;
	return Instance;
}

EDisplayClusterCommResult FDisplayClusterCommRequestHandlerRemote::WaitForGameStart()
{
	// Not used currently
	return EDisplayClusterCommResult::NotImplemented;
}

EDisplayClusterCommResult FDisplayClusterCommRequestHandlerRemote::WaitForFrameStart()
{
	// Not used currently
	return EDisplayClusterCommResult::NotImplemented;
}

EDisplayClusterCommResult FDisplayClusterCommRequestHandlerRemote::WaitForFrameEnd()
{
	// Not used currently
	return EDisplayClusterCommResult::NotImplemented;
}

EDisplayClusterCommResult FDisplayClusterCommRequestHandlerRemote::GetTimeData(double& OutDeltaTime, double& OutGameTime, TOptional<FQualifiedFrameTime>& OutFrameTime)
{
	const TSharedRef<FDisplayClusterCommDataCache> DataCache = GDisplayCluster->GetPrivateClusterMgr()->GetDataCache();

	if (DataCache->GetTimeData_OpIsCached())
	{
		DataCache->GetTimeData_OpLoad(OutDeltaTime, OutGameTime, OutFrameTime);
	}
	else
	{
		IPDisplayClusterClusterManager* const ClusterMgr = GDisplayCluster->GetPrivateClusterMgr();
		ClusterMgr->ExportTimeData(OutDeltaTime, OutGameTime, OutFrameTime);
	}

	return EDisplayClusterCommResult::Ok;
}

EDisplayClusterCommResult FDisplayClusterCommRequestHandlerRemote::GetObjectsData(EDisplayClusterSyncGroup InSyncGroup, TMap<FString, TArray<uint8>>& OutObjectsData)
{
	const TSharedRef<FDisplayClusterCommDataCache> DataCache = GDisplayCluster->GetPrivateClusterMgr()->GetDataCache();

	if (DataCache->GetObjectsData_OpIsCached(InSyncGroup)())
	{
		DataCache->GetObjectsData_OpLoad(InSyncGroup, OutObjectsData);
	}
	else
	{
		IPDisplayClusterClusterManager* const ClusterMgr = GDisplayCluster->GetPrivateClusterMgr();
		ClusterMgr->ExportObjectsData(InSyncGroup, OutObjectsData);
	}

	return EDisplayClusterCommResult::Ok;
}

EDisplayClusterCommResult FDisplayClusterCommRequestHandlerRemote::GetEventsData(TArray<TSharedPtr<FDisplayClusterClusterEventJson>>& OutJsonEvents, TArray<TSharedPtr<FDisplayClusterClusterEventBinary>>& OutBinaryEvents)
{
	const TSharedRef<FDisplayClusterCommDataCache> DataCache = GDisplayCluster->GetPrivateClusterMgr()->GetDataCache();

	if (DataCache->GetEventsData_OpIsCached())
	{
		DataCache->GetEventsData_OpLoad(OutJsonEvents, OutBinaryEvents);
	}
	else
	{
		IPDisplayClusterClusterManager* const ClusterMgr = GDisplayCluster->GetPrivateClusterMgr();
		ClusterMgr->ExportEventsData(OutJsonEvents, OutBinaryEvents);
	}

	return EDisplayClusterCommResult::Ok;
}

EDisplayClusterCommResult FDisplayClusterCommRequestHandlerRemote::GetNativeInputData(TMap<FString, TArray<uint8>>& OutNativeInputData)
{
	const TSharedRef<FDisplayClusterCommDataCache> DataCache = GDisplayCluster->GetPrivateClusterMgr()->GetDataCache();

	if (DataCache->GetNativeInputData_OpIsCached())
	{
		DataCache->GetNativeInputData_OpLoad(OutNativeInputData);
	}
	else
	{
		IPDisplayClusterClusterManager* const ClusterMgr = GDisplayCluster->GetPrivateClusterMgr();
		ClusterMgr->ExportNativeInputData(OutNativeInputData);
	}

	return EDisplayClusterCommResult::Ok;
}

EDisplayClusterCommResult FDisplayClusterCommRequestHandlerRemote::PropagateStatesData(const TArray<uint8>& InLocalStatesData, TArray<uint8>& OutClusterStatesData)
{
	// Not used currently
	return EDisplayClusterCommResult::NotImplemented;
}
