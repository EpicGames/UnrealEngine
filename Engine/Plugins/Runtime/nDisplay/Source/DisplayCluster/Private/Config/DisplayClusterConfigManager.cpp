// Copyright Epic Games, Inc. All Rights Reserved.

#include "Config/DisplayClusterConfigManager.h"

#include "IPDisplayCluster.h"
#include "Misc/DisplayClusterGlobals.h"
#include "Cluster/IPDisplayClusterClusterManager.h"

#include "DisplayClusterConfigurationTypes.h"

#include "Misc/Paths.h"

#include "Misc/DisplayClusterGlobals.h"
#include "Misc/DisplayClusterStrings.h"
#include "Misc/DisplayClusterLog.h"

#include "HAL/FileManager.h"


//////////////////////////////////////////////////////////////////////////////////////////////
// IPDisplayClusterManager
//////////////////////////////////////////////////////////////////////////////////////////////
bool FDisplayClusterConfigManager::Init(EDisplayClusterOperationMode OperationMode)
{
	return true;
}

void FDisplayClusterConfigManager::Release()
{
}

bool FDisplayClusterConfigManager::StartSession(UDisplayClusterConfigurationData* InConfigData, const FString& InNodeId)
{
	ClusterNodeId = InNodeId;

	ConfigData.Reset(InConfigData);
	return ConfigData.IsValid();
}

void FDisplayClusterConfigManager::EndSession()
{
	ClusterNodeId.Empty();
	ConfigData.Reset();
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterConfigManager
//////////////////////////////////////////////////////////////////////////////////////////////
FString FDisplayClusterConfigManager::GetPrimaryNodeId() const
{
	return ConfigData ? ConfigData->Cluster->PrimaryNode.Id : FString();
}

const UDisplayClusterConfigurationClusterNode* FDisplayClusterConfigManager::GetPrimaryNode() const
{
	return ConfigData ? ConfigData->Cluster->GetNode(GetPrimaryNodeId()) : nullptr;
}

const UDisplayClusterConfigurationClusterNode* FDisplayClusterConfigManager::GetLocalNode() const
{
	return ConfigData ? ConfigData->Cluster->GetNode(GetLocalNodeId()) : nullptr;
}

const UDisplayClusterConfigurationViewport* FDisplayClusterConfigManager::GetLocalViewport(const FString& ViewportId) const
{
	return ConfigData ? ConfigData->GetViewport(GetLocalNodeId(), ViewportId) : nullptr;
}

bool FDisplayClusterConfigManager::GetLocalPostprocess(const FString& PostprocessId, FDisplayClusterConfigurationPostprocess& OutPostprocess) const
{
	return ConfigData ? ConfigData->GetPostprocess(GetLocalNodeId(), PostprocessId, OutPostprocess) : false;
}

bool FDisplayClusterConfigManager::GetLocalProjection(const FString& ViewportId, FDisplayClusterConfigurationProjection& OutProjection) const
{
	return ConfigData ? ConfigData->GetProjectionPolicy(GetLocalNodeId(), ViewportId, OutProjection) : false;
}
