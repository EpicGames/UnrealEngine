// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Config/IPDisplayClusterConfigManager.h"
#include "DisplayClusterConfigurationTypes.h"
#include "UObject/StrongObjectPtr.h"


/**
 * Config manager. Responsible for loading data from config file and providing with it to any other classes.
 */
class FDisplayClusterConfigManager
	: public IPDisplayClusterConfigManager
{
public:
	FDisplayClusterConfigManager() = default;
	virtual ~FDisplayClusterConfigManager() = default;

public:
	//////////////////////////////////////////////////////////////////////////////////////////////
	// IPDisplayClusterManager
	//////////////////////////////////////////////////////////////////////////////////////////////
	virtual bool Init(EDisplayClusterOperationMode OperationMode) override;
	virtual void Release() override;
	virtual bool StartSession(UDisplayClusterConfigurationData* InConfigData, const FString& NodeId) override;
	virtual void EndSession() override;

public:
	//////////////////////////////////////////////////////////////////////////////////////////////
	// IDisplayClusterConfigManager
	//////////////////////////////////////////////////////////////////////////////////////////////
	virtual UDisplayClusterConfigurationData* GetConfig() const override
	{
		return ConfigData.Get();
	}

	virtual FString GetConfigPath() const override
	{
		return ConfigData.IsValid() ? ConfigData.Get()->Meta.ImportFilePath : FString();
	}

	virtual FString GetLocalNodeId() const override
	{
		return ClusterNodeId; 
	}

	virtual FString GetPrimaryNodeId() const override;

	virtual const UDisplayClusterConfigurationClusterNode* GetPrimaryNode() const override;
	virtual const UDisplayClusterConfigurationClusterNode* GetLocalNode() const override;
	virtual const UDisplayClusterConfigurationViewport*    GetLocalViewport(const FString& ViewportId) const override;

	virtual bool GetLocalPostprocess(const FString& PostprocessId, FDisplayClusterConfigurationPostprocess& OutPostprocess) const override;
	virtual bool GetLocalProjection(const FString& ViewportId, FDisplayClusterConfigurationProjection& OutProjection) const override;

private:
	FString ClusterNodeId;

	TStrongObjectPtr<UDisplayClusterConfigurationData> ConfigData;
};
