// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Config/IDisplayClusterConfigManager.h"
#include "IPDisplayClusterManager.h"


/**
 * Config manager private interface
 */
class IPDisplayClusterConfigManager
	: public IDisplayClusterConfigManager
	, public IPDisplayClusterManager
{
public:
	virtual ~IPDisplayClusterConfigManager() = default;
};
