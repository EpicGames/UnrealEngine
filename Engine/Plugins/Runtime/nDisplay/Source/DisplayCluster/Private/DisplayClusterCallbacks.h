// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDisplayClusterCallbacks.h"


/**
 * DisplayCluster callbacks API implementation
 */
class FDisplayClusterCallbacks :
	public  IDisplayClusterCallbacks
{
public:
	virtual FDisplayClusterStartSessionEvent& OnDisplayClusterStartSession() override
	{
		return DisplayClusterStartSessionEvent;
	}

	virtual FDisplayClusterEndSessionEvent& OnDisplayClusterEndSession() override
	{
		return DisplayClusterEndSessionEvent;
	}

	virtual FDisplayClusterStartFrameEvent& OnDisplayClusterStartFrame() override
	{
		return DisplayClusterStartFrameEvent;
	}

	virtual FDisplayClusterEndFrameEvent& OnDisplayClusterEndFrame() override
	{
		return DisplayClusterEndFrameEvent;
	}

	virtual FDisplayClusterPreTickEvent& OnDisplayClusterPreTick() override
	{
		return DisplayClusterPreTickEvent;
	}

	virtual FDisplayClusterTickEvent& OnDisplayClusterTick() override
	{
		return DisplayClusterTickEvent;
	}

	virtual FDisplayClusterPostTickEvent& OnDisplayClusterPostTick() override
	{
		return DisplayClusterPostTickEvent;
	}

	virtual FDisplayClusterStartSceneEvent& OnDisplayClusterStartScene() override
	{
		return DisplayClusterStartSceneEvent;
	}

	virtual FDisplayClusterEndSceneEvent& OnDisplayClusterEndScene() override
	{
		return DisplayClusterEndSceneEvent;
	}

	virtual FDisplayClusterPreSubmitViewFamilies& OnDisplayClusterPreSubmitViewFamilies() override
	{
		return DisplayClusterPreSubmitViewFamiliesEvent;
	}

	virtual FDisplayClusterCustomPresentSetEvent& OnDisplayClusterCustomPresentSet() override
	{
		return DisplayClusterCustomPresentSetEvent;
	}

	virtual FDisplayClusterPresentationPreSynchronization_RHIThread& OnDisplayClusterPresentationPreSynchronization_RHIThread() override
	{
		return DisplayClusterPresentationPreSynchronizationEvent;
	}

	virtual FDisplayClusterPresentationPostSynchronization_RHIThread& OnDisplayClusterPresentationPostSynchronization_RHIThread() override
	{
		return DisplayClusterPresentationPostSynchronizationEvent;
	}

	virtual FDisplayClusterFramePresentated_RHIThread& OnDisplayClusterFramePresented_RHIThread() override
	{
		return DisplayClusterFramePresentedEvent;
	}

	virtual FDisplayClusterFailoverNodeDown& OnDisplayClusterFailoverNodeDown() override
	{
		return DisplayClusterFailoverNodeDownEvent;
	}

	virtual FDisplayClusterFailoverPrimaryNodeChanged& OnDisplayClusterFailoverPrimaryNodeChanged() override
	{
		return DisplayClusterFailoverPrimaryNodeChangedEvent;
	}

	virtual FDisplayClusterPostTonemapPass_RenderThread& OnDisplayClusterPostTonemapPass_RenderThread() override
	{
		return DisplayClusterPostTonemapPassEvent;
	}

	virtual FDisplayClusterPostRenderViewFamily_RenderThread& OnDisplayClusterPostRenderViewFamily_RenderThread() override
	{
		return DisplayClusterPostRenderViewFamilyEvent;
	}

	virtual FDisplayClusterPreWarp_RenderThread& OnDisplayClusterPreWarp_RenderThread() override
	{
		return DisplayClusterPreWarpEvent;
	}

	virtual FDisplayClusterPreWarpViewport_RenderThread& OnDisplayClusterPreWarpViewport_RenderThread() override
	{
		return DisplayClusterPreWarpViewportEvent;
	}

	virtual FDisplayClusterPostWarp_RenderThread& OnDisplayClusterPostWarp_RenderThread() override
	{
		return DisplayClusterPostWarpEvent;
	}

	virtual FDisplayClusterPostWarpViewport_RenderThread& OnDisplayClusterPostWarpViewport_RenderThread() override
	{
		return DisplayClusterPostWarpViewportEvent;
	}

	virtual FDisplayClusterPostCrossGpuTransfer_RenderThread& OnDisplayClusterPostCrossGpuTransfer_RenderThread() override
	{
		return DisplayClusterPostCrossGpuTransferEvent;
	}

	virtual FDisplayClusterPassthroughMediaCapture_RenderThread& OnDisplayClusterPassthroughMediaCapture_RenderThread() override
	{
		return DisplayClusterPassthroughMediaCaptureEvent;
	}

	virtual FDisplayClusterPassthroughMediaInput_RenderThread& OnDisplayClusterPassthroughMediaInput_RenderThread() override
	{
		return DisplayClusterPassthroughMediaInputEvent;
	}

	virtual FDisplayClusterProcessLatency_RenderThread& OnDisplayClusterProcessLatency_RenderThread() override
	{
		return DisplayClusterProcessLatencyEvent;
	}

	virtual FDisplayClusterPostResolveOverridden_RenderThread& OnDisplayClusterPostResolveOverridden_RenderThread() override
	{
		return DisplayClusterPostResolveOverriddenEvent;
	}

	virtual FDisplayClusterPostFrameRender_RenderThread& OnDisplayClusterPostFrameRender_RenderThread() override
	{
		return DisplayClusterPostFrameRenderEvent;
	}

	virtual FDisplayClusterPostBackbufferUpdate_RenderThread& OnDisplayClusterPostBackbufferUpdate_RenderThread() override
	{
		return DisplayClusterPostBackbufferUpdateEvent;
	}

	virtual FDisplayClusterPostBackbufferUpdated_RenderThread& OnDisplayClusterPostBackbufferUpdated_RenderThread() override
	{
		return DisplayClusterPostBackbufferUpdatedEvent;
	}

	virtual FDisplayClusterPreProcessIcvfx_RenderThread& OnDisplayClusterPreProcessIcvfx_RenderThread() override
	{
		return DisplayClusterPreProcessIcvfxEvent;
	}

	virtual FDisplayClusterUpdateViewportMediaState& OnDisplayClusterUpdateViewportMediaState() override
	{
		return DisplayClusterUpdateViewportMediaStateEvent;
	}

private:
	FDisplayClusterStartSessionEvent         DisplayClusterStartSessionEvent;
	FDisplayClusterEndSessionEvent           DisplayClusterEndSessionEvent;
	FDisplayClusterStartFrameEvent           DisplayClusterStartFrameEvent;
	FDisplayClusterEndFrameEvent             DisplayClusterEndFrameEvent;
	FDisplayClusterPreTickEvent              DisplayClusterPreTickEvent;
	FDisplayClusterTickEvent                 DisplayClusterTickEvent;
	FDisplayClusterPostTickEvent             DisplayClusterPostTickEvent;
	FDisplayClusterStartSceneEvent           DisplayClusterStartSceneEvent;
	FDisplayClusterEndSceneEvent             DisplayClusterEndSceneEvent;
	FDisplayClusterPreSubmitViewFamilies     DisplayClusterPreSubmitViewFamiliesEvent;
	FDisplayClusterCustomPresentSetEvent     DisplayClusterCustomPresentSetEvent;
	FDisplayClusterUpdateViewportMediaState  DisplayClusterUpdateViewportMediaStateEvent;
	FDisplayClusterFailoverNodeDown          DisplayClusterFailoverNodeDownEvent;
	FDisplayClusterFailoverPrimaryNodeChanged DisplayClusterFailoverPrimaryNodeChangedEvent;

	FDisplayClusterPresentationPreSynchronization_RHIThread  DisplayClusterPresentationPreSynchronizationEvent;
	FDisplayClusterPresentationPostSynchronization_RHIThread DisplayClusterPresentationPostSynchronizationEvent;
	FDisplayClusterFramePresentated_RHIThread                DisplayClusterFramePresentedEvent;

	FDisplayClusterPostTonemapPass_RenderThread      DisplayClusterPostTonemapPassEvent;
	FDisplayClusterPostRenderViewFamily_RenderThread DisplayClusterPostRenderViewFamilyEvent;
	FDisplayClusterPreWarp_RenderThread              DisplayClusterPreWarpEvent;
	FDisplayClusterPreWarpViewport_RenderThread      DisplayClusterPreWarpViewportEvent;
	FDisplayClusterPostWarp_RenderThread             DisplayClusterPostWarpEvent;
	FDisplayClusterPostWarpViewport_RenderThread     DisplayClusterPostWarpViewportEvent;
	FDisplayClusterPostCrossGpuTransfer_RenderThread DisplayClusterPostCrossGpuTransferEvent;
	FDisplayClusterPassthroughMediaCapture_RenderThread DisplayClusterPassthroughMediaCaptureEvent;
	FDisplayClusterPassthroughMediaInput_RenderThread   DisplayClusterPassthroughMediaInputEvent;
	FDisplayClusterProcessLatency_RenderThread       DisplayClusterProcessLatencyEvent;
	FDisplayClusterPostResolveOverridden_RenderThread DisplayClusterPostResolveOverriddenEvent;
	FDisplayClusterPostFrameRender_RenderThread      DisplayClusterPostFrameRenderEvent;
	FDisplayClusterPostBackbufferUpdate_RenderThread DisplayClusterPostBackbufferUpdateEvent;
	FDisplayClusterPostBackbufferUpdated_RenderThread DisplayClusterPostBackbufferUpdatedEvent;
	FDisplayClusterPreProcessIcvfx_RenderThread      DisplayClusterPreProcessIcvfxEvent;
};
