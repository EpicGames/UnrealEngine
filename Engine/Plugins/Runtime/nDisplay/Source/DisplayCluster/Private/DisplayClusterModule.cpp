// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterModule.h"

#include "Cluster/DisplayClusterClusterManager.h"
#include "Config/DisplayClusterConfigManager.h"
#include "Game/DisplayClusterGameManager.h"
#include "Render/DisplayClusterRenderManager.h"

#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterLightCardActor.h"
#include "DisplayClusterRootActor.h"

#if WITH_EDITOR
#include "Filters/CustomClassFilterData.h"
#include "LevelEditor.h"
#endif

#include "Misc/DisplayClusterGlobals.h"
#include "Misc/DisplayClusterLog.h"
#include "Misc/DisplayClusterTypesConverter.h"


FDisplayClusterModule::FDisplayClusterModule()
{
	GDisplayCluster = this;

	UE_LOGF(LogDisplayClusterModule, Log, "Instantiating subsystem managers...");

	// Initialize internals (the order is important)
	Managers.Add(MgrConfig  = new FDisplayClusterConfigManager);
	Managers.Add(MgrCluster = new FDisplayClusterClusterManager);
	Managers.Add(MgrGame    = new FDisplayClusterGameManager);
	Managers.Add(MgrRender  = new FDisplayClusterRenderManager);
}

FDisplayClusterModule::~FDisplayClusterModule()
{
	GDisplayCluster = nullptr;
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IModuleInterface
//////////////////////////////////////////////////////////////////////////////////////////////
void FDisplayClusterModule::StartupModule()
{
	UE_LOGF(LogDisplayClusterModule, Log, "DisplayCluster module has been started");

#if WITH_EDITOR
	RegisterOutlinerFilters();
#endif
}

void FDisplayClusterModule::ShutdownModule()
{
	// Clean everything before .dtor call
	Release();
}

#if WITH_EDITOR
void FDisplayClusterModule::RegisterOutlinerFilters()
{
	if (FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>(TEXT("LevelEditor")))
	{
		if (const TSharedPtr<FFilterCategory> FilterCategory = LevelEditorModule->GetOutlinerFilterCategory(FLevelEditorOutlinerBuiltInCategories::VirtualProduction()))
		{
			const TSharedRef<FCustomClassFilterData> LightCardActorClassData =
				MakeShared<FCustomClassFilterData>(ADisplayClusterLightCardActor::StaticClass(), FilterCategory, FLinearColor::White);
			LevelEditorModule->AddCustomClassFilterToOutliner(LightCardActorClassData);

			const TSharedRef<FCustomClassFilterData> RootActorClassData =
				MakeShared<FCustomClassFilterData>(ADisplayClusterRootActor::StaticClass(), FilterCategory, FLinearColor::White);
			LevelEditorModule->AddCustomClassFilterToOutliner(RootActorClassData);
		}
	}
}
#endif

//////////////////////////////////////////////////////////////////////////////////////////////
// IPDisplayCluster
//////////////////////////////////////////////////////////////////////////////////////////////


//////////////////////////////////////////////////////////////////////////////////////////////
// IPDisplayClusterManager
//////////////////////////////////////////////////////////////////////////////////////////////
bool FDisplayClusterModule::Init(EDisplayClusterOperationMode OperationMode)
{
	CurrentOperationMode = OperationMode;

	UE_LOGF(LogDisplayClusterModule, Log, "Initializing subsystems to %ls operation mode", *DisplayClusterTypesConverter::template ToString(CurrentOperationMode));

	bool bResult = true;
	auto it = Managers.CreateIterator();
	while (bResult && it)
	{
		bResult = bResult && (*it)->Init(CurrentOperationMode);
		++it;
	}

	if (!bResult)
	{
		UE_LOGF(LogDisplayClusterModule, Error, "An error occurred during internal initialization");
	}

	return bResult;
}

void FDisplayClusterModule::Release()
{
	UE_LOGF(LogDisplayClusterModule, Log, "Cleaning up internals...");

	for (IPDisplayClusterManager* Manager : Managers)
	{
		Manager->Release();
		delete Manager;
	}

	MgrCluster = nullptr;
	MgrRender  = nullptr;
	MgrConfig  = nullptr;
	MgrGame    = nullptr;

	Managers.Empty();
}

bool FDisplayClusterModule::StartSession(UDisplayClusterConfigurationData* InConfigData, const FString& NodeId)
{
	UE_LOGF(LogDisplayClusterModule, Log, "StartSession with node ID '%ls'", *NodeId);

	bool bResult = true;
	auto it = Managers.CreateIterator();
	while (bResult && it)
	{
		bResult = bResult && (*it)->StartSession(InConfigData, NodeId);
		++it;
	}

	GetCallbacks().OnDisplayClusterStartSession().Broadcast();

	if (!bResult)
	{
		UE_LOGF(LogDisplayClusterModule, Error, "An error occurred during session start");
	}

	return bResult;
}

void FDisplayClusterModule::EndSession()
{
	UE_LOGF(LogDisplayClusterModule, Log, "Stopping DisplayCluster session...");

	GetCallbacks().OnDisplayClusterEndSession().Broadcast();

	for (IPDisplayClusterManager* const Manager : Managers)
	{
		Manager->EndSession();
	}
}

bool FDisplayClusterModule::StartScene(UWorld* InWorld)
{
	UE_LOGF(LogDisplayClusterModule, Log, "Starting game...");

	checkSlow(InWorld);

	bool bResult = true;
	auto it = Managers.CreateIterator();
	while (bResult && it)
	{
		bResult = bResult && (*it)->StartScene(InWorld);
		++it;
	}

	if (!bResult)
	{
		UE_LOGF(LogDisplayClusterModule, Error, "An error occurred during game (level) start");
	}

	GetCallbacks().OnDisplayClusterStartScene().Broadcast();

	return bResult;
}

void FDisplayClusterModule::EndScene()
{
	UE_LOGF(LogDisplayClusterModule, Log, "Stopping game...");

	GetCallbacks().OnDisplayClusterEndScene().Broadcast();

	for (IPDisplayClusterManager* const Manager : Managers)
	{
		Manager->EndScene();
	}
}

void FDisplayClusterModule::StartFrame(uint64 FrameNum)
{
	UE_LOGF(LogDisplayClusterModule, Verbose, "StartFrame: frame num - %llu", FrameNum);

	for (IPDisplayClusterManager* const Manager : Managers)
	{
		Manager->StartFrame(FrameNum);
	}

	GetCallbacks().OnDisplayClusterStartFrame().Broadcast(FrameNum);
}

void FDisplayClusterModule::EndFrame(uint64 FrameNum)
{
	UE_LOGF(LogDisplayClusterModule, Verbose, "EndFrame: frame num - %llu", FrameNum);

	for (IPDisplayClusterManager* const Manager : Managers)
	{
		Manager->EndFrame(FrameNum);
	}

	GetCallbacks().OnDisplayClusterEndFrame().Broadcast(FrameNum);
}

void FDisplayClusterModule::PreTick(float DeltaSeconds)
{
	UE_LOGF(LogDisplayClusterModule, Verbose, "PreTick: delta time - %f", DeltaSeconds);

	for (IPDisplayClusterManager* const Manager : Managers)
	{
		Manager->PreTick(DeltaSeconds);
	}

	GetCallbacks().OnDisplayClusterPreTick().Broadcast();
}

void FDisplayClusterModule::Tick(float DeltaSeconds)
{
	UE_LOGF(LogDisplayClusterModule, Verbose, "Tick: delta time - %f", DeltaSeconds);

	for (IPDisplayClusterManager* const Manager : Managers)
	{
		Manager->Tick(DeltaSeconds);
	}

	GetCallbacks().OnDisplayClusterTick().Broadcast();
}

void FDisplayClusterModule::PostTick(float DeltaSeconds)
{
	UE_LOGF(LogDisplayClusterModule, Verbose, "PostTick: delta time - %f", DeltaSeconds);

	for (IPDisplayClusterManager* const Manager : Managers)
	{
		Manager->PostTick(DeltaSeconds);
	}

	GetCallbacks().OnDisplayClusterPostTick().Broadcast();
}

IMPLEMENT_MODULE(FDisplayClusterModule, DisplayCluster)
