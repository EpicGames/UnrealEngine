// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPDisplayCluster.h"
#include "DisplayClusterCallbacks.h"

#include "Cluster/IPDisplayClusterClusterManager.h"
#include "Config/IPDisplayClusterConfigManager.h"
#include "Game/IPDisplayClusterGameManager.h"
#include "Render/IPDisplayClusterRenderManager.h"

class UDisplayClusterConfigurationData;


/**
 * Display Cluster module implementation
 */
class FDisplayClusterModule :
	public  IPDisplayCluster
{
public:
	FDisplayClusterModule();
	virtual ~FDisplayClusterModule();

public:
	//////////////////////////////////////////////////////////////////////////////////////////////
	// IDisplayCluster
	//////////////////////////////////////////////////////////////////////////////////////////////
	virtual EDisplayClusterOperationMode GetOperationMode() const override
	{
		return CurrentOperationMode;
	}

	virtual IDisplayClusterRenderManager*    GetRenderMgr()    const override { return MgrRender; }
	virtual IDisplayClusterClusterManager*   GetClusterMgr()   const override { return MgrCluster; }
	virtual IDisplayClusterConfigManager*    GetConfigMgr()    const override { return MgrConfig; }
	virtual IDisplayClusterGameManager*      GetGameMgr()      const override { return MgrGame; }

	virtual IDisplayClusterCallbacks& GetCallbacks() override
	{
		return Callbacks;
	}

public:
	//////////////////////////////////////////////////////////////////////////////////////////////
	// IPDisplayCluster
	//////////////////////////////////////////////////////////////////////////////////////////////
	virtual IPDisplayClusterRenderManager*    GetPrivateRenderMgr()    const override { return MgrRender; }
	virtual IPDisplayClusterClusterManager*   GetPrivateClusterMgr()   const override { return MgrCluster; }
	virtual IPDisplayClusterConfigManager*    GetPrivateConfigMgr()    const override { return MgrConfig; }
	virtual IPDisplayClusterGameManager*      GetPrivateGameMgr()      const override { return MgrGame; }

public:
	//////////////////////////////////////////////////////////////////////////////////////////////
	// IPDisplayClusterManager
	//////////////////////////////////////////////////////////////////////////////////////////////
	virtual bool Init(EDisplayClusterOperationMode OperationMode) override;
	virtual void Release() override;
	virtual bool StartSession(UDisplayClusterConfigurationData* InConfigData, const FString& NodeId) override;
	virtual void EndSession() override;
	virtual bool StartScene(UWorld* InWorld) override;
	virtual void EndScene() override;
	virtual void StartFrame(uint64 FrameNum) override;
	virtual void PreTick(float DeltaSeconds) override;
	virtual void Tick(float DeltaSeconds) override;
	virtual void PostTick(float DeltaSeconds) override;
	virtual void EndFrame(uint64 FrameNum) override;

private:
	//////////////////////////////////////////////////////////////////////////////////////////////
	// IModuleInterface
	//////////////////////////////////////////////////////////////////////////////////////////////
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	
#if WITH_EDITOR
	void RegisterOutlinerFilters();
#endif

private:
	// DisplayCluster subsystems
	IPDisplayClusterClusterManager*   MgrCluster   = nullptr;
	IPDisplayClusterRenderManager*    MgrRender    = nullptr;
	IPDisplayClusterConfigManager*    MgrConfig    = nullptr;
	IPDisplayClusterGameManager*      MgrGame      = nullptr;
	
	// Array of available managers
	TArray<IPDisplayClusterManager*> Managers;

	FDisplayClusterCallbacks Callbacks;

	// Runtime
	EDisplayClusterOperationMode CurrentOperationMode = EDisplayClusterOperationMode::Disabled;
};
