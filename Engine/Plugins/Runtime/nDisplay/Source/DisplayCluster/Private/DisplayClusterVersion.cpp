// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterVersion.h"
#include "Serialization/CustomVersion.h"


const FGuid FDisplayClusterCustomVersion::GUID(0x06C4F6DC, 0xF518B974, 0x87B0C096, 0xE3855B05);
FCustomVersionRegistration GRegisterDisplayClusterCustomVersion(FDisplayClusterCustomVersion::GUID, FDisplayClusterCustomVersion::LatestVersion, TEXT("nDisplayVer"));
