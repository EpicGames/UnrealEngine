// Copyright Epic Games, Inc. All Rights Reserved.

#include "Game/DisplayClusterGameManager.h"

#include "Engine/BlueprintGeneratedClass.h"
#include "Kismet/GameplayStatics.h"
#include "Misc/CommandLine.h"

#include "Config/IPDisplayClusterConfigManager.h"

#include "DisplayClusterConfigurationTypes.h"

#include "DisplayClusterRootActor.h"
#include "Camera/CameraComponent.h"

#include "Blueprints/DisplayClusterBlueprint.h"

#include "Components/SceneComponent.h"
#include "Components/DisplayClusterCameraComponent.h"
#include "Components/DisplayClusterScreenComponent.h"

#include "Misc/DisplayClusterGlobals.h"
#include "Misc/DisplayClusterHelpers.h"
#include "Misc/DisplayClusterLog.h"
#include "Misc/DisplayClusterStrings.h"

#include "GameFramework/Actor.h"
#include "GameFramework/PlayerStart.h"
#include "Engine/Level.h"
#include "Engine/LevelStreaming.h"
#include "EngineUtils.h"
#include "LevelUtils.h"


FDisplayClusterGameManager::FDisplayClusterGameManager()
	: ConfigData(nullptr)
	, CurrentOperationMode(EDisplayClusterOperationMode::Disabled)
	, CurrentWorld(nullptr)
{
}

FDisplayClusterGameManager::~FDisplayClusterGameManager()
{
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IPDisplayClusterManager
//////////////////////////////////////////////////////////////////////////////////////////////
bool FDisplayClusterGameManager::Init(EDisplayClusterOperationMode OperationMode)
{
	CurrentOperationMode = OperationMode;

	return true;
}

void FDisplayClusterGameManager::Release()
{
}

bool FDisplayClusterGameManager::StartSession(UDisplayClusterConfigurationData* InConfigData, const FString& InNodeId)
{
	ClusterNodeId = InNodeId;
	ConfigData = InConfigData;
	return true;
}

void FDisplayClusterGameManager::EndSession()
{
	ClusterNodeId.Reset();
}

bool FDisplayClusterGameManager::StartScene(UWorld* InWorld)
{
	FScopeLock Lock(&InternalsSyncScope);

	if(!InWorld || !ConfigData)
	{
		return false;
	}

	CurrentWorld = InWorld;

	// Find the first DCRA instance that matches to the specified configuration
	ADisplayClusterRootActor* RootActor = FindRootActor(InWorld, ConfigData);

	if (GDisplayCluster->GetOperationMode() == EDisplayClusterOperationMode::Cluster)
	{
		// If a corresponding DCRA instance was found, overwrite its settings
		if (RootActor)
		{
			UE_LOGF(LogDisplayClusterGame, Log, "Chose DCRA '%ls' corresponding to asset '%ls'", 
				*RootActor->GetName(),
				*ConfigData->Info.AssetPath
			);

			RootActor->OverrideFromConfig(ConfigData);
		}
		// If no proper DCRA found,
		// 1. Detect spawn location and rotation
		// 2. Try to spawn a blueprint from a corresponding asset
		// 3. If the asset not found, spawn an empty DCRA instance and initialize it with specified configuration data.
		else
		{
			// 1. Detect spawn location and rotation
			FVector  StartLocation = FVector::ZeroVector;
			FRotator StartRotation = FRotator::ZeroRotator;

			// Use PlayerStart transform if it exists in the world
			TActorIterator<APlayerStart> It(InWorld);
			if (It)
			{
				StartLocation = (*It)->GetActorLocation();
				StartRotation = (*It)->GetActorRotation();
			}

			// 2. Spawn the DCRA BP from a corresponding asset

			// We need to use generated class as it's the only one available in packaged buidls
			const FString AssetPath = (ConfigData->Info.AssetPath.EndsWith(TEXT("_C")) ?
				ConfigData->Info.AssetPath :
				ConfigData->Info.AssetPath + TEXT("_C"));

			if (UClass* ActorClass = Cast<UClass>(StaticLoadObject(UClass::StaticClass(), nullptr, *AssetPath)))
			{
				// Spawn the actor
				if (AActor* NewActor = CurrentWorld->SpawnActor<AActor>(ActorClass, StartLocation, StartRotation, FActorSpawnParameters()))
				{
					RootActor = Cast<ADisplayClusterRootActor>(NewActor);

					if (RootActor)
					{
						UE_LOGF(LogDisplayClusterGame, Log, "Instantiated DCRA from asset '%ls'", *ConfigData->Info.AssetPath);

						// Override actor settings in case the config file contains some updates
						RootActor->OverrideFromConfig(ConfigData);
					}
				}
			}

			// 3. Still no root actor exists? Spawn the DCRA and initialize it with the config data
			if (!RootActor)
			{
				UE_LOGF(LogDisplayClusterGame, Warning, "Couldn't find a configuration asset '%ls'. Initializing default DCRA...", *ConfigData->Info.AssetPath);

				RootActor = CurrentWorld->SpawnActor<ADisplayClusterRootActor>(ADisplayClusterRootActor::StaticClass(), StartLocation, StartRotation, FActorSpawnParameters());
				if (RootActor)
				{
					RootActor->InitializeFromConfig(ConfigData);
				}
			}
		}
	}

	// Store the DCRA instance. It's now considered as an 'active' root actor, so any nDisplay subsystem
	// or some user game logic can refer it using game manager API
	DisplayClusterRootActorRef.SetSceneActor(RootActor);

	return true;
}

void FDisplayClusterGameManager::EndScene()
{
	FScopeLock Lock(&InternalsSyncScope);
	DisplayClusterRootActorRef.ResetSceneActor();
	CurrentWorld = nullptr;
}

void FDisplayClusterGameManager::PreTick(float DeltaSeconds)
{
	// Here we check if the active DCRA instance is in a sublevel, and whether
	// the sublevel visibility is off. If so, we forcibly make the sublevel visible.
	if (const AActor* const DCRA = DisplayClusterRootActorRef.GetOrFindSceneActor())
	{
		if (const ULevel* const Level = DCRA->GetLevel())
		{
			if (!Level->bIsVisible)
			{
				if (ULevelStreaming* const StreamingLevel = FLevelUtils::FindStreamingLevel(Level))
				{
					UE_LOGF(LogDisplayClusterGame, Warning, "A streaming level containing the active DCRA instance is currently invisible. Forcing level visibility on.");
					StreamingLevel->SetShouldBeVisible(true);
				}
			}
		}
	}
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterGameManager
//////////////////////////////////////////////////////////////////////////////////////////////
ADisplayClusterRootActor* FDisplayClusterGameManager::GetRootActor() const
{
	FScopeLock Lock(&InternalsSyncScope);
	return Cast<ADisplayClusterRootActor>(DisplayClusterRootActorRef.GetOrFindSceneActor());
}

//////////////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterGameManager
//////////////////////////////////////////////////////////////////////////////////////////////
TArray<ADisplayClusterRootActor*> FDisplayClusterGameManager::GetAllRootActorsFromWorld(UWorld* InWorld)
{
	TArray<ADisplayClusterRootActor*> FoundActors;
	FoundActors.Reserve(16);

	// Find all DCRA instances in the persistent level
	FindRootActorsInWorld(InWorld, FoundActors);

	// Also search inside streamed levels
	const TArray<ULevelStreaming*>& StreamingLevels = InWorld->GetStreamingLevels();
	for (const ULevelStreaming* const StreamingLevel : StreamingLevels)
	{
		if (StreamingLevel && StreamingLevel->GetLevelStreamingState() == ELevelStreamingState::LoadedVisible)
		{
			// Look for the actor in those sub-levels that have been loaded already
			const TSoftObjectPtr<UWorld>& SubWorldAsset = StreamingLevel->GetWorldAsset();
			FindRootActorsInWorld(SubWorldAsset.Get(), FoundActors);
		}
	}

	return FoundActors;
}

ADisplayClusterRootActor* FDisplayClusterGameManager::FindRootActor(UWorld* InWorld, UDisplayClusterConfigurationData* InConfigData)
{
	TArray<ADisplayClusterRootActor*> FoundActors = FDisplayClusterGameManager::GetAllRootActorsFromWorld(InWorld);
	
#if WITH_EDITOR
	if (InWorld->IsPlayInEditor())
	{
		/** We need to fix it in the following way :
		 * - When multiple DCRA are available in the scene, and only one has Preview Node selected, then we should see the preview
		 * - When multiple DCRA are available in the sceneand multiple Preview Node is selected, we should write a warning in the logand print the name of the selected DCRA for the preview.
		 */
		TArray<ADisplayClusterRootActor*> FoundActorsForPIE;
		for (ADisplayClusterRootActor* RootActorIt : FoundActors)
		{
			if (RootActorIt && !(RootActorIt->PreviewNodeId.IsEmpty() || RootActorIt->PreviewNodeId == DisplayClusterConfigurationStrings::gui::preview::PreviewNodeNone))
			{
				FoundActorsForPIE.Add(RootActorIt);
			}
		}

		if (FoundActorsForPIE.IsEmpty())
		{
			// No DCRA for PIE preview
			return nullptr;
		}

		ADisplayClusterRootActor* ActiveRootActor = FoundActorsForPIE[0];

		if (FoundActorsForPIE.Num() > 1)
		{
			FString PreviewActorsNames;
			for (ADisplayClusterRootActor* RootActorIt : FoundActorsForPIE)
			{
				if (PreviewActorsNames.IsEmpty())
				{
					PreviewActorsNames.Appendf(TEXT("%s"), *RootActorIt->GetName());
				}
				else
				{
					PreviewActorsNames.Appendf(TEXT(", %s"), *RootActorIt->GetName());

				}
			}

			UE_LOGF(LogDisplayClusterGame, Warning, "Multiple PIE Preview Nodes selected at DisplayClusterRootActors - %ls. Active : %ls", *PreviewActorsNames , *ActiveRootActor->GetName());
		}

		return ActiveRootActor;
	}
#endif

	// Now iterate over all DCRA instances we have found to pick the one that corresponds to the config data
	for (ADisplayClusterRootActor* Actor : FoundActors)
	{
		// Check if it matches the config data
		if (DoesRootActorMatchTheAsset(Actor, InConfigData->Info.AssetPath))
		{
			return Actor;
		}
	}

	return nullptr;
}

void FDisplayClusterGameManager::FindRootActorsInWorld(UWorld* InWorld, TArray<ADisplayClusterRootActor*>& OutActors)
{
	if (InWorld && InWorld->PersistentLevel)
	{
		UClass* DisplayClusterRootActorClass = StaticLoadClass(UObject::StaticClass(), nullptr, TEXT("/Script/DisplayCluster.DisplayClusterRootActor"), NULL, LOAD_None, NULL);
		if (DisplayClusterRootActorClass)
		{
			for (TActorIterator<AActor> It(InWorld, DisplayClusterRootActorClass, EActorIteratorFlags::SkipPendingKill); It; ++It)
			{
				ADisplayClusterRootActor* RootActor = Cast<ADisplayClusterRootActor>(*It);
				if (RootActor != nullptr && !RootActor->IsTemplate())
				{
					UE_LOGF(LogDisplayClusterGame, VeryVerbose, "Found root actor - %ls", *RootActor->GetName());
					OutActors.Add(RootActor);
				}
			}
		}
	}
}

bool FDisplayClusterGameManager::DoesRootActorMatchTheAsset(ADisplayClusterRootActor* RootActor, const FString& AssetReference)
{
	// We only interested in DCRA blueprints
	if (RootActor->IsBlueprint())
	{
		// Iterate over class hierarchy
		for (UClass* Class = RootActor->GetClass(); Class; Class = Class->GetSuperClass())
		{
			// Get BP interface
			if (UBlueprintGeneratedClass* BPClass = Cast<UBlueprintGeneratedClass>(Class))
			{
				// Get asset path
				FString AssetPath = BPClass->GetPathName();
				AssetPath.RemoveFromEnd(TEXT("_C"));

				// Check if this BP was made of a specific asset
				if (AssetPath.Equals(AssetReference, ESearchCase::IgnoreCase))
				{
					// This DCRA instance matches our search criteria
					return true;
				}
			}
		}
	}

	return false;
}
