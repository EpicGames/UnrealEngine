// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/DisplayClusterObjectRef.h"
#include "Game/IPDisplayClusterGameManager.h"

class AActor;
class ADisplayClusterRootActor;
class USceneComponent;
class UDisplayClusterCameraComponent;
class UDisplayClusterSceneComponent;
class UDisplayClusterScreenComponent;
class UDisplayClusterConfigurationData;


/**
 * Game manager. Responsible for building VR object hierarchy from a config file. Implements some in-game logic.
 */
class FDisplayClusterGameManager
	: public IPDisplayClusterGameManager
{
public:
	FDisplayClusterGameManager();
	virtual ~FDisplayClusterGameManager();

public:
	//////////////////////////////////////////////////////////////////////////////////////////////
	// IPDisplayClusterManager
	//////////////////////////////////////////////////////////////////////////////////////////////
	virtual bool Init(EDisplayClusterOperationMode OperationMode) override;
	virtual void Release() override;
	virtual bool StartSession(UDisplayClusterConfigurationData* InConfigData, const FString& InNodeId) override;
	virtual void EndSession() override;
	virtual bool StartScene(UWorld* World) override;
	virtual void EndScene() override;
	virtual void PreTick(float DeltaSeconds) override;

public:
	//////////////////////////////////////////////////////////////////////////////////////////////
	// IDisplayClusterGameManager
	//////////////////////////////////////////////////////////////////////////////////////////////
	virtual ADisplayClusterRootActor*     GetRootActor() const override;
	virtual TArray<ADisplayClusterRootActor*> GetAllRootActorsFromWorld(UWorld* InWorld) override;
	virtual UWorld* GetWorld() const override
	{ return CurrentWorld; }

private:
	// The main function to look for a proper DCRA in a level and its sublevels
	ADisplayClusterRootActor* FindRootActor(UWorld* World, UDisplayClusterConfigurationData* ConfigData);

	// Auxiliary function that looks up for all DCRA instances in a level
	void FindRootActorsInWorld(UWorld* World, TArray<ADisplayClusterRootActor*>& OutActors);

	// Checks if a DCRA instance corresponds to a specified asset
	bool DoesRootActorMatchTheAsset(ADisplayClusterRootActor* RootActorconst, const FString& AssetReference);

private:
	// Active DisplayCluster root
	FDisplayClusterActorRef DisplayClusterRootActorRef;

	// Actual config data
	UDisplayClusterConfigurationData* ConfigData;

	EDisplayClusterOperationMode CurrentOperationMode;
	FString ClusterNodeId;
	UWorld* CurrentWorld;

	mutable FCriticalSection InternalsSyncScope;
};
