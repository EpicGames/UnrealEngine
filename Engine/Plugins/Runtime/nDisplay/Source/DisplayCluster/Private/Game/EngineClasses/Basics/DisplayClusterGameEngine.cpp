// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterGameEngine.h"

#include "Algo/Accumulate.h"
#include "Cluster/IPDisplayClusterClusterManager.h"
#include "Cluster/Controller/IDisplayClusterClusterNodeController.h"
#include "Cluster/NetAPI/DisplayClusterNetApiFacade.h"
#include "Config/IPDisplayClusterConfigManager.h"
#include "Engine/DynamicBlueprintBinding.h"
#include "Engine/GameInstance.h"
#include "Engine/NetDriver.h"

#include "GameFramework/GameModeBase.h"
#include "HAL/IConsoleManager.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "Kismet/GameplayStatics.h"

#include "Misc/App.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "Misc/DisplayClusterAppExit.h"
#include "Misc/DisplayClusterGlobals.h"
#include "Misc/DisplayClusterHelpers.h"
#include "Misc/DisplayClusterLog.h"
#include "Misc/DisplayClusterStrings.h"
#include "Misc/Parse.h"

#include "Network/Barrier/IDisplayClusterBarrier.h"
#include "Network/Configuration/DisplayClusterNetworkConfiguration.h"
#include "Network/Service/ClusterSync/DisplayClusterClusterSyncService.h"
#include "Network/Service/DisplayClusterService.h"

#include "Stats/Stats.h"
#include "UObject/Linker.h"

#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterConfigurationVersion.h"
#include "DisplayClusterEnums.h"
#include "EngineGlobals.h"
#include "GameDelegates.h"
#include "GameMapsSettings.h"
#include "IDisplayClusterCallbacks.h"
#include "IDisplayClusterConfiguration.h"
#include "SocketSubsystem.h"


namespace DisplayClusterGameEngineUtils
{
	static const FString WaitForGameCategory = DisplayClusterResetSyncType;
	static const FString WaitForGameName     = TEXT("WaitForGameStart");
}

// Advanced cluster synchronization during LoadMap
static TAutoConsoleVariable<int32> CVarGameStartBarrierAvoidance(
	TEXT("nDisplay.game.GameStartBarrierAvoidance"),
	1,
	TEXT("Avoid entering GameStartBarrier on loading level\n")
	TEXT("0 : disabled\n")
	TEXT("1 : enabled\n")
);

void UDisplayClusterGameEngine::Init(class IEngineLoop* InEngineLoop)
{
	UE_LOGF(LogDisplayClusterEngine, Log, "UDisplayClusterGameEngine::Init");

	// Detect requested operation mode
	OperationMode = DetectOperationMode();

	// Modify any required CVars before keep initialization
	ModifyCVars();

	// Initialize Display Cluster
	if (!GDisplayCluster->Init(OperationMode))
	{
		FDisplayClusterAppExit::ExitApplication(FString("Couldn't initialize DisplayCluster module"), FDisplayClusterAppExit::EExitType::KillImmediately);
	}

	// This is required to prevent GameThread dead-locking when Alt+F4 is used
	// to terminate p-node of a 2+ nodes cluster. This gets called before
	// virtual PreExit which allows to unlock GameThread in advance.
	FCoreDelegates::GetApplicationWillTerminateDelegate().AddLambda([this]()
	{
		PreExitImpl();
	});

	if (OperationMode == EDisplayClusterOperationMode::Cluster)
	{
		// Our parsing function for arguments like:
		// -ArgName1="ArgValue 1" -ArgName2=ArgValue2 ArgName3=ArgValue3
		//
		auto ParseCommandArg = [](const FString& CommandLine, const FString& ArgName, FString& OutArgVal)
		{
			const FString Tag = FString::Printf(TEXT("-%s="), *ArgName);
			const int32 TagPos = CommandLine.Find(Tag);

			if (TagPos == INDEX_NONE)
			{
				// Try old method, where the '-' prefix is missing and quoted values with spaces are not supported.
				return FParse::Value(*CommandLine, *ArgName, OutArgVal);
			}

			const TCHAR* TagValue = &CommandLine[TagPos + Tag.Len()];
			return FParse::Token(TagValue, OutArgVal, false);
		};

		// Extract config path from command line
		FString ConfigPath;
		if (!ParseCommandArg(FCommandLine::Get(), DisplayClusterStrings::args::Config, ConfigPath))
		{
			FDisplayClusterAppExit::ExitApplication(FString("No config file specified. Cluster operation mode requires config file."), FDisplayClusterAppExit::EExitType::KillImmediately);
		}

		// Clean the file path before using it
		DisplayClusterHelpers::str::TrimStringValue(ConfigPath);

		// Validate the config file first. Since 4.27, we don't allow the old formats to be used
		const EDisplayClusterConfigurationVersion ConfigVersion = IDisplayClusterConfiguration::Get().GetConfigVersion(ConfigPath);
		if (!ValidateConfigFile(ConfigPath))
		{
			FDisplayClusterAppExit::ExitApplication(FString("An invalid or outdated configuration file was specified. Please consider using nDisplay configurator to update the config files."), FDisplayClusterAppExit::EExitType::KillImmediately);
		}

		// Load config data
		UDisplayClusterConfigurationData* ConfigData = IDisplayClusterConfiguration::Get().LoadConfig(ConfigPath);
		if (!ConfigData)
		{
			FDisplayClusterAppExit::ExitApplication(FString("An error occurred during loading the configuration file"), FDisplayClusterAppExit::EExitType::KillImmediately);
		}

		// Extract node ID from command line
		FString NodeId;
		if (!ParseCommandArg(FCommandLine::Get(), DisplayClusterStrings::args::Node, NodeId))
		{
			UE_LOGF(LogDisplayClusterEngine, Log, "Node ID is not specified. Trying to resolve from host address...");

			// Find node ID based on the host address
			if (!GetResolvedNodeId(ConfigData, NodeId))
			{
				FDisplayClusterAppExit::ExitApplication(FString("Couldn't resolve node ID. Try to specify host addresses explicitly."), FDisplayClusterAppExit::EExitType::KillImmediately);
			}

			UE_LOGF(LogDisplayClusterEngine, Log, "Node ID has been successfully resolved: %ls", *NodeId);
		}

		// Clean node ID string
		DisplayClusterHelpers::str::TrimStringValue(NodeId);

		// Start game session
		if (!GDisplayCluster->StartSession(ConfigData, NodeId))
		{
			FDisplayClusterAppExit::ExitApplication(FString("Couldn't start DisplayCluster session"), FDisplayClusterAppExit::EExitType::KillImmediately);
		}

		// Initialize internals
		InitializeInternals();
	}

	// Initialize base stuff.
	UGameEngine::Init(InEngineLoop);

	OnOverrideBrowseURL.BindUObject(this, &UDisplayClusterGameEngine::BrowseLoadMap);
	OnOverridePendingNetGameUpdate.BindUObject(this, &UDisplayClusterGameEngine::PendingLevelUpdate);
}

EDisplayClusterOperationMode UDisplayClusterGameEngine::DetectOperationMode() const
{
	EDisplayClusterOperationMode OpMode = EDisplayClusterOperationMode::Disabled;
	if (FParse::Param(FCommandLine::Get(), DisplayClusterStrings::args::Cluster))
	{
		OpMode = EDisplayClusterOperationMode::Cluster;
	}

	UE_LOGF(LogDisplayClusterEngine, Log, "Detected operation mode: %ls", *DisplayClusterTypesConverter::template ToString(OpMode));

	return OpMode;
}

bool UDisplayClusterGameEngine::InitializeInternals()
{
	// To prevent static analyzer of reporting unchecked pointers usage
	const IPDisplayCluster* const DCAPI = GDisplayCluster;
	if (!DCAPI)
	{
		UE_LOGF(LogDisplayClusterEngine, Warning, "Could not initialize DC game engine, invalid GDisplayCluster");
		return false;
	}

	// This function is called after a session had been started so it's safe to get config data from the config manager
	const UDisplayClusterConfigurationData* Config = DCAPI->GetPrivateConfigMgr()->GetConfig();
	check(Config);

	// Store diagnostics settings locally
	Diagnostics = Config->Diagnostics;

	// Listen for JSON sync events
	FOnClusterEventJsonListener GameSyncTransition = FOnClusterEventJsonListener::CreateUObject(this, &UDisplayClusterGameEngine::GameSyncChange);
	DCAPI->GetPrivateClusterMgr()->AddClusterEventJsonListener(GameSyncTransition);

	// Enable/disable audio on this node
	const UDisplayClusterConfigurationClusterNode* CfgLocalNode = DCAPI->GetPrivateConfigMgr()->GetLocalNode();
	const bool bSoundEnabled = (CfgLocalNode ? CfgLocalNode->bIsSoundEnabled : false);
	UE_LOGF(LogDisplayClusterEngine, Log, "Configuring sound enabled: %ls", *DisplayClusterTypesConverter::template ToString(bSoundEnabled));
	if (!bSoundEnabled)
	{
		AudioDeviceManager = nullptr;
	}

	return true;
}

// This function works if you have 1 cluster node per PC. In case of multiple nodes, all of them will have the same node ID.
bool UDisplayClusterGameEngine::GetResolvedNodeId(const UDisplayClusterConfigurationData* ConfigData, FString& NodeId) const
{
	TArray<TSharedPtr<FInternetAddr>> LocalAddresses;
	if (!ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->GetLocalAdapterAddresses(LocalAddresses))
	{
		UE_LOGF(LogDisplayClusterCluster, Error, "Couldn't get local addresses list. Cannot find node ID by its address.");
		return false;
	}

	if (LocalAddresses.Num() < 1)
	{
		UE_LOGF(LogDisplayClusterCluster, Error, "No local addresses found");
		return false;
	}

	for (const auto& it : ConfigData->Cluster->Nodes)
	{
		for (const auto& LocalAddress : LocalAddresses)
		{
			const FIPv4Endpoint ep(LocalAddress);
			const FString epaddr = ep.Address.ToString();

			UE_LOGF(LogDisplayClusterCluster, Log, "Comparing addresses: %ls - %ls", *epaddr, *it.Value->Host);

			//@note: don't add "127.0.0.1" or "localhost" here. There will be a bug. It has been proved already.
			if (epaddr.Equals(it.Value->Host, ESearchCase::IgnoreCase))
			{
				// Found!
				NodeId = it.Key;
				return true;
			}
		}
	}

	// We haven't found anything
	return false;
}

bool UDisplayClusterGameEngine::ValidateConfigFile(const FString& FilePath)
{
	const EDisplayClusterConfigurationVersion ConfigVersion = IDisplayClusterConfiguration::Get().GetConfigVersion(FilePath);
	switch (ConfigVersion)
	{
		case EDisplayClusterConfigurationVersion::Version_426:
			// Old 4.26 and 4.27p1 formats are not allowed as well
			UE_LOGF(LogDisplayClusterEngine, Error, "Detected old (.ndisplay 4.26 or .ndisplay 4.27p1) config format. Please upgrade to the actual version.");
			return true;

		case EDisplayClusterConfigurationVersion::Version_427:
			// Ok, it's one of the actual config formats
			UE_LOGF(LogDisplayClusterEngine, Log, "Detected (.ndisplay 4.27) config format");
			return true;

		case EDisplayClusterConfigurationVersion::Version_500:
			// Ok, it's one of the actual config formats
			UE_LOGF(LogDisplayClusterEngine, Log, "Detected (.ndisplay 5.00) config format");
			return true;

		case EDisplayClusterConfigurationVersion::Unknown:
		default:
			// Something unexpected came here
			UE_LOGF(LogDisplayClusterEngine, Error, "Unknown or unsupported config format");
			break;
	}

	return false;
}

void UDisplayClusterGameEngine::PreExit()
{
	PreExitImpl();

	// Release the engine
	UGameEngine::PreExit();
}

void UDisplayClusterGameEngine::PreExitImpl()
{
	UE_LOGF(LogDisplayClusterEngine, Log, "UDisplayClusterGameEngine::PreExitImpl");

	if (OperationMode == EDisplayClusterOperationMode::Cluster)
	{
		// Finalize current world
		GDisplayCluster->EndScene();
		// Close current DisplayCluster session
		GDisplayCluster->EndSession();
	}
}

bool UDisplayClusterGameEngine::LoadMap(FWorldContext& WorldContext, FURL URL, class UPendingNetGame* Pending, FString& Error)
{
	UE_LOGF(LogDisplayClusterEngine, Log, "UDisplayClusterGameEngine::LoadMap, URL=%ls", *URL.ToString());

	if (OperationMode == EDisplayClusterOperationMode::Cluster)
	{
		// Finish previous scene
		GDisplayCluster->EndScene();

		// Perform map loading
		if (!Super::LoadMap(WorldContext, URL, Pending, Error))
		{
			return false;
		}

		// Start new scene
		GDisplayCluster->StartScene(WorldContext.World());
		WorldContextObject = WorldContext.World();

		UGameplayStatics::SetGamePaused(WorldContextObject, BarrierAvoidanceOn());

		if(BarrierAvoidanceOn() && RunningMode != EDisplayClusterRunningMode::Startup)
		{
			IPDisplayClusterClusterManager& ClusterMgr = *GDisplayCluster->GetPrivateClusterMgr();

			FDisplayClusterClusterEventJson WaitForGameEvent;
			WaitForGameEvent.Category = DisplayClusterGameEngineUtils::WaitForGameCategory;
			WaitForGameEvent.Type = URL.ToString();
			WaitForGameEvent.Name = ClusterMgr.GetNodeId();
			WaitForGameEvent.bIsSystemEvent = true;
			WaitForGameEvent.bShouldDiscardOnRepeat = false;

			ClusterMgr.EmitClusterEventJson(WaitForGameEvent, false);

			RunningMode = EDisplayClusterRunningMode::WaitingForSync;
			// Assume that all nodes are now out of sync.
			UE_LOGF(LogDisplayClusterEngine, Display, "LoadMap occurred after startup for Level %ls", *WaitForGameEvent.Type);
		}
		else
		{
			CheckGameStartBarrier();
		}
	}
	else
	{
		return Super::LoadMap(WorldContext, URL, Pending, Error);
	}

	return true;
}

void UDisplayClusterGameEngine::Tick(float DeltaSeconds, bool bIdleMode)
{
	UE_LOGF(LogDisplayClusterEngine, Verbose, "UDisplayClusterGameEngine::Tick, Delta=%f, Idle=%d", DeltaSeconds, bIdleMode ? 1 : 0);

	if (CanTick())
	{
		IPDisplayClusterClusterManager& ClusterMgr = *GDisplayCluster->GetPrivateClusterMgr();

		//////////////////////////////////////////////////////////////////////////////////////////////
		// Frame start barrier
		UE_LOGF(LogDisplayClusterEngine, Verbose, "Sync frame start");
		ClusterMgr.GetNetApi().GetClusterSyncAPI()->WaitForFrameStart();

		// Perform StartFrame notification
		GDisplayCluster->StartFrame(GFrameCounter);

		UE_LOGF(LogDisplayClusterEngine, Verbose, "DisplayCluster delta seconds: %f", DeltaSeconds);

		// Perform PreTick for DisplayCluster module
		UE_LOGF(LogDisplayClusterEngine, Verbose, "Perform PreTick()");
		GDisplayCluster->PreTick(DeltaSeconds);

		// Perform UGameEngine::Tick() calls for scene actors
		UE_LOGF(LogDisplayClusterEngine, Verbose, "Perform UGameEngine::Tick()");
		Super::Tick(DeltaSeconds, bIdleMode || bForcedTickIdleMode);

		// Perform PostTick for DisplayCluster module
		UE_LOGF(LogDisplayClusterEngine, Verbose, "Perform PostTick()");
		GDisplayCluster->PostTick(DeltaSeconds);

		if (Diagnostics.bSimulateLag)
		{
			const float LagTime = FMath::RandRange(Diagnostics.MinLagTime, Diagnostics.MaxLagTime);
			UE_LOGF(LogDisplayClusterEngine, Log, "Simulating lag: %f seconds", LagTime);
			FPlatformProcess::Sleep(LagTime);
		}

		//////////////////////////////////////////////////////////////////////////////////////////////
		// Frame end barrier
		ClusterMgr.GetNetApi().GetClusterSyncAPI()->WaitForFrameEnd();

		// Perform EndFrame notification
		GDisplayCluster->EndFrame(GFrameCounter);

		if (bIsRenderingSuspended)
		{
			bIsRenderingSuspended = false;
		}

		UE_LOGF(LogDisplayClusterEngine, Verbose, "Sync frame end");
	}
	else
	{
		UE_LOGF(LogDisplayClusterEngine, Verbose, "UDisplayClusterGameEngine::Tick, Tick() is not allowed");
		Super::Tick(DeltaSeconds, bIdleMode);
	}
}


void UDisplayClusterGameEngine::UpdateTimeAndHandleMaxTickRate()
{
	UE_LOGF(LogDisplayClusterEngine, Verbose, "UDisplayClusterGameEngine::UpdateTimeAndHandleMaxTickRate");

	UEngine::UpdateTimeAndHandleMaxTickRate();

	if (CanTick())
	{
		// Synchronize time data
		GDisplayCluster->GetPrivateClusterMgr()->SyncTimeData();
	}
}

bool UDisplayClusterGameEngine::CanTick() const
{
	return (RunningMode == EDisplayClusterRunningMode::Synced
			|| RunningMode == EDisplayClusterRunningMode::WaitingForSync)
		&& OperationMode == EDisplayClusterOperationMode::Cluster;
}

bool UDisplayClusterGameEngine::BarrierAvoidanceOn() const
{
	return CVarGameStartBarrierAvoidance.GetValueOnGameThread() != 0;
}

bool UDisplayClusterGameEngine::OutOfSync() const
{
	return SyncMap.Num() != 0;
}

void UDisplayClusterGameEngine::ReceivedSync(const FString &Level, const FString &NodeId)
{
	UE_LOGF(LogDisplayClusterEngine,Display, "GameSyncChange event received.");
	TSet<FString> &SyncItem = SyncMap.FindOrAdd(Level);
	SyncItem.Add(NodeId);
	if (SyncItem.Num() == GDisplayCluster->GetPrivateClusterMgr()->GetNodesAmount())
	{
		SyncMap.Remove(Level);
	}
	for (const TTuple<FString, TSet<FString>>& SyncObj : SyncMap)
	{
		FString Join = Algo::Accumulate(SyncObj.Value, FString(), [](FString Result, const FString& Value)
		{
			Result = Result + ", " + Value;
			return MoveTemp(Result);
		});
		UE_LOGF(LogDisplayClusterEngine,Display, "    %ls -> %ls", *SyncObj.Key, *Join);
	}
}

void UDisplayClusterGameEngine::CheckGameStartBarrier()
{
	if (!BarrierAvoidanceOn())
	{
		GDisplayCluster->GetPrivateClusterMgr()->GetNetApi().GetClusterSyncAPI()->WaitForGameStart();
	}
	else
	{
		if (!OutOfSync())
		{
			UE_LOGF(LogDisplayClusterEngine, Display, "CheckGameStartBarrier - we are no longer out of sync. Restoring Play.");
			if (RunningMode == EDisplayClusterRunningMode::Startup)
			{
				GDisplayCluster->GetPrivateClusterMgr()->GetNetApi().GetClusterSyncAPI()->WaitForGameStart();
			}
			UGameplayStatics::SetGamePaused(WorldContextObject,false);
			RunningMode = EDisplayClusterRunningMode::Synced;
		}
		else if (!UGameplayStatics::IsGamePaused(WorldContextObject))
		{
			UE_LOGF(LogDisplayClusterEngine, Display, "CheckGameStartBarrier - we are out of sync. Pausing Play.");
			// A 1 or more nodes is out of sync. Do not advance game until everyone is back in sync.
			//
			UGameplayStatics::SetGamePaused(WorldContextObject,true);
		}
	}
}

void UDisplayClusterGameEngine::GameSyncChange(const FDisplayClusterClusterEventJson& InEvent)
{
	if (BarrierAvoidanceOn())
	{
		if(InEvent.Category == DisplayClusterGameEngineUtils::WaitForGameCategory)
		{
			ReceivedSync(InEvent.Type,InEvent.Name);
			CheckGameStartBarrier();
		}
	}
}

EBrowseReturnVal::Type UDisplayClusterGameEngine::BrowseLoadMap(FWorldContext& WorldContext, FURL URL, FString& Error)
{
	FString DisplayClusterServerType;
	FParse::Value(FCommandLine::Get(), TEXT("dc_replicationserver_type"), DisplayClusterServerType);

	const bool IsDisplayCluster = (IDisplayCluster::Get().GetOperationMode() == EDisplayClusterOperationMode::Cluster);
	const bool IsDisplayClusterListenServer = IsDisplayCluster && DisplayClusterServerType.Equals(TEXT("listen"));

	if (URL.IsLocalInternal() && !IsDisplayClusterListenServer)
	{
		// Local map file.
		return LoadMap(WorldContext, URL, NULL, Error) ? EBrowseReturnVal::Success : EBrowseReturnVal::Failure;
	}
	else if ((URL.IsInternal() && GIsClient) || (URL.IsLocalInternal() && IsDisplayClusterListenServer))
	{
		EBrowseReturnVal::Type BrowseResult = EBrowseReturnVal::Failure;

		if (IsDisplayClusterListenServer)
		{
			BrowseResult = LoadMap(WorldContext, URL, NULL, Error) ? EBrowseReturnVal::Success : EBrowseReturnVal::Failure;
		}

		if ((URL.IsInternal() && GIsClient) || (BrowseResult == EBrowseReturnVal::Success))
		{
			// Network URL.
			if (WorldContext.PendingNetGame)
			{
				CancelPending(WorldContext);
			}

			// Clean up the netdriver/socket so that the pending level succeeds
			if (WorldContext.World() && ShouldShutdownWorldNetDriver())
			{
				ShutdownWorldNetDriver(WorldContext.World());
			}

			WorldContext.PendingNetGame = NewObject<UPendingNetGame>();
			WorldContext.PendingNetGame->Initialize(URL);
			WorldContext.PendingNetGame->InitNetDriver();

			UNetDriver* PendingNetDriver = WorldContext.PendingNetGame->GetNetDriver();

			if (IsDisplayCluster && PendingNetDriver)
			{
				const bool bIsDisplayClusterNetDriver = PendingNetDriver->GetClass()->GetName().Equals(TEXT("DisplayClusterNetDriver"));
				
				if (bIsDisplayClusterNetDriver)
				{
					// multiplayer packes, including session handshake are processed on ticks thus we need to enforce engine to tick but preven from rendering until cluster is ready
					// Force tick idle mode for multiplayer connections
					bForcedTickIdleMode = true;

					// Suspend rendering until the cluster is ready
					// By default, engine allowed to render while session being established
					// to prevent rendering from being invoked we overrided function IsRenderingSuspended() and only enabling this flag here in BrowseLoadMap which is called on in multiplayer
					// once loading is finished and game tick called in the flag will be reset to false
					bIsRenderingSuspended = true;
				}
			}

			if (!WorldContext.PendingNetGame)
			{
				// If the inital packet sent in InitNetDriver results in a socket error, HandleDisconnect() and CancelPending() may be called, which will null the PendingNetGame.
				Error = NSLOCTEXT("Engine", "PendingNetGameInitFailure", "Error initializing the network driver.").ToString();
				BroadcastTravelFailure(WorldContext.World(), ETravelFailure::PendingNetGameCreateFailure, Error);
				return EBrowseReturnVal::Failure;
			}

			if (!WorldContext.PendingNetGame->NetDriver)
			{
				// UPendingNetGame will set the appropriate error code and connection lost type, so
				// we just have to propagate that message to the game.
				BroadcastTravelFailure(WorldContext.World(), ETravelFailure::PendingNetGameCreateFailure, WorldContext.PendingNetGame->ConnectionError);
				WorldContext.PendingNetGame = NULL;
				return EBrowseReturnVal::Failure;
			}
			return EBrowseReturnVal::Pending;
		}
	}
	else if (URL.IsInternal())
	{
		// Invalid.
		Error = NSLOCTEXT("Engine", "ServerOpen", "Servers can't open network URLs").ToString();
		return EBrowseReturnVal::Failure;
	}

	return EBrowseReturnVal::Failure;
}

void UDisplayClusterGameEngine::PendingLevelUpdate(FWorldContext& Context, float DeltaSeconds)
{
	// Update the pending level.
	if (Context.PendingNetGame)
	{
		Context.PendingNetGame->Tick(DeltaSeconds);
		if (Context.PendingNetGame && Context.PendingNetGame->ConnectionError.Len() > 0)
		{
			BroadcastNetworkFailure(NULL, Context.PendingNetGame->NetDriver, ENetworkFailure::PendingConnectionFailure, Context.PendingNetGame->ConnectionError);
			CancelPending(Context);
		}
		else if (Context.PendingNetGame && Context.PendingNetGame->bSuccessfullyConnected && !Context.PendingNetGame->bSentJoinRequest && !Context.PendingNetGame->bLoadedMapSuccessfully && (Context.OwningGameInstance == NULL || !Context.OwningGameInstance->DelayPendingNetGameTravel()))
		{
			if (Context.PendingNetGame->HasFailedTravel())
			{
				BrowseToDefaultMap(Context);
				BroadcastTravelFailure(Context.World(), ETravelFailure::TravelFailure, TEXT("Travel failed for unknown reason"));
			}
			else if (!MakeSureMapNameIsValid(Context.PendingNetGame->URL.Map))
			{
				BrowseToDefaultMap(Context);
				BroadcastTravelFailure(Context.World(), ETravelFailure::PackageMissing, Context.PendingNetGame->URL.Map);
			}
			else if (!Context.PendingNetGame->bLoadedMapSuccessfully)
			{
				// Attempt to load the map.
				FString Error;
				FString DisplayClusterServerType;
				FParse::Value(FCommandLine::Get(), TEXT("dc_replicationserver_type"), DisplayClusterServerType);

				const bool IsDisplayCluster = FParse::Param(FCommandLine::Get(), TEXT("dc_cluster"));
				const bool IsDisplayClusterListenServer = IsDisplayCluster && DisplayClusterServerType.Equals(TEXT("listen"));

				bool bLoadedMapSuccessfully = true;

				if (IsDisplayClusterListenServer)
				{
					MovePendingLevel(Context);
				}
				else
				{
					bLoadedMapSuccessfully = LoadMap(Context, Context.PendingNetGame->URL, Context.PendingNetGame, Error);
				}

				if (Context.PendingNetGame != nullptr)
				{
					if (!Context.PendingNetGame->LoadMapCompleted(this, Context, bLoadedMapSuccessfully, Error))
					{
						BrowseToDefaultMap(Context);
						BroadcastTravelFailure(Context.World(), ETravelFailure::LoadMapFailure, Error);
					}
				}
				else
				{
					BrowseToDefaultMap(Context);
					BroadcastTravelFailure(Context.World(), ETravelFailure::TravelFailure, Error);
				}
			}
		}

		if (Context.PendingNetGame && Context.PendingNetGame->bLoadedMapSuccessfully && (Context.OwningGameInstance == NULL || !Context.OwningGameInstance->DelayCompletionOfPendingNetGameTravel()))
		{
			if (!Context.PendingNetGame->HasFailedTravel())
			{
				Context.PendingNetGame->TravelCompleted(this, Context);
				Context.PendingNetGame = nullptr;
			}
			else
			{
				CancelPending(Context);
				BrowseToDefaultMap(Context);
				BroadcastTravelFailure(Context.World(), ETravelFailure::LoadMapFailure, TEXT("Travel failed for unknown reason"));
			}
		}
	}
	else if (TransitionType == ETransitionType::WaitingToConnect)
	{
		TransitionType = ETransitionType::None;
	}
}

void UDisplayClusterGameEngine::ModifyCVars()
{
	// Always call Present() from the RHI thread to prevent the submission thread from deadlocking.
	// Some synchronization policies wait for frame completion. In other words, they use GPU fences
	// for synchronization during Present() call. This can't be done on the submission thread as
	// it would deadlock itself.
	if (IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.D3D12.PresentOnSubmissionThread")))
	{
		CVar->Set(false);
	}
}
