// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterPlayerInput.h"

#include "DisplayClusterConfigurationStrings.h"
#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterEnums.h"

#include "Cluster/IPDisplayClusterClusterManager.h"
#include "Cluster/NetAPI/DisplayClusterNetApiFacade.h"
#include "Components/InputComponent.h"
#include "Config/IPDisplayClusterConfigManager.h"

#include "Engine/World.h"

#include "Misc/DisplayClusterGlobals.h"
#include "Misc/DisplayClusterHelpers.h"
#include "Misc/DisplayClusterLog.h"
#include "Misc/DisplayClusterTypesConverter.h"

#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"


UDisplayClusterPlayerInput::UDisplayClusterPlayerInput()
	: Super()
	, bReplicatePrimary(false) // no replication by default
{
	// Replication makes sense in 'cluster' mode only
	if (GDisplayCluster && GDisplayCluster->GetOperationMode() == EDisplayClusterOperationMode::Cluster)
	{
		if (const IPDisplayClusterConfigManager* const ConfigMgr = GDisplayCluster->GetPrivateConfigMgr())
		{
			if (const UDisplayClusterConfigurationData* ConfigData = ConfigMgr->GetConfig())
			{
				const FString& SyncPolicyType = ConfigData->Cluster->Sync.InputSyncPolicy.Type;
				UE_LOGF(LogDisplayClusterGame, Log, "Native input sync policy: %ls", *SyncPolicyType);

				// Optionally activate native input synchronization
				if (SyncPolicyType.Equals(DisplayClusterConfigurationStrings::config::cluster::input_sync::InputSyncPolicyReplicatePrimary, ESearchCase::IgnoreCase))
				{
					// Ok, replication is needed
					bReplicatePrimary = true;
				}
			}
		}
	}
}

void UDisplayClusterPlayerInput::ProcessInputStack(const TArray<UInputComponent*>& InputComponentStack, const float DeltaTime, const bool bGamePaused)
{
	// Input replication
	if (bReplicatePrimary)
	{
		ProcessPolicy_ReplicatePrimary(InputComponentStack, DeltaTime, bGamePaused);
	}

	Super::ProcessInputStack(InputComponentStack, DeltaTime, bGamePaused);
}

bool UDisplayClusterPlayerInput::SerializeKeyStateMap(TMap<FString, TArray<uint8>>& OutKeyStateMap)
{
	// Serialize each key state individually
	TMap<FKey, FKeyState>& StateMap = GetKeyStateMap();
	for (TPair<FKey, FKeyState>& It : StateMap)
	{
		// Output buffer
		TArray<uint8> KeyStateBuffer;
		FMemoryWriter KeyStateWriter(KeyStateBuffer);

		// Serialized simple member variables
		KeyStateWriter << It.Value.RawValue;
		KeyStateWriter << It.Value.Value;
		KeyStateWriter << It.Value.LastUpDownTransitionTime;
		KeyStateWriter << It.Value.SampleCountAccumulator;
		KeyStateWriter << It.Value.RawValueAccumulator;

		// Serialize the bitfield
		uint8 BitField = 0;
		BitField |= (It.Value.bDown           & 0x01) << 0;
		BitField |= (It.Value.bDownPrevious   & 0x01) << 1;
		BitField |= (It.Value.bConsumed       & 0x01) << 2;
		BitField |= (It.Value.bWasJustFlushed & 0x01) << 3;
		BitField |= (It.Value.PairSampledAxes & 0x07) << 4;
		KeyStateWriter << BitField;

		// Serialize EventCounts arrays
		for (int ArrayIdx = 0; ArrayIdx < IE_MAX; ++ArrayIdx)
		{
			KeyStateWriter << It.Value.EventCounts[ArrayIdx];
		}

		// Serialize EventAccumulator arrays
		for (int ArrayIdx = 0; ArrayIdx < IE_MAX; ++ArrayIdx)
		{
			KeyStateWriter << It.Value.EventAccumulator[ArrayIdx];
		}

		FString KeyName = It.Key.ToString();
		UE_LOGF(LogDisplayClusterGame, VeryVerbose, "Input data serialized: %ls @ %d bytes", *KeyName, KeyStateBuffer.Num());
		OutKeyStateMap.Emplace(MoveTemp(KeyName), MoveTemp(KeyStateBuffer));
	}

	return true;
}

bool UDisplayClusterPlayerInput::DeserializeKeyStateMap(const TMap<FString, TArray<uint8>>& InKeyStateMap)
{
	TMap<FKey, FKeyState>& StateMap = GetKeyStateMap();

	// Reset local key state map
	StateMap.Reset();

	// Deserialize each key state individually
	for (const TPair<FString, TArray<uint8>>& It : InKeyStateMap)
	{
		UE_LOGF(LogDisplayClusterGame, VeryVerbose, "Deserializing input data: %ls @ %d bytes", *It.Key, It.Value.Num());

		// Deserialized data
		FKey Key(*It.Key);
		FKeyState KeyState;

		FMemoryReader KeyStateReader(It.Value);

		// Deserialize simple member variables
		KeyStateReader << KeyState.RawValue;
		KeyStateReader << KeyState.Value;
		KeyStateReader << KeyState.LastUpDownTransitionTime;
		KeyStateReader << KeyState.SampleCountAccumulator;
		KeyStateReader << KeyState.RawValueAccumulator;

		// Deserialize the bitfield
		uint8 BitField = 0;
		KeyStateReader << BitField;
		KeyState.bDown           = (BitField >> 0) & 0x01;
		KeyState.bDownPrevious   = (BitField >> 1) & 0x01;
		KeyState.bConsumed       = (BitField >> 2) & 0x01;
		KeyState.bWasJustFlushed = (BitField >> 3) & 0x01;
		KeyState.PairSampledAxes = (BitField >> 4) & 0x07;

		// Deserialize EventCounts arrays
		for (int ArrayIdx = 0; ArrayIdx < IE_MAX; ++ArrayIdx)
		{
			KeyStateReader << KeyState.EventCounts[ArrayIdx];
		}

		// Deserialize EventAccumulator arrays
		for (int ArrayIdx = 0; ArrayIdx < IE_MAX; ++ArrayIdx)
		{
			KeyStateReader << KeyState.EventAccumulator[ArrayIdx];
		}

		// Add incoming data to the local map
		StateMap.Emplace(MoveTemp(Key), MoveTemp(KeyState));
	}

	return true;
}

void UDisplayClusterPlayerInput::ProcessPolicy_ReplicatePrimary(const TArray<UInputComponent*>& InputComponentStack, const float DeltaTime, const bool bGamePaused)
{
	UE_LOGF(LogDisplayClusterGame, Verbose, "Processing input stack...");

	if (IPDisplayClusterClusterManager* const ClusterMgr = GDisplayCluster->GetPrivateClusterMgr())
	{
		TMap<FString, TArray<uint8>> KeyStates;

		// Always cache local inputs whether we're primary or not. If we're secondary now,
		// we may turn into primary in a moment, and the data would be available for others.
		{
			SerializeKeyStateMap(KeyStates);
			ClusterMgr->ImportNativeInputData(KeyStates);
		}

		// Get external input data only if we're not primary
		if (!ClusterMgr->IsPrimary())
		{
			ClusterMgr->GetNetApi().GetClusterSyncAPI()->GetNativeInputData(KeyStates);
			DeserializeKeyStateMap(KeyStates);
		}
	}
}
