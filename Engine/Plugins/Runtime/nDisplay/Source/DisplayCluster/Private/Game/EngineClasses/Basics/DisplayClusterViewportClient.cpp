// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterViewportClient.h"

#include "AudioDevice.h"
#include "AudioDeviceManager.h"
#include "BufferVisualizationData.h"
#include "EngineModule.h"
#include "EngineStats.h"
#include "EngineUtils.h"
#include "FXSystem.h"
#include "GlobalRenderResources.h"
#include "IHeadMountedDisplay.h"
#include "IXRTrackingSystem.h"
#include "RenderGraph.h"
#include "SceneManagement.h"
#include "SceneView.h"
#include "SceneViewExtension.h"
#include "UnrealEngine.h"

#include "Audio/AudioDebug.h"
#include "Components/LineBatchComponent.h"
#include "Debug/DebugDrawService.h"
#include "Engine/Engine.h"
#include "Engine/Canvas.h"
#include "Engine/Console.h"
#include "Engine/CoreSettings.h"
#include "Engine/LocalPlayer.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Framework/Application/SlateApplication.h"
#include "GameFramework/GameUserSettings.h"
#include "GameFramework/PlayerController.h"
#include "GameFramework/HUD.h"
#include "Slate/SceneViewport.h"
#include "UObject/Package.h"
#include "Widgets/SWindow.h"

#include "DisplayClusterEnums.h"
#include "DisplayClusterSceneViewExtensions.h"
#include "IDisplayCluster.h"
#include "IDisplayClusterCallbacks.h"

#include "Config/DisplayClusterConfigManager.h"
#include "Game/IDisplayClusterGameManager.h"
#include "Misc/DisplayClusterGlobals.h"
#include "Render/Device/IDisplayClusterRenderDevice.h"
#include "Render/GUILayer/DisplayClusterGuiLayerController.h"
#include "Render/Viewport/IDisplayClusterViewport.h"
#include "Render/Viewport/IDisplayClusterViewportManager.h"
#include "Render/Viewport/RenderFrame/DisplayClusterRenderFrame.h"

#if WITH_STATE_STREAM
#include "StateStreamDebugRenderer.h"
#include "StateStreamManager.h"
#endif


// Debug feature to synchronize and force all external resources to be transferred cross GPU at the end of graph execution.
// May be useful for testing cross GPU synchronization logic.
int32 GDisplayClusterForceCopyCrossGPU = 0;
static FAutoConsoleVariableRef CVarDisplayClusterForceCopyCrossGPU(
	TEXT("DC.ForceCopyCrossGPU"),
	GDisplayClusterForceCopyCrossGPU,
	TEXT("Force cross GPU copy of all resources after each view render.  Bad for perf, but may be useful for debugging."),
	ECVF_RenderThreadSafe
);

int32 GDisplayClusterSingleRender = 1;
static FAutoConsoleVariableRef CVarDisplayClusterSingleRender(
	TEXT("DC.SingleRender"),
	GDisplayClusterSingleRender,
	TEXT("Render Display Cluster view families in a single scene render."),
	ECVF_RenderThreadSafe
);

int32 GDisplayClusterSortViews = 1;
static FAutoConsoleVariableRef CVarDisplayClusterSortViews(
	TEXT("DC.SortViews"),
	GDisplayClusterSortViews,
	TEXT("Enable sorting of views by decreasing pixel count and decreasing GPU index.  Adds determinism, and tends to run inners first, which helps with scheduling, improving perf (default: enabled)."),
	ECVF_RenderThreadSafe
);

int32 GDisplayClusterDebugDraw = 1;
static FAutoConsoleVariableRef CVarDisplayClusterDebugDraw(
	TEXT("DC.DebugDraw"),
	GDisplayClusterDebugDraw,
	TEXT("Enable debug draw for nDisplay views.  Debug draw features are separately enabled, and default to off, this just provides an additional global toggle."),
	ECVF_RenderThreadSafe
);

// Replaces FApp::HasFocus
bool GDisplayClusterReplaceHasFocusFunction = true;
static FAutoConsoleVariableRef CVarDisplayClusterReplaceHasFocusFunction(
	TEXT("DC.ReplaceHasFocusFunction"),
	GDisplayClusterReplaceHasFocusFunction,
	TEXT("Replaces the function that FApp::HasFocus() uses, to mitigate OS stalls that happen in some systems."),
	ECVF_ReadOnly
);


struct FCompareViewFamilyBySizeAndGPU
{
	FORCEINLINE bool operator()(const FSceneViewFamilyContext& A, const FSceneViewFamilyContext& B) const
	{
		FIntPoint SizeA = A.RenderTarget->GetSizeXY();
		FIntPoint SizeB = B.RenderTarget->GetSizeXY();
		int32 AreaA = SizeA.X * SizeA.Y;
		int32 AreaB = SizeB.X * SizeB.Y;

		if (AreaA != AreaB)
		{
			// Decreasing area
			return AreaA > AreaB;
		}

		int32 GPUIndexA = A.Views[0]->GPUMask.GetFirstIndex();
		int32 GPUIndexB = B.Views[0]->GPUMask.GetFirstIndex();

		// Decreasing GPU index
		return GPUIndexA > GPUIndexB;
	}
};

UDisplayClusterViewportClient::UDisplayClusterViewportClient(FVTableHelper& Helper)
	: Super(Helper)
{
}

UDisplayClusterViewportClient::~UDisplayClusterViewportClient()
{
}

/** Util to find named canvas in transient package, and create if not found */
static UCanvas* GetCanvasByName(FName CanvasName)
{
	// Cache to avoid FString/FName conversions/compares
	static TMap<FName, UCanvas*> CanvasMap;
	UCanvas** FoundCanvas = CanvasMap.Find(CanvasName);
	if (!FoundCanvas)
	{
		UCanvas* CanvasObject = FindObject<UCanvas>(GetTransientPackage(), *CanvasName.ToString());
		if (!CanvasObject)
		{
			CanvasObject = NewObject<UCanvas>(GetTransientPackage(), CanvasName);
			CanvasObject->AddToRoot();
		}

		CanvasMap.Add(CanvasName, CanvasObject);
		return CanvasObject;
	}

	return *FoundCanvas;
}

// Wrapper for FSceneViewport to allow us to add custom stats specific to display cluster (per-view-family CPU and GPU perf)
class FDisplayClusterSceneViewport : public FSceneViewport
{
public:
	FDisplayClusterSceneViewport(const TSharedPtr<SViewport>& InViewportWidget)
		: FSceneViewport(InViewportWidget)
	{}

	static TSharedRef<FDisplayClusterSceneViewport> Create(UDisplayClusterViewportClient* InViewportClient, const TSharedPtr<SViewport>& InViewportWidget)
	{
		return FSceneViewport::Create<FDisplayClusterSceneViewport>(InViewportClient, InViewportWidget);
	}

	~FDisplayClusterSceneViewport()
	{
		for (auto It = CpuHistoryByDescription.CreateIterator(); It; ++It)
		{
			delete It.Value();
		}
	}

	float* GetNextHistoryWriteAddress(const FString& Description)
	{
		FRWScopeLock Lock(CpuHistoryMutex, SLT_Write);

		FCpuProfileHistory*& History = CpuHistoryByDescription.FindOrAdd(Description);
		if (!History)
		{
			History = new FCpuProfileHistory;
		}

		return &History->Times[(History->HistoryIndex++) % FCpuProfileHistory::HistoryCount];
	}

private:
	static void RightJustify(FCanvas* Canvas, UFont* StatsFont, const int32 X, const int32 Y, TCHAR const* Text, FLinearColor const& Color)
	{
		int32 ColumnSizeX, ColumnSizeY;
		StringSize(StatsFont, ColumnSizeX, ColumnSizeY, Text);
		Canvas->DrawShadowedString(X - ColumnSizeX, Y, Text, StatsFont, Color);
	}

	// Only callable when the CpuHistoryMutex is locked!
	float FetchHistoryAverage(const FString& Description) const
	{
		const FCpuProfileHistory* const* History = CpuHistoryByDescription.Find(Description);

		float Average = 0.f;
		if (History)
		{
			float ValidResultCount = 0.f;
			for (uint32 HistoryIndex = 0; HistoryIndex < FCpuProfileHistory::HistoryCount; HistoryIndex++)
			{
				float HistoryTime = (*History)->Times[HistoryIndex];
				if (HistoryTime > 0.f)
				{
					Average += HistoryTime;
					ValidResultCount += 1.f;
				}
			}
			if (ValidResultCount > 0.f)
			{
				Average /= ValidResultCount;
			}
		}
		return Average;
	}

	struct FCpuProfileHistory
	{
		FCpuProfileHistory()
		{
			FMemory::Memset(*this, 0);
		}

		static const uint32 HistoryCount = 64;

		// Constructor memsets everything to zero, assuming structure is Plain Old Data.  If any dynamic structures are
		// added, you'll need a more generalized constructor that zeroes out all the uninitialized data.
		uint32 HistoryIndex;
		float Times[HistoryCount];
	};

	// History payload is separately allocated in memory, as it's written to asynchronously by the Render Thread, and we
	// can't have it moved if the Map storage gets reallocated when new view families are added.
	TMap<FString, FCpuProfileHistory*> CpuHistoryByDescription;
	FRWLock CpuHistoryMutex;
};

// Override to allocate our custom viewport class
TSharedRef<FSceneViewport> UDisplayClusterViewportClient::CreateViewport(TSharedPtr<SViewport> InViewportWidget)
{
	return FDisplayClusterSceneViewport::Create(this, InViewportWidget);
}

void UDisplayClusterViewportClient::Init(struct FWorldContext& WorldContext, UGameInstance* OwningGameInstance, bool bCreateNewAudioDevice)
{
	const bool bIsNDisplayClusterMode = (GEngine->StereoRenderingDevice.IsValid() && GDisplayCluster->GetOperationMode() == EDisplayClusterOperationMode::Cluster);
	if (bIsNDisplayClusterMode)
	{
		// r.CompositionForceRenderTargetLoad
		IConsoleVariable* const ForceLoadCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.CompositionForceRenderTargetLoad"));
		if (ForceLoadCVar)
		{
			ForceLoadCVar->Set(int32(1));
		}

		// r.SceneRenderTargetResizeMethodForceOverride
		IConsoleVariable* const RTResizeForceOverrideCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.SceneRenderTargetResizeMethodForceOverride"));
		if (RTResizeForceOverrideCVar)
		{
			RTResizeForceOverrideCVar->Set(int32(1));
		}

		// r.SceneRenderTargetResizeMethod
		IConsoleVariable* const RTResizeCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.SceneRenderTargetResizeMethod"));
		if (RTResizeCVar)
		{
			RTResizeCVar->Set(int32(2));
		}

		// RHI.MaximumFrameLatency
		IConsoleVariable* const MaximumFrameLatencyCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("RHI.MaximumFrameLatency"));
		if (MaximumFrameLatencyCVar)
		{
			MaximumFrameLatencyCVar->Set(int32(1));
		}

		// vr.AllowMotionBlurInVR
		IConsoleVariable* const AllowMotionBlurInVR = IConsoleManager::Get().FindConsoleVariable(TEXT("vr.AllowMotionBlurInVR"));
		if (AllowMotionBlurInVR)
		{
			AllowMotionBlurInVR->Set(int32(1));
		}

		// Replace FApp::HasFocus to avoid stalls observed in some render nodes. 
		// It always return true, so all code behaves as if the application were in focus, even when rendering offscreen.
		if (GDisplayClusterReplaceHasFocusFunction)
		{
			FApp::SetHasFocusFunction([]() { return true; });
		}
	}

	Super::Init(WorldContext, OwningGameInstance, bCreateNewAudioDevice);
}

void UDisplayClusterViewportClient::Draw(FViewport* InViewport, FCanvas* SceneCanvas)
{
	////////////////////////////////
	// For any operation mode other than 'Cluster' we use default UGameViewportClient::Draw pipeline
	const bool bIsNDisplayClusterMode = (GEngine->StereoRenderingDevice.IsValid() && GDisplayCluster->GetOperationMode() == EDisplayClusterOperationMode::Cluster);

	// Get nDisplay stereo device
	IDisplayClusterRenderDevice* const DCRenderDevice = bIsNDisplayClusterMode ? static_cast<IDisplayClusterRenderDevice* const>(GEngine->StereoRenderingDevice.Get()) : nullptr;

	if (!bIsNDisplayClusterMode || DCRenderDevice == nullptr)
	{
#if WITH_EDITOR
		// Special render for PIE
		if (!IsRunningGame() && Draw_PIE(InViewport, SceneCanvas))
		{
			return;
		}
#endif
		return UGameViewportClient::Draw(InViewport, SceneCanvas);
	}

	//Get world for render
	UWorld* const MyWorld = GetWorld();
	if (MyWorld == nullptr)
	{
		return;
	}

	////////////////////////////////
	// Otherwise we use our own version of the UGameViewportClient::Draw which is basically
	// a simpler version of the original one but with multiple ViewFamilies support

	check(SceneCanvas);
	check(GEngine);

	OnBeginDraw().Broadcast();

	const bool bStereoRendering = GEngine->IsStereoscopic3D(InViewport);
	FCanvas* DebugCanvas = InViewport->GetDebugCanvas();

	// Create a temporary canvas if there isn't already one.
	static FName CanvasObjectName(TEXT("CanvasObject"));
	UCanvas* CanvasObject = GetCanvasByName(CanvasObjectName);
	CanvasObject->Canvas = SceneCanvas;

	// Create temp debug canvas object
	FIntPoint DebugCanvasSize = InViewport->GetSizeXY();
	if (bStereoRendering && GEngine->XRSystem.IsValid() && GEngine->XRSystem->GetHMDDevice())
	{
		DebugCanvasSize = GEngine->XRSystem->GetHMDDevice()->GetIdealDebugCanvasRenderTargetSize();
	}

	static FName DebugCanvasObjectName(TEXT("DebugCanvasObject"));
	UCanvas* DebugCanvasObject = GetCanvasByName(DebugCanvasObjectName);
	DebugCanvasObject->Init(DebugCanvasSize.X, DebugCanvasSize.Y, NULL, DebugCanvas);

	if (DebugCanvas)
	{
		DebugCanvas->SetScaledToRenderTarget(bStereoRendering);
		DebugCanvas->SetStereoRendering(bStereoRendering);

		const bool bGuiPropagationActive = FDisplayClusterGuiLayerController::Get().IsEnabled();
		DebugCanvas->SetWriteDestinationAlpha(bGuiPropagationActive);
	}
	if (SceneCanvas)
	{
		SceneCanvas->SetScaledToRenderTarget(bStereoRendering);
		SceneCanvas->SetStereoRendering(bStereoRendering);
	}

	// Force path tracing view mode, and extern code set path tracer show flags
	const bool bForcePathTracing = InViewport->GetClient()->GetEngineShowFlags()->PathTracing;
	if (bForcePathTracing)
	{
		EngineShowFlags.SetPathTracing(true);
		ViewModeIndex = VMI_PathTracing;
	}

	APlayerController* const PlayerController = GEngine->GetFirstLocalPlayerController(GetWorld());
	ULocalPlayer* LocalPlayer = nullptr;
	if (PlayerController)
	{
		LocalPlayer = PlayerController->GetLocalPlayer();
	}

	if (!PlayerController || !LocalPlayer)
	{
		return Super::Draw(InViewport, SceneCanvas);
	}

	// Gather all view families first
	TArray<FSceneViewFamilyContext*> ViewFamilies;

	// Initialize new render frame resources
	FDisplayClusterRenderFrame RenderFrame;
	if (!DCRenderDevice->BeginNewFrame(InViewport, MyWorld, RenderFrame))
	{
		// skip rendering: Can't build render frame
		return;
	}

	IDisplayClusterViewportManager* RenderFrameViewportManager = RenderFrame.GetViewportManager();
	if (!RenderFrameViewportManager)
	{
		// skip rendering: Can't find render manager
		return;
	}

	// Handle special viewports game-thread logic at frame begin
	DCRenderDevice->InitializeNewFrame();

	for (FDisplayClusterRenderFrameTarget& DCRenderTarget : RenderFrame.RenderTargets)
	{
		for (FDisplayClusterRenderFrameTargetViewFamily& DCViewFamily : DCRenderTarget.ViewFamilies)
		{
			// Create the view family for rendering the world scene to the viewport's render target
			ViewFamilies.Add(new FSceneViewFamilyContext(RenderFrameViewportManager->CreateViewFamilyConstructionValues(
				DCRenderTarget,
				MyWorld->Scene,
				EngineShowFlags,
				false				// bAdditionalViewFamily  (filled in later, after list of families is known, and optionally reordered)
			)));
			FSceneViewFamilyContext& ViewFamily = *ViewFamilies.Last();
			bool bIsFamilyVisible = false;

			// Configure family for the nDisplay.
			RenderFrameViewportManager->ConfigureViewFamily(DCRenderTarget, DCViewFamily, ViewFamily);

			ViewFamily.ViewMode = EViewModeIndex(ViewModeIndex);
			EngineShowFlagOverride(ESFIM_Game, ViewFamily.ViewMode, ViewFamily.EngineShowFlags, false);

			if (ViewFamily.EngineShowFlags.VisualizeBuffer && AllowDebugViewmodes())
			{
				// Process the buffer visualization console command
				FName NewBufferVisualizationMode = NAME_None;
				static IConsoleVariable* ICVar = IConsoleManager::Get().FindConsoleVariable(FBufferVisualizationData::GetVisualizationTargetConsoleCommandName());
				if (ICVar)
				{
					static const FName OverviewName = TEXT("Overview");
					FString ModeNameString = ICVar->GetString();
					FName ModeName = *ModeNameString;
					if (ModeNameString.IsEmpty() || ModeName == OverviewName || ModeName == NAME_None)
					{
						NewBufferVisualizationMode = NAME_None;
					}
					else
					{
						if (GetBufferVisualizationData().GetMaterial(ModeName) == NULL)
						{
							// Mode is out of range, so display a message to the user, and reset the mode back to the previous valid one
							UE_LOGF(LogConsoleResponse, Warning, "Buffer visualization mode '%ls' does not exist", *ModeNameString);
							NewBufferVisualizationMode = GetCurrentBufferVisualizationMode();
							// todo: cvars are user settings, here the cvar state is used to avoid log spam and to auto correct for the user (likely not what the user wants)
							ICVar->Set(*NewBufferVisualizationMode.GetPlainNameString(), ECVF_SetByCode);
						}
						else
						{
							NewBufferVisualizationMode = ModeName;
						}
					}
				}

				if (NewBufferVisualizationMode != GetCurrentBufferVisualizationMode())
				{
					SetCurrentBufferVisualizationMode(NewBufferVisualizationMode);
				}
			}

			TMap<ULocalPlayer*, FSceneView*> PlayerViewMap;
			FAudioDeviceHandle RetrievedAudioDevice = MyWorld->GetAudioDevice();
			TArray<FSceneView*> Views;

			for (FDisplayClusterRenderFrameTargetView& DCView : DCViewFamily.Views)
			{
				const FDisplayClusterViewport_Context ViewportContext = DCView.Viewport->GetContexts()[DCView.ContextNum];

				// Calculate the player's view information.
				FVector		ViewLocation;
				FRotator	ViewRotation;
				FSceneView* View = RenderFrameViewportManager->CalcSceneView(LocalPlayer, &ViewFamily, ViewLocation, ViewRotation, InViewport, nullptr, ViewportContext.StereoViewIndex);

				if (View && (!DCView.IsViewportContextCanBeRendered() || ViewFamily.RenderTarget == nullptr))
				{
					ViewFamily.Views.Remove(View);

					delete View;
					View = nullptr;
				}

				if (View)
				{
					Views.Add(View);

					// We don't allow instanced stereo currently
					View->bIsInstancedStereoEnabled = false;
					View->bShouldBindInstancedViewUB = false;

					if (View->Family->EngineShowFlags.Wireframe)
					{
						// Wireframe color is emissive-only, and mesh-modifying materials do not use material substitution, hence...
						View->DiffuseOverrideParameter = FVector4f(0.f, 0.f, 0.f, 0.f);
						View->SpecularOverrideParameter = FVector4f(0.f, 0.f, 0.f, 0.f);
					}
					else if (View->Family->EngineShowFlags.OverrideDiffuseAndSpecular)
					{
						View->DiffuseOverrideParameter = FVector4f(GEngine->LightingOnlyBrightness.R, GEngine->LightingOnlyBrightness.G, GEngine->LightingOnlyBrightness.B, 0.0f);
						View->SpecularOverrideParameter = FVector4f(.1f, .1f, .1f, 0.0f);
					}
					else if (View->Family->EngineShowFlags.LightingOnlyOverride)
					{
						View->DiffuseOverrideParameter = FVector4f(GEngine->LightingOnlyBrightness.R, GEngine->LightingOnlyBrightness.G, GEngine->LightingOnlyBrightness.B, 0.0f);
						View->SpecularOverrideParameter = FVector4f(0.f, 0.f, 0.f, 0.f);
					}
					else if (View->Family->EngineShowFlags.ReflectionOverride)
					{
						View->DiffuseOverrideParameter = FVector4f(0.f, 0.f, 0.f, 0.f);
						View->SpecularOverrideParameter = FVector4f(1, 1, 1, 0.0f);
						View->NormalOverrideParameter = FVector4f(0, 0, 1, 0.0f);
						View->RoughnessOverrideParameter = FVector2f(0.0f, 0.0f);
					}

					if (!View->Family->EngineShowFlags.Diffuse)
					{
						View->DiffuseOverrideParameter = FVector4f(0.f, 0.f, 0.f, 0.f);
					}

					if (!View->Family->EngineShowFlags.Specular)
					{
						View->SpecularOverrideParameter = FVector4f(0.f, 0.f, 0.f, 0.f);
					}

					if (!View->Family->EngineShowFlags.MaterialNormal)
					{
						View->NormalOverrideParameter = FVector4f(0.0f, 0.0f, 1.0f, 0.0f);
					}

					if (!View->Family->EngineShowFlags.MaterialAmbientOcclusion)
					{
						View->AmbientOcclusionOverrideParameter = FVector2f(1.0f, 0.0f);
					}

					View->CurrentBufferVisualizationMode = GetCurrentBufferVisualizationMode();

					View->CameraConstrainedViewRect = View->UnscaledViewRect;

					

					{
						// Save the location of the view.
						LocalPlayer->LastViewLocation = ViewLocation;

						PlayerViewMap.Add(LocalPlayer, View);

						// Update the listener.
						if (RetrievedAudioDevice && PlayerController != NULL)
						{
							bool bUpdateListenerPosition = true;

							// If the main audio device is used for multiple PIE viewport clients, we only
							// want to update the main audio device listener position if it is in focus
							if (GEngine)
							{
								FAudioDeviceManager* AudioDeviceManager = GEngine->GetAudioDeviceManager();

								// If there is more than one world referencing the main audio device
								if (AudioDeviceManager->GetNumMainAudioDeviceWorlds() > 1)
								{
									uint32 MainAudioDeviceID = GEngine->GetMainAudioDeviceID();
									if (AudioDevice->DeviceID == MainAudioDeviceID && !HasAudioFocus())
									{
										bUpdateListenerPosition = false;
									}
								}
							}

							if (bUpdateListenerPosition)
							{
								FVector Location;
								FVector ProjFront;
								FVector ProjRight;
								PlayerController->GetAudioListenerPosition(Location, ProjFront, ProjRight);

								FTransform ListenerTransform(FRotationMatrix::MakeFromXY(ProjFront, ProjRight));

								// Allow the HMD to adjust based on the head position of the player, as opposed to the view location
								if (GEngine->XRSystem.IsValid() && GEngine->StereoRenderingDevice.IsValid() && GEngine->StereoRenderingDevice->IsStereoEnabled())
								{
									const FVector Offset = GEngine->XRSystem->GetAudioListenerOffset();
									Location += ListenerTransform.TransformPositionNoScale(Offset);
								}

								ListenerTransform.SetTranslation(Location);
								ListenerTransform.NormalizeRotation();

								uint32 ViewportIndex = PlayerViewMap.Num() - 1;
								RetrievedAudioDevice->SetListener(MyWorld, ViewportIndex, ListenerTransform, (View->bCameraCut ? 0.f : MyWorld->GetDeltaSeconds()));

								FVector OverrideAttenuation;
								if (PlayerController->GetAudioListenerAttenuationOverridePosition(OverrideAttenuation))
								{
									RetrievedAudioDevice->SetListenerAttenuationOverride(ViewportIndex, OverrideAttenuation);
								}
								else
								{
									RetrievedAudioDevice->ClearListenerAttenuationOverride(ViewportIndex);
								}
							}
						}
					}

					// Add view information for resource streaming. Allow up to 5X boost for small FOV.
					const float StreamingScale = 1.f / FMath::Clamp<float>(View->LODDistanceFactor, .2f, 1.f);
					IStreamingManager::Get().AddViewInformation(View->ViewMatrices.GetViewOrigin(), View->UnscaledViewRect.Width(), View->UnscaledViewRect.Width() * View->ViewMatrices.GetViewToClip().M[0][0], StreamingScale);
					MyWorld->ViewLocationsRenderedLastFrame.Add(View->ViewMatrices.GetViewOrigin());

					FWorldCachedViewInfo& WorldViewInfo = World->CachedViewInfoRenderedLastFrame.AddDefaulted_GetRef();
					WorldViewInfo.ViewMatrix = View->ViewMatrices.GetWorldToView();
					WorldViewInfo.ProjectionMatrix = View->ViewMatrices.GetViewToClip();
					WorldViewInfo.ViewProjectionMatrix = View->ViewMatrices.GetWorldToClip();
					WorldViewInfo.ViewToWorld = View->ViewMatrices.GetViewToWorld();
					World->LastRenderTime = World->GetTimeSeconds();
				}
			}

#if CSV_PROFILER_STATS
			UpdateCsvCameraStats(PlayerViewMap);
#endif
			if (ViewFamily.Views.Num() > 0)
			{
				FinalizeViews(&ViewFamily, PlayerViewMap);

				// Collect rendering flags for nDisplay:
				EDisplayClusterViewportRenderingFlags ViewportRenderingFlags = EDisplayClusterViewportRenderingFlags::None;
				if (bStereoRendering)
				{
					EnumAddFlags(ViewportRenderingFlags, EDisplayClusterViewportRenderingFlags::StereoRendering);
				}
				
				// Completing the of a ViewDamily configuration.
				// The screen percentage is configurable in this function.
				RenderFrameViewportManager->PostConfigureViewFamily(DCRenderTarget, DCViewFamily, ViewFamily, Views,
					ViewportRenderingFlags, GetDPIScale());

				ViewFamily.bIsHDR = GetWindow().IsValid() ? GetWindow().Get()->GetIsHDR() : false;

#if WITH_MGPU
				ViewFamily.bForceCopyCrossGPU = GDisplayClusterForceCopyCrossGPU != 0;
#endif

				ViewFamily.ProfileDescription = DCViewFamily.Views[0].Viewport->GetId();

				// Draw the player views.
				if (!bDisableWorldRendering && PlayerViewMap.Num() > 0 && FSlateApplication::Get().GetPlatformApplication()->IsAllowedToRender())
				{
					// If we reach here, the view family should be rendered
					bIsFamilyVisible = true;
				}
			}

			if (!bIsFamilyVisible)
			{
				// Family didn't end up visible, remove last view family from the array
				delete ViewFamilies.Pop();
			}
		}
	}

	// Trigger PreSubmitViewFamilies event before submitting to render
	IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPreSubmitViewFamilies().Broadcast(ViewFamilies);

	// We gathered all the view families, now render them
	if (!ViewFamilies.IsEmpty())
	{
		if (ViewFamilies.Num() > 1)
		{
#if WITH_MGPU
			if (GDisplayClusterSortViews)
			{
				ViewFamilies.StableSort(FCompareViewFamilyBySizeAndGPU());
			}
#endif  // WITH_MGPU

			// Initialize some flags for which view family is which, now that any view family reordering has been handled.
			ViewFamilies[0]->bAdditionalViewFamily = false;
			ViewFamilies[0]->bIsFirstViewInMultipleViewFamily = true;

			for (int32 FamilyIndex = 1; FamilyIndex < ViewFamilies.Num(); FamilyIndex++)
			{
				FSceneViewFamily& ViewFamily = *ViewFamilies[FamilyIndex];
				ViewFamily.bAdditionalViewFamily = true;
				ViewFamily.bIsFirstViewInMultipleViewFamily = false;
			}
		}

		if (GDisplayClusterSingleRender)
		{
			GetRendererModule().BeginRenderingViewFamilies(
				SceneCanvas, MakeArrayView(reinterpret_cast<FSceneViewFamily* const*>(ViewFamilies.GetData()), ViewFamilies.Num()));
		}
		else
		{
			for (FSceneViewFamilyContext* ViewFamilyContext : ViewFamilies)
			{
				FSceneViewFamily& ViewFamily = *ViewFamilyContext;

				GetRendererModule().BeginRenderingViewFamily(SceneCanvas, &ViewFamily);

				if (GNumExplicitGPUsForRendering > 1)
				{
					const FRHIGPUMask SubmitGPUMask = ViewFamily.Views.Num() == 1 ? ViewFamily.Views[0]->GPUMask : FRHIGPUMask::All();
					ENQUEUE_RENDER_COMMAND(UDisplayClusterViewportClient_SubmitCommandList)(
						[SubmitGPUMask](FRHICommandListImmediate& RHICmdList)
					{
						SCOPED_GPU_MASK(RHICmdList, SubmitGPUMask);
						RHICmdList.ImmediateFlush(EImmediateFlushType::DispatchToRHIThread);
					});
				}
			}
		}
	}
	else
	{
		// Or if none to render, do logic for when rendering is skipped
		GetRendererModule().PerFrameCleanupIfSkipRenderer();
	}

	// Handle special viewports game-thread logic at frame end
	// custom postprocess single frame flag must be removed at frame end on game thread
	DCRenderDevice->FinalizeNewFrame();

	if (!GUseUnifiedTimeBudgetForStreaming)
	{
		// Update level streaming.
		MyWorld->UpdateLevelStreaming();
	}

	// Remove temporary debug lines.
	constexpr const UWorld::ELineBatcherType LineBatchersToFlush[] = { UWorld::ELineBatcherType::World, UWorld::ELineBatcherType::Foreground };
	World->FlushLineBatchers(LineBatchersToFlush);

	// Draw FX debug information.
	if (MyWorld->FXSystem)
	{
		MyWorld->FXSystem->DrawDebug(SceneCanvas);
	}

#if WITH_STATE_STREAM
	if (IStateStreamManager* Manager = MyWorld->GetStateStreamManager())
	{
		class FStateStreamDebugRenderer : public IStateStreamDebugRenderer
		{
		public:
			FStateStreamDebugRenderer(FViewport* V, UCanvas* C) : Viewport(V), Canvas(C) {}

			virtual void DrawText(const FStringView& Text, const FColor& Color) override
			{
				Canvas->SetDrawColor(Color);
				Canvas->DrawText(GEngine->GetSmallFont(), Text, 100, Y);
				Y += 20;
			}

			virtual void DrawText(uint32 InX, uint32 InY, const FStringView& Text, const FColor& Color) override
			{
				Canvas->SetDrawColor(Color);
				Canvas->DrawText(GEngine->GetSmallFont(), Text, InX, InY);
			}

			uint32 Y = 200;
			FViewport* Viewport;
			UCanvas* Canvas;
		} StateStreamDebugRenderer(InViewport, DebugCanvasObject);
		Manager->Game_DebugRender(StateStreamDebugRenderer);
	}
#endif

	{
		//ensure canvas has been flushed before rendering UI
		SceneCanvas->Flush_GameThread();

		// After all render target rendered call nDisplay frame rendering
		RenderFrameViewportManager->RenderFrame(InViewport);

		OnDrawn().Broadcast();

		// Allow the viewport to render additional stuff
		PostRender(DebugCanvasObject);
	}

	// Grab the player camera location and orientation so we can pass that along to the stats drawing code.
	FVector PlayerCameraLocation = FVector::ZeroVector;
	FRotator PlayerCameraRotation = FRotator::ZeroRotator;
	PlayerController->GetPlayerViewPoint(PlayerCameraLocation, PlayerCameraRotation);

	if (DebugCanvas)
	{
		DrawStatsHUD(MyWorld, InViewport, DebugCanvas, DebugCanvasObject, DebugProperties, PlayerCameraLocation, PlayerCameraRotation);

		if (GDisplayClusterDebugDraw && !ViewFamilies.IsEmpty())
		{
			UDebugDrawService::Draw(ViewFamilies.Last()->EngineShowFlags, InViewport, const_cast<FSceneView*>(ViewFamilies.Last()->Views[0]), DebugCanvas, DebugCanvasObject);
		}

		// Reset the debug canvas to be full-screen before drawing the console
		// (the debug draw service above has messed with the viewport size to fit it to a single player's subregion)
		DebugCanvasObject->Init(DebugCanvasSize.X, DebugCanvasSize.Y, NULL, DebugCanvas);

		// Render the console absolutely last because developer input is was matter the most.
		if (ViewportConsole)
		{
			ViewportConsole->PostRender_Console(DebugCanvasObject);
		}
	}

	if (!ViewFamilies.IsEmpty())
	{
		for (FSceneViewFamilyContext* ViewFamilyContext : ViewFamilies)
		{
			delete ViewFamilyContext;
		}
		ViewFamilies.Empty();
	}

	OnEndDraw().Broadcast();
}

#if WITH_EDITOR

#include "DisplayClusterRootActor.h"

bool UDisplayClusterViewportClient::Draw_PIE(FViewport* InViewport, FCanvas* SceneCanvas)
{
	IDisplayClusterGameManager* GameMgr = GDisplayCluster->GetGameMgr();

	if (GameMgr == nullptr || !IsInGameThread())
	{
		return false;
	}

	// Obtaining the primary root vector that can be used for PIE mode
	ADisplayClusterRootActor* const RootActor = GameMgr->GetRootActor();
	if (!RootActor || !RootActor->IsPrimaryRootActorForPIE())
	{
		return false;
	}

	check(SceneCanvas);
	check(GEngine);

	// When the PIE is used by this DCRA, we must create a new ViewportManager
	if(IDisplayClusterViewportManager* ViewportManager = RootActor->GetOrCreateViewportManager())
	{
		// Note: Logic that disable frustum preview rendering in PIE has been moved
		//       to the ADisplayClusterRootActor::PreparePreviewRendererSettings()

		// Do nothing if this DCRA is under an exclusive lock.
		if (ViewportManager->GetConfiguration().IsRootActorExclusivelyLocked())
		{
			return false;
		}

		const EDisplayClusterRenderFrameMode RenderFrameMode = ViewportManager->GetConfiguration().GetRenderModeForPIE();
		if (ViewportManager->GetViewportManagerPreview().InitializeClusterNodePreview(RenderFrameMode, GetWorld(), RootActor->PreviewNodeId, InViewport))
		{
			OnBeginDraw().Broadcast();

			ViewportManager->GetViewportManagerPreview().RenderClusterNodePreview(INDEX_NONE, InViewport, SceneCanvas);
			//ensure canvas has been flushed before rendering UI
			SceneCanvas->Flush_GameThread();

			OnEndDraw().Broadcast();

			return true;
		}
	}

	return false;
}
#endif /*WITH_EDITOR*/
