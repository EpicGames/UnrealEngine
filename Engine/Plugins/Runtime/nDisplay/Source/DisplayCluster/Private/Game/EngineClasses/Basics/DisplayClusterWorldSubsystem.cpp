// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterWorldSubsystem.h"

#include "Misc/DisplayClusterGlobals.h"
#include "DisplayClusterGameEngine.h"
#include "Game/IPDisplayClusterGameManager.h"

void UDisplayClusterWorldSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	// Whenever Streamed levels are removed or added this event is called.
	// This event is generally called when number of levels is changed.
	GetWorld()->OnLevelsChanged().AddUObject(this, &UDisplayClusterWorldSubsystem::OnLevelsChanged);
}

void UDisplayClusterWorldSubsystem::Deinitialize()
{
	GetWorld()->OnLevelsChanged().RemoveAll(this);
}

void UDisplayClusterWorldSubsystem::OnLevelsChanged()
{
	// We want to make sure that display cluster reacts to the level changes.
	// DisplayClusterGameManager needs to get fresh references to DisplayClusterRootActor.  
	if (UDisplayClusterGameEngine* GameEngine = Cast<UDisplayClusterGameEngine>(GEngine))
	{
		if (GameEngine->GetOperationMode() == EDisplayClusterOperationMode::Cluster && GameEngine->GetWorld() == GetWorld())
		{
			GDisplayCluster->EndScene();
			GDisplayCluster->StartScene(GetWorld());
		}

	}

}
