// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DisplayClusterCameraComponent.h"
#include "Components/BillboardComponent.h"
#include "Components/DisplayClusterICVFXCameraComponent.h"
#include "Camera/CameraComponent.h"
#include "Render/Viewport/IDisplayClusterViewport.h"
#include "Render/Viewport/Misc/DisplayClusterViewportHelpers.h"
#include "Render/Viewport/Configuration/DisplayClusterViewportConfigurationHelpers_Postprocess.h"

#include "DisplayClusterRootActor.h"

#include "CineCameraComponent.h"

#include "Engine/CollisionProfile.h"
#include "Engine/Texture2D.h"
#include "UObject/ConstructorHelpers.h"

#include "DisplayClusterConfigurationTypes.h"


UDisplayClusterCameraComponent::UDisplayClusterCameraComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
#if WITH_EDITOR
	, bEnableGizmo(true)
	, BaseGizmoScale(0.5f, 0.5f, 0.5f)
	, GizmoScaleMultiplier(1.f)
#endif
	, InterpupillaryDistance(6.4f)
	, bSwapEyes(false)
	, StereoOffset(EDisplayClusterEyeStereoOffset::None)
{
#if WITH_EDITOR
	if (GIsEditor)
	{
		ConstructorHelpers::FObjectFinderOptional<UTexture2D> SpriteTextureObject = TEXT("/nDisplay/Icons/S_nDisplayViewOrigin");
		SpriteTexture = SpriteTextureObject.Get();

	}

	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.TickGroup = TG_PostUpdateWork;
	PrimaryComponentTick.bStartWithTickEnabled = true;
	bAutoActivate = true;

#endif
}

void UDisplayClusterCameraComponent::ApplyViewPointComponentPostProcessesToViewport(IDisplayClusterViewport* InViewport)
{
	check(InViewport && !EnumHasAnyFlags(InViewport->GetRenderSettingsICVFX().RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::InternalResource));

	// Viewports that use the ViewPoint component get post - processing and more from the referenced camera component.
	// As we can see, we will have to use up to 3 different classes as different sources of these settings :
	// UCameraComponent->UCineCameraComponent->UDisplayClusterICVFXCameraComponent
	// Thus, we have to use all available rendering settings in our component's class.
	// Override viewport PP from camera, except internal ICVFX viewports
		
	using namespace UE::DisplayClusterViewportHelpers;
	// Get the same component from DCRA that is used as the configuration source. Then this component can also be used as a configuration data source.
	const UDisplayClusterCameraComponent& CfgCameraComponent = GetMatchingComponentFromRootActor(InViewport->GetConfiguration(), EDisplayClusterRootActorType::Configuration, *this);

	const bool bICVFXCameraBeingUsed = CfgCameraComponent.IsICVFXCameraBeingUsed();
	const bool bUseTargetCamera =
		CfgCameraComponent.IsActiveEngineCameraBeingUsed()
		|| bICVFXCameraBeingUsed
		|| CfgCameraComponent.IsExternalCameraBeingUsed();

	// Setup Outer Viewport postprocessing
	if (bUseTargetCamera)
	{
		const EDisplayClusterViewportCameraPostProcessFlags CameraPostProcessingFlags = CfgCameraComponent.GetCameraPostProcessFlags();

		// Also, if we are referencing the ICVFXCamera component, use the special ICVFX PostProcess from it.
		if (UDisplayClusterICVFXCameraComponent* SceneICVFXCameraComponent
			= !bICVFXCameraBeingUsed ? nullptr : GetRootActorComponentByName<UDisplayClusterICVFXCameraComponent>(InViewport->GetConfiguration(), EDisplayClusterRootActorType::Scene, CfgCameraComponent.ICVFXCameraComponentName))
		{
			// Use PostProcess from the ICVFXCamera
			// This function also uses PostProcess from the parent CineCamera class.
			SceneICVFXCameraComponent->ApplyICVFXCameraPostProcessesToViewport(InViewport, CameraPostProcessingFlags);
		}
		else
		{
			// Use post-processing settings from Camera/CineCamera or from the active game camera.
			FMinimalViewInfo CustomViewInfo;
			if (CfgCameraComponent.GetTargetCameraDesiredViewInternal(InViewport->GetConfiguration(), CustomViewInfo))
			{
				// Applies a filter to the post-processing settings.
				FDisplayClusterViewportConfigurationHelpers_Postprocess::FilterPostProcessSettings(CustomViewInfo.PostProcessSettings, CameraPostProcessingFlags);

				// Send camera postprocess to override
				InViewport->GetViewport_CustomPostProcessSettings().AddCustomPostProcess(IDisplayClusterViewport_CustomPostProcessSettings::ERenderPass::Override, CustomViewInfo.PostProcessSettings, CustomViewInfo.PostProcessBlendWeight, true);
			}
		}
	}
}

UCameraComponent* UDisplayClusterCameraComponent::GetTargetCameraComponent(const IDisplayClusterViewportConfiguration& InViewportConfiguration) const
{
	using namespace UE::DisplayClusterViewportHelpers;

	const UDisplayClusterCameraComponent& CfgCameraComponent = GetMatchingComponentFromRootActor(InViewportConfiguration, EDisplayClusterRootActorType::Configuration, *this);

	// 1. Active engine camera
	if (CfgCameraComponent.IsActiveEngineCameraBeingUsed())
	{
		return nullptr;
	}

	// 2. ICVFX camera component
	if (CfgCameraComponent.IsICVFXCameraBeingUsed())
	{
		if (UCameraComponent* SceneCameraComponent = GetRootActorComponentByName<UCameraComponent>(InViewportConfiguration, EDisplayClusterRootActorType::Scene, CfgCameraComponent.ICVFXCameraComponentName))
		{
			// If we use the ICVFX camera component, we must use GetActualCineCameraComponent() to get the actual camera.
			if (SceneCameraComponent->IsA<UDisplayClusterICVFXCameraComponent>())
			{
				if (UDisplayClusterICVFXCameraComponent* ICVFXCameraComponent = Cast<UDisplayClusterICVFXCameraComponent>(SceneCameraComponent))
				{
					if (UCineCameraComponent* ExtCineCameraComponent = ICVFXCameraComponent->GetActualCineCameraComponent())
					{
						// Use referenced camera as the source of Camera PP and CineCamera CustomNearClippingPlane
						return ExtCineCameraComponent;
					}
				}
			}

			return SceneCameraComponent;
		}
	}

	// 3. External camera actor
	return CfgCameraComponent.GetExternalCineCameraActorComponent();
}

bool UDisplayClusterCameraComponent::CanOverrideEyePosition() const
{
	// ICVFX camera can be used as a ViewPoint
	if (IsICVFXCameraBeingUsed())
	{
		return bUseICVFXCameraComponentTracking;
	}

	return false;
}

bool UDisplayClusterCameraComponent::IsViewPointOverrideCameraPosition() const
{
	// If the ICFX camera component is used, it can override the viewpoint position.
	if (IsICVFXCameraBeingUsed())
	{
		return !bUseICVFXCameraComponentTracking;
	}

	// Use external camera for render
	if (ShouldFollowCameraLocation())
	{
		return false;
	}

	// By default, ViewPoint is always used as the camera.
	return true;
}

bool UDisplayClusterCameraComponent::GetTargetCameraDesiredViewInternal(const IDisplayClusterViewportConfiguration& InViewportConfiguration, FMinimalViewInfo& InOutViewInfo, float* OutCustomNCP) const
{
	using namespace UE::DisplayClusterViewportHelpers;

	// Get the same component from DCRA that is used as the configuration source. Then this component can also be used as a configuration data source.
	const UDisplayClusterCameraComponent& CfgCameraComponent = GetMatchingComponentFromRootActor(InViewportConfiguration, EDisplayClusterRootActorType::Configuration, *this);

	const EDisplayClusterViewportCameraPostProcessFlags CameraPostProcessingFlags = CfgCameraComponent.GetCameraPostProcessFlags();
	const bool bUseCameraPostprocess = EnumHasAnyFlags(CameraPostProcessingFlags, EDisplayClusterViewportCameraPostProcessFlags::EnablePostProcess);

	float* OutCustomNearClippingPlane = OutCustomNCP;
	if (!EnumHasAnyFlags(CameraPostProcessingFlags, EDisplayClusterViewportCameraPostProcessFlags::EnableNearClippingPlane))
	{
		// Ignore NCP from the custom camera.
		OutCustomNearClippingPlane = nullptr;
	}

	bool bViewInfoChanged = false;

	// 1. Active engine camera
	if (CfgCameraComponent.IsActiveEngineCameraBeingUsed())
	{
		// Get ViewInfo from the Game camera.
		bViewInfoChanged = IDisplayClusterViewport::GetPlayerCameraView(InViewportConfiguration.GetCurrentWorld(), bUseCameraPostprocess, InOutViewInfo, OutCustomNearClippingPlane);
	}
	else if (UCameraComponent* SceneCameraComponent = GetTargetCameraComponent(InViewportConfiguration))
	{
		// Get ViewInfo from the camera component
		bViewInfoChanged = IDisplayClusterViewport::GetCameraComponentView(SceneCameraComponent, InViewportConfiguration.GetRootActorWorldDeltaSeconds(), bUseCameraPostprocess, InOutViewInfo, OutCustomNearClippingPlane);
	}

	if (IsViewPointOverrideCameraPosition())
	{
		// ViewPoint override camera position
		InOutViewInfo.Location = GetComponentLocation();
		InOutViewInfo.Rotation = GetComponentRotation();
	}

	return bViewInfoChanged;
}

void UDisplayClusterCameraComponent::GetDesiredView(const IDisplayClusterViewportConfiguration& InViewportConfiguration, FMinimalViewInfo& InOutViewInfo, float* OutCustomNearClippingPlane) const
{
	if (GetTargetCameraDesiredViewInternal(InViewportConfiguration, InOutViewInfo, OutCustomNearClippingPlane))
	{
		return;
	}

	// Ignore PP, because this component has no such settings
	InOutViewInfo.PostProcessBlendWeight = 0.f;

	if (OutCustomNearClippingPlane)
	{
		// Value less than zero means: don't override the NCP value
		*OutCustomNearClippingPlane = -1.f;
	}

	// By default this component is used as ViewPoint
	// Use this component as a camera
	InOutViewInfo.Location = GetComponentLocation();
	InOutViewInfo.Rotation = GetComponentRotation();
}

void UDisplayClusterCameraComponent::GetEyePosition(const IDisplayClusterViewportConfiguration& InViewportConfiguration, FVector& OutViewLocation, FRotator& OutViewRotation)
{
	// Use camera position as a ViewPoint
	if (CanOverrideEyePosition())
	{
		FMinimalViewInfo ViewInfo;
		if (GetTargetCameraDesiredViewInternal(InViewportConfiguration, ViewInfo))
		{
			OutViewLocation = ViewInfo.Location;
			OutViewRotation = ViewInfo.Rotation;

			return;
		}
	}

	// By default this component is used as ViewPoint
	// Use this component as a camera
	OutViewLocation = GetComponentLocation();
	OutViewRotation = GetComponentRotation();
}

#if WITH_EDITOR
void UDisplayClusterCameraComponent::SetVisualizationScale(float Scale)
{
	GizmoScaleMultiplier = Scale;
	RefreshVisualRepresentation();
}

void UDisplayClusterCameraComponent::SetVisualizationEnabled(bool bEnabled)
{
	bEnableGizmo = bEnabled;
	RefreshVisualRepresentation();
}
#endif

void UDisplayClusterCameraComponent::OnRegister()
{
#if WITH_EDITOR
	if (GIsEditor && !IsRunningCommandlet())
	{
		if (SpriteComponent == nullptr)
		{
			SpriteComponent = NewObject<UBillboardComponent>(this, NAME_None, RF_Transactional | RF_TextExportTransient);
			if (SpriteComponent)
			{
				SpriteComponent->SetupAttachment(this);
				SpriteComponent->SetIsVisualizationComponent(true);
				SpriteComponent->SetRelativeLocationAndRotation(FVector::ZeroVector, FRotator::ZeroRotator);
				SpriteComponent->SetMobility(EComponentMobility::Movable);
				SpriteComponent->Sprite = SpriteTexture;
				SpriteComponent->SpriteInfo.Category = TEXT("NDisplayViewOrigin");
				SpriteComponent->SpriteInfo.DisplayName = NSLOCTEXT("DisplayClusterCameraComponent", "NDisplayViewOriginSpriteInfo", "nDisplay View Point");
				SpriteComponent->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
				SpriteComponent->bHiddenInGame = true;
				SpriteComponent->bIsScreenSizeScaled = true;
				SpriteComponent->CastShadow = false;
				SpriteComponent->CreationMethod = CreationMethod;
				SpriteComponent->RegisterComponentWithWorld(GetWorld());
			}
		}
	}

	RefreshVisualRepresentation();
#endif

	Super::OnRegister();
}

#if WITH_EDITOR
bool UDisplayClusterCameraComponent::CanEditChange(const FProperty* InProperty) const
{
	// If other logic prevents editing, we want to respect that
	bool bIsEditable = Super::CanEditChange(InProperty);
	if (bIsEditable && InProperty)
	{
		// Can we edit tracking of the ICVFX camera component
		const FName PropertyName = InProperty->GetFName();
		if(PropertyName == GET_MEMBER_NAME_CHECKED(UDisplayClusterCameraComponent, bUseICVFXCameraComponentTracking)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(UDisplayClusterCameraComponent, bEnableICVFXDepthOfFieldCompensation)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(UDisplayClusterCameraComponent, bEnableICVFXColorGrading)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(UDisplayClusterCameraComponent, bEnableICVFXMotionBlur))
		{
			return IsICVFXCameraBeingUsed();
		}
	}

	return bIsEditable;
}

void UDisplayClusterCameraComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	RefreshVisualRepresentation();
}

void UDisplayClusterCameraComponent::RefreshVisualRepresentation()
{
	// Update the viz component
	if (SpriteComponent)
	{
		SpriteComponent->SetVisibility(bEnableGizmo);
		SpriteComponent->SetWorldScale3D(BaseGizmoScale * GizmoScaleMultiplier);
		// The sprite components don't get updated in real time without forcing render state dirty
		SpriteComponent->MarkRenderStateDirty();
	}
}
#endif

bool UDisplayClusterCameraComponent::IsActiveEngineCameraBeingUsed() const
{
	return TargetCameraType == EDisplayClusterTargetCameraType::ActiveEngineCamera;
}

bool UDisplayClusterCameraComponent::IsICVFXCameraBeingUsed() const
{
	return TargetCameraType == EDisplayClusterTargetCameraType::ICVFXCameraComponent && !ICVFXCameraComponentName.IsEmpty();
}

bool UDisplayClusterCameraComponent::IsExternalCameraBeingUsed() const
{
	return TargetCameraType == EDisplayClusterTargetCameraType::ExternalCineCameraActor && !IsICVFXCameraBeingUsed() && ExternalCineCameraActor.IsValid();
}

bool UDisplayClusterCameraComponent::ShouldFollowCameraLocation() const
{
	if (bFollowCameraPosition)
	{
		if (IsExternalCameraBeingUsed())
		{
			return true;
		}

		if (IsActiveEngineCameraBeingUsed())
		{
			return true;
		}
	}

	return false;
}

EDisplayClusterViewportCameraPostProcessFlags UDisplayClusterCameraComponent::GetCameraPostProcessFlags() const
{
	EDisplayClusterViewportCameraPostProcessFlags OutPostProcessFlags = EDisplayClusterViewportCameraPostProcessFlags::None;

	if (bEnablePostProcess)
	{
		EnumAddFlags(OutPostProcessFlags, EDisplayClusterViewportCameraPostProcessFlags::EnablePostProcess);
	}

	// If an ICVFX camera is used, DoF is always enabled
	if (bEnableDepthOfField || IsICVFXCameraBeingUsed())
	{
		EnumAddFlags(OutPostProcessFlags, EDisplayClusterViewportCameraPostProcessFlags::EnableDepthOfField);
	}

	// If an ICVFX camera is used, custom NCP is always enabled
	if (bEnableNearClippingPlane || IsICVFXCameraBeingUsed())
	{
		EnumAddFlags(OutPostProcessFlags, EDisplayClusterViewportCameraPostProcessFlags::EnableNearClippingPlane);
	}

	// This option requires an ICVFX camera.
	if (bEnableICVFXColorGrading && IsICVFXCameraBeingUsed())
	{
		EnumAddFlags(OutPostProcessFlags, EDisplayClusterViewportCameraPostProcessFlags::EnableICVFXColorGrading);
	}

	// This option requires an ICVFX camera.
	if (bEnableICVFXMotionBlur && IsICVFXCameraBeingUsed())
	{
		EnumAddFlags(OutPostProcessFlags, EDisplayClusterViewportCameraPostProcessFlags::EnableICVFXMotionBlur);
	}

	// This option requires an ICVFX camera.
	if (bEnableICVFXDepthOfFieldCompensation && IsICVFXCameraBeingUsed())
	{
		EnumAddFlags(OutPostProcessFlags, EDisplayClusterViewportCameraPostProcessFlags::EnableICVFXDepthOfFieldCompensation);
	}

	return OutPostProcessFlags;
}

UCameraComponent* UDisplayClusterCameraComponent::GetExternalCineCameraActorComponent() const
{
	if (IsExternalCameraBeingUsed())
	{
		if (ACineCameraActor* CineCamera = ExternalCineCameraActor.Get())
		{
			return CineCamera->GetCameraComponent();
		}
	}

	return nullptr;
}

#if WITH_EDITOR
UCameraComponent* UDisplayClusterCameraComponent::GetEditorPreviewCameraComponent()
{
	using namespace UE::DisplayClusterViewportHelpers;

	if (IsICVFXCameraBeingUsed())
	{
		if (UDisplayClusterICVFXCameraComponent* ICVFXCameraComponent = GetOwnerRootActorComponentByName<UDisplayClusterICVFXCameraComponent>(this, ICVFXCameraComponentName))
		{
			return ICVFXCameraComponent;
		}
	}
	else if (UCameraComponent* CameraComponent = GetExternalCineCameraActorComponent())
	{
		return CameraComponent;
	}

	return nullptr;
}

bool UDisplayClusterCameraComponent::GetEditorPreviewInfo(float DeltaTime, FMinimalViewInfo& ViewOut)
{
	if (UCameraComponent* CameraComponent = GetEditorPreviewCameraComponent())
	{
		return CameraComponent->GetEditorPreviewInfo(DeltaTime, ViewOut);
	}

	return false;
}

TSharedPtr<SWidget> UDisplayClusterCameraComponent::GetCustomEditorPreviewWidget()
{
	if (UCameraComponent* CameraComponent = GetEditorPreviewCameraComponent())
	{
		return CameraComponent->GetCustomEditorPreviewWidget();
	}

	return nullptr;
}
#endif