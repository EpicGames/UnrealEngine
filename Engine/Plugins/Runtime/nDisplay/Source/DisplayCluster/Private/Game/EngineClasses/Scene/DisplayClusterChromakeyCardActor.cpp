// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterChromakeyCardActor.h"

#include "DisplayClusterRootActor.h"
#include "Components/DisplayClusterChromakeyCardStageActorComponent.h"
#include "Components/DisplayClusterICVFXCameraComponent.h"
#include "Misc/DisplayClusterLog.h"


ADisplayClusterChromakeyCardActor::ADisplayClusterChromakeyCardActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer.DoNotCreateDefaultSubobject(LightCardStageActorComponentName))
{
	StageActorComponent = CreateOptionalDefaultSubobject<UDisplayClusterChromakeyCardStageActorComponent>(TEXT("ChromakeyStageActorComponent"));

#if WITH_EDITOR
	if (GIsEditor && !IsTemplate())
	{
		PropertyChangedHandle = FCoreUObjectDelegates::OnObjectPropertyChanged.AddUObject(this, &ADisplayClusterChromakeyCardActor::OnObjectPropertyChanged);
	}
#endif
}

ADisplayClusterChromakeyCardActor::~ADisplayClusterChromakeyCardActor()
{
#if WITH_EDITOR
	if (PropertyChangedHandle.IsValid())
	{
		FCoreUObjectDelegates::OnObjectPropertyChanged.Remove(PropertyChangedHandle);
	}
#endif
}

void ADisplayClusterChromakeyCardActor::Tick(float DeltaSeconds)
{
	Super::Tick(DeltaSeconds);
}

#if WITH_EDITOR
void ADisplayClusterChromakeyCardActor::PostEditUndo()
{
	Super::PostEditUndo();
	
	if (!GIsEditor && !IsTemplate())
	{
		if (const UWorld* World = GetWorld())
		{
			TWeakObjectPtr<ADisplayClusterChromakeyCardActor> WeakPtrThis(this);

			// Wait until the next frame to update color settings in -game. MU can call PostEditUndo
			// on this object before the object that made the change (ICVFX cameras) has processed their new values.
			World->GetTimerManager().SetTimerForNextTick([WeakPtrThis]()
			{
				if (WeakPtrThis.IsValid())
				{
					WeakPtrThis->UpdateChromakeySettings();
				}
			});
		}
	}
}

#endif

void ADisplayClusterChromakeyCardActor::AddToRootActor(ADisplayClusterRootActor* InRootActor)
{
	Super::AddToRootActor(InRootActor);
	UpdateChromakeySettings();
	// @todo uv chromakey cards - determine translucent sort order
}

void ADisplayClusterChromakeyCardActor::RemoveFromRootActor()
{
	Super::RemoveFromRootActor();
	UpdateChromakeySettings();
}

bool ADisplayClusterChromakeyCardActor::IsReferencedByICVFXCamera(const UDisplayClusterICVFXCameraComponent* InCamera) const
{
	check(InCamera);

	if (const UDisplayClusterChromakeyCardStageActorComponent* ChromakeyCardComponent = Cast<UDisplayClusterChromakeyCardStageActorComponent>(StageActorComponent))
	{
		if (ChromakeyCardComponent->IsReferencedByICVFXCamera(InCamera))
		{
			return true;
		}
	}
	
	const ADisplayClusterRootActor* RootActor = GetRootActorOwner();
	const FDisplayClusterConfigurationICVFX_ChromakeyRenderSettings* ChromakeyRenderSettings = RootActor ? InCamera->GetCameraSettingsICVFX().Chromakey.GetChromakeyRenderSettings(RootActor->GetStageSettings()) : nullptr;
	if (!ChromakeyRenderSettings)
	{
		return false;
	}

	if (ChromakeyRenderSettings->ShowOnlyList.Actors.Contains(this))
	{
		return true;
	}

	for (const FName& ThisLayer : Layers)
	{
		if (const FActorLayer* ExistingLayer = ChromakeyRenderSettings->ShowOnlyList.ActorLayers.FindByPredicate([ThisLayer](const FActorLayer& Layer)
		{
			return Layer.Name == ThisLayer;
		}))
		{
			return true;
		}
	}

	return false;
}

void ADisplayClusterChromakeyCardActor::UpdateChromakeySettings()
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("ADisplayClusterChromakeyCardActor::UpdateChromakeySettings"), STAT_UpdateChromakeySettings, STATGROUP_NDisplay);
	
	if (const ADisplayClusterRootActor* RootActor = GetRootActorOwner())
	{
		int32 Count = 0;
		FLinearColor ChromaColor(ForceInitToZero);
		TArray<UDisplayClusterICVFXCameraComponent*> ICVFXComponents;
		RootActor->GetComponents(ICVFXComponents);

		for (const UDisplayClusterICVFXCameraComponent* Component : ICVFXComponents)
		{
			if (IsReferencedByICVFXCamera(Component))
			{
				ChromaColor += Component->GetCameraSettingsICVFX().Chromakey.GetChromakeyColor(RootActor->GetStageSettings());
				Count++;
			}
		}

		const FLinearColor NewColor = Count > 0 ? ChromaColor / Count : FLinearColor::Green;
		if (NewColor != Color)
		{
#if WITH_EDITOR
			if (GIsEditor)
			{
				Modify(false);
			}
#endif
			
			Color = NewColor;

#if WITH_EDITOR
			if (GIsEditor)
			{
				// Update MU
				
				FProperty* Property = FindFieldChecked<FProperty>(StaticClass(), GET_MEMBER_NAME_CHECKED(ADisplayClusterChromakeyCardActor, Color));
				TArray<const FProperty*> ChangedProperties { Property };
				SnapshotTransactionBuffer(this, ChangedProperties);
			}
#endif
		}
	}
}

#if WITH_EDITOR
void ADisplayClusterChromakeyCardActor::OnObjectPropertyChanged(UObject* InObject,
	FPropertyChangedEvent& InPropertyChangedEvent)
{
	if (InObject && ((InObject->IsA<UDisplayClusterICVFXCameraComponent>()
		&& IsReferencedByICVFXCamera(CastChecked<UDisplayClusterICVFXCameraComponent>(InObject)))
		|| InObject == this))
	{
		UpdateChromakeySettings();
	}
}
#endif
