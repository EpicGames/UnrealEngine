// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DisplayClusterChromakeyCardStageActorComponent.h"
#include "DisplayClusterRootActor.h"
#include "Components/DisplayClusterICVFXCameraComponent.h"

#if WITH_EDITOR
void UDisplayClusterChromakeyCardStageActorComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();

	auto RemoveInvalidComponents = [this]()
	{
		for (auto ComponentIt = ICVFXCameras.CreateIterator(); ComponentIt; ++ComponentIt)
		{
			if (ComponentIt->OtherActor.IsValid() && ComponentIt->OtherActor.Get() != RootActor.Get())
			{
				ComponentIt.RemoveCurrent();
			}
		}
	};

	// Make sure component owner and root actor owner are in sync.
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UDisplayClusterChromakeyCardStageActorComponent, RootActor))
	{
		RemoveInvalidComponents();
		PopulateICVFXOwners();
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UDisplayClusterChromakeyCardStageActorComponent, ICVFXCameras) &&
		PropertyChangedEvent.ChangeType == EPropertyChangeType::ValueSet)
	{
		const int32 ArrayIndex = PropertyChangedEvent.GetArrayIndex(PropertyName.ToString());
		if (ArrayIndex >= 0 && ArrayIndex < ICVFXCameras.Num())
		{
			RootActor = Cast<ADisplayClusterRootActor>(ICVFXCameras[ArrayIndex].OtherActor.Get());
			RemoveInvalidComponents();
		}
	}
}
#endif

void UDisplayClusterChromakeyCardStageActorComponent::SetRootActor(ADisplayClusterRootActor* InRootActor)
{
	Super::SetRootActor(InRootActor);
	PopulateICVFXOwners();
}

bool UDisplayClusterChromakeyCardStageActorComponent::IsReferencedByICVFXCamera(
	const UDisplayClusterICVFXCameraComponent* InCamera) const
{
	for (const FSoftComponentReference& ComponentReference : GetICVFXCameras())
	{
		if (ComponentReference.GetComponent(InCamera->GetOwner()) == InCamera)
		{
			return true;
		}
	}

	return false;
}

void UDisplayClusterChromakeyCardStageActorComponent::PopulateICVFXOwners()
{
	if (RootActor.Get())
	{
		TArray<UDisplayClusterICVFXCameraComponent*> ICVFXComponents;
		RootActor.Get()->GetComponents(ICVFXComponents);

		ICVFXCameras.Reset(ICVFXComponents.Num());
	
		for (UDisplayClusterICVFXCameraComponent* ICVFXComponent : ICVFXComponents)
		{
			FSoftComponentReference ComponentReference;
			ComponentReference.OtherActor = RootActor.Get();
			ComponentReference.ComponentProperty = ICVFXComponent->GetFName();
		
			ICVFXCameras.Add(MoveTemp(ComponentReference));
		}
	}
}
