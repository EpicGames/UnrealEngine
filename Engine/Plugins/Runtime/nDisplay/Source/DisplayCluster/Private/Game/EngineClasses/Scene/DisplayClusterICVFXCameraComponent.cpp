// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DisplayClusterICVFXCameraComponent.h"
#include "Components/DrawFrustumComponent.h"

#include "Cluster/IPDisplayClusterClusterManager.h"

#include "Render/Viewport/IDisplayClusterViewport.h"

#include "Render/Viewport/Containers/DisplayClusterViewport_CameraMotionBlur.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_CustomFrustumRuntimeSettings.h"
#include "Render/Viewport/Configuration/DisplayClusterViewportConfigurationHelpers_ICVFX.h"
#include "Render/Viewport/Configuration/DisplayClusterViewportConfigurationHelpers_Postprocess.h"
#include "Render/Viewport/Misc/DisplayClusterViewportHelpers.h"

#include "Components/DisplayClusterCameraComponent.h"
#include "DisplayClusterRootActor.h"

#include "DisplayClusterConfigurationTypes.h"

#include "Misc/DisplayClusterGlobals.h"
#include "Misc/Parse.h"
#include "DisplayClusterEnums.h"
#include "Version/DisplayClusterICVFXCameraCustomVersion.h"

/** Enumerates the values of CVar "nDisplay.icvfx.camera.AdaptResolution". */
enum class EICVFXCameraAdaptResolutionMethod : uint8
{
	// The size doesn't change at all.
	Disabled = 0,

	// Respect pixels: Pixels = NewWidth * NewHeight = Width * Height.
	PreservePixelArea = 1,

	// Constant Pixel Area : Uses the maximum value of the camera frame size as the basis for the longest side of the sensor.
	PreserveLongestDimension = 2,

	// Max value in this enum.
	MAX = PreserveLongestDimension
};

/** Current method used to change the icvfx camera resolution. */
int32 GDisplayClusterICVFXCameraAdaptResolution = (uint8)EICVFXCameraAdaptResolutionMethod::PreservePixelArea;
static FAutoConsoleVariableRef CVarGDisplayClusterICVFXCameraAdaptResolution(
	TEXT("nDisplay.icvfx.camera.AdaptResolution"),
	GDisplayClusterICVFXCameraAdaptResolution,
	TEXT("Adapt camera viewport resolution with 'Filmback + CropSettings + SqueezeFactor' CineCamera settings.  (Default = 1)\n")
	TEXT("0 - Disabled.\n")
	TEXT("1 - Preserve Pixel Area: Pixels = NewWidth * NewHeight = Width * Height.\n")
	TEXT("2 - Preserve Longest Dimension : Uses the maximum value of the camera frame size as the basis for the longest side of the sensor.\n"),
	ECVF_Default
);

namespace UE::DisplayClusterICVFXCameraComponent
{
	static inline FIntPoint AdaptResolutionToAspectRatio(const FIntPoint& InResolution, const float InDesiredAspectRatio)
	{
		// Decode the method of resizing.
		const EICVFXCameraAdaptResolutionMethod ResizeMethod = (EICVFXCameraAdaptResolutionMethod)(FMath::Clamp(
			GDisplayClusterICVFXCameraAdaptResolution, 0, (uint8)EICVFXCameraAdaptResolutionMethod::MAX));

		// Implements resizing methods.
		switch (ResizeMethod)
		{
		case EICVFXCameraAdaptResolutionMethod::PreservePixelArea:
		{
			// AR = (W/H) -> W = (AR*H)
			// Pixels = W * H ->   Pixels = ((AR*H) * H) -> H^2 = (Pixels/AR)
			// H = sqrt(Pixels/AR), W = (AR*H)
			const int32 Pixels = InResolution.X * InResolution.Y;
			const float Height = FMath::Sqrt(Pixels / InDesiredAspectRatio);
			const float Width = InDesiredAspectRatio * Height;

			// Get new camera size
			return FIntPoint(FMath::RoundToInt(Width), FMath::RoundToInt(Height));
		}

		case EICVFXCameraAdaptResolutionMethod::PreserveLongestDimension:
		{
			// Use the max size of the RTT as a basis.
			const float BasisDimension = InResolution.GetMax();

			const float Width = (InDesiredAspectRatio >= 1.0)
				? BasisDimension
				: BasisDimension * InDesiredAspectRatio;

			const float Height = (InDesiredAspectRatio >= 1.0)
				? BasisDimension / InDesiredAspectRatio
				: BasisDimension;

			// Get new camera size
			return FIntPoint(FMath::RoundToInt(Width), FMath::RoundToInt(Height));
		}

		default:
			break;
		}

		return InResolution;
	}
};

UDisplayClusterICVFXCameraComponent::UDisplayClusterICVFXCameraComponent(const FObjectInitializer& ObjectInitializer)
{
	PrimaryComponentTick.bCanEverTick = true;
}

void UDisplayClusterICVFXCameraComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FDisplayClusterICVFXCameraCustomVersion::GUID);
}

void UDisplayClusterICVFXCameraComponent::PostLoad()
{
	Super::PostLoad();

	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	const int32 CustomVersion = GetLinkerCustomVersion(FDisplayClusterICVFXCameraCustomVersion::GUID);
	if (CustomVersion < FDisplayClusterICVFXCameraCustomVersion::UpdateChromakeyConfig)
	{
		const bool bHasCustomArchetype = GetArchetype() != StaticClass()->GetDefaultObject(false);
		const int32 ArchetypeVersion = GetArchetype()->GetLinkerCustomVersion(FDisplayClusterICVFXCameraCustomVersion::GUID);

		// UE-184291: If this camera component has a user-defined archetype and that archetype has been updated already, do not
		// attempt to update the component's properties; the new properties will already be set to the correct values from the
		// archetype and overriding them to these "default" values can cause bad things to happen. 
		if (!bHasCustomArchetype || ArchetypeVersion < FDisplayClusterICVFXCameraCustomVersion::UpdateChromakeyConfig)
		{
			const bool bCustomChromakey = CameraSettings.Chromakey.ChromakeyRenderTexture.bEnable_DEPRECATED;
			CameraSettings.Chromakey.ChromakeyType = bCustomChromakey ? 
				EDisplayClusterConfigurationICVFX_ChromakeyType::CustomChromakey :
				EDisplayClusterConfigurationICVFX_ChromakeyType::InnerFrustum;

			// New ICVFX cameras default to the global chromakey settings, but for pre 5.3 cameras, the source must be set to the ICVFX camera
			CameraSettings.Chromakey.ChromakeySettingsSource = EDisplayClusterConfigurationICVFX_ChromakeySettingsSource::ICVFXCamera;
		}
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
}

void UDisplayClusterICVFXCameraComponent::PostApplyToComponent()
{
	Super::PostApplyToComponent();

	CameraSettings.CameraDepthOfField.UpdateDynamicCompensationLUT();
}

void UDisplayClusterICVFXCameraComponent::GetCameraView(float DeltaTime, FMinimalViewInfo& InOutViewInfo)
{
	const ADisplayClusterRootActor* RootActor = Cast<ADisplayClusterRootActor>(GetOwner());
	if (RootActor == nullptr)
	{
		return;
	}

	if (UCineCameraComponent* ExternalCineCameraComponent = CameraSettings.GetExternalCineCameraComponent())
	{
		// Get ViewInfo from external CineCamera
		ExternalCineCameraComponent->GetCameraView(DeltaTime, InOutViewInfo);
	}
	else
	{
		// Get ViewInfo from this component
		UCineCameraComponent::GetCameraView(DeltaTime, InOutViewInfo);
	}

	CameraSettings.SetupViewInfo(RootActor->GetStageSettings(), InOutViewInfo);
}

UCineCameraComponent* UDisplayClusterICVFXCameraComponent::GetActualCineCameraComponent()
{
	UCineCameraComponent* ExternalCineCameraComponent = CameraSettings.GetExternalCineCameraComponent();

	return ExternalCineCameraComponent ? ExternalCineCameraComponent : this;
}

FString UDisplayClusterICVFXCameraComponent::GetCameraUniqueId() const
{
	return GetFName().ToString();
}

#if WITH_EDITOR
bool UDisplayClusterICVFXCameraComponent::GetEditorPreviewInfo(float DeltaTime, FMinimalViewInfo& ViewOut)
{
	UCineCameraComponent* ExternalCineCameraComponent = CameraSettings.GetExternalCineCameraComponent();
	return ExternalCineCameraComponent ?
		ExternalCineCameraComponent->GetEditorPreviewInfo(DeltaTime, ViewOut) :
		UCameraComponent::GetEditorPreviewInfo(DeltaTime, ViewOut);
}

TSharedPtr<SWidget> UDisplayClusterICVFXCameraComponent::GetCustomEditorPreviewWidget()
{
	UCineCameraComponent* ExternalCineCameraComponent = CameraSettings.GetExternalCineCameraComponent();
	return ExternalCineCameraComponent ?
		ExternalCineCameraComponent->GetCustomEditorPreviewWidget() :
		UCameraComponent::GetCustomEditorPreviewWidget();
}
#endif

void UDisplayClusterICVFXCameraComponent::TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	UpdateOverscanEstimatedFrameSize();

	if (CameraSettings.CameraDepthOfField.bAutomaticallySetDistanceToWall)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_STR("UDisplayClusterICVFXCameraComponent Query Distance To Wall");

		if (ADisplayClusterRootActor* RootActor = Cast<ADisplayClusterRootActor>(GetOwner()))
		{
			FVector CameraLocation = FVector::ZeroVector;
			FVector CameraDirection = FVector::XAxisVector;
			if (ACineCameraActor* ExternalCineCameraActor = CameraSettings.GetExternalCineCameraActor())
			{
				CameraLocation = ExternalCineCameraActor->GetActorLocation();
				CameraDirection = ExternalCineCameraActor->GetActorRotation().RotateVector(FVector::XAxisVector);
			}
			else
			{
				CameraLocation = GetComponentLocation();
				CameraDirection = GetComponentRotation().RotateVector(FVector::XAxisVector);
			}
			
			float DistanceToWall = 0.0;

			// For now, do a single trace from the center of the camera to the stage geometry.
			// Alternative methods of obtaining wall distance, such as averaging multiple points, can be performed here
			if (RootActor->GetDistanceToStageGeometry(CameraLocation, CameraDirection, DistanceToWall))
			{
				CameraSettings.CameraDepthOfField.DistanceToWall = DistanceToWall;
			}
		}
	}
}

bool UDisplayClusterICVFXCameraComponent::IsICVFXEnabled() const
{
	if (!GDisplayCluster)
	{
		return false;
	}

	if (const ADisplayClusterRootActor* const RootActor = Cast<ADisplayClusterRootActor>(GetOwner()))
	{
		if (const IDisplayClusterViewportConfiguration* const Configuration = RootActor->GetViewportConfiguration())
	{
			if (const UDisplayClusterConfigurationData* const ConfigurationData = Configuration->GetConfigurationData())
			{
				return GetCameraSettingsICVFX().IsICVFXEnabled(*ConfigurationData, Configuration->GetClusterNodeId());
			}
		}
	}

	return false;
}

const FDisplayClusterConfigurationICVFX_CameraSettings& UDisplayClusterICVFXCameraComponent::GetCameraSettingsICVFX() const
{
	return CameraSettings;
}

void UDisplayClusterICVFXCameraComponent::ApplyICVFXCameraPostProcessesToViewport(IDisplayClusterViewport* InViewport, const EDisplayClusterViewportCameraPostProcessFlags InPostProcessingFlags)
{
	if (InViewport)
	{
		using namespace UE::DisplayClusterViewportHelpers;
		// Get the same component from DCRA that is used as the configuration source. Then this component can also be used as a configuration data source.
		const UDisplayClusterICVFXCameraComponent& CfgICVFXCameraComponent = GetMatchingComponentFromRootActor(InViewport->GetConfiguration(), EDisplayClusterRootActorType::Configuration, *this);
		
		FDisplayClusterViewportConfigurationHelpers_Postprocess::ImplApplyICVFXCameraPostProcessesToViewport(InViewport->ToSharedRef().Get(), *this, CfgICVFXCameraComponent.GetCameraSettingsICVFX(), InPostProcessingFlags);
	}
}

FDisplayClusterShaderParameters_ICVFX::FCameraSettings UDisplayClusterICVFXCameraComponent::GetICVFXCameraShaderParameters(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings, const FDisplayClusterConfigurationICVFX_CameraSettings& InCameraSettings)
{
	FDisplayClusterShaderParameters_ICVFX::FCameraSettings OutCameraSettings;

	const float CameraMult =
		CameraSettings.GetCameraBufferRatio(InStageSettings)
		* CameraSettings.CustomFrustum.GetCameraAdaptResolutionRatio(InStageSettings);

	const FIntPoint CameraFrameSize = GetICVFXCameraFrameSize(InStageSettings, InCameraSettings);
	const FIntPoint RealInnerFrustumResolution(CameraFrameSize.X * CameraMult, CameraFrameSize.Y * CameraMult);

	// Creates unique name "DCRA.Component"
	const FString UniqueComponentName = FString::Printf(TEXT("%s.%s"), *GetOwner()->GetName(), *GetName());

	FIntRect RealViewportRect(FIntPoint(0, 0), RealInnerFrustumResolution);
	FDisplayClusterViewport_CustomFrustumSettings RealFrustumSettings;
	FDisplayClusterViewport_CustomFrustumRuntimeSettings RealFrustumRuntimeSettings;

	FDisplayClusterViewportConfigurationHelpers_ICVFX::UpdateCameraCustomFrustum(CameraSettings.CustomFrustum, RealFrustumSettings);
	FDisplayClusterViewport_CustomFrustumRuntimeSettings::UpdateCustomFrustumSettings(
		UniqueComponentName, RealFrustumSettings, RealFrustumRuntimeSettings, RealViewportRect, TEXT("ShaderParameters CustomFrustum"));

	const FDisplayClusterViewport_CustomFrustumRuntimeSettings::FCustomFrustumPercent& Angles = RealFrustumRuntimeSettings.CustomFrustumPercent;

	// Camera Border
	{
		if (InCameraSettings.Border.Enable)
		{
			const float RealThicknessScaleValue = 0.1f;

			OutCameraSettings.InnerCameraBorderColor = InCameraSettings.Border.Color;
			OutCameraSettings.InnerCameraBorderThickness = InCameraSettings.Border.Thickness;
		}
		else
		{
			// No border:
			OutCameraSettings.InnerCameraBorderColor = FLinearColor::Black;
			OutCameraSettings.InnerCameraBorderThickness = 0.0f;
		}
	}

	// Camera Soft Edges
	{
		FVector4 SoftEdge(
			// remap values from 0-1 GUI range into acceptable 0.0 - 0.25 shader range
			FMath::GetMappedRangeValueClamped(FVector2D(0.0, 1.0f), FVector2D(0.0, 0.25), InCameraSettings.SoftEdge.Horizontal), // Left
			FMath::GetMappedRangeValueClamped(FVector2D(0.0, 1.0f), FVector2D(0.0, 0.25), InCameraSettings.SoftEdge.Vertical), // Top

			// ZW now used in other way
			// Z for new parameter Feather
			InCameraSettings.SoftEdge.Feather
		);


		SoftEdge.X /= (1 + Angles.Left + Angles.Right);
		SoftEdge.Y /= (1 + Angles.Top + Angles.Bottom);

		OutCameraSettings.SoftEdge = SoftEdge;
	}

	return OutCameraSettings;
}

FIntPoint UDisplayClusterICVFXCameraComponent::GetICVFXCameraFrameSize(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings, const FDisplayClusterConfigurationICVFX_CameraSettings& InCameraSettings)
{
	using namespace UE::DisplayClusterICVFXCameraComponent;

	const FIntPoint CameraFrameSize = InCameraSettings.RenderSettings.CustomFrameSize.bUseCustomSize
		? FIntPoint(InCameraSettings.RenderSettings.CustomFrameSize.CustomWidth, InCameraSettings.RenderSettings.CustomFrameSize.CustomHeight)
		: FIntPoint(InStageSettings.DefaultFrameSize.Width, InStageSettings.DefaultFrameSize.Height);

	UCineCameraComponent* ActualCineCameraComponent = GetActualCineCameraComponent();
	if (!ActualCineCameraComponent)
	{
		// Adaptation math requires an actual CineCamera component
		return CameraFrameSize;
	}

	// User can disable this feature.
	const bool bAdaptFrameSize = InCameraSettings.RenderSettings.CustomFrameSize.bUseCustomSize
		? InCameraSettings.RenderSettings.CustomFrameSize.bAdaptSize
		: InStageSettings.DefaultFrameSize.bAdaptSize;

	// Get the size of the cinematic camera's cropped sensor:
	const double CroppedSensorWidth  = FMath::Tan(FMath::DegreesToRadians(ActualCineCameraComponent->GetHorizontalFieldOfView()) / 2.f) * 2.f * ActualCineCameraComponent->CurrentFocalLength;
	const double CroppedSensorHeight = FMath::Tan(FMath::DegreesToRadians(ActualCineCameraComponent->GetVerticalFieldOfView()) / 2.f) * 2.f * ActualCineCameraComponent->CurrentFocalLength;

	if (!(CroppedSensorWidth > 0.f && CroppedSensorHeight > 0.f) || !bAdaptFrameSize)
	{
		// The CineCamera cropped sensor size has invalid values.
		// or this feature is disabled.
		return CameraFrameSize;
	}

	// Desired aspect ratio
	double CroppedSensorAR = CroppedSensorWidth / CroppedSensorHeight;

	// When no adopt resolution is used we must compensate for the change in aspect ratio caused by overscan.
	if (InCameraSettings.CustomFrustum.bEnable && !InCameraSettings.CustomFrustum.bAdaptResolution)
	{
		// Creates unique name "DCRA.Component"
		const FString UniqueComponentName = FString::Printf(TEXT("%s.%s"), *GetOwner()->GetName(), *GetName());

		// Overscan should only be used through this api:
		FDisplayClusterViewport_CustomFrustumSettings CustomFrustumSettings;
		FDisplayClusterViewport_CustomFrustumRuntimeSettings CustomFrustumRuntimeSettings;

		FDisplayClusterViewportConfigurationHelpers_ICVFX::UpdateCameraCustomFrustum(InCameraSettings.CustomFrustum, CustomFrustumSettings);

		FIntPoint DesiredSize = AdaptResolutionToAspectRatio(CameraFrameSize, CroppedSensorAR);
		FIntRect ViewportRect(FIntPoint(0, 0), DesiredSize);
		FDisplayClusterViewport_CustomFrustumRuntimeSettings::UpdateCustomFrustumSettings(
			UniqueComponentName, CustomFrustumSettings, CustomFrustumRuntimeSettings, ViewportRect, TEXT("CameraFrame Size CustomFrustum"));

		// Overscan without the bAdaptResolution option does not change the RTT aspect ratio.
		// In this case, the sensor AR must be modified to include CustomFrustumPercent values.
		CroppedSensorAR *= CustomFrustumRuntimeSettings.CustomFrustumPercent.GetAspectRatioMult();
	}

	const FIntPoint AdaptedCameraFrameSize = AdaptResolutionToAspectRatio(CameraFrameSize, CroppedSensorAR);

	return AdaptedCameraFrameSize;
}

void UDisplayClusterICVFXCameraComponent::UpdateOverscanEstimatedFrameSize()
{
	const ADisplayClusterRootActor* RootActor = Cast<ADisplayClusterRootActor>(GetOwner());
	if (RootActor == nullptr)
	{
		return;
	}

	// Creates unique name "DCRA.Component"
	const FString UniqueComponentName = FString::Printf(TEXT("%s.%s"), *GetOwner()->GetName(), *GetName());

	const FDisplayClusterConfigurationICVFX_StageSettings& StageSettings = RootActor->GetStageSettings();
	{
		// calculate estimations
		FDisplayClusterConfigurationICVFX_CameraSettings EstimatedCameraSettings = CameraSettings;
		EstimatedCameraSettings.CustomFrustum.bEnable = true;
		EstimatedCameraSettings.CustomFrustum.bAdaptResolution = true;

		const float CameraMult =
			EstimatedCameraSettings.GetCameraBufferRatio(StageSettings)
			* EstimatedCameraSettings.CustomFrustum.GetCameraAdaptResolutionRatio(StageSettings);

		const FIntPoint CameraFrameSize = GetICVFXCameraFrameSize(StageSettings, EstimatedCameraSettings);
		const FIntPoint EstimatedInnerFrustumResolution(CameraFrameSize.X * CameraMult, CameraFrameSize.Y * CameraMult);

		FIntRect EstimatedViewportRect(FIntPoint(0, 0), EstimatedInnerFrustumResolution);
		FDisplayClusterViewport_CustomFrustumSettings EstimatedFrustumSettings;
		FDisplayClusterViewport_CustomFrustumRuntimeSettings EstimatedFrustumRuntimeSettings;

		FDisplayClusterViewportConfigurationHelpers_ICVFX::UpdateCameraCustomFrustum(EstimatedCameraSettings.CustomFrustum, EstimatedFrustumSettings);
		FDisplayClusterViewport_CustomFrustumRuntimeSettings::UpdateCustomFrustumSettings(
			UniqueComponentName, EstimatedFrustumSettings, EstimatedFrustumRuntimeSettings, EstimatedViewportRect, TEXT("Estimated CustomFrustum"));

		// Assign estimated calculated values
		CameraSettings.CustomFrustum.EstimatedOverscanResolution = EstimatedViewportRect.Size();
	}

	{
		const float CameraMult =
			CameraSettings.GetCameraBufferRatio(StageSettings)
			* CameraSettings.CustomFrustum.GetCameraAdaptResolutionRatio(StageSettings);

		const FIntPoint CameraFrameSize = GetICVFXCameraFrameSize(StageSettings, CameraSettings);
		const FIntPoint RealInnerFrustumResolution(CameraFrameSize.X * CameraMult, CameraFrameSize.Y * CameraMult);

		FIntRect RealViewportRect(FIntPoint(0, 0), RealInnerFrustumResolution);
		FDisplayClusterViewport_CustomFrustumSettings RealFrustumSettings;
		FDisplayClusterViewport_CustomFrustumRuntimeSettings RealFrustumRuntimeSettings;

		FDisplayClusterViewportConfigurationHelpers_ICVFX::UpdateCameraCustomFrustum(CameraSettings.CustomFrustum, RealFrustumSettings);
		FDisplayClusterViewport_CustomFrustumRuntimeSettings::UpdateCustomFrustumSettings(
			UniqueComponentName, RealFrustumSettings, RealFrustumRuntimeSettings, RealViewportRect, TEXT("Real CustomFrustum"));

		// Assign real calculated values
		CameraSettings.CustomFrustum.InnerFrustumResolution = RealViewportRect.Size();
	}
	
	const int32 EstimatedPixel = CameraSettings.CustomFrustum.EstimatedOverscanResolution.X * CameraSettings.CustomFrustum.EstimatedOverscanResolution.Y;
	const int32 BasePixels = CameraSettings.CustomFrustum.InnerFrustumResolution.X * CameraSettings.CustomFrustum.InnerFrustumResolution.Y;

	CameraSettings.CustomFrustum.OverscanPixelsIncrease = ((float)(EstimatedPixel) / (float)(BasePixels));
}

void UDisplayClusterICVFXCameraComponent::OnRegister()
{
	Super::OnRegister();

	// If the blueprint is being reconstructed, we can't update the dynamic LUT here without causing issues
	// when the reconstruction attempts to check if the component's properties are modified, as this call will
	// load the compensation LUT soft pointer, resulting in a memory difference from the archetype.
	// The PostApplyToComponent call handles rebuilding the dynamic LUT in such a case
	if (!GIsReconstructingBlueprintInstances)
	{
		CameraSettings.CameraDepthOfField.UpdateDynamicCompensationLUT();
	}

#if WITH_EDITORONLY_DATA
	// disable frustum for icvfx camera component
	if (DrawFrustum != nullptr)
	{
		DrawFrustum->bFrustumEnabled = false;
	}

	// Update ExternalCineactor behaviour
	UpdateICVFXPreviewState();
#endif
}

void UDisplayClusterICVFXCameraComponent::SetDepthOfFieldParameters(const FDisplayClusterConfigurationICVFX_CameraDepthOfField& NewDepthOfFieldParams)
{
	CameraSettings.CameraDepthOfField.bEnableDepthOfFieldCompensation = NewDepthOfFieldParams.bEnableDepthOfFieldCompensation;
	CameraSettings.CameraDepthOfField.bAutomaticallySetDistanceToWall = NewDepthOfFieldParams.bAutomaticallySetDistanceToWall;
	CameraSettings.CameraDepthOfField.DistanceToWallOffset = NewDepthOfFieldParams.DistanceToWallOffset;

	if (!NewDepthOfFieldParams.bAutomaticallySetDistanceToWall)
	{
		CameraSettings.CameraDepthOfField.DistanceToWall = NewDepthOfFieldParams.DistanceToWall;
	}

	bool bGenerateNewLUT = false;
	if (CameraSettings.CameraDepthOfField.DepthOfFieldGain != NewDepthOfFieldParams.DepthOfFieldGain)
	{
		CameraSettings.CameraDepthOfField.DepthOfFieldGain = NewDepthOfFieldParams.DepthOfFieldGain;
		bGenerateNewLUT = true;
	}

	if (CameraSettings.CameraDepthOfField.CompensationLUT != NewDepthOfFieldParams.CompensationLUT)
	{
		CameraSettings.CameraDepthOfField.CompensationLUT = NewDepthOfFieldParams.CompensationLUT;
		bGenerateNewLUT = true;
	}

	if (bGenerateNewLUT)
	{
		CameraSettings.CameraDepthOfField.UpdateDynamicCompensationLUT();
	}
}

#if WITH_EDITORONLY_DATA
void UDisplayClusterICVFXCameraComponent::PreEditChange(FProperty* PropertyThatWillChange)
{
	Super::PreEditChange(PropertyThatWillChange);

	// save the current value
	ExternalCameraCachedValue = CameraSettings.ExternalCameraActor;
}

void UDisplayClusterICVFXCameraComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	FName PropertyName = PropertyChangedEvent.GetPropertyName();
	if (PropertyName == GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_CameraDepthOfField, CompensationLUT) ||
		(PropertyName == GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_CameraDepthOfField, DepthOfFieldGain) && PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive))
	{
		CameraSettings.CameraDepthOfField.UpdateDynamicCompensationLUT();
	}

	UpdateICVFXPreviewState();
}

void UDisplayClusterICVFXCameraComponent::UpdateICVFXPreviewState()
{
	// handle frustum visibility
	if (ACineCameraActor* ExternalCineCameraActor = CameraSettings.GetExternalCineCameraActor())
	{
		UCineCameraComponent* ExternalCineCameraComponent = ExternalCineCameraActor->GetCineCameraComponent();
		if (IsValid(ExternalCineCameraComponent))
		{
			ExternalCineCameraComponent->bDrawFrustumAllowed = false;
		}

		UDrawFrustumComponent* DrawFustumComponent = Cast<UDrawFrustumComponent>(ExternalCineCameraActor->GetComponentByClass(UDrawFrustumComponent::StaticClass()));
		if (IsValid(DrawFustumComponent))
		{
			DrawFustumComponent->bFrustumEnabled = false;
			DrawFustumComponent->MarkRenderStateDirty();
		}

		if (IsValid(ProxyMeshComponent))
		{
			ProxyMeshComponent->DestroyComponent();
			ProxyMeshComponent = nullptr;
		}
	}

	// restore frustum visibility if reference was changed
	if (ACineCameraActor* ExternalCineCameraCachedActor = ExternalCameraCachedValue.Get())
	{
		if (IsValid(ExternalCineCameraCachedActor))
		{
			UDrawFrustumComponent* DrawFustumComponent = Cast<UDrawFrustumComponent>(ExternalCineCameraCachedActor->GetComponentByClass(UDrawFrustumComponent::StaticClass()));
			if (IsValid(DrawFustumComponent))
			{
				DrawFustumComponent->bFrustumEnabled = true;
				DrawFustumComponent->MarkRenderStateDirty();
			}
		}

		ExternalCameraCachedValue.Reset();
	}
}
#endif
