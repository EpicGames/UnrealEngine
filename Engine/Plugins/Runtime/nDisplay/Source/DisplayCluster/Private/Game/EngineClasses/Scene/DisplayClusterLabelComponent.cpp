// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DisplayClusterLabelComponent.h"

#include "DisplayClusterLabelWidget.h"
#include "DisplayClusterLightCardActor.h"
#include "DisplayClusterRootActor.h"
#include "DisplayClusterWidgetComponent.h"

#include "EngineUtils.h"
#include "Components/TextRenderComponent.h"
#include "Components/WidgetComponent.h"
#include "Kismet/KismetMathLibrary.h"

UDisplayClusterLabelComponent::UDisplayClusterLabelComponent()
{
	PrimaryComponentTick.bCanEverTick = true;
	
	WidgetComponent = CreateDefaultSubobject<UDisplayClusterWidgetComponent>(TEXT("LabelWidget"));
	
	// Keeps this visible even if its clipping through the owner.
	// Requires the owner have a transparent blend mode on its material.
	WidgetComponent->SetBlendMode(EWidgetBlendMode::Transparent);
	WidgetComponent->SetTranslucencySortDistanceOffset(-100.f);
	WidgetComponent->SetUsingAbsoluteScale(true);
	WidgetComponent->SetCollisionEnabled(ECollisionEnabled::NoCollision);
	WidgetComponent->SetCollisionResponseToAllChannels(ECollisionResponse::ECR_Ignore);
	WidgetComponent->SetCollisionObjectType(ECollisionChannel::ECC_Visibility);

	WidgetClass = TSoftClassPtr<UDisplayClusterLabelWidget>(FSoftClassPath(TEXT("/nDisplay/Widgets/BP_DisplayClusterLabelWidget.BP_DisplayClusterLabelWidget_C")));

	SetRelativeRotation(FRotator(90.f, 0.f, -90.f));
	SetVisibility(false);
}

void UDisplayClusterLabelComponent::SetLabelConfiguration(const FDisplayClusterLabelConfiguration& InConfiguration)
{
	SetRootActor(InConfiguration.RootActor);
	SetWidgetScale(InConfiguration.Scale);
	SetVisibility(InConfiguration.bVisible);
	LabelFlags = InConfiguration.LabelFlags;
}

void UDisplayClusterLabelComponent::SetRootActor(ADisplayClusterRootActor* InActor)
{
	RootActor = InActor;
}

void UDisplayClusterLabelComponent::SetWidgetScale(float NewValue)
{
	WidgetScale = NewValue;
	WidgetComponent->SetWidgetScale(WidgetScale);
}

float UDisplayClusterLabelComponent::GetWidgetScale() const
{
	return WidgetScale;
}

void UDisplayClusterLabelComponent::SetLabelFlags(EDisplayClusterLabelFlags InFlags)
{
	EnumAddFlags(LabelFlags, InFlags);
}

void UDisplayClusterLabelComponent::ClearLabelFlags(EDisplayClusterLabelFlags InFlags)
{
	EnumRemoveFlags(LabelFlags, InFlags);
}

bool UDisplayClusterLabelComponent::HasAnyLabelFlags(EDisplayClusterLabelFlags InFlags) const
{
	return EnumHasAnyFlags(GetLabelFlags(), InFlags);
}

void UDisplayClusterLabelComponent::TickComponent(float DeltaTime, ELevelTick TickType,
                                                  FActorComponentTickFunction* ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	CheckForVisibilityChange();
	
	if (IsVisible())
	{
		UpdateWidgetComponent();
	}
}

void UDisplayClusterLabelComponent::OnRegister()
{
	Super::OnRegister();

	CheckForVisibilityChange();
	
	WidgetComponent->SetupAttachment(this);
	WidgetComponent->SetWidgetClass(WidgetClass.LoadSynchronous());
}

UWidgetComponent* UDisplayClusterLabelComponent::GetWidgetComponent() const
{
	return WidgetComponent.Get();
}

#if WITH_EDITOR

void UDisplayClusterLabelComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UDisplayClusterLabelComponent, WidgetClass))
	{
		WidgetComponent->SetWidgetClass(WidgetClass.Get());
	}
}

#endif

void UDisplayClusterLabelComponent::UpdateWidgetComponent()
{
	FString LabelString;
	if (const ADisplayClusterLightCardActor* LightCardActor = Cast<ADisplayClusterLightCardActor>(GetOwner()))
	{
#if WITH_EDITOR
		LabelString = LightCardActor->GetActorLabel();
#else
		LabelString = LightCardActor->GetName();
#endif
		
		if (!RootActor.IsValid())
		{
			// The light card editor normally will have set this, but if we're running as -game then it will
			// be set from the DCRA instead
			SetRootActor(LightCardActor->GetRootActorOwner());
		}

#if WITH_EDITOR
		const bool bIsGame = !GIsEditor;
#else
		const bool bIsGame = true;
#endif
		
		if (bIsGame)
		{
			const USceneComponent* Viewpoint = RootActor.IsValid() ? RootActor->GetCommonViewPoint() : nullptr;
			if (Viewpoint != nullptr)
			{
				const FTransform LightCardTransform = LightCardActor->GetStageActorTransform();
				const FVector LightCardLocation = LightCardTransform.GetLocation();
				const FVector DestinationPoint = Viewpoint->GetComponentLocation();

				// Keep the text facing up.
				FRotator Rotation = UKismetMathLibrary::FindLookAtRotation(LightCardLocation, DestinationPoint);
				Rotation.Roll = 0.f;
			
				WidgetComponent->SetWorldRotation(Rotation);

				// Keep the label the same size regardless of distance to the view point.
				const float Distance = (DestinationPoint - LightCardLocation).Length();

				const float BaseLabelScale = 0.0025f;
				const FVector VectorScale(Distance * BaseLabelScale * WidgetScale);
			
				WidgetComponent->SetWorldScale3D(VectorScale);
			}
		}
	}
	else
	{
		LabelString = TEXT("Label");
	}

	if (WidgetComponent->GetWidgetClass() != WidgetClass.Get() ||
		(WidgetClass.IsValid() && WidgetComponent->GetWidget() == nullptr))
	{
		WidgetComponent->SetWidgetClass(WidgetClass.Get());
		WidgetComponent->InitWidget();
	}

	if (UDisplayClusterLabelWidget* Widget = Cast<UDisplayClusterLabelWidget>(WidgetComponent->GetWidget()))
	{
		Widget->SetLabelText(FText::FromString(LabelString));
	}

#if WITH_EDITOR
	WidgetComponent->SetHiddenInSceneCapture(true);
#endif
}

void UDisplayClusterLabelComponent::CheckForVisibilityChange()
{
	// MU doesn't seem to register nested sub component visibility changes, and OnVisibilityChanged won't
	// fire on the MU client so we have to manually check here.

#if WITH_EDITOR
	const bool bIsGame = !GIsEditor;
#else
	const bool bIsGame = true;
#endif
	
	bool bDisplayWidget = GetVisibleFlag();
	
	if (bIsGame && !HasAnyLabelFlags(EDisplayClusterLabelFlags::DisplayInGame))
	{
		bDisplayWidget = false;
	}
	else if (!bIsGame && !HasAnyLabelFlags(EDisplayClusterLabelFlags::DisplayInEditor))
	{
		bDisplayWidget = false;
	}
	
	if (bDisplayWidget != WidgetComponent->GetVisibleFlag())
	{
		WidgetComponent->SetVisibility(bDisplayWidget);
	}
}