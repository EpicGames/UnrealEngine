// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Blueprint/UserWidget.h"

#include "DisplayClusterLabelWidget.generated.h"

/**
 * A base widget to display on actor labels.
 */
UCLASS(Abstract)
class DISPLAYCLUSTER_API UDisplayClusterLabelWidget : public UUserWidget
{
public:
	GENERATED_BODY()

	UFUNCTION(BlueprintImplementableEvent, Category=Label)
	void SetLabelText(const FText& InText);
	
};
