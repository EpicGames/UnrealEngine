// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterLightCardActor.h"

#include "Blueprints/DisplayClusterBlueprintLib.h"
#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterRootActor.h"
#include "Features/IModularFeatures.h"
#include "Misc/DisplayClusterLog.h"
#include "DisplayClusterWidgetComponent.h"

#include "IDisplayClusterLightCardActorExtender.h"

#include "Components/DisplayClusterLabelComponent.h"
#include "Components/DisplayClusterLightCardStageActorComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Features/IModularFeatures.h"
#include "GameFramework/SpringArmComponent.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Engine/StaticMesh.h"
#include "UObject/ConstructorHelpers.h"

#if WITH_EDITOR
#include "Framework/Application/SlateApplication.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#endif

#if WITH_OPENCV

#include "OpenCVHelper.h"

#include "PreOpenCVHeaders.h"
#include "opencv2/imgproc.hpp"
#include "PostOpenCVHeaders.h"

#endif //WITH_OPENCV

int32 GDisplayClusterLightCardPolygonTextureSize = 128;
static FAutoConsoleVariableRef CVarDisplayClusterLightCardPolygonTextureSize(
	TEXT("DC.LightCardPolygonTextureSize"),
	GDisplayClusterLightCardPolygonTextureSize,
	TEXT("Size of the textures generated by light cards that use a polygon to define its alpha mask. Use a power of 2."),
	ECVF_Default
);

const float ADisplayClusterLightCardActor::UVPlaneDefaultSize = 200.0f;
const float ADisplayClusterLightCardActor::UVPlaneDefaultDistance = 100.0f;
const FName ADisplayClusterLightCardActor::LightCardStageActorComponentName = TEXT("LightCardStageActorComponent");

ADisplayClusterLightCardActor::ADisplayClusterLightCardActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, DistanceFromCenter(300.f)
	, Longitude(0.f)
	, Latitude(30.f)
	, Spin(0.f)
	, Pitch(0.f)
	, Yaw(0.f)
	, Scale(FVector2D(1.f))
	, RadialOffset(-1)
	, bAlwaysFlushToWall(true)
	, PerLightcardRenderMode(EDisplayClusterConfigurationICVFX_PerLightcardRenderMode::Default)
	, Mask(EDisplayClusterLightCardMask::Circle)
	, Texture(nullptr)
	, Color(FLinearColor(1.f, 1.f, 1.f, 1.f))
	, Temperature(6500)
	, Tint(0.f)
	, Exposure(0.f)
	, Gain(1.f)
	, Opacity(1.f)
	, Feathering(0.f)
{
	PrimaryActorTick.bCanEverTick = true;

	DefaultSceneRootComponent = CreateDefaultSubobject<USceneComponent>(TEXT("DefaultSceneRoot"));
	SetRootComponent(DefaultSceneRootComponent);

	MainSpringArmComponent = CreateDefaultSubobject<USpringArmComponent>(TEXT("MainSpringArm"));
	MainSpringArmComponent->AttachToComponent(DefaultSceneRootComponent, FAttachmentTransformRules::KeepRelativeTransform);
	MainSpringArmComponent->bDoCollisionTest = false;

	LightCardTransformerComponent = CreateDefaultSubobject<USceneComponent>(TEXT("LightCardTransformer"));
	LightCardTransformerComponent->AttachToComponent(MainSpringArmComponent, FAttachmentTransformRules::KeepRelativeTransform);

	{
		static ConstructorHelpers::FObjectFinder<UStaticMesh> PlaneObj(TEXT("/nDisplay/LightCard/SM_LightCardPlane"));

		LightCardComponent = CreateDefaultSubobject<UStaticMeshComponent>(TEXT("LightCard"));
		LightCardComponent->AttachToComponent(LightCardTransformerComponent, FAttachmentTransformRules::KeepRelativeTransform);
		LightCardComponent->SetCollisionProfileName(TEXT("OverlapAllDynamic"));
		LightCardComponent->Mobility = EComponentMobility::Movable;
		LightCardComponent->SetStaticMesh(PlaneObj.Object);

		// Overlap events are expensive and seem to not be needed at the moment, so we disable them.
		LightCardComponent->SetGenerateOverlapEvents(false);
	}

#if WITH_EDITOR
	{
		static ConstructorHelpers::FObjectFinder<UStaticMesh> UVIndicatorObj(TEXT("/nDisplay/LightCard/SM_UVIndicator"));

		UVIndicatorComponent = CreateDefaultSubobject<UStaticMeshComponent>(TEXT("UVIndicator"));
		UVIndicatorComponent->AttachToComponent(LightCardComponent, FAttachmentTransformRules::KeepRelativeTransform);
		UVIndicatorComponent->SetCollisionEnabled(ECollisionEnabled::NoCollision);
		UVIndicatorComponent->Mobility = EComponentMobility::Movable;
		UVIndicatorComponent->SetStaticMesh(UVIndicatorObj.Object);

		UVIndicatorComponent->SetRelativeLocation(FVector(0, 0, 1)); // Slightly in front of the parent
		UVIndicatorComponent->SetVisibility(false);
	}

	if (!IsTemplate() && GEngine)
	{
		GEngine->OnLevelActorDeleted().AddUObject(this, &ADisplayClusterLightCardActor::OnLevelActorDeleted);
	}
#endif // WITH_EDITOR

	StageActorComponent = CreateOptionalDefaultSubobject<UDisplayClusterLightCardStageActorComponent>(LightCardStageActorComponentName);
	
	UpdateStageActorTransform();

	LabelComponent = CreateOptionalDefaultSubobject<UDisplayClusterLabelComponent>(TEXT("Label"), true);
	LabelComponent->AttachToComponent(LightCardComponent, FAttachmentTransformRules::KeepRelativeTransform);

	CreateComponentsForExtenders();
}

ADisplayClusterLightCardActor::~ADisplayClusterLightCardActor()
{
#if WITH_EDITOR
	if (GEngine)
	{
		GEngine->OnLevelActorDeleted().RemoveAll(this);
	}
#endif
}

void ADisplayClusterLightCardActor::PostLoad()
{
	Super::PostLoad();

	CleanUpComponentsForExtenders();
}

void ADisplayClusterLightCardActor::OnConstruction(const FTransform& Transform)
{
	Super::OnConstruction(Transform);

	UpdatePolygonTexture();

	if (UMaterialInterface* Material = LightCardComponent->GetMaterial(0))
	{
		if (UMaterialInstanceDynamic* MaterialInstanceDynamic = Cast<UMaterialInstanceDynamic>(Material))
		{
			// Clear param values and always call UpdateLightCardMaterialInstance(). Otherwise a snapshots restore
			// may result in a desync between actor settings and MID parameter values.

			// When level snapshots restores an actor the CS can run multiple times (spawn, posteditchangeproperty, etc).
			// Our mesh component is a default sub object, so the "new" copy of it exists at this point and we set the MID.
			// But after the actor is finished constructing, snapshots deserializes components reattaching them to the actor.
			// What looks like is happening is the MID instance is somehow preserved, but parameter values are lost/outdated.
			// This causes a problem in that we already set the parameter value so the value on the render thread is correct,
			// but the value we have on the game thread is not. IE, on construction "UseMask" is true initially and the
			// MID is updated correctly, but after snapshots loads our properties "UseMask" might be "false". Then,
			// when snapshots deserializes the components the "true" value on the game thread incorrectly becomes "false".
			// When we attempt to update the MID on tick to the correct value of "false" it thinks it already has
			// the "false" value so it doesn't update the parameter render thread which is currently "true".
			
			// As long as we always call UpdateLightCardMaterialInstance() we can avoid the problem, but clearing the
			// parameters out is an extra precaution to ensure we have the correct values.
			
			MaterialInstanceDynamic->ClearParameterValues();
		}
		else
		{
			// Initial MID creation
			UMaterialInstanceDynamic* LightCardMatInstance = UMaterialInstanceDynamic::Create(Material, LightCardComponent, TEXT("LightCardMID"));
			LightCardComponent->SetMaterial(0, LightCardMatInstance);
		}

		UpdateLightCardMaterialInstance();
	}
}

void ADisplayClusterLightCardActor::Tick(float DeltaSeconds)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(DCLightcard_Tick);

	Super::Tick(DeltaSeconds);

	if (!MainSpringArmComponent || !LightCardTransformerComponent || !LightCardComponent)
	{
		// @todo Using WP with MU can trigger this in -game if an actor was added prior to converting to an MU session, and that actor was deleted.
		// It looks like the actor isn't being destroyed properly on the clients.
		UE_LOGF(LogDisplayClusterGame, Error, "Light Card Actor is missing components.");
		return;
	}
	
	ClampLatitudeAndLongitude(Latitude, Longitude);

	UpdateLightCardPositionToRootActor();
	UpdateStageActorTransform();
	UpdateLightCardMaterialInstance();
	UpdateLightCardVisibility();
	UpdateUVIndicator();
	if (LabelComponent->IsVisible())
	{
		LabelComponent->GetWidgetComponent()->SetTranslucentSortPriority(LightCardComponent->TranslucencySortPriority + 1);
	}
}

void ADisplayClusterLightCardActor::Destroyed()
{
	RemoveFromRootActor();
}

#if WITH_EDITOR

void ADisplayClusterLightCardActor::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property && (
		PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ADisplayClusterLightCardActor, DistanceFromCenter) ||
		PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ADisplayClusterLightCardActor, Longitude) ||
		PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ADisplayClusterLightCardActor, Latitude) ||
		PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ADisplayClusterLightCardActor, Spin) ||
		PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ADisplayClusterLightCardActor, Pitch) ||
		PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ADisplayClusterLightCardActor, Yaw) ||
		PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ADisplayClusterLightCardActor, RadialOffset) ||
		PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ADisplayClusterLightCardActor, Scale) ||
		PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ADisplayClusterLightCardActor, bIsUVLightCard) ||
		PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ADisplayClusterLightCardActor, bAlwaysFlushToWall)))
	{
		UpdateStageActorTransform();

		if (bAlwaysFlushToWall)
		{
			MakeFlushToWall();
		}
	}

	if (PropertyChangedEvent.Property && (
		PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ADisplayClusterLightCardActor, Polygon) ||
		PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ADisplayClusterLightCardActor, Feathering)))
	{
		UpdatePolygonTexture();
	}

	if (PropertyChangedEvent.Property && (
		   PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ADisplayClusterLightCardActor, Mask)
		|| PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ADisplayClusterLightCardActor, Polygon) 
		|| PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ADisplayClusterLightCardActor, Texture) 
		|| PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ADisplayClusterLightCardActor, Color)
		|| PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ADisplayClusterLightCardActor, Exposure)
		|| PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ADisplayClusterLightCardActor, Gain)
		|| PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ADisplayClusterLightCardActor, Opacity)
		|| PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ADisplayClusterLightCardActor, Feathering)
		|| PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ADisplayClusterLightCardActor, AlphaGradient)
		|| PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ADisplayClusterLightCardActor, Tint)
		|| PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ADisplayClusterLightCardActor, Temperature)
		))
	{
		UpdateLightCardMaterialInstance();
	}

	if (PropertyChangedEvent.Property && (
		PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ADisplayClusterLightCardActor, bIsUVLightCard)))
	{
		UpdateLightCardVisibility();
	}
}

FName ADisplayClusterLightCardActor::GetCustomIconName() const
{
	if (bIsUVLightCard)
	{
		return TEXT("ClassIcon.DisplayClusterLightCardActor.UVLightCard");
	}

	if (bIsLightCardFlag)
	{
		return TEXT("ClassIcon.DisplayClusterLightCardActor.Flag");
	}
	
	return Super::GetCustomIconName();
}

#endif

void ADisplayClusterLightCardActor::GetLightCardMeshComponents(TArray<UMeshComponent*>& MeshComponents) const
{
	MeshComponents.Add(LightCardComponent.Get());

#if WITH_EDITOR
	if (GIsEditor && bIsUVLightCard)
	{
		MeshComponents.Add(UVIndicatorComponent.Get());
	}
#endif // WITH_EDITOR
}

UStaticMesh* ADisplayClusterLightCardActor::GetStaticMesh() const
{
	return LightCardComponent->GetStaticMesh();
}

void ADisplayClusterLightCardActor::SetStaticMesh(UStaticMesh* InStaticMesh)
{
	LightCardComponent->SetStaticMesh(InStaticMesh);
}

void ADisplayClusterLightCardActor::UpdateLightCardMaterialInstance()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(DCLightcard_UpdateLightCardMaterialInstance);

	if (UMaterialInstanceDynamic* LightCardMaterialInstance = Cast<UMaterialInstanceDynamic>(LightCardComponent->GetMaterial(0)))
	{
		// Showing proxy with low opacity to make it less distracting when it doesn't line up well with its projection in the Light Card Editor.
		constexpr float ProxyOpacity = 0;

		LightCardMaterialInstance->SetVectorParameterValue(TEXT("CardColor"), Color);
		LightCardMaterialInstance->SetScalarParameterValue(TEXT("Temperature"), Temperature);
		LightCardMaterialInstance->SetScalarParameterValue(TEXT("Tint"), Tint);
		LightCardMaterialInstance->SetScalarParameterValue(TEXT("Exposure"), Exposure);
		LightCardMaterialInstance->SetScalarParameterValue(TEXT("Gain"), Gain);
		LightCardMaterialInstance->SetScalarParameterValue(TEXT("Opacity"), IsProxy() ? ProxyOpacity : Opacity);

		// Here we assign the textures. These properties can become null but since SetTextureParameterValue(nullptr) is a no-op,
		// when the textures are null we instead assign the parent material default texture.

		if (Texture)
		{
			LightCardMaterialInstance->SetTextureParameterValue(TEXT("Texture"), Texture);
		}
		else
		{
			UTexture* DefaultTexture = nullptr;

			if (ensure(LightCardMaterialInstance->Parent))
			{
				LightCardMaterialInstance->Parent->GetTextureParameterValue(TEXT("Texture"), DefaultTexture);
				LightCardMaterialInstance->SetTextureParameterValue(TEXT("Texture"), DefaultTexture);
			}
		}

		if (PolygonMask)
		{
			LightCardMaterialInstance->SetTextureParameterValue(TEXT("AlphaTexture"), PolygonMask);
		}
		else
		{
			UTexture* DefaultTexture = nullptr;

			if (ensure(LightCardMaterialInstance->Parent))
			{
				LightCardMaterialInstance->Parent->GetTextureParameterValue(TEXT("AlphaTexture"), DefaultTexture);
				LightCardMaterialInstance->SetTextureParameterValue(TEXT("AlphaTexture"), DefaultTexture);
			}
		}

		LightCardMaterialInstance->SetScalarParameterValue(TEXT("AlphaGradient"), AlphaGradient.bEnableAlphaGradient);
		LightCardMaterialInstance->SetScalarParameterValue(TEXT("AlphaGradientStartingAlpha"), AlphaGradient.StartingAlpha);
		LightCardMaterialInstance->SetScalarParameterValue(TEXT("AlphaGradientEndingAlpha"), AlphaGradient.EndingAlpha);
		LightCardMaterialInstance->SetScalarParameterValue(TEXT("AlphaGradientAngle"), AlphaGradient.Angle);

		bool bUseMask = true; // Enable masking
		bool bUseTextureAlpha = false; // Use the alpha channel of Texture as mask
		bool bUseAlphaTexture = false; // Use the AlphaTexture
		float FeatheringValue = Feathering;

		switch (Mask)
		{
		case EDisplayClusterLightCardMask::Square:
			bUseMask = false;
			break;

		case EDisplayClusterLightCardMask::UseTextureAlpha:
			bUseTextureAlpha = true;
			break;

		case EDisplayClusterLightCardMask::Polygon:
			bUseTextureAlpha = true;
			bUseAlphaTexture = true;
			FeatheringValue = 0.;
			break;
		}

		LightCardMaterialInstance->SetScalarParameterValue(TEXT("Feather"), FeatheringValue);
		LightCardMaterialInstance->SetScalarParameterValue(TEXT("UseMask"), bUseMask);
		LightCardMaterialInstance->SetScalarParameterValue(TEXT("UseTextureAlpha"), bUseTextureAlpha);
		LightCardMaterialInstance->SetScalarParameterValue(TEXT("UseAlphaTexture"), bUseAlphaTexture);
	}
}

void ADisplayClusterLightCardActor::UpdatePolygonTexture()
{
#if WITH_OPENCV

	if (Polygon.Num() < 3)
	{
		PolygonMask = nullptr;
		return;
	}

	if (GDisplayClusterLightCardPolygonTextureSize < 2)
	{
		return;
	}

	const int32 TextureWidth = GDisplayClusterLightCardPolygonTextureSize;
	const int32 TextureHeight = GDisplayClusterLightCardPolygonTextureSize;

	cv::Mat GrayMat(cv::Size(TextureHeight, TextureWidth), CV_8UC1, cv::Scalar(0));

	std::vector<cv::Point2i> PolyVec; // polygon with LC shape that to be filled with cv::fillPoly
	PolyVec.reserve(Polygon.Num());

	for (const FVector2d& PolyPoint : Polygon)
	{
		PolyVec.push_back(cv::Point2i(
			FMath::RoundToInt(TextureWidth  * PolyPoint.X), 
			FMath::RoundToInt(TextureHeight * PolyPoint.Y))
		);
	}

	// Fill the polygon pixels
	cv::fillPoly(GrayMat, PolyVec, 255, cv::LINE_AA); // Polygon inside is 255, outside is 0

	// Apply feathering effect
	if (Feathering > 0)
	{
		// Overscan it so that the effects near the edges are as desired
		const int32 Overscan = (TextureWidth + TextureHeight) / 16;
		const cv::Size OverscanMatSize(GrayMat.rows + 2 * Overscan, GrayMat.cols + 2 * Overscan);
		cv::Mat OverscanMat(OverscanMatSize, GrayMat.type(), cv::Scalar(0));

		const cv::Rect OverscanActiveRect(Overscan, Overscan, GrayMat.cols, GrayMat.rows);

		GrayMat.copyTo(OverscanMat(OverscanActiveRect));

		// Erode to compensate for the growth that the blur causes
		{
			const int32 KernelSize = int32((Feathering * TextureWidth) / 6) * 2 + 1;

			const cv::Mat ErodeKernel = cv::getStructuringElement(
				cv::MORPH_ERODE,
				cv::Size(KernelSize, KernelSize)
			);

			cv::erode(OverscanMat, OverscanMat, ErodeKernel);
		}

		// Blur
		{
			const int32 KernelSize = int32((Feathering * TextureWidth) / 2) * 2 + 1;

			// We are going to blur the outside
			OverscanMat = 255 - OverscanMat;

			const float SigmaX = 0.2 * ((KernelSize - 1) * 0.5 - 1) + 0.8;
			const float SigmaY = SigmaX;

			cv::GaussianBlur(OverscanMat, OverscanMat, cv::Size(KernelSize, KernelSize), SigmaX, SigmaY);

			// We recover the non-inverted alpha now
			OverscanMat = 255 - OverscanMat;
		}

		// Put back to Gray
		OverscanMat(OverscanActiveRect).copyTo(GrayMat);
	}

	// Create a texture, all white and and the shape goes in the alpha
	cv::Mat RGBAMat(cv::Size(TextureWidth, TextureHeight), CV_8UC4, cv::Scalar::all(255));
	cv::insertChannel(GrayMat, RGBAMat, 3);

	// Create UTexture
	PolygonMask = FOpenCVHelper::TextureFromCvMat(RGBAMat);

#endif // WITH_OPENCV
}

void ADisplayClusterLightCardActor::UpdateLightCardVisibility()
{
	if (bIsUVLightCard)
	{
		// Only render the UV light card if this is a proxy actor
		const bool bShouldBeVisible = IsProxy();

		if (LightCardComponent->IsVisible() != bShouldBeVisible)
		{
			LightCardComponent->SetVisibility(bShouldBeVisible);
		}
	}
}


void ADisplayClusterLightCardActor::UpdateUVIndicator()
{
#if WITH_EDITOR
	if (GIsEditor && bIsUVLightCard && IsProxy())
	{
		if (!UVIndicatorComponent->IsVisible())
		{
			UVIndicatorComponent->SetVisibility(true);
		}
	}
	else if (UVIndicatorComponent->IsVisible())
	{
		UVIndicatorComponent->SetVisibility(false);
	}

	if (GIsEditor && bIsUVLightCard)
	{
		// Always keep indicator on top of the light card
		UVIndicatorComponent->TranslucencySortPriority = LightCardComponent->TranslucencySortPriority;
		UVIndicatorComponent->TranslucencySortDistanceOffset = LightCardComponent->TranslucencySortDistanceOffset - 0.1f;
		
		// Keep 1:1 aspect ratio
		if (const USceneComponent* UVParentComponent = UVIndicatorComponent->GetAttachParent())
		{
			const FVector ParentWorldScale = UVParentComponent->GetComponentScale();

			const double ParentScaleAbsX = FMath::Abs(ParentWorldScale.X);
			const double ParentScaleAbsY = FMath::Abs(ParentWorldScale.Y);

			const double ParentScaleMin = FMath::Min(ParentScaleAbsX, ParentScaleAbsY);

			if (ParentScaleMin > KINDA_SMALL_NUMBER)
			{
				constexpr double UVScale = 0.25;
				FVector ScaleFactor(1, 1, 1);

				if (ParentScaleAbsX > ParentScaleAbsY)
				{
					ScaleFactor.X = ParentScaleAbsY / ParentScaleAbsX;
				}
				else
				{
					ScaleFactor.Y = ParentScaleAbsX / ParentScaleAbsY;
				}

				// Don't let the uv indicator be overly big if the light card itself is big.
				constexpr double MaxAllowedParentScale = 0.5;
				const double ClampScaleFactor = FMath::Clamp(MaxAllowedParentScale / ParentScaleMin, 0, 1);

				UVIndicatorComponent->SetRelativeScale3D(UVScale * ClampScaleFactor * ScaleFactor);
			}
		}
	}
#endif // WITH_EDITOR
}

void ADisplayClusterLightCardActor::UpdateLightCardPositionToRootActor()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(DCLightcard_UpdateLightCardPositionToRootActor);

	if (!bLockToOwningRootActor || HasAnyFlags(RF_Transient) /* Proxies update from world instance positions instead */)
	{
		return;
	}
	
	if (const ADisplayClusterRootActor* RootActor = GetRootActorOwner())
	{
		if (const UDisplayClusterConfigurationData* ConfigData = RootActor->GetConfigData())
		{
			if (ConfigData->StageSettings.Lightcard.bEnable)
			{
				// Find a origin to use as the transform "anchor" of each light card. At the moment, assume the first view point component
				// found is the correct view point (the same assumption is made in the light card editor). If no view point is found, use the root component
				const USceneComponent* ViewPointComponent = RootActor->GetRootComponent();

				TArray<UDisplayClusterCameraComponent*> ViewPointComponents;
				RootActor->GetComponents(ViewPointComponents);

				if (ViewPointComponents.Num())
				{
					ViewPointComponent = ViewPointComponents[0];
				}
		
				const FVector Location = ViewPointComponent ? ViewPointComponent->GetComponentLocation() : RootActor->GetActorLocation();
				const FVector RelativeLocation = RootActor->GetTransform().InverseTransformPosition(Location);
				const FRotator Rotation = RootActor->GetActorRotation();
				
				const bool bRepositionLightCards = RelativeLocation != LastOrbitLocation && !IsUVActor();

				const FTransform OldLightCardTransform = GetStageActorTransform();
				
				SetActorLocation(Location);
				SetActorRotation(Rotation);

				if (bRepositionLightCards)
				{
					const FTransform StageActorTransform = GetOrigin();
					FDisplayClusterPositionalParams PositionalParams = GetPositionalParams();
					// We want to adjust the positional params so that the light card stays in the same world position.
					// First, compute the desired relative location based on the new orbit origin and the light card's previous world position
					// Then convert that to longitude and latitude values
					const FVector OrbitLocation = Rotation.UnrotateVector(OldLightCardTransform.GetLocation() - Location);
					PositionalParams.DistanceFromCenter = OrbitLocation.Length();
					if (PositionalParams.DistanceFromCenter > UE_SMALL_NUMBER)
					{
						PositionalParams.Latitude = FMath::RadiansToDegrees(FMath::Asin(OrbitLocation.Z / PositionalParams.DistanceFromCenter));
						PositionalParams.Longitude = FMath::RadiansToDegrees(FMath::Atan2(OrbitLocation.Y, OrbitLocation.X)) + 180.0;
					}
					else
					{
						PositionalParams.Latitude = 0.0;
						PositionalParams.Longitude = 180.0;
					}
					// The light card's orientation needs to be adjusted as well, since the orientation parameters are defined in radial space,
					// which is relative to the orbit origin
					const FVector WorldNormal = OldLightCardTransform.GetRotation().RotateVector(FVector::ForwardVector);
					const FVector LocalNormal = FRotationMatrix::MakeFromX(OrbitLocation.GetSafeNormal()).InverseTransformVector(StageActorTransform.InverseTransformVectorNoScale(WorldNormal));
					const FRotator NormalRotation = FRotationMatrix::MakeFromX(-LocalNormal).Rotator();
					PositionalParams.Pitch = NormalRotation.Pitch;
					PositionalParams.Yaw = NormalRotation.Yaw;
					// The spin also needs to be adjusted. This can be done by transforming the old spin vector from world space into the new normal space,
					// and then converting to an angle
					const FVector WorldSpin = OldLightCardTransform.GetRotation().RotateVector(FVector::UpVector);
					const FVector LocalSpin = FRotationMatrix::MakeFromX(OrbitLocation.GetSafeNormal()).InverseTransformVector(StageActorTransform.InverseTransformVectorNoScale(WorldSpin));
					const FVector NormalSpin = FRotationMatrix::MakeFromX(LocalNormal).InverseTransformVector(LocalSpin);
					PositionalParams.Spin = FMath::RadiansToDegrees(FMath::Atan2(NormalSpin.Y, NormalSpin.Z));
					SetPositionalParams(PositionalParams);

					LastOrbitLocation = RelativeLocation;
				}
			}
		}
	}
}

void ADisplayClusterLightCardActor::MakeFlushToWall()
{
	if (ADisplayClusterRootActor* RootActor = GetRootActorOwner())
	{
		RootActor->MakeStageActorFlushToWall(this);
	}
}

void ADisplayClusterLightCardActor::SetIsLightCardFlag(bool bNewFlagValue)
{
	bIsLightCardFlag = bNewFlagValue;
}

void ADisplayClusterLightCardActor::SetIsUVActor(bool bNewUVValue)
{
	bIsUVLightCard = bNewUVValue;
}

void ADisplayClusterLightCardActor::ShowLightCardLabel(const FDisplayClusterLabelConfiguration& InLabelConfiguration)
{
	if (IsUVActor())
	{
		// UV actors don't currently support labels
		return;
	}

#if WITH_EDITOR
	LabelComponent->Modify(false);
	LightCardComponent->Modify(false);
#endif
	LabelComponent->SetLabelConfiguration(InLabelConfiguration);
}

void ADisplayClusterLightCardActor::ShowLightCardLabel(bool bValue, float ScaleValue, ADisplayClusterRootActor* InRootActor)
{
	FDisplayClusterLabelConfiguration LabelConfiguration;
	{
		LabelConfiguration.bVisible = bValue;
		LabelConfiguration.Scale = ScaleValue;
		LabelConfiguration.RootActor = InRootActor;
		LabelConfiguration.LabelFlags = LabelComponent->GetLabelFlags();
	}
	ShowLightCardLabel(MoveTemp(LabelConfiguration));
}

void ADisplayClusterLightCardActor::SetWeakRootActorOwner(ADisplayClusterRootActor* InRootActor)
{
	// Pre 5.2 light cards won't have a StageActorComponent->RootActor, but still need to know the root actor
	// owner for certain operations. We shouldn't automatically set the RootActorOwner, because it will cause competing
	// ownership of the light card. Legacy LCs have a ShowOnlyList on the root actor to determine ownership, whereas
	// new LCs determine the root actor they belong to.
	const bool bCanSetWeakActorPtr = !StageActorComponent || !StageActorComponent->GetRootActor().IsValid()
		|| InRootActor == StageActorComponent->GetRootActor().Get() || InRootActor == nullptr;
	if (!bCanSetWeakActorPtr && !HasAnyFlags(RF_Transient))
	{
#if WITH_EDITOR
		check(InRootActor && StageActorComponent && StageActorComponent->GetRootActor().IsValid());
		const FText CurrentClassName = GetClass()->GetDisplayNameText();
		const FText CurrentActorName = FText::FromString(GetActorLabel());
		const FText CurrentRootActorName = FText::FromString(StageActorComponent->GetRootActor()->GetActorLabel());
		const FText NewRootActorName = FText::FromString(InRootActor->GetActorLabel());
		const FText ErrorMessage = FText::Format(NSLOCTEXT("DisplayClusterLightCard", "InvalidRootActorOwner",
"The {0} '{1}' belongs to '{2}' but appears in the content for '{3}'. Remove it from the content and only use the {0}'s nDisplay Root Actor property."),
					CurrentClassName, CurrentActorName, CurrentRootActorName, NewRootActorName);
		
		// Only warn once so as not to spam the message log since this function is called per tick.
		if (!bHadRootActorMismatch && GIsEditor && FSlateApplication::IsInitialized())
		{
			FNotificationInfo Info(ErrorMessage);
			Info.Image = FAppStyle::GetBrush(TEXT("Icons.Error"));
			Info.bFireAndForget = true;
			Info.bUseSuccessFailIcons = false;
			Info.ExpireDuration = 10.0f;

			FSlateNotificationManager::Get().AddNotification(Info);
			UE_LOGF(LogDisplayClusterGame, Error, "%ls", *ErrorMessage.ToString());

			bHadRootActorMismatch = true;
		}
		else
		{
			UE_LOGF(LogDisplayClusterGame, Verbose, "%ls", *ErrorMessage.ToString());
		}
#endif
	}
	else
	{
		WeakRootActorOwner = InRootActor;
	}
}

void ADisplayClusterLightCardActor::SetRootActorOwner(ADisplayClusterRootActor* InRootActor)
{
	if (StageActorComponent)
	{
		StageActorComponent->SetRootActor(InRootActor);
	}
}

ADisplayClusterRootActor* ADisplayClusterLightCardActor::GetRootActorOwner() const
{
	return StageActorComponent && StageActorComponent->GetRootActor().IsValid() ? StageActorComponent->GetRootActor().Get() : WeakRootActorOwner.Get();
}

UDisplayClusterStageActorComponent* ADisplayClusterLightCardActor::GetStageActorComponent() const
{
	return StageActorComponent;
}

void ADisplayClusterLightCardActor::AddToLightCardLayer(ADisplayClusterRootActor* InRootActor)
{
	AddToRootActor(InRootActor);
}

void ADisplayClusterLightCardActor::AddToRootActor(ADisplayClusterRootActor* InRootActor)
{
	check(InRootActor);

#if WITH_EDITOR
	Modify();
#endif
	
	// Adjust transparent sort order for UV light cards
	if (IsUVActor())
	{
		TSet<ADisplayClusterLightCardActor*> LightCards;
		UDisplayClusterBlueprintLib::FindLightCardsForRootActor(InRootActor, LightCards);
		
		int32 HighestPriority = MIN_int32;
		bool bExistingPrioritiesFound = false;

		for (const ADisplayClusterLightCardActor* LightCard : LightCards)
		{
			if (LightCard->IsUVActor())
			{
				if (LightCard->LightCardComponent->TranslucencySortPriority > HighestPriority)
				{
					HighestPriority = LightCard->LightCardComponent->TranslucencySortPriority;
				}
				bExistingPrioritiesFound = true;
			}
		}

		if (bExistingPrioritiesFound)
		{
			// Newly added cards should be on the top-most visibility layer
			LightCardComponent->TranslucencySortPriority = HighestPriority < MAX_int32 ? HighestPriority + 1 : HighestPriority;
		}
	}

#if WITH_EDITOR
	StageActorComponent->Modify();
#endif
	StageActorComponent->SetRootActor(InRootActor);
}

void ADisplayClusterLightCardActor::RemoveFromRootActor()
{
#if WITH_EDITOR
	Modify();
#endif


	if (ADisplayClusterRootActor* RootActorOwner = GetRootActorOwner())
	{
		// Remove this from the list of LCs for the owning root actor
		RemoveFromRootActorList(RootActorOwner);
	}
	else
	{
		// This may be a legacy LC, so we need to check if any root actors in the scene own it and remove it from their lists if so
		if (const UWorld* World = GetWorld())
		{
			for (TActorIterator<ADisplayClusterRootActor> Iter(World); Iter; ++Iter)
			{
				RemoveFromRootActorList(*Iter);
			}
		}
	}

	if (StageActorComponent)
	{
		StageActorComponent->SetRootActor(nullptr);
	}

	WeakRootActorOwner.Reset();
}

void ADisplayClusterLightCardActor::UpdateStageActorTransform()
{
	// If the light card is in UV space, set the spring arm's transform to be zero, effectively removing it from the transform hierarchy
	// This allows the light card to be positioned with the actor's cartesian coordinates instead of longitude and latitude
	if (bIsUVLightCard)
	{
		MainSpringArmComponent->TargetArmLength = 0.0f;
		MainSpringArmComponent->SetRelativeRotation(FRotator(0.0, 180.0, 0.0));

		// Set world location and rotation such that the light card is always projected onto a YZ plane a distance of UVPlaneDefaultDistance from the world origin, facing in the -X direction
		// This ensures that when the UV light cards are rendered to the light card map, they are always positioned and oriented correctly regardless of the stages location and rotation.
		
		// We place them slightly closer than UVPlaneDefaultDistance so that mouse clicks hit it first instead of DCRA meshes (avoids re-linetracing).
		// UV LCs operate in orthographic projection so this should have no visual effect.
		constexpr float DistanceFactor = 0.99f;

		LightCardTransformerComponent->SetWorldLocation(FVector(DistanceFactor * UVPlaneDefaultDistance, -UVPlaneDefaultSize * (0.5 - UVCoordinates.X), UVPlaneDefaultSize * (0.5 - UVCoordinates.Y)));
		LightCardTransformerComponent->SetWorldRotation(FVector(-1, 0, 0).Rotation());
		LightCardTransformerComponent->SetWorldScale3D(FVector::OneVector);
	}
	else
	{
		MainSpringArmComponent->TargetArmLength = DistanceFromCenter + RadialOffset;
		MainSpringArmComponent->SetRelativeRotation(FRotator(-Latitude, Longitude, 0.0));
	}

	FRotator LightCardOrientation = FRotator(-Pitch, Yaw, Spin);

	LightCardComponent->SetRelativeRotation((LightCardOrientation.Quaternion() * PlaneMeshRotation.Quaternion()).Rotator());
	LightCardComponent->SetRelativeScale3D(FVector(Scale, 1.f));
}

FTransform ADisplayClusterLightCardActor::GetStageActorTransform(bool bRemoveOrigin) const
{
	FTransform Transform;

	const FVector ForwardVector = (GetActorRotation().Quaternion() * FRotator(-Latitude, Longitude, 0.0).Quaternion()).RotateVector(-FVector::ForwardVector);
	constexpr float DistanceFactor = 0.99f;
	FVector Position = bIsUVLightCard ?
		FVector(DistanceFactor * UVPlaneDefaultDistance, -UVPlaneDefaultSize * (0.5 - UVCoordinates.X), UVPlaneDefaultSize * (0.5 - UVCoordinates.Y)) :
		GetActorLocation() + DistanceFromCenter * ForwardVector;

	if (bRemoveOrigin)
	{
		Position -= GetActorLocation();
	}
	Transform.SetLocation(MoveTemp(Position));

	// Use the light card component's orientation, but remove the plane mesh rotation so that the returned transform's local x axis
	// points radially inwards to match engine convention
	const FQuat LightCardOrientation = LightCardComponent->GetComponentQuat() * PlaneMeshRotation.Quaternion().Inverse();

	Transform.SetRotation(LightCardOrientation);

	return Transform;
}

FBox ADisplayClusterLightCardActor::GetBoxBounds(bool bLocalSpace) const
{
	FTransform ObjectOrientedTransform;
	ObjectOrientedTransform.SetRotation(PlaneMeshRotation.Quaternion());

	if (!bLocalSpace)
	{
		ObjectOrientedTransform.SetTranslation(LightCardComponent->GetComponentLocation());
		ObjectOrientedTransform.SetScale3D(LightCardComponent->GetComponentScale());
	}

	return LightCardComponent->CalcBounds(ObjectOrientedTransform).GetBox();
}

void ADisplayClusterLightCardActor::SetLongitude(double InValue)
{
	Longitude = InValue;
}

double ADisplayClusterLightCardActor::GetLongitude() const
{
	return Longitude;
}

void ADisplayClusterLightCardActor::SetLatitude(double InValue)
{
	Latitude = InValue;
}

double ADisplayClusterLightCardActor::GetLatitude() const
{
	return Latitude;
}

void ADisplayClusterLightCardActor::SetDistanceFromCenter(double InValue)
{
	DistanceFromCenter = InValue;
}

double ADisplayClusterLightCardActor::GetDistanceFromCenter() const
{
	return DistanceFromCenter;
}

void ADisplayClusterLightCardActor::SetSpin(double InValue)
{
	Spin = InValue;
}

void ADisplayClusterLightCardActor::SetPitch(double InValue)
{
	Pitch = InValue;
}

double ADisplayClusterLightCardActor::GetSpin() const
{
	return Spin;
}

double ADisplayClusterLightCardActor::GetPitch() const
{
	return Pitch;
}

void ADisplayClusterLightCardActor::SetYaw(double InValue)
{
	Yaw = InValue;
}

double ADisplayClusterLightCardActor::GetYaw() const
{
	return Yaw;
}

void ADisplayClusterLightCardActor::SetRadialOffset(double InValue)
{
	RadialOffset = InValue;
}

double ADisplayClusterLightCardActor::GetRadialOffset() const
{
	return RadialOffset;
}

bool ADisplayClusterLightCardActor::IsUVActor() const
{
	return bIsUVLightCard;
}

void ADisplayClusterLightCardActor::SetOrigin(const FTransform& InOrigin)
{
	SetActorLocation(InOrigin.GetLocation());
	SetActorRotation(InOrigin.GetRotation());
	// Scale not currently used for origin
}

FTransform ADisplayClusterLightCardActor::GetOrigin() const
{
	return {GetActorRotation(), GetActorLocation(), FVector::One()};
}

void ADisplayClusterLightCardActor::SetScale(const FVector2D& InScale)
{
	Scale = InScale;
}

FVector2D ADisplayClusterLightCardActor::GetScale() const
{
	return Scale;
}

void ADisplayClusterLightCardActor::SetUVCoordinates(const FVector2D& InUVCoordinates)
{
	UVCoordinates = InUVCoordinates;
}

FVector2D ADisplayClusterLightCardActor::GetUVCoordinates() const
{
	return UVCoordinates;
}

void ADisplayClusterLightCardActor::GetPositionalProperties(FPositionalPropertyArray& OutPropertyPairs) const
{
	void* Container = (void*)(this);

	const TSet<FName>& PropertyNames = GetPositionalPropertyNames();
	OutPropertyPairs.Reserve(PropertyNames.Num());

	for (const FName& PropertyName : PropertyNames)
	{
		if (FProperty* Property = FindFProperty<FProperty>(GetClass(), PropertyName))
		{
			OutPropertyPairs.Emplace(Container, Property);
		}
	}
}

void ADisplayClusterLightCardActor::CreateComponentsForExtenders()
{
	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	const TArray<IDisplayClusterLightCardActorExtender*> Extenders = ModularFeatures.GetModularFeatureImplementations<IDisplayClusterLightCardActorExtender>(IDisplayClusterLightCardActorExtender::ModularFeatureName);
	for (IDisplayClusterLightCardActorExtender* Extender : Extenders)
	{
		UClass* ExtenderComponentClass = Extender->GetAdditionalSubobjectClass().Get();
		const FName ExtenderName = Extender->GetExtenderName();
		if (!ensureAlwaysMsgf(ExtenderComponentClass, TEXT("Cannot apply display cluster Extender. Invalid component class.")))
		{
			continue;
		}
		else if (!ensureAlwaysMsgf(!ExtenderName.IsNone(), TEXT("Cannot apply display cluster Extender. Extender name cannot be empty.")))
		{
			continue;
		}
		else if (!ensureAlwaysMsgf(!ExtenderNameToComponentMap.Contains(ExtenderName), TEXT("Cannot apply display cluster Extender. Extender name already in use.")))
		{
			continue;
		}

		const FName ObjectName = MakeUniqueObjectName(this, ExtenderComponentClass, ExtenderName);
		constexpr bool bIsRequired = false;
		constexpr bool bIsTransient = false;
		UObject* Component = CreateDefaultSubobject(ObjectName, UActorComponent::StaticClass(), ExtenderComponentClass, bIsRequired, bIsTransient);
		ExtenderNameToComponentMap.Add(ExtenderName, CastChecked<UActorComponent>(Component));
	}
}

void ADisplayClusterLightCardActor::CleanUpComponentsForExtenders()
{
	TArray<FName> AvailableExtenders;
	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	const TArray<IDisplayClusterLightCardActorExtender*> Extenders = ModularFeatures.GetModularFeatureImplementations<IDisplayClusterLightCardActorExtender>(IDisplayClusterLightCardActorExtender::ModularFeatureName);
	for (IDisplayClusterLightCardActorExtender* Extender : Extenders)
	{
		AvailableExtenders.Add(Extender->GetExtenderName());
	}
	const TMap<FName, TObjectPtr<UActorComponent>> CachedExtenderNameToComponentMap(ExtenderNameToComponentMap);
	for (const TTuple<FName, TObjectPtr<UActorComponent>>& ExtenderNameToComponentPair : CachedExtenderNameToComponentMap)
	{
		if (!AvailableExtenders.Contains(ExtenderNameToComponentPair.Key))
		{
			ExtenderNameToComponentMap.Remove(ExtenderNameToComponentPair.Key);
		}
	}
}

void ADisplayClusterLightCardActor::RemoveFromRootActorList(ADisplayClusterRootActor* RootActor)
{
	if (!RootActor)
	{
		return;
	}

	if (UDisplayClusterConfigurationData* CurrentData = RootActor->GetConfigData())
	{
		for (auto Iter = CurrentData->StageSettings.Lightcard.ShowOnlyList.Actors.CreateIterator(); Iter; ++Iter)
		{
			if (Iter->Get() == this)
			{
				Iter.RemoveCurrent();
			}
		}
	}
}

#if WITH_EDITOR
void ADisplayClusterLightCardActor::OnLevelActorDeleted(AActor* DeletedActor)
{
	ADisplayClusterRootActor* RootActorOwner = GetRootActorOwner();
	if (DeletedActor && RootActorOwner && StageActorComponent && Cast<ADisplayClusterRootActor>(DeletedActor) == RootActorOwner)
	{
		RemoveFromRootActor();
	}
}
#endif
