// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DisplayClusterOriginComponent.h"
#include "Components/StaticMeshComponent.h"

#include "Engine/CollisionProfile.h"
#include "Engine/StaticMesh.h"
#include "UObject/ConstructorHelpers.h"


UDisplayClusterOriginComponent::UDisplayClusterOriginComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
#if WITH_EDITOR
	, bEnableGizmo(true)
	, BaseGizmoScale(FVector(0.1f, 0.1f, 0.1f))
	, GizmoScaleMultiplier(1.f)
#endif
{
#if WITH_EDITOR
	if (GIsEditor && !IsRunningCommandlet())
	{
		static ConstructorHelpers::FObjectFinder<UStaticMesh> ProxyMeshRef(TEXT("/nDisplay/Meshes/SM_nDisplayOrigin"));
		ProxyMesh = ProxyMeshRef.Object;
	}
#endif
}

#if WITH_EDITOR
void UDisplayClusterOriginComponent::SetVisualizationScale(float Scale)
{
	GizmoScaleMultiplier = Scale;
	RefreshVisualRepresentation();
}

void UDisplayClusterOriginComponent::SetVisualizationEnabled(bool bEnabled)
{
	bEnableGizmo = bEnabled;
	RefreshVisualRepresentation();
}
#endif

void UDisplayClusterOriginComponent::OnRegister()
{
#if WITH_EDITOR
	if (GIsEditor && !IsRunningCommandlet())
	{
		if (ProxyMeshComponent == nullptr)
		{
			ProxyMeshComponent = NewObject<UStaticMeshComponent>(this, NAME_None, RF_Transactional | RF_TextExportTransient);
			ProxyMeshComponent->SetupAttachment(this);
			ProxyMeshComponent->SetRelativeLocationAndRotation(FVector::ZeroVector, FRotator::ZeroRotator);
			ProxyMeshComponent->SetMobility(EComponentMobility::Movable);
			ProxyMeshComponent->SetIsVisualizationComponent(true);
			ProxyMeshComponent->SetStaticMesh(ProxyMesh);
			ProxyMeshComponent->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
			ProxyMeshComponent->bHiddenInGame = true;
			ProxyMeshComponent->CastShadow = false;
			ProxyMeshComponent->CreationMethod = CreationMethod;
			ProxyMeshComponent->RegisterComponentWithWorld(GetWorld());
		}
	}

	RefreshVisualRepresentation();
#endif

	Super::OnRegister();
}

#if WITH_EDITOR
void UDisplayClusterOriginComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	RefreshVisualRepresentation();
}

void UDisplayClusterOriginComponent::RefreshVisualRepresentation()
{
	// Update the proxy mesh if necessary
	if (ProxyMeshComponent && ProxyMeshComponent->GetStaticMesh() != ProxyMesh)
	{
		ProxyMeshComponent->SetStaticMesh(ProxyMesh);

		bEnableGizmo = true;
		BaseGizmoScale = FVector::OneVector;
		GizmoScaleMultiplier = 1.f;
	}

	if (ProxyMeshComponent)
	{
		ProxyMeshComponent->SetVisibility(bEnableGizmo);
		ProxyMeshComponent->SetWorldScale3D(BaseGizmoScale * GizmoScaleMultiplier);
	}
}
#endif
