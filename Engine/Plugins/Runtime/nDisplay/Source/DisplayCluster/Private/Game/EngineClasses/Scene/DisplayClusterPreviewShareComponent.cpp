// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DisplayClusterPreviewShareComponent.h"
#include "Render/Viewport/DisplayClusterViewportManager.h"
#include "Render/Viewport/IDisplayClusterViewportManagerPreview.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_PreviewSettings.h"

#include "CineCameraComponent.h"
#include "Components/DisplayClusterICVFXCameraComponent.h"
#include "DisplayClusterRootActor.h"
#include "Engine/TextureRenderTarget2D.h"
#include "IDisplayCluster.h"
#include "Game/IDisplayClusterGameManager.h"
#include "MediaCapture.h"
#include "Misc/DisplayClusterLog.h"
#include "SharedMemoryMediaOutput.h"
#include "SharedMemoryMediaSource.h"
#include "UObject/Package.h"

#if WITH_EDITOR
#include "LevelEditor.h"
#endif

namespace UE::DisplayCluster::PreviewShareComponent
{
	/** Convenience struct when iterating over config viewports */
	struct FViewportData
	{
		// Id of the node for this viewport
		FString NodeId;

		// Id of the viewport itself
		FString ViewportId;

		// Config viewport itself
		UDisplayClusterConfigurationViewport* Viewport;

		FViewportData(const FString& NodeId, const FString& ViewportId, UDisplayClusterConfigurationViewport* Viewport)
			: NodeId(NodeId)
			, ViewportId(ViewportId)
			, Viewport(Viewport)
		{}
	};

	/** Convenience function to get the data needed when iterating over config viewports of a given nDisplay actor */
	static inline TArray<FViewportData> GetConfigViewportsFromRootActor(const ADisplayClusterRootActor* RootActor)
	{
		TArray<FViewportData> ViewportDatas;

		if (!RootActor)
		{
			return ViewportDatas;
		}

		UDisplayClusterConfigurationData* ConfigData = RootActor->GetConfigData();

		if (!ConfigData || !ConfigData->Cluster)
		{
			return ViewportDatas;
		}

		for (const auto& NodeElem : ConfigData->Cluster->Nodes)
		{
			if (!NodeElem.Value)
			{
				continue;
			}

			for (const auto& ViewportElem : NodeElem.Value->Viewports)
			{
				TObjectPtr<UDisplayClusterConfigurationViewport> Viewport = ViewportElem.Value;

				if (!Viewport)
				{
					continue;
				}

				const FString& NodeId = NodeElem.Key;
				const FString& ViewportId = ViewportElem.Key;

				ViewportDatas.Emplace(NodeId, ViewportId, Viewport);
			}
		}

		return ViewportDatas;
	}

	/** Retrieves the viewport configuration from the DCRA's current config using the given NodeId and ViewportId */
	static inline UDisplayClusterConfigurationViewport* GetViewportFromDCRA(ADisplayClusterRootActor* RootActor, const FString& NodeId, const FString& ViewportId)
	{
		if (!RootActor)
		{
			return nullptr;
		}

		UDisplayClusterConfigurationData* ConfigData = RootActor->GetConfigData();

		if (!ConfigData)
		{
			return nullptr;
		}

		const TObjectPtr<UDisplayClusterConfigurationClusterNode>* NodePtr = ConfigData->Cluster->Nodes.Find(NodeId);

		if (!NodePtr)
		{
			return nullptr;
		}

		const UDisplayClusterConfigurationClusterNode* Node = *NodePtr;

		if (!Node)
		{
			return nullptr;
		}

		const TObjectPtr<UDisplayClusterConfigurationViewport>* ViewportPtr = Node->Viewports.Find(ViewportId);

		if (!ViewportPtr)
		{
			return nullptr;
		}

		return *ViewportPtr;
	}

	static inline void RemoveExternalMapReferences(ADisplayClusterRootActor* RootActor)
	{
		// In all cases, we should make sure we never restore in illegal external reference,
		// because if we do, it will be very hard for the user to fix this and save the map, since these
		// references are not exposed in the details panel of the instance.

		if (!RootActor)
		{
			return;
		}

		const UPackage* RootActorPackage = RootActor->GetPackage();

		if (!RootActorPackage)
		{
			return;
		}

		for (const FViewportData& ViewportData : GetConfigViewportsFromRootActor(RootActor))
		{
			if (ViewportData.Viewport->RenderSettings.Replace.SourceTexture)
			{
				const UPackage* TexturePackage = ViewportData.Viewport->RenderSettings.Replace.SourceTexture->GetPackage();

				if (TexturePackage && TexturePackage->ContainsMap() && (TexturePackage != RootActorPackage))
				{
					UE_LOGF(LogDisplayClusterGame, Warning,
						"Removing unexpected external reference to texture package '%ls', a map different to the owning RootActor package '%ls'",
						*TexturePackage->GetName(),
						*RootActorPackage->GetName()
					);

					ViewportData.Viewport->RenderSettings.Replace.SourceTexture = nullptr;
				}
			}
		}
	}
};

UDisplayClusterPreviewShareComponent::UDisplayClusterPreviewShareComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.bStartWithTickEnabled = false;
	SetTickEnable(false);
}

bool UDisplayClusterPreviewShareComponent::AllowedToShare() const
{
	// This component should be inactive if it is a CDO.
	if (HasAnyFlags(RF_ArchetypeObject | RF_ClassDefaultObject))
	{
		return false;
	}

	// This component should be inactive if in a PreviewWorld

	UWorld* World = GetWorld();

	if (!World)
	{
		return false;
	}

	if (World->IsPreviewWorld())
	{
		return false;
	}

	// We don't allow sharing if the parent is the active root actor
	{
		IDisplayCluster& Display = IDisplayCluster::Get();
		IDisplayClusterGameManager* GameMgr = Display.GetGameMgr();

		if (GameMgr && GameMgr->GetRootActor() == GetOwner())
		{
			return false;
		}
	}

	return true;
}

void UDisplayClusterPreviewShareComponent::ModeChanged()
{
	// We ignore the desired Mode if it is not allowed.
	if (!AllowedToShare())
	{
		Mode = EDisplayClusterPreviewShareMode::None;
	}

	// Close all media before restarting the sharing
	CloseAllMedia();

	// Restore default preview settings
	RestoreDefaultPreviewSettings();

	// Restore original root actor settings (e.g. texture replace settings that we overwrote)
	RestoreRootActorOriginalSettings();

	switch (Mode)
	{
	case EDisplayClusterPreviewShareMode::PullActor:
	case EDisplayClusterPreviewShareMode::Send:
	case EDisplayClusterPreviewShareMode::Receive:
		SetTickEnable(true);
		break;

	default:
		// Default to no tick
		SetTickEnable(false);
		break;
	}
}

void UDisplayClusterPreviewShareComponent::SetTickEnable(const bool bEnable)
{
	bTickInEditor = bEnable;
	PrimaryComponentTick.SetTickFunctionEnable(bEnable);
}

void UDisplayClusterPreviewShareComponent::RestoreRootActorOriginalSettings()
{
	using namespace UE::DisplayCluster::PreviewShareComponent;

	ADisplayClusterRootActor* RootActor = Cast<ADisplayClusterRootActor>(GetOwner());

	if (!RootActor)
	{
		return;
	}

	if (OriginalSourceTextures.IsEmpty() && OriginalTextureReplaces.IsEmpty())
	{
		return;
	}

	// Iterate over viewports and restore their settings
	for (const FViewportData& ViewportData : GetConfigViewportsFromRootActor(RootActor))
	{
		check(ViewportData.Viewport); // The iterator implementation guarantees that it is not nullptr

		const FString ViewportKey = GenerateViewportKey(ViewportData.NodeId, ViewportData.ViewportId);

		if (bool* BoolPtr = OriginalTextureReplaces.Find(ViewportKey))
		{
			ViewportData.Viewport->RenderSettings.Replace.bAllowReplace = *BoolPtr;
		}

		if (TObjectPtr<UTexture>* OriginalTexturePtr = OriginalSourceTextures.Find(ViewportKey))
		{
			// It should not be possible to clear OriginalSourceTextures without restoring them first.
			// Or assign textures without saving the original references.
			// 
			// * If we did, then we'd leave assigned the pulled ones (since there would be nothing to restore), 
			//   potentially with external map references, which would not let the user save the map.
			// 
			// * In addition, it should not be possible to think that the currently assigned replace textures are
			//   the original ones when they really aren't. Otherwise we'd be eventually be restoring textures
			//   that belong to an external map.

			ViewportData.Viewport->RenderSettings.Replace.SourceTexture = *OriginalTexturePtr;
		}
	}

	OriginalSourceTextures.Empty();
	OriginalTextureReplaces.Empty();
}

void UDisplayClusterPreviewShareComponent::CloseAllMedia()
{
	// Stop all media
	for (const TPair<FString, TObjectPtr<UMediaCapture>>& Pair : MediaCaptures)
	{
		UMediaCapture* MediaCapture = Pair.Value;

		if (!IsValid(MediaCapture))
		{
			continue;
		}

		MediaCapture->StopCapture(false);
	}

	for (const TPair<FString, TObjectPtr<UMediaPlayer>>& Pair : MediaPlayers)
	{
		UMediaPlayer* MediaPlayer = Pair.Value;

		if (!IsValid(MediaPlayer))
		{
			continue;
		}

		MediaPlayer->Close();
	}

	// Clear media

	MediaCaptures.Empty();
	MediaOutputs.Empty();

	MediaSources.Empty();
	MediaPlayers.Empty();
	MediaTextures.Empty();
}

void UDisplayClusterPreviewShareComponent::RestoreDefaultPreviewSettings()
{
	// Release custom viewport manager (used by 'Send')
	CustomViewportManager.Reset();

	// Restore default preview settings on owner RootActor (Used by 'PullActor')
	if (ADisplayClusterRootActor* RootActor = Cast<ADisplayClusterRootActor>(GetOwner()))
	{
		if (IDisplayClusterViewportManager* ViewportManager = RootActor->GetViewportManager())
		{
			// Use preview settings from RootActor
			RootActor->PreviewSetttingsSource = EDisplayClusterConfigurationRootActorPreviewSettingsSource::RootActor;

			// reconfigure to rendering from owner root actor
			ViewportManager->GetConfiguration().SetRootActor(EDisplayClusterRootActorType::Scene | EDisplayClusterRootActorType::Configuration, RootActor);
		}
	}
}

bool UDisplayClusterPreviewShareComponent::UpdateCustomViewportManager(const ADisplayClusterRootActor* InSrcRootActor)
{
	check(IsInGameThread());

	if (!InSrcRootActor)
	{
		// The source RootActor is required
		return false;
	}

	if (!CustomViewportManager.IsValid())
	{
		CustomViewportManager = IDisplayClusterViewportManager::CreateViewportManager();
	}

	IDisplayClusterViewportConfiguration& CustomConfiguration = CustomViewportManager->GetConfiguration();

	// Assign DCRA to render, excluding PreviewRootActor since we are not using the preview mesh, only the viewport textures.
	CustomConfiguration.SetRootActor(EDisplayClusterRootActorType::Scene | EDisplayClusterRootActorType::Configuration, InSrcRootActor);

	// Get preview settings from InSrcRootActor properties
	FDisplayClusterViewport_PreviewSettings NewPreviewSettings = InSrcRootActor->GetPreviewSettings(true);

	// Override preview settings:
	NewPreviewSettings.bPreviewEnable = true;
	NewPreviewSettings.bEnablePreviewMesh = false;
	NewPreviewSettings.bEnablePreviewEditableMesh = false;
	NewPreviewSettings.bPreviewICVFXFrustums = false; // disable frustum rendering
	CustomConfiguration.SetPreviewSettings(NewPreviewSettings);

	// Request rendering of the entire cluster for our custom viewport manager
	CustomViewportManager->GetViewportManagerPreview().UpdateEntireClusterPreviewRender(true);

	return true;
}

FString UDisplayClusterPreviewShareComponent::GenerateViewportKey(const FString& NodeName, const FString& ViewportName) const
{
	return FString::Printf(TEXT("%s_%s"),
		*NodeName,
		*ViewportName
	);
}

FString UDisplayClusterPreviewShareComponent::GenerateMediaUniqueName(const FString& ActorName, const FString& UniqueViewportName) const
{
	return FString::Printf(TEXT("%s_%s"),
		UniqueName.IsEmpty() ? *ActorName : *UniqueName,
		*UniqueViewportName
	);
}

void UDisplayClusterPreviewShareComponent::TickSend()
{
	using namespace UE::DisplayCluster::PreviewShareComponent;

	// Rendering RootActor on CustomViewportManager with custom settings and sending results
	ADisplayClusterRootActor* RootActor = Cast<ADisplayClusterRootActor>(GetOwner());
	if(!UpdateCustomViewportManager(RootActor))
	{
		CloseAllMedia();
		RestoreDefaultPreviewSettings();

		return;
	}

	// Make sure we're up to date with the viewports
	// We will use this array to close and remove unused media captures.
	TSet<FString> LeftoverViewportKeys;
	MediaOutputs.GetKeys(LeftoverViewportKeys);

	for (const TSharedPtr<IDisplayClusterViewportPreview, ESPMode::ThreadSafe>& ViewportPreviewIt : CustomViewportManager->GetViewportManagerPreview().GetEntireClusterPreviewViewports())
	{
		if (!ViewportPreviewIt.IsValid())
		{
			continue;
		}

		// Get the texture to share
		UTextureRenderTarget2D* PreviewTexture = ViewportPreviewIt->GetPreviewTextureRenderTarget2D();
		if (!PreviewTexture)
		{
			continue;
		}

		// Each viewport gets a unique name

		const FString ViewportKey = GenerateViewportKey(
			ViewportPreviewIt->GetClusterNodeId(),
			ViewportPreviewIt->GetId()
		);

		LeftoverViewportKeys.Remove(ViewportKey);

		// Make sure the media output exists for that ViewportKey

		USharedMemoryMediaOutput* MediaOutput = nullptr;

		if (TObjectPtr<UMediaOutput>* MediaOutputPtr = MediaOutputs.Find(ViewportKey))
		{
			MediaOutput = Cast<USharedMemoryMediaOutput>(*MediaOutputPtr);
		}

		// If the media output for the given unique name does not exist, create it.
		if (!MediaOutput)
		{
			// Instantiate the media output and give it the corresponding unique name of the viewport.
			MediaOutput = NewObject<USharedMemoryMediaOutput>(GetTransientPackage(), USharedMemoryMediaOutput::StaticClass());

			MediaOutput->UniqueName = GenerateMediaUniqueName(
				RootActor->GetActorNameOrLabel(),
				ViewportKey
			);

			// Create the associated media capture
			UMediaCapture* MediaCapture = MediaOutput->CreateMediaCapture();
			check(MediaCapture);

			MediaCapture->SetMediaOutput(MediaOutput);

			// Start the media capture right away

			// Prepare the media capture options.
			FMediaCaptureOptions MediaCaptureOptions;
			MediaCaptureOptions.NumberOfFramesToCapture = -1;
			MediaCaptureOptions.bAutoRestartOnSourceSizeChange = true;
			MediaCaptureOptions.bSkipFrameWhenRunningExpensiveTasks = false;
			MediaCaptureOptions.OverrunAction = EMediaCaptureOverrunAction::Skip;

			const bool bCaptureStarted = MediaCapture->CaptureTextureRenderTarget2D(PreviewTexture, MediaCaptureOptions);

			if (bCaptureStarted)
			{
				UE_LOGF(LogDisplayClusterGame, Log, "Started media capture for viewport '%ls'", *MediaOutput->UniqueName);
			}
			else
			{
				UE_LOGF(LogDisplayClusterGame, Warning, "Couldn't start media capture for viewport '%ls'", *MediaOutput->UniqueName);
			}

			MediaOutputs.Add(ViewportKey, MediaOutput);
			MediaCaptures.Add(ViewportKey, MediaCapture);
		}

		// @todo: Update the render target if it has changed.
	}

	// Stop and remove unused media captures
	for (const FString& ViewportKey : LeftoverViewportKeys)
	{
		MediaOutputs.Remove(ViewportKey);
		UMediaCapture* MediaCapture = MediaCaptures.FindAndRemoveChecked(ViewportKey);
		check(MediaCapture);
		MediaCapture->StopCapture(false /* bAllowPendingFrameToBeProcess */);
	}

	// Restart stopped Captures (they may have stopped if e.g. the texture resolution changed)
	{
		TArray<FString> StalledCaptureViewportKeys;

		for (const TPair<FString, TObjectPtr<UMediaCapture>>& Pair : MediaCaptures)
		{
			UMediaCapture* MediaCapture = Pair.Value;

			if (!IsValid(MediaCapture))
			{
				StalledCaptureViewportKeys.Add(Pair.Key);
				continue;
			}

			if (   MediaCapture->GetState() == EMediaCaptureState::Stopped 
				|| MediaCapture->GetState() == EMediaCaptureState::Error)
			{
				MediaCapture->StopCapture(false /* bAllowPendingFrameToBeProcess */);
				StalledCaptureViewportKeys.Add(Pair.Key);
				continue;
			}
		}

		for (const FString& ViewportKey : StalledCaptureViewportKeys)
		{
			MediaCaptures.Remove(ViewportKey);
			MediaOutputs.Remove(ViewportKey);
		}
	}

}

void UDisplayClusterPreviewShareComponent::TickPullActor(const bool bUseSourceActorSettings)
{
	using namespace UE::DisplayCluster::PreviewShareComponent;

	ADisplayClusterRootActor* SourceRootActor = SourceNDisplayActor.Get();
	ADisplayClusterRootActor* RootActor = Cast<ADisplayClusterRootActor>(GetOwner());
	IDisplayClusterViewportManager* ViewportManager = RootActor ? RootActor->GetViewportManager() : nullptr;
	if (!IsValid(SourceRootActor) || !IsValid(RootActor) || RootActor == SourceRootActor || !ViewportManager)
	{
		// Disallow self referencing
		RestoreDefaultPreviewSettings();

		return;
	}

	// Pull preview textures
	{
		// Reconfigure to rendering from external DCRA
		ViewportManager->GetConfiguration().SetRootActor(EDisplayClusterRootActorType::Scene | EDisplayClusterRootActorType::Configuration, SourceRootActor);

		// Gets preview settings from various RootActors
		ADisplayClusterRootActor* PreviewSettingsRootActor = bUseSourceActorSettings ? SourceRootActor : RootActor;

		// Override preview settings:
		FDisplayClusterViewport_PreviewSettings NewPreviewSettings = PreviewSettingsRootActor->GetPreviewSettings(true);

		// When using the preview settings from the local RootActor, we also use the DisplayDevice from the same.
		NewPreviewSettings.DisplayDeviceRootActorType = bUseSourceActorSettings ? EDisplayClusterRootActorType::Configuration : EDisplayClusterRootActorType::Preview;

		ViewportManager->GetConfiguration().SetPreviewSettings(NewPreviewSettings);

		// Sets configuration mode to use overridden values instead of values from RootActor.
		RootActor->PreviewSetttingsSource = EDisplayClusterConfigurationRootActorPreviewSettingsSource::Configuration;
	}

	// Note: Did we really need to synchronize the ICVFX camera function for PullActor mode?
	// 
	// Sync Icvfx cameras
	switch (IcvfxCamerasSyncType)
	{
	case EDisplayClusterPreviewShareIcvfxSync::PullActor:
		SyncIcvxCamerasFromSourceActor(SourceRootActor, RootActor);
		break;

	case EDisplayClusterPreviewShareIcvfxSync::PushActor:
		SyncIcvxCamerasFromSourceActor(RootActor, SourceRootActor);
		break;

	default:
		break;
	}
}

void UDisplayClusterPreviewShareComponent::SyncIcvxCamerasFromSourceActor(const ADisplayClusterRootActor* SourceRootActor, const ADisplayClusterRootActor* DstRootActor)
{
	using namespace UE::DisplayCluster::PreviewShareComponent;

	if (!SourceRootActor || !DstRootActor)
	{
		return;
	}

	// Disallow self referencing

	if (DstRootActor == SourceRootActor)
	{
		return;
	}

	// Gather source icvfx cameras

	TMap<FString, UDisplayClusterICVFXCameraComponent*> SrcCameraComponentWithId;

	for (UActorComponent* SrcComponent : SourceRootActor->K2_GetComponentsByClass(UDisplayClusterICVFXCameraComponent::StaticClass()))
	{
		UDisplayClusterICVFXCameraComponent* SrcCameraComponent = Cast<UDisplayClusterICVFXCameraComponent>(SrcComponent);

		if (!SrcCameraComponent)
		{
			continue;
		}

		SrcCameraComponentWithId.Add(SrcCameraComponent->GetCameraUniqueId(), SrcCameraComponent);
	}

	// Iterate over destination cameras and sync the desired settings

	for (UActorComponent* DstComponent : DstRootActor->K2_GetComponentsByClass(UDisplayClusterICVFXCameraComponent::StaticClass()))
	{
		UDisplayClusterICVFXCameraComponent* DstIcvfxCameraComponent = Cast<UDisplayClusterICVFXCameraComponent>(DstComponent);

		if (!DstIcvfxCameraComponent)
		{
			continue;
		}

		UDisplayClusterICVFXCameraComponent** SrcIcvfxCameraComponentPtr = SrcCameraComponentWithId.Find(DstIcvfxCameraComponent->GetCameraUniqueId());

		if (!SrcIcvfxCameraComponentPtr || !*SrcIcvfxCameraComponentPtr)
		{
			continue;
		}

		UDisplayClusterICVFXCameraComponent* SrcIcvfxCameraComponent = *SrcIcvfxCameraComponentPtr;
		check(SrcIcvfxCameraComponent);

		// This one sould be the actual source camera, e.g. the camera component of the referenced cine camera.
		const UCineCameraComponent* SrcCameraComponent = SrcIcvfxCameraComponent->GetActualCineCameraComponent();
		check(SrcCameraComponent);
		
		// Get the source camera transform with respect to the Source root actor
		const FTransform SrcCameraToSrcRootActorTransform = SrcCameraComponent->GetComponentTransform().GetRelativeTransform(SourceRootActor->GetTransform());

		// Apply this transform to the destination camera

		UCineCameraComponent* DstCameraComponent = DstIcvfxCameraComponent->GetActualCineCameraComponent();

		AActor* DstCameraOwnerActor = DstCameraComponent->GetOwner();
		
		if (!DstCameraOwnerActor)
		{
			continue;
		}

		const FTransform DstCameraComponentWorldTransform = SrcCameraToSrcRootActorTransform * DstRootActor->GetTransform();

		// If the dst owning actor is a cine camera, then we move the cine camera itself instead of the component
		if (DstCameraOwnerActor->IsA(ACineCameraActor::StaticClass()))
		{
			// DCr * Aw = DCw
			// Aw = DCr_i * DCw

			const FTransform DstCameraToOwingActorTransform = DstCameraComponent->GetComponentTransform().GetRelativeTransform(DstCameraOwnerActor->GetTransform());

			DstCameraOwnerActor->SetActorTransform(DstCameraToOwingActorTransform.Inverse() * DstCameraComponentWorldTransform);
		}
		else
		{
			// Otherwise we set the world transform of the component directly
			DstCameraComponent->SetWorldTransform(DstCameraComponentWorldTransform);
		}

		// Copy relevant lens settings. 
		{
			// Currently we only support this when source and destination are cine camera components.

			const UCineCameraComponent* SrcCineCameraComponent = Cast<UCineCameraComponent>(SrcCameraComponent);
			UCineCameraComponent* DstCineCameraComponent = Cast<UCineCameraComponent>(DstCameraComponent);

			if (!SrcCineCameraComponent || !DstCineCameraComponent)
			{
				continue;
			}

			DstCineCameraComponent->Filmback = SrcCineCameraComponent->Filmback;
			DstCineCameraComponent->SetCurrentFocalLength(SrcCineCameraComponent->CurrentFocalLength);
		}
	}
}

void UDisplayClusterPreviewShareComponent::TickReceive()
{
	using namespace UE::DisplayCluster::PreviewShareComponent;

	ADisplayClusterRootActor* RootActor = Cast<ADisplayClusterRootActor>(GetOwner());
	IDisplayClusterViewportManager* ViewportManager = RootActor ? RootActor->GetViewportManager() : nullptr;
	if (!ViewportManager)
	{
		CloseAllMedia();
		return;
	}

	// Make sure we're up to date with the viewports
	// We will use this array to close and remove unused media sources.
	TSet<FString> LeftoverViewportKeys;
	MediaSources.GetKeys(LeftoverViewportKeys);

	for (TSharedPtr<IDisplayClusterViewport, ESPMode::ThreadSafe>& ViewportIt : ViewportManager->GetEntireClusterViewports())
	{
		if (!ViewportIt.IsValid())
		{
			continue;
		}

		// We will ultimately update the viewport so let's make sure it exists.

		UDisplayClusterConfigurationViewport* Viewport = GetViewportFromDCRA(RootActor, ViewportIt->GetClusterNodeId(), ViewportIt->GetId());

		if (!Viewport)
		{
			continue;
		}

		// Each viewport gets associated with a uniquely named shared texture.

		const FString ViewportKey = GenerateViewportKey(
			ViewportIt->GetClusterNodeId(),
			ViewportIt->GetId()
		);

		LeftoverViewportKeys.Remove(ViewportKey);

		// Make sure the media source exists for that ViewportKey

		USharedMemoryMediaSource* MediaSource = nullptr;

		if (TObjectPtr<UMediaSource>* MediaOutputPtr = MediaSources.Find(ViewportKey))
		{
			MediaSource = Cast<USharedMemoryMediaSource>(*MediaOutputPtr);
		}

		// If the media output for the given unique name does not exist, create it.
		if (!MediaSource)
		{
			// Instantiate the media source and give it the corresponding unique name of the viewport.
			MediaSource = NewObject<USharedMemoryMediaSource>(GetTransientPackage(), USharedMemoryMediaSource::StaticClass());

			MediaSource->UniqueName = GenerateMediaUniqueName(
				RootActor->GetActorNameOrLabel(),
				ViewportKey
			);

			// Note: Choosing Freerun for now but consider genlock to force the engines to run at the same rate, 
			// which would make them immune to Windows GPU throttling of out of focus appliations.
			MediaSource->Mode = ESharedMemoryMediaSourceMode::Freerun;

			// Create the associated media player and texture

			UMediaPlayer* MediaPlayer = NewObject<UMediaPlayer>();
			if (!MediaPlayer)
			{
				UE_LOGF(LogDisplayClusterGame, Error, "Failed to create MediaPlayer");
				continue;
			}

			MediaPlayer->SetLooping(false);
			MediaPlayer->PlayOnOpen = false;

			UMediaTexture* MediaTexture = NewObject<UMediaTexture>();
			if (!MediaTexture)
			{
				UE_LOGF(LogDisplayClusterGame, Error, "Failed to create MediaTexture");
				continue;
			}

			MediaTexture->NewStyleOutput = true;
			MediaTexture->SetRenderMode(UMediaTexture::ERenderMode::Default); //@todo convert to just in time
			MediaTexture->SetMediaPlayer(MediaPlayer);
			MediaTexture->UpdateResource();

			// Remember original replacement textures

			if (!OriginalTextureReplaces.Find(ViewportKey))
			{
				OriginalTextureReplaces.Add(ViewportKey, Viewport->RenderSettings.Replace.bAllowReplace);
			}

			if (!OriginalSourceTextures.Find(ViewportKey))
			{
				OriginalSourceTextures.Add(ViewportKey, Viewport->RenderSettings.Replace.SourceTexture);
			}

			// Start the player right away
			MediaPlayer->PlayOnOpen = true;
			MediaPlayer->OpenSource(MediaSource);

			// Add the new media objects to our map so that they don't get garbage collected.

			MediaSources.Add(ViewportKey, MediaSource);
			MediaPlayers.Add(ViewportKey, MediaPlayer);
			MediaTextures.Add(ViewportKey, MediaTexture);
		}

		// Refresh the texture replace settings since reconstruction may overwrite it.
		if (TObjectPtr<UMediaTexture>* MediaTexturePtr = MediaTextures.Find(ViewportKey))
		{
			Viewport->RenderSettings.Replace.bAllowReplace = true;
			Viewport->RenderSettings.Replace.SourceTexture = *MediaTexturePtr;
		}
	}

	// Stop and remove unused media sources
	for (const FString& ViewportKey : LeftoverViewportKeys)
	{
		MediaSources.Remove(ViewportKey);

		UMediaPlayer* MediaPlayer = MediaPlayers.FindAndRemoveChecked(ViewportKey);
		check(MediaPlayer);

		MediaPlayer->Close();

		MediaTextures.Remove(ViewportKey);
	}

	// If there are invalid or closed players, start them from scratch
	{
		TArray<FString> StalledPlayerViewportKeys;

		for (const TPair<FString, TObjectPtr<UMediaPlayer>>& Pair : MediaPlayers)
		{
			UMediaPlayer* MediaPlayer = Pair.Value;

			if (!IsValid(MediaPlayer))
			{
				StalledPlayerViewportKeys.Add(Pair.Key);
				continue;
			}

			if (MediaPlayer->IsClosed())
			{
				StalledPlayerViewportKeys.Add(Pair.Key);
				continue;
			}
		}

		for (const FString& ViewportKey : StalledPlayerViewportKeys)
		{
			MediaPlayers.Remove(ViewportKey);
			MediaSources.Remove(ViewportKey);
			MediaTextures.Remove(ViewportKey);
		}
	}
}

void UDisplayClusterPreviewShareComponent::TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	switch (Mode)
	{
	case EDisplayClusterPreviewShareMode::None:
	{
		break;
	}
	case EDisplayClusterPreviewShareMode::PullActor:
	{
		TickPullActor(false);
		break;
	}
	case EDisplayClusterPreviewShareMode::Send:
	{
		TickSend();
		break;
	}
	case EDisplayClusterPreviewShareMode::Receive:
	{
		TickReceive();
		break;
	}
	}
}

void UDisplayClusterPreviewShareComponent::OnComponentDestroyed(bool bDestroyingHierarchy)
{
	// Close all media to avoid keeping resources alive even though the component was destroyed (and possibly kept in undo buffer).
	// This call also restores the dcra original settings.
	SetMode(EDisplayClusterPreviewShareMode::None);

#if WITH_EDITOR
	if (GIsEditor)
	{
		// Unregister Map Change Events
		if (FLevelEditorModule* LevelEditor = FModuleManager::GetModulePtr<FLevelEditorModule>("LevelEditor"))
		{
			LevelEditor->OnMapChanged().RemoveAll(this);
		}
	}
#endif // WITH_EDITOR

	Super::OnComponentDestroyed(bDestroyingHierarchy);
}

void UDisplayClusterPreviewShareComponent::OnRegister()
{
	Super::OnRegister();

	// Make sure it is in a functional state after registration. This is useful when component is re-created from the undo buffer
	ModeChanged();
}

void UDisplayClusterPreviewShareComponent::Serialize(FArchive& Ar)
{
	using namespace UE::DisplayCluster::PreviewShareComponent;

	if (Ar.IsSaving())
	{
		// Because for speed we may be directly referencing the source nDisplay actor preview textures,
		// but these may belong to a different world, we must clear these references when saving.

		if (Mode == EDisplayClusterPreviewShareMode::PullActor)
		{
			RestoreRootActorOriginalSettings();
			RemoveExternalMapReferences(Cast<ADisplayClusterRootActor>(GetOwner()));
		}
	}

	Super::Serialize(Ar);
}

void UDisplayClusterPreviewShareComponent::PostLoad()
{
	Super::PostLoad();

	// Make sure we are in the right state after loading
	ModeChanged();

#if WITH_EDITOR
	if (GIsEditor)
	{
		// Register for Map Change Events. We do this to remove texture external texture reference that
		// prevents proper world GC. In particular, our owing actor reference to the source nDisplay actor preview texture.
		if (FLevelEditorModule* LevelEditor = FModuleManager::GetModulePtr<FLevelEditorModule>("LevelEditor"))
		{
			LevelEditor->OnMapChanged().AddUObject(this, &UDisplayClusterPreviewShareComponent::HandleMapChanged_Editor);
		}
	}
#endif // WITH_EDITOR
}

void UDisplayClusterPreviewShareComponent::SetMode(EDisplayClusterPreviewShareMode NewMode)
{
	// Nothing to do if the mode is unchanged.
	if (Mode == NewMode)
	{
		return;
	}

	Mode = NewMode;

	ModeChanged();
}

void UDisplayClusterPreviewShareComponent::SetUniqueName(const FString& NewUniqueName)
{
	UniqueName = NewUniqueName.TrimStartAndEnd();

	// All the names are now invalid, so we need to close all media.
	CloseAllMedia();
}

#if WITH_EDITOR

void UDisplayClusterPreviewShareComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UDisplayClusterPreviewShareComponent, Mode))
	{
		ModeChanged();
	}
	else if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UDisplayClusterPreviewShareComponent, UniqueName))
	{
		// Remove spaces to reduce chances of the user not realizing that there is a mismatch with its counterpart.
		UniqueName = UniqueName.TrimStartAndEnd();

		// All the names are now invalid, so we need to close all media.
		CloseAllMedia();
	}
	else if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UDisplayClusterPreviewShareComponent, SourceNDisplayActor))
	{
		ModeChanged();
	}
}

void UDisplayClusterPreviewShareComponent::HandleMapChanged_Editor(UWorld* InWorld, EMapChangeType InMapChangeType)
{
	// We remove the reference to the source actor preview texture to avoid this being interpreted as a 
	// memory leak when unloadin the source actor map while the current map is kept open.
	if (Mode == EDisplayClusterPreviewShareMode::PullActor)
	{
		RestoreRootActorOriginalSettings();
	}
}

#endif // WITH_EDITOR