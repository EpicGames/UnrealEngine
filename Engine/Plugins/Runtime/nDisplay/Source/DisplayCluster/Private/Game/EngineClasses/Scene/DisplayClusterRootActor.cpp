// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterRootActor.h"
#include "DisplayClusterRootActorContainers.h"

#include "Async/ParallelFor.h"
#include "Components/SceneComponent.h"
#include "Components/DisplayClusterOriginComponent.h"
#include "Components/DisplayClusterCameraComponent.h"
#include "Components/DisplayClusterScreenComponent.h"
#include "Components/DisplayClusterSceneComponentSyncParent.h"
#include "Components/DisplayClusterSyncTickComponent.h"
#include "Components/DisplayClusterICVFXCameraComponent.h"
#include "Components/DisplayClusterSceneComponentSyncThis.h"
#include "Render/DisplayDevice/Components/DisplayClusterDisplayDeviceBaseComponent.h"
#include "Render/DisplayDevice/Components/DisplayClusterDisplayDeviceComponent.h"
#include "CineCameraComponent.h"
#include "DisplayClusterChromakeyCardActor.h"
#include "ProceduralMeshComponent.h"

#include "DisplayClusterLightCardActor.h"

#include "Config/IPDisplayClusterConfigManager.h"
#include "DisplayClusterConfigurationStrings.h"

#include "IDisplayClusterConfiguration.h"
#include "DisplayClusterConfigurationTypes.h"

#include "Blueprints/DisplayClusterBlueprint.h"
#include "Blueprints/DisplayClusterBlueprintGeneratedClass.h"

#include "GameFramework/PlayerController.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "Engine/GameInstance.h"

#include "HAL/IConsoleManager.h"

#include "Misc/DisplayClusterAppExit.h"
#include "Misc/DisplayClusterGlobals.h"
#include "Misc/DisplayClusterHelpers.h"
#include "Misc/DisplayClusterLog.h"
#include "Misc/DisplayClusterStrings.h"

#include "Render/Viewport/DisplayClusterViewportManager.h"

#include "Game/EngineClasses/Scene/DisplayClusterRootActorInitializer.h"

#include "Algo/MaxElement.h"
#include "Components/DisplayClusterChromakeyCardStageActorComponent.h"
#include "Components/DisplayClusterStageActorComponent.h"
#include "Components/DisplayClusterStageGeometryComponent.h"
#include "Components/DisplayClusterStageIsosphereComponent.h"
#include "Components/LineBatchComponent.h"

#include "CompositeCoreSubsystem.h"
#include "UObject/Package.h"

namespace UE::DisplayCluster::RootActor
{
	/** Collect components by type
	* 
	* @param OutPrimitives - (out) Output variable for collected components.
	* @param pComp         - (in) The component from which the collection will be made.
	* @param bForceHide    - (opt) if true, collects components with any value of the bHiddenInGame property.
	* @param bCollectChildrenVisualizationComponent - If true collects child components.
	*/
	template <typename TComp>
	void CollectPrimitiveComponentsImpl(TSet<FPrimitiveComponentId>& OutPrimitives, const TComp* pComp, bool bForceHide = false, const bool bCollectChildrenVisualizationComponent = true)
	{
		if (const UPrimitiveComponent* PrimComp = Cast<const UPrimitiveComponent>(pComp))
		{
			if (PrimComp->bHiddenInGame
			|| bForceHide
#if WITH_EDITOR
			|| (GIsEditor && PrimComp->bHiddenInSceneCapture /* We are running as a scene capture for preview */)
#endif
				)
			{
				const FPrimitiveComponentId CompId = PrimComp->GetPrimitiveSceneId();
				OutPrimitives.Add(CompId);
			}
		}

		if (bCollectChildrenVisualizationComponent)
		{
			if (const USceneComponent* SceneComp = Cast<const USceneComponent>(pComp))
			{
				TArray<USceneComponent*> ChildrenComponents;
				SceneComp->GetChildrenComponents(false, ChildrenComponents);

				for (const USceneComponent* CompIt : ChildrenComponents)
				{
					CollectPrimitiveComponentsImpl(OutPrimitives, CompIt, bForceHide, bCollectChildrenVisualizationComponent);
				}
			}
		}
	}

	/**
	 * Syncs default media settings to instance.
	 *
	 * @param InstanceDCRA Instance of the nDisplay root actor.
	 */
	static void PropagateMediaSettingsFromDefault(ADisplayClusterRootActor* InstanceDCRA)
	{
		if (!InstanceDCRA || InstanceDCRA->IsTemplate())
		{
			return;
		}

		// Note: ICVFX Camera Component Media property propagation is done in the ICVFXCamera itself.

		// Propagate Backbuffer and Viewports MediaSettings property
		//
		const UDisplayClusterConfigurationData* const ArchetypeConfig = InstanceDCRA->GetDefaultConfigDataFromAsset();
		const UDisplayClusterConfigurationData* const InstanceConfig = InstanceDCRA->GetConfigData();

		if (!IsValid(ArchetypeConfig)
			|| !IsValid(InstanceConfig)
			|| !IsValid(ArchetypeConfig->Cluster)
			|| !IsValid(InstanceConfig->Cluster))
		{
			return;
		}

		// Iterate over instance nodes, find the archetype, and copy the Media setting.
		for (auto ItNode = InstanceConfig->Cluster->Nodes.CreateIterator(); ItNode; ++ItNode)
		{
			const FString& NodeName = ItNode.Key();
			TObjectPtr<UDisplayClusterConfigurationClusterNode> InstanceNode = ItNode.Value();

			if (!InstanceNode)
			{
				continue;
			}

			TObjectPtr<UDisplayClusterConfigurationClusterNode>* ArchetypeNodePtr = ArchetypeConfig->Cluster->Nodes.Find(NodeName);

			if (!ArchetypeNodePtr)
			{
				continue;
			}

			TObjectPtr<UDisplayClusterConfigurationClusterNode> ArchetypeNode = *ArchetypeNodePtr;

			if (!ArchetypeNode)
			{
				continue;
			}

			// Copy node media settings.
			InstanceNode->MediaSettings = ArchetypeNode->MediaSettings;

			// Viewports Media property
			//

			// Iterate over instance viewports, find the archetype, and deep-copy the Media setting.
			for (auto ItViewport = InstanceNode->Viewports.CreateIterator(); ItViewport; ++ItViewport)
			{
				const FString& ViewportName = ItViewport.Key();
				TObjectPtr<UDisplayClusterConfigurationViewport> InstanceViewport = ItViewport.Value();

				if (!InstanceViewport)
				{
					continue;
				}

				TObjectPtr<UDisplayClusterConfigurationViewport>* ArchetypeViewportPtr = ArchetypeNode->Viewports.Find(ViewportName);

				if (!ArchetypeViewportPtr)
				{
					continue;
				}

				TObjectPtr<UDisplayClusterConfigurationViewport> ArchetypeViewport = *ArchetypeViewportPtr;

				if (!ArchetypeViewport)
				{
					continue;
				}

				// Copy viewport media settings.
				InstanceViewport->RenderSettings.Media = ArchetypeViewport->RenderSettings.Media;
			}
		}
	}
};

//////////////////////////////////////////////////////////////////////////////////////////////
// ADisplayClusterRootActor
//////////////////////////////////////////////////////////////////////////////////////////////
ADisplayClusterRootActor::ADisplayClusterRootActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, OperationMode(EDisplayClusterOperationMode::Disabled)
{
	/*
	 * Origin component
	 *
	 * We HAVE to store the origin (root) component in our own UPROPERTY marked visible.
	 * Live link has a property which maintains a component reference. Live link sets this
	 * through their details panel automatically, which unreal validates in
	 * FComponentReferenceCustomization::IsComponentReferenceValid.
	 *
	 * Unreal won't allow native components that don't have CPF_Edit to be set. Luckily
	 * they search the owning class for a property containing the component.
	 */
	{
		DisplayClusterRootComponent = CreateDefaultSubobject<UDisplayClusterOriginComponent>(TEXT("RootComponent"));
		SetRootComponent(DisplayClusterRootComponent);
	}

	// A helper component to trigger nDisplay Tick() during Tick phase
	SyncTickComponent = CreateDefaultSubobject<UDisplayClusterSyncTickComponent>(TEXT("DisplayClusterSyncTick"));

	StageGeometryComponent = CreateDefaultSubobject<UDisplayClusterStageGeometryComponent>(TEXT("DisplayClusterStageGeometry"));

	StageIsosphereComponent = CreateDefaultSubobject<UDisplayClusterStageIsosphereComponent>(TEXT("DisplayClusterStageIsosphere"));
	StageIsosphereComponent->AttachToComponent(GetRootComponent(), FAttachmentTransformRules::KeepRelativeTransform);

	// Default nDisplay camera
	DefaultViewPoint = CreateDefaultSubobject<UDisplayClusterCameraComponent>(TEXT("DefaultViewPoint"));
	DefaultViewPoint->AttachToComponent(GetRootComponent(), FAttachmentTransformRules::KeepRelativeTransform);
	DefaultViewPoint->SetRelativeLocation(FVector(0.f, 0.f, 50.f));

	PrimaryActorTick.bCanEverTick = true;
	PrimaryActorTick.TickGroup = ETickingGroup::TG_PostUpdateWork;

	bFindCameraComponentWhenViewTarget = false;
	bReplicates = false;
	SpawnCollisionHandlingMethod = ESpawnActorCollisionHandlingMethod::AdjustIfPossibleButAlwaysSpawn;

	// Enabled by default to avoid being active on Stage
	SetActorHiddenInGame(true);

	// By default, we don't want the NDC to be spatially loaded.
#if WITH_EDITORONLY_DATA
	bIsSpatiallyLoaded = false;
#endif

	// Set to internal default, user may change
	DefaultDisplayDeviceName = GetInternalDisplayDeviceName();

	// Our internal display device which always exists
	BasicDisplayDeviceComponent = CreateDefaultSubobject<UDisplayClusterDisplayDeviceComponent>(GetInternalDisplayDeviceName());

	// Create transient line batcher component
	LineBatcherComponent = CreateDefaultSubobject<ULineBatchComponent>(TEXT("LineBatcher"), true);

	ResetEntireClusterPreviewRendering();

#if WITH_EDITOR
	Constructor_Editor();
#endif
}

ADisplayClusterRootActor::~ADisplayClusterRootActor()
{
#if WITH_EDITOR
	Destructor_Editor();
#endif
}

void ADisplayClusterRootActor::ResetEntireClusterPreviewRendering()
{
	if (IDisplayClusterViewportManager* ViewportManager = GetViewportManager())
	{
		// Update the preview settings as is from this DCRA
		ViewportManager->GetViewportManagerPreview().ResetEntireClusterPreviewRendering();
	}
}

IDisplayClusterViewportManager* ADisplayClusterRootActor::GetViewportManager() const
{
	return GetViewportManagerImpl();
}

IDisplayClusterViewportManager* ADisplayClusterRootActor::GetOrCreateViewportManager()
{
	check(IsInGameThread());

	if (!ViewportManagerPtr.IsValid())
	{
		UE_LOGF(LogDisplayClusterGame, Log, "DCRA '%ls': Create viewport manager.", *GetName());

		ViewportManagerPtr = IDisplayClusterViewportManager::CreateViewportManager()->ToSharedRef();

		// Set the owner's DCRA to the newly created viewport manager.
		ViewportManagerPtr->GetConfiguration().SetRootActor(EDisplayClusterRootActorType::Any, this);
	}

	return GetViewportManagerImpl();
}

FDisplayClusterViewportManager* ADisplayClusterRootActor::GetViewportManagerImpl() const
{
	return ViewportManagerPtr.Get();
}

IDisplayClusterViewportConfiguration* ADisplayClusterRootActor::GetViewportConfiguration() const
{
	return ViewportManagerPtr.IsValid() ? &ViewportManagerPtr->GetConfiguration() : nullptr;
}

void ADisplayClusterRootActor::RemoveViewportManager()
{
	if (ViewportManagerPtr.IsValid())
	{
		UE_LOGF(LogDisplayClusterGame, Log, "DCRA '%ls': Remove viewport manager.", *GetName());

		// This function must be called before destructor.
		ViewportManagerPtr->Release();

		// Immediately release the viewport manager with resources
		ViewportManagerPtr.Reset();
	}
}

bool ADisplayClusterRootActor::IsPrimaryRootActor() const
{
	if (IDisplayCluster::Get().GetGameMgr()->GetRootActor() == this)
	{
		return true;
	}

	return false;
}

bool ADisplayClusterRootActor::IsPrimaryRootActorForPIE() const
{
#if WITH_EDITOR
	// A preview node must be selected in DCRA so that it can be used in PIE mode
	if (PreviewNodeId == DisplayClusterConfigurationStrings::gui::preview::PreviewNodeNone)
	{
		return false;
	}

	// Only PIE is currently supported
	return IsPrimaryRootActor();
#else
	return false;
#endif
}

bool ADisplayClusterRootActor::IsRunningPIE() const
{
#if WITH_EDITOR
	const UWorld* World = GetWorld();
	return World && World->IsPlayInEditor();
#else
	// Without editor return false
	return false;
#endif
}

bool ADisplayClusterRootActor::IsRunningDisplayCluster() const
{
	if (OperationMode == EDisplayClusterOperationMode::Cluster || OperationMode == EDisplayClusterOperationMode::Editor)
	{
		return true;
	}

	return false;
}

const FDisplayClusterConfigurationICVFX_StageSettings& ADisplayClusterRootActor::GetStageSettings() const
{
	// [TEMPORARY FIX]
	// There are various ways we can get to this function while still having an uninitialized 'CurrentConfigData'.
	// Some of them have already been fixed, while others are still unclear how to reproduce.
	// UE-297696, UE-218690, UE-222703
	// Normally, CurrentConfigData should never be nullptr. As a quick work around, we always return
	// some default settings to avoid nullptr crash. Debug builds will keep reporting the issue.
	checkSlow(CurrentConfigData);
	ensureMsgf(IsValid(CurrentConfigData), TEXT("Found invalid DCRA configuration"));
	static const FDisplayClusterConfigurationICVFX_StageSettings DummyStageSettings;
	return IsValid(CurrentConfigData) ? CurrentConfigData->StageSettings : DummyStageSettings;
}

const FDisplayClusterConfigurationRenderFrame& ADisplayClusterRootActor::GetRenderFrameSettings() const 
{
	// [TEMPORARY FIX]
	// There are various ways we can get to this function while still having an uninitialized 'CurrentConfigData'.
	// Some of them have already been fixed, while others are still unclear how to reproduce.
	// UE-297696, UE-218690, UE-222703
	// Normally, CurrentConfigData should never be nullptr. As a quick work around, we always return
	// some default settings to avoid nullptr crash. Debug builds will keep reporting the issue.
	checkSlow(CurrentConfigData);
	ensureMsgf(IsValid(CurrentConfigData), TEXT("Found invalid DCRA configuration"));
	static const FDisplayClusterConfigurationRenderFrame DummyRenderFrameSettings;
	return IsValid(CurrentConfigData) ? CurrentConfigData->RenderFrameSettings : DummyRenderFrameSettings;
}

void ADisplayClusterRootActor::InitializeFromConfig(UDisplayClusterConfigurationData* ConfigData)
{
	if (ConfigData)
	{
		// Store new config data object
		UpdateConfigDataInstance(ConfigData, true);

		BuildHierarchy();
	}
}

void ADisplayClusterRootActor::OverrideFromConfig(UDisplayClusterConfigurationData* ConfigData)
{
	check(ConfigData);
	check(ConfigData->Scene);
	check(ConfigData->Cluster);

	// Override base types and structures
	CurrentConfigData->Meta = ConfigData->Meta;
	CurrentConfigData->Info = ConfigData->Info;
	CurrentConfigData->CustomParameters = ConfigData->CustomParameters;
	CurrentConfigData->Diagnostics = ConfigData->Diagnostics;
	CurrentConfigData->bFollowLocalPlayerCamera = ConfigData->bFollowLocalPlayerCamera;
	CurrentConfigData->bExitOnEsc = ConfigData->bExitOnEsc;

	// Override Scene, but without changing its name to avoid the Editor not being able to control it via MultiUser.
	{
		FName SceneName = NAME_None;

		if (CurrentConfigData->Scene)
		{
			SceneName = CurrentConfigData->Scene->GetFName();

			const FName DeadName = MakeUniqueObjectName(CurrentConfigData, UDisplayClusterConfigurationScene::StaticClass(), "DEAD_DisplayClusterConfigurationScene");
			CurrentConfigData->Scene->Rename(*DeadName.ToString());
		}

		CurrentConfigData->Scene = DuplicateObject(ConfigData->Scene, CurrentConfigData, SceneName);
	}

	// Override Cluster
	if (CurrentConfigData->Cluster)
	{
		CurrentConfigData->Cluster->PrimaryNode = ConfigData->Cluster->PrimaryNode;
		CurrentConfigData->Cluster->Sync        = ConfigData->Cluster->Sync;
		CurrentConfigData->Cluster->Network     = ConfigData->Cluster->Network;
		CurrentConfigData->Cluster->Failover    = ConfigData->Cluster->Failover;

		// Remove nodes in current config that are not in the new config

		for (auto CurrentNodeIt = CurrentConfigData->Cluster->Nodes.CreateIterator(); CurrentNodeIt; ++CurrentNodeIt)
		{
			if (!ConfigData->Cluster->Nodes.Find(CurrentNodeIt->Key))
			{
				CurrentNodeIt.RemoveCurrent();
			}
		}

		// Go over new nodes and override the current ones (or add new ones that didn't exist)

		for (auto NewNodeIt = ConfigData->Cluster->Nodes.CreateConstIterator(); NewNodeIt; ++NewNodeIt)
		{
			TObjectPtr<UDisplayClusterConfigurationClusterNode>* CurrentNodePtr = CurrentConfigData->Cluster->Nodes.Find(NewNodeIt->Key);

			// Add the node if it doesn't exist
			if (!CurrentNodePtr)
			{
				CurrentConfigData->Cluster->Nodes.Emplace(
					NewNodeIt->Key,
					DuplicateObject(NewNodeIt->Value, CurrentConfigData->Cluster, NewNodeIt->Value->GetFName())
				);

				continue;
			}

			UDisplayClusterConfigurationClusterNode* CurrentNode = *CurrentNodePtr;
			check(CurrentNode);

			const UDisplayClusterConfigurationClusterNode* NewNode = NewNodeIt->Value;
			check(NewNode);

			// Override Node settings

			CurrentNode->Host = NewNode->Host;
			CurrentNode->bIsSoundEnabled = NewNode->bIsSoundEnabled;
			CurrentNode->bIsFullscreen = NewNode->bIsFullscreen;
			CurrentNode->WindowRect = NewNode->WindowRect;
			CurrentNode->Postprocess = NewNode->Postprocess;
			CurrentNode->OutputRemap = NewNode->OutputRemap;

			if (ConfigData->bOverrideViewportsFromExternalConfig)
			{
				// Remove viewports in current node that are not in the new node
				for (auto CurrentViewportIt = CurrentNode->Viewports.CreateIterator(); CurrentViewportIt; ++CurrentViewportIt)
				{
					if (!NewNode->Viewports.Find(CurrentViewportIt->Key))
					{
						CurrentViewportIt.RemoveCurrent();
					}
				}

				// Go over viewport and override the current ones (or add new ones that didn't exist)
				for (auto NewViewportIt = NewNode->Viewports.CreateConstIterator(); NewViewportIt; ++NewViewportIt)
				{
					TObjectPtr<UDisplayClusterConfigurationViewport>* CurrentViewportPtr = CurrentNode->Viewports.Find(NewViewportIt->Key);

					// Add the viewport if it doesn't exist
					if (!CurrentViewportPtr)
					{
						CurrentNode->Viewports.Add(
							NewViewportIt->Key,
							DuplicateObject(NewViewportIt->Value, CurrentNode, NewViewportIt->Value->GetFName())
						);
						continue;
					}

					UDisplayClusterConfigurationViewport* CurrentViewport = *CurrentViewportPtr;
					check(CurrentViewport);

					const UDisplayClusterConfigurationViewport* NewViewport = NewViewportIt->Value;
					check(NewViewport);

					// Override viewport settings

					CurrentViewport->Camera = NewViewport->Camera;
					CurrentViewport->RenderSettings.BufferRatio = NewViewport->RenderSettings.BufferRatio;
					CurrentViewport->GPUIndex = NewViewport->GPUIndex;
					CurrentViewport->Region = NewViewport->Region;
					CurrentViewport->ProjectionPolicy = NewViewport->ProjectionPolicy;
				}
			}
		}
	}

	// Update component transforms if any of them have been changed in the file
	if (ConfigData->bOverrideTransformsFromExternalConfig)
	{
		TUniquePtr<FDisplayClusterRootActorInitializer> Initializer = MakeUnique<FDisplayClusterRootActorInitializer>();
		Initializer->UpdateComponentTransformsOnly(this, ConfigData);
	}

	// There is no sense to call BuildHierarchy because it works for non-BP root actors.
	// On the other hand, OverwriteFromConfig method is called for BP root actors only by nature.
}

void ADisplayClusterRootActor::UpdateConfigDataInstance(UDisplayClusterConfigurationData* ConfigDataTemplate, bool bForceRecreate)
{
	if (ConfigDataTemplate == nullptr)
	{
		CurrentConfigData = nullptr;
		ConfigDataName = TEXT("");
	}
	else
	{
		if (CurrentConfigData == nullptr)
		{
			// Only create config data once. Do not create in constructor as default sub objects or individual properties won't sync
			// properly with instanced values.

			const EObjectFlags CommonFlags = RF_Public | RF_Transactional;

			CurrentConfigData = NewObject<UDisplayClusterConfigurationData>(
				this,
				UDisplayClusterConfigurationData::StaticClass(),
				NAME_None,
				IsTemplate() ? RF_ArchetypeObject | CommonFlags : CommonFlags,
				ConfigDataTemplate);

			if (CurrentConfigData->Cluster == nullptr)
			{
				CurrentConfigData->Cluster = NewObject<UDisplayClusterConfigurationCluster>(
					CurrentConfigData,
					UDisplayClusterConfigurationCluster::StaticClass(),
					NAME_None,
					IsTemplate() ? RF_ArchetypeObject | CommonFlags : CommonFlags);
			}

			if (CurrentConfigData->Scene == nullptr)
			{
				CurrentConfigData->Scene = NewObject<UDisplayClusterConfigurationScene>(
					CurrentConfigData,
					UDisplayClusterConfigurationScene::StaticClass(),
					NAME_None,
					IsTemplate() ? RF_ArchetypeObject | CommonFlags : CommonFlags);
			}
		}
		else if (bForceRecreate)
		{
			UEngine::FCopyPropertiesForUnrelatedObjectsParams Params;
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			// Leaving this enabled for now for the purposes of the aggressive replacement auditing
			Params.bAggressiveDefaultSubobjectReplacement = true;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			Params.bNotifyObjectReplacement = false;
			Params.bDoDelta = false;
			UEngine::CopyPropertiesForUnrelatedObjects(ConfigDataTemplate, CurrentConfigData, Params);
		}

		ConfigDataName = CurrentConfigData->GetFName();
	}
}

UDisplayClusterConfigurationData* ADisplayClusterRootActor::GetDefaultConfigDataFromAsset() const
{
	UClass* CurrentClass = GetClass();
	while (CurrentClass)
	{
		if (UObject* FoundTemplate = Cast<UObject>(CurrentClass->GetDefaultSubobjectByName(ConfigDataName)))
		{
			return Cast<UDisplayClusterConfigurationData>(FoundTemplate);
		}
		CurrentClass = CurrentClass->GetSuperClass();
	}

	return nullptr;
}

UDisplayClusterConfigurationData* ADisplayClusterRootActor::GetConfigData() const
{
	return CurrentConfigData;
}

bool ADisplayClusterRootActor::IsInnerFrustumEnabled(const FString& InnerFrustumID) const
{
	// Common condition for any camera/frustum
	const bool bClusterGlobalInnersEnabled = (CurrentConfigData ? CurrentConfigData->StageSettings.bEnableInnerFrustums : true);

	// add more GUI rules here
	// Inner Frustum Enabled
	//  Camera_1  [ ]
	//  Camera_2  [X]
	//  Camera_3  [X]

	return bClusterGlobalInnersEnabled; // && bOtherCondition(s)
}

int ADisplayClusterRootActor::GetInnerFrustumPriority(const FString& InnerFrustumID) const
{
	int Order = 100000;
	for (const FDisplayClusterComponentRef& It : InnerFrustumPriority)
	{
		if (It.Name.Compare(InnerFrustumID, ESearchCase::IgnoreCase) == 0)
		{
			return Order;
		}
		Order--;
	}

	return -1;
}

template <typename TComp>
void ADisplayClusterRootActor::GetTypedPrimitives(TSet<FPrimitiveComponentId>& OutPrimitives, const TArray<FString>* InCompNames, bool bForceHide, bool bCollectChildrenVisualizationComponent) const
{
	using namespace UE::DisplayCluster::RootActor;

	TArray<TComp*> TypedComponents;
	GetComponents<TComp>(TypedComponents, true);

	for (TComp*& CompIt : TypedComponents)
	{
		if (CompIt)
		{
			if (InCompNames != nullptr)
			{
				if (InCompNames->Find(CompIt->GetName()) != INDEX_NONE)
				{
					// add only comp from names list
					CollectPrimitiveComponentsImpl(OutPrimitives, CompIt, bForceHide, bCollectChildrenVisualizationComponent);
				}
			}
			else
			{
				CollectPrimitiveComponentsImpl(OutPrimitives, CompIt, bForceHide, bCollectChildrenVisualizationComponent);
			}
		}
	}
}

bool ADisplayClusterRootActor::FindPrimitivesByName(const TArray<FString>& InNames, TSet<FPrimitiveComponentId>& OutPrimitives, bool bForceHide)
{
	GetTypedPrimitives<UActorComponent>(OutPrimitives, &InNames, bForceHide, false);

	return !OutPrimitives.IsEmpty();
}

// Gather components not rendered in game
bool ADisplayClusterRootActor::GetHiddenInGamePrimitives(TSet<FPrimitiveComponentId>& OutPrimitives)
{
	using namespace UE::DisplayCluster::RootActor;

	TRACE_CPUPROFILER_EVENT_SCOPE(DCRootActor_GetHiddenInGamePrimitives);

	check(IsInGameThread());

	OutPrimitives.Empty();

	if (CurrentConfigData)
	{
		// Add warp meshes assigned in the configuration into hide lists
		TArray<FString> WarpMeshNames;
		CurrentConfigData->GetReferencedMeshNames(WarpMeshNames);
		if (WarpMeshNames.Num() > 0)
		{
			GetTypedPrimitives<UStaticMeshComponent>(OutPrimitives, &WarpMeshNames);
			GetTypedPrimitives<UProceduralMeshComponent>(OutPrimitives, &WarpMeshNames);
		}
	}

	// Always hide DC Screen component in game
	GetTypedPrimitives<UDisplayClusterScreenComponent>(OutPrimitives);

	if (FDisplayClusterViewportManager* ViewportManager = GetViewportManagerImpl())
	{
		// Always hide preview meshes for preview
		for (TSharedPtr<IDisplayClusterViewport, ESPMode::ThreadSafe>& ViewportIt : ViewportManager->GetEntireClusterViewports())
		{
			if (ViewportIt.IsValid())
			{
				if (UMeshComponent* PreviewMesh = ViewportIt->GetViewportPreview().GetPreviewMeshComponent())
				{
					if (UPrimitiveComponent* PrimComp = Cast<UPrimitiveComponent>(PreviewMesh))
					{
						OutPrimitives.Add(PrimComp->GetPrimitiveSceneId());
					}
				}

				if (UMeshComponent* PreviewEditableMesh = ViewportIt->GetViewportPreview().GetPreviewEditableMeshComponent())
				{
					if (UPrimitiveComponent* PrimComp = Cast<UPrimitiveComponent>(PreviewEditableMesh))
					{
						OutPrimitives.Add(PrimComp->GetPrimitiveSceneId());
					}
				}
			}
		}
	}

#if WITH_EDITOR
	// Hide visualization and hidden components from RootActor
	{
		TArray<UPrimitiveComponent*> PrimitiveComponents;
		GetComponents(PrimitiveComponents);

		for (UPrimitiveComponent* CompIt : PrimitiveComponents)
		{
			CollectPrimitiveComponentsImpl(OutPrimitives, CompIt);
		}
	}

	// Hide visualization and hidden components from scene

	const UWorld* CurrentWorld = GetWorld();

	if (CurrentWorld && !CurrentWorld->IsGameWorld()) // Note: We should only have to do this for preview in editor (not already game worlds).
	{
		TArray<AActor*> Actors;
		{
			int32 NumActors = 0;

			for (const ULevel* Level : CurrentWorld->GetLevels())
			{
				if (Level && Level->bIsVisible)
				{
					NumActors += Level->Actors.Num();
				}
			}

			// Presize the array
			Actors.Reserve(NumActors);

			// Fill the array
			for (const ULevel* Level : CurrentWorld->GetLevels())
			{
				if (Level && Level->bIsVisible)
				{
					Actors.Append(Level->Actors);
				}
			}
		}

		// Create as many threads as cores, minus an arbitrary number of reserved cores to avoid starving other subsystems
		int32 NumIterThreads;
		{
			constexpr int32 ReservedCores = 4;
			static const int32 NumberOfCores = FPlatformMisc::NumberOfCores();

			NumIterThreads = NumberOfCores - ReservedCores;

			// Make sure there are enough actors to make it worth a new thread
			constexpr int32 MinActorsPerThread = 64;
			NumIterThreads = FMath::Min(NumIterThreads, Actors.Num() / MinActorsPerThread);

			// There should be at least one thread
			NumIterThreads = FMath::Max(NumIterThreads, 1);
		}

		// Allocate primitive sets for each thread
		TArray<TSet<FPrimitiveComponentId>> PrimitiveComponentsArray;
		PrimitiveComponentsArray.AddDefaulted(NumIterThreads);

		// Start the iteration parallel threads
		ParallelFor(NumIterThreads, [NumIterThreads, CurrentWorld, &Actors, CurrentRootActor = this, &PrimitiveComponentsArray](int32 Index)
			{
				// Using inline allocator for efficiency
				constexpr int32 MaxExpectedComponentsPerActor = 64;
				TArray<UPrimitiveComponent*, TInlineAllocator<MaxExpectedComponentsPerActor>> PrimitiveComponents;

				// The thread index is our starting actor index
				int32 ActorIdx = Index;

				while (ActorIdx < Actors.Num())
				{
					const AActor* Actor = Actors[ActorIdx];

					// RootActor visibility is already handled by this GetHiddenInGamePrimitives() function. (See the code above.)
					// The code below is generic for all actor types and hides all primitives without following the DCRA rules used above.
					if (IsValid(Actor) && Actor != CurrentRootActor)
					{
						Actor->GetComponents(PrimitiveComponents);
						for (UPrimitiveComponent* PrimComp : PrimitiveComponents)
						{
							CollectPrimitiveComponentsImpl(PrimitiveComponentsArray[Index], PrimComp, Actor->IsHidden());
						}

						// Empty w/o ever shrinking, for efficiency
						PrimitiveComponents.Empty(PrimitiveComponents.Max());
					}

					// Jump to every other NumIterThreads actors.
					ActorIdx += NumIterThreads;
				}
			});

		// Join all the found primitives arrays
		for (const TSet<FPrimitiveComponentId>& PrimitiveComponents : PrimitiveComponentsArray)
		{
			OutPrimitives.Append(PrimitiveComponents);
		}
	}

#endif

	if (ULineBatchComponent* LineBatch = GetLineBatchComponent())
	{
		// Always hide RootActor batches on viewports
		CollectPrimitiveComponentsImpl(OutPrimitives, LineBatch, true);
	}

	return OutPrimitives.Num() > 0;
}

void ADisplayClusterRootActor::InitializeRootActor()
{
	const UWorld* World = GetWorld();

	if (!World)
	{
		return;
	}

	bool bIsPIE = false;

	if (!CurrentConfigData && !ConfigDataName.IsNone())
	{
		// Attempt load from embedded data.
		UpdateConfigDataInstance(GetDefaultConfigDataFromAsset());
	}

	StageGeometryComponent->Invalidate();

	if (CurrentConfigData)
	{
		BuildHierarchy();
	}

	// Propagate Media settings that should always override the instance settings.
	// This is currently required due to limitation of instanced property propagation,
	// where the instanced media settings may be incorrectly considered dirty and therefore
	// not propagated from the template to the instance.
	UE::DisplayCluster::RootActor::PropagateMediaSettingsFromDefault(this);
}

void ADisplayClusterRootActor::UpdateProceduralMeshComponentData(const UProceduralMeshComponent* InProceduralMeshComponent)
{
	check(IsInGameThread());

	if (FDisplayClusterViewportManager* ViewportManager = GetViewportManagerImpl())
	{
		FName ProceduralComponentName = (InProceduralMeshComponent==nullptr) ? NAME_None : InProceduralMeshComponent->GetFName();

		// Support for all hidden internal refs
		ViewportManager->MarkComponentGeometryDirty(ProceduralComponentName);
	}
}

void ADisplayClusterRootActor::SetPreviewEnablePostProcess(const bool bNewPreviewEnablePostProcess)
{
#if WITH_EDITOR
	if (bNewPreviewEnablePostProcess != bPreviewEnablePostProcess)
	{
		bPreviewEnablePostProcess = bNewPreviewEnablePostProcess;

		ResetEntireClusterPreviewRendering();
	}
#endif // WITH_EDITOR
}

void ADisplayClusterRootActor::SetPreviewEnableHoldoutComposite(const bool bNewPreviewEnableHoldoutComposite)
{
#if WITH_EDITOR
	if (bNewPreviewEnableHoldoutComposite != bPreviewEnableHoldoutComposite)
	{
		bPreviewEnableHoldoutComposite = bNewPreviewEnableHoldoutComposite;

		ResetEntireClusterPreviewRendering();
	}
#endif // WITH_EDITOR
}

bool ADisplayClusterRootActor::BuildHierarchy()
{
	check(CurrentConfigData);
	check(IsInGameThread());

	if (!IsBlueprint())
	{
		// Temporary solution. The whole initialization stuff has been moved to a separate initialization class. Since
		// it won't be possible to configure any components in a config file, and the proper asset initialization will
		// be performed on the configurator side, the DCRA won't need to have any custom logic around the components.
		TUniquePtr<FDisplayClusterRootActorInitializer> Initializer = MakeUnique<FDisplayClusterRootActorInitializer>();
		Initializer->Initialize(this, CurrentConfigData);
	}

	return true;
}

void ADisplayClusterRootActor::SetLightCardOwnership()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(DCRootActor_SetLightCardOwnership);

	if (UDisplayClusterConfigurationData* CurrentData = GetConfigData())
	{
		FDisplayClusterConfigurationICVFX_VisibilityList& LightCardVisibilityList = CurrentData->StageSettings.Lightcard.ShowOnlyList;
		LightCardVisibilityList.AutoAddedActors.Reset();

		TArray<UDisplayClusterICVFXCameraComponent*> ICVFXComponents;
		GetComponents(ICVFXComponents);

		// Iterate over the VisibilityList.Actors array, looking for legacy cards and set the owner so the
		// card can look the root actor up in certain situations, like adjusting labels when running as -game.
		{
			for (const TSoftObjectPtr<AActor>& Actor : LightCardVisibilityList.Actors)
			{
				if (ADisplayClusterLightCardActor* LightCardActor = Actor.IsValid() ? Cast<ADisplayClusterLightCardActor>(Actor.Get()) : nullptr)
				{
					LightCardActor->SetWeakRootActorOwner(this);
				}
			}

			for (UDisplayClusterICVFXCameraComponent* Camera : ICVFXComponents)
			{
				FDisplayClusterConfigurationICVFX_ChromakeyRenderSettings* CameraChromakeyRenderSettings = Camera->CameraSettings.Chromakey.GetWritableChromakeyRenderSettings(GetStageSettings());
				if (!CameraChromakeyRenderSettings)
				{
					// Updating the CK visibility list only for cameras using chromakey actor rendering
					continue;
				}

				FDisplayClusterConfigurationICVFX_VisibilityList& ChromakeyCards = CameraChromakeyRenderSettings->ShowOnlyList;
				ChromakeyCards.AutoAddedActors.Reset();

				for (const TSoftObjectPtr<AActor>& Actor : ChromakeyCards.Actors)
				{
					if (ADisplayClusterChromakeyCardActor* ChromakeyCardActor = Actor.IsValid() ? Cast<ADisplayClusterChromakeyCardActor>(Actor.Get()) : nullptr)
					{
						ChromakeyCardActor->SetWeakRootActorOwner(this);
					}
				}
			}
		}

		// Next iterate over all light card world actors looking for 5.2+ light cards that determine the root actor
		// they belong to, as well as handle legacy layer operations.
		if (const UWorld* World = GetWorld())
		{
			for (TActorIterator<ADisplayClusterLightCardActor> Iter(World); Iter; ++Iter)
			{
				ADisplayClusterLightCardActor* LightCardActor = *Iter;
				if (ADisplayClusterChromakeyCardActor* ChromakeyCardActor = Cast<ADisplayClusterChromakeyCardActor>(LightCardActor))
				{
					for (UDisplayClusterICVFXCameraComponent* Camera : ICVFXComponents)
					{
						FDisplayClusterConfigurationICVFX_ChromakeyRenderSettings* CameraChromakeyRenderSettings = Camera->CameraSettings.Chromakey.GetWritableChromakeyRenderSettings(GetStageSettings());
						if (!CameraChromakeyRenderSettings)
						{
							// Updating the CK visibility list only for cameras using chromakey actor rendering
							continue;
						}

						FDisplayClusterConfigurationICVFX_VisibilityList& ChromakeyCards = CameraChromakeyRenderSettings->ShowOnlyList;

						const UDisplayClusterChromakeyCardStageActorComponent* ChromakeyStageActor = Cast<UDisplayClusterChromakeyCardStageActorComponent>(ChromakeyCardActor->GetStageActorComponent());
						if (ChromakeyStageActor && ChromakeyStageActor->IsReferencedByICVFXCamera(Camera))
						{
							// 5.2+ chromakey cards
							ChromakeyCards.AutoAddedActors.Add(ChromakeyCardActor);
						}
						else
						{
							// Legacy chromakey layers
							for (const FActorLayer& ActorLayer : ChromakeyCards.ActorLayers)
							{
								if (ChromakeyCardActor->Layers.Contains(ActorLayer.Name))
								{
									ChromakeyCardActor->SetWeakRootActorOwner(this);
									break;
								}
							}
						}
					}
				}
				else
				{
					if (LightCardActor->GetStageActorComponent() &&
						LightCardActor->GetStageActorComponent()->GetRootActor() == this)
					{
						// 5.2+ light cards
						LightCardVisibilityList.AutoAddedActors.Add(LightCardActor);
					}
					else if (LightCardVisibilityList.ActorLayers.Num() > 0)
					{
						// Legacy light card layers
						for (const FActorLayer& ActorLayer : LightCardVisibilityList.ActorLayers)
						{
							if (LightCardActor->Layers.Contains(ActorLayer.Name))
							{
								LightCardActor->SetWeakRootActorOwner(this);
								break;
							}
						}
					}
				}
			}
		}
	}
}

bool ADisplayClusterRootActor::GetFlushPositionAndNormal(const FVector& WorldPosition, FVector& OutPosition, FVector& OutNormal)
{
	const FVector StagePosition = GetCommonViewPoint()->GetComponentLocation();
	const FVector Direction = (WorldPosition - StagePosition).GetSafeNormal();

	float Distance;
	FVector Normal;

	if (StageGeometryComponent->GetStageDistanceAndNormal(Direction, Distance, Normal))
	{
		OutPosition = Distance * Direction + GetCommonViewPoint()->GetComponentLocation();

		// Normal is returned in the local "radial basis", meaning that the x axis of the basis points radially inwards.
		// Convert this to world coordinates using the radial basis made from the world direction (no need to account
		// for root actor rotation here since Direction is already in world coordinates)
		const FMatrix RadialBasis = FRotationMatrix::MakeFromX(Direction);
		OutNormal = RadialBasis.TransformVector(Normal);
		return true;
	}

	return false;
}

bool ADisplayClusterRootActor::MakeStageActorFlushToWall(const TScriptInterface<IDisplayClusterStageActor>& StageActor, double DesiredOffsetFromFlush)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(DCRootActor_MakeStageActorFlushToWall);

	if (StageActor.GetObject() && !StageActor->IsUVActor())
	{
		FVector Position;
		FVector Normal;
		if (GetFlushPositionAndNormal(StageActor->GetStageActorTransform().GetLocation(), Position, Normal))
		{
			const FTransform StageActorTransform = StageActor->GetOrigin();
			const FVector LocalPosition = StageActorTransform.InverseTransformPosition(Position);
			const FVector LocalNormal = FRotationMatrix::MakeFromX(LocalPosition.GetSafeNormal()).InverseTransformVector(StageActorTransform.InverseTransformVectorNoScale(Normal));
			const FRotator Rotation = FRotationMatrix::MakeFromX(-LocalNormal).Rotator();
			const float Distance = FMath::Max(FMath::Min(LocalPosition.Length(), StageGeometryComponent->GetStageBoundingRadius()) + DesiredOffsetFromFlush, 0);

			StageActor->SetDistanceFromCenter(Distance);
			StageActor->SetPitch(Rotation.Pitch);
			StageActor->SetYaw(Rotation.Yaw);

			StageActor->UpdateStageActorTransform();
			return true;
		}
	}

	return false;
}

bool ADisplayClusterRootActor::GetDistanceToStageGeometry(const FVector& WorldPosition, const FVector& WorldDirection, float& OutDistance) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ADisplayClusterRootActor::GetDistanceToStageGeometry);

	if (StageGeometryComponent->IsGeometryMapValid())
	{
		FHitResult HitResult;

		// Trace far enough to ensure the line trace will actually intersect any side of the stage isosphere from any world position
		const float TraceDistance = FVector::Distance(WorldPosition, GetActorLocation()) + StageGeometryComponent->GetStageBoundingRadius();
		const FVector TraceEnd = WorldPosition + WorldDirection.GetSafeNormal() * TraceDistance;

		FCollisionQueryParams Params(SCENE_QUERY_STAT(DisplayClusterStageTrace), true);
		if (StageIsosphereComponent->LineTraceComponent(HitResult, WorldPosition, TraceEnd, Params))
		{
			OutDistance = HitResult.Distance;
			return true;
		}

		OutDistance = 0;
		return false;
	}

	return false;
}

bool ADisplayClusterRootActor::IsBlueprint() const
{
	for (UClass* Class = GetClass(); Class; Class = Class->GetSuperClass())
	{
		if (Cast<UBlueprintGeneratedClass>(Class) != nullptr)
		{
			return true;
		}
	}
	
	return false;
}

void ADisplayClusterRootActor::BeginPlay()
{
	Super::BeginPlay();

	// Store current operation mode
	OperationMode = GDisplayCluster->GetOperationMode();

	InitializeRootActor();
}

void ADisplayClusterRootActor::Tick(float DeltaSeconds)
{
	// Update saved DeltaSeconds for root actor
	LastDeltaSecondsValue = DeltaSeconds;

	if (ULineBatchComponent* LineBatch = GetLineBatchComponent())
	{
		LineBatch->Flush();
	}

	// Support for DCRA preview in the scene for standalone\package
	// Use settings only from active DCRA
	const bool bIsPrimaryRootActor = IsPrimaryRootActor();
	const bool bIsRunningDisplayCluster = IsRunningDisplayCluster();
	if (bIsRunningDisplayCluster && bIsPrimaryRootActor)
	{
		UWorld* const CurWorld = GetWorld();
		if (CurWorld && CurrentConfigData)
		{
			APlayerController* const CurPlayerController = CurWorld->GetFirstPlayerController();
			if (CurPlayerController)
			{
				// Depending on the flag state the DCRA follows or not the current player's camera
				if (CurrentConfigData->bFollowLocalPlayerCamera)
				{
					APlayerCameraManager* const CurPlayerCameraManager = CurPlayerController->PlayerCameraManager;
					if (CurPlayerCameraManager)
					{
						SetActorLocationAndRotation(CurPlayerCameraManager->GetCameraLocation(), CurPlayerCameraManager->GetCameraRotation());
					}
				}

				if (CurrentConfigData->bExitOnEsc)
				{
					if (CurPlayerController->WasInputKeyJustPressed(EKeys::Escape))
					{
						FDisplayClusterAppExit::ExitApplication(FString("Exit on ESC requested"));
					}
				}
			}
		}
	}

	// Show 'not supported' warning if instanced stereo is used
	if (OperationMode != EDisplayClusterOperationMode::Disabled && bIsPrimaryRootActor)
	{
		static const TConsoleVariableData<int32>* const InstancedStereoCVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("vr.InstancedStereo"));
		if (InstancedStereoCVar)
		{
			const bool bIsInstancedStereoRequested = (InstancedStereoCVar->GetValueOnGameThread() != 0);
			if (bIsInstancedStereoRequested)
			{
				UE_LOGF(LogDisplayClusterGame, Error, "Instanced stereo was requested. nDisplay doesn't support instanced stereo so far.");
				GEngine->AddOnScreenDebugMessage(-1, 0.f, FColor::Red, TEXT("nDisplay doesn't support instanced stereo"));
			}
		}
	}

	TickPreviewRenderer();

	SetLightCardOwnership();

	Super::Tick(DeltaSeconds);
}

bool ADisplayClusterRootActor::PreparePreviewRendererSettings(FDisplayClusterViewport_PreviewSettings& OutPreviewSettings) const
{
	const bool bIsPrimaryRootActor = IsPrimaryRootActor();
	const bool bIsRunningDisplayCluster = IsRunningDisplayCluster();

	// Get preview settings from the this actor current source
	// as they may already be configured externally, so just get the preview settings from the current source
	OutPreviewSettings = GetPreviewSettings();

	const bool bIsRunningPIE = IsRunningPIE();
	const bool bIsRunningGame = IsRunningGame();

	// RootActor can have a preview in the scene
	bool bEnablePreviewInScene = OutPreviewSettings.bPreviewEnable;
	bool bPreviewInGame = false;

	if (bIsRunningPIE)
	{
		bPreviewInGame = true;

		if (IsPrimaryRootActorForPIE())
		{
			bEnablePreviewInScene = false;

			// Disable frustum rendering in PIE
			OutPreviewSettings.bPreviewICVFXFrustums = false;
		}
		else
		{
			// When running in a game or PIE, a special flag is required to show a preview for PIE/Standalone/Package
			bEnablePreviewInScene = OutPreviewSettings.bPreviewEnable && OutPreviewSettings.bPreviewInGameEnable;
		}
	}
	else if (bIsRunningGame || bIsRunningDisplayCluster)
	{
		bPreviewInGame = true;

		// If RootActor is used in a cluster/game , special rules must be used for previewing in the scene:
		if (bIsPrimaryRootActor)
		{
			// Disable preview in scene rendering for the primary RootActor in the game
			bEnablePreviewInScene = false;
		}
		else
		{
			// When running in a game or PIE, a special flag is required to show a preview for PIE/Standalone/Package
			bEnablePreviewInScene = OutPreviewSettings.bPreviewEnable && OutPreviewSettings.bPreviewInGameEnable;
		}
	}

	if (GetGameInstance() && GetGameInstance()->IsDedicatedServerInstance())
	{
		// Skip rendering on dedicated server
		bEnablePreviewInScene = false;
	}

	if (bPreviewInGame && !OutPreviewSettings.bPreviewInGameRenderFrustum)
	{
		// Disable frustum preview rendering in game
		OutPreviewSettings.bPreviewICVFXFrustums = false;
	}

	if (!bEnablePreviewInScene)
	{
		// Disable preview rendering
		OutPreviewSettings.bPreviewEnable = false;
	}

	// To use the `HoldoutComposite` plugin, a post-process is required
	if (!OutPreviewSettings.bPreviewEnablePostProcess)
	{
		OutPreviewSettings.bPreviewEnableHoldoutComposite = false;
	}

	// The HoldoutComposite plugin requires special project settings.
	if (OutPreviewSettings.bPreviewEnableHoldoutComposite
		&& !UCompositeCoreSubsystem::IsProjectSettingsValid())
	{
		OutPreviewSettings.bPreviewEnableHoldoutComposite = false;

		// In this particular case, also disable the postprocess.
		OutPreviewSettings.bPreviewEnablePostProcess = false;
	}

	return bEnablePreviewInScene;
}

void ADisplayClusterRootActor::TickPreviewRenderer()
{
	FDisplayClusterViewport_PreviewSettings NewPreviewSettings;
	bool bEnablePreviewInScene = PreparePreviewRendererSettings(NewPreviewSettings);

	if (IDisplayClusterViewportManager* ViewportManager = bEnablePreviewInScene ? GetOrCreateViewportManager() : GetViewportManager())
	{
		// Disable preview rendering if this DCRA is under an exclusive lock.
		if (ViewportManager->GetConfiguration().IsRootActorExclusivelyLocked())
		{
			// Disable preview in scene
			NewPreviewSettings.bPreviewEnable = false;
			bEnablePreviewInScene = false;
		}

		// Update preview settings to new
		ViewportManager->GetConfiguration().SetPreviewSettings(NewPreviewSettings);

		// Stop preview rendering
		ViewportManager->GetViewportManagerPreview().UpdateEntireClusterPreviewRender(bEnablePreviewInScene);
	}

#if WITH_EDITOR
	// The StageIsosphereComponent can only be seen if DCRA preview is enabled.
	const bool bShowStageIsosphere = NewPreviewSettings.bPreviewEnable && bPreviewStageGeometryMesh;
	if (StageIsosphereComponent && StageIsosphereComponent->GetVisibleFlag() != bShowStageIsosphere)
	{
		StageIsosphereComponent->SetVisibility(bShowStageIsosphere);
	}
#endif

	// Update RootActor visibility for game
	SetActorHiddenInGame(!bEnablePreviewInScene);
}

FDisplayClusterViewport_PreviewSettings ADisplayClusterRootActor::GetPreviewSettings(bool bIgnorePreviewSetttingsSource) const
{
	if (!bIgnorePreviewSetttingsSource)
	{
		// Obtain preview settings from other sources:
		switch (PreviewSetttingsSource)
		{
		case EDisplayClusterConfigurationRootActorPreviewSettingsSource::Configuration:
			if (IDisplayClusterViewportManager* ViewportManager = GetViewportManager())
			{
				return ViewportManager->GetConfiguration().GetPreviewSettings();
			}
			break;


		case EDisplayClusterConfigurationRootActorPreviewSettingsSource::RootActor:
		default:
			break;
		}
	}

	// Get preview settings from RootActor properties:

	FDisplayClusterViewport_PreviewSettings OutPreviewSettings;

	// By default RootActor renders in scene colors
	OutPreviewSettings.EntireClusterPreviewRenderMode = EDisplayClusterRenderFrameMode::PreviewInScene;

	OutPreviewSettings.bPreviewEnable       = bPreviewEnable;
	OutPreviewSettings.bFreezePreviewRender = bFreezePreviewRender;

	OutPreviewSettings.bEnablePreviewTechvis     = bEnablePreviewTechvis;
	OutPreviewSettings.bPreviewEnablePostProcess = bPreviewEnablePostProcess;
	OutPreviewSettings.bPreviewEnableOCIO        = bPreviewEnableOCIO;

	OutPreviewSettings.bPreviewEnableOverlayMaterial = bPreviewEnableOverlayMaterial;

	OutPreviewSettings.bPreviewICVFXFrustums           = bPreviewICVFXFrustums;
	OutPreviewSettings.PreviewICVFXFrustumsFarDistance = PreviewICVFXFrustumsFarDistance;

	OutPreviewSettings.bEnablePreviewMesh         = bEnablePreviewMesh;
	OutPreviewSettings.bEnablePreviewEditableMesh = bEnablePreviewEditableMesh;

	OutPreviewSettings.PreviewRenderTargetRatioMult = PreviewRenderTargetRatioMult;
	OutPreviewSettings.PreviewMaxTextureDimension   = PreviewMaxTextureDimension;

	OutPreviewSettings.TickPerFrame      = TickPerFrame;
	OutPreviewSettings.ViewportsPerFrame = ViewportsPerFrame;

	OutPreviewSettings.bPreviewInGameEnable        = bPreviewInGameEnable;
	OutPreviewSettings.bPreviewInGameRenderFrustum = bPreviewInGameRenderFrustum;

	OutPreviewSettings.bPreviewEnableHoldoutComposite = bPreviewEnableHoldoutComposite;
	OutPreviewSettings.bPreviewEnableTSR = bPreviewEnableTSR;

	return OutPreviewSettings;
}

void ADisplayClusterRootActor::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	PostLoad_Editor();
#endif

	InitializeRootActor();
}

void ADisplayClusterRootActor::PostActorCreated()
{
	Super::PostActorCreated();

#if WITH_EDITOR
	PostActorCreated_Editor();
#endif

	InitializeRootActor();
}

void ADisplayClusterRootActor::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
#if WITH_EDITOR
	EndPlay_Editor(EndPlayReason);
#endif

	Super::EndPlay(EndPlayReason);
}

void ADisplayClusterRootActor::Destroyed()
{
#if WITH_EDITOR
	Destroyed_Editor();
#endif

	// Release viewport manager with resources immediatelly
	RemoveViewportManager();

	Super::Destroyed();
}

void ADisplayClusterRootActor::BeginDestroy()
{
#if WITH_EDITOR
	BeginDestroy_Editor();
#endif

	// Release viewport manager with resources immediatelly
	RemoveViewportManager();

	Super::BeginDestroy();
}

UDisplayClusterCameraComponent* ADisplayClusterRootActor::GetDefaultCamera() const
{
	return DefaultViewPoint;
}

FName ADisplayClusterRootActor::GetInternalDisplayDeviceName() const
{
	return FName("BasicDisplayDevice");
}

UDisplayClusterDisplayDeviceBaseComponent* ADisplayClusterRootActor::GetDefaultDisplayDevice() const
{
	if (DefaultDisplayDeviceComponent)
	{
		// Check already assigned/created
		if (DefaultDisplayDeviceComponent->GetFName() == DefaultDisplayDeviceName)
		{
			return DefaultDisplayDeviceComponent;
		}
	}

	DefaultDisplayDeviceComponent = nullptr;

	// User assigned default
	if (!DefaultDisplayDeviceName.IsNone() && DefaultDisplayDeviceName != GetInternalDisplayDeviceName())
	{
		DefaultDisplayDeviceComponent = GetComponentByName<UDisplayClusterDisplayDeviceBaseComponent>(DefaultDisplayDeviceName.ToString());

		if (!DefaultDisplayDeviceComponent)
		{
			UE_LOGF(LogDisplayClusterGame, Warning, "Invalid default display device. Using internal nDisplay default device.");
		}
	}

	// Fallback to our internal default
	if (!DefaultDisplayDeviceComponent)
	{
		DefaultDisplayDeviceComponent = BasicDisplayDeviceComponent;
	}

	return DefaultDisplayDeviceComponent;
}

ULineBatchComponent* ADisplayClusterRootActor::GetLineBatchComponent() const
{
	return LineBatcherComponent;
}

USceneComponent* ADisplayClusterRootActor::GetCommonViewPoint() const
{
	if (UDisplayClusterConfigurationData* ConfigData = GetConfigData())
	{
		if (ConfigData->Cluster)
		{
			TMap<FString, int32> ViewPointCounts;

			// Get all the camera names used by viewports in this cluster
			for (const TPair<FString, TObjectPtr<UDisplayClusterConfigurationClusterNode>>& NodePair : ConfigData->Cluster->Nodes)
			{
				if (!NodePair.Value)
				{
					continue;
				}

				for (const TPair<FString, TObjectPtr<UDisplayClusterConfigurationViewport>>& ViewportPair : NodePair.Value->Viewports)
				{
					if (!ViewportPair.Value)
					{
						continue;
					}

					ViewPointCounts.FindOrAdd(ViewportPair.Value->Camera)++;
				}
			}

			if (!ViewPointCounts.IsEmpty())
			{
				// Find the viewpoint with the most references
				TPair<FString, int32>* MostCommonViewPointCount = Algo::MaxElementBy(ViewPointCounts,
					[](const TPair<FString, int32>& Pair)
					{
						return Pair.Value;
					}
				);

				if (MostCommonViewPointCount && !MostCommonViewPointCount->Key.IsEmpty())
				{
					// Try to return the camera with the most common name
					if (USceneComponent* Camera = GetComponentByName<UDisplayClusterCameraComponent>(MostCommonViewPointCount->Key))
					{
						return Camera;
					}
				}
			}
		}
	}

	// We didn't find a common camera override (or it was empty), so fall back to the default camera
	if (USceneComponent* DefaultCamera = GetDefaultCamera())
	{
		return DefaultCamera;
	}

	// No default camera, so fall back to the cluster root
	return GetRootComponent();
}

bool ADisplayClusterRootActor::SetReplaceTextureFlagForAllViewports(bool bReplace)
{
	IDisplayCluster& Display = IDisplayCluster::Get();

	UDisplayClusterConfigurationData* ConfigData = GetConfigData();

	if (!ConfigData)
	{
		UE_LOGF(LogDisplayClusterGame, Warning, "RootActor's ConfigData was null");
		return false;
	}

	if (!ConfigData->Cluster)
	{
		UE_LOGF(LogDisplayClusterGame, Warning, "ConfigData's Cluster was null");
		return false;
	}

	// Only update the viewports of the current node. If there isn't one, which is normal in Editor, we update them all.

	const FString NodeId = Display.GetClusterMgr()->GetNodeId();
	TArray<typename decltype(UDisplayClusterConfigurationCluster::Nodes)::ValueType, TInlineAllocator<1>> Nodes;

	if (NodeId.IsEmpty())
	{
		ConfigData->Cluster->Nodes.GenerateValueArray(Nodes);
	}
	else
	{
		// No need to set this on a non operational nDisplay root actor.
		if ((GDisplayCluster->GetOperationMode() == EDisplayClusterOperationMode::Cluster)
			&& (this != Display.GetGameMgr()->GetRootActor()))
		{
			return false;
		}

		UDisplayClusterConfigurationClusterNode* Node = ConfigData->Cluster->GetNode(NodeId);

		if (!Node)
		{
			UE_LOGF(LogDisplayClusterGame, Warning, "NodeId '%ls' not found in ConfigData", *NodeId);
			return false;
		}

		Nodes.Add(Node);
	}

	for (const UDisplayClusterConfigurationClusterNode* Node : Nodes)
	{
		check(Node != nullptr);

		for (const TPair<FString, TObjectPtr<UDisplayClusterConfigurationViewport>>& ViewportItem : Node->Viewports)
		{
			if (ViewportItem.Value)
			{
				ViewportItem.Value->RenderSettings.Replace.bAllowReplace = bReplace;
			}
		}
	}

	return true;
}

bool ADisplayClusterRootActor::SetFreezeOuterViewports(bool bEnable)
{
	UDisplayClusterConfigurationData* ConfigData = GetConfigData();

	if (!ConfigData)
	{
		UE_LOGF(LogDisplayClusterGame, Warning, "ADisplayClusterRootActor::SetFreezeOuterViewports failed because ConfigData was null");
		return false;
	}

	if (ConfigData->StageSettings.bFreezeRenderOuterViewports != bEnable)
	{
#if WITH_EDITOR
		Modify();
		ConfigData->Modify();
#endif
		ConfigData->StageSettings.bFreezeRenderOuterViewports = bEnable;
	}

	return true;
}

void ADisplayClusterRootActor::RemoveRootActorExpiredLockOwners()
{
	for (auto LockOwnerIt = RootActorExclusiveLockOwners.CreateIterator(); LockOwnerIt; ++LockOwnerIt)
	{
		if (!LockOwnerIt->IsValid()
		 || !LockOwnerIt->Pin().IsValid()
		 || !LockOwnerIt->Pin().Get()->GetConfiguration().IsRootActorLockedByThis(this))
		{
			LockOwnerIt.RemoveCurrent();
		}
	}
}

void ADisplayClusterRootActor::AddRootActorExclusiveLockOwner(const TSharedRef<IDisplayClusterViewportManager, ESPMode::ThreadSafe> InOwner)
{
	FScopeLock Lock(&RootActorExclusiveLockOwnersMutex);
	RootActorExclusiveLockOwners.Add(InOwner);
	RemoveRootActorExpiredLockOwners();
}

void ADisplayClusterRootActor::RemoveRootActorExclusiveLockOwner(const TSharedRef<IDisplayClusterViewportManager, ESPMode::ThreadSafe> InOwner)
{
	FScopeLock Lock(&RootActorExclusiveLockOwnersMutex);
	RootActorExclusiveLockOwners.Remove(InOwner);
	RemoveRootActorExpiredLockOwners();
}

bool ADisplayClusterRootActor::IsRootActorExclusivelyLocked() const
{
	FScopeLock Lock(&RootActorExclusiveLockOwnersMutex);

	if (RootActorExclusiveLockOwners.IsEmpty())
	{
		return false;
	}

	for(const TWeakPtr<IDisplayClusterViewportManager, ESPMode::ThreadSafe>& LockOwnerIt : RootActorExclusiveLockOwners)
	{
		if (LockOwnerIt.IsValid()
		 && LockOwnerIt.Pin().IsValid()
		 && LockOwnerIt.Pin().Get()->GetConfiguration().IsRootActorLockedByThis(this))
		{
			return true;
		}
	}

	return false;
}

void ADisplayClusterRootActor::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	ADisplayClusterRootActor* This = CastChecked<ADisplayClusterRootActor>(InThis);
	if (FDisplayClusterViewportManager* ViewportManager = This ? This->GetViewportManagerImpl() : nullptr)
	{
		ViewportManager->AddReferencedObjects(Collector);
	}

	Super::AddReferencedObjects(InThis, Collector);
}

#define OVERRIDE_PROPERTY(PROPERTY_NAME)\
	if (InPropertyOverrides.PROPERTY_NAME.IsSet())\
	{\
		PROPERTY_NAME = InPropertyOverrides.PROPERTY_NAME.GetValue();\
	}

void ADisplayClusterRootActor::OverrideRootActorProperties(const FDisplayClusterRootActorPropertyOverrides& InPropertyOverrides)
{
	OVERRIDE_PROPERTY(bPreviewInGameEnable);
	OVERRIDE_PROPERTY(bPreviewInGameRenderFrustum);
	OVERRIDE_PROPERTY(bPreviewEnable);
	OVERRIDE_PROPERTY(PreviewRenderTargetRatioMult);
	OVERRIDE_PROPERTY(bPreviewEnablePostProcess);
	OVERRIDE_PROPERTY(bPreviewEnableOCIO);
	OVERRIDE_PROPERTY(bPreviewEnableHoldoutComposite);
	OVERRIDE_PROPERTY(bPreviewEnableTSR);
	OVERRIDE_PROPERTY(bPreviewEnableOverlayMaterial);
	OVERRIDE_PROPERTY(bEnablePreviewTechvis);
	OVERRIDE_PROPERTY(bEnablePreviewMesh);
	OVERRIDE_PROPERTY(bEnablePreviewEditableMesh);
	OVERRIDE_PROPERTY(PreviewSetttingsSource);
	OVERRIDE_PROPERTY(bFreezePreviewRender);
	OVERRIDE_PROPERTY(bPreviewICVFXFrustums);
	OVERRIDE_PROPERTY(PreviewICVFXFrustumsFarDistance);
	OVERRIDE_PROPERTY(TickPerFrame);
	OVERRIDE_PROPERTY(ViewportsPerFrame);
	OVERRIDE_PROPERTY(PreviewMaxTextureDimension);
}

#undef OVERRIDE_PROPERTY
