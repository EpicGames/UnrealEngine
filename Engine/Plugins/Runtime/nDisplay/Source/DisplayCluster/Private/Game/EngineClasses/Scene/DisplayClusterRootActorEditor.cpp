// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterRootActor.h"

#include "Blueprints/DisplayClusterBlueprint.h"
#include "Components/DisplayClusterCameraComponent.h"
#include "Components/DisplayClusterICVFXCameraComponent.h"
#include "Components/DisplayClusterOriginComponent.h"
#include "Components/DisplayClusterSceneComponentSyncParent.h"
#include "Components/DisplayClusterScreenComponent.h"
#include "Components/DisplayClusterStageGeometryComponent.h"
#include "Components/DisplayClusterStageIsosphereComponent.h"
#include "Config/IPDisplayClusterConfigManager.h"
#include "DisplayClusterConfigurationStrings.h"
#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterPlayerInput.h"
#include "DisplayClusterProjectionStrings.h"
#include "IDisplayClusterConfiguration.h"
#include "Misc/DisplayClusterGlobals.h"
#include "Misc/DisplayClusterHelpers.h"
#include "Misc/DisplayClusterLog.h"
#include "Misc/DisplayClusterStrings.h"
#include "Render/Viewport/DisplayClusterViewportManager.h"
#include "Render/Viewport/IDisplayClusterViewport.h"
#include "Render/Viewport/IDisplayClusterViewportManager.h"
#include "Render/Viewport/IDisplayClusterViewportProxy.h"
#include "Render/Viewport/RenderFrame/DisplayClusterRenderFrame.h"
#include "Render/Viewport/RenderFrame/DisplayClusterRenderFrameSettings.h"

#include "Components/MeshComponent.h"
#include "Components/SceneComponent.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/Engine.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/World.h"
#include "GameFramework/PlayerController.h"
#include "Materials/Material.h"
#include "Misc/TransactionObjectEvent.h"
#include "TextureResource.h"
#include "UObject/UnrealType.h"



//////////////////////////////////////////////////////////////////////////////////////////////
// IN-EDITOR STUFF
//////////////////////////////////////////////////////////////////////////////////////////////

#if WITH_EDITOR

#include "IConcertSyncClientModule.h"
#include "IConcertClientWorkspace.h"
#include "IConcertSyncClient.h"
#include "AssetToolsModule.h"

#include "Async/Async.h"
#include "EditorSupportDelegates.h"
#include "LevelEditor.h"

namespace UE::DisplayCluster::RootActor_Editor
{
	/**
	* Propagate map changes from the DefaultsOwner to the InstanceOwner. The map must be <FString, UObject*>.
	* Each instanced object value will also be setup to use default propagation.
	*
	* If the DefaultsOwner has added a key it will be added to all instances with a new UObject value templated from the
	* default value.
	*
	* If the DefaultsOwner has removed a key it will be removed from all instances.
	*
	* All other elements will be left alone and element values will always work through the default propagation system.
	*
	* This is necessary in the event the instance has modified individual element value properties. Default container
	* propagation will treat the entire container as dirty and not add or remove elements to the instance.
	*
	* For nDisplay the instance can only edit element properties, not the container size.
	* This allows us to safely propagate size changes from the default map through a custom propagation system.
	*
	* @param MapProperty The map property to sync.
	* @param DefaultsOwner The direct object owning the map property with the default values.
	* @param InstanceOwner The direct object owning the map property with the instance values.
	*/
	static void PropagateDefaultMapToInstancedMap_Editor(const FMapProperty* MapProperty, UObject* DefaultsOwner, UObject* InstanceOwner)
	{
		check(DefaultsOwner);
		check(InstanceOwner);

		const void* MapContainerDefaults = MapProperty->ContainerPtrToValuePtr<void*>(DefaultsOwner);
		const void* MapContainerInstance = MapProperty->ContainerPtrToValuePtr<void*>(InstanceOwner);
		FScriptMapHelper MapDefaultsHelper(MapProperty, MapContainerDefaults);
		FScriptMapHelper MapInstanceHelper(MapProperty, MapContainerInstance);

		bool bHasChanged = false;

		auto AddKeyWithInstancedObject = [&](const FString* Key, UObject* ArchetypeToUse)
			{
				if (const uint8* ExistingPair = MapInstanceHelper.FindMapPairPtrFromHash(Key))
				{
					MapInstanceHelper.RemovePair(ExistingPair);
				}

				// Existing objects should only occur in the case of delete and undo.
				UObject* ObjectToAdd = FindObject<UObject>(InstanceOwner, *ArchetypeToUse->GetName());
				if (ObjectToAdd == nullptr)
				{
					// Create the instance to assign to the map. Provide the archetype as a template with
					// the same name so properties are propagated automatically.
					ObjectToAdd = NewObject<UObject>(InstanceOwner, ArchetypeToUse->GetClass(),
						ArchetypeToUse->GetFName(), RF_Transactional, ArchetypeToUse);
				}
#if WITH_EDITOR
				else if (GIsTransacting)
				{
					// HACK: Projection policy parameters can become cleared when a VP is deleted, undone, redone, undone.
					// The policy params are meant to only be set from the DCRA CDO, so resetting them back to the VP archetype is safe.
					// @todo figure out how the parameters are being cleared between the redo/undo. No other properties seem to be impacted. 
					if (UDisplayClusterConfigurationViewport* InstanceViewport = Cast<UDisplayClusterConfigurationViewport>(ObjectToAdd))
					{
						const UDisplayClusterConfigurationViewport* ArchetypeViewport = CastChecked<UDisplayClusterConfigurationViewport>(ArchetypeToUse);
						if (InstanceViewport->ProjectionPolicy.Parameters.IsEmpty() && !ArchetypeViewport->ProjectionPolicy.Parameters.IsEmpty())
						{
							UE_LOGF(LogDisplayClusterBlueprint, Warning, "Projection Policy mismatch from archetype on viewport, correcting. Instance: %ls, Archetype: %ls.",
								*ObjectToAdd->GetName(), *ObjectToAdd->GetArchetype()->GetName());

							InstanceViewport->ProjectionPolicy.Parameters = ArchetypeViewport->ProjectionPolicy.Parameters;
						}
					}
				}
#endif

				MapInstanceHelper.AddPair(Key, &ObjectToAdd);

				bHasChanged = true;
			};

		// Look for elements that should be added.
		for (FScriptMapHelper::FIterator DefaultIt = MapDefaultsHelper.CreateIterator(); DefaultIt; ++DefaultIt)
		{
			uint8* DefaultPairPtr = MapDefaultsHelper.GetPairPtr(DefaultIt);
			check(DefaultPairPtr);

			FString* Key = MapProperty->KeyProp->ContainerPtrToValuePtr<FString>(DefaultPairPtr);
			check(Key);

			UObject** DefaultObjectPtr = MapProperty->ValueProp->ContainerPtrToValuePtr<UObject*>(DefaultPairPtr);
			check(DefaultObjectPtr);

			UObject* DefaultObject = *DefaultObjectPtr;
			check(DefaultObject);

			if (UObject** InstancedObjectPtr = (UObject**)MapInstanceHelper.FindValueFromHash(Key))
			{
				if (const UObject* InstancedObject = *InstancedObjectPtr)
				{
					const bool bArchetypeCorrect = DefaultObject == InstancedObject->GetArchetype();

					// The archetype should always match the new default. There are edge cases with MU
					// where the instance may be updated prior to the CDO and should be corrected once
					// the BP is saved. If this occurs outside of MU there could be a serious problem.
					if (!bArchetypeCorrect)
					{
						const ADisplayClusterRootActor* RootActor =
							Cast<ADisplayClusterRootActor>(InstancedObject->GetTypedOuter(ADisplayClusterRootActor::StaticClass()));
						if (!RootActor)
						{
							// Undo transactions can potentially trigger this while an object was renamed to the transient package.
							check(InstancedObject->GetPackage() == GetTransientPackage());
							continue;
						}
#if WITH_EDITOR
						if (GEditor)
						{
							bool bIsMultiUserSession = false;
							TSharedPtr<IConcertSyncClient> ConcertSyncClient = IConcertSyncClientModule::Get().GetClient(TEXT("MultiUser"));
							if (ConcertSyncClient.IsValid())
							{
								TSharedPtr<IConcertClientWorkspace> Workspace = ConcertSyncClient->GetWorkspace();
								bIsMultiUserSession = Workspace.IsValid();
							}

							ensure(bArchetypeCorrect || bIsMultiUserSession);
						}
#endif
						UE_LOGF(LogDisplayClusterBlueprint, Warning, "Archetype mismatch on nDisplay config %ls. Make sure the config is compiled and saved. Property: %ls, Instance: %ls, Archetype: %ls, Default: %ls.",
							*RootActor->GetName(), *MapProperty->GetName(), *InstancedObject->GetName(), *InstancedObject->GetArchetype()->GetName(), *DefaultObject->GetName());
					}
					continue;
				}
			}

			AddKeyWithInstancedObject(Key, DefaultObject);
		}

		// Look for elements that should be removed.
		for (FScriptMapHelper::FIterator InstanceIt = MapInstanceHelper.CreateIterator(); InstanceIt; ++InstanceIt)
		{
			uint8* InstancePairPtr = MapInstanceHelper.GetPairPtr(InstanceIt);
			check(InstancePairPtr);

			FString* Key = MapProperty->KeyProp->ContainerPtrToValuePtr<FString>(InstancePairPtr);
			check(Key);

			if (!MapDefaultsHelper.FindValueFromHash(Key))
			{
				if (UObject** InstanceObjectPtr = MapProperty->ValueProp->ContainerPtrToValuePtr<UObject*>(InstancePairPtr))
				{
					if (UObject* InstanceObject = *InstanceObjectPtr)
					{
						// Trash the object -- default propagation won't handle this any more.
						// RemoveAt below will remove the reference to it. This transaction can still be undone.
						// Rename to transient package now so the same name is available immediately.
						// It's possible a new object needs to be created with this outer using the same name.
						InstanceObject->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors);
						InstanceObject->SetFlags(RF_Transient);
					}
				}

				MapInstanceHelper.RemoveAt(InstanceIt.GetInternalIndex());
				bHasChanged = true;
			}
		}

		if (bHasChanged)
		{
			MapInstanceHelper.Rehash();
#if WITH_EDITOR
			InstanceOwner->PostEditChange();
#endif
		}
	}

	/**
	 * Syncs default config data changes to a config instance.
	 *
	 * @param InDefaultConfigData The class default config data object.
	 * @param InInstanceConfigData An instance config data object.
	 */
	static void PropagateDataFromDefaultConfig_Editor(UDisplayClusterConfigurationData* InDefaultConfigData, UDisplayClusterConfigurationData* InInstanceConfigData)
	{
		check(InDefaultConfigData);
		check(InInstanceConfigData);

		const FMapProperty* ClusterNodesMapProperty = FindFieldChecked<FMapProperty>(UDisplayClusterConfigurationCluster::StaticClass(),
			GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationCluster, Nodes));
		const FMapProperty* ViewportsMapProperty = FindFieldChecked<FMapProperty>(UDisplayClusterConfigurationClusterNode::StaticClass(),
			GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationClusterNode, Viewports));

		PropagateDefaultMapToInstancedMap_Editor(ClusterNodesMapProperty, InDefaultConfigData->Cluster, InInstanceConfigData->Cluster);

		for (const TTuple<FString, TObjectPtr<UDisplayClusterConfigurationClusterNode>>& ClusterKeyVal : InDefaultConfigData->Cluster->Nodes)
		{
			UDisplayClusterConfigurationClusterNode* DestinationValue = InInstanceConfigData->Cluster->Nodes.FindChecked(
				ClusterKeyVal.Key);
			PropagateDefaultMapToInstancedMap_Editor(ViewportsMapProperty, ClusterKeyVal.Value, DestinationValue);
		}
	}


	/**
	 * Syncs default media settings to instance.
	 *
	 * @param InstanceDCRA An instance of the nDisplay root actor.
	 */
	static void PropagateMediaSettingsFromDefault_Editor(ADisplayClusterRootActor* InstanceDCRA)
	{
		if (!InstanceDCRA)
		{
			return;
		}

		// Propagate ICVFX Camera Component Media property
		//
		{
			TArray<UDisplayClusterICVFXCameraComponent*> InstanceCameras;
			InstanceDCRA->GetComponents<UDisplayClusterICVFXCameraComponent>(InstanceCameras);

			for (UDisplayClusterICVFXCameraComponent* InstanceCamera : InstanceCameras)
			{
				if (!InstanceCamera)
				{
					continue;
				}

				const UDisplayClusterICVFXCameraComponent* CameraArchetype = Cast<UDisplayClusterICVFXCameraComponent>(InstanceCamera->GetArchetype());

				if (!CameraArchetype)
				{
					continue;
				}

				// We do not expect to ever pass an archetype instance to this function.
				check(InstanceCamera != CameraArchetype);

				// Deep-copy the media settings from the ICVFX Camera archetype to the instance.

				const FProperty* MediaProperty = FindFProperty<FProperty>(
					FDisplayClusterConfigurationICVFX_CameraRenderSettings::StaticStruct(), 
					GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_CameraRenderSettings, Media)
				);

				check(MediaProperty);

				void* MediaInstancePtr = &InstanceCamera->CameraSettings.RenderSettings.Media;
				const void* MediaArchetypePtr = &CameraArchetype->CameraSettings.RenderSettings.Media;

				// Perform a deep copy of the Media property
				MediaProperty->CopyCompleteValue(MediaInstancePtr /*Dest*/, MediaArchetypePtr /*Src*/);
			}
		}

		// Propagate Backbuffer and Viewports MediaSettings property
		//

		const UDisplayClusterBlueprint* DCBP = Cast<UDisplayClusterBlueprint>(UBlueprint::GetBlueprintFromClass(InstanceDCRA->GetClass()));

		if (!DCBP)
		{
			return;
		}

		const UDisplayClusterConfigurationData* const ArchetypeConfig = DCBP->GetConfig();
		const UDisplayClusterConfigurationData* const InstanceConfig = InstanceDCRA->GetConfigData();

		if (!IsValid(ArchetypeConfig)
			|| !IsValid(InstanceConfig)
			|| !IsValid(ArchetypeConfig->Cluster)
			|| !IsValid(InstanceConfig->Cluster))
		{
			return;
		}

		// Iterate over instance nodes, find the archetype, and deep-copy the Media setting.
		for (auto ItNode = InstanceConfig->Cluster->Nodes.CreateIterator(); ItNode; ++ItNode)
		{
			const FString& NodeName = ItNode.Key();
			TObjectPtr<UDisplayClusterConfigurationClusterNode> InstanceNode = ItNode.Value();

			if (!InstanceNode)
			{
				continue;
			}

			TObjectPtr<UDisplayClusterConfigurationClusterNode>* ArchetypeNodePtr = ArchetypeConfig->Cluster->Nodes.Find(NodeName);

			if (!ArchetypeNodePtr)
			{
				continue;
			}

			TObjectPtr<UDisplayClusterConfigurationClusterNode> ArchetypeNode = *ArchetypeNodePtr;

			if (!ArchetypeNode)
			{
				continue;
			}

			// Get the property so that we can use its CopyCompleteValue function for the deep copy.

			const FProperty* NodeMediaProperty = FindFProperty<FProperty>(
				UDisplayClusterConfigurationClusterNode::StaticClass(),
				GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationClusterNode, MediaSettings)
			);

			check(NodeMediaProperty);

			// Get the pointers to the property values on instance and archetype
			void* NodeMediaInstancePtr = &InstanceNode->MediaSettings;
			const void* NodeMediaArchetypePtr = &ArchetypeNode->MediaSettings;

			// Perform a deep copy of the Media property
			NodeMediaProperty->CopyCompleteValue(NodeMediaInstancePtr /*Dest*/, NodeMediaArchetypePtr /*Src*/);

			// Viewports Media property
			//

			// Iterate over instance viewports, find the archetype, and deep-copy the Media setting.
			for (auto ItViewport = InstanceNode->Viewports.CreateIterator(); ItViewport; ++ItViewport)
			{
				const FString& ViewportName = ItViewport.Key();
				TObjectPtr<UDisplayClusterConfigurationViewport> InstanceViewport = ItViewport.Value();

				if (!InstanceViewport)
				{
					continue;
				}

				TObjectPtr<UDisplayClusterConfigurationViewport>* ArchetypeViewportPtr = ArchetypeNode->Viewports.Find(ViewportName);

				if (!ArchetypeViewportPtr)
				{
					continue;
				}

				TObjectPtr<UDisplayClusterConfigurationViewport> ArchetypeViewport = *ArchetypeViewportPtr;

				if (!ArchetypeViewport)
				{
					continue;
				}
				
				// Get the property so that we can use its CopyCompleteValue function for the deep copy.
				
				const FProperty* ViewportMediaProperty = FindFProperty<FProperty>(
					FDisplayClusterConfigurationViewport_RenderSettings::StaticStruct(),
					GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_RenderSettings, Media)
				);

				check(ViewportMediaProperty);

				// Get the pointers to the property values on instance and archetype
				void* ViewportMediaInstancePtr = &InstanceViewport->RenderSettings.Media;
				const void* ViewportMediaArchetypePtr = &ArchetypeViewport->RenderSettings.Media;

				// Perform a deep copy of the Media property
				ViewportMediaProperty->CopyCompleteValue(ViewportMediaInstancePtr /*Dest*/, ViewportMediaArchetypePtr /*Src*/);
			}
		}

	}
};


//////////////////////////////////////////////////////////////////////////////////////////////
// ADisplayClusterRootActor
//////////////////////////////////////////////////////////////////////////////////////////////

void ADisplayClusterRootActor::RerunConstructionScripts()
{
	using namespace UE::DisplayCluster::RootActor_Editor;

	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("ADisplayClusterRootActor::RerunConstructionScripts"), STAT_RerunConstructionScripts, STATGROUP_NDisplay);

	const IDisplayClusterConfiguration& Config = IDisplayClusterConfiguration::Get();
	if (!Config.IsTransactingSnapshot())
	{
		Super::RerunConstructionScripts();

		if (!IsTemplate())
		{
			if (UDisplayClusterConfigurationData* CurrentData = GetConfigData())
			{
				const ADisplayClusterRootActor* CDO = CastChecked<ADisplayClusterRootActor>(GetClass()->GetDefaultObject());
				UDisplayClusterConfigurationData* DefaultData = CDO->GetConfigData();
				PropagateDataFromDefaultConfig_Editor(DefaultData, CurrentData);
			}

			// We propagate the media settings to the instances to keep them in sync.
			// This is not strictly necessary since the custom serializers make sure the intances always
			// load archetype defaults but this makes it so they are always in sync even when
			// the archetype is being edited, since the instances get reconstructed but not re-deserialized.
			PropagateMediaSettingsFromDefault_Editor(this);
		}
		RerunConstructionScripts_Editor();
	}
}

void ADisplayClusterRootActor::Constructor_Editor()
{
	// Allow tick in editor for preview rendering
	PrimaryActorTick.bStartWithTickEnabled = true;

	FCoreUObjectDelegates::OnPackageReloaded.AddUObject(this, &ADisplayClusterRootActor::HandleAssetReload);

	if (GEditor)
	{
		GEditor->OnEndObjectMovement().AddUObject(this, &ADisplayClusterRootActor::OnEndObjectMovement);
	}
}

void ADisplayClusterRootActor::Destructor_Editor()
{
	FCoreUObjectDelegates::OnPackageReloaded.RemoveAll(this);

	if (GEditor)
	{
		GEditor->OnEndObjectMovement().RemoveAll(this);
	}
}

void ADisplayClusterRootActor::PostActorCreated_Editor()
{
	ResetEntireClusterPreviewRendering();
}

void ADisplayClusterRootActor::PostLoad_Editor()
{
	ResetEntireClusterPreviewRendering();
}

void ADisplayClusterRootActor::EndPlay_Editor(const EEndPlayReason::Type EndPlayReason)
{
}

void ADisplayClusterRootActor::Destroyed_Editor()
{
	ResetEntireClusterPreviewRendering();

	MarkAsGarbage();
}

void ADisplayClusterRootActor::BeginDestroy_Editor()
{
	ResetEntireClusterPreviewRendering();
}

void ADisplayClusterRootActor::RerunConstructionScripts_Editor()
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("ADisplayClusterRootActor::RerunConstructionScripts_Editor"), STAT_RerunConstructionScripts_Editor, STATGROUP_NDisplay);
	
	/* We need to reinitialize since our components are being regenerated here. */
	InitializeRootActor();

	UpdateInnerFrustumPriority();
}

void ADisplayClusterRootActor::UpdateInnerFrustumPriority()
{
	if (InnerFrustumPriority.Num() == 0)
	{
		ResetInnerFrustumPriority();
		return;
	}

	TArray<UDisplayClusterICVFXCameraComponent*> Components;
	GetComponents(Components);

	TArray<FString> ValidCameras;
	for (UDisplayClusterICVFXCameraComponent* Camera : Components)
	{
		FString CameraName = Camera->GetName();
		InnerFrustumPriority.AddUnique(CameraName);
		ValidCameras.Add(CameraName);
	}

	// Removes invalid cameras or duplicate cameras.
	InnerFrustumPriority.RemoveAll([ValidCameras, this](const FDisplayClusterComponentRef& CameraRef)
	{
		return !ValidCameras.Contains(CameraRef.Name) || InnerFrustumPriority.FilterByPredicate([CameraRef](const FDisplayClusterComponentRef& CameraRefN2)
		{
			return CameraRef == CameraRefN2;
		}).Num() > 1;
	});

	for (int32 Idx = 0; Idx < InnerFrustumPriority.Num(); ++Idx)
	{
		if (UDisplayClusterICVFXCameraComponent* CameraComponent = FindObjectFast<UDisplayClusterICVFXCameraComponent>(this, *InnerFrustumPriority[Idx].Name))
		{
			CameraComponent->CameraSettings.RenderSettings.RenderOrder = Idx;
		}
	}
}

void ADisplayClusterRootActor::ResetInnerFrustumPriority()
{
	TArray<UDisplayClusterICVFXCameraComponent*> Components;
	GetComponents(Components);

	InnerFrustumPriority.Reset(Components.Num());
	for (UDisplayClusterICVFXCameraComponent* Camera : Components)
	{
		InnerFrustumPriority.Add(Camera->GetName());
	}

	// Initialize based on current render priority.
	InnerFrustumPriority.Sort([this](const FDisplayClusterComponentRef& CameraA, const FDisplayClusterComponentRef& CameraB)
	{
		UDisplayClusterICVFXCameraComponent* CameraComponentA = FindObjectFast<UDisplayClusterICVFXCameraComponent>(this, *CameraA.Name);
		UDisplayClusterICVFXCameraComponent* CameraComponentB = FindObjectFast<UDisplayClusterICVFXCameraComponent>(this, *CameraB.Name);

		if (CameraComponentA && CameraComponentB)
		{
			return CameraComponentA->CameraSettings.RenderSettings.RenderOrder < CameraComponentB->CameraSettings.RenderSettings.RenderOrder;
		}

		return false;
	});
}

bool ADisplayClusterRootActor::IsSelectedInEditor() const
{
	return bIsSelectedInEditor || Super::IsSelectedInEditor();
}

void ADisplayClusterRootActor::SetIsSelectedInEditor(bool bValue)
{
	bIsSelectedInEditor = bValue;
}

void ADisplayClusterRootActor::PostTransacted(const FTransactionObjectEvent& TransactionEvent)
{
	if (bRequiresComponentRefresh
		&& TransactionEvent.GetEventType() == ETransactionObjectEventType::Finalized)
	{
		if ((GEditor && GEditor->bIsSimulatingInEditor && GetWorld() != nullptr) || ReregisterComponentsWhenModified())
		{
			UnregisterAllComponents();
			ReregisterAllComponents();
		}
		bRequiresComponentRefresh = false;
	}
	Super::PostTransacted(TransactionEvent);
}

void ADisplayClusterRootActor::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChainEvent)
{
	const FProperty* RootProperty = PropertyChainEvent.PropertyChain.GetActiveNode()->GetValue();
	const FProperty* TailProperty = PropertyChainEvent.PropertyChain.GetTail()->GetValue();
	if (PropertyChainEvent.ChangeType == EPropertyChangeType::Interactive && RootProperty && TailProperty)
	{
		const FName RootPropertyName = RootProperty->GetFName();
		const FName TailPropertyName = TailProperty->GetFName();
		if (RootPropertyName == GET_MEMBER_NAME_CHECKED(ADisplayClusterRootActor, CurrentConfigData)
			&& RootPropertyName != TailPropertyName)
		{
			// Do not propagate the PostEditChangeProperty because we are in an interactive edit of a suboject
			// and we do not need to rerun any construction scripts.
			bIsInteractiveEditingSubobject = true;
			bRequiresComponentRefresh = true;
		}
	}

	Super::PostEditChangeChainProperty(PropertyChainEvent);
	bIsInteractiveEditingSubobject = false;
}

void ADisplayClusterRootActor::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = (PropertyChangedEvent.Property != nullptr) ? PropertyChangedEvent.Property->GetFName() : NAME_None;

	if (bIsInteractiveEditingSubobject)
	{
		return;
	}

	static FName Name_RelativeLocation = USceneComponent::GetRelativeLocationPropertyName();
	static FName Name_RelativeRotation = USceneComponent::GetRelativeRotationPropertyName();
	static FName Name_RelativeScale3D = USceneComponent::GetRelativeScale3DPropertyName();

	// The AActor method, simplified and modified to skip construction scripts.
	// Component registration still needs to occur or the actor will look like it disappeared.
	auto SuperCallWithoutConstructionScripts = [&]
	{
		if (IsPropertyChangedAffectingDataLayers(PropertyChangedEvent))
		{
			FixupDataLayers(/*bRevertChangesOnLockedDataLayer*/true);
		}

		const bool bTransformationChanged = (PropertyName == Name_RelativeLocation || PropertyName == Name_RelativeRotation || PropertyName == Name_RelativeScale3D);

		if ((GEditor && GEditor->bIsSimulatingInEditor && GetWorld() != nullptr) || ReregisterComponentsWhenModified())
		{
			// If a transaction is occurring we rely on the true parent method instead.
			check(!CurrentTransactionAnnotation.IsValid());

			UnregisterAllComponents();
			ReregisterAllComponents();
		}

		// Let other systems know that an actor was moved
		if (bTransformationChanged)
		{
			GEngine->BroadcastOnActorMoved(this);
		}

		FEditorSupportDelegates::UpdateUI.Broadcast();
		UObject::PostEditChangeProperty(PropertyChangedEvent);
	};
	
	bool bResetInnerFrustumPriority = false;
	bool bResetEntireClusterPreviewRendering = false;

	// Config file has been changed, we should rebuild the nDisplay hierarchy
	// Cluster node ID has been changed
	bool bReinitializeActor = true;
	if (PropertyName == GET_MEMBER_NAME_CHECKED(ADisplayClusterRootActor, bFreezePreviewRender)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(ADisplayClusterRootActor, PreviewRenderTargetRatioMult)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(ADisplayClusterRootActor, bPreviewICVFXFrustums)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(ADisplayClusterRootActor, PreviewICVFXFrustumsFarDistance)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(ADisplayClusterRootActor, PreviewMaxTextureDimension)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(ADisplayClusterRootActor, PreviewNodeId)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(ADisplayClusterRootActor, RenderMode))
	{
		bReinitializeActor = false;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(ADisplayClusterRootActor, TickPerFrame)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(ADisplayClusterRootActor, ViewportsPerFrame)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(ADisplayClusterRootActor, bPreviewEnablePostProcess)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(ADisplayClusterRootActor, bPreviewEnableHoldoutComposite)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(ADisplayClusterRootActor, DefaultDisplayDeviceName))
	{
		bResetEntireClusterPreviewRendering = true;

		bReinitializeActor = false;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(ADisplayClusterRootActor, InnerFrustumPriority))
	{
		bResetInnerFrustumPriority = true;
	}

	// Changes inside CurrentConfigData never alter DCRA component topology, so we always skip construction scripts for them,
	// independent of the Blueprint's bRunConstructionScriptOnInteractiveChange flag. We still need to rebuild the nDisplay
	// hierarchy afterwards (handled below via bReinitializeActor).
	const bool bChangeInCurrentConfigData = PropertyName == GET_MEMBER_NAME_CHECKED(ADisplayClusterRootActor, CurrentConfigData);

	// For non-config interactive edits (e.g. dragging a slider), skipping construction scripts is opt-in per Blueprint.
	bool bCanSkipForInteractive = false;
	if (PropertyChangedEvent.ChangeType == EPropertyChangeType::Interactive)
	{
		if (const UDisplayClusterBlueprint* Blueprint = Cast<UDisplayClusterBlueprint>(UBlueprint::GetBlueprintFromClass(GetClass())))
		{
			bCanSkipForInteractive = !Blueprint->bRunConstructionScriptOnInteractiveChange;
		}
	}

	if (!CurrentTransactionAnnotation.IsValid() && (bChangeInCurrentConfigData || bCanSkipForInteractive))
	{
		SuperCallWithoutConstructionScripts();
	}
	else
	{
		// runs CS
		Super::PostEditChangeProperty(PropertyChangedEvent);

		// InitializeRootActor() already called in PostEditChangeProperty()->RerunConstructionScripts()->RerunConstructionScripts_Editor()
		bReinitializeActor = false;
	}

	if (bResetEntireClusterPreviewRendering)
	{
		ResetEntireClusterPreviewRendering();
	}

	if (bResetInnerFrustumPriority)
	{
		ResetInnerFrustumPriority();
	}

	if (bReinitializeActor)
	{
		InitializeRootActor();
	}
}

void ADisplayClusterRootActor::PostEditMove(bool bFinished)
{
	// Don't update the preview with the config data if we're just moving the actor.
	Super::PostEditMove(bFinished);
}

void ADisplayClusterRootActor::HandleAssetReload(const EPackageReloadPhase InPackageReloadPhase,
	FPackageReloadedEvent* InPackageReloadedEvent)
{
	if (InPackageReloadPhase == EPackageReloadPhase::PrePackageFixup)
	{
		// Preview components need to be released here. During a package reload BeginDestroy will be called on the
		// actor, but the preview components are already detached and never have DestroyComponent called on them.
		// This causes the package to keep references around and cause a crash during the reload.
		BeginDestroy_Editor();
	}
}

void ADisplayClusterRootActor::OnEndObjectMovement(UObject& InObject)
{
	// If any of this stage actor's components have been moved, invalidate the stage geometry map
	if (USceneComponent* SceneComponent = Cast<USceneComponent>(&InObject))
	{
		if (SceneComponent->GetOwner() == this)
		{
			// Check to see if the object being moved is a part of the stage's geometry or a view point
			TArray<FString> ProjectionMeshNames;
			if (UDisplayClusterConfigurationData* Config = GetConfigData())
			{
				Config->GetReferencedMeshNames(ProjectionMeshNames);
			}

			const bool bIsProjectionMesh = SceneComponent->IsA<UStaticMeshComponent>() && ProjectionMeshNames.Contains(SceneComponent->GetName());
			const bool bIsScreen = SceneComponent->IsA<UDisplayClusterScreenComponent>();
			const bool bIsViewPoint = SceneComponent->IsA<UDisplayClusterCameraComponent>();

			if (bIsProjectionMesh || bIsScreen || bIsViewPoint)
			{
				StageGeometryComponent->Invalidate();
			}
		}
	}
}

#endif
