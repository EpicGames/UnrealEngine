// Copyright Epic Games, Inc. All Rights Reserved.

#include "Game/EngineClasses/Scene/DisplayClusterRootActorInitializer.h"

#include "DisplayClusterRootActor.h"
#include "Components/DisplayClusterCameraComponent.h"
#include "Components/DisplayClusterScreenComponent.h"
#include "Components/DisplayClusterXformComponent.h"

#include "DisplayClusterConfigurationTypes.h"

#include "Blueprints/DisplayClusterBlueprintGeneratedClass.h"
#include "Misc/DisplayClusterLog.h"


void FDisplayClusterRootActorInitializer::Initialize(ADisplayClusterRootActor* InActor, UDisplayClusterConfigurationData* ConfigData)
{
	if (!ConfigData)
	{
		return;
	}

	// Cleanup in case of reinitialization
	Cleanup();

	// Save actor instance
	ProcessingActor = InActor;

	// Create all components from the config
	SpawnComponents<UDisplayClusterXformComponent,  UDisplayClusterConfigurationSceneComponentXform>(ConfigData->Scene->Xforms,   XformComponents);
	SpawnComponents<UDisplayClusterCameraComponent, UDisplayClusterConfigurationSceneComponentCamera>(ConfigData->Scene->Cameras, CameraComponents);
	SpawnComponents<UDisplayClusterScreenComponent, UDisplayClusterConfigurationSceneComponentScreen>(ConfigData->Scene->Screens, ScreenComponents);

	if (ProcessingActor->GetWorld())
	{
		// Reregister only needed during an active world. This could be assembled during new asset import when there isn't a world.
		ProcessingActor->ReregisterAllComponents();
	}

	// Initialize the components with config data
	ApplyConfigDataToComponents();

	// Cleanup
	Cleanup();
}

void FDisplayClusterRootActorInitializer::UpdateComponentTransformsOnly(ADisplayClusterRootActor* Actor, UDisplayClusterConfigurationData* ConfigData)
{
	if (!(Actor && ConfigData))
	{
		return;
	}

	// Xform components
	for (const TPair<FString, TObjectPtr<UDisplayClusterConfigurationSceneComponentXform>>& Xform : ConfigData->Scene->Xforms)
	{
		if (USceneComponent* const ActorComponent = Actor->GetComponentByName<USceneComponent>(Xform.Key))
		{
			ActorComponent->SetRelativeLocationAndRotation(Xform.Value->Location, Xform.Value->Rotation);
		}
	}

	// Camera components
	for (const TPair<FString, TObjectPtr<UDisplayClusterConfigurationSceneComponentCamera>>& Camera : ConfigData->Scene->Cameras)
	{
		if (USceneComponent* const ActorComponent = Actor->GetComponentByName<USceneComponent>(Camera.Key))
		{
			ActorComponent->SetRelativeLocationAndRotation(Camera.Value->Location, Camera.Value->Rotation);
		}
	}

	// Screen components
	for (const TPair<FString, TObjectPtr<UDisplayClusterConfigurationSceneComponentScreen>>& Screen : ConfigData->Scene->Screens)
	{
		if (USceneComponent* const ActorComponent = Actor->GetComponentByName<USceneComponent>(Screen.Key))
		{
			ActorComponent->SetRelativeLocationAndRotation(Screen.Value->Location, Screen.Value->Rotation);
		}
	}
}

void FDisplayClusterRootActorInitializer::ApplyConfigDataToComponents()
{
	// Configure xform components
	for (const TPair<FString, FCompInitInfo<UDisplayClusterXformComponent, UDisplayClusterConfigurationSceneComponentXform>>& Xform : XformComponents)
	{
		if (!DoesComponentBelongToBlueprint(Xform.Value.Component))
		{
			ConfigureXformComponent(Xform.Value.Component, Xform.Value.Config);
		}
	}

	// Configure camera components
	for (const TPair<FString, FCompInitInfo<UDisplayClusterCameraComponent, UDisplayClusterConfigurationSceneComponentCamera>>& Camera : CameraComponents)
	{
		if (!DoesComponentBelongToBlueprint(Camera.Value.Component))
		{
			ConfigureCameraComponent(Camera.Value.Component, Camera.Value.Config);
		}
	}

	// Configure screen components
	for (const TPair<FString, FCompInitInfo<UDisplayClusterScreenComponent, UDisplayClusterConfigurationSceneComponentScreen>>& Screen : ScreenComponents)
	{
		if (!DoesComponentBelongToBlueprint(Screen.Value.Component))
		{
			ConfigureScreenComponent(Screen.Value.Component, Screen.Value.Config);
		}
	}
}

void FDisplayClusterRootActorInitializer::ConfigureCameraComponent(UDisplayClusterCameraComponent* Component, UDisplayClusterConfigurationSceneComponentCamera* ComponentCfg)
{
	// Set camera specific parameters
	Component->SetInterpupillaryDistance(ComponentCfg->InterpupillaryDistance);
	Component->SetSwapEyes(ComponentCfg->bSwapEyes);

	switch (ComponentCfg->StereoOffset)
	{
	case EDisplayClusterConfigurationEyeStereoOffset::Left:
		Component->SetStereoOffset(EDisplayClusterEyeStereoOffset::Left);
		break;

	case EDisplayClusterConfigurationEyeStereoOffset::None:
		Component->SetStereoOffset(EDisplayClusterEyeStereoOffset::None);
		break;

	case EDisplayClusterConfigurationEyeStereoOffset::Right:
		Component->SetStereoOffset(EDisplayClusterEyeStereoOffset::Right);
		break;

	default:
		Component->SetStereoOffset(EDisplayClusterEyeStereoOffset::None);
		break;
	}

	// Set generic xform related parameters
	ConfigureComponentHierarchy(Component, ComponentCfg);
}

void FDisplayClusterRootActorInitializer::ConfigureScreenComponent(UDisplayClusterScreenComponent* Component, UDisplayClusterConfigurationSceneComponentScreen* ComponentCfg)
{
	// Set screen specific parameters
	Component->SetScreenSize(ComponentCfg->Size);
	// Set generic xform related parameters
	ConfigureComponentHierarchy(Component, ComponentCfg);
}

void FDisplayClusterRootActorInitializer::ConfigureXformComponent(UDisplayClusterXformComponent* Component, UDisplayClusterConfigurationSceneComponentXform* ComponentCfg)
{
	// Set generic xform related parameters
	ConfigureComponentHierarchy(Component, ComponentCfg);
}

void FDisplayClusterRootActorInitializer::ConfigureComponentHierarchy(USceneComponent* Component, UDisplayClusterConfigurationSceneComponent* ComponentCfg)
{
	// Take place in the components hierarchy
	if (!ComponentCfg->ParentId.IsEmpty())
	{
		USceneComponent* const ParentComp = ProcessingActor->GetComponentByName<USceneComponent>(ComponentCfg->ParentId);
		if (ParentComp)
		{
			UE_LOGF(LogDisplayClusterGame, Log, "Attaching %ls to %ls", *Component->GetName(), *ParentComp->GetName());
			Component->AttachToComponent(ParentComp, FAttachmentTransformRules(EAttachmentRule::KeepRelative, false));
		}
		else
		{
			UE_LOGF(LogDisplayClusterGame, Warning, "Couldn't attach %ls to %ls", *Component->GetName(), *ComponentCfg->ParentId);
		}
	}

	// Set up location and rotation
	Component->SetRelativeLocationAndRotation(ComponentCfg->Location, ComponentCfg->Rotation);
}

bool FDisplayClusterRootActorInitializer::DoesComponentBelongToBlueprint(UActorComponent* Component) const
{
	bool bIsForBlueprint = false;
	if (ProcessingActor)
	{
		if (ProcessingActor->IsBlueprint())
		{
			bIsForBlueprint = true;
		}
	}
	else if (Component->GetTypedOuter<UDisplayClusterBlueprintGeneratedClass>() != nullptr)
	{
		bIsForBlueprint = true;
	}

	return bIsForBlueprint;
}

void FDisplayClusterRootActorInitializer::Cleanup()
{
	ProcessingActor = nullptr;

	AllComponents.Reset();
	XformComponents.Reset();
	CameraComponents.Reset();
	ScreenComponents.Reset();
}

template <typename TComp, typename TCfgData, typename TCfgDataPtr>
void FDisplayClusterRootActorInitializer::SpawnComponents(const TMap<FString, TCfgDataPtr>& InConfigData, TMap<FString, FCompInitInfo<TComp, TCfgData>>& OutTypedMap)
{
	// Iterate over all the config items
	for (const TPair<FString, TCfgDataPtr>& Item : InConfigData)
	{
		// Just in case, ignore components with duplicate names
		if (!AllComponents.Contains(Item.Key))
		{
			// Check if a component with the same name is already stored on the BP.
			UActorComponent* const ExistingComponent = ProcessingActor->GetComponentByName<UActorComponent>(Item.Key);
			if (!ExistingComponent)
			{
				// Create new component
				TComp* NewComponent = NewObject<TComp>(ProcessingActor, FName(*Item.Key));
#if !WITH_EDITOR
				NewComponent->SetFlags(EObjectFlags::RF_DuplicateTransient | RF_Transient | RF_TextExportTransient);
#endif
				NewComponent->AttachToComponent(ProcessingActor->GetRootComponent(), FAttachmentTransformRules(EAttachmentRule::KeepRelative, false));

				// Save for the next initialization step
				AllComponents.Emplace(Item.Key, NewComponent);
				OutTypedMap.Emplace(Item.Key, FCompInitInfo<TComp, TCfgData>(NewComponent, ToRawPtr(Item.Value)));
			}
		}
	}
}
