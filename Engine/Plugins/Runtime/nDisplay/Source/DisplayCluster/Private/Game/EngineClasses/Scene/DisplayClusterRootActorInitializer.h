// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class ADisplayClusterRootActor;
class UActorComponent;
class USceneComponent;
class UDisplayClusterCameraComponent;
class UDisplayClusterScreenComponent;
class UDisplayClusterXformComponent;
class UDisplayClusterConfigurationData;
class UDisplayClusterConfigurationSceneComponent;
class UDisplayClusterConfigurationSceneComponentXform;
class UDisplayClusterConfigurationSceneComponentCamera;
class UDisplayClusterConfigurationSceneComponentScreen;


/**
 * This is a temporary class that is responsible for DCRA components initialization. In the future,
 * we won't need it since the initialization will be performed in a native way in the DC configurator.
 */
class FDisplayClusterRootActorInitializer
{
public:
	FDisplayClusterRootActorInitializer()
		: ProcessingActor(nullptr)
	{ }

	~FDisplayClusterRootActorInitializer() = default;

public:
	// Main initialization function that manages the whole initialization process
	void Initialize(ADisplayClusterRootActor* Actor, UDisplayClusterConfigurationData* ConfigData);

	// Updates component transforms only based on the config settings
	void UpdateComponentTransformsOnly(ADisplayClusterRootActor* Actor, UDisplayClusterConfigurationData* ConfigData);

private:
	// Applies configuration data to the corresponding components
	void ApplyConfigDataToComponents();

	// Helpers to configure component instances depending on their type
	void ConfigureCameraComponent(UDisplayClusterCameraComponent* Component, UDisplayClusterConfigurationSceneComponentCamera* ComponentCfg);
	void ConfigureScreenComponent(UDisplayClusterScreenComponent* Component, UDisplayClusterConfigurationSceneComponentScreen* ComponentCfg);
	void ConfigureXformComponent(UDisplayClusterXformComponent* Component, UDisplayClusterConfigurationSceneComponentXform* ComponentCfg);
	void ConfigureComponentHierarchy(USceneComponent* Component, UDisplayClusterConfigurationSceneComponent* ComponentCfg);

	// Checks if specified component belongs to the DCRA blueprint
	bool DoesComponentBelongToBlueprint(UActorComponent* Component) const;

	// Cleanup internall temporary stuff
	void Cleanup();

private:
	// Helper struct to bind a concrete component to its concrete config data type
	template <typename TComp, typename TCfgData>
	struct FCompInitInfo
	{
		FCompInitInfo(TComp* Comp, TCfgData* CfgData)
			: Component(Comp)
			, Config(CfgData)
		{ }

		TComp* Component;
		TCfgData* Config;
	};

	// Spawns components listed in a config
	template <typename TComp, typename TCfgData, typename TCfgDataPtr>
	void SpawnComponents(const TMap<FString, TCfgDataPtr>& InConfigData, TMap<FString, FCompInitInfo<TComp, TCfgData>>& OutTypedMap);

	// Internal component references
	ADisplayClusterRootActor* ProcessingActor;
	TMap<FString, USceneComponent*> AllComponents;
	TMap<FString, FCompInitInfo<UDisplayClusterXformComponent,  UDisplayClusterConfigurationSceneComponentXform>>  XformComponents;
	TMap<FString, FCompInitInfo<UDisplayClusterCameraComponent, UDisplayClusterConfigurationSceneComponentCamera>> CameraComponents;
	TMap<FString, FCompInitInfo<UDisplayClusterScreenComponent, UDisplayClusterConfigurationSceneComponentScreen>> ScreenComponents;
};
