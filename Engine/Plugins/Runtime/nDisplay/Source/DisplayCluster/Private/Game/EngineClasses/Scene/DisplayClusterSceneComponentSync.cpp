// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DisplayClusterSceneComponentSync.h"

#include "DisplayClusterEnums.h"

#include "GameFramework/Actor.h"
#include "Cluster/IDisplayClusterClusterManager.h"

#include "Misc/DisplayClusterGlobals.h"
#include "Misc/DisplayClusterLog.h"
#include "Misc/DisplayClusterTypesConverter.h"

#include "Serialization/Archive.h"


void UDisplayClusterSceneComponentSync::BeginPlay()
{
	Super::BeginPlay();

	const EDisplayClusterOperationMode OpMode = GDisplayCluster->GetOperationMode();
	if (OpMode == EDisplayClusterOperationMode::Cluster)
	{
		// Generate unique sync id
		SyncId = GenerateSyncId();

		// Register sync object
		if (IDisplayClusterClusterManager* ClusterMgr = GDisplayCluster->GetClusterMgr())
		{
			UE_LOGF(LogDisplayClusterGame, Log, "Registering sync object %ls...", *SyncId);
			ClusterMgr->RegisterSyncObject(this, EDisplayClusterSyncGroup::Tick);
		}
		else
		{
			UE_LOGF(LogDisplayClusterGame, Warning, "Couldn't register %ls scene component sync.", *SyncId);
		}
	}
}

void UDisplayClusterSceneComponentSync::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	const EDisplayClusterOperationMode OpMode = GDisplayCluster->GetOperationMode();
	if (OpMode == EDisplayClusterOperationMode::Cluster)
	{
		// Unregister sync object
		if (IDisplayClusterClusterManager* ClusterMgr = GDisplayCluster->GetClusterMgr())
		{
			UE_LOGF(LogDisplayClusterGame, Log, "Unregistering sync object %ls...", *SyncId);
			ClusterMgr->UnregisterSyncObject(this);
		}
	}

	Super::EndPlay(EndPlayReason);
}

bool UDisplayClusterSceneComponentSync::IsActive() const
{
	return IsValidChecked(this);
}

FString UDisplayClusterSceneComponentSync::GenerateSyncId()
{
	return FString::Printf(TEXT("S_%s"), *GetFullName());
}

void UDisplayClusterSceneComponentSync::SerializeDC(FArchive& Ar)
{
	FTransform Transform;

	if (Ar.IsLoading())
	{
		Ar << Transform;
		UE_LOGF(LogDisplayClusterGame, Verbose, "%ls: applying transform data <%ls>", *SyncId, *Transform.ToHumanReadableString());
		SetSyncTransform(Transform);
	}
	else
	{
		Transform = GetSyncTransform();
		Ar << Transform;
	}
}
