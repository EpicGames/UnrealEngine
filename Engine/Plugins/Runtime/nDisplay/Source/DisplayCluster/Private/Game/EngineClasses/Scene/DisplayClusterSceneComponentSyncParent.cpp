// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DisplayClusterSceneComponentSyncParent.h"
#include "GameFramework/Actor.h"

#include "Misc/DisplayClusterHelpers.h"
#include "Misc/DisplayClusterLog.h"


UDisplayClusterSceneComponentSyncParent::UDisplayClusterSceneComponentSyncParent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterClusterSyncObject
//////////////////////////////////////////////////////////////////////////////////////////////
bool UDisplayClusterSceneComponentSyncParent::IsDirty() const
{
	USceneComponent* const pParent = GetAttachParent();
	if (IsValid(pParent))
	{
		const bool bIsDirty = (LastSyncLoc != pParent->GetRelativeLocation() || LastSyncRot != pParent->GetRelativeRotation() || LastSyncScale != pParent->GetRelativeScale3D());
		UE_LOGF(LogDisplayClusterGame, Verbose, "SYNC_PARENT: %ls dirty state is %ls", *GetSyncId(), *DisplayClusterHelpers::str::BoolToStr(bIsDirty));
		return bIsDirty;
	}

	return false;
}

void UDisplayClusterSceneComponentSyncParent::ClearDirty()
{
	USceneComponent* const pParent = GetAttachParent();
	if (IsValid(pParent))
	{
		LastSyncLoc   = pParent->GetRelativeLocation();
		LastSyncRot   = pParent->GetRelativeRotation();
		LastSyncScale = pParent->GetRelativeScale3D();
	}
}


//////////////////////////////////////////////////////////////////////////////////////////////
// UDisplayClusterSceneComponentSync
//////////////////////////////////////////////////////////////////////////////////////////////
FString UDisplayClusterSceneComponentSyncParent::GenerateSyncId()
{
	return FString::Printf(TEXT("SP_%s"), *GetAttachParent()->GetFullName());
}

FTransform UDisplayClusterSceneComponentSyncParent::GetSyncTransform() const
{
	return GetAttachParent()->GetRelativeTransform();
}

void UDisplayClusterSceneComponentSyncParent::SetSyncTransform(const FTransform& t)
{
	GetAttachParent()->SetRelativeTransform(t);
}
