// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DisplayClusterSceneComponentSyncThis.h"

#include "GameFramework/Actor.h"


UDisplayClusterSceneComponentSyncThis::UDisplayClusterSceneComponentSyncThis(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterClusterSyncObject
//////////////////////////////////////////////////////////////////////////////////////////////
bool UDisplayClusterSceneComponentSyncThis::IsDirty() const
{
	return (LastSyncLoc != GetRelativeLocation() || LastSyncRot != GetRelativeRotation() || LastSyncScale != GetRelativeScale3D());
}

void UDisplayClusterSceneComponentSyncThis::ClearDirty()
{
	LastSyncLoc = GetRelativeLocation();
	LastSyncRot = GetRelativeRotation();
	LastSyncScale = GetRelativeScale3D();
}


//////////////////////////////////////////////////////////////////////////////////////////////
// UDisplayClusterSceneComponentSync
//////////////////////////////////////////////////////////////////////////////////////////////
FString UDisplayClusterSceneComponentSyncThis::GenerateSyncId()
{
	return FString::Printf(TEXT("ST_%s"), *GetFullName());
}

FTransform UDisplayClusterSceneComponentSyncThis::GetSyncTransform() const
{
	return GetRelativeTransform();
}

void UDisplayClusterSceneComponentSyncThis::SetSyncTransform(const FTransform& t)
{
	SetRelativeTransform(t);
}
