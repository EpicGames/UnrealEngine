// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DisplayClusterScreenComponent.h"

#include "DisplayClusterVersion.h"
#include "Components/StaticMeshComponent.h"

#include "Engine/StaticMesh.h"
#include "Materials/MaterialInterface.h"
#include "Materials/Material.h"
#include "UObject/ConstructorHelpers.h"
#include "UObject/StructOnScope.h"


UDisplayClusterScreenComponent::UDisplayClusterScreenComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	static ConstructorHelpers::FObjectFinder<UStaticMesh> ScreenMesh(TEXT("/nDisplay/Meshes/plane_hd_1x1"));
	SetStaticMesh(ScreenMesh.Object);

	SetMobility(EComponentMobility::Movable);
	SetCollisionEnabled(ECollisionEnabled::NoCollision);
	SetVisibility(false);
	SetHiddenInGame(true);
	SetCastShadow(false);

#if WITH_EDITOR
	if (GIsEditor)
	{
		SetVisibility(true);
	}
#endif

	bVisibleInReflectionCaptures = false;
	bVisibleInRayTracing = false;
	bVisibleInRealTimeSkyCaptures = false;

	// Default screen size
	SetScreenSize(FVector2D(100.f, 56.25f));
}

FVector2D UDisplayClusterScreenComponent::GetScreenSize() const
{
	const FVector ComponentScale = GetRelativeScale3D();
	const FVector2D ComponentScale2D(ComponentScale.Y, ComponentScale.Z);
	return ComponentScale2D;
}

void UDisplayClusterScreenComponent::SetScreenSize(const FVector2D& InSize)
{
	SetRelativeScale3D(FVector(1.f, InSize.X, InSize.Y));

#if WITH_EDITOR
	Size = InSize;
#endif
}

void UDisplayClusterScreenComponent::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FDisplayClusterCustomVersion::GUID);
	if (Ar.IsLoading() &&
		Ar.CustomVer(FDisplayClusterCustomVersion::GUID) < FDisplayClusterCustomVersion::ComponentParentChange_4_27)
	{
		USceneComponent::Serialize(Ar);

#if WITH_EDITOR
		// Filtering by GetOwner() allows to avoid double upscaling for many cases. But it's not enough unfortunately.
		// There are still some cases where screens get upscaled repeatedly and get x10000 in the end.
		SetScreenSize(Size * (GetOwner() ? 1.f : 100.f));
#endif
	}
	else
	{
		Super::Serialize(Ar);
	}
}

#if WITH_EDITOR
void UDisplayClusterScreenComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.MemberProperty)
	{
		if (PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UDisplayClusterScreenComponent, Size))
		{
			SetScreenSize(Size);
		}
		else if (PropertyChangedEvent.MemberProperty->GetFName() == GetRelativeScale3DPropertyName())
		{
			UpdateScreenSizeFromScale();
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UDisplayClusterScreenComponent::UpdateScreenSizeFromScale()
{
	// The Size property just reflects the 2D [Y,Z] relative transform scale and should always be in sync.
	const FVector2D ScreenSize = GetScreenSize();
	const TSharedPtr<FStructOnScope> ScreenSizeStructHighPrecision = MakeShared<FStructOnScope>(TBaseStructure<FVector2D>::Get(),
		(uint8*)&ScreenSize);

	const FStructProperty* SizeProperty = FindFieldChecked<FStructProperty>(UDisplayClusterScreenComponent::StaticClass(),
		GET_MEMBER_NAME_CHECKED(UDisplayClusterScreenComponent, Size));
	uint8* SizePropAddr = SizeProperty->ContainerPtrToValuePtr<uint8>(this);

	// HACK: We need to purposely lose precision when storing the size in order to match what the UI would show if entering
	// it directly in the details panel. If we don't then it can be easy to lose the ability to propagate changes from
	// the CDO to instances when entering in high precision numbers such as using the slider.
	//
	// What happens is PropertyNode.cpp won't propagate a change if the instance property value isn't identical
	// to the previous CDO value. This is a deep comparison comparing the actual values of the property (high precision).
	// But the previous value is exported as text which loses precision, then is imported into a temporary property for comparison.
	// The deep comparison is done against the live property (this property) which has the true double values, but the temporary
	// property only has the imported text low precision values, so the comparison will fail and the value will not propagate.
	
	// TODO: Instead of using a separate FVector2D for size, we should probably just use the correct scale property handles
	// through customization. Doing so may require public editor API changes since the transform handles are all customized by default
	// and fairly locked down. We would also need to still support the locked aspect ration option.
	
	FString ScreenSizeStringLowPrecision;
	SizeProperty->ExportText_Direct(ScreenSizeStringLowPrecision, ScreenSizeStructHighPrecision->GetStructMemory(), ScreenSizeStructHighPrecision->GetStructMemory(), this, PPF_None);
	SizeProperty->ImportText_Direct(*ScreenSizeStringLowPrecision, SizePropAddr, this, PPF_SerializedAsImportText);
}
#endif
