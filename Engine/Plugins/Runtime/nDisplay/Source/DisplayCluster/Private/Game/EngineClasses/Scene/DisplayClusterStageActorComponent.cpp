// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DisplayClusterStageActorComponent.h"
#include "DisplayClusterRootActor.h"
#include "Components/DisplayClusterICVFXCameraComponent.h"

void UDisplayClusterStageActorComponent::SetRootActor(ADisplayClusterRootActor* InRootActor)
{
#if WITH_EDITOR
	Modify();
#endif
	
	RootActor = InRootActor;
}

const TSoftObjectPtr<ADisplayClusterRootActor>& UDisplayClusterStageActorComponent::GetRootActor() const
{
	return RootActor;
}