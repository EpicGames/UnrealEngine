// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DisplayClusterStageGeometryComponent.h"

#include "CanvasTypes.h"
#include "DisplayClusterMeshProjectionRenderer.h"
#include "DisplayClusterRootActor.h"
#include "DisplayClusterConfigurationTypes.h"
#include "ProceduralMeshComponent.h"
#include "TextureResource.h"
#include "Components/DisplayClusterScreenComponent.h"
#include "Components/DisplayClusterStageIsosphereComponent.h"
#include "Engine/TextureRenderTarget2D.h"

const uint32 UDisplayClusterStageGeometryComponent::GeometryMapSize = 512;
const float UDisplayClusterStageGeometryComponent::GeometryMapFOV = 2.0f * FMath::RadiansToDegrees(FMath::Atan(0.55 * PI));

UDisplayClusterStageGeometryComponent::UDisplayClusterStageGeometryComponent()
{
	SetActive(true);
	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.bAllowTickOnDedicatedServer = false;

	Renderer = MakeShared<FDisplayClusterMeshProjectionRenderer>();
}

void UDisplayClusterStageGeometryComponent::TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	if (bUpdateGeometryMap)
	{
		RedrawGeometryMap();
		UpdateStageIsosphere();
	}
}

void UDisplayClusterStageGeometryComponent::Invalidate(bool bForceImmediateRedraw)
{
	if (bForceImmediateRedraw)
	{
		RedrawGeometryMap();
		UpdateStageIsosphere();
	}
	else
	{
		bUpdateGeometryMap = true;
	}
}

bool UDisplayClusterStageGeometryComponent::GetStageDistanceAndNormal(const FVector& InDirection, float& OutDistance, FVector& OutNormal)
{
	if (bGeometryMapLoaded)
	{
		// Bias ever so slightly to the north to ensure all equator vertices are queried from the same map
		const bool bIsQueryingNorthernHemisphere = InDirection.Z > -0.001;
		FDisplayClusterStageGeometryMap& GeometryMap = bIsQueryingNorthernHemisphere ? NorthGeometryMap : SouthGeometryMap;

		auto GetPixel = [&GeometryMap](uint32 InX, uint32 InY)
		{
			const uint32 ClampedX = FMath::Clamp(InX, (uint32)0, GeometryMapSize - 1);
			const uint32 ClampedY = FMath::Clamp(InY, (uint32)0, GeometryMapSize - 1);

			const uint32 PixelIndex = ClampedY * GeometryMapSize + ClampedX;
			if (!GeometryMap.GeometryData.IsValidIndex(PixelIndex))
			{
				// Deny access to a pixel that is out of range.
				return FLinearColor::Black;
			}

			return GeometryMap.GeometryData[PixelIndex].GetFloats();
		};

		// Use the view matrix of the geometry map to convert from world coordinates to view coordinates, then convert those coordinates into dome projection space
		const FVector ViewPos = GeometryMap.ViewMatrices.GetWorldToView().TransformVector(InDirection);
		const FVector ProjectedViewPos = FDisplayClusterMeshProjectionRenderer::ProjectViewPosition(ViewPos, EDisplayClusterMeshProjectionType::Azimuthal);

		const FVector4 ScreenPos = GeometryMap.ViewMatrices.GetViewToClip().TransformFVector4(FVector4(ProjectedViewPos, 1));

		if (ScreenPos.W != 0.0)
		{
			const float InvW = (ScreenPos.W > 0.0f ? 1.0f : -1.0f) / ScreenPos.W;
			const float Y = (GProjectionSignY > 0.0f) ? ScreenPos.Y : 1.0f - ScreenPos.Y;
			const FVector2D PixelPos = FVector2D((0.5f + ScreenPos.X * 0.5f * InvW) * GeometryMapSize, (0.5f - Y * 0.5f * InvW) * GeometryMapSize);

			// Perform a bilinear interpolation on the computed pixel position to ensure a continuous normal regardless of the resolution of the normal map
			const uint32 PixelX = FMath::Floor(PixelPos.X - 0.5f);
			const uint32 PixelY = FMath::Floor(PixelPos.Y - 0.5f);
			const float PixelXFrac = FMath::Frac(PixelPos.X);
			const float PixelYFrac = FMath::Frac(PixelPos.Y);

			const FLinearColor NormalData = FMath::Lerp(
				FMath::Lerp(GetPixel(PixelX, PixelY), GetPixel(PixelX + 1, PixelY), PixelXFrac),
				FMath::Lerp(GetPixel(PixelX, PixelY + 1), GetPixel(PixelX + 1, PixelY + 1), PixelXFrac),
				PixelYFrac);

			const FVector NormalVector = 2.f * FVector(NormalData.R, NormalData.G, NormalData.B) - 1.f;
			OutNormal = NormalVector.GetSafeNormal();

			// Make sure the depth value is not 0, as that will cause a divide by zero when transformed, resulting in an NaN distance being returned
			const float Depth = FMath::Max(0.001f, NormalData.A);

			FVector4 DepthPos = GeometryMap.ViewMatrices.GetClipToView().TransformFVector4(FVector4(ScreenPos.X * InvW, ScreenPos.Y * InvW, Depth, 1.0f));
			DepthPos /= DepthPos.W;

			const FVector UnprojectedDepthPos = FDisplayClusterMeshProjectionRenderer::UnprojectViewPosition(DepthPos, EDisplayClusterMeshProjectionType::Azimuthal);
			OutDistance = UnprojectedDepthPos.Length();
			return true;
		}
	}

	OutNormal = FVector::ZeroVector;
	OutDistance = 0.0;
	return false;
}

bool UDisplayClusterStageGeometryComponent::MorphProceduralMesh(UProceduralMeshComponent* InProceduralMeshComponent, bool bSyncMeshLocation)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UDisplayClusterStageGeometryComponent::MorphProceduralMesh);

	if (bGeometryMapLoaded)
	{
		if (USceneComponent* CommonViewPoint = GetCommonViewPoint())
		{
			if (bSyncMeshLocation)
			{
				// Align the procedural mesh with the view point that the stage geometry component was rendered from to ensure an accurate morph.
				// The procedural mesh's world orientation must also be set to zero, as the geometry component render was computed in world coordinates
				InProceduralMeshComponent->SetWorldLocationAndRotation(CommonViewPoint->GetComponentLocation(), FRotator::ZeroRotator);
			}

			if (FProcMeshSection* Section = InProceduralMeshComponent->GetProcMeshSection(0))
			{
				for (int32 Index = 0; Index < Section->ProcVertexBuffer.Num(); ++Index)
				{
					FProcMeshVertex& Vertex = Section->ProcVertexBuffer[Index];

					const FVector VertexDirection = Vertex.Position.GetSafeNormal();
					float Distance;
					FVector Normal;
					GetStageDistanceAndNormal(VertexDirection, Distance, Normal);

					Vertex.Position = VertexDirection * Distance;

					const FMatrix RadialBasis = FRotationMatrix::MakeFromX(VertexDirection);

					const FVector WorldNormal = RadialBasis.TransformVector(Normal);
					Vertex.Normal = WorldNormal;
				}

				InProceduralMeshComponent->SetProcMeshSection(0, *Section);

				return true;
			}
		}
	}

	return false;
}

USceneComponent* UDisplayClusterStageGeometryComponent::GetCommonViewPoint() const
{
	if (ADisplayClusterRootActor* RootActor = Cast<ADisplayClusterRootActor>(GetOwner()))
	{
		return RootActor->GetCommonViewPoint();
	}

	return nullptr;
}

UTextureRenderTarget2D* UDisplayClusterStageGeometryComponent::CreateRenderTarget()
{
	UTextureRenderTarget2D* RenderTarget = NewObject<UTextureRenderTarget2D>(this);
	RenderTarget->InitCustomFormat(GeometryMapSize, GeometryMapSize, EPixelFormat::PF_FloatRGBA, false);
	RenderTarget->ClearColor = FLinearColor(0.0f, 0.0f, 0.0f, 1.0f);

	return RenderTarget;
}

void UDisplayClusterStageGeometryComponent::RedrawGeometryMap()
{
	if (!bGeometryMapLoaded)
	{
		NorthGeometryMap.RenderTarget = CreateRenderTarget();
		SouthGeometryMap.RenderTarget = CreateRenderTarget();

		bGeometryMapLoaded = true;
	}

	UpdateStageGeometry();
	GenerateGeometryMap(true);
	GenerateGeometryMap(false);

	bUpdateGeometryMap = false;
}

void UDisplayClusterStageGeometryComponent::UpdateStageGeometry()
{
	Renderer->ClearScene();

	if (ADisplayClusterRootActor* RootActor = Cast<ADisplayClusterRootActor>(GetOwner()))
	{
		TArray<FString> ProjectionMeshNames;

		if (UDisplayClusterConfigurationData* Config = RootActor->GetConfigData())
		{
			Config->GetReferencedMeshNames(ProjectionMeshNames);
		}

		Renderer->AddActor(RootActor, [&ProjectionMeshNames](const UPrimitiveComponent* PrimitiveComponent)
		{
			// Filter out any primitive component that isn't a projection mesh (a static mesh that has a Mesh projection configured for it) or a screen component
			const bool bIsProjectionMesh = PrimitiveComponent->IsA<UStaticMeshComponent>() && ProjectionMeshNames.Contains(PrimitiveComponent->GetName());
			const bool bIsScreen = PrimitiveComponent->IsA<UDisplayClusterScreenComponent>();
			return bIsProjectionMesh || bIsScreen;
		});

		// Get the bounding box for all of the stage's screens
		FBox StageBoundingBox(ForceInit);
		RootActor->ForEachComponent<UPrimitiveComponent>(false, [&StageBoundingBox, &ProjectionMeshNames](const UPrimitiveComponent* PrimitiveComponent)
		{
			// Filter out any primitive component that isn't a projection mesh (a static mesh that has a Mesh projection configured for it) or a screen component
			const bool bIsProjectionMesh = PrimitiveComponent->IsA<UStaticMeshComponent>() && ProjectionMeshNames.Contains(PrimitiveComponent->GetName());
			const bool bIsScreen = PrimitiveComponent->IsA<UDisplayClusterScreenComponent>();

			if (PrimitiveComponent->IsRegistered() && (bIsProjectionMesh || bIsScreen))
			{
				StageBoundingBox += PrimitiveComponent->Bounds.GetBox();
			}
		});

		// Make sure to add the viewport the stage geometry has been rendered from, in case that viewpoint is outside the stage geometry bounds
		StageBoundingBox += RootActor->GetCommonViewPoint()->GetComponentLocation();

		StageBoundingBox = StageBoundingBox.ShiftBy(-RootActor->GetActorLocation());
		StageBoundingRadius = FMath::Max(StageBoundingBox.Min.Length(), StageBoundingBox.Max.Length());
	}
}

void UDisplayClusterStageGeometryComponent::UpdateStageIsosphere()
{
	if (AActor* RootActor = GetOwner())
	{
		if (UDisplayClusterStageIsosphereComponent* IsosphereComponent = RootActor->FindComponentByClass<UDisplayClusterStageIsosphereComponent>())
		{
			IsosphereComponent->ResetIsosphere();
			MorphProceduralMesh(IsosphereComponent, true);
		}
	}
}

void UDisplayClusterStageGeometryComponent::GenerateGeometryMap(bool bIsNorthMap)
{
	if (ADisplayClusterRootActor* RootActor = Cast<ADisplayClusterRootActor>(GetOwner()))
	{
		UWorld* World = RootActor->GetWorld();
		if (!World)
		{
			return;
		}

		FDisplayClusterStageGeometryMap& GeometryMap = bIsNorthMap ? NorthGeometryMap : SouthGeometryMap;

		// Rotate the normal map's direction to account for the actor's world rotation
		FRotator InverseRootRotation = RootActor->GetTransform().Rotator().GetInverse();
		FVector NormalMapDirection = InverseRootRotation.RotateVector(bIsNorthMap ? FVector::UpVector : FVector::DownVector);

		FDisplayClusterMeshProjectionRenderSettings RenderSettings;
		RenderSettings.RenderType = EDisplayClusterMeshProjectionOutput::Normals;
		RenderSettings.ProjectionType = EDisplayClusterMeshProjectionType::Azimuthal;
		RenderSettings.NormalCorrectionMatrix = FMatrix44f(FRotationMatrix::Make(InverseRootRotation));

		GetSceneViewInitOptions(NormalMapDirection, RenderSettings.ViewInitOptions);
		GeometryMap.ViewMatrices = FViewMatrices(RenderSettings.ViewInitOptions);

		FTextureRenderTarget2DResource* TexResource = GeometryMap.RenderTarget ? (FTextureRenderTarget2DResource*)GeometryMap.RenderTarget->GetResource() : nullptr;
		if (TexResource)
		{
			FCanvas Canvas(TexResource, nullptr, World, World->Scene->GetFeatureLevel());
			Canvas.Clear(FLinearColor::Black);

			Renderer->Render(&Canvas, World->Scene, RenderSettings);
			Canvas.Flush_GameThread();

			TexResource->ReadFloat16Pixels(GeometryMap.GeometryData);
		}
	}
}

void UDisplayClusterStageGeometryComponent::GetSceneViewInitOptions(const FVector& ViewDirection, FSceneViewInitOptions& OutViewInitOptions)
{
	if (ADisplayClusterRootActor* RootActor = Cast<ADisplayClusterRootActor>(GetOwner()))
	{
		OutViewInitOptions.ViewLocation = RootActor->GetCommonViewPoint()->GetComponentLocation();
		OutViewInitOptions.ViewRotation = FRotator(RootActor->GetActorRotation().Quaternion() * ViewDirection.Rotation().Quaternion());
		OutViewInitOptions.ViewOrigin = OutViewInitOptions.ViewLocation;

		OutViewInitOptions.SetViewRectangle(FIntRect(0, 0, GeometryMapSize, GeometryMapSize));

		AWorldSettings* WorldSettings = nullptr;

		if (RootActor->GetWorld())
		{
			WorldSettings = RootActor->GetWorld()->GetWorldSettings();
		}

		if (WorldSettings != nullptr)
		{
			OutViewInitOptions.WorldToMetersScale = WorldSettings->WorldToMeters;
		}

		// Rotate view 90 degrees
		OutViewInitOptions.ViewRotationMatrix = FInverseRotationMatrix(OutViewInitOptions.ViewRotation) * FMatrix(
			FPlane(0, 0, 1, 0),
			FPlane(1, 0, 0, 0),
			FPlane(0, 1, 0, 0),
			FPlane(0, 0, 0, 1));

		const float MinZ = GNearClippingPlane;
		const float MaxZ = FMath::Max(StageBoundingRadius, MinZ);

		// Avoid zero ViewFOVs which cause divide by zero's in projection matrix
		const float MatrixFOV = FMath::Max(0.001f, GeometryMapFOV) * (float)PI / 360.0f;

		const float XAxisMultiplier = 1.0f;
		const float YAxisMultiplier = 1.0f;

		OutViewInitOptions.ProjectionMatrix = FReversedZPerspectiveMatrix(
			MatrixFOV,
			MatrixFOV,
			XAxisMultiplier,
			YAxisMultiplier,
			MinZ,
			MaxZ);

		OutViewInitOptions.FOV = GeometryMapFOV;
	}
}
