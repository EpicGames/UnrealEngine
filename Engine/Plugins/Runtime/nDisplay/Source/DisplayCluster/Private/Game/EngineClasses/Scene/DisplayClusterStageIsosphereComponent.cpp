// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DisplayClusterStageIsosphereComponent.h"

#include "Engine/StaticMesh.h"
#include "KismetProceduralMeshLibrary.h"
#include "PhysicsEngine/BodySetup.h"
#include "UObject/ConstructorHelpers.h"

UDisplayClusterStageIsosphereComponent::UDisplayClusterStageIsosphereComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ConstructorHelpers::FObjectFinder<UStaticMesh>  IsosphereMeshFinder(TEXT("/nDisplay/Meshes/SM_IcoSphere.SM_IcoSphere"));

	IsosphereMesh = IsosphereMeshFinder.Object;

	BodyInstance.SetCollisionEnabled(ECollisionEnabled::QueryOnly);
	BodyInstance.SetObjectType(ECC_WorldDynamic);
	BodyInstance.SetResponseToAllChannels(ECR_Ignore);
	BodyInstance.SetResponseToChannel(ECC_Visibility, ECR_Block);

	SetHiddenInGame(true);
	SetVisibility(false);
}

void UDisplayClusterStageIsosphereComponent::OnComponentCreated()
{
	Super::OnComponentCreated();

	// Load the isosphere's geometry into the procedural mesh
	if (IsIsosphereMeshValid())
	{
		const int32 NumSections = IsosphereMesh->GetNumSections(0);
		for (int32 SectionIndex = 0; SectionIndex < NumSections; SectionIndex++)
		{
			TArray<FVector> Vertices;
			TArray<int32> Triangles;
			TArray<FVector> Normals;
			TArray<FVector2D> UVs;
			TArray<FProcMeshTangent> Tangents;

			UKismetProceduralMeshLibrary::GetSectionFromStaticMesh(IsosphereMesh, 0, SectionIndex, Vertices, Triangles, Normals, UVs, Tangents);

			TArray<FVector2D> EmptyUVs;
			TArray<FLinearColor> EmptyColors;
			CreateMeshSection_LinearColor(SectionIndex, Vertices, Triangles, Normals, UVs, EmptyUVs, EmptyUVs, EmptyUVs, EmptyColors, Tangents, true);

			for (int32 Index = 0; Index < IsosphereMesh->GetStaticMaterials().Num(); ++Index)
			{
				if (UMaterialInterface* MaterialInterface = IsosphereMesh->GetStaticMaterials()[Index].MaterialInterface)
				{
					SetMaterial(Index, MaterialInterface);
				}
			}
		}
	}
}

void UDisplayClusterStageIsosphereComponent::ResetIsosphere()
{
	if (IsIsosphereMeshValid())
	{
		const int32 NumSections = IsosphereMesh->GetNumSections(0);
		for (int32 SectionIndex = 0; SectionIndex < NumSections; SectionIndex++)
		{
			TArray<FVector> Vertices;
			TArray<int32> Triangles;
			TArray<FVector> Normals;
			TArray<FVector2D> UVs;
			TArray<FProcMeshTangent> Tangents;

			UKismetProceduralMeshLibrary::GetSectionFromStaticMesh(IsosphereMesh, 0, SectionIndex, Vertices, Triangles, Normals, UVs, Tangents);

			if (FProcMeshSection* Section = GetProcMeshSection(SectionIndex))
			{
				for (int32 Index = 0; Index < Vertices.Num(); ++Index)
				{
					FProcMeshVertex& Vertex = Section->ProcVertexBuffer[Index];
					Vertex.Position = Vertices[Index];
					Vertex.Normal = Normals[Index];
				}

				SetProcMeshSection(SectionIndex, *Section);
			}
		}

		// Match the sided-ness setting of the IsosphereMesh.
		const UBodySetup* IsosphereMeshBodySetup = IsosphereMesh->GetBodySetup();
		if (ProcMeshBodySetup && IsosphereMeshBodySetup)
		{
			ProcMeshBodySetup->bDoubleSidedGeometry = IsosphereMeshBodySetup->bDoubleSidedGeometry;
		}
	}
}

bool UDisplayClusterStageIsosphereComponent::IsIsosphereMeshValid() const
{
	return
		ensureMsgf(IsosphereMesh, TEXT("UDisplayClusterStageIsosphereComponent::IsosphereMesh was not properly loaded")) &&
		ensureMsgf(IsosphereMesh->bAllowCPUAccess, TEXT("UDisplayClusterStageIsosphereComponent::IsosphereMesh does not have Allow CPUAccess enabled"));
}
