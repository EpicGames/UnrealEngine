// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DisplayClusterSyncTickComponent.h"

#include "IPDisplayCluster.h"
#include "Game/IPDisplayClusterGameManager.h"
#include "Misc/DisplayClusterGlobals.h"
#include "DisplayClusterRootActor.h"


UDisplayClusterSyncTickComponent::UDisplayClusterSyncTickComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.bTickEvenWhenPaused = true;
	PrimaryComponentTick.TickGroup = TG_PostPhysics;
	PrimaryComponentTick.bStartWithTickEnabled = true;
	bAutoActivate = true;
}

void UDisplayClusterSyncTickComponent::TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	if (GDisplayCluster->GetOperationMode() != EDisplayClusterOperationMode::Disabled)
	{
		// Since we may have multiple DCRAs available in the world, only 'Main' DCRA
		// should be able to throw tick call to nDisplay
		if (GDisplayCluster->GetGameMgr()->GetRootActor() == Cast<ADisplayClusterRootActor>(GetOwner()))
		{
			GDisplayCluster->Tick(DeltaTime);
		}
	}

	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);
}
