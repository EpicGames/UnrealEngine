// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterTestPatternsActor.h"

#include "Components/PostProcessComponent.h"
#include "Components/SceneComponent.h"

#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"

#include "IDisplayCluster.h"

#include "DisplayClusterConfigurationTypes.h"

#include "Cluster/DisplayClusterClusterEvent.h"
#include "Game/IDisplayClusterGameManager.h"

#include "DisplayClusterRootActor.h"

#include "Misc/DisplayClusterGlobals.h"
#include "Misc/DisplayClusterLog.h"
#include "Misc/DisplayClusterHelpers.h"
#include "Misc/DisplayClusterTypesConverter.h"

#include "UObject/ConstructorHelpers.h"


// Calibration patterns control
static TAutoConsoleVariable<FString> CVarCalibrationPattern(
	TEXT("nDisplay.Calibration.Pattern"),
	FString(),
	TEXT("Viewport to calibrate\n"),
	ECVF_Default);

namespace
{
	const FString GStrEventCategory    (TEXT("nDisplay"));
	const FString GStrEventType        (TEXT("Calibration"));
	const FString GStrEventName        (TEXT("Pattern"));
	const FString GStrParamPatternId   (TEXT("PatternId"));
	const FString GStrParamViewportId  (TEXT("ViewportId"));
	const FString GStrParamViewportAll (TEXT("*"));

	const FString GStrMaterialParamSeparator  (TEXT(":"));
	const FString GStrMaterialParamTypeScalar (TEXT("Scalar"));
	const FString GStrMaterialParamTypeColor  (TEXT("Color"));
}


ADisplayClusterTestPatternsActor::ADisplayClusterTestPatternsActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Root component
	RootComponent = CreateDefaultSubobject<USceneComponent>(TEXT("RootComponent"));

	// Postprocess component for non-nDisplay usage
	PostProcessComponent = CreateDefaultSubobject<UPostProcessComponent>(TEXT("PostProcessComponent"));
	PostProcessComponent->Priority = 99999.f;
	PostProcessComponent->bUnbound = true;
	PostProcessComponent->bEnabled = false;

	// Setup materials static references
	InitializeMaterials();

	PrimaryActorTick.bCanEverTick = true;
}

void ADisplayClusterTestPatternsActor::BeginPlay()
{
	// Store current operation mode
	OperationMode = GDisplayCluster->GetOperationMode();

	// Set main cluster event handler. This is an entry point for any incoming cluster events.
	OnClusterEvent.BindUObject(this, &ADisplayClusterTestPatternsActor::OnClusterEventHandler);
	// Subscribe for cluster events
	IDisplayCluster::Get().GetClusterMgr()->AddClusterEventJsonListener(OnClusterEvent);

	// Use our postprocess component if we're running without nDisplay
	PostProcessComponent->bEnabled = (OperationMode != EDisplayClusterOperationMode::Cluster);

	// Bind CVar delegate to process commands from console variable
	CVarCalibrationPattern.AsVariable()->SetOnChangedCallback(FConsoleVariableDelegate::CreateUObject(this, &ADisplayClusterTestPatternsActor::OnConsoleVariableChangedPattern));

	// Initialize internals
	InitializeInternals();

	Super::BeginPlay();
}

void ADisplayClusterTestPatternsActor::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	// Unsubscribe from cluster events
	IDisplayCluster::Get().GetClusterMgr()->RemoveClusterEventJsonListener(OnClusterEvent);

	Super::EndPlay(EndPlayReason);
}

void ADisplayClusterTestPatternsActor::Tick(float DeltaSeconds)
{
	// Override post-process settings if nDisplay is active
	if (OperationMode != EDisplayClusterOperationMode::Disabled)
	{
		//@todo: additional test+refactor stuff before release
		static IDisplayClusterGameManager* const GameMgr = GDisplayCluster->GetGameMgr();
		ADisplayClusterRootActor* RootActor = GameMgr->GetRootActor();
		if (RootActor)
		{
			const FString LocalNodeId = GDisplayCluster->GetConfigMgr()->GetLocalNodeId();

			for (auto it = ViewportPPSettings.CreateConstIterator(); it; ++it)
			{
				if (RootActor->CurrentConfigData != nullptr)
				{
					UDisplayClusterConfigurationViewport* ViewportCfg = RootActor->CurrentConfigData->GetViewport(LocalNodeId, it->Key);
					if (ViewportCfg)
					{
						// Assign current post-process settigns for each viewport
						FDisplayClusterConfigurationViewport_CustomPostprocessSettings& DstPostProcess = ViewportCfg->RenderSettings.CustomPostprocess.Override;
						DstPostProcess.bIsEnabled = true;
						DstPostProcess.bIsOneFrame = true;
						DstPostProcess.PostProcessSettings = it->Value;
					}
				}
			}
		}
	}

	Super::Tick(DeltaSeconds);
}

void ADisplayClusterTestPatternsActor::InitializeInternals()
{
	// PP settings shared by all viewports or the whole screen if nDisplay is not active
	ViewportPPSettings.Emplace(GStrParamViewportAll, CreatePPSettings(nullptr));

	// Initialize defaults for nDisplay viewports
	if (OperationMode != EDisplayClusterOperationMode::Disabled)
	{
		// Get local node configuration
		const UDisplayClusterConfigurationClusterNode* Node = GDisplayCluster->GetConfigMgr()->GetLocalNode();
		if (Node)
		{
			// For each local viewport create a PP settings structure with no PP material assigned
			for (const TPair<FString, TObjectPtr<UDisplayClusterConfigurationViewport>>& it : Node->Viewports)
			{
				ViewportPPSettings.Emplace(it.Key, CreatePPSettings(nullptr));
			}
		}
	}
}

void ADisplayClusterTestPatternsActor::InitializeMaterials()
{
	// M_TPSCircles
	static ConstructorHelpers::FObjectFinder<UMaterial> MatTPSCircles(TEXT("Material'/nDisplay/TestPatterns/Statics/M_TPSCircles.M_TPSCircles'"));
	check(MatTPSCircles.Object);
	CalibrationPatterns.Emplace(FString("TPSCircles"), MatTPSCircles.Object);

	// M_TPSColoredBars
	static ConstructorHelpers::FObjectFinder<UMaterial> MatTPSColoredBars(TEXT("Material'/nDisplay/TestPatterns/Statics/M_TPSColoredBars.M_TPSColoredBars'"));
	check(MatTPSColoredBars.Object);
	CalibrationPatterns.Emplace(FString("TPSColoredBars"), MatTPSColoredBars.Object);

	// M_TPSGrayBars
	static ConstructorHelpers::FObjectFinder<UMaterial> MatTPSGrayBars(TEXT("Material'/nDisplay/TestPatterns/Statics/M_TPSGrayBars.M_TPSGrayBars'"));
	check(MatTPSGrayBars.Object);
	CalibrationPatterns.Emplace(FString("TPSGrayBars"), MatTPSGrayBars.Object);

	// M_TPSGridCircles
	static ConstructorHelpers::FObjectFinder<UMaterial> MatTPSGridCircles(TEXT("Material'/nDisplay/TestPatterns/Statics/M_TPSGridCircles.M_TPSGridCircles'"));
	check(MatTPSGridCircles.Object);
	CalibrationPatterns.Emplace(FString("TPSGridCircles"), MatTPSGridCircles.Object);

	// M_TPAChevron
	static ConstructorHelpers::FObjectFinder<UMaterial> MatTPAChevron(TEXT("Material'/nDisplay/TestPatterns/Animated/M_TPAChevron.M_TPAChevron'"));
	check(MatTPAChevron.Object);
	CalibrationPatterns.Emplace(FString("TPAChevron"), MatTPAChevron.Object);

	// M_TPAGrid
	static ConstructorHelpers::FObjectFinder<UMaterial> MatTPAGrid(TEXT("Material'/nDisplay/TestPatterns/Animated/M_TPAGrid.M_TPAGrid'"));
	check(MatTPAGrid.Object);
	CalibrationPatterns.Emplace(FString("TPAGrid"), MatTPAGrid.Object);

	// M_TPAMirroredChevron
	static ConstructorHelpers::FObjectFinder<UMaterial> MatTPAMirroredChevron(TEXT("Material'/nDisplay/TestPatterns/Animated/M_TPAMirroredChevron.M_TPAMirroredChevron'"));
	check(MatTPAMirroredChevron.Object);
	CalibrationPatterns.Emplace(FString("TPAMirroredChevron"), MatTPAMirroredChevron.Object);

	// M_TPARadar
	static ConstructorHelpers::FObjectFinder<UMaterial> MatTPARadar(TEXT("Material'/nDisplay/TestPatterns/Animated/M_TPARadar.M_TPARadar'"));
	check(MatTPARadar.Object);
	CalibrationPatterns.Emplace(FString("TPARadar"), MatTPARadar.Object);

	// M_TPAStrips
	static ConstructorHelpers::FObjectFinder<UMaterial> MatTPAStrips(TEXT("Material'/nDisplay/TestPatterns/Animated/M_TPAStrips.M_TPAStrips'"));
	check(MatTPAStrips.Object);
	CalibrationPatterns.Emplace(FString("TPAStrips"), MatTPAStrips.Object);
}

void ADisplayClusterTestPatternsActor::UpdatePattern(const TMap<FString, FString>& Params)
{
	FScopeLock Lock(&InternalsSyncScope);

	TArray<FString> ViewportIds;
	FString PatternId;
	TMap<FString, FMaterialParameter> MaterialParams;

	// Parse parameters map to extract pattern ID, viewport IDs and pattern parameters
	for (auto it = Params.CreateConstIterator(); it; ++it)
	{
		// 1st parameter must be pattern ID
		if (it->Key.Equals(GStrParamPatternId))
		{
			PatternId = it->Value;
		}
		// 2nd parameter is viewport IDs list
		else if (it->Key.Equals(GStrParamViewportId))
		{
			it->Value.ParseIntoArray(ViewportIds, TEXT(","));
		}
		// Other parameters are custom material properties
		else
		{
			FString StrType;
			FString StrValue;

			// Colon separated property type and value
			if (it->Value.Split(GStrMaterialParamSeparator, &StrType, &StrValue) && !StrType.IsEmpty() && !StrValue.IsEmpty())
			{
				// Scalar
				if (StrType.Equals(GStrMaterialParamTypeScalar, ESearchCase::IgnoreCase))
				{
					MaterialParams.Emplace(it->Key, FMaterialParameter(EParamType::TypeScalar, StrValue));
				}
				// Color
				else if (StrType.Equals(GStrMaterialParamTypeColor, ESearchCase::IgnoreCase))
				{
					MaterialParams.Emplace(it->Key, FMaterialParameter(EParamType::TypeVector, StrValue));
				}
				else
				{
					UE_LOGF(LogDisplayClusterGame, Warning, "Calibration: Unknown material parameter type: %ls", *StrType);
				}
			}
		}
	}

	// Update pattern for specified viewports
	if (OperationMode != EDisplayClusterOperationMode::Disabled && OperationMode != EDisplayClusterOperationMode::Editor)
	{
		for (const FString& ViewportId : ViewportIds)
		{
			// Skip viewports that don't exist
			if (!ViewportPPSettings.Contains(ViewportId))
			{
				UE_LOGF(LogDisplayClusterGame, Warning, "Calibration: Viewport <%ls> not found", *ViewportId);
				continue;
			}

			// Setup new pattern material
			FPostProcessSettings& PPSettings = ViewportPPSettings[ViewportId];
			PPSettings.WeightedBlendables.Array.Empty();
			PPSettings.WeightedBlendables.Array.Add(CreateWeightedBlendable(PatternId));
			SetupMaterialParameters(Cast<UMaterialInstanceDynamic>(PPSettings.WeightedBlendables.Array[0].Object), MaterialParams);

			// In case all viewports were requested, set this pattern to all of them
			if (ViewportId.Equals(GStrParamViewportAll))
			{
				for (auto it = ViewportPPSettings.CreateIterator(); it; ++it)
				{
					it->Value.WeightedBlendables.Array = ViewportPPSettings[ViewportId].WeightedBlendables.Array;
				}
			}
		}
	}
	// If no nDisplay running, use our postprocess component to show the pattern
	else
	{
		PostProcessComponent->Settings.WeightedBlendables.Array.Empty();
		PostProcessComponent->Settings.WeightedBlendables.Array.Add(CreateWeightedBlendable(PatternId));
		SetupMaterialParameters(Cast<UMaterialInstanceDynamic>(PostProcessComponent->Settings.WeightedBlendables.Array[0].Object), MaterialParams);
	}
}

void ADisplayClusterTestPatternsActor::SetupMaterialParameters(UMaterialInstanceDynamic* DynamicMaterialInstance, const TMap<FString, FMaterialParameter>& Params) const
{
	if (!DynamicMaterialInstance)
	{
		return;
	}

	// Parse and set material parameters
	for (auto it = Params.CreateConstIterator(); it; ++it)
	{
		if (it->Value.Type == EParamType::TypeScalar)
		{
			DynamicMaterialInstance->SetScalarParameterValue(FName(*it->Key), DisplayClusterTypesConverter::template FromString<float>(it->Value.Value));
		}
		else if (it->Value.Type == EParamType::TypeVector)
		{
			FLinearColor Value;

			TArray<FString> ValueComponents;
			it->Value.Value.ParseIntoArray(ValueComponents, TEXT(","));

			Value.R = (ValueComponents.Num() > 0 ? DisplayClusterTypesConverter::FromString<float>(ValueComponents[0]) : 0.f);
			Value.G = (ValueComponents.Num() > 1 ? DisplayClusterTypesConverter::FromString<float>(ValueComponents[1]) : 0.f);
			Value.B = (ValueComponents.Num() > 2 ? DisplayClusterTypesConverter::FromString<float>(ValueComponents[2]) : 0.f);
			Value.A = (ValueComponents.Num() > 3 ? DisplayClusterTypesConverter::FromString<float>(ValueComponents[3]) : 0.f);

			DynamicMaterialInstance->SetVectorParameterValue(FName(*it->Key), Value);
		}
	}
}

UMaterialInstanceDynamic* ADisplayClusterTestPatternsActor::CreateMaterialInstance(const FString& PatternName)
{
	if (!CalibrationPatterns.Contains(PatternName))
	{
		UE_LOGF(LogDisplayClusterGame, Warning, "Calibration: no pattern material found: %ls", *PatternName);
		return nullptr;
	}

	// Get base material
	UMaterial* const SelectedMat = CalibrationPatterns[PatternName];
	// Create dynamic material instance
	UMaterialInstanceDynamic* SelectedMatInstance = UMaterialInstanceDynamic::Create(SelectedMat, this);

	return SelectedMatInstance;
}

FWeightedBlendable ADisplayClusterTestPatternsActor::CreateWeightedBlendable(const FString& PatternId)
{
	return CreateWeightedBlendable(CreateMaterialInstance(PatternId));
}

FWeightedBlendable ADisplayClusterTestPatternsActor::CreateWeightedBlendable(UMaterialInstanceDynamic* DynamicMaterialInstance)
{
	FWeightedBlendable NewBlendable;
	NewBlendable.Weight = 1.f;
	NewBlendable.Object = DynamicMaterialInstance;
	return NewBlendable;
}

FPostProcessSettings ADisplayClusterTestPatternsActor::CreatePPSettings(const FString& PatternName)
{
	return CreatePPSettings(CreateMaterialInstance(PatternName));
}

FPostProcessSettings ADisplayClusterTestPatternsActor::CreatePPSettings(UMaterialInstanceDynamic* DynamicMaterialInstance)
{
	FPostProcessSettings PPSettings;
	PPSettings.WeightedBlendables.Array.Add(CreateWeightedBlendable(DynamicMaterialInstance));
	return PPSettings;
}

void ADisplayClusterTestPatternsActor::OnConsoleVariableChangedPattern(IConsoleVariable* Var)
{
	// Array of space separated parameters parsed from the command
	TArray<FString> ParamArray;
	Var->GetString().ParseIntoArray(ParamArray, TEXT(" "));

	// Build command map
	TMap<FString, FString> ParamMap;
	for (int i = 0; i < ParamArray.Num(); ++i)
	{
		// Pattern ID
		if (i == 0)
		{
			ParamMap.Emplace(GStrParamPatternId, ParamArray[i]);
		}
		// Viewport IDs or * (all viewports)
		else if (i == 1)
		{
			ParamMap.Emplace(GStrParamViewportId, ParamArray[i]);
		}
		// Colon separated material parameters, i.e. someparam:somevalue
		else
		{
			FString MaterialParamName;
			FString MaterialParamValue;

			// Store parsed material parameter
			if (ParamArray[i].Split(GStrMaterialParamSeparator, &MaterialParamName, &MaterialParamValue))
			{
				ParamMap.Emplace(MaterialParamName, MaterialParamValue);
			}
		}
	}

	// Now process the command
	UpdatePattern(ParamMap);
}

void ADisplayClusterTestPatternsActor::OnClusterEventHandler(const FDisplayClusterClusterEventJson& Event)
{
	// Skip all events except of nDisplay:calibration:pattern
	if (Event.Category.Equals(GStrEventCategory, ESearchCase::IgnoreCase))
	{
		if (Event.Type.Equals(GStrEventType, ESearchCase::IgnoreCase))
		{
			if (Event.Name.Equals(GStrEventName, ESearchCase::IgnoreCase))
			{
				// This is our event, let's process it
				UpdatePattern(Event.Parameters);
			}
		}
	}
}
