// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterWidgetComponent.h"

#include "DynamicMeshBuilder.h"
#include "Engine/Engine.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "SceneView.h"
#include "Slate/WidgetRenderer.h"
#if !UE_SERVER
#include "RHI.h"
#endif

#if WITH_EDITOR
#include "Editor.h"
#include "LocalVertexFactory.h"
#include "PrimitiveSceneProxy.h"
#include "SceneInterface.h"
#include "StaticMeshResources.h"

/** Represents a billboard widget which displays a widget render target & material. Based on FMaterialSpriteSceneProxy. */
class FEditorWidgetBillboardProxy final : public FPrimitiveSceneProxy
{
public:
	SIZE_T GetTypeHash() const override
	{
		static size_t UniquePointer;
		return reinterpret_cast<size_t>(&UniquePointer);
	}

	/** Initialization constructor. */
	FEditorWidgetBillboardProxy(const UDisplayClusterWidgetComponent* InComponent)
		: FPrimitiveSceneProxy(InComponent)
		  , VertexFactory(GetScene().GetFeatureLevel(), "FEditorWidgetBillboardProxy")
		  , Component(InComponent)
	{
		bWillEverBeLit = false;

		// GetDynamicMeshElements needs to run single pass, because a single set of StaticMeshVertexBuffers are shared across all views.
		bSinglePassGDME = true;

		StaticMeshVertexBuffers.PositionVertexBuffer.Init(1);
		StaticMeshVertexBuffers.StaticMeshVertexBuffer.Init(1, 1);
		StaticMeshVertexBuffers.ColorVertexBuffer.Init(1);

		ENQUEUE_RENDER_COMMAND(FWidgetBillboardProxyInit)(
			[this](FRHICommandListImmediate& RHICmdList)
		{
			this->StaticMeshVertexBuffers.PositionVertexBuffer.InitResource(RHICmdList);
			this->StaticMeshVertexBuffers.StaticMeshVertexBuffer.InitResource(RHICmdList);
			this->StaticMeshVertexBuffers.ColorVertexBuffer.InitResource(RHICmdList);

			FLocalVertexFactory::FDataType Data;
			this->StaticMeshVertexBuffers.PositionVertexBuffer.BindPositionVertexBuffer(&this->VertexFactory, Data);
			this->StaticMeshVertexBuffers.StaticMeshVertexBuffer.BindTangentVertexBuffer(&this->VertexFactory, Data);
			this->StaticMeshVertexBuffers.StaticMeshVertexBuffer.BindPackedTexCoordVertexBuffer(&this->VertexFactory, Data);
			this->StaticMeshVertexBuffers.StaticMeshVertexBuffer.BindLightMapVertexBuffer(&this->VertexFactory, Data, 0);
			this->StaticMeshVertexBuffers.ColorVertexBuffer.BindColorVertexBuffer(&this->VertexFactory, Data);
			this->VertexFactory.SetData(RHICmdList, Data);

			this->VertexFactory.InitResource(RHICmdList);
		});
	}

	virtual ~FEditorWidgetBillboardProxy() override
	{
		VertexFactory.ReleaseResource();
		StaticMeshVertexBuffers.PositionVertexBuffer.ReleaseResource();
		StaticMeshVertexBuffers.StaticMeshVertexBuffer.ReleaseResource();
		StaticMeshVertexBuffers.ColorVertexBuffer.ReleaseResource();
	}

	// FPrimitiveSceneProxy interface.
	
	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_DisplayClusterWidgetComponent_GetDynamicMeshElements);
		
		const int32 ElementStride = 4;
		const int32 WorstCaseVertexBufferSize = ElementStride * Views.Num();
		
		if (!Component.IsValid() || WorstCaseVertexBufferSize <= 0)
		{
			return;
		}
		
		StaticMeshVertexBuffers.PositionVertexBuffer.Init(WorstCaseVertexBufferSize);
		StaticMeshVertexBuffers.StaticMeshVertexBuffer.Init(WorstCaseVertexBufferSize, 1);
		StaticMeshVertexBuffers.ColorVertexBuffer.Init(WorstCaseVertexBufferSize);
		
		if (const UTextureRenderTarget2D* RenderTarget = Component->GetRenderTarget())
		{
			if (const UMaterialInstanceDynamic* MaterialInstance = Component->GetMaterialInstance())
			{
				if (const FMaterialRenderProxy* ParentMaterialProxy = MaterialInstance->GetRenderProxy())
				{
					for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
					{
						if (VisibilityMap & (1 << ViewIndex))
						{
							// Render the material of the render target as a billboard always facing the view
						
							const FSceneView* View = Views[ViewIndex];

							const FMatrix WorldToLocal = GetLocalToWorld().InverseFast();
							const FMatrix ViewToLocal = View->ViewMatrices.GetViewToWorld() * WorldToLocal;
							const FVector TangentX = -ViewToLocal.TransformVector(FVector(1.0f, 0.0f, 0.0f));
							const FVector TangentY = -ViewToLocal.TransformVector(FVector(0.0f, 1.0f, 0.0f));
							const FVector TangentZ = -ViewToLocal.TransformVector(FVector(0.0f, 0.0f, 1.0f));

							// Evaluate the size of the sprite.
							const float SizeX = (RenderTarget->SizeX * Component->GetWidgetScale()) * 0.25f;
							const float SizeY = (RenderTarget->SizeY * Component->GetWidgetScale()) * 0.25f;

							const float WorldSizeX = SizeX;
							const float WorldSizeY = SizeY;

							FLinearColor Color = FLinearColor::White;
				
							const int WriteOffset = ElementStride * ViewIndex;
							for (uint32 VertexIndex = 0; VertexIndex < 4; ++VertexIndex)
							{
								const int WriteIndex = WriteOffset + VertexIndex;
								StaticMeshVertexBuffers.StaticMeshVertexBuffer.SetVertexTangents(WriteIndex,
									static_cast<FVector3f>(-TangentX),
									static_cast<FVector3f>(TangentY),
									static_cast<FVector3f>(TangentZ));
								StaticMeshVertexBuffers.ColorVertexBuffer.VertexColor(WriteIndex) = Color.ToFColor(true);
							}

							// Set up the sprite vertex positions and texture coordinates.
							StaticMeshVertexBuffers.PositionVertexBuffer.VertexPosition(WriteOffset + 0) = FVector3f(-WorldSizeX * TangentY + +WorldSizeY * TangentX);
							StaticMeshVertexBuffers.PositionVertexBuffer.VertexPosition(WriteOffset + 1) = FVector3f(+WorldSizeX * TangentY + +WorldSizeY * TangentX);
							StaticMeshVertexBuffers.PositionVertexBuffer.VertexPosition(WriteOffset + 2) = FVector3f(-WorldSizeX * TangentY + -WorldSizeY * TangentX);
							StaticMeshVertexBuffers.PositionVertexBuffer.VertexPosition(WriteOffset + 3) = FVector3f(+WorldSizeX * TangentY + -WorldSizeY * TangentX);

							StaticMeshVertexBuffers.StaticMeshVertexBuffer.SetVertexUV(WriteOffset + 0, 0, FVector2f(0, 0));
							StaticMeshVertexBuffers.StaticMeshVertexBuffer.SetVertexUV(WriteOffset + 1, 0, FVector2f(0, 1));
							StaticMeshVertexBuffers.StaticMeshVertexBuffer.SetVertexUV(WriteOffset + 2, 0, FVector2f(1, 0));
							StaticMeshVertexBuffers.StaticMeshVertexBuffer.SetVertexUV(WriteOffset + 3, 0, FVector2f(1, 1));

							// Set up the FMeshElement.
							FMeshBatch& Mesh = Collector.AllocateMesh();

							Mesh.VertexFactory = &VertexFactory;
							Mesh.MaterialRenderProxy = ParentMaterialProxy;
							Mesh.LCI = nullptr;
							Mesh.ReverseCulling = IsLocalToWorldDeterminantNegative();
							Mesh.CastShadow = false;
							Mesh.DepthPriorityGroup = GetDepthPriorityGroup(View);
							Mesh.Type = PT_TriangleStrip;
							Mesh.bDisableBackfaceCulling = true;

							// Set up the FMeshBatchElement.
							FMeshBatchElement& BatchElement = Mesh.Elements[0];
							BatchElement.IndexBuffer = nullptr;
							BatchElement.FirstIndex = 0;
							BatchElement.MinVertexIndex = 0;
							BatchElement.MaxVertexIndex = 3;
							BatchElement.NumPrimitives = 2;
							BatchElement.BaseVertexIndex = WriteOffset;
							BatchElement.bPreserveInstanceOrder = false;

							Mesh.bCanApplyViewModeOverrides = true;
							Mesh.bUseWireframeSelectionColoring = IsSelected();

							Collector.AddMesh(ViewIndex, Mesh);
						}
					}
				}
			}

			FLocalVertexFactory* VertexFactoryPtr = &VertexFactory;
			FRHICommandListBase& RHICmdList = Collector.GetRHICommandList();

			StaticMeshVertexBuffers.PositionVertexBuffer.UpdateRHI(RHICmdList);
			StaticMeshVertexBuffers.StaticMeshVertexBuffer.UpdateRHI(RHICmdList);
			StaticMeshVertexBuffers.ColorVertexBuffer.UpdateRHI(RHICmdList);

			FLocalVertexFactory::FDataType Data;
			StaticMeshVertexBuffers.PositionVertexBuffer.BindPositionVertexBuffer(VertexFactoryPtr, Data);
			StaticMeshVertexBuffers.StaticMeshVertexBuffer.BindTangentVertexBuffer(VertexFactoryPtr, Data);
			StaticMeshVertexBuffers.StaticMeshVertexBuffer.BindPackedTexCoordVertexBuffer(VertexFactoryPtr, Data);
			StaticMeshVertexBuffers.StaticMeshVertexBuffer.BindLightMapVertexBuffer(VertexFactoryPtr, Data, 0);
			StaticMeshVertexBuffers.ColorVertexBuffer.BindColorVertexBuffer(VertexFactoryPtr, Data);
			VertexFactoryPtr->SetData(RHICmdList, Data);
			VertexFactoryPtr->UpdateRHI(RHICmdList);
		}
	}

	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
	{
		FPrimitiveViewRelevance Result;
		if (Component.IsValid() && Component->GetMaterialInstance())
		{
			Component->GetMaterialInstance()->GetRelevance_Concurrent(GetScene().GetShaderPlatform()).SetPrimitiveViewRelevance(Result);
		}
		
		Result.bDrawRelevance = IsShown(View) && View->Family->EngineShowFlags.WidgetComponents;
		Result.bDynamicRelevance = true;
		Result.bShadowRelevance = IsShadowCast(View);
		Result.bEditorPrimitiveRelevance = UseEditorCompositing(View);
		
		return Result;
	}

	virtual bool CanBeOccluded() const override
	{
		return false;
	}

	virtual void GetLightRelevance(const FLightSceneProxy* LightSceneProxy, bool& bDynamic, bool& bRelevant, bool& bLightMapped, bool& bShadowMapped) const override
	{
		bDynamic = false;
		bRelevant = false;
		bLightMapped = false;
		bShadowMapped = false;
	}
	
	virtual uint32 GetMemoryFootprint(void) const override { return(sizeof(*this) + GetAllocatedSize()); }
	
	uint32 GetAllocatedSize(void) const { return( FPrimitiveSceneProxy::GetAllocatedSize() ); }
	
	// End of FPrimitiveSceneProxy interface.
	
private:
	/** The buffer containing vertex data. */
	mutable FStaticMeshVertexBuffers StaticMeshVertexBuffers;
	mutable FLocalVertexFactory VertexFactory;
	/** The owning component. */
	TWeakObjectPtr<const UDisplayClusterWidgetComponent> Component;
};

#endif

UDisplayClusterWidgetComponent::UDisplayClusterWidgetComponent()
{
#if WITH_EDITOR
	FWorldDelegates::OnWorldCleanup.AddUObject(this, &UDisplayClusterWidgetComponent::OnWorldCleanup);
#endif
}

UDisplayClusterWidgetComponent::~UDisplayClusterWidgetComponent()
{
#if WITH_EDITOR
	FWorldDelegates::OnWorldCleanup.RemoveAll(this);
#endif
}

void UDisplayClusterWidgetComponent::SetWidgetScale(float NewValue)
{
	WidgetScale = NewValue;
}

void UDisplayClusterWidgetComponent::OnRegister()
{
	// Set this prior to registering the scene component so that bounds are calculated correctly.
	CurrentDrawSize = DrawSize;

	UPrimitiveComponent::OnRegister();

#if !UE_SERVER
	FWorldDelegates::LevelRemovedFromWorld.AddUObject(this, &ThisClass::OnLevelRemovedFromWorld);

	if ( !IsRunningDedicatedServer() )
	{
		const bool bIsGameWorld = GetWorld()->IsGameWorld();
		if ( Space != EWidgetSpace::Screen )
		{
			if ( CanReceiveHardwareInput() && bIsGameWorld )
			{
				TSharedPtr<SViewport> GameViewportWidget = GEngine->GetGameViewportWidget();
				RegisterHitTesterWithViewport(GameViewportWidget);
			}

			if ( !WidgetRenderer && !GUsingNullRHI )
			{
				WidgetRenderer = new FWidgetRenderer(bApplyGammaCorrection);
			}
		}

		BodySetup = nullptr;

		// No editor init -- optimization
	}
#endif // !UE_SERVER
}

void UDisplayClusterWidgetComponent::OnUnregister()
{
#if !UE_SERVER
	FWorldDelegates::LevelRemovedFromWorld.RemoveAll(this);

	if ( GetWorld()->IsGameWorld() )
	{
		TSharedPtr<SViewport> GameViewportWidget = GEngine->GetGameViewportWidget();
		if ( GameViewportWidget.IsValid() )
		{
			UnregisterHitTesterWithViewport(GameViewportWidget);
		}
	}
#endif

	// No editor release -- optimization

	UPrimitiveComponent::OnUnregister();
}

FPrimitiveSceneProxy* UDisplayClusterWidgetComponent::CreateSceneProxy()
{
#if WITH_EDITOR
	if (GEditor && !GetOwner()->HasAnyFlags(RF_Transient) && WidgetRenderer)
	{
		// Use our custom widget billboard proxy if this is the level instance in the editor,
		// otherwise this is either handled by the default scene proxy (-game) or manually in
		// FDisplayClusterLightCardEditorViewportClient.
		
		RequestRenderUpdate();
		LastWidgetRenderTime = 0;
	
		return new FEditorWidgetBillboardProxy(this);
	}
#endif
	
	return Super::CreateSceneProxy();
}

#if WITH_EDITOR
void UDisplayClusterWidgetComponent::OnWorldCleanup(UWorld* InWorld, bool bSessionEnded, bool bCleanupResources)
{
	const UWorld* World = GetWorld();
	
	if (World == InWorld && bCleanupResources && World != nullptr)
	{
		if (!World->IsGameWorld())
		{
			// Prevents crash when a world is being cleaned up and CheckForWorldGCLeaks is called.
			ReleaseResources();
		}
	}
}
#endif
