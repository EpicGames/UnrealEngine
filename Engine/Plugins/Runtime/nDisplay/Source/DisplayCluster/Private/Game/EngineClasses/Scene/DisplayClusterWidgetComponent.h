// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/WidgetComponent.h"

#include "DisplayClusterWidgetComponent.generated.h"

/**
 * Extend the widget component to optimize when the widget is initialized and released.
 */
UCLASS(transient, NotBlueprintable, NotBlueprintType)
class UDisplayClusterWidgetComponent final : public UWidgetComponent
{
	GENERATED_BODY()
public:
	UDisplayClusterWidgetComponent();
	virtual ~UDisplayClusterWidgetComponent() override;

	void SetWidgetScale(float NewValue);
	float GetWidgetScale() const { return WidgetScale; }
private:
	virtual void OnRegister() override;
	virtual void OnUnregister() override;

	virtual FPrimitiveSceneProxy* CreateSceneProxy() override;

#if WITH_EDITOR
	void OnWorldCleanup(UWorld* InWorld, bool bSessionEnded, bool bCleanupResources);
#endif

private:
	float WidgetScale = 1.f;
};
