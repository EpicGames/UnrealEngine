// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Game/IDisplayClusterGameManager.h"
#include "IPDisplayClusterManager.h"


/**
 * Game manager private interface
 */
class IPDisplayClusterGameManager
	: public IDisplayClusterGameManager
	, public IPDisplayClusterManager
{
public:
	virtual ~IPDisplayClusterGameManager() = default;
};
