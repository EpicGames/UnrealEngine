// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterAppExit.h"
#include "Misc/DisplayClusterLog.h"
#include "Engine/GameEngine.h"

#include "Async/Async.h"

#if WITH_EDITOR
#include "Editor/UnrealEdEngine.h"
#include "UnrealEdGlobals.h"
#endif


void FDisplayClusterAppExit::ExitApplication(const FString& Msg, EExitType ExitType)
{
	if (GEngine && GEngine->IsEditor())
	{
#if WITH_EDITOR
		UE_LOGF(LogDisplayClusterModule, Log, "PIE end requested - %ls", *Msg);
		GUnrealEd->RequestEndPlayMap();
#endif
	}
	else
	{
		if (ExitType == EExitType::Normal)
		{
			if (!IsEngineExitRequested())
			{
				UE_LOGF(LogDisplayClusterModule, Log, "Exit requested - %ls", *Msg);

				if (IsInGameThread())
				{
					FPlatformMisc::RequestExit(false);
				}
				else
				{
					// For some reason UE generates crash info if FPlatformMisc::RequestExit gets called
					// from a thread other than GameThread. Since it may be called from the networking
					// session threads (failover pipeline), we don't want to generate unnecessary crash reports.
					AsyncTask(ENamedThreads::GameThread, []()
						{
							FPlatformMisc::RequestExit(false);
						});
				}
			}
		}
		else if(ExitType == EExitType::KillImmediately)
		{
			UE_LOGF(LogDisplayClusterModule, Error, "KillImmediately requested - %ls", *Msg);

			// Linux platform layer doesn't support force kill. Use normal exit instead.
#if PLATFORM_LINUX
			ExitApplication(Msg, EExitType::Normal);
#else
			FProcHandle hProc = FPlatformProcess::OpenProcess(FPlatformProcess::GetCurrentProcessId());
			FPlatformProcess::TerminateProc(hProc, true);
#endif
		}
	}
}
