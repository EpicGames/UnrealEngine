// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"


/**
 * Auxiliary class. Responsible for terminating application.
 */
class FDisplayClusterAppExit
{
public:
	enum class EExitType
	{
		// Delicate UE exit, full resource cleaning.
		Normal,
		// Kills current process. No resource cleaning performed.
		KillImmediately,
	};

public:
	static void ExitApplication(const FString& Msg, EExitType ExitType = FDisplayClusterAppExit::EExitType::Normal);
};
