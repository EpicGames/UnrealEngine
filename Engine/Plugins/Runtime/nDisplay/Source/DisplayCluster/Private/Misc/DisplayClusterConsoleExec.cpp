// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/DisplayClusterConsoleExec.h"

#include "Cluster/DisplayClusterClusterEvent.h"
#include "Engine/Engine.h"
#include "Engine/LocalPlayer.h"
#include "Features/IModularFeatures.h"
#include "Misc/DisplayClusterLog.h"


bool FDisplayClusterConsoleExec::Exec(const FDisplayClusterClusterEventJson& InEvent)
{
	// Extract command
	const FString* const ExecString = InEvent.Parameters.Find("ExecString");
	if (!ExecString || ExecString->IsEmpty())
	{
		UE_LOGF(LogDisplayClusterModule, Warning, "ConsoleExec ignoring cluster event with no ExecString");
		return false;
	}

	// Extract executor name
	const FString* const RequestedExecutor = InEvent.Parameters.Find("Executor");

	UE_LOGF(LogDisplayClusterModule, Verbose, "ConsoleExec cluster event: Executor='%ls', ExecString='%ls'", RequestedExecutor ? **RequestedExecutor : TEXT(""), **ExecString);

	// Execute
	return Exec(RequestedExecutor ? **RequestedExecutor : FString(), *ExecString);
}

bool FDisplayClusterConsoleExec::Exec(const FString& InExecutorName, const FString& InExecCommand)
{
	if (InExecCommand.IsEmpty())
	{
		return false;
	}

	UE_LOGF(LogDisplayClusterModule, Log, "ConsoleExec command: Executor='%ls', ExecString='%ls'", *InExecutorName, *InExecCommand);

	if (!InExecutorName.IsEmpty())
	{
		const FName FeatureName = IConsoleCommandExecutor::ModularFeatureName();
		TArray<IConsoleCommandExecutor*> CommandExecutors = IModularFeatures::Get().GetModularFeatureImplementations<IConsoleCommandExecutor>(FeatureName);

		const FName ExecutorName = *InExecutorName;
		for (IConsoleCommandExecutor* CommandExecutor : CommandExecutors)
		{
			if (CommandExecutor->GetName() == ExecutorName)
			{
				return CommandExecutor->Exec(*InExecCommand);
			}
		}

		UE_LOGF(LogDisplayClusterModule, Warning, "ConsoleExec couldn't find requested executor: %ls", *InExecutorName);
		return false;
	}
	else
	{
		// With no explicit executor, try to route this as an Unreal console
		// command as best we can. Available context is limited here.
		if (GEngine)
		{
			if (ULocalPlayer* const Player = GEngine->GetDebugLocalPlayer())
			{
				return Player->Exec(Player->GetWorld(), *InExecCommand, *GLog);
			}
			else
			{
				return GEngine->Exec(nullptr, *InExecCommand, *GLog);
			}
		}
	}

	return false;
}
