// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterGlobals.h"


IPDisplayCluster* GDisplayCluster = nullptr;

