// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterLog.h"

// Plugin-wide log categories
DEFINE_LOG_CATEGORY(LogDisplayClusterBarrier);
DEFINE_LOG_CATEGORY(LogDisplayClusterBarrierGP);
DEFINE_LOG_CATEGORY(LogDisplayClusterBlueprint);
DEFINE_LOG_CATEGORY(LogDisplayClusterCluster);
DEFINE_LOG_CATEGORY(LogDisplayClusterConfig);
DEFINE_LOG_CATEGORY(LogDisplayClusterEngine);
DEFINE_LOG_CATEGORY(LogDisplayClusterFailover);
DEFINE_LOG_CATEGORY(LogDisplayClusterGame);
DEFINE_LOG_CATEGORY(LogDisplayClusterModule);
DEFINE_LOG_CATEGORY(LogDisplayClusterNetwork);
DEFINE_LOG_CATEGORY(LogDisplayClusterRender);
DEFINE_LOG_CATEGORY(LogDisplayClusterRenderSync);
DEFINE_LOG_CATEGORY(LogDisplayClusterViewport);
