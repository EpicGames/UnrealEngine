// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"
#include "Stats/Stats.h"

// Module-wide log categories
DECLARE_LOG_CATEGORY_EXTERN(LogDisplayClusterBarrier,    Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogDisplayClusterBarrierGP,  Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogDisplayClusterBlueprint,  Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogDisplayClusterCluster,    Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogDisplayClusterConfig,     Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogDisplayClusterEngine,     Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogDisplayClusterFailover,   Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogDisplayClusterGame,       Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogDisplayClusterModule,     Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogDisplayClusterNetwork,    Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogDisplayClusterRender,     Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogDisplayClusterRenderSync, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogDisplayClusterViewport,   Log, All);


DECLARE_STATS_GROUP(TEXT("nDisplay"), STATGROUP_NDisplay, STATCAT_Advanced)
