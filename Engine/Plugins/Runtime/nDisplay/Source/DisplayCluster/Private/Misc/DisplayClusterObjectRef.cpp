// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/DisplayClusterObjectRef.h"

USceneComponent* FDisplayClusterSceneComponentRef::GetOrFindSceneComponent() const
{
	FScopeLock lock(&DataGuard);

	if (!IsDefinedSceneComponent())
	{
		return nullptr;
	}

	if (!IsSceneComponentValid())
	{
		ComponentPtr.Reset();

		AActor* Actor = GetOrFindSceneActor();
		if (Actor)
		{
			for (UActorComponent* ItActorComponent : Actor->GetComponents())
			{
				// Component can be null when a viewport is deleted and GC occurs.
				if (ItActorComponent && ItActorComponent->GetFName() == ComponentName)
				{
					USceneComponent* SceneComponent = Cast<USceneComponent>(ItActorComponent);
					if (SceneComponent)
					{
						ComponentPtr = TWeakObjectPtr<USceneComponent>(SceneComponent);
						return SceneComponent;
					}
				}
			}
			// Component not found. Actor structure changed??
		}
	}

	return ComponentPtr.Get();
}
