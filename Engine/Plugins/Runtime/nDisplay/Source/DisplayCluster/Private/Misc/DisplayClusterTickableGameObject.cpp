// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/DisplayClusterTickableGameObject.h"

void FDisplayClusterTickableGameObject::Tick(float DeltaTime)
{
	if (TickEvent.IsBound())
	{
		TickEvent.Broadcast(DeltaTime);
	}
}
