// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/DisplayClusterTypesConverter.h"
#include "Network/DisplayClusterNetworkTypes.h"
#include "Network/Barrier/IDisplayClusterBarrier.h"


//////////////////////////////////////////////////////////////////////////////////////////////
// TYPE --> STRING
//////////////////////////////////////////////////////////////////////////////////////////////
template <> inline FString DisplayClusterTypesConverter::ToString<>(const EDisplayClusterCommResult& From)
{
	switch (From)
	{
	case EDisplayClusterCommResult::Ok:
		return FString("Ok");
	case EDisplayClusterCommResult::WrongRequestData:
		return FString("WrongRequestData");
	case EDisplayClusterCommResult::WrongResponseData:
		return FString("WrongResponseData");
	case EDisplayClusterCommResult::NetworkError:
		return FString("NetworkError");
	case EDisplayClusterCommResult::NotImplemented:
		return FString("NotImplemented");
	case EDisplayClusterCommResult::NotAllowed:
		return FString("NotAllowed");
	case EDisplayClusterCommResult::InternalError:
		return FString("InternalError");
	default:
		return FString("Unknown");
	}
}

template <> inline FString DisplayClusterTypesConverter::ToString<>(const EDisplayClusterBarrierWaitResult& From)
{
	switch (From)
	{
	case EDisplayClusterBarrierWaitResult::Ok:
		return FString("Ok");
	case EDisplayClusterBarrierWaitResult::NotActive:
		return FString("NotActive");
	case EDisplayClusterBarrierWaitResult::TimeOut:
		return FString("TimeOut");
	case EDisplayClusterBarrierWaitResult::NotAllowed:
		return FString("NotAllowed");
	default:
		return FString("Unknown");
	}
}


//////////////////////////////////////////////////////////////////////////////////////////////
// STRING --> TYPE
//////////////////////////////////////////////////////////////////////////////////////////////
