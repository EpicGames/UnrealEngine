// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/DisplayClusterWatchdogTimer.h"
#include "Misc/DisplayClusterLog.h"
#include "Misc/ScopeLock.h"
#include "HAL/RunnableThread.h"


FDisplayClusterWatchdogTimer::FDisplayClusterWatchdogTimer(const FString& InTimerName)
	: TimerName(InTimerName)
{
	ThreadObj.Reset(FRunnableThread::Create(this, *(TimerName + FString("_thread")), 128 * 1024, TPri_Normal));
}

FDisplayClusterWatchdogTimer::~FDisplayClusterWatchdogTimer()
{
	// Stop working thread
	Stop();
}


bool FDisplayClusterWatchdogTimer::SetTimer(const uint32 InTimeout)
{
	FScopeLock Lock(&InternalsCS);

	// Check if timer was set already
	if (bIsTimerSet)
	{
		return false;
	}

	// Update timer internals
	bIsTimerSet = true;
	Timeout = InTimeout;

	// Start timer
	EvtStartTimer->Trigger();

	return true;
}

void FDisplayClusterWatchdogTimer::ResetTimer()
{
	FScopeLock Lock(&InternalsCS);

	// Check if timer was set previously
	if (bIsTimerSet)
	{
		// Set 'reset' flag to let the working thread know we're reseting the timer
		bTimerWasReset = true;
		// Release waitable object
		EvtWaitableObject->Trigger();
	}
}

uint32 FDisplayClusterWatchdogTimer::Run()
{
	UE_LOGF(LogDisplayClusterNetwork, Log, "Watchdog %ls: working thread started", *TimerName);

	while (!bThreadExitRequested)
	{
		// Wait for timer start command
		EvtStartTimer->Wait();

		// Check if need to stop the thread
		if (bThreadExitRequested)
		{
			break;
		}

		UE_LOGF(LogDisplayClusterNetwork, Verbose, "Watchdog %ls: timer set - %u ms", *TimerName, Timeout);

		// Wait for specified amount of time
		EvtWaitableObject->Wait(Timeout);

		// Check again if need to stop the thread
		if (bThreadExitRequested)
		{
			break;
		}

		{
			FScopeLock Lock(&InternalsCS);

			// If the timer was not reset manually, we need to notify the listeners about timeout
			if (!bTimerWasReset)
			{
				UE_LOGF(LogDisplayClusterNetwork, Log, "Watchdog %ls: timeout!", *TimerName);
				OnWatchdogTimeOut().Broadcast();
			}
			else
			{
				UE_LOGF(LogDisplayClusterNetwork, Verbose, "Watchdog %ls: timer reset", *TimerName);
			}

			// Reset internal state so the timer is ready for new start
			bIsTimerSet    = false;
			bTimerWasReset = false;

			EvtStartTimer->Reset();
			EvtWaitableObject->Reset();
		}
	}

	UE_LOGF(LogDisplayClusterNetwork, Log, "Watchdog %ls: working thread finished", *TimerName);

	return 0;
}

void FDisplayClusterWatchdogTimer::Stop()
{
	// Let the working thread know it has to stop
	bThreadExitRequested = true;

	// Release sync objects to unblock the working thread
	EvtStartTimer->Trigger();
	EvtWaitableObject->Trigger();

	// Wait unless working thread is finished
	if (ThreadObj)
	{
		ThreadObj->WaitForCompletion();
	}
}
