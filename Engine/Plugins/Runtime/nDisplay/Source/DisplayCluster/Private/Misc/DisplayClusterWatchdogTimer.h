// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Delegates/DelegateCombinations.h"

#include "HAL/Event.h"
#include "HAL/Runnable.h"


/**
 * Watchdog timer
 */
class FDisplayClusterWatchdogTimer
	: protected FRunnable
{
public:
	FDisplayClusterWatchdogTimer(const FString& InTimerName);
	~FDisplayClusterWatchdogTimer();

public:
	// Start watchdog timer with specified timeout duration (ms)
	bool SetTimer(const uint32 InTimeout);
	// Reset timer before it's timed out
	void ResetTimer();

	// Returns timer name
	const FString& GetName() const
	{
		return TimerName;
	}

	// Notification interface
	DECLARE_EVENT(FDisplayClusterWatchdogTimer, FWatchdogTimeoutEvent);
	FWatchdogTimeoutEvent& OnWatchdogTimeOut()
	{
		return WatchdogTimeOutEvent;
	}

protected:
	//////////////////////////////////////////////////////////////////////////////////////////////
	// FRunnable
	//////////////////////////////////////////////////////////////////////////////////////////////
	virtual uint32 Run() override;
	virtual void Stop() override final;

private:
	// Timer name
	const FString TimerName;

	// Timer state
	bool bIsTimerSet = false;
	// Thread exit request flag
	bool bThreadExitRequested = false;
	// Timer reset flag
	bool bTimerWasReset = false;
	// Timer timeout
	uint32 Timeout = 0;

	// Holds the thread object
	TUniquePtr<FRunnableThread> ThreadObj;

	// Timeout notification
	FWatchdogTimeoutEvent WatchdogTimeOutEvent;

	// Working thread control events
	FEventRef EvtStartTimer     { EEventMode::ManualReset };
	FEventRef EvtWaitableObject { EEventMode::ManualReset };

private:
	// Critical section for access control
	FCriticalSection InternalsCS;
};
