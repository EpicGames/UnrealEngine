// Copyright Epic Games, Inc. All Rights Reserved.

#include "Network/Barrier/DisplayClusterBarrier.h"

#include "Misc/DisplayClusterLog.h"
#include "Misc/ScopeLock.h"


FDisplayClusterBarrier::FDisplayClusterBarrier(const FString& InName, const TSet<FString>& InCallersAllowed, const uint32 InTimeout)
	: Name(InName)
	, CallersAllowed(InCallersAllowed)
	, Timeout(InTimeout)
	, WatchdogTimer(InName + TEXT("_watchdog"))
{
	UE_LOGF(LogDisplayClusterBarrier, Log, "Initialized barrier '%ls' with timeout %u ms and threads limit: %u", *Name, Timeout, CallersAllowed.Num());

	int32 Idx = 0;
	for (const FString& CallerId : CallersAllowed)
	{
		UE_LOGF(LogDisplayClusterBarrier, Verbose, "Barrier '%ls': client (%d): '%ls'", *Name, Idx++, *CallerId);
	}

	// Subscribe for timeout events
	WatchdogTimer.OnWatchdogTimeOut().AddRaw(this, &FDisplayClusterBarrier::HandleBarrierTimeout);
}

FDisplayClusterBarrier::~FDisplayClusterBarrier()
{
	// Release threads if there any
	Deactivate();
}


bool FDisplayClusterBarrier::Activate()
{
	FScopeLock Lock(&DataCS);

	UE_LOGF(LogDisplayClusterBarrier, Log, "Barrier '%ls': activating...", *Name);

	if (!bActive)
	{
		bActive = true;
		CallersAwaiting.Reset();

		// No exit allowed
		EventOutputGateOpen->Reset();
		// Allow join
		EventInputGateOpen->Trigger();
	}

	return true;
}

void FDisplayClusterBarrier::Deactivate()
{
	FScopeLock Lock(&DataCS);

	if (bActive)
	{
		UE_LOGF(LogDisplayClusterBarrier, Log, "Barrier '%ls': deactivating...", *Name);

		bActive = false;

		// Release all threads that are currently at the barrier
		EventInputGateOpen->Trigger();
		EventOutputGateOpen->Trigger();

		// No more threads awaiting
		CallersAwaiting.Reset();

		// And reset timer of course
		WatchdogTimer.ResetTimer();
	}
}

bool FDisplayClusterBarrier::IsActivated() const
{
	FScopeLock Lock(&DataCS);
	return bActive;
}


EDisplayClusterBarrierWaitResult FDisplayClusterBarrier::Wait(const FString& CallerId, double* ThreadWaitTime /*= nullptr*/, double* BarrierWaitTime /*= nullptr*/)
{
	UE_LOGF(LogDisplayClusterBarrier, VeryVerbose, "Barrier '%ls': caller arrived '%ls'", *Name, *CallerId);

	double ThreadWaitTimeStart = 0;

	{
		FScopeLock LockEntrance(&EntranceCS);

		// Wait unless the barrier allows new threads to join. This will happen
		// once all threads from the previous sync iteration leave the barrier,
		// or the barrier gets deactivated.
		EventInputGateOpen->Wait();

		{
			FScopeLock LockData(&DataCS);

			// Check if this thread has been previously dropped
			if (CallersTimedout.Contains(CallerId))
			{
				UE_LOGF(LogDisplayClusterBarrier, Verbose, "Barrier '%ls': caller '%ls' not allowed to join, it has been timed out previously", *Name, *CallerId);
				return EDisplayClusterBarrierWaitResult::TimeOut;
			}

			// Check if the barrier is active
			if (!bActive)
			{
				UE_LOGF(LogDisplayClusterBarrier, Verbose, "Barrier '%ls': not active", *Name);
				return EDisplayClusterBarrierWaitResult::NotActive;
			}

			// Check if this thread is allowed to sync at this barrier
			if (!CallersAllowed.Contains(CallerId))
			{
				UE_LOGF(LogDisplayClusterBarrier, Verbose, "Barrier '%ls': caller '%ls' not allowed to join, no permission", *Name, *CallerId);
				return EDisplayClusterBarrierWaitResult::NotAllowed;
			}

			// Register caller
			CallersAwaiting.Add(CallerId);

			// Make sure the barrier was not deactivated while this thread was waiting at the input gate
			if (bActive)
			{
				// Fixate awaiting start for this particular thread
				ThreadWaitTimeStart = FPlatformTime::Seconds();

				// In case this thread came first to the barrier, we need:
				// - to fixate barrier awaiting start time
				// - start watchdog timer
				if (CallersAwaiting.Num() == 1)
				{
					UE_LOGF(LogDisplayClusterBarrier, Verbose, "Barrier '%ls': sync start, cycle %llu", *Name, SyncCycleCounter);

					// Prepare for new sync iteration
					HandleBarrierPreSyncStart();

					BarrierWaitTimeStart = ThreadWaitTimeStart;
					WatchdogTimer.SetTimer(Timeout);
				}

				UE_LOGF(LogDisplayClusterBarrier, Verbose, "Barrier '%ls': awaiting threads amount - %d", *Name, CallersAwaiting.Num());

				// In case this thread is the last one the barrier is awaiting for, we need:
				// - to fixate barrier awaiting finish time
				// - to open the output gate (release the barrier)
				// - to close the input gate
				// - reset watchdog timer
				if (CallersAwaiting.Num() == CallersAllowed.Num())
				{
					BarrierWaitTimeFinish = FPlatformTime::Seconds();
					BarrierWaitTimeOverall = BarrierWaitTimeFinish - BarrierWaitTimeStart;

					UE_LOGF(LogDisplayClusterBarrier, Verbose, "Barrier '%ls': sync end, cycle %llu, barrier wait time %f", *Name, SyncCycleCounter, BarrierWaitTimeOverall);

					// Increment cycle counter
					++SyncCycleCounter;

					// All callers are here, pet the watchdog
					WatchdogTimer.ResetTimer();

					// Process sync done before allowing the threads to leave
					HandleBarrierPreSyncEnd();

					// Close input gate, open output gate
					EventInputGateOpen->Reset();
					EventOutputGateOpen->Trigger();
				}
			}
		}
	}

	// Wait for the barrier to open
	EventOutputGateOpen->Wait();

	// Fixate awaiting finish for this particular thread
	const double ThreadWaitTimeFinish = FPlatformTime::Seconds();

	{
		UE_LOGF(LogDisplayClusterBarrier, Verbose, "Barrier '%ls': caller '%ls' is leaving the barrier", *Name, *CallerId);

		{
			FScopeLock LockData(&DataCS);

			// In case there are any 'simulated' callers that have been dropped actually,
			// modify "Awaiting" and "Allowed" lists so they could properly handle the next sync cycle.
			if (!CallersToForget.IsEmpty())
			{
				CallersAwaiting = CallersAwaiting.Difference(CallersToForget);
				CallersAllowed  = CallersAllowed.Difference(CallersToForget);
				CallersToForget.Reset();
			}

			// Unregister caller
			CallersAwaiting.Remove(CallerId);

			// Make sure the barrier was not deactivated while this thread was waiting at the output gate
			if (bActive)
			{
				// In case this thread is leaving last, close output and open input
				if (CallersAwaiting.IsEmpty())
				{
					EventOutputGateOpen->Reset();
					EventInputGateOpen->Trigger();
				}
			}
		}
	}

	// Export barrier overall waiting time
	if (BarrierWaitTime)
	{
		*BarrierWaitTime = BarrierWaitTimeOverall;
	}

	// Export thread waiting time
	if (ThreadWaitTime)
	{
		*ThreadWaitTime = ThreadWaitTimeFinish - ThreadWaitTimeStart;
	}

	UE_LOGF(LogDisplayClusterBarrier, VeryVerbose, "Barrier '%ls': caller left '%ls'", *Name, *CallerId);

	return EDisplayClusterBarrierWaitResult::Ok;
}

EDisplayClusterBarrierWaitResult FDisplayClusterBarrier::WaitWithData(const FString& CallerId, const TArray<uint8>& RequestData, TArray<uint8>& OutResponseData, double* OutThreadWaitTime /* = nullptr */, double* OutBarrierWaitTime /* = nullptr */)
{
	{
		FScopeLock Lock(&CommDataCS);
		// Store request data so it can be used once all the threads arrived
		ClientsRequestData.Emplace(CallerId, RequestData);
	}

	// Wait at the barrier
	const EDisplayClusterBarrierWaitResult WaitResult = Wait(CallerId, OutThreadWaitTime, OutBarrierWaitTime);

	{
		FScopeLock Lock(&CommDataCS);

		if (TArray<uint8>* const FoundThreadResponseData = ClientsResponseData.Find(CallerId))
		{
			OutResponseData = MoveTemp(*FoundThreadResponseData);
			ClientsResponseData.Remove(CallerId);
		}
	}

	return WaitResult;
}

void FDisplayClusterBarrier::UnregisterSyncCaller(const FString& CallerId)
{
	UE_LOGF(LogDisplayClusterBarrier, Log, "Barrier '%ls': unregistering caller '%ls'...", *Name, *CallerId);

	FScopeLock Lock(&DataCS);

	// Ignore if it has been processed it already
	if (CallersDropped.Contains(CallerId))
	{
		return;
	}
	else
	{
		CallersDropped.Add(CallerId);
	}

	// If synchronization cycle has not started yet, just remove it from 'Allowed' list
	if (CallersAwaiting.IsEmpty())
	{
		CallersAllowed.Remove(CallerId);
		return;
	}
	// Otherwise, simulate this node just arrived to synchronize. But remember that to clean up later.
	else
	{
		CallersAwaiting.Add(CallerId);
		CallersToForget.Add(CallerId);
	}

	// In case it's a last missing caller, we need to open the barrier
	if (CallersAwaiting.Num() == CallersAllowed.Num())
	{
		// In case there are any 'simulated' callers that have been dropped actually,
		// modify "Awaiting" and "Allowed" lists so they could properly handle the next sync cycle.
		if (!CallersToForget.IsEmpty())
		{
			CallersAwaiting = CallersAwaiting.Difference(CallersToForget);
			CallersAllowed  = CallersAllowed.Difference(CallersToForget);
			CallersToForget.Reset();
		}

		BarrierWaitTimeFinish  = FPlatformTime::Seconds();
		BarrierWaitTimeOverall = BarrierWaitTimeFinish - BarrierWaitTimeStart;

		UE_LOGF(LogDisplayClusterBarrier, Verbose, "Barrier '%ls': sync end, barrier wait time %f", *Name, BarrierWaitTimeOverall);

		// All callers here, pet the watchdog
		WatchdogTimer.ResetTimer();

		// Close the input gate, and open the output gate
		EventInputGateOpen->Reset();
		EventOutputGateOpen->Trigger();
	}
}

void FDisplayClusterBarrier::HandleBarrierPreSyncStart()
{
}

void FDisplayClusterBarrier::HandleBarrierPreSyncEnd()
{
	// Prepare callback data and call the handler
	FDisplayClusterBarrierPreSyncEndDelegateData PreSyncEndCallbackData{ Name, ClientsRequestData, ClientsResponseData };
	BarrierPreSyncEndDelegate.ExecuteIfBound(PreSyncEndCallbackData);

	// We can clean request data now before next iteration
	ClientsRequestData.Empty(CallersAllowed.Num());
}

void FDisplayClusterBarrier::HandleBarrierTimeout()
{
	// Being here means some callers have not come to the barrier yet during specific time period. Those
	// missing callers will be considered as the lost ones. The barrier will continue working with the remaining callers only.

	FScopeLock Lock(&DataCS);

	UE_LOGF(LogDisplayClusterBarrier, Log, "Barrier '%ls': Time out! %d callers missing", *Name, CallersAllowed.Num() - CallersAwaiting.Num());

	// First of all, update the time variables
	BarrierWaitTimeFinish = FPlatformTime::Seconds();
	BarrierWaitTimeOverall = BarrierWaitTimeFinish - BarrierWaitTimeStart;

	// List of the threads that timed out at the current synchronization cycle
	const TSet<FString> CallersTimedOutOnLastSync = CallersAllowed.Difference(CallersAwaiting);
	for (const FString& CallerId : CallersTimedOutOnLastSync)
	{
		UE_LOGF(LogDisplayClusterBarrier, Log, "Barrier '%ls': caller '%ls' was moved to the 'TimedOut' list", *Name, *CallerId);
	}

	// Update timedout and dropped list
	CallersTimedout.Append(CallersTimedOutOnLastSync);
	CallersDropped.Append(CallersTimedOutOnLastSync);

	// Update the list of permitted callers
	CallersAllowed = CallersAwaiting;

	UE_LOGF(LogDisplayClusterBarrier, Log, "Barrier '%ls': new threads limit %d", *Name, CallersAllowed.Num());

	// Notify listeners
	OnBarrierTimeout().Broadcast(Name, CallersTimedOutOnLastSync);

	// Close the input gate, and open the output gate to let the remaining callers go
	EventInputGateOpen->Reset();
	EventOutputGateOpen->Trigger();
}
