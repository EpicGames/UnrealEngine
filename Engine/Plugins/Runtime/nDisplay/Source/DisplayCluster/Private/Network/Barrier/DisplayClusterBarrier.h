// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Network/Barrier/IDisplayClusterBarrier.h"
#include "Misc/DisplayClusterWatchdogTimer.h"

#include "HAL/Event.h"


/**
 * Thread barrier v2
 */
class FDisplayClusterBarrier
	: public IDisplayClusterBarrier
{
public:
	FDisplayClusterBarrier(const FString& Name, const TSet<FString>& CallersAllowed, const uint32 Timeout);
	virtual  ~FDisplayClusterBarrier();

public:
	// Barrier name
	virtual const FString& GetName() const override
	{
		return Name;
	}

	// Activate barrier
	virtual bool Activate() override;
	// Deactivate barrier, no threads will be blocked
	virtual void Deactivate() override;
	// Returns true if the barrier has been activated
	virtual bool IsActivated() const override;

	// Wait until all caller threads arrived
	virtual EDisplayClusterBarrierWaitResult Wait(const FString& CallerId, double* OutThreadWaitTime = nullptr, double* OutBarrierWaitTime = nullptr) override;

	// Wait until all threads arrive (with data)
	virtual EDisplayClusterBarrierWaitResult WaitWithData(const FString& CallerId, const TArray<uint8>& RequestData, TArray<uint8>& OutResponseData, double* OutThreadWaitTime = nullptr, double* OutBarrierWaitTime = nullptr) override;

	// Remove specified caller from the sync pipeline
	virtual void UnregisterSyncCaller(const FString& CallerId) override;

	// Returns barrier PreSyncEnd delegate
	virtual FDisplayClusterBarrierPreSyncEndDelegate& GetPreSyncEndDelegate() override
	{
		return BarrierPreSyncEndDelegate;
	}

	// Barrier timout notification
	virtual FDisplayClusterBarrierTimeoutEvent& OnBarrierTimeout() override
	{
		return BarrierTimeoutEvent;
	}

private:
	// Do some job before starting new sync iteration (opening the entrance gate)
	void HandleBarrierPreSyncStart();

	// Do some job before opening the exit gate
	void HandleBarrierPreSyncEnd();

	// Handler for barrier timouts
	void HandleBarrierTimeout();

private:
	// Barrier name
	const FString Name;
	// Barrier state
	bool bActive = false;


	// Caller that are allowed to join the barrier
	TSet<FString> CallersAllowed;

	// Callers that are already waiting at the barrier
	TSet<FString> CallersAwaiting;

	// Callers that were dropped during synchronization cycle. Holds them temporary to provide correct cycle enclosure.
	TSet<FString> CallersToForget;

	// Cluster Callers that have been timed out previously
	TSet<FString> CallersTimedout;

	// Cluster Callers that have been detached from barrier synchronization
	TSet<FString> CallersDropped;

	// Timeout for the barrier
	const uint32 Timeout = 0;

	// Synchronization cycles counter
	uint64 SyncCycleCounter = 0;

	// Events to control when threads can join (input) and leave (output) the barrier
	FEventRef EventInputGateOpen { EEventMode::ManualReset };
	FEventRef EventOutputGateOpen{ EEventMode::ManualReset };

	// Watchdog timer to detect barrier waiting timeouts
	FDisplayClusterWatchdogTimer WatchdogTimer;

	// PreSyncEnd delegate. It's called when all threads arrived before opening the gate.
	FDisplayClusterBarrierPreSyncEndDelegate BarrierPreSyncEndDelegate;
	// Barrier timeout event
	FDisplayClusterBarrierTimeoutEvent BarrierTimeoutEvent;

	// Request data from the callers
	TMap<FString, TArray<uint8>> ClientsRequestData;
	// Response data for the callers
	TMap<FString, TArray<uint8>> ClientsResponseData;

	// Diagnostics data
	double BarrierWaitTimeStart   = 0;
	double BarrierWaitTimeFinish  = 0;
	double BarrierWaitTimeOverall = 0;

	// Barrier state CS
	mutable FCriticalSection DataCS;
	// Barrier entrance CS
	mutable FCriticalSection EntranceCS;
	// Request/response data CS
	mutable FCriticalSection CommDataCS;
};
