// Copyright Epic Games, Inc. All Rights Reserved.

#include "Network/Barrier/DisplayClusterBarrierFactory.h"
#include "Network/Barrier/DisplayClusterBarrier.h"

#include "Misc/DisplayClusterLog.h"


IDisplayClusterBarrier* FDisplayClusterBarrierFactory::CreateBarrier(const FString& BarrierId, const TSet<FString>& CallerIds, const uint32 Timeout)
{
	UE_LOGF(LogDisplayClusterBarrier, Log, "Instantiating unique barrier '%ls': Threads=%d, Timeout=%u ms", *BarrierId, CallerIds.Num(), Timeout);
	return new FDisplayClusterBarrier(BarrierId, CallerIds, Timeout);
}
