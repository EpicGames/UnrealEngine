// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"

class IDisplayClusterBarrier;


/**
 * Barrier factory
 */
class FDisplayClusterBarrierFactory
{
private:

	FDisplayClusterBarrierFactory();

public:

	/** Factory method to instantiate a barrier based on the user settings */
	static IDisplayClusterBarrier* CreateBarrier(const FString& BarrierId, const TSet<FString>& CallerIds, const uint32 Timeout);
};
