// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"

class FDisplayClusterWatchdogTimer;


/**
 * Barrier wait result
 */
enum class EDisplayClusterBarrierWaitResult : uint8
{
	Ok,
	NotActive,
	TimeOut,
	NotAllowed,
};


/**
 * Barrier PreSyncEnd callback data
 */
struct FDisplayClusterBarrierPreSyncEndDelegateData
{
	/** Barrier ID */
	const FString& BarrierId;

	/** Binary data provided on sync request (CallerId-to-Data mapping) */
	const TMap<FString, TArray<uint8>>& RequestData;

	/** Binary data to respond (CallerId-to-Data mapping) */
	TMap<FString, TArray<uint8>>& ResponseData;
};


/**
 * Thread barrier interface
 */
class IDisplayClusterBarrier
{
public:

	virtual ~IDisplayClusterBarrier() = default;

public:

	/** Barrier name */
	virtual const FString& GetName() const = 0;

	/** Activate barrier */
	virtual bool Activate() = 0;

	/** Deactivate barrier, no threads will be blocked */
	virtual void Deactivate() = 0;

	/** Returns true if the barrier has been activated */
	virtual bool IsActivated() const = 0;

	/** Wait until all threads arrive */
	virtual EDisplayClusterBarrierWaitResult Wait(const FString& CallerId, double* OutThreadWaitTime = nullptr, double* OutBarrierWaitTime = nullptr) = 0;

	/** Wait until all threads arrive(with data) */
	virtual EDisplayClusterBarrierWaitResult WaitWithData(const FString& CallerId, const TArray<uint8>& RequestData, TArray<uint8>& OutResponseData, double* OutThreadWaitTime = nullptr, double* OutBarrierWaitTime = nullptr) = 0;

	/** Remove specified caller from the sync pipeline */
	virtual void UnregisterSyncCaller(const FString& CallerId) = 0;

	/** Barrier PreSyncEnd delegate. Called when all calling threads arrived right before opening the gate. */
	DECLARE_DELEGATE_OneParam(FDisplayClusterBarrierPreSyncEndDelegate, FDisplayClusterBarrierPreSyncEndDelegateData&);
	virtual FDisplayClusterBarrierPreSyncEndDelegate& GetPreSyncEndDelegate() = 0;

	/** Barrier timout notification (provides BarrierName and CallersTimedOut in parameters) */
	DECLARE_EVENT_TwoParams(IDisplayClusterBarrier, FDisplayClusterBarrierTimeoutEvent, const FString&, const TSet<FString>&);
	virtual FDisplayClusterBarrierTimeoutEvent& OnBarrierTimeout() = 0;
};
