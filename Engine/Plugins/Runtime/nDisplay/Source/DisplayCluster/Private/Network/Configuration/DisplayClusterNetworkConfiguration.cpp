// Copyright Epic Games, Inc. All Rights Reserved.

#include "Network/Configuration/DisplayClusterNetworkConfiguration.h"

#include "HAL/Platform.h"


namespace UE::nDisplay::Network::Configuration
{
	const FName ClusterSyncClientName    = TEXT("CLN_CS");
	const FName ClusterSyncServerName    = TEXT("SRV_CS");

	const FName RenderSyncClientName     = TEXT("CLN_RS");
	const FName RenderSyncServerName     = TEXT("SRV_RS");

	const FName InternalCommClientName   = TEXT("CLN_IC");
	const FName InternalCommServerName   = TEXT("SRV_IC");

	const FName GenericBarrierClientName = TEXT("CLN_GB");
	const FName GenericBarrierServerName = TEXT("SRV_GB");

	const FName BinaryEventsClientName   = TEXT("CLN_CEB");
	const FName BinaryEventsServerName   = TEXT("SRV_CEB");

	const FName JsonEventsClientName     = TEXT("CLN_CEJ");
	const FName JsonEventsServerName     = TEXT("SRV_CEJ");

	const FName BinaryEventsExternalClientName = TEXT("CLN_CEB_Ext");
	const FName BinaryEventsExternalServerName = TEXT("SRV_CEB_Ext");

	const FName JsonEventsExternalClientName = TEXT("CLN_CEJ_Ext");
	const FName JsonEventsExternalServerName = TEXT("SRV_CEJ_Ext");
};
