// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"


namespace UE::nDisplay::Network::Configuration
{
	/** Networking buffer size */
	static constexpr int32  PacketBufferSize = 4 * 1024 * 1024; // bytes

	/** ClusterSync client name */
	extern const FName ClusterSyncClientName;

	/** ClusterSync server name */
	extern const FName ClusterSyncServerName;


	/** RenderSync client name */
	extern const FName RenderSyncClientName;

	/** RenderSync server name */
	extern const FName RenderSyncServerName;


	/** InternalComm client name */
	extern const FName InternalCommClientName;

	/** InternalComm server name */
	extern const FName InternalCommServerName;


	/** GenericBarrier client name */
	extern const FName GenericBarrierClientName;

	/** GenericBarrier server name */
	extern const FName GenericBarrierServerName;


	/** Binary events client name */
	extern const FName BinaryEventsClientName;

	/** Binary events server name */
	extern const FName BinaryEventsServerName;


	/** JSON events client name */
	extern const FName JsonEventsClientName;

	/** JSON events server name */
	extern const FName JsonEventsServerName;


	/** Binary events client name (external) */
	extern const FName BinaryEventsExternalClientName;

	/** Binary events server name (external) */
	extern const FName BinaryEventsExternalServerName;


	/** JSON events client name (external) */
	extern const FName JsonEventsExternalClientName;

	/** JSON events server name (external) */
	extern const FName JsonEventsExternalServerName;
};
