// Copyright Epic Games, Inc. All Rights Reserved.

#include "Network/Conversion/DisplayClusterNetworkDataConversion.h"

#include "Cluster/DisplayClusterClusterEvent.h"
#include "Dom/JsonObject.h"

#include "Network/Packet/DisplayClusterPacketBinary.h"
#include "Network/Packet/DisplayClusterPacketJson.h"
#include "Network/Packet/DisplayClusterPacketInternal.h"
#include "Network/Service/ClusterSync/DisplayClusterClusterSyncStrings.h"

#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"

#include "JsonObjectConverter.h"


namespace UE::nDisplay::DisplayClusterNetworkDataConversion
{
	void JsonEventsFromInternalPacket(const TSharedPtr<FDisplayClusterPacketInternal>& Packet, TArray<TSharedPtr<FDisplayClusterClusterEventJson, ESPMode::ThreadSafe>>& JsonEvents)
	{
		TArray<TArray<uint8>> BinaryObjects;
		Packet->GetBinObjects(DisplayClusterClusterSyncStrings::ArgumentsJsonEvents, BinaryObjects, false);

		for (const TArray<uint8>& BinaryObject : BinaryObjects)
		{
			TSharedPtr<FDisplayClusterClusterEventJson, ESPMode::ThreadSafe> JsonEvent = MakeShared<FDisplayClusterClusterEventJson, ESPMode::ThreadSafe>();

			FMemoryReader MemoryReader(BinaryObject);
			JsonEvent->Serialize(MemoryReader);

			JsonEvents.Add(JsonEvent);
		}
	}

	void JsonEventsToInternalPacket(const TArray<TSharedPtr<FDisplayClusterClusterEventJson, ESPMode::ThreadSafe>>& JsonEvents, TSharedPtr<FDisplayClusterPacketInternal>& Packet)
	{
		for (const TSharedPtr<FDisplayClusterClusterEventJson, ESPMode::ThreadSafe>& JsonEvent : JsonEvents)
		{
			TArray<uint8> BinaryObject;
			FMemoryWriter MemoryWriter(BinaryObject);
			JsonEvent->Serialize(MemoryWriter);

			Packet->AddBinObject(DisplayClusterClusterSyncStrings::ArgumentsJsonEvents, BinaryObject);
		}
	}

	void BinaryEventsFromInternalPacket(const TSharedPtr<FDisplayClusterPacketInternal>& Packet, TArray<TSharedPtr<FDisplayClusterClusterEventBinary, ESPMode::ThreadSafe>>& BinaryEvents)
	{
		TArray<TArray<uint8>> BinaryObjects;
		Packet->GetBinObjects(DisplayClusterClusterSyncStrings::ArgumentsBinaryEvents, BinaryObjects, false);

		for (const TArray<uint8>& BinaryObject : BinaryObjects)
		{
			TSharedPtr<FDisplayClusterClusterEventBinary, ESPMode::ThreadSafe> BinaryEvent = MakeShared<FDisplayClusterClusterEventBinary, ESPMode::ThreadSafe>();

			FMemoryReader MemoryReader(BinaryObject);
			BinaryEvent->Serialize(MemoryReader);

			BinaryEvents.Add(BinaryEvent);
		}
	}

	void BinaryEventsToInternalPacket(const TArray<TSharedPtr<FDisplayClusterClusterEventBinary, ESPMode::ThreadSafe>>& BinaryEvents, TSharedPtr<FDisplayClusterPacketInternal>& Packet)
	{
		for (const TSharedPtr<FDisplayClusterClusterEventBinary, ESPMode::ThreadSafe>& BinaryEvent : BinaryEvents)
		{
			TArray<uint8> BinaryObject;
			FMemoryWriter MemoryWriter(BinaryObject);
			BinaryEvent->Serialize(MemoryWriter);

			Packet->AddBinObject(DisplayClusterClusterSyncStrings::ArgumentsBinaryEvents, BinaryObject);
		}
	}

	bool JsonPacketToJsonEvent(const TSharedPtr<FDisplayClusterPacketJson>& Packet, FDisplayClusterClusterEventJson& OutBinaryEvent)
	{
		return FJsonObjectConverter::JsonObjectToUStruct(Packet->GetJsonData().ToSharedRef(), &OutBinaryEvent);
	}

	TSharedPtr<FDisplayClusterPacketJson> JsonEventToJsonPacket(const FDisplayClusterClusterEventJson& JsonEvent)
	{
		TSharedPtr<FDisplayClusterPacketJson> Packet;

		TSharedPtr<FJsonObject> JsonObject = FJsonObjectConverter::UStructToJsonObject(JsonEvent);
		if (JsonObject)
		{
			Packet = MakeShared<FDisplayClusterPacketJson>();
			Packet->SetJsonData(JsonObject);
		}

		return Packet;
	}

	bool BinaryPacketToBinaryEvent(const TSharedPtr<FDisplayClusterPacketBinary>& Packet, FDisplayClusterClusterEventBinary& OutBinaryEvent)
	{
		const TArray<uint8>& Buffer = Packet->GetPacketData();
		return OutBinaryEvent.DeserializeFromByteArray(Buffer);
	}

	TSharedPtr<FDisplayClusterPacketBinary> BinaryEventToBinaryPacket(FDisplayClusterClusterEventBinary& BinaryEvent)
	{
		TSharedPtr<FDisplayClusterPacketBinary> Packet = MakeShared<FDisplayClusterPacketBinary>();
		TArray<uint8>& Buffer = Packet->GetPacketData();
		BinaryEvent.SerializeToByteArray(Buffer);
		return Packet;
	}
}
