// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

struct FDisplayClusterClusterEventBinary;
struct FDisplayClusterClusterEventJson;
class FDisplayClusterPacketBinary;
class FDisplayClusterPacketJson;
class FDisplayClusterPacketInternal;


/*
 * Some helpers that simplify export/import of internal non-trivial data types
 */ 
namespace UE::nDisplay::DisplayClusterNetworkDataConversion
{
	/* [Internal] Extracts JSON events from binary objects (internal in-cluster replication only) */
	void JsonEventsFromInternalPacket(const TSharedPtr<FDisplayClusterPacketInternal>& Packet, TArray<TSharedPtr<FDisplayClusterClusterEventJson, ESPMode::ThreadSafe>>& JsonEvents);

	/* [Internal] Converts JSON events into binary objects (internal in-cluster replication only) */
	void JsonEventsToInternalPacket(const TArray<TSharedPtr<FDisplayClusterClusterEventJson, ESPMode::ThreadSafe>>& JsonEvents, TSharedPtr<FDisplayClusterPacketInternal>& Packet);

	/* [Internal] Extracts Binary events from binary objects (internal in-cluster replication only) */
	void BinaryEventsFromInternalPacket(const TSharedPtr<FDisplayClusterPacketInternal>& Packet, TArray<TSharedPtr<FDisplayClusterClusterEventBinary, ESPMode::ThreadSafe>>& BinaryEvents);

	/* [Internal] Converts Binary events into binary objects (internal in-cluster replication only) */
	void BinaryEventsToInternalPacket(const TArray<TSharedPtr<FDisplayClusterClusterEventBinary, ESPMode::ThreadSafe>>& BinaryEvents, TSharedPtr<FDisplayClusterPacketInternal>& Packet);


	/** [External] Extracts JSON event data from an external JSON packet */
	bool JsonPacketToJsonEvent(const TSharedPtr<FDisplayClusterPacketJson>& Packet, FDisplayClusterClusterEventJson& OutBinaryEvent);

	/** [External] Creates an external JSON packet with a specific JSON event set */
	TSharedPtr<FDisplayClusterPacketJson> JsonEventToJsonPacket(const FDisplayClusterClusterEventJson& BinaryEvent);

	/** [External] Extracts Binary event data from an external Binary packet */
	bool BinaryPacketToBinaryEvent(const TSharedPtr<FDisplayClusterPacketBinary>& Packet, FDisplayClusterClusterEventBinary& OutBinaryEvent);

	/** [External] Creates an external Binary packet with a specific Binary event set */
	TSharedPtr<FDisplayClusterPacketBinary> BinaryEventToBinaryPacket(FDisplayClusterClusterEventBinary& BinaryEvent);
}
