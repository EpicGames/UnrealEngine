// Copyright Epic Games, Inc. All Rights Reserved.

#include "Network/DisplayClusterClient.h"
#include "Common/TcpSocketBuilder.h"

#include "Misc/ScopeLock.h"

#include "Misc/DisplayClusterLog.h"


bool FDisplayClusterClientBase::Connect(const FString& InAddr, const uint16 InPort, const uint32 ConnectRetriesAmount, const uint32 ConnectRetryDelay)
{
	FScopeLock Lock(&GetSyncObj());

	// Generate IPv4 address
	FIPv4Address IPAddr;
	if (!FIPv4Address::Parse(InAddr, IPAddr))
	{
		UE_LOGF(LogDisplayClusterNetwork, Error, "%ls couldn't parse the address: %ls", *GetName(), *InAddr);
		return false;
	}

	// Generate internet address
	TSharedRef<FInternetAddr> InternetAddr = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->CreateInternetAddr();
	InternetAddr->SetIp(IPAddr.Value);
	InternetAddr->SetPort(InPort);

	// Start connection loop
	uint32 TryIdx = 0;
	while(ConnectSocket(*InternetAddr) == false)
	{
		UE_LOGF(LogDisplayClusterNetwork, Log, "%ls couldn't connect to the server %ls [%d]", *GetName(), *(InternetAddr->ToString(true)), TryIdx);
		if (ConnectRetriesAmount > 0 && ++TryIdx >= ConnectRetriesAmount)
		{
			UE_LOGF(LogDisplayClusterNetwork, Log, "%ls connection attempts limit reached", *GetName());
			return false;
		}

		// Sleep some time before next try
		FPlatformProcess::Sleep(ConnectRetryDelay / 1000.f);
	}

	return IsOpen();
}

void FDisplayClusterClientBase::Disconnect()
{
	UE_LOGF(LogDisplayClusterNetwork, Log, "%ls disconnecting...", *GetName());

	CloseSocket();
}

FSocket* FDisplayClusterClientBase::CreateSocket(const FString& InName, int32 LingerTime)
{
	FSocket* NewSocket = FTcpSocketBuilder(*InName).AsBlocking().Lingering(LingerTime);
	check(NewSocket);

	// Set other socket properties
	NewSocket->SetNoDelay(true);

	return NewSocket;
}
