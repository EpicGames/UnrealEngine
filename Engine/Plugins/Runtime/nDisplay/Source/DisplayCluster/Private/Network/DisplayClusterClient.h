// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Network/IDisplayClusterClient.h"
#include "Network/Configuration/DisplayClusterNetworkConfiguration.h"
#include "Network/Transport/DisplayClusterSocketOperations.h"
#include "Network/Transport/DisplayClusterSocketOperationsHelper.h"


/**
 * Base DisplayCluster TCP client
 */
class FDisplayClusterClientBase
	: public IDisplayClusterClient
	, public FDisplayClusterSocketOperations
{
public:

	FDisplayClusterClientBase(const FString& InName, int32 InLingerTime = 0)
		: FDisplayClusterSocketOperations(CreateSocket(InName, InLingerTime), UE::nDisplay::Network::Configuration::PacketBufferSize, InName)
	{ }

public:

	/** Connects to a server */
	virtual bool Connect(const FString& Address, const uint16 Port, const uint32 ConnectRetriesAmount = 0, const uint32 ConnectRetryDelay = 0) override;

	/** Terminates current connection */
	virtual void Disconnect() override;

	/** Returns client name */
	virtual FString GetName() const override
	{
		return GetConnectionName();
	}

	/** Returns true if the client is currently connected */
	virtual bool IsConnected() const override
	{
		return IsOpen();
	}

protected:

	/** Creates client socket */
	FSocket* CreateSocket(const FString& InName, int32 LingerTime);
};


template <typename TPacketType>
class FDisplayClusterClient
	: public    FDisplayClusterClientBase
	, protected FDisplayClusterSocketOperationsHelper<TPacketType>
{
public:
	FDisplayClusterClient(const FString& InName, int32 InLingerTime = 0)
		: FDisplayClusterClientBase(InName, InLingerTime)
		, FDisplayClusterSocketOperationsHelper<TPacketType>(*this, InName)
	{ }
};
