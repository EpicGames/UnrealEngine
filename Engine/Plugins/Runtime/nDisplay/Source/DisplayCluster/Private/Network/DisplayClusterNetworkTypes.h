// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "Misc/Optional.h"

class FSocket;


/**
 * Error codes for net communication
 */
enum class EDisplayClusterCommResult : uint8
{
	Ok,
	WrongRequestData,
	WrongResponseData,
	NetworkError,
	NotImplemented,
	NotAllowed,
	InternalError,
};


/**
 * Represents session information
 */
struct FDisplayClusterSessionInfo
{
	// Provide access to the protected section for all session classes
	template <typename TPacketType, bool bIsBidirectional>
	friend class FDisplayClusterSession;

public:
	FSocket*      Socket = nullptr;
	FIPv4Endpoint Endpoint;
	FString       Protocol;
	uint64        SessionId = 0;
	FString       SessionName;

	TOptional<FString> NodeId;

	double TimeStart = 0;
	double TimeEnd   = 0;

public:
	bool IsTerminatedByServer() const
	{
		return bTerminatedByServer;
	}

protected:
	// Termination flag to simplify failover pipelines
	bool bTerminatedByServer = false;

public:
	FString ToString() const
	{
		return FString::Printf(TEXT("%llu:%s:%s:%s:%s"), SessionId, *Protocol, *Endpoint.ToString(), *SessionName, *NodeId.Get(FString()));
	}
};
