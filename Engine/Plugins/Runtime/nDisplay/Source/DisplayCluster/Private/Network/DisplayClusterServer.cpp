// Copyright Epic Games, Inc. All Rights Reserved.

#include "Network/DisplayClusterServer.h"

#include "Engine/GameEngine.h"

#include "Misc/DateTime.h"
#include "Misc/DisplayClusterLog.h"
#include "Misc/ScopeLock.h"

#include "Network/Session/IDisplayClusterSession.h"
#include "Network/Listener/DisplayClusterTcpListener.h"
#include "Network/DisplayClusterNetworkTypes.h"

#include "TimerManager.h"


// Give 10 seconds minimum for every session that finished its job to finalize
// the working thread and other internals before freeing the session object
const double FDisplayClusterServer::CleanSessionResourcesSafePeriod = 10.f;


FDisplayClusterServer::FDisplayClusterServer(const FString& InInstanceName)
	: InstanceName(InInstanceName)
{
	// Subscribe for session events
	OnSessionOpened().AddRaw(this, &FDisplayClusterServer::ProcessSessionOpened);
	OnSessionClosed().AddRaw(this, &FDisplayClusterServer::ProcessSessionClosed);
}

FDisplayClusterServer::~FDisplayClusterServer()
{
	// Unsubscribe from events
	OnSessionOpened().RemoveAll(this);
	OnSessionClosed().RemoveAll(this);

	ShutdownImpl();
}


bool FDisplayClusterServer::Start(const FString& Address, const uint16 Port)
{
	FScopeLock Lock(&ServerStateCritSec);

	// Nothing to do if already running
	if (bIsRunning)
	{
		return true;
	}

	UE_LOGF(LogDisplayClusterNetwork, Log, "%ls starting...", *InstanceName);

	// Instantiate internal listener
	Listener = MakeShared<FDisplayClusterTcpListener>(false, InstanceName + FString(TEXT("_listener")));
	// Bind connection handler method
	Listener->OnConnectionAccepted().BindRaw(this, &FDisplayClusterServer::ConnectionHandler);

	if (!Listener->StartListening(Address, Port))
	{
		UE_LOGF(LogDisplayClusterNetwork, Error, "%ls couldn't start connection listener [%ls:%d]", *InstanceName, *Address, Port);
		return false;
	}

	// Update server state
	bIsRunning = true;

	return bIsRunning;
}

bool FDisplayClusterServer::Start(TSharedRef<FDisplayClusterTcpListener>& ExternalListener)
{
	FScopeLock Lock(&ServerStateCritSec);

	// Set connection handler up
	const FString ProtocolName = GetProtocolName();
	ExternalListener->OnConnectionAccepted(ProtocolName).BindRaw(this, &FDisplayClusterServer::ConnectionHandler);

	// Nothing to do if alrady running
	if (bIsRunning)
	{
		return true;
	}

	// Keep external listener ptr
	Listener = ExternalListener;

	// Update server state
	bIsRunning = true;

	return bIsRunning;
}

void FDisplayClusterServer::Shutdown()
{
	ShutdownImpl();
}

bool FDisplayClusterServer::IsRunning() const
{
	FScopeLock Lock(&ServerStateCritSec);
	return bIsRunning;
}

FString FDisplayClusterServer::GetAddress() const
{
	FScopeLock Lock(&SessionsCritSec);
	return IsRunning() ? Listener->GetListeningHost() : FString();
}

// Server port
uint16 FDisplayClusterServer::GetPort() const
{
	FScopeLock Lock(&SessionsCritSec);
	return IsRunning() ? Listener->GetListeningPort() : 0;
}

void FDisplayClusterServer::KillSession(const FString& NodeId)
{
	FScopeLock Lock(&SessionsCritSec);

	if (IsRunning())
	{
		// Iterate over all active sessions
		for (TPair<uint64, TSharedPtr<IDisplayClusterSession>>& SessionIt : ActiveSessions)
		{
			// Make sure the session ID is valid (only for in-cluster comm sessions)
			const FString SessionNodeId = SessionIt.Value->GetSessionInfo().NodeId.Get(FString());
			if (!SessionNodeId.IsEmpty())
			{
				// Stop session if ID matches the requested one
				if (SessionNodeId.Equals(NodeId, ESearchCase::IgnoreCase))
				{
					// No need to wait for completion to prevent deadlock in case the KillSession pipeline
					// is triggered from one of the sessions
					SessionIt.Value->StopSession(false);
				}
			}
		}
	}
}

bool FDisplayClusterServer::ConnectionHandler(FDisplayClusterSessionInfo& SessionInfo)
{
	check(SessionInfo.Socket);

	FScopeLock Lock(&SessionsCritSec);

	if (IsRunning() && (IsConnectionAllowedDelegate.IsBound() ? IsConnectionAllowedDelegate.Execute(SessionInfo) : true))
	{
		UE_LOGF(LogDisplayClusterNetwork, Verbose, "Connection %ls is allowed %ls",
			*SessionInfo.ToString(),
			IsConnectionAllowedDelegate.IsBound() ? TEXT("by a validator") : TEXT("(no validation)"));

		// Set socket properties (non-blocking, no delay)
		SessionInfo.Socket->SetNoDelay(true);
		SessionInfo.Socket->SetLinger(true, 0);
		SessionInfo.Socket->SetNonBlocking(false);
		SessionInfo.Socket->SetReuseAddr(false);
		// Update other session info
		SessionInfo.Protocol  = GetProtocolName();
		SessionInfo.SessionId = IncrementalSessionId++;

		// Create new session for this incoming connection
		TSharedPtr<IDisplayClusterSession> Session = CreateSession(SessionInfo);
		check(Session.IsValid());
		if (Session)
		{
			// Move the session to the pending session list unless it's started
			PendingSessions.Emplace(SessionInfo.SessionId, Session);
			// And trigger session start
			return Session->StartSession();
		}
	}
	else
	{
		UE_LOGF(LogDisplayClusterNetwork, Log, "Connection request %ls has been rejected", *SessionInfo.ToString());
	}

	return false;
}

void FDisplayClusterServer::ProcessSessionOpened(const FDisplayClusterSessionInfo& SessionInfo)
{
	FScopeLock Lock(&SessionsCritSec);

	if (IsRunning())
	{
		UE_LOGF(LogDisplayClusterNetwork, Verbose, "Session start: %ls", *SessionInfo.SessionName);

		// Change session status from 'Pending' to 'Active'
		TSharedPtr<IDisplayClusterSession> Session;
		if (PendingSessions.RemoveAndCopyValue(SessionInfo.SessionId, Session))
		{
			ActiveSessions.Emplace(SessionInfo.SessionId, Session);
		}
	}
}

void FDisplayClusterServer::ProcessSessionClosed(const FDisplayClusterSessionInfo& SessionInfo)
{
	FScopeLock Lock(&SessionsCritSec);

	if (IsRunning())
	{
		UE_LOGF(LogDisplayClusterNetwork, Verbose, "Session end: %ls", *SessionInfo.SessionName);

		// Change session status from 'Active' to 'PendingKill'
		TSharedPtr<IDisplayClusterSession> Session;
		if (ActiveSessions.RemoveAndCopyValue(SessionInfo.SessionId, Session))
		{
			PendingKillSessions.Emplace(SessionInfo.SessionId, Session);
		}

		// Clean sessions that have been queued previously
		CleanPendingKillSessions();
	}
}

void FDisplayClusterServer::ShutdownImpl()
{

	TArray<TSharedPtr<IDisplayClusterSession>> ActiveSessionsToWait;

	{
		FScopeLock Lock(&ServerStateCritSec);

		// Nothing to do if not running
		if (!IsRunning())
		{
			return;
		}

		UE_LOGF(LogDisplayClusterNetwork, Log, "%ls stopping...", *InstanceName);

		// Update server state so no new sessions allowed anymore
		bIsRunning = false;
		// Stop listening for new connections
		Listener.Reset();

		// Make copy of all active sessions to wait for completion later
		ActiveSessions.GenerateValueArray(ActiveSessionsToWait);

		// Send stop command to all active sessions
		for (auto It = ActiveSessions.CreateIterator(); It; ++It)
		{
			It->Value->StopSession(false);
		}
	}

	// Now wait unless all session threads are finished. We're waiting outside of the
	// critical section so the sessions won't be deadlocked notifying the server about their state changes.
	for (TSharedPtr<IDisplayClusterSession>& Session : ActiveSessionsToWait)
	{
		Session->WaitForCompletion();
	}

	{
		FScopeLock Lock(&ServerStateCritSec);

		// Finally, cleanup internals
		ActiveSessions.Reset();
		PendingSessions.Reset();
		PendingKillSessions.Reset();
	}
}

void FDisplayClusterServer::CleanPendingKillSessions()
{
	FScopeLock Lock(&SessionsCritSec);

	const double CurrentTime = FPlatformTime::Seconds();

	// Release all PendingKill sessions that have timed out (safe release period)
	for (auto SessionIt = PendingKillSessions.CreateIterator(); SessionIt; ++SessionIt)
	{
		if ((CurrentTime - SessionIt->Value->GetSessionInfo().TimeEnd) > FDisplayClusterServer::CleanSessionResourcesSafePeriod)
		{
			UE_LOGF(LogDisplayClusterNetwork, Verbose, "%ls - cleaning session resources, SessionId=%llu...", *InstanceName, SessionIt->Key);
			SessionIt.RemoveCurrent();
		}
	}
}
