// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Network/IDisplayClusterServer.h"
#include "Network/DisplayClusterNetworkTypes.h"

struct FIPv4Endpoint;
class FSocket;
class IDisplayClusterSession;
class FDisplayClusterTcpListener;


/**
 * Base DisplayCluster TCP server
 */
class FDisplayClusterServer
	: public IDisplayClusterServer
{
public:

	/** Minimal time(seconds) before cleaning resources of the 'pending kill' sessions */
	static const double CleanSessionResourcesSafePeriod;

public:

	FDisplayClusterServer(const FString& InstanceName);
	virtual ~FDisplayClusterServer();

public:

	//~ Begin IDisplayClusterServer
	
	/** Start server on a specific socket */
	virtual bool Start(const FString& Address, const uint16 Port) override;

	/** Start server with a specified listener */
	virtual bool Start(TSharedRef<FDisplayClusterTcpListener>& ExternalListener) override;

	/** Stop server */
	virtual void Shutdown() override;

	/** Returns current server state */
	virtual bool IsRunning() const override;

	/** Server name */
	virtual FString GetName() const override
	{
		return InstanceName;
	}

	/** Server address */
	virtual FString GetAddress() const override;

	/** Server port */
	virtual uint16 GetPort() const override;

	/** Kills all sessions of a specified cluster node */
	virtual void KillSession(const FString& NodeId) override;

	/** Connection validation delegate */
	virtual FIsConnectionAllowedDelegate& OnIsConnectionAllowed() override
	{
		return IsConnectionAllowedDelegate;
	}

	/** Session opened event */
	virtual FSessionOpenedEvent& OnSessionOpened() override
	{
		return SessionOpenedEvent;
	}

	/** Session closed event */
	virtual FSessionClosedEvent& OnSessionClosed() override
	{
		return SessionClosedEvent;
	}

	// Wraps SessionOpened to emit it safely from any thread.
	virtual void NotifySessionOpened(const FDisplayClusterSessionInfo& SessionInfo) override
	{
		FScopeLock Lock(&SessionNotifyCritSec);
		OnSessionOpened().Broadcast(SessionInfo);
	}

	// Wraps SessionClosed to emit it safely from any thread.
	virtual void NotifySessionClosed(const FDisplayClusterSessionInfo& SessionInfo) override
	{
		FScopeLock Lock(&SessionNotifyCritSec);
		OnSessionClosed().Broadcast(SessionInfo);
	}

	//~ End IDisplayClusterServer

protected:

	/** Allow to specify custom session class */
	virtual TSharedPtr<IDisplayClusterSession> CreateSession(FDisplayClusterSessionInfo& SessionInfo) = 0;

private:

	/** Handle incoming connections */
	bool ConnectionHandler(FDisplayClusterSessionInfo& SessionInfo);

	/** Callback on session opened */
	void ProcessSessionOpened(const FDisplayClusterSessionInfo& SessionInfo);

	/** Callback on session closed */
	void ProcessSessionClosed(const FDisplayClusterSessionInfo& SessionInfo);

	/** Non-virtual shutdown implementation */
	void ShutdownImpl();

	/** Free resources of the sessions that already finished their job */
	void CleanPendingKillSessions();

private:

	/** Server instance name */
	const FString InstanceName;
	
	/** Server running state */
	bool bIsRunning = false;

	/** Socket listener */
	TSharedPtr<FDisplayClusterTcpListener> Listener;

	/** Connection approval delegate */
	FIsConnectionAllowedDelegate IsConnectionAllowedDelegate;

	/** Session opened event */
	FSessionOpenedEvent SessionOpenedEvent;

	/** Session closed event */
	FSessionClosedEvent SessionClosedEvent;

private:

	/** Session counter used for session ID generation */
	uint64 IncrementalSessionId = 0;


	/** Pending sessions */
	TMap<uint64, TSharedPtr<IDisplayClusterSession>> PendingSessions;

	/** Active sessions */
	TMap<uint64, TSharedPtr<IDisplayClusterSession>> ActiveSessions;

	/** Closed sessions, awaiting for cleaning */
	TMap<uint64, TSharedPtr<IDisplayClusterSession>> PendingKillSessions;

private:

	/** Critical section to manipulate server states */
	mutable FCriticalSection ServerStateCritSec;

	/** Critical section to manipulate the connection sessions */
	mutable FCriticalSection SessionsCritSec;

	/**
	 * Critical section for safe session notifications. The same critical section
	 * is used for both SessionOpened and SessionClosed events because they don't
	 * really interfere in runtime. If there are new session events at some point,
	 * we might probably need separate critical sections for those.
	 */
	mutable FCriticalSection SessionNotifyCritSec;
};
