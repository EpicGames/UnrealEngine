// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Cluster/IDisplayClusterGenericBarriersClient.h"


/**
 * DisplayCluster TCP client interface
 */
class IDisplayClusterClient
{
public:
	virtual ~IDisplayClusterClient() = default;

public:

	/** Connects to a server */
	virtual bool Connect(const FString& Address, const uint16 Port, const uint32 ConnectRetriesAmount, const uint32 ConnectRetryDelay) = 0;

	/** Terminates current connection */
	virtual void Disconnect() = 0;

	/** Returns connection status */
	virtual bool IsConnected() const = 0;

	/** Returns client name */
	virtual FString GetName() const = 0;
};
