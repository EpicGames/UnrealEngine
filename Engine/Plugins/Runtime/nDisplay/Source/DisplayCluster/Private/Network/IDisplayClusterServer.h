// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Network/DisplayClusterNetworkTypes.h"

class FDisplayClusterTcpListener;


/**
 * DisplayCluster TCP server interface
 */
class IDisplayClusterServer
{
public:

	virtual ~IDisplayClusterServer() = default;

public:

	/** Start server on a specific socket */
	virtual bool Start(const FString& Address, const uint16 Port) = 0;

	/** Start server with a specified listener */
	virtual bool Start(TSharedRef<FDisplayClusterTcpListener>& Listener) = 0;

	/** Stop server */
	virtual void Shutdown() = 0;

	/** Returns true if server is currently running */
	virtual bool IsRunning() const = 0;

	/** Returns server instance name */
	virtual FString GetName() const = 0;

	/** Returns server address */
	virtual FString GetAddress() const = 0;

	/** Returns server port */
	virtual uint16 GetPort() const = 0;

	/** Returns server protocol name */
	virtual FString GetProtocolName() const = 0;

	/** Kill all sessions of a specific node */
	virtual void KillSession(const FString& NodeId) = 0;

public:

	/** Connection validation delegate */
	DECLARE_DELEGATE_RetVal_OneParam(bool, FIsConnectionAllowedDelegate, const FDisplayClusterSessionInfo&);
	virtual FIsConnectionAllowedDelegate& OnIsConnectionAllowed() = 0;

	/** Session opened event */
	DECLARE_EVENT_OneParam(IDisplayClusterServer, FSessionOpenedEvent, const FDisplayClusterSessionInfo&);
	virtual FSessionOpenedEvent& OnSessionOpened() = 0;

	/** Session closed event */
	DECLARE_EVENT_OneParam(IDisplayClusterServer, FSessionClosedEvent, const FDisplayClusterSessionInfo&);
	virtual FSessionClosedEvent& OnSessionClosed() = 0;

	/** Safe SessionOpened notification from a session thread */
	virtual void NotifySessionOpened(const FDisplayClusterSessionInfo& SessionInfo) = 0;

	/** Safe SessionClosed notification from a session thread */
	virtual void NotifySessionClosed(const FDisplayClusterSessionInfo& SessionInfo) = 0;
};
