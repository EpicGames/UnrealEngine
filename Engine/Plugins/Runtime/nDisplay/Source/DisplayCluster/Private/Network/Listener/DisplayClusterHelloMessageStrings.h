// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"


/**
 * Hello message strings
 */
namespace DisplayClusterHelloMessageStrings
{
	constexpr static const TCHAR* ArgumentsDefaultCategory = TEXT("HC");

	namespace Hello
	{
		constexpr static const TCHAR* Name        = TEXT("Hello");
		constexpr static const TCHAR* TypeRequest = TEXT("Request");
		constexpr static const TCHAR* ArgNodeId   = TEXT("NodeId");
	};
};
