// Copyright Epic Games, Inc. All Rights Reserved.

#include "Network/Listener/DisplayClusterTcpListener.h"
#include "Network/Listener/DisplayClusterHelloMessageStrings.h"
#include "Network/Packet/DisplayClusterPacketInternal.h"
#include "Network/Transport/DisplayClusterSocketOperations.h"
#include "Network/Transport/DisplayClusterSocketOperationsHelper.h"
#include "Network/DisplayClusterNetworkTypes.h"

#include "Misc/DisplayClusterHelpers.h"
#include "Misc/DisplayClusterLog.h"

#include "Common/TcpSocketBuilder.h"
#include "HAL/RunnableThread.h"


FDisplayClusterTcpListener::FDisplayClusterTcpListener(bool bIsShared, const FString& InName)
	: Name(InName)
{
	// In case the listener is used as a shared one, we process all incoming
	// connections and dispatch them based on the protocol mapping.
	if (bIsShared)
	{
		// Bind internal function for processing all incoming connections. It will redirect
		// them to the specific servers based on their subscription.
		OnConnectionAccepted().BindRaw(this, &FDisplayClusterTcpListener::ProcessIncomingConnection);
	}
}

FDisplayClusterTcpListener::~FDisplayClusterTcpListener()
{
	// Just free resources by stopping the listening
	StopListening(true);
}

bool FDisplayClusterTcpListener::StartListening(const FString& InAddr, const uint16 InPort)
{
	FScopeLock Lock(&InternalsCS);

	if (IsListening())
	{
		return true;
	}

	FIPv4Endpoint EP;
	if (!GenIPv4Endpoint(InAddr, InPort, EP))
	{
		return false;
	}

	return StartListening(EP);
}

bool FDisplayClusterTcpListener::StartListening(const FIPv4Endpoint& InEndpoint)
{
	FScopeLock Lock(&InternalsCS);

	if (IsListening())
	{
		return true;
	}

	// Save new endpoint
	Endpoint = InEndpoint;

	// Create listening thread
	ThreadObj.Reset(FRunnableThread::Create(this, *(Name + FString("_thread")), 128 * 1024, TPri_Normal));
	ensure(ThreadObj);

	// This must have been updated from the worker thread. FRunnableThread::Create uses
	// internal event ThreadInitSyncEvent to wait until FRunnable::Init is done (it's
	// FDisplayClusterTcpListener::Init here).
	return bIsListening;
}

void FDisplayClusterTcpListener::StopListening(bool bWaitForCompletion)
{
	FScopeLock Lock(&InternalsCS);

	if (!IsListening())
	{
		return;
	}

	FString ListeningAddr;
	uint16 ListeningPort = 0;
	GetListeningParams(ListeningAddr, ListeningPort);

	// Ask runnable to stop
	Stop();

	// Wait for thread finish if needed
	if (bWaitForCompletion)
	{
		WaitForCompletion();
	}

	UE_LOGF(LogDisplayClusterNetwork, Log, "TCP listener %ls: stopped listening to %ls:%d...", *Name, *ListeningAddr, ListeningPort);
}

void FDisplayClusterTcpListener::WaitForCompletion()
{
	FScopeLock Lock(&InternalsCS);

	// Wait for thread finish and release it then
	if (ThreadObj)
	{
		ThreadObj->WaitForCompletion();
	}
}

bool FDisplayClusterTcpListener::GetListeningParams(FString& OutAddr, uint16& OutPort)
{
	if (!IsListening())
	{
		return false;
	}

	OutAddr = GetListeningHost();
	OutPort = GetListeningPort();

	return true;
}

FString FDisplayClusterTcpListener::GetListeningHost() const
{
	if (IsListening() && SocketObj)
	{
		TSharedRef<FInternetAddr> ListeningAddress = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->CreateInternetAddr();
		SocketObj->GetAddress(*ListeningAddress);

		uint32 IpAddrAsNum = 0;
		ListeningAddress->GetIp(IpAddrAsNum);

		FIPv4Address IpAddress(IpAddrAsNum);
		const FString IpAddrAsString = IpAddress.ToString();

		return IpAddrAsString;
	}

	return FString();
}

uint16 FDisplayClusterTcpListener::GetListeningPort() const
{
	if (IsListening() && SocketObj)
	{
		const int32 PortNum32 = SocketObj->GetPortNo();
		check(PortNum32 > 0 && PortNum32 <= TNumericLimits<uint16>::Max());
		const uint16 PortNum16 = static_cast<uint16>(PortNum32 & TNumericLimits<uint16>::Max());
		return PortNum16;
	}

	return 0;
}

FDisplayClusterTcpListener::FConnectionAcceptedDelegate& FDisplayClusterTcpListener::OnConnectionAccepted(const FString& ProtocolName)
{
	FScopeLock Lock(&InternalsCS);

	// Look up for a corresponding delegate. Create new one if not found.
	FConnectionAcceptedDelegate* Delegate = ProtocolDispatchingMap.Find(ProtocolName);
	if (!Delegate)
	{
		Delegate = &ProtocolDispatchingMap.Emplace(ProtocolName);
	}

	return *Delegate;
}

bool FDisplayClusterTcpListener::Init()
{
	// Create socket
	SocketObj = FTcpSocketBuilder(*Name)
		.AsBlocking()
		.Lingering(0)
		.AsReusable(false)
		.Listening(128)
		.BoundToEndpoint(Endpoint);

	if (SocketObj)
	{
		// Set socket properties (non-blocking, no delay)
		SocketObj->SetNoDelay(true);
		SocketObj->SetNonBlocking(false);
		SocketObj->SetLinger(true, 0);
		SocketObj->SetReuseAddr(false);
	
		// Update listening state
		bIsListening = true;
	}

	return SocketObj != nullptr;
}

uint32 FDisplayClusterTcpListener::Run()
{
	TSharedRef<FInternetAddr> RemoteAddress = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->CreateInternetAddr();

	// Using TLS dramatically speeds up clusters with large numbers of nodes
	FMemory::SetupTLSCachesOnCurrentThread();

	if (SocketObj)
	{
		FString ListeningAddr;
		uint16 ListeningPort = 0;
		GetListeningParams(ListeningAddr, ListeningPort);

		UE_LOGF(LogDisplayClusterNetwork, Log, "TCP listener %ls: started listening to %ls:%d...", *Name, *ListeningAddr, ListeningPort);

		while (FSocket* NewSock = SocketObj->Accept(*RemoteAddress, TEXT("DisplayCluster session")))
		{
			UE_LOGF(LogDisplayClusterNetwork, Verbose, "TCP listener %ls: New incoming connection: %ls", *Name, *RemoteAddress->ToString(true));

			if (NewSock)
			{
				// Prepare connection info
				FDisplayClusterSessionInfo SessionInfo;
				SessionInfo.Socket = NewSock;
				SessionInfo.Endpoint = FIPv4Endpoint(RemoteAddress);

				// Notify corresponding server about new incoming connection
				if (ConnectionAcceptedDelegate.IsBound() && ConnectionAcceptedDelegate.Execute(SessionInfo))
				{
					UE_LOGF(LogDisplayClusterNetwork, Verbose, "TCP listener %ls: New incoming connection accepted: %ls", *Name, *RemoteAddress->ToString(true));
				}
				else
				{
					UE_LOGF(LogDisplayClusterNetwork, Verbose, "TCP listener %ls: New incoming connection declined or handler is not bound: %ls", *Name, *RemoteAddress->ToString(true));
					NewSock->Close();
					ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->DestroySocket(NewSock);
				}
			}
		}
	}
	else
	{
		UE_LOGF(LogDisplayClusterNetwork, Error, "Socket %ls is not initialized", *Name);
	}

	return 0;
}

void FDisplayClusterTcpListener::Stop()
{
	// Close the socket to unblock thread
	if (SocketObj)
	{
		SocketObj->Close();
	}
}

void FDisplayClusterTcpListener::Exit()
{
	// Release the socket
	if (SocketObj)
	{
		ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->DestroySocket(SocketObj);
		SocketObj = nullptr;
	}
}

bool FDisplayClusterTcpListener::GenIPv4Endpoint(const FString& Addr, const uint16 Port, FIPv4Endpoint& EP) const
{
	FIPv4Address IPv4Addr;
	if (!FIPv4Address::Parse(Addr, IPv4Addr))
	{
		return false;
	}

	EP = FIPv4Endpoint(IPv4Addr, Port);
	return true;
}

bool FDisplayClusterTcpListener::ProcessIncomingConnection(FDisplayClusterSessionInfo& SessionInfo)
{
	FScopeLock Lock(&InternalsCS);

	// Instantiate socket operations object for reading hello packet
	TUniquePtr<FDisplayClusterSocketOperations> SocketOps = MakeUnique<FDisplayClusterSocketOperations>(SessionInfo.Socket, 1024, FString("awaiting_for_hello_packet"), false);
	check(SocketOps);

	// Instantiate socket operations helper to be able to operate on the nDisplay packets abstraction level
	TUniquePtr<FDisplayClusterSocketOperationsHelper<FDisplayClusterPacketInternal>> SocketOpsHelper = MakeUnique<FDisplayClusterSocketOperationsHelper<FDisplayClusterPacketInternal>>(*SocketOps.Get());
	check(SocketOpsHelper);

	// 1. Get info packet and make sure it's valid
	TSharedPtr<FDisplayClusterPacketInternal> HelloPacket = SocketOpsHelper->ReceivePacket();
	if (!HelloPacket)
	{
		UE_LOGF(LogDisplayClusterNetwork, Warning, "%ls - Couldn't receive 'hello' packet", *SocketOps->GetConnectionName());
		return false;
	}

	// 2. Make sure the packet is valid, and extract data. Update connection info with the data we just received.
	FString MsgNodeId;
	const FString MsgProtocol = HelloPacket->GetProtocol();
	if (HelloPacket->GetName().Equals(DisplayClusterHelloMessageStrings::Hello::Name, ESearchCase::IgnoreCase) &&
		HelloPacket->GetTextArg(DisplayClusterHelloMessageStrings::ArgumentsDefaultCategory, DisplayClusterHelloMessageStrings::Hello::ArgNodeId, MsgNodeId) &&
		!MsgNodeId.IsEmpty() &&
		!MsgProtocol.IsEmpty())
	{
		SessionInfo.Protocol = MsgProtocol;
		SessionInfo.NodeId   = MsgNodeId;
	}
	else
	{
		UE_LOGF(LogDisplayClusterNetwork, Warning, "Couldn't parse the 'hello' message: %ls", *HelloPacket->ToLogString());
		return false;
	}

	// 3. Look up for a server responsible for this protocol
	FConnectionAcceptedDelegate* const Handler = ProtocolDispatchingMap.Find(SessionInfo.Protocol);
	if (!Handler)
	{
		UE_LOGF(LogDisplayClusterNetwork, Warning, "No responsible service found for protocol: %ls", *SessionInfo.Protocol);
		return false;
	}

	// 4. There is one. Transfer connection (socket) ownership to it.
	// note: false will also be returned if the delegate is not bound
	if (!Handler->Execute(SessionInfo))
	{
		UE_LOGF(LogDisplayClusterNetwork, Warning, "Couldn't transfer session ownership to a responsible server: %ls", *SessionInfo.ToString());
		return false;
	}

	// 5. Everything is OK. Return true so the socket won't be released on the caller side. It now belongs to the corresponding server.
	return true;
}
