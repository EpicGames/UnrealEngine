// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Runnable.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "Network/DisplayClusterNetworkTypes.h"

#include "Sockets.h"

#include <atomic>


/**
 * TCP connection listener.
 *
 * Listens for incoming connections and redirects the requests to specific server implementations.
 * Can be shared by nDisplay services that use internal communication protocol.
 */
class FDisplayClusterTcpListener
	: protected FRunnable
{
public:

	FDisplayClusterTcpListener(bool bIsShared, const FString& InName);
	virtual ~FDisplayClusterTcpListener();

public:

	/** Start listening to address:port */
	bool StartListening(const FString& InAddr, const uint16 InPort);

	/** Start listening to an endpoint */
	bool StartListening(const FIPv4Endpoint& Endpoint);

	/** Stop listening */
	void StopListening(bool bWaitForCompletion);

	/** Wait unless working thread is finished */
	void WaitForCompletion();

	/** Is currently listening */
	bool IsListening() const
	{
		return bIsListening;
	}

	/** Returns listenting address & port */
	bool GetListeningParams(FString& OutAddr, uint16& OutPort);

	/** Returns listening host */
	FString GetListeningHost() const;

	/** Returns listening port */
	uint16 GetListeningPort() const;

	/** Delegate for processing incoming connections */
	DECLARE_DELEGATE_RetVal_OneParam(bool, FConnectionAcceptedDelegate, FDisplayClusterSessionInfo&);

	/** Returns connection validation delegate */
	FConnectionAcceptedDelegate& OnConnectionAccepted()
	{
		return ConnectionAcceptedDelegate;
	}

	/** Returns protocol - bound delegate */
	FConnectionAcceptedDelegate& OnConnectionAccepted(const FString& ProtocolName);

protected:

	//~Begin FRunnable
	virtual bool Init() override;
	virtual uint32 Run() override;
	virtual void Stop() override;
	virtual void Exit() override;
	//~End FRunnable

protected:

	/** Fills endpoint data with given address and port */
	bool GenIPv4Endpoint(const FString& Addr, const uint16 Port, FIPv4Endpoint& EP) const;

private:

	/** Socket name */
	FString Name;

	/** Listening socket */
	FSocket* SocketObj = nullptr;

	/** Listening endpoint */
	FIPv4Endpoint Endpoint;

	/** Holds the listening thread object */
	TUniquePtr<FRunnableThread> ThreadObj;

	/** Current listening state */
	std::atomic<bool> bIsListening = false;

	/** Holds a delegate to be invoked when an incoming connection has been accepted. */
	FConnectionAcceptedDelegate ConnectionAcceptedDelegate;

private:

	/** Handles incoming connections */
	bool ProcessIncomingConnection(FDisplayClusterSessionInfo& SessionInfo);

	/** ProtocolName - to - ServiceDelegate map for transferring connection ownership to an appropriate server */
	TMap<FString, FConnectionAcceptedDelegate> ProtocolDispatchingMap;

	/** Critical section for access control */
	FCriticalSection InternalsCS;
};
