// Copyright Epic Games, Inc. All Rights Reserved.

#include "Network/Packet/DisplayClusterPacketBinary.h"
#include "Network/Transport/DisplayClusterSocketOperations.h"

#include "Misc/DisplayClusterLog.h"
#include "Misc/ScopeLock.h"


bool FDisplayClusterPacketBinary::SendPacket(FDisplayClusterSocketOperations& SocketOps)
{
	FScopeLock Lock(&SocketOps.GetSyncObj());

	if (!SocketOps.IsOpen())
	{
		UE_LOGF(LogDisplayClusterNetwork, Error, "%ls not connected", *SocketOps.GetConnectionName());
		return false;
	}

	UE_LOGF(LogDisplayClusterNetwork, VeryVerbose, "%ls - sending binary...", *SocketOps.GetConnectionName());

	// We'll be working with internal buffer to save some time on memcpy operation
	TArray<uint8>& DataBuffer = SocketOps.GetPersistentBuffer();
	DataBuffer.Reset();

	// Buffer offset for 'memcpy' operations
	uint32 WriteOffset = 0;

	// Fill packet header with data and write to the buffer
	FPacketHeader PacketHeader;
	PacketHeader.PacketBodyLength = static_cast<uint32>(PacketData.Num());
	UE_LOGF(LogDisplayClusterNetwork, VeryVerbose, "%ls - Outgoing packet body length %d", *SocketOps.GetConnectionName(), PacketHeader.PacketBodyLength);
	FMemory::Memcpy(DataBuffer.GetData() + WriteOffset, &PacketHeader, sizeof(FPacketHeader));
	WriteOffset += sizeof(FPacketHeader);

	// Write packet body data to the buffer
	FMemory::Memcpy(DataBuffer.GetData() + WriteOffset, PacketData.GetData(), PacketData.Num());
	WriteOffset += PacketData.Num();

	// Send packet
	if (!SocketOps.SendChunk(DataBuffer, WriteOffset, FString("send-binary")))
	{
		UE_LOGF(LogDisplayClusterNetwork, Warning, "%ls - couldn't send binary data", *SocketOps.GetConnectionName());
		return false;
	}

	UE_LOGF(LogDisplayClusterNetwork, VeryVerbose, "%ls - Binary packet sent", *SocketOps.GetConnectionName());
	return true;
}

bool FDisplayClusterPacketBinary::RecvPacket(FDisplayClusterSocketOperations& SocketOps)
{
	FScopeLock Lock(&SocketOps.GetSyncObj());

	if (!SocketOps.IsOpen())
	{
		UE_LOGF(LogDisplayClusterNetwork, Error, "%ls - not connected", *SocketOps.GetConnectionName());
		return false;
	}

	// We'll be working with internal buffer to save some time on memcpy operation
	TArray<uint8>& DataBuffer = SocketOps.GetPersistentBuffer();

	// Read packet header
	DataBuffer.Reset();
	if (!SocketOps.RecvChunk(DataBuffer, sizeof(FPacketHeader), FString("recv-binary-chunk-header")))
	{
		UE_LOGF(LogDisplayClusterNetwork, Warning, "%ls - couldn't receive binary packet header. Remote host has disconnected.", *SocketOps.GetConnectionName());
		return false;
	}

	// Now we can extract header data
	FPacketHeader PacketHeader;
	FMemory::Memcpy(&PacketHeader, DataBuffer.GetData(), sizeof(FPacketHeader));
	UE_LOGF(LogDisplayClusterNetwork, VeryVerbose, "%ls - binary header received: %ls", *SocketOps.GetConnectionName(), *PacketHeader.ToString());
	check(PacketHeader.PacketBodyLength > 0);

	// Read packet body
	DataBuffer.Reset();
	if (!SocketOps.RecvChunk(PacketData, PacketHeader.PacketBodyLength, FString("recv-binary-chunk-data")))
	{
		UE_LOGF(LogDisplayClusterNetwork, Warning, "%ls - couldn't receive binary packet body", *SocketOps.GetConnectionName());
		return false;
	}

	UE_LOGF(LogDisplayClusterNetwork, VeryVerbose, "%ls - binary body received", *SocketOps.GetConnectionName());

	return true;
}

FString FDisplayClusterPacketBinary::ToLogString(bool bDetailed) const
{
	if (bDetailed)
	{
		const int32 LogRange = 16;

		if (PacketData.Num() > (LogRange * 2))
		{
			return FString::Printf(TEXT("<length=%d, body=[%s...%s]>"),
				PacketData.Num(),
				*BytesToHex(PacketData.GetData(), LogRange),
				*BytesToHex(PacketData.GetData() + PacketData.Num() - LogRange, LogRange));
		}
		else
		{
			return FString::Printf(TEXT("<length=%d, body=%s>"), PacketData.Num(), *BytesToHex(PacketData.GetData(), PacketData.Num()));
		}
	}
	else
	{
		return FString::Printf(TEXT("<length=%d>"), PacketData.Num());
	}
}
