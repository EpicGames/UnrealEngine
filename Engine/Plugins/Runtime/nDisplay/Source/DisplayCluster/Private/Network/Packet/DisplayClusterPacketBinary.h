// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Network/Packet/IDisplayClusterPacket.h"


/**
 * Binary packet type for display cluster networking
 */
class FDisplayClusterPacketBinary
	: public IDisplayClusterPacket
{
public:
	TArray<uint8>& GetPacketData()
	{
		return PacketData;
	}

	const TArray<uint8>& GetPacketData() const
	{
		return PacketData;
	}

	void SetEventData(const TArray<uint8>& InPacketData)
	{
		PacketData = InPacketData;
	}

public:
	//////////////////////////////////////////////////////////////////////////////////////////////
	// IDisplayClusterPacket
	//////////////////////////////////////////////////////////////////////////////////////////////
	virtual bool SendPacket(FDisplayClusterSocketOperations& SocketOps) override;
	virtual bool RecvPacket(FDisplayClusterSocketOperations& SocketOps) override;
	virtual FString ToLogString(bool bDetailed = false) const override;

private:
	struct FPacketHeader
	{
		uint32 PacketBodyLength;

		FString ToString()
		{
			return FString::Printf(TEXT("<length=%u>"), PacketBodyLength);
		}
	};

private:
	TArray<uint8> PacketData;
};
