// Copyright Epic Games, Inc. All Rights Reserved.

#include "Network/Packet/DisplayClusterPacketInternal.h"
#include "Network/Transport/DisplayClusterSocketOperations.h"

#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"

#include "Misc/DisplayClusterLog.h"


void FDisplayClusterPacketInternal::SerializeDC(FArchive& Ar)
{
	// Header
	Ar << Name;
	Ar << Type;
	Ar << Protocol;

	// Comm result
	Ar << CommResult;

	// Arguments
	Ar << TextArguments;
	Ar << BinaryArguments;

	// Objects
	Ar << TextObjects;
	Ar << BinaryObjects;
}

FString FDisplayClusterPacketInternal::ToString() const
{
	return FString::Printf(TEXT("<Protocol=%s, Type=%s, Name=%s, CommErr=%d Args={%s} Text_Objects=%d Bin_Objects=%d>"),
		*GetProtocol(), *GetType(), *GetName(), EnumToUnderlyingType(CommResult), *ArgsToString(), TextObjects.Num(), BinaryObjects.Num());
}

FString FDisplayClusterPacketInternal::ArgsToString() const
{
	FString TmpStr;
	TmpStr.Reserve(512);
	
	for (const auto& Category : TextArguments)
	{
		TmpStr += FString::Printf(TEXT("%s=["), *Category.Key);

		for (const auto& Argument : Category.Value)
		{
			TmpStr += FString::Printf(TEXT("%s=%s "), *Argument.Key, *Argument.Value);
		}

		TmpStr += FString("] ");
	}

	return TmpStr;
}

bool FDisplayClusterPacketInternal::SendPacket(FDisplayClusterSocketOperations& SocketOps)
{
	FScopeLock Lock(&SocketOps.GetSyncObj());

	if (!SocketOps.IsOpen())
	{
		UE_LOGF(LogDisplayClusterNetwork, Error, "%ls not connected", *SocketOps.GetConnectionName());
		return false;
	}

	UE_LOGF(LogDisplayClusterNetwork, VeryVerbose, "%ls - sending internal packet...", *SocketOps.GetConnectionName());

	TArray<uint8> DataBuffer;
	FMemoryWriter MemoryWriter(DataBuffer);

	// Reserve space for packet header
	MemoryWriter.Seek(sizeof(FPacketHeader));

	// Serialize the packet body
	SerializeDC(MemoryWriter);

	// Initialize the packet header
	FPacketHeader PacketHeader;
	PacketHeader.PacketBodyLength = static_cast<uint32>(DataBuffer.Num() - sizeof(FPacketHeader));
	UE_LOGF(LogDisplayClusterNetwork, VeryVerbose, "%ls - Outgoing packet header: %ls", *SocketOps.GetConnectionName(), *PacketHeader.ToString());

	// Fill packet header with packet data length
	FMemory::Memcpy(DataBuffer.GetData(), &PacketHeader, sizeof(FPacketHeader));

	// Send the header
	static const FString SendMsgLogStr(TEXT("send-internal-msg"));
	if (!SocketOps.SendChunk(DataBuffer, DataBuffer.Num(), SendMsgLogStr))
	{
		UE_LOGF(LogDisplayClusterNetwork, Warning, "%ls - Couldn't send a packet", *SocketOps.GetConnectionName());
		return false;
	}

	UE_LOGF(LogDisplayClusterNetwork, VeryVerbose, "%ls - Packet sent", *SocketOps.GetConnectionName());

	return true;
}

bool FDisplayClusterPacketInternal::RecvPacket(FDisplayClusterSocketOperations& SocketOps)
{
	FScopeLock Lock(&SocketOps.GetSyncObj());

	if (!SocketOps.IsOpen())
	{
		UE_LOGF(LogDisplayClusterNetwork, Error, "%ls - not connected", *SocketOps.GetConnectionName());
		return false;
	}

	// We'll be working with internal buffer to save some time on memcpy operation
	TArray<uint8>& DataBuffer = SocketOps.GetPersistentBuffer();

	// Read packet header
	DataBuffer.Reset();
	static const FString RecvChunkHeaderLogStr(TEXT("recv-internal-chunk-header"));
	if (!SocketOps.RecvChunk(DataBuffer, sizeof(FPacketHeader), RecvChunkHeaderLogStr))
	{
		UE_LOGF(LogDisplayClusterNetwork, Warning, "%ls couldn't receive packet header", *SocketOps.GetConnectionName());
		return false;
	}

	// Ok. Now we can extract header data
	FPacketHeader PacketHeader;
	FMemory::Memcpy(&PacketHeader, DataBuffer.GetData(), sizeof(FPacketHeader));
	DataBuffer.Reset();

	UE_LOGF(LogDisplayClusterNetwork, VeryVerbose, "%ls - packet header received: %ls", *SocketOps.GetConnectionName(), *PacketHeader.ToString());
	check(PacketHeader.PacketBodyLength > 0);

	// Read packet body
	static const FString RecvChunkBodyLogStr(TEXT("recv-internal-chunk-body"));
	if (!SocketOps.RecvChunk(DataBuffer, PacketHeader.PacketBodyLength, RecvChunkBodyLogStr))
	{
		UE_LOGF(LogDisplayClusterNetwork, Warning, "%ls couldn't receive packet body", *SocketOps.GetConnectionName());
		return false;
	}

	// We need to set a correct value for array size before deserialization
	DataBuffer.SetNumUninitialized(PacketHeader.PacketBodyLength, EAllowShrinking::No);

	UE_LOGF(LogDisplayClusterNetwork, VeryVerbose, "%ls - packet body received", *SocketOps.GetConnectionName());

	// Deserialize packet from buffer
	FMemoryReader MemoryReader(DataBuffer);
	SerializeDC(MemoryReader);

	// Succeeded
	return true;
}

FString FDisplayClusterPacketInternal::ToLogString(bool bDetailed) const
{
	return bDetailed ? ToString() : FString::Printf(TEXT("<Protocol=%s, Type=%s, Name=%s, CommErr=%d>"), *Protocol, *Type, *Name, EnumToUnderlyingType(CommResult));
}
