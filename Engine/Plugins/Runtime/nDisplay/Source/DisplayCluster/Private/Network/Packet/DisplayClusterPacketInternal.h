// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Network/Packet/IDisplayClusterPacket.h"
#include "IDisplayClusterSerializable.h"

#include "Misc/DisplayClusterTypesConverter.h"
#include "Network/DisplayClusterNetworkTypes.h"

class FArchive;
class FDisplayClusterSocketOperations;


/**
 * Internal packet type. Used for in-cluster synchronization and data replication.
 */
class FDisplayClusterPacketInternal
	: public IDisplayClusterPacket
	, public IDisplayClusterSerializable
{
public:
	FDisplayClusterPacketInternal()
	{ }

	FDisplayClusterPacketInternal(const FString& InName, const FString& InType, const FString& InProtocol)
		: Name(InName)
		, Type(InType)
		, Protocol(InProtocol)
		, CommResult(EDisplayClusterCommResult::Ok)
	{ }

public:
	// Packet name
	inline const FString& GetName() const
	{
		return Name;
	}

	// Packet type
	inline const FString& GetType() const
	{
		return Type;
	}

	// Packet protocol
	inline const FString& GetProtocol() const
	{
		return Protocol;
	}

	inline EDisplayClusterCommResult GetCommResult() const
	{
		return CommResult;
	}

	inline void SetCommResult(EDisplayClusterCommResult InCommResult)
	{
		CommResult = InCommResult;
	}

public:
	// Get specific text argument from the packet
	template <typename ValType>
	bool GetTextArg(const FString& SectionName, const FString& ArgName, ValType& ArgVal) const
	{
		const TMap<FString, FString>* Section = TextArguments.Find(SectionName);
		if (!Section)
		{
			return false;
		}

		const FString* Value = Section->Find(ArgName);
		if (!Value)
		{
			return false;
		}

		ArgVal = DisplayClusterTypesConverter::template FromString<ValType>(*Value);
		return true;
	}

	// Set new text argument to the packet
	template <typename ValType>
	void SetTextArg(const FString& SectionName, const FString& ArgName, const ValType& ArgVal)
	{
		TMap<FString, FString>* Section = TextArguments.Find(SectionName);
		if (!Section)
		{
			Section = &TextArguments.Emplace(SectionName);
		}

		Section->Emplace(ArgName, DisplayClusterTypesConverter::template ToString<ValType>(ArgVal));
	}

	// Remove text argument
	void RemoveTextArg(const FString& SectionName, const FString& ArgName)
	{
		TMap<FString, FString>* Section = TextArguments.Find(SectionName);
		if (Section)
		{
			Section->Remove(ArgName);
		}
	}

	// Get all text arguments of a specified section
	const TMap<FString, FString> GetTextArgs(const FString& SectionName) const
	{
		const TMap<FString, FString>* Section = TextArguments.Find(SectionName);
		return Section ? *Section : TMap<FString, FString>();
	}

	// Set all text arguments to a specified section
	void SetTextArgs(const FString& SectionName, TMap<FString, FString> Data)
	{
		TextArguments.Emplace(SectionName, MoveTemp(Data));
	}

public:
	// Get specific binary argument from the packet
	bool GetBinArg(const FString& SectionName, const FString& ArgName, TArray<uint8>& ArgVal, bool bKeepData = false)
	{
		TMap<FString, TArray<uint8>>* Section = BinaryArguments.Find(SectionName);
		if (!Section)
		{
			return false;
		}

		if (bKeepData)
		{
			const TArray<uint8>* Value = Section->Find(ArgName);
			ArgVal = Value ? *Value : TArray<uint8>();
			return Value != nullptr;
		}
		else
		{
			return Section->RemoveAndCopyValue(ArgName, ArgVal);
		}
	}

	// Set new binary argument to the packet
	void SetBinArg(const FString& SectionName, const FString& ArgName, TArray<uint8> ArgVal)
	{
		BinaryArguments.FindOrAdd(SectionName).Emplace(ArgName, MoveTemp(ArgVal));
	}

	// Remove binary argument
	void RemoveBinArg(const FString& SectionName, const FString& ArgName)
	{
		TMap<FString, TArray<uint8>>* Section = BinaryArguments.Find(SectionName);
		if (Section)
		{
			Section->Remove(ArgName);
		}
	}

	// Get all binary arguments of a specified section
	TMap<FString, TArray<uint8>> GetBinArgs(const FString& SectionName, bool bKeepData = false)
	{
		if (bKeepData)
		{
			TMap<FString, TArray<uint8>>* Section = BinaryArguments.Find(SectionName);
			return Section ? *Section : TMap<FString, TArray<uint8>>();
		}
		else
		{
			TMap<FString, TArray<uint8>> SectionData;
			BinaryArguments.RemoveAndCopyValue(SectionName, SectionData);
			return MoveTemp(SectionData);
		}
	}

	// Set all binary arguments to a specified section
	void SetBinArgs(const FString& SectionName, TMap<FString, TArray<uint8>> Data)
	{
		BinaryArguments.Emplace(SectionName, MoveTemp(Data));
	}

public:
	// Adds a text object to the packet
	void AddTextObject(const FString& SectionName, const FString& Object)
	{
		TArray<FString>* Section = TextObjects.Find(SectionName);
		if (!Section)
		{
			Section = &TextObjects.Emplace(SectionName);
		}

		Section->Add(Object);
	}

	// Get all text objects
	void GetTextObjects(const FString& SectionName, TArray<FString>& OutObjects, bool bKeepData = false)
	{
		if (bKeepData)
		{
			const TArray<FString>* Objects = TextObjects.Find(SectionName);
			OutObjects = (Objects ? *Objects : TArray<FString>());
		}
		else
		{
			TextObjects.RemoveAndCopyValue(SectionName, OutObjects);
		}
	}

public:
	// Adds a binary object to the packet
	void AddBinObject(const FString& SectionName, TArray<uint8> Object)
	{
		BinaryObjects.FindOrAdd(SectionName).Add(MoveTemp(Object));
	}

	// Get all binary objects
	void GetBinObjects(const FString& SectionName, TArray<TArray<uint8>>& OutObjects, bool bKeepData = false)
	{
		if (bKeepData)
		{
			const TArray<TArray<uint8>>* Objects = BinaryObjects.Find(SectionName);
			OutObjects = (Objects ? *Objects : TArray<TArray<uint8>>());
		}
		else
		{
			BinaryObjects.RemoveAndCopyValue(SectionName, OutObjects);
		}
	}

public:

	//~ Begin IDisplayClusterPacket
	virtual bool SendPacket(FDisplayClusterSocketOperations& SocketOps) override;
	virtual bool RecvPacket(FDisplayClusterSocketOperations& SocketOps) override;
	virtual FString ToLogString(bool bDetailed = false) const override;
	//~ End IDisplayClusterPacket

public:

	//~ Begin IDisplayClusterSerializable
	virtual void SerializeDC(FArchive& Ar) override;
	//~ End IDisplayClusterSerializable

protected:

	// Text representation of the whole meessage
	FString ToString() const;
	FString ArgsToString() const;

protected:
	struct FPacketHeader
	{
		uint32 PacketBodyLength;

		FString ToString()
		{
			return FString::Printf(TEXT("<length=%u>"), PacketBodyLength);
		}
	};

private:
	FString Name;
	FString Type;
	FString Protocol;

	EDisplayClusterCommResult CommResult;

	TMap<FString, TMap<FString, FString>>       TextArguments;
	TMap<FString, TMap<FString, TArray<uint8>>> BinaryArguments;
	
	TMap<FString, TArray<FString>> TextObjects;
	TMap<FString, TArray<TArray<uint8>>>   BinaryObjects;
};
