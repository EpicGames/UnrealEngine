// Copyright Epic Games, Inc. All Rights Reserved.

#include "Network/Packet/DisplayClusterPacketJson.h"

#include "Network/Transport/DisplayClusterSocketOperations.h"

#include "Dom/JsonObject.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonWriter.h"
#include "Serialization/JsonSerializer.h"

#include "Misc/DisplayClusterLog.h"
#include "Misc/ScopeLock.h"


bool FDisplayClusterPacketJson::SendPacket(FDisplayClusterSocketOperations& SocketOps)
{
	FScopeLock Lock(&SocketOps.GetSyncObj());

	if (!SocketOps.IsOpen())
	{
		UE_LOGF(LogDisplayClusterNetwork, Error, "%ls not connected", *SocketOps.GetConnectionName());
		return false;
	}

	UE_LOGF(LogDisplayClusterNetwork, VeryVerbose, "%ls - sending json...", *SocketOps.GetConnectionName());

	// We'll be working with internal buffer to save some time on memcpy operation
	TArray<uint8>& DataBuffer = SocketOps.GetPersistentBuffer();
	DataBuffer.Reset();

	// Serialize data to json text
	FString OutputString;
	TSharedRef<TJsonWriter<>> JsonWriter = TJsonWriterFactory<>::Create(&OutputString);
	if(!FJsonSerializer::Serialize(JsonData.ToSharedRef(), JsonWriter))
	{
		UE_LOGF(LogDisplayClusterNetwork, Error, "%ls - Couldn't serialize json data", *SocketOps.GetConnectionName());
		return false;
	}

	uint32 WriteOffset = 0;

	// Fil packet header with data
	FPacketHeader PacketHeader;
	PacketHeader.PacketBodyLength = OutputString.Len();
	UE_LOGF(LogDisplayClusterNetwork, VeryVerbose, "%ls - Outgoing packet header: %ls", *SocketOps.GetConnectionName(), *PacketHeader.ToString());

	// Write packet header
	FMemory::Memcpy(DataBuffer.GetData() + WriteOffset, &PacketHeader, sizeof(FPacketHeader));
	WriteOffset += sizeof(FPacketHeader);

	// Write packet body
	FMemory::Memcpy(DataBuffer.GetData() + WriteOffset, TCHAR_TO_ANSI(*OutputString), PacketHeader.PacketBodyLength);
	WriteOffset += PacketHeader.PacketBodyLength;

	// Send packet
	if (!SocketOps.SendChunk(DataBuffer, WriteOffset, FString("send-json")))
	{
		UE_LOGF(LogDisplayClusterNetwork, Warning, "%ls - Couldn't send json", *SocketOps.GetConnectionName());
		return false;
	}

	UE_LOGF(LogDisplayClusterNetwork, VeryVerbose, "%ls - Json sent", *SocketOps.GetConnectionName());
	return true;
}

bool FDisplayClusterPacketJson::RecvPacket(FDisplayClusterSocketOperations& SocketOps)
{
	FScopeLock Lock(&SocketOps.GetSyncObj());

	if (!SocketOps.IsOpen())
	{
		UE_LOGF(LogDisplayClusterNetwork, Error, "%ls - not connected", *SocketOps.GetConnectionName());
		return false;
	}

	// We'll be working with internal buffer to save some time on memcpy operation
	TArray<uint8>& DataBuffer = SocketOps.GetPersistentBuffer();

	// Read packet header
	FPacketHeader PacketHeader;
	DataBuffer.Reset();
	if (!SocketOps.RecvChunk(DataBuffer, sizeof(FPacketHeader), FString("recv-json-chunk-header")))
	{
		UE_LOGF(LogDisplayClusterNetwork, Warning, "%ls - couldn't receive packet header. Remote host has disconnected.", *SocketOps.GetConnectionName());
		return false;
	}

	// Ok. Now we can extract header data
	FMemory::Memcpy(&PacketHeader, DataBuffer.GetData(), sizeof(FPacketHeader));
	UE_LOGF(LogDisplayClusterNetwork, VeryVerbose, "%ls - json header received: %ls", *SocketOps.GetConnectionName(), *PacketHeader.ToString());
	check(PacketHeader.PacketBodyLength > 0);

	// Read packet body
	DataBuffer.Reset();
	if (!SocketOps.RecvChunk(DataBuffer, PacketHeader.PacketBodyLength, FString("recv-json-chunk-body")))
	{
		UE_LOGF(LogDisplayClusterNetwork, Warning, "%ls - couldn't receive packet body", *SocketOps.GetConnectionName());
		return false;
	}

	UE_LOGF(LogDisplayClusterNetwork, VeryVerbose, "%ls - json body received", *SocketOps.GetConnectionName());

	// Add string zero terminator
	DataBuffer.GetData()[PacketHeader.PacketBodyLength] = 0;
	DataBuffer.AddUninitialized(PacketHeader.PacketBodyLength + 1);

	const FString JsonString(reinterpret_cast<char*>(DataBuffer.GetData()));
	TSharedRef<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(JsonString);

	// Now we deserialize a string to Json object
	if (!FJsonSerializer::Deserialize(JsonReader, JsonData))
	{
		UE_LOGF(LogDisplayClusterNetwork, Error, "%ls couldn't deserialize json packet", *SocketOps.GetConnectionName());
		return false;
	}

	// Succeeded
	UE_LOGF(LogDisplayClusterNetwork, VeryVerbose, "%ls - received json packet: %ls", *SocketOps.GetConnectionName(), *JsonString);
	return true;
}

FString FDisplayClusterPacketJson::ToLogString(bool bDetailed) const
{
	if (bDetailed)
	{
		FString JsonString;
		TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&JsonString);
		FJsonSerializer::Serialize(JsonData.ToSharedRef(), Writer);

		return FString::Printf(TEXT("<%s>"), *JsonString);
	}
	else
	{
		return FString("JsonObject");
	}
}
