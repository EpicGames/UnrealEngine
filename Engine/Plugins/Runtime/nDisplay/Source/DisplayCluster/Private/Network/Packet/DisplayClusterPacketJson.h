// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Network/Packet/IDisplayClusterPacket.h"
#include "Dom/JsonObject.h"


/**
 * JSON based packet type for display cluster networking
 */
class FDisplayClusterPacketJson
	: public IDisplayClusterPacket
{
public:
	FDisplayClusterPacketJson()
		: JsonData(new FJsonObject)
	{ }

public:
	inline const TSharedPtr<FJsonObject> GetJsonData() const
	{
		return JsonData;
	}

	inline void SetJsonData(TSharedPtr<FJsonObject> InJsonData)
	{
		JsonData = InJsonData;
	}

public:
	//////////////////////////////////////////////////////////////////////////////////////////////
	// IDisplayClusterPacket
	//////////////////////////////////////////////////////////////////////////////////////////////
	virtual bool SendPacket(FDisplayClusterSocketOperations& SocketOps) override;
	virtual bool RecvPacket(FDisplayClusterSocketOperations& SocketOps) override;
	virtual FString ToLogString(bool bDetailed = false) const override;

private:
	struct FPacketHeader
	{
		uint32 PacketBodyLength = 0;

		FString ToString()
		{
			return FString::Printf(TEXT("<length=%u>"), PacketBodyLength);
		}
	};

private:
	TSharedPtr<FJsonObject> JsonData;
};
