// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FDisplayClusterSocketOperations;


/**
 * Network packet interface
 */
class IDisplayClusterPacket
{
public:
	virtual ~IDisplayClusterPacket() = default;

public:
	virtual bool SendPacket(FDisplayClusterSocketOperations& SocketOps) = 0;
	virtual bool RecvPacket(FDisplayClusterSocketOperations& SocketOps) = 0;
	virtual FString ToLogString(bool bDetailed = false) const = 0;
};
