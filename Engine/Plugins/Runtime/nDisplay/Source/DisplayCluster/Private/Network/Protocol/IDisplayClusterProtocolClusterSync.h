// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DisplayClusterEnums.h"

#include "Cluster/DisplayClusterClusterEvent.h"

#include "Misc/Optional.h"
#include "Misc/QualifiedFrameTime.h"

#include "Network/DisplayClusterNetworkTypes.h"


/**
 * Cluster synchronization protocol. Used to synchronize/replicate any
 * DisplayCluster data on the game thread.
 */
class IDisplayClusterProtocolClusterSync
{
public:

	virtual ~IDisplayClusterProtocolClusterSync() = default;

public:

	/** Game start barrier syncrhonization */
	virtual EDisplayClusterCommResult WaitForGameStart() = 0;

	/** Frame start barrier syncrhonization */
	virtual EDisplayClusterCommResult WaitForFrameStart() = 0;

	/** Frame end barrier syncrhonization */
	virtual EDisplayClusterCommResult WaitForFrameEnd() = 0;

	/** Engine time syncrhonization */
	virtual EDisplayClusterCommResult GetTimeData(double& OutDeltaTime, double& OutGameTime, TOptional<FQualifiedFrameTime>& OutFrameTime) = 0;

	/** Custom objects syncrhonization */
	virtual EDisplayClusterCommResult GetObjectsData(EDisplayClusterSyncGroup InSyncGroup, TMap<FString, TArray<uint8>>& OutObjectsData) = 0;

	/** Cluster events syncrhonization */
	virtual EDisplayClusterCommResult GetEventsData(TArray<TSharedPtr<FDisplayClusterClusterEventJson, ESPMode::ThreadSafe>>& OutJsonEvents, TArray<TSharedPtr<FDisplayClusterClusterEventBinary, ESPMode::ThreadSafe>>& OutBinaryEvents) = 0;

	/** Native UE input syncrhonization */
	virtual EDisplayClusterCommResult GetNativeInputData(TMap<FString, TArray<uint8>>& OutNativeInputData) = 0;

	/** Custom states synchronization */
	virtual EDisplayClusterCommResult PropagateStatesData(const TArray<uint8>& InLocalStatesData, TArray<uint8>& OutClusterStatesData) = 0;
};
