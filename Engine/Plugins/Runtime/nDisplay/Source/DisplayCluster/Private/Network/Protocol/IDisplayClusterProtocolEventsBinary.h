// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Network/DisplayClusterNetworkTypes.h"

struct FDisplayClusterClusterEventBinary;


/**
 * Binary cluster events protocol.
 */
class IDisplayClusterProtocolEventsBinary
{
public:

	virtual ~IDisplayClusterProtocolEventsBinary() = default;

public:

	/** Emits binary cluster event */
	virtual EDisplayClusterCommResult EmitClusterEventBinary(const FDisplayClusterClusterEventBinary& Event) = 0;
};
