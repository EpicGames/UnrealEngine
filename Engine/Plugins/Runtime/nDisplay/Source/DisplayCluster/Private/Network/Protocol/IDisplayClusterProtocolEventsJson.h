// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Network/DisplayClusterNetworkTypes.h"

struct FDisplayClusterClusterEventJson;


/**
 * JSON cluster events protocol
 */
class IDisplayClusterProtocolEventsJson
{
public:

	virtual ~IDisplayClusterProtocolEventsJson() = default;

public:

	/** Emits JSON cluster event */
	virtual EDisplayClusterCommResult EmitClusterEventJson(const FDisplayClusterClusterEventJson& Event) = 0;
};
