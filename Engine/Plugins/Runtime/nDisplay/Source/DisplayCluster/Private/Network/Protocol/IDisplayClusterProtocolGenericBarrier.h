// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Network/DisplayClusterNetworkTypes.h"


/**
 * Generic barriers protocol.
 * Provides generic barrier sync mechanism.
 */
class IDisplayClusterProtocolGenericBarrier
{
public:

	virtual ~IDisplayClusterProtocolGenericBarrier() = default;

public:

	/**
	 * Just a union of different barrier control operation results to simplify the management API
	 */
	enum class EBarrierControlResult : uint8
	{
		CreatedSuccessfully = 0,
		AlreadyExists,
		NotFound,
		ReleasedSuccessfully,
		SynchronizedSuccessfully,
		UnknownError
	};

public:

	/** Creates new barrier */
	virtual EDisplayClusterCommResult CreateBarrier(const FString& BarrierId, const TMap<FString, TSet<FString>>& NodeToSyncCallers, uint32 Timeout, EBarrierControlResult& Result) = 0;

	/** Wait until a barrier with specific ID is created and ready to go */
	virtual EDisplayClusterCommResult WaitUntilBarrierIsCreated(const FString& BarrierId, EBarrierControlResult& Result) = 0;

	/** Checks if a specific barrier exists */
	virtual EDisplayClusterCommResult IsBarrierAvailable(const FString& BarrierId, EBarrierControlResult& Result) = 0;

	/** Releases specific barrier */
	virtual EDisplayClusterCommResult ReleaseBarrier(const FString& BarrierId, EBarrierControlResult& Result) = 0;

	/** Synchronize calling thread on a specific barrier */
	virtual EDisplayClusterCommResult SyncOnBarrier(const FString& BarrierId, const FString& CallerId, EBarrierControlResult& Result) = 0;

	/** Synchronize calling thread on a specific barrier(with custom data) */
	virtual EDisplayClusterCommResult SyncOnBarrierWithData(const FString& BarrierId, const FString& CallerId, const TArray<uint8>& RequestData, TArray<uint8>& OutResponseData, EBarrierControlResult& Result) = 0;
};
