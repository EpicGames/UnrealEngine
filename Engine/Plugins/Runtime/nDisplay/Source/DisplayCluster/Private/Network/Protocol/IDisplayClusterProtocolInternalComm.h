// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Network/DisplayClusterNetworkTypes.h"
#include "Serialization/Archive.h"


/**
 * In-cluster communication protocol. Used to maintain cluster lifetime.
 */
class IDisplayClusterProtocolInternalComm
{
public:

	virtual ~IDisplayClusterProtocolInternalComm() = default;

public: // GatherServicesHostingInfo types

	/**
	 * Holds all the ports being listened on by a single node
	 */
	struct FNodeServicesHostingInfo
	{
		/** Cluster syncrhonization internal port */
		uint16 ClusterSyncPort = 0;

		/** Binary events external port */
		uint16 BinaryEventsPort = 0;

		/** JSON events external port */
		uint16 JsonEventsPort = 0;


		/** Serialization operator */
		friend FArchive& operator << (FArchive& Ar, FNodeServicesHostingInfo& Struct)
		{
			Ar << Struct.ClusterSyncPort;
			Ar << Struct.BinaryEventsPort;
			Ar << Struct.JsonEventsPort;

			return Ar;
		}
	};

	/**
	 * Holds per-node information about the ports being listened
	 */
	struct FClusterServicesHostingInfo
	{
		/** NodeId-to-HostingInfo map */
		TMap<FName, FNodeServicesHostingInfo> ClusterHostingInfo;
	};

public:

	/** Provides whole cluster hosting information */
	virtual EDisplayClusterCommResult GatherServicesHostingInfo(const FNodeServicesHostingInfo& ThisNodeInfo, FClusterServicesHostingInfo& OutHostingInfo) = 0;

	/** A synchronization/negotiation step to recover after failure */
	virtual EDisplayClusterCommResult PostFailureNegotiate(TArray<uint8>& InOutRecoveryData) = 0;

	/** Primary-to-Secondary notification about secondary node losses */
	virtual EDisplayClusterCommResult RequestNodeDrop(const FString& NodeId, uint8 DropReason) = 0;
};
