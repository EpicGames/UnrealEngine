// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Network/DisplayClusterNetworkTypes.h"


/**
 * Rendering synchronization protocol.
 * Used to synchronize frame presentation on RHI thread.
 */
class IDisplayClusterProtocolRenderSync
{
public:

	virtual ~IDisplayClusterProtocolRenderSync() = default;

public:

	/** Synchronize RHI thread on a network barrier */
	virtual EDisplayClusterCommResult SynchronizeOnBarrier() = 0;
};
