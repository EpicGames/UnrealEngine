// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Network/DisplayClusterClient.h"
#include "Network/Protocol/IDisplayClusterProtocolEventsBinary.h"
#include "Network/Packet/DisplayClusterPacketBinary.h"

struct FDisplayClusterClusterEventBinary;


/**
 * Binary cluster events TCP client
 */
class FDisplayClusterClusterEventsBinaryClient
	: public FDisplayClusterClient<FDisplayClusterPacketBinary>
	, public IDisplayClusterProtocolEventsBinary
{
public:

	FDisplayClusterClusterEventsBinaryClient(const FName& InName, bool bIsInternal = true);

public:

	//~ Begin IDisplayClusterClient
	virtual bool Connect(const FString& Address, const uint16 Port, const uint32 ConnectRetriesAmount = 0, const uint32 ConnectRetryDelay = 0) override;
	//~ End IDisplayClusterClient

public:

	//~ Begin IDisplayClusterProtocolEventsBinary
	EDisplayClusterCommResult EmitClusterEventBinary(const FDisplayClusterClusterEventBinary& Event) override;
	//~ End IDisplayClusterProtocolEventsBinary

private:

	/** Whether this client is intended to be used with an internal server, and therefore greet on connection */
	const bool bIsInternalClient;
};
