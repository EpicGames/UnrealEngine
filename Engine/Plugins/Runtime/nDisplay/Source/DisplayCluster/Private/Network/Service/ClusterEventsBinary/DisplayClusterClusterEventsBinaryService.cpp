// Copyright Epic Games, Inc. All Rights Reserved.

#include "Network/Service/ClusterEventsBinary/DisplayClusterClusterEventsBinaryService.h"
#include "Network/Service/ClusterEventsBinary/DisplayClusterClusterEventsBinaryStrings.h"
#include "Network/Session/DisplayClusterSession.h"
#include "Network/Configuration/DisplayClusterNetworkConfiguration.h"
#include "Network/Conversion/DisplayClusterNetworkDataConversion.h"

#include "Cluster/IPDisplayClusterClusterManager.h"
#include "Cluster/DisplayClusterClusterEvent.h"

#include "Misc/DisplayClusterGlobals.h"
#include "Misc/DisplayClusterLog.h"

#include "Interfaces/IPv4/IPv4Endpoint.h"


FDisplayClusterClusterEventsBinaryService::FDisplayClusterClusterEventsBinaryService(const FName& InInstanceName)
	: FDisplayClusterService(InInstanceName.ToString())
{
	// Subscribe for SessionClosed events
	OnSessionClosed().AddRaw(this, &FDisplayClusterClusterEventsBinaryService::ProcessSessionClosed);
}

FDisplayClusterClusterEventsBinaryService::~FDisplayClusterClusterEventsBinaryService()
{
	// Unsubscribe from SessionClosed notifications
	OnSessionClosed().RemoveAll(this);
}


FString FDisplayClusterClusterEventsBinaryService::GetProtocolName() const
{
	return DisplayClusterClusterEventsBinaryStrings::ProtocolName;
}

TSharedPtr<IDisplayClusterSession> FDisplayClusterClusterEventsBinaryService::CreateSession(FDisplayClusterSessionInfo& SessionInfo)
{
	SessionInfo.SessionName = FString::Printf(TEXT("%s_%llu_%s_%s"),
		*GetName(),
		SessionInfo.SessionId,
		*SessionInfo.Endpoint.ToString(),
		*SessionInfo.NodeId.Get(TEXT("(na)"))
	);

	return MakeShared<FDisplayClusterSession<FDisplayClusterPacketBinary, false>>(SessionInfo, *this, *this, FDisplayClusterService::GetThreadPriority());
}

void FDisplayClusterClusterEventsBinaryService::ProcessSessionClosed(const FDisplayClusterSessionInfo& SessionInfo)
{
	if (!SessionInfo.IsTerminatedByServer())
	{
		// Ignore sessions with empty NodeId as those could be external
		if (SessionInfo.NodeId.IsSet())
		{
			// Prepare failure info
			FDisplayClusterServiceFailureEvent EventInfo;
			EventInfo.NodeFailed = SessionInfo.NodeId;
			EventInfo.FailureType = FDisplayClusterServiceFailureEvent::ENodeFailType::ConnectionLost;

			// Notify others about node fail
			OnNodeFailed().Broadcast(EventInfo);
		}
	}
}

//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterSessionListener
//////////////////////////////////////////////////////////////////////////////////////////////
typename IDisplayClusterSessionPacketHandler<FDisplayClusterPacketBinary, false>::ReturnType FDisplayClusterClusterEventsBinaryService::ProcessPacket(const TSharedPtr<FDisplayClusterPacketBinary>& Request, const FDisplayClusterSessionInfo& SessionInfo)
{
	// Check the pointer
	if (!Request.IsValid())
	{
		UE_LOGF(LogDisplayClusterNetwork, Error, "%ls - Invalid request data (nullptr)", *GetName());
		return IDisplayClusterSessionPacketHandler<FDisplayClusterPacketBinary, false>::ReturnType();
	}

	// Convert net packet to the internal event data type
	FDisplayClusterClusterEventBinary ClusterEvent;
	if (!UE::nDisplay::DisplayClusterNetworkDataConversion::BinaryPacketToBinaryEvent(Request, ClusterEvent))
	{
		UE_LOGF(LogDisplayClusterNetwork, Error, "%ls - couldn't translate net packet data to binary event", *GetName());
		return typename IDisplayClusterSessionPacketHandler<FDisplayClusterPacketBinary, false>::ReturnType();
	}

	// Emit the event
	UE_LOGF(LogDisplayClusterNetwork, Verbose, "%ls - re-emitting cluster event for internal replication...", *GetName());
	EmitClusterEventBinary(ClusterEvent);

	return typename IDisplayClusterSessionPacketHandler<FDisplayClusterPacketBinary, false>::ReturnType();
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProtocolEventsBinary
//////////////////////////////////////////////////////////////////////////////////////////////
EDisplayClusterCommResult FDisplayClusterClusterEventsBinaryService::EmitClusterEventBinary(const FDisplayClusterClusterEventBinary& Event)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SRV_CEB::EmitClusterEventBinary);

	GDisplayCluster->GetPrivateClusterMgr()->EmitClusterEventBinary(Event, true);
	return EDisplayClusterCommResult::Ok;
}
