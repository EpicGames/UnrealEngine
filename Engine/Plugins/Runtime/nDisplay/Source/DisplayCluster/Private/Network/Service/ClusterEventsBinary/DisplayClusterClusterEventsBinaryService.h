// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Network/Service/DisplayClusterService.h"
#include "Network/Session/IDisplayClusterSessionPacketHandler.h"
#include "Network/Protocol/IDisplayClusterProtocolEventsBinary.h"
#include "Network/Packet/DisplayClusterPacketBinary.h"


/**
 * Binary cluster events server
 */
class FDisplayClusterClusterEventsBinaryService
	: public    FDisplayClusterService
	, public    IDisplayClusterSessionPacketHandler<FDisplayClusterPacketBinary, false>
	, protected IDisplayClusterProtocolEventsBinary
{
public:

	FDisplayClusterClusterEventsBinaryService(const FName& InstanceName);

	virtual ~FDisplayClusterClusterEventsBinaryService();

public:

	//~ Begin IDisplayClusterServer
	virtual FString GetProtocolName() const override;
	//~ End IDisplayClusterServer

protected:

	/** Creates session instance for this service */
	virtual TSharedPtr<IDisplayClusterSession> CreateSession(FDisplayClusterSessionInfo& SessionInfo) override;

	/** Callback when a session is closed */
	void ProcessSessionClosed(const FDisplayClusterSessionInfo& SessionInfo);

protected:

	//~ Begin IDisplayClusterSessionPacketHandler
	virtual typename IDisplayClusterSessionPacketHandler<FDisplayClusterPacketBinary, false>::ReturnType ProcessPacket(const TSharedPtr<FDisplayClusterPacketBinary>& Request, const FDisplayClusterSessionInfo& SessionInfo) override;
	//~ End IDisplayClusterSessionPacketHandler

protected:

	//~ Begin IDisplayClusterProtocolEventsBinary
	virtual EDisplayClusterCommResult EmitClusterEventBinary(const FDisplayClusterClusterEventBinary& Event) override;
	//~ End IDisplayClusterProtocolEventsBinary
};
