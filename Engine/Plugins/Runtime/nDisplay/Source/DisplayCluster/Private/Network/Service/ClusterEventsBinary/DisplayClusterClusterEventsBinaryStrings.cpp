// Copyright Epic Games, Inc. All Rights Reserved.

#include "Network/Service/ClusterEventsBinary/DisplayClusterClusterEventsBinaryStrings.h"


namespace DisplayClusterClusterEventsBinaryStrings
{
	const FString ProtocolName = TEXT("EventsBinary");
	const FString TypeRequest  = TEXT("Request");
};
