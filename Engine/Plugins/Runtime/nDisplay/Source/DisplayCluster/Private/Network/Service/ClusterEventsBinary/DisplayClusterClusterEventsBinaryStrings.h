// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"


/**
 * Binary cluster event protocol strings
 */
namespace DisplayClusterClusterEventsBinaryStrings
{
	extern const FString ProtocolName;
	extern const FString TypeRequest;
};
