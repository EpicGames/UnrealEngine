// Copyright Epic Games, Inc. All Rights Reserved.

#include "Network/Service/ClusterEventsJson/DisplayClusterClusterEventsJsonClient.h"

#include "Cluster/DisplayClusterClusterEvent.h"
#include "Cluster/IPDisplayClusterClusterManager.h"

#include "Misc/DisplayClusterGlobals.h"
#include "Misc/DisplayClusterLog.h"

#include "Network/Configuration/DisplayClusterNetworkConfiguration.h"
#include "Network/Conversion/DisplayClusterNetworkDataConversion.h"
#include "Network/Listener/DisplayClusterHelloMessageStrings.h"
#include "Network/Packet/DisplayClusterPacketInternal.h"
#include "Network/Service/ClusterEventsJson/DisplayClusterClusterEventsJsonStrings.h"


FDisplayClusterClusterEventsJsonClient::FDisplayClusterClusterEventsJsonClient(const FName& InName, bool bIsInternal)
	: FDisplayClusterClient(InName.ToString(), 1)
	, bIsInternalClient(bIsInternal)
{
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterClient
//////////////////////////////////////////////////////////////////////////////////////////////
bool FDisplayClusterClusterEventsJsonClient::Connect(const FString& Address, const uint16 Port, const uint32 ConnectRetriesAmount, const uint32 ConnectRetryDelay)
{
	// First, allow base class to perform connection
	if (!FDisplayClusterClient::Connect(Address, Port, ConnectRetriesAmount, ConnectRetryDelay))
	{
		return false;
	}

	// Only internal clients greet the server on connection
	if (bIsInternalClient)
	{
		// Prepare 'hello' message
		TSharedPtr<FDisplayClusterPacketInternal> HelloMsg = MakeShared<FDisplayClusterPacketInternal>(
			DisplayClusterHelloMessageStrings::Hello::Name,
			DisplayClusterHelloMessageStrings::Hello::TypeRequest,
			DisplayClusterClusterEventsJsonStrings::ProtocolName
		);

		// Fill in the message with data
		const FString NodeId = GDisplayCluster->GetPrivateClusterMgr()->GetNodeId();
		HelloMsg->SetTextArg(DisplayClusterHelloMessageStrings::ArgumentsDefaultCategory, DisplayClusterHelloMessageStrings::Hello::ArgNodeId, NodeId);

		// Send message (no response awaiting)
		return SendPacket(HelloMsg);
	}

	return true;
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProtocolEventsJson
//////////////////////////////////////////////////////////////////////////////////////////////
EDisplayClusterCommResult FDisplayClusterClusterEventsJsonClient::EmitClusterEventJson(const FDisplayClusterClusterEventJson& Event)
{
	// Convert internal json event type to json net packet
	TSharedPtr<FDisplayClusterPacketJson> Request = UE::nDisplay::DisplayClusterNetworkDataConversion::JsonEventToJsonPacket(Event);
	if (!Request)
	{
		UE_LOGF(LogDisplayClusterNetwork, Error, "Couldn't convert json cluster event data to net packet");
		return EDisplayClusterCommResult::WrongRequestData;
	}

	bool bResult = false;

	{
		// Send event
		TRACE_CPUPROFILER_EVENT_SCOPE(CLN_CEJ::EmitClusterEventJson);
		bResult = SendPacket(Request);
	}

	if (!bResult)
	{
		UE_LOGF(LogDisplayClusterNetwork, Error, "Couldn't send json cluster event");
		return EDisplayClusterCommResult::NetworkError;
	}

	return EDisplayClusterCommResult::Ok;
}
