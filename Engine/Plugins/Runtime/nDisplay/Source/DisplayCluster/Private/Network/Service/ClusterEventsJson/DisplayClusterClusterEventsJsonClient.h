// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Network/DisplayClusterClient.h"
#include "Network/Protocol/IDisplayClusterProtocolEventsJson.h"
#include "Network/Packet/DisplayClusterPacketJson.h"


/**
 * JSON cluster events TCP client
 */
class FDisplayClusterClusterEventsJsonClient
	: public FDisplayClusterClient<FDisplayClusterPacketJson>
	, public IDisplayClusterProtocolEventsJson
{
public:

	FDisplayClusterClusterEventsJsonClient(const FName& InName, bool bIsInternal = true);

public:

	//~ Begin IDisplayClusterClient
	virtual bool Connect(const FString& Address, const uint16 Port, const uint32 ConnectRetriesAmount = 0, const uint32 ConnectRetryDelay = 0) override;
	//~ End IDisplayClusterClient

public:

	//~ Begin IDisplayClusterProtocolEventsJson
	EDisplayClusterCommResult EmitClusterEventJson(const FDisplayClusterClusterEventJson& Event) override;
	//~ End IDisplayClusterProtocolEventsJson

private:

	/** Whether this client is intended to be used with an internal server, and therefore greet on connection */
	const bool bIsInternalClient;
};
