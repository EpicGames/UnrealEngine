// Copyright Epic Games, Inc. All Rights Reserved.

#include "Network/Service/ClusterEventsJson/DisplayClusterClusterEventsJsonService.h"
#include "Network/Service/ClusterEventsJson/DisplayClusterClusterEventsJsonStrings.h"
#include "Network/Session/DisplayClusterSession.h"
#include "Network/Configuration/DisplayClusterNetworkConfiguration.h"
#include "Network/Conversion/DisplayClusterNetworkDataConversion.h"

#include "Cluster/IPDisplayClusterClusterManager.h"
#include "Cluster/DisplayClusterClusterEvent.h"

#include "Dom/JsonObject.h"

#include "Misc/DisplayClusterGlobals.h"
#include "Misc/DisplayClusterLog.h"

#include "Interfaces/IPv4/IPv4Endpoint.h"


FDisplayClusterClusterEventsJsonService::FDisplayClusterClusterEventsJsonService(const FName& InInstanceName)
	: FDisplayClusterService(InInstanceName.ToString())
{
	// Subscribe for SessionClosed events
	OnSessionClosed().AddRaw(this, &FDisplayClusterClusterEventsJsonService::ProcessSessionClosed);
}

FDisplayClusterClusterEventsJsonService::~FDisplayClusterClusterEventsJsonService()
{
	// Unsubscribe from SessionClosed notifications
	OnSessionClosed().RemoveAll(this);
}


FString FDisplayClusterClusterEventsJsonService::GetProtocolName() const
{
	return DisplayClusterClusterEventsJsonStrings::ProtocolName;
}

TSharedPtr<IDisplayClusterSession> FDisplayClusterClusterEventsJsonService::CreateSession(FDisplayClusterSessionInfo& SessionInfo)
{
	SessionInfo.SessionName = FString::Printf(TEXT("%s_%llu_%s_%s"),
		*GetName(),
		SessionInfo.SessionId,
		*SessionInfo.Endpoint.ToString(),
		*SessionInfo.NodeId.Get(TEXT("(na)"))
	);

	return MakeShared<FDisplayClusterSession<FDisplayClusterPacketJson, false>>(SessionInfo, *this, *this, FDisplayClusterService::GetThreadPriority());
}

void FDisplayClusterClusterEventsJsonService::ProcessSessionClosed(const FDisplayClusterSessionInfo& SessionInfo)
{
	if (!SessionInfo.IsTerminatedByServer())
	{
		// Ignore the session with empty NodeId as those could be the external ones
		if (SessionInfo.NodeId.IsSet())
		{
			// Prepare failure info
			FDisplayClusterServiceFailureEvent EventInfo;
			EventInfo.NodeFailed = SessionInfo.NodeId;
			EventInfo.FailureType = FDisplayClusterServiceFailureEvent::ENodeFailType::ConnectionLost;

			// Notify others about node fail
			OnNodeFailed().Broadcast(EventInfo);
		}
	}
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterSessionListener
//////////////////////////////////////////////////////////////////////////////////////////////
typename IDisplayClusterSessionPacketHandler<FDisplayClusterPacketJson, false>::ReturnType FDisplayClusterClusterEventsJsonService::ProcessPacket(const TSharedPtr<FDisplayClusterPacketJson>& Request, const FDisplayClusterSessionInfo& SessionInfo)
{
	// Check the pointer
	if (!Request.IsValid())
	{
		UE_LOGF(LogDisplayClusterNetwork, Error, "%ls - Invalid request data (nullptr)", *GetName());
		return IDisplayClusterSessionPacketHandler<FDisplayClusterPacketJson, false>::ReturnType();
	}

	bool MandatoryFieldsExist = true;

	if (!Request->GetJsonData()->HasField(DisplayClusterClusterEventsJsonStrings::ArgName))
	{
		UE_LOGF(LogDisplayClusterNetwork, Error, "Json packet doesn't have a mandatory field: %ls", *DisplayClusterClusterEventsJsonStrings::ArgName);
		return IDisplayClusterSessionPacketHandler<FDisplayClusterPacketJson, false>::ReturnType();
	}

	if (!Request->GetJsonData()->HasField(DisplayClusterClusterEventsJsonStrings::ArgType))
	{
		UE_LOGF(LogDisplayClusterNetwork, Error, "Json packet doesn't have a mandatory field: %ls", *DisplayClusterClusterEventsJsonStrings::ArgType);
		return IDisplayClusterSessionPacketHandler<FDisplayClusterPacketJson, false>::ReturnType();
	}

	if (!Request->GetJsonData()->HasField(DisplayClusterClusterEventsJsonStrings::ArgCategory))
	{
		UE_LOGF(LogDisplayClusterNetwork, Error, "Json packet doesn't have a mandatory field: %ls", *DisplayClusterClusterEventsJsonStrings::ArgCategory);
		return IDisplayClusterSessionPacketHandler<FDisplayClusterPacketJson, false>::ReturnType();
	}

	// Convert net packet to the internal event data type
	FDisplayClusterClusterEventJson ClusterEvent;
	if(!UE::nDisplay::DisplayClusterNetworkDataConversion::JsonPacketToJsonEvent(Request, ClusterEvent))
	{
		UE_LOGF(LogDisplayClusterNetwork, Error, "%ls - couldn't translate net packet data to json event", *GetName());
		return IDisplayClusterSessionPacketHandler<FDisplayClusterPacketJson, false>::ReturnType();
	}

	// Emit the event
	UE_LOGF(LogDisplayClusterNetwork, Verbose, "%ls - re-emitting cluster event for internal replication...", *GetName());
	EmitClusterEventJson(ClusterEvent);

	return IDisplayClusterSessionPacketHandler<FDisplayClusterPacketJson, false>::ReturnType();
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProtocolEventsJson
//////////////////////////////////////////////////////////////////////////////////////////////
EDisplayClusterCommResult FDisplayClusterClusterEventsJsonService::EmitClusterEventJson(const FDisplayClusterClusterEventJson& Event)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SRV_CEJ::EmitClusterEventJson);

	GDisplayCluster->GetPrivateClusterMgr()->EmitClusterEventJson(Event, true);
	return EDisplayClusterCommResult::Ok;
}
