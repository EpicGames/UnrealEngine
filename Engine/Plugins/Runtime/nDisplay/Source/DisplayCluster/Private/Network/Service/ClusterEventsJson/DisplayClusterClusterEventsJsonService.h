// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Network/Service/DisplayClusterService.h"
#include "Network/Session/IDisplayClusterSessionPacketHandler.h"
#include "Network/Packet/DisplayClusterPacketJson.h"
#include "Network/Protocol/IDisplayClusterProtocolEventsJson.h"

#include "Dom/JsonObject.h"

/**
 * JSON cluster events server
 */
class FDisplayClusterClusterEventsJsonService
	: public    FDisplayClusterService
	, public    IDisplayClusterSessionPacketHandler<FDisplayClusterPacketJson, false>
	, protected IDisplayClusterProtocolEventsJson
{
public:

	FDisplayClusterClusterEventsJsonService(const FName& InInstanceName);

	virtual ~FDisplayClusterClusterEventsJsonService();

public:

	//~ Begin IDisplayClusterServer
	virtual FString GetProtocolName() const override;
	//~ End IDisplayClusterServer

protected:

	/** Creates session instance for this service */
	virtual TSharedPtr<IDisplayClusterSession> CreateSession(FDisplayClusterSessionInfo& SessionInfo) override;

private:

	/** Callback when a session is closed */
	void ProcessSessionClosed(const FDisplayClusterSessionInfo& SessionInfo);

protected:

	//~ Begin IDisplayClusterSessionPacketHandler
	virtual typename IDisplayClusterSessionPacketHandler<FDisplayClusterPacketJson, false>::ReturnType ProcessPacket(const TSharedPtr<FDisplayClusterPacketJson>& Request, const FDisplayClusterSessionInfo& SessionInfo) override;
	//~ End IDisplayClusterSessionPacketHandler

protected:

	//~ Begin IDisplayClusterProtocolEventsJson
	virtual EDisplayClusterCommResult EmitClusterEventJson(const FDisplayClusterClusterEventJson& Event) override;
	//~ End IDisplayClusterProtocolEventsJson
};
