// Copyright Epic Games, Inc. All Rights Reserved.

#include "Network/Service/ClusterEventsJson/DisplayClusterClusterEventsJsonStrings.h"


namespace DisplayClusterClusterEventsJsonStrings
{
	const FString ProtocolName = TEXT("EventsJSON");
	const FString TypeRequest  = TEXT("Request");

	const FString ArgName       = TEXT("Name");
	const FString ArgType       = TEXT("Type");
	const FString ArgCategory   = TEXT("Category");
};
