// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"


/**
 * JSON cluster event protocol strings
 */
namespace DisplayClusterClusterEventsJsonStrings
{
	extern const FString ProtocolName;
	extern const FString TypeRequest;

	extern const FString ArgName;
	extern const FString ArgType;
	extern const FString ArgCategory;
};
