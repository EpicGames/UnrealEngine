// Copyright Epic Games, Inc. All Rights Reserved.

#include "Network/Service/ClusterSync/DisplayClusterClusterSyncClient.h"
#include "Network/Service/ClusterSync/DisplayClusterClusterSyncStrings.h"
#include "Network/Configuration/DisplayClusterNetworkConfiguration.h"
#include "Network/Conversion/DisplayClusterNetworkDataConversion.h"
#include "Network/Packet/DisplayClusterPacketInternal.h"
#include "Network/Listener/DisplayClusterHelloMessageStrings.h"

#include "Cluster/IPDisplayClusterClusterManager.h"

#include "Misc/ScopeLock.h"
#include "Misc/FrameRate.h"
#include "Misc/Timecode.h"

#include "Misc/DisplayClusterGlobals.h"
#include "Misc/DisplayClusterHelpers.h"
#include "Misc/DisplayClusterLog.h"


FDisplayClusterClusterSyncClient::FDisplayClusterClusterSyncClient(const FName& InName)
	: FDisplayClusterClient(InName.ToString())
{
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterClient
//////////////////////////////////////////////////////////////////////////////////////////////
bool FDisplayClusterClusterSyncClient::Connect(const FString& Address, const uint16 Port, const uint32 ConnectRetriesAmount, const uint32 ConnectRetryDelay)
{
	// First, allow base class to perform connection
	if(!FDisplayClusterClient::Connect(Address, Port, ConnectRetriesAmount, ConnectRetryDelay))
	{
		return false;
	}

	// Prepare 'hello' message
	TSharedPtr<FDisplayClusterPacketInternal> HelloMsg = MakeShared<FDisplayClusterPacketInternal>(
		DisplayClusterHelloMessageStrings::Hello::Name,
		DisplayClusterHelloMessageStrings::Hello::TypeRequest,
		DisplayClusterClusterSyncStrings::ProtocolName
	);

	// Fill in the message with data
	const FString NodeId = GDisplayCluster->GetPrivateClusterMgr()->GetNodeId();
	HelloMsg->SetTextArg(DisplayClusterHelloMessageStrings::ArgumentsDefaultCategory, DisplayClusterHelloMessageStrings::Hello::ArgNodeId, NodeId);

	// Send message (no response awaiting)
	return SendPacket(HelloMsg);
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProtocolClusterSync
//////////////////////////////////////////////////////////////////////////////////////////////
EDisplayClusterCommResult FDisplayClusterClusterSyncClient::WaitForGameStart()
{
	static const TSharedPtr<FDisplayClusterPacketInternal> Request = MakeShared<FDisplayClusterPacketInternal>(
			DisplayClusterClusterSyncStrings::WaitForGameStart::Name,
			DisplayClusterClusterSyncStrings::TypeRequest,
			DisplayClusterClusterSyncStrings::ProtocolName);

	TSharedPtr<FDisplayClusterPacketInternal> Response;

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CLN_CS::WaitForGameStart);
		Response = SendRecvPacket(Request);
	}

	if (!Response)
	{
		UE_LOGF(LogDisplayClusterNetwork, Warning, "Network error on '%ls'", *Request->GetName());
		return EDisplayClusterCommResult::NetworkError;
	}

	return Response->GetCommResult();
}

EDisplayClusterCommResult FDisplayClusterClusterSyncClient::WaitForFrameStart()
{
	static const TSharedPtr<FDisplayClusterPacketInternal> Request = MakeShared<FDisplayClusterPacketInternal>(
			DisplayClusterClusterSyncStrings::WaitForFrameStart::Name,
			DisplayClusterClusterSyncStrings::TypeRequest,
			DisplayClusterClusterSyncStrings::ProtocolName);

	TSharedPtr<FDisplayClusterPacketInternal> Response;

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CLN_CS::WaitForFrameStart);
		Response = SendRecvPacket(Request);
	}

	if (!Response)
	{
		UE_LOGF(LogDisplayClusterNetwork, Warning, "Network error on '%ls'", *Request->GetName());
		return EDisplayClusterCommResult::NetworkError;
	}

	return Response->GetCommResult();
}

EDisplayClusterCommResult FDisplayClusterClusterSyncClient::WaitForFrameEnd()
{
	static const TSharedPtr<FDisplayClusterPacketInternal> Request = MakeShared<FDisplayClusterPacketInternal>(
			DisplayClusterClusterSyncStrings::WaitForFrameEnd::Name,
			DisplayClusterClusterSyncStrings::TypeRequest,
			DisplayClusterClusterSyncStrings::ProtocolName);

	TSharedPtr<FDisplayClusterPacketInternal> Response;

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CLN_CS::WaitForFrameEnd);
		Response = SendRecvPacket(Request);
	}

	if (!Response)
	{
		UE_LOGF(LogDisplayClusterNetwork, Warning, "Network error on '%ls'", *Request->GetName());
		return EDisplayClusterCommResult::NetworkError;
	}

	return Response->GetCommResult();
}

EDisplayClusterCommResult FDisplayClusterClusterSyncClient::GetTimeData(double& OutDeltaTime, double& OutGameTime, TOptional<FQualifiedFrameTime>& OutFrameTime)
{
	static const TSharedPtr<FDisplayClusterPacketInternal> Request = MakeShared<FDisplayClusterPacketInternal>(
			DisplayClusterClusterSyncStrings::GetTimeData::Name,
			DisplayClusterClusterSyncStrings::TypeRequest,
			DisplayClusterClusterSyncStrings::ProtocolName);

	TSharedPtr<FDisplayClusterPacketInternal> Response;

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CLN_CS::GetTimeData);
		Response = SendRecvPacket(Request);
	}

	if (!Response)
	{
		UE_LOGF(LogDisplayClusterNetwork, Warning, "Network error on '%ls'", *Request->GetName());
		return EDisplayClusterCommResult::NetworkError;
	}

	// Extract sync data from response packet
	FString StrDeltaTime;
	if (!Response->GetTextArg(DisplayClusterClusterSyncStrings::ArgumentsDefaultCategory, DisplayClusterClusterSyncStrings::GetTimeData::ArgDeltaTime, StrDeltaTime))
	{
		UE_LOGF(LogDisplayClusterNetwork, Warning, "Couldn't extract parameter: %ls", *DisplayClusterClusterSyncStrings::GetTimeData::ArgDeltaTime);
		return EDisplayClusterCommResult::WrongResponseData;
	}

	// Convert from hex string to float
	OutDeltaTime = DisplayClusterTypesConverter::template FromHexString<double>(StrDeltaTime);

	FString StrGameTime;
	if (!Response->GetTextArg(DisplayClusterClusterSyncStrings::ArgumentsDefaultCategory, DisplayClusterClusterSyncStrings::GetTimeData::ArgGameTime, StrGameTime))
	{
		UE_LOGF(LogDisplayClusterNetwork, Warning, "Couldn't extract parameter: %ls", *DisplayClusterClusterSyncStrings::GetTimeData::ArgGameTime);
		return EDisplayClusterCommResult::WrongResponseData;
	}

	// Convert from hex string to float
	OutGameTime = DisplayClusterTypesConverter::template FromHexString<double>(StrGameTime);

	// Extract sync data from response packet
	bool bIsFrameTimeValid = false;
	if (!Response->GetTextArg(DisplayClusterClusterSyncStrings::ArgumentsDefaultCategory, DisplayClusterClusterSyncStrings::GetTimeData::ArgIsFrameTimeValid, bIsFrameTimeValid))
	{
		UE_LOGF(LogDisplayClusterNetwork, Warning, "Couldn't extract parameter: %ls", *DisplayClusterClusterSyncStrings::GetTimeData::ArgIsFrameTimeValid);
		return EDisplayClusterCommResult::WrongResponseData;
	}

	if (bIsFrameTimeValid)
	{
		FQualifiedFrameTime NewFrameTime;
		if (!Response->GetTextArg(DisplayClusterClusterSyncStrings::ArgumentsDefaultCategory, DisplayClusterClusterSyncStrings::GetTimeData::ArgFrameTime, NewFrameTime))
		{
			UE_LOGF(LogDisplayClusterNetwork, Warning, "Couldn't extract parameter: %ls", *DisplayClusterClusterSyncStrings::GetTimeData::ArgFrameTime);
			return EDisplayClusterCommResult::WrongResponseData;
		}

		OutFrameTime = NewFrameTime;
	}

	return Response->GetCommResult();
}

EDisplayClusterCommResult FDisplayClusterClusterSyncClient::GetObjectsData(EDisplayClusterSyncGroup InSyncGroup, TMap<FString, TArray<uint8>>& OutObjectsData)
{
	static TSharedPtr<FDisplayClusterPacketInternal> Request = MakeShared<FDisplayClusterPacketInternal>(
			DisplayClusterClusterSyncStrings::GetObjectsData::Name,
			DisplayClusterClusterSyncStrings::TypeRequest,
			DisplayClusterClusterSyncStrings::ProtocolName);
	
	Request->SetTextArg(DisplayClusterClusterSyncStrings::ArgumentsDefaultCategory, DisplayClusterClusterSyncStrings::GetObjectsData::ArgSyncGroup, static_cast<uint8>(InSyncGroup));

	TSharedPtr<FDisplayClusterPacketInternal> Response;

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CLN_CS::GetObjectsData);
		Response = SendRecvPacket(Request);
	}

	if (!Response)
	{
		UE_LOGF(LogDisplayClusterNetwork, Warning, "Network error on '%ls'", *Request->GetName());
		return EDisplayClusterCommResult::NetworkError;
	}

	// Extract data from response packet
	OutObjectsData = Response->GetBinArgs(DisplayClusterClusterSyncStrings::ArgumentsDefaultCategory);

	return Response->GetCommResult();
}

EDisplayClusterCommResult FDisplayClusterClusterSyncClient::GetEventsData(TArray<TSharedPtr<FDisplayClusterClusterEventJson, ESPMode::ThreadSafe>>& OutJsonEvents, TArray<TSharedPtr<FDisplayClusterClusterEventBinary, ESPMode::ThreadSafe>>& OutBinaryEvents)
{
	static const TSharedPtr<FDisplayClusterPacketInternal> Request = MakeShared<FDisplayClusterPacketInternal>(
			DisplayClusterClusterSyncStrings::GetEventsData::Name,
			DisplayClusterClusterSyncStrings::TypeRequest,
			DisplayClusterClusterSyncStrings::ProtocolName);

	TSharedPtr<FDisplayClusterPacketInternal> Response;

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CLN_CS::GetEventsData);
		Response = SendRecvPacket(Request);
	}

	if (!Response)
	{
		UE_LOGF(LogDisplayClusterNetwork, Warning, "Network error on '%ls'", *Request->GetName());
		return EDisplayClusterCommResult::NetworkError;
	}

	// Extract events data from response packet
	UE::nDisplay::DisplayClusterNetworkDataConversion::JsonEventsFromInternalPacket(Response,   OutJsonEvents);
	UE::nDisplay::DisplayClusterNetworkDataConversion::BinaryEventsFromInternalPacket(Response, OutBinaryEvents);

	return Response->GetCommResult();
}

EDisplayClusterCommResult FDisplayClusterClusterSyncClient::GetNativeInputData(TMap<FString, TArray<uint8>>& OutNativeInputData)
{
	static const TSharedPtr<FDisplayClusterPacketInternal> Request = MakeShared<FDisplayClusterPacketInternal>(
			DisplayClusterClusterSyncStrings::GetNativeInputData::Name,
			DisplayClusterClusterSyncStrings::TypeRequest,
			DisplayClusterClusterSyncStrings::ProtocolName);

	TSharedPtr<FDisplayClusterPacketInternal> Response;

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CLN_CS::GetNativeInputData);
		Response = SendRecvPacket(Request);
	}

	if (!Response)
	{
		UE_LOGF(LogDisplayClusterNetwork, Warning, "Network error on '%ls'", *Request->GetName());
		return EDisplayClusterCommResult::NetworkError;
	}

	// Extract data from response packet
	OutNativeInputData = Response->GetBinArgs(DisplayClusterClusterSyncStrings::ArgumentsDefaultCategory);

	return Response->GetCommResult();
}

EDisplayClusterCommResult FDisplayClusterClusterSyncClient::PropagateStatesData(const TArray<uint8>& InLocalStatesData, TArray<uint8>& OutClusterStatesData)
{
	static const TSharedPtr<FDisplayClusterPacketInternal> Request = MakeShared<FDisplayClusterPacketInternal>(
		DisplayClusterClusterSyncStrings::PropagateStatesData::Name,
		DisplayClusterClusterSyncStrings::TypeRequest,
		DisplayClusterClusterSyncStrings::ProtocolName);

	Request->SetBinArg(DisplayClusterClusterSyncStrings::ArgumentsDefaultCategory, DisplayClusterClusterSyncStrings::PropagateStatesData::ArgLocalStatesData, InLocalStatesData);

	TSharedPtr<FDisplayClusterPacketInternal> Response;

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CLN_CS::PropagateStatesData);
		Response = SendRecvPacket(Request);
	}

	if (!Response)
	{
		UE_LOGF(LogDisplayClusterNetwork, Warning, "Network error on '%ls'", *Request->GetName());
		return EDisplayClusterCommResult::NetworkError;
	}

	// Extract data from response packet
	Response->GetBinArg(DisplayClusterClusterSyncStrings::ArgumentsDefaultCategory, DisplayClusterClusterSyncStrings::PropagateStatesData::ArgClusterStatesData, OutClusterStatesData);

	return Response->GetCommResult();
}
