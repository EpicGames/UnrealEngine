// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Network/DisplayClusterClient.h"
#include "Network/Packet/DisplayClusterPacketInternal.h"
#include "Network/Protocol/IDisplayClusterProtocolClusterSync.h"


/**
 * Cluster synchronization TCP client
 */
class FDisplayClusterClusterSyncClient
	: public FDisplayClusterClient<FDisplayClusterPacketInternal>
	, public IDisplayClusterProtocolClusterSync
{
public:

	FDisplayClusterClusterSyncClient(const FName& InName);

public:

	//~ Begin IDisplayClusterClient
	virtual bool Connect(const FString& Address, const uint16 Port, const uint32 ConnectRetriesAmount = 0, const uint32 ConnectRetryDelay = 0) override;
	//~ End IDisplayClusterClient

public:

	//~ Begin IDisplayClusterProtocolClusterSync
	virtual EDisplayClusterCommResult WaitForGameStart() override;
	virtual EDisplayClusterCommResult WaitForFrameStart() override;
	virtual EDisplayClusterCommResult WaitForFrameEnd() override;
	virtual EDisplayClusterCommResult GetTimeData(double& OutDeltaTime, double& OutGameTime, TOptional<FQualifiedFrameTime>& OutFrameTime) override;
	virtual EDisplayClusterCommResult GetObjectsData(EDisplayClusterSyncGroup InSyncGroup, TMap<FString, TArray<uint8>>& OutObjectsData) override;
	virtual EDisplayClusterCommResult GetEventsData(TArray<TSharedPtr<FDisplayClusterClusterEventJson, ESPMode::ThreadSafe>>& OutJsonEvents, TArray<TSharedPtr<FDisplayClusterClusterEventBinary, ESPMode::ThreadSafe>>& OutBinaryEvents) override;
	virtual EDisplayClusterCommResult GetNativeInputData(TMap<FString, TArray<uint8>>& OutNativeInputData) override;
	virtual EDisplayClusterCommResult PropagateStatesData(const TArray<uint8>& InLocalStatesData, TArray<uint8>& OutClusterStatesData) override;
	//~ End IDisplayClusterProtocolClusterSync
};
