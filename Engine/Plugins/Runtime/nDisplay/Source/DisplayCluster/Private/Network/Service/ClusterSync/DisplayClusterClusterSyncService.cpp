// Copyright Epic Games, Inc. All Rights Reserved.

#include "Network/Service/ClusterSync/DisplayClusterClusterSyncService.h"
#include "Network/Service/ClusterSync/DisplayClusterClusterSyncStrings.h"

#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterEnums.h"
#include "IDisplayCluster.h"

#include "Async/ParallelFor.h"

#include "Cluster/DisplayClusterClusterEvent.h"
#include "Cluster/IPDisplayClusterClusterManager.h"
#include "Cluster/Controller/IDisplayClusterClusterNodeController.h"
#include "Cluster/CustomStates/DisplayClusterCustomStateHelpers.h"
#include "Cluster/Failover/DisplayClusterCommDataCache.h"
#include "Cluster/RequestHandler/DisplayClusterCommRequestHandlerLocal.h"
#include "Cluster/RequestHandler/DisplayClusterCommRequestHandlerRemote.h"

#include "Config/IPDisplayClusterConfigManager.h"
#include "Game/IPDisplayClusterGameManager.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"

#include "Misc/DisplayClusterGlobals.h"
#include "Misc/DisplayClusterLog.h"
#include "Misc/QualifiedFrameTime.h"

#include "Network/Barrier/DisplayClusterBarrierFactory.h"
#include "Network/Barrier/IDisplayClusterBarrier.h"
#include "Network/Configuration/DisplayClusterNetworkConfiguration.h"
#include "Network/Conversion/DisplayClusterNetworkDataConversion.h"
#include "Network/Listener/DisplayClusterTcpListener.h"
#include "Network/Session/DisplayClusterSession.h"


FDisplayClusterClusterSyncService::FDisplayClusterClusterSyncService(const FName& InInstanceName)
	: FDisplayClusterService(InInstanceName.ToString())
{
	IDisplayCluster& DCAPI = IDisplayCluster::Get();

	// Get list of cluster node IDs
	TSet<FString> NodeIds;
	if (const IDisplayClusterClusterManager* const ClusterMgr = DCAPI.GetClusterMgr())
	{
		ClusterMgr->GetNodeIds(NodeIds);
	}

	// Get cluster configuration
	UDisplayClusterConfigurationData* ConfigData = nullptr;
	if (const IDisplayClusterConfigManager* const ConfigMgr = DCAPI.GetConfigMgr())
	{
		ConfigData = ConfigMgr->GetConfig();
	}

	// Instantiate service barriers
	if (ConfigData)
	{
		// Network settings from config data
		const FDisplayClusterConfigurationNetworkSettings& NetworkSettings = ConfigData->Cluster->Network;

		BarrierGameStart.Reset( FDisplayClusterBarrierFactory::CreateBarrier(TEXT("GameStart_barrier"),  NodeIds, NetworkSettings.GameStartBarrierTimeout));
		BarrierFrameStart.Reset(FDisplayClusterBarrierFactory::CreateBarrier(TEXT("FrameStart_barrier"), NodeIds, NetworkSettings.FrameStartBarrierTimeout));
		BarrierFrameEnd.Reset(  FDisplayClusterBarrierFactory::CreateBarrier(TEXT("FrameEnd_barrier"),   NodeIds, NetworkSettings.FrameEndBarrierTimeout));
		BarrierStatesSync.Reset(FDisplayClusterBarrierFactory::CreateBarrier(TEXT("StatesSync_barrier"), NodeIds, NetworkSettings.FrameStartBarrierTimeout));
	}
	else
	{
		UE_LOGF(LogDisplayClusterNetwork, Warning, "%ls - Couldn't obtain cluster configuration. Using default barrier timeouts.", *GetName());

		FDisplayClusterConfigurationNetworkSettings DefaultNetworkSettings;
		BarrierGameStart.Reset( FDisplayClusterBarrierFactory::CreateBarrier(TEXT("GameStart_barrier"),  NodeIds, DefaultNetworkSettings.GameStartBarrierTimeout));
		BarrierFrameStart.Reset(FDisplayClusterBarrierFactory::CreateBarrier(TEXT("FrameStart_barrier"), NodeIds, DefaultNetworkSettings.FrameStartBarrierTimeout));
		BarrierFrameEnd.Reset(  FDisplayClusterBarrierFactory::CreateBarrier(TEXT("FrameEnd_barrier"),   NodeIds, DefaultNetworkSettings.FrameEndBarrierTimeout));
		BarrierStatesSync.Reset(FDisplayClusterBarrierFactory::CreateBarrier(TEXT("StatesSync_barrier"), NodeIds, DefaultNetworkSettings.FrameStartBarrierTimeout));
	}

	// Put the barriers into an aux container
	ServiceBarriers.Emplace(BarrierGameStart->GetName(),  &BarrierGameStart);
	ServiceBarriers.Emplace(BarrierFrameStart->GetName(), &BarrierFrameStart);
	ServiceBarriers.Emplace(BarrierFrameEnd->GetName(),   &BarrierFrameEnd);
	ServiceBarriers.Emplace(BarrierStatesSync->GetName(), &BarrierStatesSync);

	// Subscribe for barrier timeout events
	for (TPair<FString, TUniquePtr<IDisplayClusterBarrier>*>& BarrierIt : ServiceBarriers)
	{
		(*BarrierIt.Value)->OnBarrierTimeout().AddRaw(this, &FDisplayClusterClusterSyncService::ProcessBarrierTimeout);
	}

	// Subscribe for SessionClosed events
	OnSessionClosed().AddRaw(this, &FDisplayClusterClusterSyncService::ProcessSessionClosed);

	// Add sync function that will be responsible for parsing requests, generating and propagating state responses
	BarrierStatesSync->GetPreSyncEndDelegate().BindRaw(this, &FDisplayClusterClusterSyncService::OnStatesSync);
}

FDisplayClusterClusterSyncService::~FDisplayClusterClusterSyncService()
{
	// Unsubscribe from barrier timeout events
	for (TPair<FString, TUniquePtr<IDisplayClusterBarrier>*>& BarrierIt : ServiceBarriers)
	{
		(*BarrierIt.Value)->OnBarrierTimeout().RemoveAll(this);
	}

	// Reset states sync delegate
	BarrierStatesSync->GetPreSyncEndDelegate().Unbind();

	// Unsubscribe from SessionClosed notifications
	OnSessionClosed().RemoveAll(this);

	ShutdownImpl();
}


bool FDisplayClusterClusterSyncService::Start(const FString& Address, const uint16 Port)
{
	// Start internals
	StartInternal();

	return FDisplayClusterServer::Start(Address, Port);
}

bool FDisplayClusterClusterSyncService::Start(TSharedRef<FDisplayClusterTcpListener>& ExternalListener)
{
	// Start internals
	StartInternal();

	return FDisplayClusterServer::Start(ExternalListener);
}

void FDisplayClusterClusterSyncService::Shutdown()
{
	// Release internals
	ShutdownImpl();

	return FDisplayClusterServer::Shutdown();
}

FString FDisplayClusterClusterSyncService::GetProtocolName() const
{
	return DisplayClusterClusterSyncStrings::ProtocolName;
}

void FDisplayClusterClusterSyncService::KillSession(const FString& NodeId)
{
	// Before killing the session on the parent level, we need to unregister this node from the barriers
	for (TPair<FString, TUniquePtr<IDisplayClusterBarrier>*>& BarrierIt : ServiceBarriers)
	{
		(*BarrierIt.Value)->UnregisterSyncCaller(NodeId);
	}

	// Now do the session related job
	FDisplayClusterServer::KillSession(NodeId);
}

TSharedPtr<IDisplayClusterSession> FDisplayClusterClusterSyncService::CreateSession(FDisplayClusterSessionInfo& SessionInfo)
{
	SessionInfo.SessionName = FString::Printf(TEXT("%s_%llu_%s_%s"),
		*GetName(),
		SessionInfo.SessionId,
		*SessionInfo.Endpoint.ToString(),
		*SessionInfo.NodeId.Get(TEXT("(na)"))
	);

	return MakeShared<FDisplayClusterSession<FDisplayClusterPacketInternal, true>>(SessionInfo, *this, *this, FDisplayClusterService::GetThreadPriority());
}

void FDisplayClusterClusterSyncService::StartInternal()
{
	// Activate all barriers
	for (TPair<FString, TUniquePtr<IDisplayClusterBarrier>*>& BarrierIt : ServiceBarriers)
	{
		(*BarrierIt.Value)->Activate();
	}
}

void FDisplayClusterClusterSyncService::ProcessSessionClosed(const FDisplayClusterSessionInfo& SessionInfo)
{
	if (!SessionInfo.IsTerminatedByServer())
	{
		// Ignore sessions with empty NodeId
		if (SessionInfo.NodeId.IsSet())
		{
			const FString NodeId = SessionInfo.NodeId.Get(FString());

			// We have to unregister the node that just disconnected from all barriers
			for (TPair<FString, TUniquePtr<IDisplayClusterBarrier>*>& BarrierIt : ServiceBarriers)
			{
				(*BarrierIt.Value)->UnregisterSyncCaller(NodeId);
			}

			// Prepare failure info
			FDisplayClusterServiceFailureEvent EventInfo;
			EventInfo.NodeFailed  = SessionInfo.NodeId;
			EventInfo.FailureType = FDisplayClusterServiceFailureEvent::ENodeFailType::ConnectionLost;

			// Notify others about node fail
			OnNodeFailed().Broadcast(EventInfo);
		}
	}
}

void FDisplayClusterClusterSyncService::ProcessBarrierTimeout(const FString& BarrierName, const TSet<FString>& NodesTimedOut)
{
	// We have to unregister the node that just timed out from all the barriers
	for (TPair<FString, TUniquePtr<IDisplayClusterBarrier>*>& BarrierIt : ServiceBarriers)
	{
		for (const FString& NodeId : NodesTimedOut)
		{
			(*BarrierIt.Value)->UnregisterSyncCaller(NodeId);
		}
	}

	// Notify others about timeout
	for (const FString& NodeId : NodesTimedOut)
	{
		// Prepare failure info
		FDisplayClusterServiceFailureEvent EventInfo;
		EventInfo.NodeFailed  = NodeId;
		EventInfo.FailureType = FDisplayClusterServiceFailureEvent::ENodeFailType::BarrierTimeOut;

		// Notify others about node fail
		OnNodeFailed().Broadcast(EventInfo);
	}
}

void FDisplayClusterClusterSyncService::ShutdownImpl()
{
	// Deactivate all barriers
	for (TPair<FString, TUniquePtr<IDisplayClusterBarrier>*>& BarrierIt : ServiceBarriers)
	{
		(*BarrierIt.Value)->Deactivate();
	}
}

void FDisplayClusterClusterSyncService::OnStatesSync(FDisplayClusterBarrierPreSyncEndDelegateData& SyncData)
{
	using namespace UE::nDisplay::Cluster::Private;

	// Cluster nodes num
	const int32 NodesNum = SyncData.RequestData.Num();

	// Intermediate input data. Array of cluster nodes, each one has its own state-to-data map.
	TArray<TMap<FName, FCustomStateRequestBlob>> InputData;
	InputData.AddDefaulted(NodesNum);

	// Generate indexed node Ids for safe access from parallel jobs
	TArray<FString> NodeIds;
	SyncData.RequestData.GenerateKeyArray(NodeIds);

	// Pre-generate node names to avoid local names generation
	TArray<FName> NodeNames;
	NodeNames.Reserve(NodeIds.Num());
	for(const FString& NodeId : NodeIds)
	{
		NodeNames.Add(*NodeId);
	}

	// Make sure the response map has pre-allocated all the keys so it can be const while processed in parallel
	for (const FString& NodeId : NodeIds)
	{
		SyncData.ResponseData.Emplace(NodeId);
	}

	// Per-node request parsing
	ParallelFor(
		NodesNum,
		[
			NodeIds = static_cast<const TArray<FString>&>(NodeIds),
			Request = SyncData.RequestData,
			&InputData
		](int32 JobIndex)
		{
			// Deserialize request from a single cluster node
			const TArray<uint8>& NodeRequestData = Request.FindChecked(NodeIds[JobIndex]);
			FMemoryReader DataReader(NodeRequestData);

			// Read states num
			int32 StatesNum = 0;
			DataReader << StatesNum;

			// Deserialize all the blobs
			for (int32 StateIdx = 0; StateIdx < StatesNum; ++StateIdx)
			{
				FCustomStateRequestBlob StateBlob;
				StateBlob.Serialize(DataReader);
				InputData[JobIndex].Emplace(StateBlob.StateName, MoveTemp(StateBlob));
			}
		});

	// Intermediate output data. Array of cluster nodes, each one has its own NodeId-to-StateName-to-StateData map.
	TArray<TMap<FName, FCustomStateResponseBlob>> OutputData;
	OutputData.AddDefaulted(NodesNum);

	// Per-node response generating
	ParallelFor(
		NodesNum,
		[
			NodesNum,
			NodeIds   = static_cast<const TArray<FString>&>(NodeIds),
			NodeNames = static_cast<const TArray<FName>&>(NodeNames),
			InputData = static_cast<const TArray<TMap<FName, FCustomStateRequestBlob>>&>(InputData),
			&OutputData,
			&ResponseContainer = SyncData.ResponseData
		](int32 JobIndex)
		{
			// Iterate through the states available on the node that is being processed currently (destination states)
			const TMap<FName, FCustomStateRequestBlob>& DstNode = InputData[JobIndex];
			for (const TPair<FName, FCustomStateRequestBlob>& DstState : DstNode)
			{
				// Iterate through every node to collect all the data we want to grab
				for (int32 SrcNodeIdx = 0; SrcNodeIdx < NodesNum; ++SrcNodeIdx)
				{
					// Don't send data back. So far, it's not expected that any sender's data
					// can be changed on the server side. Therefore no need to waste traffic
					// on sending this data back. But this may change in the future.
					if (SrcNodeIdx == JobIndex)
					{
						continue;
					}

					// Node ID that we want to grab data from
					const FName& SrcNodeName = NodeNames[SrcNodeIdx];

					// If destination state wants to get update for the source node
					if (!DstState.Value.bCustomUpstream || DstState.Value.UpstreamNodes.Contains(SrcNodeName))
					{
						// If the source node has registered the same state ...
						if (const FCustomStateRequestBlob* const FoundSrcBlob = InputData[SrcNodeIdx].Find(DstState.Key))
						{
							// ... and exposed it
							if (FoundSrcBlob->bExposesData)
							{
								// Then it needs to be sent to the destination
								FCustomStateResponseBlob& ResponseBlob = OutputData[JobIndex].FindOrAdd(DstState.Key);
								ResponseBlob.StateName = DstState.Key;
								ResponseBlob.NodeStates.Emplace(SrcNodeName, MakeTuple(FoundSrcBlob->StateType, FoundSrcBlob->StateData));
							}
						}
					}
				}
			}

			// Response buffer for current node (JobIndex)
			const FString NodeId = NodeIds[JobIndex];
			TArray<uint8>& ResponseBuffer = ResponseContainer.FindChecked(NodeId);
			FMemoryWriter DataWriter(ResponseBuffer);

			// Response data for current node
			TMap<FName, FCustomStateResponseBlob>& OutputMap = OutputData[JobIndex];

			// Write states num
			int32 NumStates = OutputMap.Num();
			DataWriter << NumStates;

			// Write states
			for (TPair<FName, FCustomStateResponseBlob>& SrcState : OutputMap)
			{
				SrcState.Value.Serialize(DataWriter);
			}
		});
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterSessionPacketHandler
//////////////////////////////////////////////////////////////////////////////////////////////
TSharedPtr<FDisplayClusterPacketInternal> FDisplayClusterClusterSyncService::ProcessPacket(const TSharedPtr<FDisplayClusterPacketInternal>& Request, const FDisplayClusterSessionInfo& SessionInfo)
{
	// Check the pointer
	if (!Request.IsValid())
	{
		UE_LOGF(LogDisplayClusterNetwork, Error, "%ls - Invalid request data (nullptr)", *GetName());
		return nullptr;
	}

	// Cache session info
	SetSessionInfoCache(SessionInfo);

	UE_LOGF(LogDisplayClusterNetwork, Verbose, "%ls - Processing packet: %ls", *GetName(), *Request->ToLogString());

	// Check protocol and type
	if (!Request->GetProtocol().Equals(DisplayClusterClusterSyncStrings::ProtocolName, ESearchCase::IgnoreCase) ||
		!Request->GetType().Equals(DisplayClusterClusterSyncStrings::TypeRequest, ESearchCase::IgnoreCase))
	{
		UE_LOGF(LogDisplayClusterNetwork, Error, "%ls - Unsupported packet type: %ls", *GetName(), *Request->ToLogString());
		return nullptr;
	}

	// Initialize response packet
	TSharedPtr<FDisplayClusterPacketInternal> Response = MakeShared<FDisplayClusterPacketInternal>(Request->GetName(), DisplayClusterClusterSyncStrings::TypeResponse, Request->GetProtocol());

	// Dispatch the packet
	const FString& ReqName = Request->GetName();
	if (ReqName.Equals(DisplayClusterClusterSyncStrings::WaitForGameStart::Name, ESearchCase::IgnoreCase))
	{
		const EDisplayClusterCommResult CommResult = WaitForGameStart();
		Response->SetCommResult(CommResult);

		return Response;
	}
	else if (ReqName.Equals(DisplayClusterClusterSyncStrings::WaitForFrameStart::Name, ESearchCase::IgnoreCase))
	{
		const EDisplayClusterCommResult CommResult = WaitForFrameStart();
		Response->SetCommResult(CommResult);

		return Response;
	}
	else if (ReqName.Equals(DisplayClusterClusterSyncStrings::WaitForFrameEnd::Name, ESearchCase::IgnoreCase))
	{
		const EDisplayClusterCommResult CommResult = WaitForFrameEnd();
		Response->SetCommResult(CommResult);

		return Response;
	}
	else if (ReqName.Equals(DisplayClusterClusterSyncStrings::GetTimeData::Name, ESearchCase::IgnoreCase))
	{
		double DeltaTime = 0.0f;
		double GameTime  = 0.0f;
		TOptional<FQualifiedFrameTime> FrameTime;

		const EDisplayClusterCommResult CommResult = GetTimeData(DeltaTime, GameTime, FrameTime);

		// Convert to hex strings
		const FString StrDeltaTime = DisplayClusterTypesConverter::template ToHexString<double>(DeltaTime);
		const FString StrGameTime  = DisplayClusterTypesConverter::template ToHexString<double>(GameTime);

		Response->SetTextArg(DisplayClusterClusterSyncStrings::ArgumentsDefaultCategory, DisplayClusterClusterSyncStrings::GetTimeData::ArgDeltaTime,        StrDeltaTime);
		Response->SetTextArg(DisplayClusterClusterSyncStrings::ArgumentsDefaultCategory, DisplayClusterClusterSyncStrings::GetTimeData::ArgGameTime,         StrGameTime);
		Response->SetTextArg(DisplayClusterClusterSyncStrings::ArgumentsDefaultCategory, DisplayClusterClusterSyncStrings::GetTimeData::ArgIsFrameTimeValid, FrameTime.IsSet());
		
		if (FrameTime.IsSet())
		{
			Response->SetTextArg(DisplayClusterClusterSyncStrings::ArgumentsDefaultCategory, DisplayClusterClusterSyncStrings::GetTimeData::ArgFrameTime, FrameTime.GetValue());
		}
		else
		{
			Response->RemoveTextArg(DisplayClusterClusterSyncStrings::ArgumentsDefaultCategory, DisplayClusterClusterSyncStrings::GetTimeData::ArgFrameTime);
		}

		Response->SetCommResult(CommResult);

		return Response;
	}
	else if (ReqName.Equals(DisplayClusterClusterSyncStrings::GetObjectsData::Name, ESearchCase::IgnoreCase))
	{
		uint8 SyncGroupNum = 0;
		Request->GetTextArg(DisplayClusterClusterSyncStrings::ArgumentsDefaultCategory, DisplayClusterClusterSyncStrings::GetObjectsData::ArgSyncGroup, SyncGroupNum);
		EDisplayClusterSyncGroup SyncGroup = (EDisplayClusterSyncGroup)SyncGroupNum;

		TMap<FString, TArray<uint8>> ObjectsData;
		const EDisplayClusterCommResult CommResult = GetObjectsData(SyncGroup, ObjectsData);

		Response->SetBinArgs(DisplayClusterClusterSyncStrings::ArgumentsDefaultCategory, MoveTemp(ObjectsData));

		Response->SetCommResult(CommResult);

		return Response;
	}
	else if (ReqName.Equals(DisplayClusterClusterSyncStrings::GetEventsData::Name, ESearchCase::IgnoreCase))
	{
		TArray<TSharedPtr<FDisplayClusterClusterEventJson, ESPMode::ThreadSafe>>   JsonEvents;
		TArray<TSharedPtr<FDisplayClusterClusterEventBinary, ESPMode::ThreadSafe>> BinaryEvents;
		
		const EDisplayClusterCommResult CommResult = GetEventsData(JsonEvents, BinaryEvents);

		UE::nDisplay::DisplayClusterNetworkDataConversion::JsonEventsToInternalPacket(JsonEvents, Response);
		UE::nDisplay::DisplayClusterNetworkDataConversion::BinaryEventsToInternalPacket(BinaryEvents, Response);

		Response->SetCommResult(CommResult);

		return Response;
	}
	else if (ReqName.Equals(DisplayClusterClusterSyncStrings::GetNativeInputData::Name, ESearchCase::IgnoreCase))
	{
		TMap<FString, TArray<uint8>> NativeInputData;

		const EDisplayClusterCommResult CommResult = GetNativeInputData(NativeInputData);

		Response->SetBinArgs(DisplayClusterClusterSyncStrings::ArgumentsDefaultCategory, MoveTemp(NativeInputData));

		Response->SetCommResult(CommResult);

		return Response;
	}
	else if (ReqName.Equals(DisplayClusterClusterSyncStrings::PropagateStatesData::Name, ESearchCase::IgnoreCase))
	{
		TArray<uint8> NodeStatesData;

		Request->GetBinArg(DisplayClusterClusterSyncStrings::ArgumentsDefaultCategory, DisplayClusterClusterSyncStrings::PropagateStatesData::ArgLocalStatesData, NodeStatesData);

		TArray<uint8> ClusterStatesData;
		const EDisplayClusterCommResult CommResult = PropagateStatesData(NodeStatesData, ClusterStatesData);

		Response->SetBinArg(DisplayClusterClusterSyncStrings::ArgumentsDefaultCategory, DisplayClusterClusterSyncStrings::PropagateStatesData::ArgClusterStatesData, ClusterStatesData);

		Response->SetCommResult(CommResult);

		return Response;
	}

	// Being here means that we have no appropriate dispatch logic for this packet
	UE_LOGF(LogDisplayClusterNetwork, Warning, "%ls - No dispatcher found for packet '%ls'", *GetName(), *Request->GetName());

	return nullptr;
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProtocolClusterSync
//////////////////////////////////////////////////////////////////////////////////////////////
EDisplayClusterCommResult FDisplayClusterClusterSyncService::WaitForGameStart()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SRV_CS::WaitForGameStart);

	const FString CachedNodeId = GetSessionInfoCache().NodeId.Get(FString());
	BarrierGameStart->Wait(CachedNodeId);
	return EDisplayClusterCommResult::Ok;
}

EDisplayClusterCommResult FDisplayClusterClusterSyncService::WaitForFrameStart()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SRV_CS::WaitForFrameStart);

	const FString CachedNodeId = GetSessionInfoCache().NodeId.Get(FString());
	BarrierFrameStart->Wait(CachedNodeId);
	return EDisplayClusterCommResult::Ok;
}

EDisplayClusterCommResult FDisplayClusterClusterSyncService::WaitForFrameEnd()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SRV_CS::WaitForFrameEnd);

	const FString CachedNodeId = GetSessionInfoCache().NodeId.Get(FString());
	BarrierFrameEnd->Wait(CachedNodeId);
	return EDisplayClusterCommResult::Ok;
}

EDisplayClusterCommResult FDisplayClusterClusterSyncService::GetTimeData(double& OutDeltaTime, double& OutGameTime, TOptional<FQualifiedFrameTime>& OutFrameTime)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SRV_CS::GetTimeData);

	FDisplayClusterCommRequestHandlerBase& Handler = (IsLocalRequest() ? 
		static_cast<FDisplayClusterCommRequestHandlerBase&>(FDisplayClusterCommRequestHandlerLocal::Get()) :
		static_cast<FDisplayClusterCommRequestHandlerBase&>(FDisplayClusterCommRequestHandlerRemote::Get()));

	return Handler.GetTimeData(OutDeltaTime, OutGameTime, OutFrameTime);
}

EDisplayClusterCommResult FDisplayClusterClusterSyncService::GetObjectsData(EDisplayClusterSyncGroup InSyncGroup, TMap<FString, TArray<uint8>>& OutObjectsData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SRV_CS::GetObjectsData);

	FDisplayClusterCommRequestHandlerBase& Handler = (IsLocalRequest() ?
		static_cast<FDisplayClusterCommRequestHandlerBase&>(FDisplayClusterCommRequestHandlerLocal::Get()) :
		static_cast<FDisplayClusterCommRequestHandlerBase&>(FDisplayClusterCommRequestHandlerRemote::Get()));

	return Handler.GetObjectsData(InSyncGroup, OutObjectsData);
}

EDisplayClusterCommResult FDisplayClusterClusterSyncService::GetEventsData(TArray<TSharedPtr<FDisplayClusterClusterEventJson, ESPMode::ThreadSafe>>& OutJsonEvents, TArray<TSharedPtr<FDisplayClusterClusterEventBinary, ESPMode::ThreadSafe>>& OutBinaryEvents)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SRV_CS::GetEventsData);

	FDisplayClusterCommRequestHandlerBase& Handler = (IsLocalRequest() ?
		static_cast<FDisplayClusterCommRequestHandlerBase&>(FDisplayClusterCommRequestHandlerLocal::Get()) :
		static_cast<FDisplayClusterCommRequestHandlerBase&>(FDisplayClusterCommRequestHandlerRemote::Get()));

	return Handler.GetEventsData(OutJsonEvents, OutBinaryEvents);
}

EDisplayClusterCommResult FDisplayClusterClusterSyncService::GetNativeInputData(TMap<FString, TArray<uint8>>& OutNativeInputData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SRV_CS::GetNativeInputData);

	FDisplayClusterCommRequestHandlerBase& Handler = (IsLocalRequest() ?
		static_cast<FDisplayClusterCommRequestHandlerBase&>(FDisplayClusterCommRequestHandlerLocal::Get()) :
		static_cast<FDisplayClusterCommRequestHandlerBase&>(FDisplayClusterCommRequestHandlerRemote::Get()));

	return Handler.GetNativeInputData(OutNativeInputData);
}

EDisplayClusterCommResult FDisplayClusterClusterSyncService::PropagateStatesData(const TArray<uint8>& InLocalStatesData, TArray<uint8>& OutClusterStatesData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SRV_CS::PropagateStatesData);

	const FString CachedNodeId = GetSessionInfoCache().NodeId.Get(FString());
	BarrierStatesSync->WaitWithData(CachedNodeId, InLocalStatesData, OutClusterStatesData);
	return EDisplayClusterCommResult::Ok;
}
