// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Network/Service/DisplayClusterService.h"
#include "Network/Session/IDisplayClusterSessionPacketHandler.h"
#include "Network/Protocol/IDisplayClusterProtocolClusterSync.h"
#include "Network/Packet/DisplayClusterPacketInternal.h"

class IDisplayClusterBarrier;


/**
 * Cluster synchronization TCP server
 */
class FDisplayClusterClusterSyncService
	: public    FDisplayClusterService
	, public    IDisplayClusterSessionPacketHandler<FDisplayClusterPacketInternal, true>
	, protected IDisplayClusterProtocolClusterSync
{
public:

	FDisplayClusterClusterSyncService(const FName& InInstanceName);

	virtual ~FDisplayClusterClusterSyncService();

public:

	//~ Begin IDisplayClusterServer
	virtual bool Start(const FString& Address, const uint16 Port) override;
	virtual bool Start(TSharedRef<FDisplayClusterTcpListener>& ExternalListener) override;
	virtual void Shutdown() override;
	virtual FString GetProtocolName() const override;
	virtual void KillSession(const FString& NodeId) override;
	//~ End IDisplayClusterServer

protected:

	/** Creates session instance for this service */
	virtual TSharedPtr<IDisplayClusterSession> CreateSession(FDisplayClusterSessionInfo& SessionInfo) override;

private:

	/** Performs internal initialization during server start */
	void StartInternal();

	/** Callback when a session is closed */
	void ProcessSessionClosed(const FDisplayClusterSessionInfo& SessionInfo);

	/** Callbck on barrier timeout */
	void ProcessBarrierTimeout(const FString& BarrierName, const TSet<FString>& NodesTimedOut);

	/** Non-virtual shutdown implementation */
	void ShutdownImpl();

private:

	/** Handles states synchronization. Generates and propagates data according to the propagation strategy. */
	void OnStatesSync(FDisplayClusterBarrierPreSyncEndDelegateData& SyncData);


protected:

	//~ Begin IDisplayClusterSessionPacketHandler
	virtual TSharedPtr<FDisplayClusterPacketInternal> ProcessPacket(const TSharedPtr<FDisplayClusterPacketInternal>& Request, const FDisplayClusterSessionInfo& SessionInfo) override;
	//~ End IDisplayClusterSessionPacketHandler

protected:

	//~ Begin IDisplayClusterProtocolClusterSync
	virtual EDisplayClusterCommResult WaitForGameStart() override;
	virtual EDisplayClusterCommResult WaitForFrameStart() override;
	virtual EDisplayClusterCommResult WaitForFrameEnd() override;
	virtual EDisplayClusterCommResult GetTimeData(double& OutDeltaTime, double& OutGameTime, TOptional<FQualifiedFrameTime>& OutFrameTime) override;
	virtual EDisplayClusterCommResult GetObjectsData(EDisplayClusterSyncGroup InSyncGroup, TMap<FString, TArray<uint8>>& OutObjectsData) override;
	virtual EDisplayClusterCommResult GetEventsData(TArray<TSharedPtr<FDisplayClusterClusterEventJson, ESPMode::ThreadSafe>>& OutJsonEvents, TArray<TSharedPtr<FDisplayClusterClusterEventBinary, ESPMode::ThreadSafe>>& OutBinaryEvents) override;
	virtual EDisplayClusterCommResult GetNativeInputData(TMap<FString, TArray<uint8>>& OutNativeInputData) override;
	virtual EDisplayClusterCommResult PropagateStatesData(const TArray<uint8>& InLocalStatesData, TArray<uint8>& OutClusterStatesData) override;
	//~ End IDisplayClusterProtocolClusterSync

private:

	/** Game start sync barrier */
	TUniquePtr<IDisplayClusterBarrier> BarrierGameStart;

	/** Frame start barrier */
	TUniquePtr<IDisplayClusterBarrier> BarrierFrameStart;

	/** Frame end barrier */
	TUniquePtr<IDisplayClusterBarrier> BarrierFrameEnd;

	/** Custom states sync barrier  */
	TUniquePtr<IDisplayClusterBarrier> BarrierStatesSync;

	/** Auxiliary container that keeps all the barriers */
	TMap<FString, TUniquePtr<IDisplayClusterBarrier>*> ServiceBarriers;
};
