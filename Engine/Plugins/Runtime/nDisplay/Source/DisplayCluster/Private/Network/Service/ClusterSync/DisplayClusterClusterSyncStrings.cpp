// Copyright Epic Games, Inc. All Rights Reserved.

#include "Network/Service/ClusterSync/DisplayClusterClusterSyncStrings.h"


namespace DisplayClusterClusterSyncStrings
{
	const FString ProtocolName = TEXT("ClusterSync");

	const FString TypeRequest  = TEXT("Request");
	const FString TypeResponse = TEXT("Response");

	const FString ArgumentsDefaultCategory = TEXT("CS");
	const FString ArgumentsJsonEvents      = TEXT("CS_JE");
	const FString ArgumentsBinaryEvents    = TEXT("CS_BE");

	namespace WaitForGameStart
	{
		const FString Name = TEXT("WaitForGameStart");
	}

	namespace WaitForFrameStart
	{
		const FString Name = TEXT("WaitForFrameStart");
	}

	namespace WaitForFrameEnd
	{
		const FString Name = TEXT("WaitForFrameEnd");
	}

	namespace GetTimeData
	{
		const FString Name = TEXT("GetTimeData");

		const FString ArgDeltaTime        = TEXT("DeltaTime");
		const FString ArgGameTime         = TEXT("GameTime");
		const FString ArgIsFrameTimeValid = TEXT("IsFrameTimeValid");
		const FString ArgFrameTime        = TEXT("FrameTime");
	}

	namespace GetObjectsData
	{
		const FString Name = TEXT("GetObjectsData");

		const FString ArgSyncGroup = TEXT("SyncGroup");
	}

	namespace GetEventsData
	{
		const FString Name = TEXT("GetEventsData");
	}

	namespace GetNativeInputData
	{
		const FString Name = TEXT("GetNativeInputData");

		const FString ArgNativeInputData = TEXT("NativeInputData");
	}

	namespace PropagateStatesData
	{
		const FString Name = TEXT("PropagateStatesData");

		const FString ArgLocalStatesData   = TEXT("LocalData");
		const FString ArgClusterStatesData = TEXT("ClusterData");
	}
};
