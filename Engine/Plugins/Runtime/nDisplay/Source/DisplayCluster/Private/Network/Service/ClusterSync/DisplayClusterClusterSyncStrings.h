// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"


/**
 * Cluster synchronization messages
 */
namespace DisplayClusterClusterSyncStrings
{
	extern const FString ProtocolName;

	extern const FString TypeRequest;
	extern const FString TypeResponse;

	extern const FString ArgumentsDefaultCategory;
	extern const FString ArgumentsJsonEvents;
	extern const FString ArgumentsBinaryEvents;

	namespace WaitForGameStart
	{
		extern const FString Name;
	}

	namespace WaitForFrameStart
	{
		extern const FString Name;
	}

	namespace WaitForFrameEnd
	{
		extern const FString Name;
	}

	namespace GetTimeData
	{
		extern const FString Name;

		extern const FString ArgDeltaTime;
		extern const FString ArgGameTime;
		extern const FString ArgIsFrameTimeValid;
		extern const FString ArgFrameTime;
	}

	namespace GetObjectsData
	{
		extern const FString Name;

		extern const FString ArgSyncGroup;
	}

	namespace GetEventsData
	{
		extern const FString Name;
	}

	namespace GetNativeInputData
	{
		extern const FString Name;

		extern const FString ArgNativeInputData;
	}

	namespace PropagateStatesData
	{
		extern const FString Name;

		extern const FString ArgLocalStatesData;
		extern const FString ArgClusterStatesData;
	}
};
