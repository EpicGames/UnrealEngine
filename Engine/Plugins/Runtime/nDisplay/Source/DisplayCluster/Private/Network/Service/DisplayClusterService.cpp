// Copyright Epic Games, Inc. All Rights Reserved.

#include "Network/Service/DisplayClusterService.h"

#include "Cluster/IPDisplayClusterClusterManager.h"
#include "Config/IPDisplayClusterConfigManager.h"

#include "Misc/DisplayClusterGlobals.h"
#include "Misc/DisplayClusterLog.h"

#include "Network/DisplayClusterNetworkTypes.h"

#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "HAL/IConsoleManager.h"
#include "HAL/RunnableThread.h"

#include "DisplayClusterConfigurationTypes.h"


// nDisplay service threads priority
static TAutoConsoleVariable<int32> CVarServiceThreadsPriority(
	TEXT("nDisplay.Service.ThreadsPriority"),
	3,
	TEXT("Service threads priority:\n")
	TEXT("0 : Lowest\n")
	TEXT("1 : Below normal\n")
	TEXT("2 : Slightly below normal\n")
	TEXT("3 : Normal\n")
	TEXT("4 : Above normal\n")
	TEXT("5 : Highest\n")
	TEXT("6 : Time critical\n")
	,
	ECVF_Default
);


FDisplayClusterService::FDisplayClusterService(const FString& Name)
	: FDisplayClusterServer(Name)
{
}


EThreadPriority FDisplayClusterService::ConvertThreadPriorityFromCvarValue(int ThreadPriority)
{
	switch (ThreadPriority)
	{
	case 0:
		return EThreadPriority::TPri_Lowest;
	case 1:
		return EThreadPriority::TPri_BelowNormal;
	case 2:
		return EThreadPriority::TPri_SlightlyBelowNormal;
	case 3:
		return EThreadPriority::TPri_Normal;
	case 4:
		return EThreadPriority::TPri_AboveNormal;
	case 5:
		return EThreadPriority::TPri_Highest;
	case 6:
		return EThreadPriority::TPri_TimeCritical;
	default:
		break;
	}

	return EThreadPriority::TPri_Normal;
}

EThreadPriority FDisplayClusterService::GetThreadPriority()
{
	return FDisplayClusterService::ConvertThreadPriorityFromCvarValue(CVarServiceThreadsPriority.GetValueOnAnyThread());
}

void FDisplayClusterService::SetSessionInfoCache(const FDisplayClusterSessionInfo& SessionInfo)
{
	const uint32 ThreadId = FPlatformTLS::GetCurrentThreadId();

	// Cache new data if there is no record for this thread
	{
		FScopeLock Lock(&SessionInfoCacheCS);
		if (!SessionInfoCache.Contains(ThreadId))
		{
			SessionInfoCache.Emplace(ThreadId, SessionInfo);
		}
	}
}

const FDisplayClusterSessionInfo& FDisplayClusterService::GetSessionInfoCache() const
{
	static const FDisplayClusterSessionInfo DataNotAvailableDefaultResponse;

	const FDisplayClusterSessionInfo* FoundData = nullptr;

	const uint32 ThreadId = FPlatformTLS::GetCurrentThreadId();

	{
		FScopeLock Lock(&SessionInfoCacheCS);
		FoundData = SessionInfoCache.Find(ThreadId);
	}

	return FoundData ? *FoundData : DataNotAvailableDefaultResponse;
}

void FDisplayClusterService::ClearCache()
{
	FScopeLock Lock(&SessionInfoCacheCS);
	SessionInfoCache.Reset();
}

bool FDisplayClusterService::IsLocalRequest() const
{
	static const FString ThisNodeId = GDisplayCluster->GetClusterMgr()->GetNodeId();

	// Check if this session belongs to this node. If so, it's a local request.
	const bool bLocalRequest = ThisNodeId.Equals(GetSessionInfoCache().NodeId.Get(FString()), ESearchCase::IgnoreCase);
	return bLocalRequest;
}
