// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Network/DisplayClusterServer.h"

#include "GenericPlatform/GenericPlatformAffinity.h"

#include "Misc/Optional.h"

#include "Network/DisplayClusterNetworkTypes.h"
#include "Network/Barrier/IDisplayClusterBarrier.h"


class FSocket;
struct FIPv4Endpoint;
struct FDisplayClusterSessionInfo;


/**
 * A container to keep all the information about any potential networking failures
 */
struct FDisplayClusterServiceFailureEvent
{
public:

	/** Cluster node lost */
	enum class ENodeFailType : uint8
	{
		Unknown,
		BarrierTimeOut,
		ConnectionLost,
	};

public:

	/** The ID of the cluster node that failed */
	TOptional<FString> NodeFailed;

	/** Failure type */
	ENodeFailType FailureType = ENodeFailType::Unknown;
};


/**
 * Abstract DisplayCluster service
 */
class FDisplayClusterService
	: public FDisplayClusterServer

{
public:

	FDisplayClusterService(const FString& Name);

public:

	/** A helper function to convert a cvar integer into the thread priority EThreadPriority enum value */
	static EThreadPriority ConvertThreadPriorityFromCvarValue(int ThreadPriority);

	/** Returns thread priority that is currently set by the corresponding CVar */
	static EThreadPriority GetThreadPriority();

public:

	/** Networking failure notification event */
	DECLARE_EVENT_OneParam(FDisplayClusterService, FNodeFailedEvent, const FDisplayClusterServiceFailureEvent&);
	FNodeFailedEvent& OnNodeFailed()
	{
		return NodeFailedEvent;
	}

protected:

	/** Cache session info data (calling thread) if needed for child service implementations */
	void SetSessionInfoCache(const FDisplayClusterSessionInfo& SessionInfo);

	/** Returns session info of the calling thread */
	const FDisplayClusterSessionInfo& GetSessionInfoCache() const;

	/** Resets whole session info cache */
	void ClearCache();

	/** Checks if an incoming request is local (was sent by this node) */
	bool IsLocalRequest() const;

private:

	/** Session info cache */
	TMap<uint32, FDisplayClusterSessionInfo> SessionInfoCache;

	/** A critical section to safely accesss the session info cache */
	mutable FCriticalSection SessionInfoCacheCS;

private:

	/** Failure reporting event */
	FNodeFailedEvent NodeFailedEvent;
};
