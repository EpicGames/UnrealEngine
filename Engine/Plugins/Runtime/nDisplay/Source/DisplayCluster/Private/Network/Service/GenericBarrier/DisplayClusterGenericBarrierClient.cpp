// Copyright Epic Games, Inc. All Rights Reserved.

#include "Network/Service/GenericBarrier/DisplayClusterGenericBarrierClient.h"

#include "Cluster/IPDisplayClusterClusterManager.h"
#include "Cluster/Controller/IDisplayClusterClusterNodeController.h"

#include "Config/IPDisplayClusterConfigManager.h"

#include "Misc/DisplayClusterGlobals.h"
#include "Misc/DisplayClusterHelpers.h"
#include "Misc/DisplayClusterLog.h"
#include "Misc/DisplayClusterStrings.h"
#include "Misc/DisplayClusterTypesConverter.h"

#include "Network/Barrier/IDisplayClusterBarrier.h"
#include "Network/Configuration/DisplayClusterNetworkConfiguration.h"
#include "Network/Service/GenericBarrier/DisplayClusterGenericBarrierService.h"
#include "Network/Service/GenericBarrier/DisplayClusterGenericBarrierStrings.h"
#include "Network/Packet/DisplayClusterPacketInternal.h"
#include "Network/Listener/DisplayClusterHelloMessageStrings.h"

#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterEnums.h"


FDisplayClusterGenericBarrierClient::FDisplayClusterGenericBarrierClient(const FName& InName)
	: FDisplayClusterClient(InName.ToString())
{
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterClient
//////////////////////////////////////////////////////////////////////////////////////////////
bool FDisplayClusterGenericBarrierClient::Connect(const FString& Address, const uint16 Port, const uint32 ConnectRetriesAmount, const uint32 ConnectRetryDelay)
{
	// First, allow base class to perform connection
	if (!FDisplayClusterClient::Connect(Address, Port, ConnectRetriesAmount, ConnectRetryDelay))
	{
		return false;
	}

	// Prepare 'hello' message
	TSharedPtr<FDisplayClusterPacketInternal> HelloMsg = MakeShared<FDisplayClusterPacketInternal>(
		DisplayClusterHelloMessageStrings::Hello::Name,
		DisplayClusterHelloMessageStrings::Hello::TypeRequest,
		DisplayClusterGenericBarrierStrings::ProtocolName
	);

	// Fill in the message with data
	const FString NodeId = GDisplayCluster->GetPrivateClusterMgr()->GetNodeId();
	HelloMsg->SetTextArg(DisplayClusterHelloMessageStrings::ArgumentsDefaultCategory, DisplayClusterHelloMessageStrings::Hello::ArgNodeId, NodeId);

	// Send message (no response awaiting)
	return SendPacket(HelloMsg);
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProtocolGenericBarrier
//////////////////////////////////////////////////////////////////////////////////////////////
EDisplayClusterCommResult FDisplayClusterGenericBarrierClient::CreateBarrier(const FString& BarrierId, const TMap<FString, TSet<FString>>& NodeToSyncCallers, uint32 Timeout, EBarrierControlResult& Result)
{
	const TSharedPtr<FDisplayClusterPacketInternal> Request = MakeShared<FDisplayClusterPacketInternal>(
		DisplayClusterGenericBarrierStrings::CreateBarrier::Name,
		DisplayClusterGenericBarrierStrings::TypeRequest,
		DisplayClusterGenericBarrierStrings::ProtocolName);

	// Request arguments
	Request->SetTextArg(DisplayClusterGenericBarrierStrings::ArgumentsDefaultCategory, DisplayClusterGenericBarrierStrings::ArgBarrierId, BarrierId);
	Request->SetTextArg(DisplayClusterGenericBarrierStrings::ArgumentsDefaultCategory, DisplayClusterGenericBarrierStrings::CreateBarrier::ArgTimeout, Timeout);

	{
		TArray<uint8> NodeToSyncCallersData;
		FMemoryWriter MemoryWriter(NodeToSyncCallersData);
		MemoryWriter << const_cast<TMap<FString, TSet<FString>>&>(NodeToSyncCallers);
		Request->SetBinArg(DisplayClusterGenericBarrierStrings::ArgumentsDefaultCategory, DisplayClusterGenericBarrierStrings::CreateBarrier::ArgCallers, NodeToSyncCallersData);
	}

	TSharedPtr<FDisplayClusterPacketInternal> Response;

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CLN_GB::CreateBarrier);
		Response = SendRecvPacket(Request);
	}

	if (!Response)
	{
		UE_LOGF(LogDisplayClusterNetwork, Warning, "Network error on '%ls'", *Request->GetName());
		return EDisplayClusterCommResult::NetworkError;
	}

	// Extract response data
	FString CtrlResult;
	Response->GetTextArg(DisplayClusterGenericBarrierStrings::ArgumentsDefaultCategory, DisplayClusterGenericBarrierStrings::ArgResult, CtrlResult);
	Result = static_cast<EBarrierControlResult>(DisplayClusterTypesConverter::template FromString<uint8>(CtrlResult));

	return Response->GetCommResult();
}

EDisplayClusterCommResult FDisplayClusterGenericBarrierClient::WaitUntilBarrierIsCreated(const FString& BarrierId, EBarrierControlResult& Result)
{
	const TSharedPtr<FDisplayClusterPacketInternal> Request = MakeShared<FDisplayClusterPacketInternal>(
		DisplayClusterGenericBarrierStrings::WaitUntilBarrierIsCreated::Name,
		DisplayClusterGenericBarrierStrings::TypeRequest,
		DisplayClusterGenericBarrierStrings::ProtocolName);

	// Request arguments
	Request->SetTextArg(DisplayClusterGenericBarrierStrings::ArgumentsDefaultCategory, DisplayClusterGenericBarrierStrings::ArgBarrierId, BarrierId);

	TSharedPtr<FDisplayClusterPacketInternal> Response;

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CLN_GB::WaitUntilBarrierIsCreated);
		Response = SendRecvPacket(Request);
	}

	if (!Response)
	{
		UE_LOGF(LogDisplayClusterNetwork, Warning, "Network error on '%ls'", *Request->GetName());
		return EDisplayClusterCommResult::NetworkError;
	}

	// Extract response data
	FString CtrlResult;
	Response->GetTextArg(DisplayClusterGenericBarrierStrings::ArgumentsDefaultCategory, DisplayClusterGenericBarrierStrings::ArgResult, CtrlResult);
	Result = static_cast<EBarrierControlResult>(DisplayClusterTypesConverter::template FromString<uint8>(CtrlResult));

	return Response->GetCommResult();
}

EDisplayClusterCommResult FDisplayClusterGenericBarrierClient::IsBarrierAvailable(const FString& BarrierId, EBarrierControlResult& Result)
{
	const TSharedPtr<FDisplayClusterPacketInternal> Request = MakeShared<FDisplayClusterPacketInternal>(
		DisplayClusterGenericBarrierStrings::IsBarrierAvailable::Name,
		DisplayClusterGenericBarrierStrings::TypeRequest,
		DisplayClusterGenericBarrierStrings::ProtocolName);

	// Request arguments
	Request->SetTextArg(DisplayClusterGenericBarrierStrings::ArgumentsDefaultCategory, DisplayClusterGenericBarrierStrings::ArgBarrierId, BarrierId);

	TSharedPtr<FDisplayClusterPacketInternal> Response;

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CLN_GB::IsBarrierAvailable);
		Response = SendRecvPacket(Request);
	}

	if (!Response)
	{
		UE_LOGF(LogDisplayClusterNetwork, Warning, "Network error on '%ls'", *Request->GetName());
		return EDisplayClusterCommResult::NetworkError;
	}

	// Extract response data
	FString CtrlResult;
	Response->GetTextArg(DisplayClusterGenericBarrierStrings::ArgumentsDefaultCategory, DisplayClusterGenericBarrierStrings::ArgResult, CtrlResult);
	Result = static_cast<EBarrierControlResult>(DisplayClusterTypesConverter::template FromString<uint8>(CtrlResult));

	return Response->GetCommResult();
}

EDisplayClusterCommResult FDisplayClusterGenericBarrierClient::ReleaseBarrier(const FString& BarrierId, EBarrierControlResult& Result)
{
	const TSharedPtr<FDisplayClusterPacketInternal> Request = MakeShared<FDisplayClusterPacketInternal>(
		DisplayClusterGenericBarrierStrings::ReleaseBarrier::Name,
		DisplayClusterGenericBarrierStrings::TypeRequest,
		DisplayClusterGenericBarrierStrings::ProtocolName);

	// Request arguments
	Request->SetTextArg(DisplayClusterGenericBarrierStrings::ArgumentsDefaultCategory, DisplayClusterGenericBarrierStrings::ArgBarrierId, BarrierId);

	TSharedPtr<FDisplayClusterPacketInternal> Response;

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CLN_GB::ReleaseBarrier);
		Response = SendRecvPacket(Request);
	}

	if (!Response)
	{
		UE_LOGF(LogDisplayClusterNetwork, Warning, "Network error on '%ls'", *Request->GetName());
		return EDisplayClusterCommResult::NetworkError;
	}

	// Extract response data
	FString CtrlResult;
	Response->GetTextArg(DisplayClusterGenericBarrierStrings::ArgumentsDefaultCategory, DisplayClusterGenericBarrierStrings::ArgResult, CtrlResult);
	Result = static_cast<EBarrierControlResult>(DisplayClusterTypesConverter::template FromString<uint8>(CtrlResult));

	return Response->GetCommResult();
}

EDisplayClusterCommResult FDisplayClusterGenericBarrierClient::SyncOnBarrier(const FString& BarrierId, const FString& CallerId, EBarrierControlResult& Result)
{
	const TSharedPtr<FDisplayClusterPacketInternal> Request = MakeShared<FDisplayClusterPacketInternal>(
		DisplayClusterGenericBarrierStrings::SyncOnBarrier::Name,
		DisplayClusterGenericBarrierStrings::TypeRequest,
		DisplayClusterGenericBarrierStrings::ProtocolName);

	// Request arguments
	Request->SetTextArg(DisplayClusterGenericBarrierStrings::ArgumentsDefaultCategory, DisplayClusterGenericBarrierStrings::ArgBarrierId, BarrierId);
	Request->SetTextArg(DisplayClusterGenericBarrierStrings::ArgumentsDefaultCategory, DisplayClusterGenericBarrierStrings::SyncOnBarrier::ArgCallerId, CallerId);

	TSharedPtr<FDisplayClusterPacketInternal> Response;

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CLN_GB::SyncOnBarrier);
		Response = SendRecvPacket(Request);
	}

	if (!Response)
	{
		UE_LOGF(LogDisplayClusterNetwork, Warning, "Network error on '%ls'", *Request->GetName());
		return EDisplayClusterCommResult::NetworkError;
	}

	// Extract response data
	FString CtrlResult;
	Response->GetTextArg(DisplayClusterGenericBarrierStrings::ArgumentsDefaultCategory, DisplayClusterGenericBarrierStrings::ArgResult, CtrlResult);
	Result = static_cast<EBarrierControlResult>(DisplayClusterTypesConverter::template FromString<uint8>(CtrlResult));

	return Response->GetCommResult();
}

EDisplayClusterCommResult FDisplayClusterGenericBarrierClient::SyncOnBarrierWithData(const FString& BarrierId, const FString& CallerId, const TArray<uint8>& RequestData, TArray<uint8>& OutResponseData, EBarrierControlResult& Result)
{
	const TSharedPtr<FDisplayClusterPacketInternal> Request = MakeShared<FDisplayClusterPacketInternal>(
		DisplayClusterGenericBarrierStrings::SyncOnBarrierWithData::Name,
		DisplayClusterGenericBarrierStrings::TypeRequest,
		DisplayClusterGenericBarrierStrings::ProtocolName);

	// Request arguments
	Request->SetTextArg(DisplayClusterGenericBarrierStrings::ArgumentsDefaultCategory, DisplayClusterGenericBarrierStrings::ArgBarrierId, BarrierId);
	Request->SetTextArg(DisplayClusterGenericBarrierStrings::ArgumentsDefaultCategory, DisplayClusterGenericBarrierStrings::SyncOnBarrierWithData::ArgCallerId, CallerId);
	Request->SetBinArg(DisplayClusterGenericBarrierStrings::ArgumentsDefaultCategory, DisplayClusterGenericBarrierStrings::SyncOnBarrierWithData::ArgRequestData, RequestData);

	TSharedPtr<FDisplayClusterPacketInternal> Response;

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CLN_GB::SyncOnBarrierWithData);
		Response = SendRecvPacket(Request);
	}

	if (!Response)
	{
		UE_LOGF(LogDisplayClusterNetwork, Warning, "Network error on '%ls'", *Request->GetName());
		return EDisplayClusterCommResult::NetworkError;
	}

	// Extract response data
	Response->GetBinArg(DisplayClusterGenericBarrierStrings::ArgumentsDefaultCategory, DisplayClusterGenericBarrierStrings::SyncOnBarrierWithData::ArgResponseData, OutResponseData);

	FString CtrlResult;
	Response->GetTextArg(DisplayClusterGenericBarrierStrings::ArgumentsDefaultCategory, DisplayClusterGenericBarrierStrings::ArgResult, CtrlResult);
	Result = static_cast<EBarrierControlResult>(DisplayClusterTypesConverter::template FromString<uint8>(CtrlResult));

	return Response->GetCommResult();
}
