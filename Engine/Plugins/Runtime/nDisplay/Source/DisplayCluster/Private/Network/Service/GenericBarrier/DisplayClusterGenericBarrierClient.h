// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Network/DisplayClusterClient.h"
#include "Network/Packet/DisplayClusterPacketInternal.h"
#include "Network/Protocol/IDisplayClusterProtocolGenericBarrier.h"

class FDisplayClusterGenericBarrierService;
struct FDisplayClusterBarrierPreSyncEndDelegateData;


/**
 * Generic barriers TCP client
 */
class FDisplayClusterGenericBarrierClient
	: public FDisplayClusterClient<FDisplayClusterPacketInternal>
	, public IDisplayClusterProtocolGenericBarrier
{
public:

	FDisplayClusterGenericBarrierClient(const FName& InName);

public:

	//~ Begin IDisplayClusterClient
	virtual bool Connect(const FString& Address, const uint16 Port, const uint32 ConnectRetriesAmount = 0, const uint32 ConnectRetryDelay = 0) override;
	//~ End IDisplayClusterClient

public:

	//~ Begin IDisplayClusterProtocolGenericBarrier
	virtual EDisplayClusterCommResult CreateBarrier(const FString& BarrierId, const TMap<FString, TSet<FString>>& NodeToSyncCallers, uint32 Timeout, EBarrierControlResult& Result) override;
	virtual EDisplayClusterCommResult WaitUntilBarrierIsCreated(const FString& BarrierId, EBarrierControlResult& Result) override;
	virtual EDisplayClusterCommResult IsBarrierAvailable(const FString& BarrierId, EBarrierControlResult& Result) override;
	virtual EDisplayClusterCommResult ReleaseBarrier(const FString& BarrierId, EBarrierControlResult& Result) override;
	virtual EDisplayClusterCommResult SyncOnBarrier(const FString& BarrierId, const FString& CallerId, EBarrierControlResult& Result) override;
	virtual EDisplayClusterCommResult SyncOnBarrierWithData(const FString& BarrierId, const FString& CallerId, const TArray<uint8>& RequestData, TArray<uint8>& OutResponseData, EBarrierControlResult& Result) override;
	//~ End IDisplayClusterProtocolGenericBarrier
};
