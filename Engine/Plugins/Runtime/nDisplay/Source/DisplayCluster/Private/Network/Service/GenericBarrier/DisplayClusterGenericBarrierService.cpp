// Copyright Epic Games, Inc. All Rights Reserved.

#include "Network/Service/GenericBarrier/DisplayClusterGenericBarrierService.h"
#include "Network/Service/GenericBarrier/DisplayClusterGenericBarrierStrings.h"
#include "Network/Barrier/DisplayClusterBarrierFactory.h"
#include "Network/Barrier/IDisplayClusterBarrier.h"
#include "Network/Configuration/DisplayClusterNetworkConfiguration.h"
#include "Network/Packet/DisplayClusterPacketInternal.h"
#include "Network/Session/DisplayClusterSession.h"

#include "Misc/DisplayClusterHelpers.h"
#include "Misc/DisplayClusterLog.h"
#include "Misc/DisplayClusterStrings.h"
#include "Misc/DisplayClusterTypesConverter.h"

#include "HAL/Event.h"

#include "Serialization/MemoryReader.h"


FDisplayClusterGenericBarrierService::FDisplayClusterGenericBarrierService(const FName& InInstanceName)
	: FDisplayClusterService(InInstanceName.ToString())
{
	// Subscribe for SessionClosed events
	OnSessionClosed().AddRaw(this, &FDisplayClusterGenericBarrierService::ProcessSessionClosed);
}

FDisplayClusterGenericBarrierService::~FDisplayClusterGenericBarrierService()
{
	ShutdownImpl();
}


void FDisplayClusterGenericBarrierService::Shutdown()
{
	ShutdownImpl();
	return FDisplayClusterServer::Shutdown();
}

FString FDisplayClusterGenericBarrierService::GetProtocolName() const
{
	return DisplayClusterGenericBarrierStrings::ProtocolName;
}

void FDisplayClusterGenericBarrierService::KillSession(const FString& NodeId)
{
	// Unbind its threads from the barriers
	UnregisterClusterNode(NodeId);

	// Now do the session related job
	FDisplayClusterServer::KillSession(NodeId);
}

TSharedPtr<IDisplayClusterBarrier> FDisplayClusterGenericBarrierService::GetBarrier(const FString& BarrierId)
{
	FScopeLock Lock(&BarriersCS);
	TSharedPtr<IDisplayClusterBarrier>* BarrierFound = Barriers.Find(BarrierId);
	return BarrierFound ? *BarrierFound : nullptr;
}

TSharedPtr<const FDisplayClusterGenericBarrierService::FBarrierInfo> FDisplayClusterGenericBarrierService::GetBarrierInfo(const FString& BarrierId) const
{
	FScopeLock Lock(&BarriersInfoCS);

	const TSharedRef<FBarrierInfo>* BarrierInfo = BarriersInfo.Find(BarrierId);
	return BarrierInfo ? BarrierInfo->ToSharedPtr() : nullptr;
}

TSharedPtr<IDisplayClusterSession> FDisplayClusterGenericBarrierService::CreateSession(FDisplayClusterSessionInfo& SessionInfo)
{
	SessionInfo.SessionName = FString::Printf(TEXT("%s_%llu_%s_%s"),
		*GetName(),
		SessionInfo.SessionId,
		*SessionInfo.Endpoint.ToString(),
		*SessionInfo.NodeId.Get(TEXT("(na)"))
	);

	return MakeShared<FDisplayClusterSession<FDisplayClusterPacketInternal, true>>(SessionInfo, *this, *this, FDisplayClusterService::GetThreadPriority());
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterSessionPacketHandler
//////////////////////////////////////////////////////////////////////////////////////////////
TSharedPtr<FDisplayClusterPacketInternal> FDisplayClusterGenericBarrierService::ProcessPacket(const TSharedPtr<FDisplayClusterPacketInternal>& Request, const FDisplayClusterSessionInfo& SessionInfo)
{
	// Check the pointer
	if (!Request.IsValid())
	{
		UE_LOGF(LogDisplayClusterNetwork, Error, "%ls - Invalid request data (nullptr)", *GetName());
		return nullptr;
	}

	// Cache session info
	SetSessionInfoCache(SessionInfo);

	UE_LOGF(LogDisplayClusterNetwork, Verbose, "%ls - Processing packet: %ls", *GetName(), *Request->ToLogString());

	// Check protocol and type
	if (!Request->GetProtocol().Equals(DisplayClusterGenericBarrierStrings::ProtocolName, ESearchCase::IgnoreCase) ||
		!Request->GetType().Equals(DisplayClusterGenericBarrierStrings::TypeRequest, ESearchCase::IgnoreCase))
	{
		UE_LOGF(LogDisplayClusterNetwork, Error, "%ls - Unsupported packet type: %ls", *GetName(), *Request->ToLogString());
		return nullptr;
	}

	// Initialize response packet
	TSharedPtr<FDisplayClusterPacketInternal> Response = MakeShared<FDisplayClusterPacketInternal>(Request->GetName(), DisplayClusterGenericBarrierStrings::TypeResponse, Request->GetProtocol());

	// Dispatch the packet
	if (Request->GetName().Equals(DisplayClusterGenericBarrierStrings::SyncOnBarrier::Name, ESearchCase::IgnoreCase))
	{
		// Extract parameter: barrier ID
		FString BarrierId;
		Request->GetTextArg(DisplayClusterGenericBarrierStrings::ArgumentsDefaultCategory, DisplayClusterGenericBarrierStrings::ArgBarrierId, BarrierId);

		// Extract parameter: caller ID
		FString CallerId;
		Request->GetTextArg(DisplayClusterGenericBarrierStrings::ArgumentsDefaultCategory, DisplayClusterGenericBarrierStrings::SyncOnBarrier::ArgCallerId, CallerId);

		// Process command
		EBarrierControlResult CtrlResult = EBarrierControlResult::UnknownError;
		const EDisplayClusterCommResult Result = SyncOnBarrier(BarrierId, CallerId, CtrlResult);

		// Set response data
		Response->SetTextArg(DisplayClusterGenericBarrierStrings::ArgumentsDefaultCategory, DisplayClusterGenericBarrierStrings::ArgResult, DisplayClusterTypesConverter::template ToString((uint8)CtrlResult));
		Response->SetCommResult(Result);

		return Response;
	}
	else if (Request->GetName().Equals(DisplayClusterGenericBarrierStrings::SyncOnBarrierWithData::Name, ESearchCase::IgnoreCase))
	{
		// Extract parameter: barrier ID
		FString BarrierId;
		Request->GetTextArg(DisplayClusterGenericBarrierStrings::ArgumentsDefaultCategory, DisplayClusterGenericBarrierStrings::ArgBarrierId, BarrierId);

		// Extract parameter: caller ID
		FString CallerId;
		Request->GetTextArg(DisplayClusterGenericBarrierStrings::ArgumentsDefaultCategory, DisplayClusterGenericBarrierStrings::SyncOnBarrierWithData::ArgCallerId, CallerId);

		// Extract parameter: request data
		TArray<uint8> RequestData;
		Request->GetBinArg(DisplayClusterGenericBarrierStrings::ArgumentsDefaultCategory, DisplayClusterGenericBarrierStrings::SyncOnBarrierWithData::ArgRequestData, RequestData);

		// Process command
		EBarrierControlResult CtrlResult = EBarrierControlResult::UnknownError;
		TArray<uint8> ResponseData;
		const EDisplayClusterCommResult Result = SyncOnBarrierWithData(BarrierId, CallerId, RequestData, ResponseData, CtrlResult);

		// Set response data
		Response->SetBinArg(DisplayClusterGenericBarrierStrings::ArgumentsDefaultCategory, DisplayClusterGenericBarrierStrings::SyncOnBarrierWithData::ArgResponseData, ResponseData);
		Response->SetTextArg(DisplayClusterGenericBarrierStrings::ArgumentsDefaultCategory, DisplayClusterGenericBarrierStrings::ArgResult, DisplayClusterTypesConverter::template ToString((uint8)CtrlResult));
		Response->SetCommResult(Result);

		return Response;
	}
	else if (Request->GetName().Equals(DisplayClusterGenericBarrierStrings::CreateBarrier::Name, ESearchCase::IgnoreCase))
	{
		// Extract parameter: barrier ID
		FString BarrierId;
		Request->GetTextArg(DisplayClusterGenericBarrierStrings::ArgumentsDefaultCategory, DisplayClusterGenericBarrierStrings::ArgBarrierId, BarrierId);

		// Extract parameter: timeout
		uint32 Timeout = 0;
		Request->GetTextArg(DisplayClusterGenericBarrierStrings::ArgumentsDefaultCategory, DisplayClusterGenericBarrierStrings::CreateBarrier::ArgTimeout, Timeout);

		// Extract parameter: sync callers
		TMap<FString, TSet<FString>> NodeToSyncCallers;
		{
			TArray<uint8> NodeToSyncCallersData;
			Request->GetBinArg(DisplayClusterGenericBarrierStrings::ArgumentsDefaultCategory, DisplayClusterGenericBarrierStrings::CreateBarrier::ArgCallers, NodeToSyncCallersData);

			FMemoryReader MemoryReader(NodeToSyncCallersData);
			MemoryReader << NodeToSyncCallers;
		}

		// Process command
		EBarrierControlResult CtrlResult = EBarrierControlResult::UnknownError;
		const EDisplayClusterCommResult Result = CreateBarrier(BarrierId, NodeToSyncCallers, Timeout, CtrlResult);

		// Set response data
		Response->SetTextArg(DisplayClusterGenericBarrierStrings::ArgumentsDefaultCategory, DisplayClusterGenericBarrierStrings::ArgResult, DisplayClusterTypesConverter::template ToString((uint8)CtrlResult));
		Response->SetCommResult(Result);

		return Response;
	}
	else if (Request->GetName().Equals(DisplayClusterGenericBarrierStrings::WaitUntilBarrierIsCreated::Name, ESearchCase::IgnoreCase))
	{
		// Extract parameter: barrier ID
		FString BarrierId;
		Request->GetTextArg(DisplayClusterGenericBarrierStrings::ArgumentsDefaultCategory, DisplayClusterGenericBarrierStrings::ArgBarrierId, BarrierId);

		// Process command
		EBarrierControlResult CtrlResult = EBarrierControlResult::UnknownError;
		const EDisplayClusterCommResult Result = WaitUntilBarrierIsCreated(BarrierId, CtrlResult);

		// Set response data
		Response->SetTextArg(DisplayClusterGenericBarrierStrings::ArgumentsDefaultCategory, DisplayClusterGenericBarrierStrings::ArgResult, DisplayClusterTypesConverter::template ToString((uint8)CtrlResult));
		Response->SetCommResult(Result);

		return Response;
	}
	else if (Request->GetName().Equals(DisplayClusterGenericBarrierStrings::ReleaseBarrier::Name, ESearchCase::IgnoreCase))
	{
		// Extract parameter: barrier ID
		FString BarrierId;
		Request->GetTextArg(DisplayClusterGenericBarrierStrings::ArgumentsDefaultCategory, DisplayClusterGenericBarrierStrings::ArgBarrierId, BarrierId);

		// Process command
		EBarrierControlResult CtrlResult = EBarrierControlResult::UnknownError;
		const EDisplayClusterCommResult Result = ReleaseBarrier(BarrierId, CtrlResult);

		// Set response data
		Response->SetTextArg(DisplayClusterGenericBarrierStrings::ArgumentsDefaultCategory, DisplayClusterGenericBarrierStrings::ArgResult, DisplayClusterTypesConverter::template ToString((uint8)CtrlResult));
		Response->SetCommResult(Result);

		return Response;
	}
	else if (Request->GetName().Equals(DisplayClusterGenericBarrierStrings::IsBarrierAvailable::Name, ESearchCase::IgnoreCase))
	{
		// Extract parameter: barrier ID
		FString BarrierId;
		Request->GetTextArg(DisplayClusterGenericBarrierStrings::ArgumentsDefaultCategory, DisplayClusterGenericBarrierStrings::ArgBarrierId, BarrierId);

		// Process command
		EBarrierControlResult CtrlResult = EBarrierControlResult::UnknownError;
		const EDisplayClusterCommResult Result = IsBarrierAvailable(BarrierId, CtrlResult);

		// Set response data
		Response->SetTextArg(DisplayClusterGenericBarrierStrings::ArgumentsDefaultCategory, DisplayClusterGenericBarrierStrings::ArgResult, DisplayClusterTypesConverter::template ToString((uint8)CtrlResult));
		Response->SetCommResult(Result);

		return Response;
	}

	// Being here means that we have no appropriate dispatch logic for this packet
	UE_LOGF(LogDisplayClusterNetwork, Warning, "%ls - No dispatcher found for packet '%ls'", *GetName(), *Request->GetName());

	return nullptr;
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProtocolGenericBarrier
//////////////////////////////////////////////////////////////////////////////////////////////
EDisplayClusterCommResult FDisplayClusterGenericBarrierService::CreateBarrier(const FString& BarrierId, const TMap<FString, TSet<FString>>& NodeToSyncCallers, uint32 Timeout, EBarrierControlResult& Result)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SRV_GB::CreateBarrier);

	// Validate input data
	if (BarrierId.IsEmpty() || NodeToSyncCallers.Num() < 1 || Timeout == 0)
	{
		UE_LOGF(LogDisplayClusterBarrierGP, Warning, "%ls - CreateBarrier - invalid request data", *GetName());
		return EDisplayClusterCommResult::WrongRequestData;
	}

	{
		FScopeLock Lock(&BarriersCS);

		// Check if exists
		if (Barriers.Contains(BarrierId))
		{
			UE_LOGF(LogDisplayClusterBarrierGP, Verbose, "%ls - CreateBarrier - Barrier '%ls' already exists", *GetName(), *BarrierId);
			Result = EBarrierControlResult::AlreadyExists;
			return EDisplayClusterCommResult::Ok;
		}

		// Make full set of callers
		TSet<FString> AllSyncCallers;
		for (const TPair<FString, TSet<FString>>& PerNodeCallers : NodeToSyncCallers)
		{
			AllSyncCallers.Append(PerNodeCallers.Value);
		}

		// Create a new one
		TSharedPtr<IDisplayClusterBarrier, ESPMode::ThreadSafe> NewBarrier(FDisplayClusterBarrierFactory::CreateBarrier(BarrierId, AllSyncCallers, Timeout));
		checkSlow(NewBarrier);

		// Set up an info container for this new barrier
		InitializeBarrierInfo(BarrierId, NodeToSyncCallers);

		// Activate barrier
		NewBarrier->Activate();

		// Store it
		Barriers.Emplace(BarrierId, MoveTemp(NewBarrier));

		// Notify listeners if there are any
		if (FEvent** BarrierCreatedEvent = BarrierCreationEvents.Find(BarrierId))
		{
			(*BarrierCreatedEvent)->Trigger();
		}

		UE_LOGF(LogDisplayClusterBarrierGP, Verbose, "%ls - CreateBarrier - Barrier '%ls' created successfully", *GetName(), *BarrierId);

		Result = EBarrierControlResult::CreatedSuccessfully;
	}

	return EDisplayClusterCommResult::Ok;
}

EDisplayClusterCommResult FDisplayClusterGenericBarrierService::WaitUntilBarrierIsCreated(const FString& BarrierId, EBarrierControlResult& Result)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SRV_GB::WaitUntilBarrierIsCreated);

	// Validate input data
	if (BarrierId.IsEmpty())
	{
		UE_LOGF(LogDisplayClusterBarrierGP, Warning, "%ls - WaitUntilBarrierIsCreated - invalid request data", *GetName());
		return EDisplayClusterCommResult::WrongRequestData;
	}

	FEvent** BarrierAvailableEvent = nullptr;

	{
		FScopeLock Lock(&BarriersCS);

		// Check if the barrier exists already
		if (Barriers.Contains(BarrierId))
		{
			UE_LOGF(LogDisplayClusterBarrierGP, Verbose, "%ls - WaitUntilBarrierIsCreated - Barrier '%ls' already exists", *GetName(), *BarrierId);
			Result = EBarrierControlResult::AlreadyExists;
			return EDisplayClusterCommResult::Ok;
		}
		// If no, set up a notification event
		else
		{
			UE_LOGF(LogDisplayClusterBarrierGP, Verbose, "%ls - WaitUntilBarrierIsCreated - Barrier '%ls' not found. Waiting until it's created.", *GetName(), *BarrierId);

			// Check if notification event has been created previously
			BarrierAvailableEvent = BarrierCreationEvents.Find(BarrierId);
			if (!BarrierAvailableEvent)
			{
				// It hasn't, so create it
				BarrierAvailableEvent = &BarrierCreationEvents.Emplace(BarrierId, FPlatformProcess::GetSynchEventFromPool(true));
			}
		}
	}

	// So the barrier has not been created yet, we need to wait

	// Make sure the event is valid
	if (!BarrierAvailableEvent)
	{
		Result = EBarrierControlResult::UnknownError;
		return EDisplayClusterCommResult::InternalError;
	}

	// Wait until created
	(*BarrierAvailableEvent)->Wait();

	UE_LOGF(LogDisplayClusterBarrierGP, Verbose, "%ls - WaitUntilBarrierIsCreated - Barrier '%ls' already exists", *GetName(), *BarrierId);

	Result = EBarrierControlResult::AlreadyExists;
	return EDisplayClusterCommResult::Ok;
}

EDisplayClusterCommResult FDisplayClusterGenericBarrierService::IsBarrierAvailable(const FString& BarrierId, EBarrierControlResult& Result)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SRV_GB::IsBarrierAvailable);

	// Validate input data
	if (BarrierId.IsEmpty())
	{
		UE_LOGF(LogDisplayClusterBarrierGP, Warning, "%ls - IsBarrierAvailable - invalid request data", *GetName());
		return EDisplayClusterCommResult::WrongRequestData;
	}

	{
		FScopeLock Lock(&BarriersCS);
		Result = (Barriers.Contains(BarrierId) ? EBarrierControlResult::AlreadyExists : EBarrierControlResult::NotFound);
		UE_LOGF(LogDisplayClusterBarrierGP, Verbose, "%ls - IsBarrierAvailable - Barrier '%ls' is %ls", *GetName(), *BarrierId, Barriers.Contains(BarrierId) ? TEXT("available") : TEXT("not available"));
	}

	return EDisplayClusterCommResult::Ok;
}

EDisplayClusterCommResult FDisplayClusterGenericBarrierService::ReleaseBarrier(const FString& BarrierId, EBarrierControlResult& Result)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SRV_GB::ReleaseBarrier);

	// Validate input data
	if (BarrierId.IsEmpty())
	{
		UE_LOGF(LogDisplayClusterBarrierGP, Warning, "%ls - ReleaseBarrier - invalid request data", *GetName());
		return EDisplayClusterCommResult::WrongRequestData;
	}

	{
		FScopeLock Lock(&BarriersCS);

		if (TSharedPtr<IDisplayClusterBarrier, ESPMode::ThreadSafe>* Barrier = Barriers.Find(BarrierId))
		{
			// Deactivate barrier first because it can be used by other clients currently.
			// In this case the destructor won't be called.
			(*Barrier)->Deactivate();

			// And forget about it. Once all the clients leave the barrier, the instance will be released.
			Barriers.Remove(BarrierId);

			// We can safely release the creation event as well
			if (FEvent** BarrierCreatedEvent = BarrierCreationEvents.Find(BarrierId))
			{
				FPlatformProcess::ReturnSynchEventToPool(*BarrierCreatedEvent);
				BarrierCreationEvents.Remove(BarrierId);
			}

			// Remove clients info of this barrier
			ReleaseBarrierInfo(BarrierId);

			UE_LOGF(LogDisplayClusterBarrierGP, Verbose, "%ls - ReleaseBarrier - Barrier '%ls' has been released", *GetName(), *BarrierId);

			Result = EBarrierControlResult::ReleasedSuccessfully;
		}
		else
		{
			// Not exists
			UE_LOGF(LogDisplayClusterBarrierGP, Verbose, "%ls - ReleaseBarrier - Barrier '%ls' not found", *GetName(), *BarrierId);
			Result = EBarrierControlResult::NotFound;
		}
	}

	return EDisplayClusterCommResult::Ok;
}

EDisplayClusterCommResult FDisplayClusterGenericBarrierService::SyncOnBarrier(const FString& BarrierId, const FString& CallerId, EBarrierControlResult& Result)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SRV_GB::SyncOnBarrier);

	// Validate input data
	if (BarrierId.IsEmpty() || CallerId.IsEmpty())
	{
		UE_LOGF(LogDisplayClusterBarrierGP, Warning, "%ls - SyncOnBarrier - invalid request data", *GetName());
		Result = EBarrierControlResult::UnknownError;
		return EDisplayClusterCommResult::WrongRequestData;
	}

	TSharedPtr<IDisplayClusterBarrier, ESPMode::ThreadSafe>* Barrier = nullptr;

	{
		FScopeLock Lock(&BarriersCS);

		// Get barrier ptr to be able to use outside of the critical section
		// so the other clients can also access it.
		Barrier = Barriers.Find(BarrierId);
	}

	// More validation
	if (!Barrier)
	{
		// Barrier not found
		Result = EBarrierControlResult::NotFound;
		return EDisplayClusterCommResult::WrongRequestData;
	}
	else if(!(*Barrier)->IsActivated())
	{
		Result = EBarrierControlResult::UnknownError;
		return EDisplayClusterCommResult::NotAllowed;
	}

	// Sync on the barrier
	UE_LOGF(LogDisplayClusterBarrierGP, VeryVerbose, "%ls - SyncOnBarrier - Barrier '%ls' wait start", *GetName(), *BarrierId);
	(*Barrier)->Wait(CallerId);
	UE_LOGF(LogDisplayClusterBarrierGP, VeryVerbose, "%ls - SyncOnBarrier - Barrier '%ls' wait end", *GetName(), *BarrierId);

	Result = EBarrierControlResult::SynchronizedSuccessfully;
	return EDisplayClusterCommResult::Ok;
}

EDisplayClusterCommResult FDisplayClusterGenericBarrierService::SyncOnBarrierWithData(const FString& BarrierId, const FString& CallerId, const TArray<uint8>& RequestData, TArray<uint8>& OutResponseData, EBarrierControlResult& Result)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SRV_GB::SyncOnBarrierWithData);

	// Validate input data
	if (BarrierId.IsEmpty() || CallerId.IsEmpty())
	{
		UE_LOGF(LogDisplayClusterBarrierGP, Warning, "%ls - SyncOnBarrier - invalid request data", *GetName());
		Result = EBarrierControlResult::UnknownError;
		return EDisplayClusterCommResult::WrongRequestData;
	}

	TSharedPtr<IDisplayClusterBarrier, ESPMode::ThreadSafe>* Barrier = nullptr;

	{
		FScopeLock Lock(&BarriersCS);

		// Get barrier ptr to be able to use outside of the critical section
		// so the other clients can also access it.
		Barrier = Barriers.Find(BarrierId);
	}

	// More validation
	if (!Barrier)
	{
		// Barrier not found
		UE_LOGF(LogDisplayClusterBarrierGP, Verbose, "%ls - SyncOnBarrierWithData - Barrier '%ls' not found", *GetName(), *BarrierId);
		Result = EBarrierControlResult::NotFound;
		return EDisplayClusterCommResult::WrongRequestData;
	}
	else if (!(*Barrier)->IsActivated())
	{
		UE_LOGF(LogDisplayClusterBarrierGP, Verbose, "%ls - SyncOnBarrierWithData - Barrier '%ls' is not active", *GetName(), *BarrierId);
		Result = EBarrierControlResult::UnknownError;
		return EDisplayClusterCommResult::NotAllowed;
	}

	// Sync on the barrier
	UE_LOGF(LogDisplayClusterBarrierGP, VeryVerbose, "%ls - SyncOnBarrierWithData - Barrier '%ls' wait start", *GetName(), *BarrierId);
	(*Barrier)->WaitWithData(CallerId, RequestData, OutResponseData);
	UE_LOGF(LogDisplayClusterBarrierGP, VeryVerbose, "%ls - SyncOnBarrierWithData - Barrier '%ls' wait end", *GetName(), *BarrierId);

	Result = EBarrierControlResult::SynchronizedSuccessfully;
	return EDisplayClusterCommResult::Ok;
}

void FDisplayClusterGenericBarrierService::ShutdownImpl()
{
	{
		FScopeLock Lock(&BarriersCS);

		// Deactivate all barriers
		for (TPair<FString, TSharedPtr<IDisplayClusterBarrier>>& Barrier : Barriers)
		{
			Barrier.Value->Deactivate();
		}

		// Release events
		for (TPair<FString, FEvent*>& EventIt : BarrierCreationEvents)
		{
			EventIt.Value->Trigger();
			FPlatformProcess::ReturnSynchEventToPool(EventIt.Value);
		}

		BarrierCreationEvents.Empty();
	}

	OnSessionClosed().RemoveAll(this);
}

void FDisplayClusterGenericBarrierService::InitializeBarrierInfo(const FString& BarrierId, const TMap<FString, TSet<FString>>& NodeToSyncCallers)
{
	FScopeLock Lock(&BarriersInfoCS);

	if (BarriersInfo.Contains(BarrierId))
	{
		return;
	}

	// Create new info slot
	TSharedRef<FBarrierInfo> NewBarrierInfo = BarriersInfo.Emplace(BarrierId, MakeShared<FBarrierInfo>());

	// Node-to-SyncCallers mapping
	NewBarrierInfo->NodeToThreadsMapping = NodeToSyncCallers;

	// SyncCaller-to-Node mapping
	for (const TPair<FString, TSet<FString>>& NodeMapping : NodeToSyncCallers)
	{
		for (const FString& SyncCaller : NodeMapping.Value)
		{
			NewBarrierInfo->ThreadToNodeMapping.Emplace(SyncCaller, NodeMapping.Key);
		}
	}
}

void FDisplayClusterGenericBarrierService::ReleaseBarrierInfo(const FString& BarrierId)
{
	FScopeLock LockInfo(&BarriersInfoCS);
	BarriersInfo.Remove(BarrierId);
}

void FDisplayClusterGenericBarrierService::ProcessSessionClosed(const FDisplayClusterSessionInfo& SessionInfo)
{
	if (!SessionInfo.IsTerminatedByServer())
	{
		// Ignore sessions with empty NodeId
		if (SessionInfo.NodeId.IsSet())
		{
			const FString NodeId = SessionInfo.NodeId.Get(FString());
			UnregisterClusterNode(NodeId);
		}
	}
}

void FDisplayClusterGenericBarrierService::UnregisterClusterNode(const FString& NodeId)
{
	FScopeLock Lock(&BarriersCS);

	// For every registered barrier
	for (TPair<FString, TSharedPtr<IDisplayClusterBarrier>>& Barrier : Barriers)
	{
		// Get its corresponding barrier info
		if (const TSharedPtr<const FBarrierInfo> BarrierInfo = GetBarrierInfo(Barrier.Key))
		{
			// See if there are any barrier sync callers associated with the node
			if (const TSet<FString>* const CallersAssociatedWithNode = BarrierInfo->NodeToThreadsMapping.Find(NodeId))
			{
				// If so, unregister all the caller IDs of that node
				for (const FString& CallerId : *CallersAssociatedWithNode)
				{
					Barrier.Value->UnregisterSyncCaller(CallerId);
				}
			}
		}
	}
}
