// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Network/Service/DisplayClusterService.h"
#include "Network/Session/IDisplayClusterSessionPacketHandler.h"
#include "Network/Protocol/IDisplayClusterProtocolGenericBarrier.h"
#include "Network/Packet/DisplayClusterPacketInternal.h"

#include "Cluster/IDisplayClusterGenericBarriersClient.h"

class FEvent;


/**
 * Generic barriers TCP server
 */
class FDisplayClusterGenericBarrierService
	: public    FDisplayClusterService
	, public    IDisplayClusterSessionPacketHandler<FDisplayClusterPacketInternal, true>
	, protected IDisplayClusterProtocolGenericBarrier
{
public:

	/**
	 * Additional barrier information that might be useful outside of the server
	 */
	struct FBarrierInfo
	{
		/** Holds ClusterNodeId-to-CallerIDs mapping */
		TMap<FString, TSet<FString>> NodeToThreadsMapping;

		/** Holds CallerID-to-ClusterNodeId mapping (reversed version of NodeToThreadsMapping) */
		TMap<FString, FString> ThreadToNodeMapping;
	};

public:

	FDisplayClusterGenericBarrierService(const FName& InInstanceName);

	virtual ~FDisplayClusterGenericBarrierService();

public:

	//~ Begin IDisplayClusterServer
	virtual void Shutdown() override;
	virtual FString GetProtocolName() const override;
	virtual void KillSession(const FString& NodeId) override;
	//~ End IDisplayClusterServer

public:

	/** Returns barrier by ID */
	TSharedPtr<IDisplayClusterBarrier> GetBarrier(const FString& BarrierId);

	/** Returns barrier information */
	TSharedPtr<const FBarrierInfo> GetBarrierInfo(const FString& BarrierId) const;

protected:

	/** Creates session instance for this service */
	virtual TSharedPtr<IDisplayClusterSession> CreateSession(FDisplayClusterSessionInfo& SessionInfo) override;

protected:

	//~ Begin IDisplayClusterSessionPacketHandler
	virtual TSharedPtr<FDisplayClusterPacketInternal> ProcessPacket(const TSharedPtr<FDisplayClusterPacketInternal>& Request, const FDisplayClusterSessionInfo& SessionInfo) override;
	//~ End IDisplayClusterSessionPacketHandler

protected:

	//~ Begin IDisplayClusterProtocolGenericBarrier
	virtual EDisplayClusterCommResult CreateBarrier(const FString& BarrierId, const TMap<FString, TSet<FString>>& NodeToSyncCallers, uint32 Timeout, EBarrierControlResult& Result) override;
	virtual EDisplayClusterCommResult WaitUntilBarrierIsCreated(const FString& BarrierId, EBarrierControlResult& Result) override;
	virtual EDisplayClusterCommResult IsBarrierAvailable(const FString& BarrierId, EBarrierControlResult& Result) override;
	virtual EDisplayClusterCommResult ReleaseBarrier(const FString& BarrierId, EBarrierControlResult& Result) override;
	virtual EDisplayClusterCommResult SyncOnBarrier(const FString& BarrierId, const FString& CallerId, EBarrierControlResult& Result) override;
	virtual EDisplayClusterCommResult SyncOnBarrierWithData(const FString& BarrierId, const FString& CallerId, const TArray<uint8>& RequestData, TArray<uint8>& OutResponseData, EBarrierControlResult& Result) override;
	//~ End IDisplayClusterProtocolGenericBarrier

private:

	/** Non-virtual shutdown implementation */
	void ShutdownImpl();

	/** An auxiliary function that is responsible for barrier information slot initialization */
	void InitializeBarrierInfo(const FString& BarrierId, const TMap<FString, TSet<FString>>& NodeToSyncCallers);

	/** An auxiliary function that is responsible for barrier information cleanup */
	void ReleaseBarrierInfo(const FString& BarrierId);

	/** Callback when a session is closed */
	void ProcessSessionClosed(const FDisplayClusterSessionInfo& SessionInfo);

	/** Unbinds specific cluster node and all its sync callers from all GP barriers */
	void UnregisterClusterNode(const FString& NodeId);

private:

	/** The barriers managed by this server */
	TMap<FString, TSharedPtr<IDisplayClusterBarrier>> Barriers;

	/** Barrier creation events */
	TMap<FString, FEvent*> BarrierCreationEvents;

	/** Critical section for internal data access synchronization */
	mutable FCriticalSection BarriersCS;

private:

	/** Holds extra information per - barrier */
	TMap<FString, TSharedRef<FBarrierInfo>> BarriersInfo;

	/** Critical section to synchronize access to the barriers information container */
	mutable FCriticalSection BarriersInfoCS;
};
