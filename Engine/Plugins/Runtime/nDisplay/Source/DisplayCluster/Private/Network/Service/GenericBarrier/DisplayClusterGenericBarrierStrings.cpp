// Copyright Epic Games, Inc. All Rights Reserved.

#include "Network/Service/GenericBarrier/DisplayClusterGenericBarrierStrings.h"


namespace DisplayClusterGenericBarrierStrings
{
	const FString ProtocolName = TEXT("GenericBarrier");

	const FString TypeRequest  = TEXT("Request");
	const FString TypeResponse = TEXT("Response");

	const FString ArgumentsDefaultCategory = TEXT("GB");

	// Shared arguments
	const FString ArgBarrierId = TEXT("BarrierId");
	const FString ArgResult    = TEXT("CtrlResult");

	namespace CreateBarrier
	{
		const FString Name = TEXT("CreateBarrier");

		const FString ArgCallers = TEXT("SyncCallers");
		const FString ArgTimeout = TEXT("Timeout");
	}

	namespace WaitUntilBarrierIsCreated
	{
		const FString Name = TEXT("WaitUntilBarrierIsCreated");
	}

	namespace IsBarrierAvailable
	{
		const FString Name = TEXT("IsBarrierAvailable");
	}

	namespace ReleaseBarrier
	{
		const FString Name = TEXT("ReleaseBarrier");
	}

	namespace SyncOnBarrier
	{
		const FString Name = TEXT("Sync");

		const FString ArgCallerId = TEXT("CallerId");
	}

	namespace SyncOnBarrierWithData
	{
		const FString Name = TEXT("SyncWithData");

		const FString ArgCallerId     = TEXT("CallerId");
		const FString ArgRequestData  = TEXT("ReqData");
		const FString ArgResponseData = TEXT("RespData");
	}
};
