// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"


/**
 * Generic barriers protocol strings
 */
namespace DisplayClusterGenericBarrierStrings
{
	extern const FString ProtocolName;

	extern const FString TypeRequest;
	extern const FString TypeResponse;

	extern const FString ArgumentsDefaultCategory;

	// Shared arguments
	extern const FString ArgBarrierId;
	extern const FString ArgResult;

	namespace CreateBarrier
	{
		extern const FString Name;

		extern const FString ArgCallers;
		extern const FString ArgTimeout;
	}

	namespace WaitUntilBarrierIsCreated
	{
		extern const FString Name;
	}

	namespace IsBarrierAvailable
	{
		extern const FString Name;
	}

	namespace ReleaseBarrier
	{
		extern const FString Name;
	}

	namespace SyncOnBarrier
	{
		extern const FString Name;

		extern const FString ArgCallerId;
	}

	namespace SyncOnBarrierWithData
	{
		extern const FString Name;

		extern const FString ArgCallerId;
		extern const FString ArgRequestData;
		extern const FString ArgResponseData;
	}
};
