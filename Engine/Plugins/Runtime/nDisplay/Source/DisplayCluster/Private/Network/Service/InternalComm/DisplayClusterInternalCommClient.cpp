// Copyright Epic Games, Inc. All Rights Reserved.

#include "Network/Service/InternalComm/DisplayClusterInternalCommClient.h"

#include "Cluster/IPDisplayClusterClusterManager.h"

#include "Misc/DisplayClusterGlobals.h"
#include "Misc/DisplayClusterHelpers.h"
#include "Misc/DisplayClusterLog.h"

#include "Misc/ScopeLock.h"
#include "Misc/FrameRate.h"
#include "Misc/Timecode.h"

#include "Network/Service/InternalComm/DisplayClusterInternalCommStrings.h"
#include "Network/Listener/DisplayClusterHelloMessageStrings.h"
#include "Network/Packet/DisplayClusterPacketInternal.h"
#include "Network/Configuration/DisplayClusterNetworkConfiguration.h"

#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"


FDisplayClusterInternalCommClient::FDisplayClusterInternalCommClient(const FName& InName)
	: FDisplayClusterClient(InName.ToString())
{
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterClient
//////////////////////////////////////////////////////////////////////////////////////////////
bool FDisplayClusterInternalCommClient::Connect(const FString& Address, const uint16 Port, const uint32 ConnectRetriesAmount, const uint32 ConnectRetryDelay)
{
	// First, allow base class to perform connection
	if(!FDisplayClusterClient::Connect(Address, Port, ConnectRetriesAmount, ConnectRetryDelay))
	{
		return false;
	}

	// Prepare 'hello' message
	TSharedPtr<FDisplayClusterPacketInternal> HelloMsg = MakeShared<FDisplayClusterPacketInternal>(
		DisplayClusterHelloMessageStrings::Hello::Name,
		DisplayClusterHelloMessageStrings::Hello::TypeRequest,
		DisplayClusterInternalCommStrings::ProtocolName
	);

	// Fill in the message with data
	const FString NodeId = GDisplayCluster->GetPrivateClusterMgr()->GetNodeId();
	HelloMsg->SetTextArg(DisplayClusterHelloMessageStrings::ArgumentsDefaultCategory, DisplayClusterHelloMessageStrings::Hello::ArgNodeId, NodeId);

	// Send message (no response awaiting)
	return SendPacket(HelloMsg);
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProtocolInternalComm
//////////////////////////////////////////////////////////////////////////////////////////////
EDisplayClusterCommResult FDisplayClusterInternalCommClient::GatherServicesHostingInfo(const FNodeServicesHostingInfo& ThisNodeInfo, FClusterServicesHostingInfo& OutHostingInfo)
{
	static TSharedPtr<FDisplayClusterPacketInternal> Request = MakeShared<FDisplayClusterPacketInternal>(
		DisplayClusterInternalCommStrings::GatherServicesHostingInfo::Name,
		DisplayClusterInternalCommStrings::TypeRequest,
		DisplayClusterInternalCommStrings::ProtocolName);

	// Param: Node hosting info
	{
		TArray<uint8> RequestData;
		FMemoryWriter MemoryWriter(RequestData);
		MemoryWriter << const_cast<FNodeServicesHostingInfo&>(ThisNodeInfo);
		Request->SetBinArg(DisplayClusterInternalCommStrings::ArgumentsDefaultCategory, DisplayClusterInternalCommStrings::GatherServicesHostingInfo::ArgNodeHostingInfo, RequestData);
	}

	TSharedPtr<FDisplayClusterPacketInternal> Response;

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CLN_IC::GatherServicesHostingInfo);
		Response = SendRecvPacket(Request);
	}

	if (!Response)
	{
		UE_LOGF(LogDisplayClusterNetwork, Warning, "Network error on '%ls'", *Request->GetName());
		return EDisplayClusterCommResult::NetworkError;
	}

	// Param: Cluster hosting info
	{
		TArray<uint8> ResponseData;
		if (!Response->GetBinArg(DisplayClusterInternalCommStrings::ArgumentsDefaultCategory, DisplayClusterInternalCommStrings::GatherServicesHostingInfo::ArgClusterHostingInfo, ResponseData))
		{
			UE_LOGF(LogDisplayClusterNetwork, Warning, "Couldn't extract parameter: %ls", *DisplayClusterInternalCommStrings::GatherServicesHostingInfo::ArgClusterHostingInfo);
			return EDisplayClusterCommResult::WrongResponseData;
		}

		OutHostingInfo.ClusterHostingInfo.Empty();

		FMemoryReader MemoryReader(ResponseData);

		int32 ItemsNum = 0;
		MemoryReader << ItemsNum;

		for (int32 Idx = 0; Idx < ItemsNum; ++Idx)
		{
			FString NodeId;
			MemoryReader << NodeId;

			FNodeServicesHostingInfo& HostingInfo = OutHostingInfo.ClusterHostingInfo.Emplace(*NodeId);
			MemoryReader.Serialize(&HostingInfo, sizeof(FNodeServicesHostingInfo));
		}
	}

	return Response->GetCommResult();
}

EDisplayClusterCommResult FDisplayClusterInternalCommClient::PostFailureNegotiate(TArray<uint8>& InOutRecoveryData)
{
	static const TSharedPtr<FDisplayClusterPacketInternal> Request = MakeShared<FDisplayClusterPacketInternal>(
		DisplayClusterInternalCommStrings::PostFailureNegotiate::Name,
		DisplayClusterInternalCommStrings::TypeRequest,
		DisplayClusterInternalCommStrings::ProtocolName);

	// Param: SyncState
	{
		Request->SetBinArg(DisplayClusterInternalCommStrings::ArgumentsDefaultCategory, DisplayClusterInternalCommStrings::PostFailureNegotiate::ArgSyncStateData, InOutRecoveryData);
	}

	TSharedPtr<FDisplayClusterPacketInternal> Response;

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CLN_IC::PostFailureNegotiate);
		Response = SendRecvPacket(Request);
	}

	if (!Response)
	{
		UE_LOGF(LogDisplayClusterNetwork, Warning, "Network error on '%ls'", *Request->GetName());
		return EDisplayClusterCommResult::NetworkError;
	}

	// Extract response data
	InOutRecoveryData.Reset();
	Response->GetBinArg(DisplayClusterInternalCommStrings::ArgumentsDefaultCategory, DisplayClusterInternalCommStrings::PostFailureNegotiate::ArgRecoveryData, InOutRecoveryData);

	return Response->GetCommResult();
}

EDisplayClusterCommResult FDisplayClusterInternalCommClient::RequestNodeDrop(const FString& NodeId, uint8 DropReason)
{
	static const TSharedPtr<FDisplayClusterPacketInternal> Request = MakeShared<FDisplayClusterPacketInternal>(
		DisplayClusterInternalCommStrings::RequestNodeDrop::Name,
		DisplayClusterInternalCommStrings::TypeRequest,
		DisplayClusterInternalCommStrings::ProtocolName);

	// Param: NodeId
	{
		Request->SetTextArg(DisplayClusterInternalCommStrings::ArgumentsDefaultCategory, DisplayClusterInternalCommStrings::RequestNodeDrop::ArgNodeId, NodeId);
	}

	// Param: DropReason
	{
		TArray<uint8> ParamDropReason;
		ParamDropReason.Add(static_cast<uint8>(DropReason));
		Request->SetBinArg(DisplayClusterInternalCommStrings::ArgumentsDefaultCategory, DisplayClusterInternalCommStrings::RequestNodeDrop::ArgDropReason, ParamDropReason);
	}

	TSharedPtr<FDisplayClusterPacketInternal> Response;

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CLN_IC::RequestNodeDrop);
		Response = SendRecvPacket(Request);
	}

	if (!Response)
	{
		UE_LOGF(LogDisplayClusterNetwork, Warning, "Network error on '%ls'", *Request->GetName());
		return EDisplayClusterCommResult::NetworkError;
	}

	return Response->GetCommResult();
}
