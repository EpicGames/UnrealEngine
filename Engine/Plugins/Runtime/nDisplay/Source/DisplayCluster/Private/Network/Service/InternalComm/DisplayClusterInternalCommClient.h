// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Network/DisplayClusterClient.h"
#include "Network/Packet/DisplayClusterPacketInternal.h"
#include "Network/Protocol/IDisplayClusterProtocolInternalComm.h"


/**
 * Cluster synchronization TCP client
 */
class FDisplayClusterInternalCommClient
	: public FDisplayClusterClient<FDisplayClusterPacketInternal>
	, public IDisplayClusterProtocolInternalComm
{
public:

	FDisplayClusterInternalCommClient(const FName& InName);

public:

	//~Begin IDisplayClusterClient
	virtual bool Connect(const FString& Address, const uint16 Port, const uint32 ConnectRetriesAmount = 0, const uint32 ConnectRetryDelay = 0) override;
	//~End IDisplayClusterClient

public:

	//~ Begin IDisplayClusterProtocolInternalComm
	virtual EDisplayClusterCommResult GatherServicesHostingInfo(const FNodeServicesHostingInfo& ThisNodeInfo, FClusterServicesHostingInfo& OutHostingInfo) override;
	virtual EDisplayClusterCommResult PostFailureNegotiate(TArray<uint8>& InOutRecoveryData) override;
	virtual EDisplayClusterCommResult RequestNodeDrop(const FString& NodeId, uint8 DropReason) override;
	//~ End IDisplayClusterProtocolInternalComm
};
