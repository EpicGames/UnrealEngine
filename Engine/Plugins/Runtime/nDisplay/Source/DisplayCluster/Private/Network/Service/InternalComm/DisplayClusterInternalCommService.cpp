// Copyright Epic Games, Inc. All Rights Reserved.

#include "Network/Service/InternalComm/DisplayClusterInternalCommService.h"

#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterEnums.h"
#include "IDisplayCluster.h"

#include "Cluster/DisplayClusterClusterEvent.h"
#include "Cluster/IPDisplayClusterClusterManager.h"
#include "Cluster/Controller/IDisplayClusterClusterNodeController.h"

#include "Config/IPDisplayClusterConfigManager.h"
#include "Game/IPDisplayClusterGameManager.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"

#include "Misc/DisplayClusterAppExit.h"
#include "Misc/QualifiedFrameTime.h"
#include "Misc/DisplayClusterGlobals.h"
#include "Misc/DisplayClusterLog.h"

#include "Network/Service/InternalComm/DisplayClusterInternalCommStrings.h"
#include "Network/Barrier/DisplayClusterBarrierFactory.h"
#include "Network/Barrier/IDisplayClusterBarrier.h"
#include "Network/Configuration/DisplayClusterNetworkConfiguration.h"
#include "Network/Conversion/DisplayClusterNetworkDataConversion.h"
#include "Network/Listener/DisplayClusterTcpListener.h"
#include "Network/Session/DisplayClusterSession.h"

#include "Serialization/MemoryWriter.h"


FDisplayClusterInternalCommService::FDisplayClusterInternalCommService(const FName& InInstanceName)
	: FDisplayClusterService(InInstanceName.ToString())
{
	IDisplayCluster& DCAPI = IDisplayCluster::Get();

	// Get list of cluster node IDs
	TSet<FString> NodeIds;
	if (const IDisplayClusterClusterManager* const ClusterMgr = DCAPI.GetClusterMgr())
	{
		ClusterMgr->GetNodeIds(NodeIds);
	}

	// Get cluster configuration
	UDisplayClusterConfigurationData* ConfigData = nullptr;
	if (const IDisplayClusterConfigManager* const ConfigMgr = DCAPI.GetConfigMgr())
	{
		ConfigData = ConfigMgr->GetConfig();
	}

	// Setup NetInfoSync barrier and callbacks
	const uint32 HostingInfoSyncBarrierTimeout = ConfigData ? ConfigData->Cluster->Network.GameStartBarrierTimeout : TNumericLimits<uint32>::Max();
	HostingInfoSyncBarrier = MakeShareable(FDisplayClusterBarrierFactory::CreateBarrier(TEXT("IC_NetInfoSync_Barrier"), NodeIds, HostingInfoSyncBarrierTimeout));
	HostingInfoSyncBarrier->OnBarrierTimeout().AddRaw(this, &FDisplayClusterInternalCommService::ProcessBarrierTimeout);
	HostingInfoSyncBarrier->GetPreSyncEndDelegate().BindRaw(this, &FDisplayClusterInternalCommService::OnHostingInfoSynchronization);

	// Setup PostFailSync barrier and callbacks
	const uint32 PostFailureNegotiationBarrierTimeout = ConfigData ? ConfigData->Cluster->Network.FrameStartBarrierTimeout : 5000;
	PostFailureNegotiationBarrier = MakeShareable(FDisplayClusterBarrierFactory::CreateBarrier(TEXT("IC_PostFailSync_Barrier"), NodeIds, PostFailureNegotiationBarrierTimeout));
	PostFailureNegotiationBarrier->OnBarrierTimeout().AddRaw(this, &FDisplayClusterInternalCommService::ProcessBarrierTimeout);

	// Subscribe for SessionClosed events
	OnSessionClosed().AddRaw(this, &FDisplayClusterInternalCommService::ProcessSessionClosed);
}

FDisplayClusterInternalCommService::~FDisplayClusterInternalCommService()
{
	// Unsubscribe from barrier timeout events
	PostFailureNegotiationBarrier->OnBarrierTimeout().RemoveAll(this);
	HostingInfoSyncBarrier->OnBarrierTimeout().RemoveAll(this);
	HostingInfoSyncBarrier->GetPreSyncEndDelegate().Unbind();

	// Unsubscribe from SessionClosed notifications
	OnSessionClosed().RemoveAll(this);

	ShutdownImpl();
}


bool FDisplayClusterInternalCommService::Start(const FString& Address, const uint16 Port)
{
	// Start internals
	StartInternal();

	return FDisplayClusterServer::Start(Address, Port);
}

bool FDisplayClusterInternalCommService::Start(TSharedRef<FDisplayClusterTcpListener>& ExternalListener)
{
	// Start internals
	StartInternal();

	return FDisplayClusterServer::Start(ExternalListener);
}

void FDisplayClusterInternalCommService::Shutdown()
{
	// Release internals
	ShutdownImpl();

	return FDisplayClusterServer::Shutdown();
}

FString FDisplayClusterInternalCommService::GetProtocolName() const
{
	return DisplayClusterInternalCommStrings::ProtocolName;
}

void FDisplayClusterInternalCommService::KillSession(const FString& NodeId)
{
	// Before killing the session on the parent level, we need to unregister this node from the barriers
	UnregisterClusterNode(NodeId);

	// Now do the session related job
	FDisplayClusterServer::KillSession(NodeId);
}

TSharedPtr<IDisplayClusterBarrier> FDisplayClusterInternalCommService::GetPostFailureNegotiationBarrier()
{
	return PostFailureNegotiationBarrier;
}

TSharedPtr<IDisplayClusterSession> FDisplayClusterInternalCommService::CreateSession(FDisplayClusterSessionInfo& SessionInfo)
{
	SessionInfo.SessionName = FString::Printf(TEXT("%s_%llu_%s_%s"),
		*GetName(),
		SessionInfo.SessionId,
		*SessionInfo.Endpoint.ToString(),
		*SessionInfo.NodeId.Get(TEXT("(na)"))
	);

	return MakeShared<FDisplayClusterSession<FDisplayClusterPacketInternal, true>>(SessionInfo, *this, *this, FDisplayClusterService::GetThreadPriority());
}

void FDisplayClusterInternalCommService::StartInternal()
{
	// Activate all barriers
	PostFailureNegotiationBarrier->Activate();
	HostingInfoSyncBarrier->Activate();
}

void FDisplayClusterInternalCommService::ProcessSessionClosed(const FDisplayClusterSessionInfo& SessionInfo)
{
	if (!SessionInfo.IsTerminatedByServer())
	{
		if (SessionInfo.NodeId.IsSet())
		{
			// Get node ID
			const FString NodeId = SessionInfo.NodeId.Get(FString());

			// We have to unregister the node that just disconnected from all barriers
			UnregisterClusterNode(NodeId);

			// Prepare failure info
			FDisplayClusterServiceFailureEvent EventInfo;
			EventInfo.NodeFailed  = SessionInfo.NodeId;
			EventInfo.FailureType = FDisplayClusterServiceFailureEvent::ENodeFailType::ConnectionLost;

			// Notify others about node fail
			OnNodeFailed().Broadcast(EventInfo);
		}
	}
}

void FDisplayClusterInternalCommService::UnregisterClusterNode(const FString& NodeId)
{
	PostFailureNegotiationBarrier->UnregisterSyncCaller(NodeId);
	HostingInfoSyncBarrier->UnregisterSyncCaller(NodeId);
}

void FDisplayClusterInternalCommService::ProcessBarrierTimeout(const FString& BarrierName, const TSet<FString>& NodesTimedOut)
{
	// We have to unregister the node that just timed out from all barriers
	for (const FString& NodeId : NodesTimedOut)
	{
		UnregisterClusterNode(NodeId);
	}

	// Notify others about timeout
	for (const FString& NodeId : NodesTimedOut)
	{
		// Prepare failure info
		FDisplayClusterServiceFailureEvent EventInfo;
		EventInfo.NodeFailed  = NodeId;
		EventInfo.FailureType = FDisplayClusterServiceFailureEvent::ENodeFailType::BarrierTimeOut;

		OnNodeFailed().Broadcast(EventInfo);
	}
}

void FDisplayClusterInternalCommService::OnHostingInfoSynchronization(FDisplayClusterBarrierPreSyncEndDelegateData& SyncData)
{
	TArray<uint8> ResponseData;
	FMemoryWriter MemoryWriter(ResponseData);

	// Nodes amount
	int32 NodesNum = SyncData.RequestData.Num();
	MemoryWriter << NodesNum;
	
	// N records of type [NodeId, BinaryRequestData]
	for (TPair<FString, TArray<uint8>>& Request : const_cast<TMap<FString, TArray<uint8>>&>(SyncData.RequestData))
	{
		MemoryWriter << Request.Key;
		MemoryWriter.Serialize(Request.Value.GetData(), Request.Value.Num());

		SyncData.ResponseData.Emplace(Request.Key);
	}

	// We need to send the same response to every node
	for (TPair<FString, TArray<uint8>>& NodeResponse : SyncData.ResponseData)
	{
		NodeResponse.Value = ResponseData;
	}
}

void FDisplayClusterInternalCommService::ShutdownImpl()
{
	// Deactivate the barriers

	if (PostFailureNegotiationBarrier)
	{
		PostFailureNegotiationBarrier->Deactivate();
	}

	if (HostingInfoSyncBarrier)
	{
		HostingInfoSyncBarrier->Deactivate();
	}
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterSessionPacketHandler
//////////////////////////////////////////////////////////////////////////////////////////////
TSharedPtr<FDisplayClusterPacketInternal> FDisplayClusterInternalCommService::ProcessPacket(const TSharedPtr<FDisplayClusterPacketInternal>& Request, const FDisplayClusterSessionInfo& SessionInfo)
{
	// Check the pointer
	if (!Request.IsValid())
	{
		UE_LOGF(LogDisplayClusterNetwork, Error, "%ls - Invalid request data (nullptr)", *GetName());
		return nullptr;
	}

	// Cache session info
	SetSessionInfoCache(SessionInfo);

	UE_LOGF(LogDisplayClusterNetwork, Verbose, "%ls - Processing packet: %ls", *GetName(), *Request->ToLogString());

	// Check protocol and type
	if (!Request->GetProtocol().Equals(DisplayClusterInternalCommStrings::ProtocolName, ESearchCase::IgnoreCase) ||
		!Request->GetType().Equals(DisplayClusterInternalCommStrings::TypeRequest, ESearchCase::IgnoreCase))
	{
		UE_LOGF(LogDisplayClusterNetwork, Error, "%ls - Unsupported packet type: %ls", *GetName(), *Request->ToLogString());
		return nullptr;
	}

	// Initialize response packet
	TSharedPtr<FDisplayClusterPacketInternal> Response = MakeShared<FDisplayClusterPacketInternal>(Request->GetName(), DisplayClusterInternalCommStrings::TypeResponse, Request->GetProtocol());

	// Dispatch the packet
	const FString& ReqName = Request->GetName();
	if (ReqName.Equals(DisplayClusterInternalCommStrings::RequestNodeDrop::Name, ESearchCase::IgnoreCase))
	{
		FString ParamNodeId;
		uint8 ParamDropReason = 0;
		TArray<uint8> ParamDropReasonBin;

		Request->GetTextArg(DisplayClusterInternalCommStrings::ArgumentsDefaultCategory, DisplayClusterInternalCommStrings::RequestNodeDrop::ArgNodeId, ParamNodeId);
		Request->GetBinArg(DisplayClusterInternalCommStrings::ArgumentsDefaultCategory, DisplayClusterInternalCommStrings::RequestNodeDrop::ArgDropReason, ParamDropReasonBin);

		checkSlow(ParamDropReasonBin.Num() == 1);
		ParamDropReason = ParamDropReasonBin[0];

		const EDisplayClusterCommResult CommResult = RequestNodeDrop(ParamNodeId, ParamDropReason);
		Response->SetCommResult(CommResult);

		return Response;
	}
	else if (ReqName.Equals(DisplayClusterInternalCommStrings::PostFailureNegotiate::Name, ESearchCase::IgnoreCase))
	{
		TArray<uint8> InOutRecoveryData;
		Request->GetBinArg(DisplayClusterInternalCommStrings::ArgumentsDefaultCategory, DisplayClusterInternalCommStrings::PostFailureNegotiate::ArgSyncStateData, InOutRecoveryData);

		const EDisplayClusterCommResult CommResult = PostFailureNegotiate(InOutRecoveryData);

		Response->SetBinArg(DisplayClusterInternalCommStrings::ArgumentsDefaultCategory, DisplayClusterInternalCommStrings::PostFailureNegotiate::ArgRecoveryData, InOutRecoveryData);
		Response->SetCommResult(CommResult);

		return Response;
	}
	else if (ReqName.Equals(DisplayClusterInternalCommStrings::GatherServicesHostingInfo::Name, ESearchCase::IgnoreCase))
	{
		TArray<uint8> HostingInfoData;
		Request->GetBinArg(DisplayClusterInternalCommStrings::ArgumentsDefaultCategory, DisplayClusterInternalCommStrings::GatherServicesHostingInfo::ArgNodeHostingInfo, HostingInfoData);

		TArray<uint8> ClusterInfoData;
		const EDisplayClusterCommResult CommResult = GatherServicesHostingInfoImpl(HostingInfoData, ClusterInfoData);

		Response->SetBinArg(DisplayClusterInternalCommStrings::ArgumentsDefaultCategory, DisplayClusterInternalCommStrings::GatherServicesHostingInfo::ArgClusterHostingInfo, ClusterInfoData);
		Response->SetCommResult(CommResult);

		return Response;
	}

	// Being here means that we have no appropriate dispatch logic for this packet
	UE_LOGF(LogDisplayClusterNetwork, Warning, "%ls - No dispatcher found for packet '%ls'", *GetName(), *Request->GetName());

	return nullptr;
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProtocolClusterSync
//////////////////////////////////////////////////////////////////////////////////////////////
EDisplayClusterCommResult FDisplayClusterInternalCommService::GatherServicesHostingInfo(const FNodeServicesHostingInfo& ThisNodeInfo, FClusterServicesHostingInfo& OutHostingInfo)
{
	// Like any other protocol function, we could call this handler to do the job. This would require the following:
	// 1. Deserialize message data into the function parameters above
	// 2. Serialize them back to binary
	// 3. Sync on barrier with data
	// To avoid unnecessary deserialization and serialization, we pass message data to the barrier as is.
	// Refer GatherServicesHostingInfoImpl below.
	return EDisplayClusterCommResult::NotImplemented;
}

EDisplayClusterCommResult FDisplayClusterInternalCommService::PostFailureNegotiate(TArray<uint8>& InOutRecoveryData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SRV_IC::PostFailureNegotiate);

	TArray<uint8> RecoveryData;
	PostFailureNegotiationBarrier->WaitWithData(GetSessionInfoCache().NodeId.Get(FString()), InOutRecoveryData, RecoveryData);
	InOutRecoveryData = MoveTemp(RecoveryData);

	return EDisplayClusterCommResult::Ok;
}

EDisplayClusterCommResult FDisplayClusterInternalCommService::RequestNodeDrop(const FString& NodeId, uint8 DropReason)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SRV_IC::RequestNodeDrop);

	const FString ThisNodeId = GDisplayCluster->GetPrivateClusterMgr()->GetNodeId();
	if (ThisNodeId.Equals(NodeId, ESearchCase::IgnoreCase))
	{
		FDisplayClusterAppExit::ExitApplication(FString::Printf(TEXT("Exit requested, reason=%u"), DropReason));
	}

	return EDisplayClusterCommResult::Ok;
}


//////////////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterInternalCommService
//////////////////////////////////////////////////////////////////////////////////////////////
EDisplayClusterCommResult FDisplayClusterInternalCommService::GatherServicesHostingInfoImpl(const TArray<uint8>& RequestData, TArray<uint8>& ResponseData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SRV_IC::GatherServicesHostingInfo);

	// Node Id is used as a caller ID
	const FString CallingNodeId = GetSessionInfoCache().NodeId.Get(FString());

	// Sync on the barrier
	HostingInfoSyncBarrier->WaitWithData(CallingNodeId, RequestData, ResponseData);

	return EDisplayClusterCommResult::Ok;
}
