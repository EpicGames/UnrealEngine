// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Network/Service/DisplayClusterService.h"
#include "Network/Session/IDisplayClusterSessionPacketHandler.h"
#include "Network/Protocol/IDisplayClusterProtocolInternalComm.h"
#include "Network/Packet/DisplayClusterPacketInternal.h"

class IDisplayClusterBarrier;


/**
 * In-cluster communication TCP server
 */
class FDisplayClusterInternalCommService
	: public    FDisplayClusterService
	, public    IDisplayClusterSessionPacketHandler<FDisplayClusterPacketInternal, true>
	, protected IDisplayClusterProtocolInternalComm
{
public:

	FDisplayClusterInternalCommService(const FName& InInstanceName);

	virtual ~FDisplayClusterInternalCommService();

public:

	//~ Begin IDisplayClusterServer
	virtual bool Start(const FString& Address, const uint16 Port) override;
	virtual bool Start(TSharedRef<FDisplayClusterTcpListener>& ExternalListener) override;
	virtual void Shutdown() override;
	virtual FString GetProtocolName() const override;
	virtual void KillSession(const FString& NodeId) override;
	//~ End IDisplayClusterServer

public:

	/** Returns PostFailureNegotiation barrier */
	TSharedPtr<IDisplayClusterBarrier> GetPostFailureNegotiationBarrier();

protected:

	/** Creates session instance for this service */
	virtual TSharedPtr<IDisplayClusterSession> CreateSession(FDisplayClusterSessionInfo& SessionInfo) override;

private:

	/** Performs internal initialization during server start */
	void StartInternal();

	/** Callback when a session is closed */
	void ProcessSessionClosed(const FDisplayClusterSessionInfo& SessionInfo);

	/** Unregister cluster node from a barrier */
	void UnregisterClusterNode(const FString& NodeId);

	/** Callbck on barrier timeout */
	void ProcessBarrierTimeout(const FString& BarrierName, const TSet<FString>& NodesTimedOut);

	/** Hosting info synchronization delegate. It's used to generate barrier response data. */
	void OnHostingInfoSynchronization(FDisplayClusterBarrierPreSyncEndDelegateData& SyncData);

	/** Non-virtual shutdown implementation */
	void ShutdownImpl();

protected:

	//~ Begin IDisplayClusterSessionPacketHandler
	virtual TSharedPtr<FDisplayClusterPacketInternal> ProcessPacket(const TSharedPtr<FDisplayClusterPacketInternal>& Request, const FDisplayClusterSessionInfo& SessionInfo) override;
	//~ End IDisplayClusterSessionPacketHandler

protected:

	//~ Begin IDisplayClusterProtocolClusterSync
	virtual EDisplayClusterCommResult GatherServicesHostingInfo(const FNodeServicesHostingInfo& ThisNodeInfo, FClusterServicesHostingInfo& OutHostingInfo) override;
	virtual EDisplayClusterCommResult PostFailureNegotiate(TArray<uint8>& InOutRecoveryData) override;
	virtual EDisplayClusterCommResult RequestNodeDrop(const FString& NodeId, uint8 DropReason) override;
	//~ End IDisplayClusterProtocolClusterSync

private:

	/** Effective replacement of GatherServicesHostingInfo to prevent unnecessary deserialization/serialization. */
	EDisplayClusterCommResult GatherServicesHostingInfoImpl(const TArray<uint8>& RequestData, TArray<uint8>& ResponseData);

private:

	/** This barrier is used to synchronize nodes hosting information on start. */
	TSharedPtr<IDisplayClusterBarrier> HostingInfoSyncBarrier;

	/** This barrier is used to synchronize all the cluster nodes during failover procedure. */
	TSharedPtr<IDisplayClusterBarrier> PostFailureNegotiationBarrier;
};
