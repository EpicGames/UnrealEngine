// Copyright Epic Games, Inc. All Rights Reserved.

#include "Network/Service/InternalComm/DisplayClusterInternalCommStrings.h"


namespace DisplayClusterInternalCommStrings
{
	const FString ProtocolName = TEXT("InternalComm");

	const FString TypeRequest  = TEXT("Request");
	const FString TypeResponse = TEXT("Response");

	const FString ArgumentsDefaultCategory = TEXT("IC");

	namespace GatherServicesHostingInfo
	{
		const FString Name = TEXT("GatherServicesHostingInfo");

		const FString ArgNodeHostingInfo    = TEXT("NodeHostingInfo");
		const FString ArgClusterHostingInfo = TEXT("ClusterHostingInfo");
	}

	namespace PostFailureNegotiate
	{
		const FString Name = TEXT("PostFailureNegotiate");

		const FString ArgSyncStateData = TEXT("SyncState");
		const FString ArgRecoveryData  = TEXT("RecoveryData");
	}

	namespace RequestNodeDrop
	{
		const FString Name = TEXT("RequestNodeDrop");

		const FString ArgNodeId     = TEXT("NodeId");
		const FString ArgDropReason = TEXT("DropReason");
	}
};
