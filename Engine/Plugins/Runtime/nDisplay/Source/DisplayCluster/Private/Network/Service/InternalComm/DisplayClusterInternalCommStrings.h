// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"


/**
 * Internal comm related strings
 */
namespace DisplayClusterInternalCommStrings
{
	extern const FString ProtocolName;

	extern const FString TypeRequest;
	extern const FString TypeResponse;

	extern const FString ArgumentsDefaultCategory;

	namespace GatherServicesHostingInfo
	{
		extern const FString Name;

		extern const FString ArgNodeHostingInfo;
		extern const FString ArgClusterHostingInfo;
	}

	namespace PostFailureNegotiate
	{
		extern const FString Name;

		extern const FString ArgSyncStateData;
		extern const FString ArgRecoveryData;
	}

	namespace RequestNodeDrop
	{
		extern const FString Name;

		extern const FString ArgNodeId;
		extern const FString ArgDropReason;
	}
};
