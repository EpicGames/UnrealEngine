// Copyright Epic Games, Inc. All Rights Reserved.

#include "Network/Service/RenderSync/DisplayClusterRenderSyncClient.h"
#include "Network/Service/RenderSync/DisplayClusterRenderSyncStrings.h"
#include "Network/Configuration/DisplayClusterNetworkConfiguration.h"
#include "Network/Packet/DisplayClusterPacketInternal.h"
#include "Network/Listener/DisplayClusterHelloMessageStrings.h"

#include "Cluster/IPDisplayClusterClusterManager.h"

#include "Misc/DisplayClusterGlobals.h"
#include "Misc/DisplayClusterLog.h"


FDisplayClusterRenderSyncClient::FDisplayClusterRenderSyncClient(const FName& InName)
	: FDisplayClusterClient(InName.ToString())
{
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterClient
//////////////////////////////////////////////////////////////////////////////////////////////
bool FDisplayClusterRenderSyncClient::Connect(const FString& Address, const uint16 Port, const uint32 ConnectRetriesAmount, const uint32 ConnectRetryDelay)
{
	// First, allow base class to perform connection
	if (!FDisplayClusterClient::Connect(Address, Port, ConnectRetriesAmount, ConnectRetryDelay))
	{
		return false;
	}

	// Prepare 'hello' message
	TSharedPtr<FDisplayClusterPacketInternal> HelloMsg = MakeShared<FDisplayClusterPacketInternal>(
		DisplayClusterHelloMessageStrings::Hello::Name,
		DisplayClusterHelloMessageStrings::Hello::TypeRequest,
		DisplayClusterRenderSyncStrings::ProtocolName
	);

	// Fill in the message with data
	const FString NodeId = GDisplayCluster->GetPrivateClusterMgr()->GetNodeId();
	HelloMsg->SetTextArg(DisplayClusterHelloMessageStrings::ArgumentsDefaultCategory, DisplayClusterHelloMessageStrings::Hello::ArgNodeId, NodeId);

	// Send message (no response awaiting)
	return SendPacket(HelloMsg);
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProtocolRenderSync
//////////////////////////////////////////////////////////////////////////////////////////////
EDisplayClusterCommResult FDisplayClusterRenderSyncClient::SynchronizeOnBarrier()
{
	static const TSharedPtr<FDisplayClusterPacketInternal> Request(
		new FDisplayClusterPacketInternal(
			DisplayClusterRenderSyncStrings::SynchronizeOnBarrier::Name,
			DisplayClusterRenderSyncStrings::TypeRequest,
			DisplayClusterRenderSyncStrings::ProtocolName)
	);

	TSharedPtr<FDisplayClusterPacketInternal> Response;

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CLN_RS::WaitForSwapSync);
		Response = SendRecvPacket(Request);
	}

	if (!Response)
	{
		UE_LOGF(LogDisplayClusterNetwork, Warning, "Network error on '%ls'", *Request->GetName());
		return EDisplayClusterCommResult::NetworkError;
	}

	return Response->GetCommResult();
}
