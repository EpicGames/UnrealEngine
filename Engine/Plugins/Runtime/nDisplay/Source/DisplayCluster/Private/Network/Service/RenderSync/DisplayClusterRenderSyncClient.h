// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Network/DisplayClusterClient.h"
#include "Network/Protocol/IDisplayClusterProtocolRenderSync.h"
#include "Network/Packet/DisplayClusterPacketInternal.h"


/**
 * Rendering synchronization TCP client
 */
class FDisplayClusterRenderSyncClient
	: public FDisplayClusterClient<FDisplayClusterPacketInternal>
	, public IDisplayClusterProtocolRenderSync
{
public:

	FDisplayClusterRenderSyncClient(const FName& InName);

public:

	//~ Begin IDisplayClusterClient
	virtual bool Connect(const FString& Address, const uint16 Port, const uint32 ConnectRetriesAmount = 0, const uint32 ConnectRetryDelay = 0) override;
	//~ End IDisplayClusterClient

public:

	//~ Begin IDisplayClusterProtocolRenderSync
	virtual EDisplayClusterCommResult SynchronizeOnBarrier() override;
	//~ End IDisplayClusterProtocolRenderSync
};
