// Copyright Epic Games, Inc. All Rights Reserved.

#include "Network/Service/RenderSync/DisplayClusterRenderSyncService.h"
#include "Network/Service/RenderSync/DisplayClusterRenderSyncStrings.h"
#include "Network/Barrier/DisplayClusterBarrierFactory.h"
#include "Network/Barrier/IDisplayClusterBarrier.h"
#include "Network/Configuration/DisplayClusterNetworkConfiguration.h"
#include "Network/Listener/DisplayClusterTcpListener.h"
#include "Network/Packet/DisplayClusterPacketInternal.h"
#include "Network/Session/DisplayClusterSession.h"

#include "Cluster/IPDisplayClusterClusterManager.h"
#include "Config/IPDisplayClusterConfigManager.h"

#include "DisplayClusterConfigurationStrings.h"
#include "DisplayClusterConfigurationTypes.h"

#include "Misc/DisplayClusterGlobals.h"
#include "Misc/DisplayClusterLog.h"


FDisplayClusterRenderSyncService::FDisplayClusterRenderSyncService(const FName& InInstanceName)
	: FDisplayClusterService(InInstanceName.ToString())
{
	// Perform barriers initialization depending on current circumstances
	InitializeBarriers();
	// Subscribe for SessionClosed events
	OnSessionClosed().AddRaw(this, &FDisplayClusterRenderSyncService::ProcessSessionClosed);
}

FDisplayClusterRenderSyncService::~FDisplayClusterRenderSyncService()
{
	ShutdownImpl();

	// Unsubscribe from barrier timeout events
	UnsubscribeFromAllBarrierEvents();
	// Unsubscribe from SessionClosed notifications
	OnSessionClosed().RemoveAll(this);
}


bool FDisplayClusterRenderSyncService::Start(const FString& Address, const uint16 Port)
{
	// Start internals
	StartInternal();

	return FDisplayClusterServer::Start(Address, Port);
}

bool FDisplayClusterRenderSyncService::Start(TSharedRef<FDisplayClusterTcpListener>& ExternalListener)
{
	// Start internals
	StartInternal();

	return FDisplayClusterServer::Start(ExternalListener);
}

void FDisplayClusterRenderSyncService::Shutdown()
{
	ShutdownImpl();

	return FDisplayClusterServer::Shutdown();
}

FString FDisplayClusterRenderSyncService::GetProtocolName() const
{
	return DisplayClusterRenderSyncStrings::ProtocolName;
}

void FDisplayClusterRenderSyncService::KillSession(const FString& NodeId)
{
	// Before killing the session on the parent level, we need to unregister this node from the barriers
	UnregisterClusterNode(NodeId);

	// Now do the session related job
	FDisplayClusterServer::KillSession(NodeId);
}

TSharedPtr<IDisplayClusterSession> FDisplayClusterRenderSyncService::CreateSession(FDisplayClusterSessionInfo& SessionInfo)
{
	SessionInfo.SessionName = FString::Printf(TEXT("%s_%llu_%s_%s"),
		*GetName(),
		SessionInfo.SessionId,
		*SessionInfo.Endpoint.ToString(),
		*SessionInfo.NodeId.Get(TEXT("(na)"))
	);

	return MakeShared<FDisplayClusterSession<FDisplayClusterPacketInternal, true>>(SessionInfo, *this, *this, FDisplayClusterService::GetThreadPriority());
}

void FDisplayClusterRenderSyncService::ProcessSessionClosed(const FDisplayClusterSessionInfo& SessionInfo)
{
	if (!SessionInfo.IsTerminatedByServer())
	{
		// Ignore sessions with empty NodeId
		if (SessionInfo.NodeId.IsSet())
		{
			const FString NodeId = SessionInfo.NodeId.Get(FString());

			// We have to unregister the node that just disconnected from all barriers
			UnregisterClusterNode(NodeId);

			FDisplayClusterServiceFailureEvent EventInfo;
			EventInfo.NodeFailed  = SessionInfo.NodeId;
			EventInfo.FailureType = FDisplayClusterServiceFailureEvent::ENodeFailType::ConnectionLost;

			// Notify others about node fail
			OnNodeFailed().Broadcast(EventInfo);
		}
	}
}

// Callbck on barrier timeout
void FDisplayClusterRenderSyncService::ProcessBarrierTimeout(const FString& BarrierName, const TSet<FString>& NodesTimedOut)
{
	for (const FString& NodeId : NodesTimedOut)
	{
		// We have to unregister the node that just timed out from all barriers
		UnregisterClusterNode(NodeId);

		FDisplayClusterServiceFailureEvent EventInfo;
		EventInfo.NodeFailed  = NodeId;
		EventInfo.FailureType = FDisplayClusterServiceFailureEvent::ENodeFailType::BarrierTimeOut;

		// Notify others about timeout
		OnNodeFailed().Broadcast(EventInfo);
	}
}

void FDisplayClusterRenderSyncService::StartInternal()
{
	// Activate all barriers
	ActivateAllBarriers();
}

void FDisplayClusterRenderSyncService::InitializeBarriers()
{
	// Get configuration data
	const UDisplayClusterConfigurationData* const Config = GDisplayCluster->GetPrivateConfigMgr()->GetConfig();
	if (!Config)
	{
		UE_LOGF(LogDisplayClusterNetwork, Error, "%ls - No configuartion data, can't initialize barriers", *GetName());
		return;
	}

	// Get list of cluster node IDs (runtime nodes)
	TSet<FString> RuntimeNodeIds;
	GDisplayCluster->GetPrivateClusterMgr()->GetNodeIds(RuntimeNodeIds);

	// Filter nodes that are expected to present in runtime
	const TMap<FString, TObjectPtr<UDisplayClusterConfigurationClusterNode>> RuntimeNodeCfgs = Config->Cluster->Nodes.FilterByPredicate(
		[&RuntimeNodeIds](const TPair<FString, TObjectPtr<UDisplayClusterConfigurationClusterNode>>& Pair)
		{
			return RuntimeNodeIds.Contains(Pair.Key);
		});

	// Build sync groups (this one is an aux mapping of policy IDs to their node IDs)
	TMap<FString, TSet<FString>> SyncGroups;
	for (const TPair<FString, TObjectPtr<UDisplayClusterConfigurationClusterNode>>& RuntimeNodeCfg : RuntimeNodeCfgs)
	{
		FString SyncPolicyId;

		// Headless nodes always use the same policy
		if (RuntimeNodeCfg.Value->bRenderHeadless)
		{
			SyncPolicyId = DisplayClusterConfigurationStrings::config::cluster::render_sync::HeadlessRenderingSyncPolicy;
		}
#if 0 // Not implemented yet
		// Individual sync policy
		else if(NodeCfgIt.Value->bOverrideSyncPolicy)
		{
			SyncPolicyId = NodeCfgIt.Value->Sync.RenderSyncPolicy.Type;
		}
#endif
		// Then default sync policy
		else
		{
			SyncPolicyId = Config->Cluster->Sync.RenderSyncPolicy.Type;
		}

		// To prevent any case related issues caused by manual .ndisplay file editing. We won't
		// need it once we refactor all the IDs to be FName's.
		SyncPolicyId.ToLowerInline();

		// Create new sync group if not yet available
		TSet<FString>* NodesInSyncGroup = SyncGroups.Find(SyncPolicyId);
		if (!NodesInSyncGroup)
		{
			NodesInSyncGroup = &SyncGroups.Emplace(SyncPolicyId);
		}

		// Assign this cluster node to the specific sync group
		NodesInSyncGroup->Add(RuntimeNodeCfg.Key);
		// Update node-policy mapping
		NodeToPolicyMap.Emplace(RuntimeNodeCfg.Key, SyncPolicyId);
	}

	// Finally, initialize barriers for every sync group
	for (const TPair<FString, TSet<FString>>& SyncGroup : SyncGroups)
	{
		const FString BarrierName = SyncGroup.Key + TEXT("_present_barrier");

		// Instantiate the barrier
		if (IDisplayClusterBarrier* const Barrier = FDisplayClusterBarrierFactory::CreateBarrier(BarrierName, SyncGroup.Value, Config->Cluster->Network.RenderSyncBarrierTimeout))
		{
			// Subscribe for barrier timeout events
			Barrier->OnBarrierTimeout().AddRaw(this, &FDisplayClusterRenderSyncService::ProcessBarrierTimeout);
			// Store barrier instance
			PolicyToBarrierMap.Emplace(SyncGroup.Key, Barrier);
		}
	}
}

void FDisplayClusterRenderSyncService::ActivateAllBarriers()
{
	for (TPair<FString, TUniquePtr<IDisplayClusterBarrier>>& PolicyToBarrierIt : PolicyToBarrierMap)
	{
		PolicyToBarrierIt.Value->Activate();
	}
}

void FDisplayClusterRenderSyncService::DeactivateAllBarriers()
{
	for (TPair<FString, TUniquePtr<IDisplayClusterBarrier>>& PolicyToBarrierIt : PolicyToBarrierMap)
	{
		PolicyToBarrierIt.Value->Deactivate();
	}
}

void FDisplayClusterRenderSyncService::UnsubscribeFromAllBarrierEvents()
{
	for (TPair<FString, TUniquePtr<IDisplayClusterBarrier>>& PolicyToBarrierIt : PolicyToBarrierMap)
	{
		PolicyToBarrierIt.Value->OnBarrierTimeout().RemoveAll(this);
	}
}

IDisplayClusterBarrier* FDisplayClusterRenderSyncService::GetBarrierForNode(const FString& NodeId) const
{
	if (const FString* const Policy = NodeToPolicyMap.Find(NodeId))
	{
		if (const TUniquePtr<IDisplayClusterBarrier>* const Barrier = PolicyToBarrierMap.Find(*Policy))
		{
			return Barrier->Get();
		}
	}

	return nullptr;
}

void FDisplayClusterRenderSyncService::UnregisterClusterNode(const FString& NodeId)
{
	if (IDisplayClusterBarrier* Barrier = GetBarrierForNode(NodeId))
	{
		Barrier->UnregisterSyncCaller(NodeId);
	}
}

void FDisplayClusterRenderSyncService::ShutdownImpl()
{
	// Deactivate all barriers
	DeactivateAllBarriers();
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterSessionPacketHandler
//////////////////////////////////////////////////////////////////////////////////////////////
TSharedPtr<FDisplayClusterPacketInternal> FDisplayClusterRenderSyncService::ProcessPacket(const TSharedPtr<FDisplayClusterPacketInternal>& Request, const FDisplayClusterSessionInfo& SessionInfo)
{
	// Check the pointer
	if (!Request.IsValid())
	{
		UE_LOGF(LogDisplayClusterNetwork, Error, "%ls - Invalid request data (nullptr)", *GetName());
		return nullptr;
	}

	// Cache session info
	SetSessionInfoCache(SessionInfo);

	UE_LOGF(LogDisplayClusterNetwork, Verbose, "%ls - Processing packet: %ls", *GetName(), *Request->ToLogString());

	// Check protocol and type
	if (!Request->GetProtocol().Equals(DisplayClusterRenderSyncStrings::ProtocolName, ESearchCase::IgnoreCase) ||
		!Request->GetType().Equals(DisplayClusterRenderSyncStrings::TypeRequest, ESearchCase::IgnoreCase))
	{
		UE_LOGF(LogDisplayClusterNetwork, Error, "%ls - Unsupported packet type: %ls", *GetName(), *Request->ToLogString());
		return nullptr;
	}

	// Initialize response packet
	TSharedPtr<FDisplayClusterPacketInternal> Response = MakeShared<FDisplayClusterPacketInternal>(Request->GetName(), DisplayClusterRenderSyncStrings::TypeResponse, Request->GetProtocol());

	// Dispatch the packet
	if (Request->GetName().Equals(DisplayClusterRenderSyncStrings::SynchronizeOnBarrier::Name, ESearchCase::IgnoreCase))
	{
		// Process command
		const EDisplayClusterCommResult CommResult = SynchronizeOnBarrier();
		Response->SetCommResult(CommResult);

		return Response;
	}

	// Being here means that we have no appropriate dispatch logic for this packet
	UE_LOGF(LogDisplayClusterNetwork, Warning, "%ls - No dispatcher found for packet '%ls'", *GetName(), *Request->GetName());

	return nullptr;
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProtocolRenderSync
//////////////////////////////////////////////////////////////////////////////////////////////
EDisplayClusterCommResult FDisplayClusterRenderSyncService::SynchronizeOnBarrier()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SRV_RS::SyncOnBarrier);

	const FString NodeId = GetSessionInfoCache().NodeId.Get(FString());

	IDisplayClusterBarrier* Barrier = GetBarrierForNode(NodeId);
	if (ensureMsgf(Barrier, TEXT("%s could not find a barrier for node '%s'"), *GetName(), *NodeId))
	{
		Barrier->Wait(NodeId);
	}

	return EDisplayClusterCommResult::Ok;
}
