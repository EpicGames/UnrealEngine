// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Network/Service/DisplayClusterService.h"
#include "Network/Session/IDisplayClusterSessionPacketHandler.h"
#include "Network/Protocol/IDisplayClusterProtocolRenderSync.h"
#include "Network/Packet/DisplayClusterPacketInternal.h"

class IDisplayClusterBarrier;
struct FIPv4Endpoint;


/**
 * Rendering synchronization TCP server
 */
class FDisplayClusterRenderSyncService
	: public    FDisplayClusterService
	, public    IDisplayClusterSessionPacketHandler<FDisplayClusterPacketInternal, true>
	, protected IDisplayClusterProtocolRenderSync
{
public:

	FDisplayClusterRenderSyncService(const FName& InInstanceName);

	virtual ~FDisplayClusterRenderSyncService();

public:

	//~ Begin IDisplayClusterServer
	virtual bool Start(const FString& Address, const uint16 Port) override;
	virtual bool Start(TSharedRef<FDisplayClusterTcpListener>& ExternalListener) override;
	virtual void Shutdown() override;
	virtual FString GetProtocolName() const override;
	virtual void KillSession(const FString& NodeId) override;
	//~ End IDisplayClusterServer

protected:

	/** Creates session instance for this service */
	virtual TSharedPtr<IDisplayClusterSession> CreateSession(FDisplayClusterSessionInfo& SessionInfo) override;

	/** Callback when a session is closed */
	void ProcessSessionClosed(const FDisplayClusterSessionInfo& SessionInfo);

	/** Callback on barrier timeout */
	void ProcessBarrierTimeout(const FString& BarrierName, const TSet<FString>& NodesTimedOut);

protected:

	//~ Begin IDisplayClusterSessionPacketHandler
	virtual TSharedPtr<FDisplayClusterPacketInternal> ProcessPacket(const TSharedPtr<FDisplayClusterPacketInternal>& Request, const FDisplayClusterSessionInfo& SessionInfo) override;
	//~ End IDisplayClusterSessionPacketHandler

private:

	//~ Begin IDisplayClusterProtocolRenderSync
	virtual EDisplayClusterCommResult SynchronizeOnBarrier() override;
	//~ End IDisplayClusterProtocolRenderSync

private:

	/** Performs internal initialization during server start */
	void StartInternal();

	/** Initializes barriers based on available nodes and their sync policies */
	void InitializeBarriers();

	/** Activates all service barriers */
	void ActivateAllBarriers();

	/** Deactivates all service barriers */
	void DeactivateAllBarriers();

	/** Unsubscribe from all events of all internal barriers */
	void UnsubscribeFromAllBarrierEvents();

	/** Returns barrier of a node's sync group */
	IDisplayClusterBarrier* GetBarrierForNode(const FString& NodeId) const;

	/** Unregister cluster node from a barrier */
	void UnregisterClusterNode(const FString& NodeId);

	/** Non-virtual shutdown implementation */
	void ShutdownImpl();

private:

	// We can now use different sync policies within the same cluster. Since each policy may
	// have its own synchronization logic, mainly barriers utilization, we need to have
	// individual sync barriers for every sync group. A sync group is a list of nodes that
	// use the same sync policy.
	TMap<FString, TUniquePtr<IDisplayClusterBarrier>> PolicyToBarrierMap;

	/** Node ID to sync policy ID(or sync group) mapping */
	TMap<FString, FString> NodeToPolicyMap;
};
