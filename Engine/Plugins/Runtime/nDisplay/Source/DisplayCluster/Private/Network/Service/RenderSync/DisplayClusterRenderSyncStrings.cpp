// Copyright Epic Games, Inc. All Rights Reserved.

#include "Network/Service/RenderSync/DisplayClusterRenderSyncStrings.h"


namespace DisplayClusterRenderSyncStrings
{
	const FString ProtocolName = TEXT("RenderSync");

	const FString TypeRequest  = TEXT("Request");
	const FString TypeResponse = TEXT("Response");

	const FString ArgumentsDefaultCategory = TEXT("RS");

	namespace SynchronizeOnBarrier
	{
		const FString Name = TEXT("SyncOnBarrier");
	};
};
