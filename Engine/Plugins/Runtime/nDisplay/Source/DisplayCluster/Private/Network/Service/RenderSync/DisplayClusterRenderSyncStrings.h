// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"


/**
 * Rendering synchronization protocol strings
 */
namespace DisplayClusterRenderSyncStrings
{
	extern const FString ProtocolName;

	extern const FString TypeRequest;
	extern const FString TypeResponse;

	extern const FString ArgumentsDefaultCategory;

	namespace SynchronizeOnBarrier
	{
		extern const FString Name;
	};
};
