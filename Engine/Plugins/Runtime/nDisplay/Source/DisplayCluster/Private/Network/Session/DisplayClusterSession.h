// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericPlatform/GenericPlatformAffinity.h"

#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"

#include "Misc/DisplayClusterLog.h"

#include "Network/DisplayClusterNetworkTypes.h"
#include "Network/IDisplayClusterServer.h"
#include "Network/Configuration/DisplayClusterNetworkConfiguration.h"
#include "Network/Packet/IDisplayClusterPacket.h"
#include "Network/Session/IDisplayClusterSession.h"
#include "Network/Session/IDisplayClusterSessionPacketHandler.h"
#include "Network/Transport/DisplayClusterSocketOperations.h"
#include "Network/Transport/DisplayClusterSocketOperationsHelper.h"


/**
 * Base server socket session class
 */
template <typename TPacketType, bool bIsBidirectional>
class FDisplayClusterSession
	: public    IDisplayClusterSession
	, protected FRunnable
	, protected FDisplayClusterSocketOperations
	, protected FDisplayClusterSocketOperationsHelper<TPacketType>
{
public:
	FDisplayClusterSession(
			const FDisplayClusterSessionInfo& InSessionInfo,
			IDisplayClusterServer& InOwningServer,
			IDisplayClusterSessionPacketHandler<TPacketType, bIsBidirectional>& InPacketHandler,
			EThreadPriority InThreadPriority = EThreadPriority::TPri_Normal)

		: FDisplayClusterSocketOperations(InSessionInfo.Socket, UE::nDisplay::Network::Configuration::PacketBufferSize, InSessionInfo.SessionName)
		, FDisplayClusterSocketOperationsHelper<TPacketType>(*this, InSessionInfo.SessionName)
		, SessionInfo(InSessionInfo)
		, OwningServer(InOwningServer)
		, PacketHandler(InPacketHandler)
		, ThreadPriority(InThreadPriority)
	{
		static_assert(std::is_base_of<IDisplayClusterPacket, TPacketType>::value, "TPacketType is not derived from IDisplayClusterPacket");

		checkSlow(InSessionInfo.Socket);
	}

	virtual ~FDisplayClusterSession()
	{
		Stop();
	}

public:
	//////////////////////////////////////////////////////////////////////////////////////////////
	// IDisplayClusterSession
	//////////////////////////////////////////////////////////////////////////////////////////////
	virtual const FDisplayClusterSessionInfo& GetSessionInfo() const override final
	{
		return SessionInfo;
	}

	virtual bool StartSession() override
	{
		ThreadObj.Reset(FRunnableThread::Create(this, *SessionInfo.SessionName, 1024 * 1024, ThreadPriority, FPlatformAffinity::GetMainGameMask()));
		ensure(ThreadObj);

		return ThreadObj.IsValid();
	}
	
	virtual void StopSession(bool bWaitForCompletion) override
	{
		// Set termination flag
		SessionInfo.bTerminatedByServer = true;

		// Stop working thread
		Stop();

		if (bWaitForCompletion)
		{
			WaitForCompletion();
		}
	}

	virtual void WaitForCompletion() override
	{
		if (ThreadObj)
		{
			ThreadObj->WaitForCompletion();
		}
	}

protected:
	//////////////////////////////////////////////////////////////////////////////////////////////
	// FRunnable
	//////////////////////////////////////////////////////////////////////////////////////////////
	virtual uint32 Run() override
	{
		UE_LOGF(LogDisplayClusterNetwork, Log, "Session thread %ls has started", *SessionInfo.SessionName);

		// Using TLS dramatically speeds up clusters with large numbers of nodes
		FMemory::SetupTLSCachesOnCurrentThread();

		// Set session start time
		SessionInfo.TimeStart = FPlatformTime::Seconds();

		// Notify owner about new session
		GetOwner().NotifySessionOpened(SessionInfo);

		// Process all incoming messages unless server is shutdown or remote host is up
		while (FDisplayClusterSocketOperations::IsOpen())
		{
			// Receive a packet
			TSharedPtr<TPacketType> Request = FDisplayClusterSocketOperationsHelper<TPacketType>::ReceivePacket();
			if (!Request)
			{
				UE_LOGF(LogDisplayClusterNetwork, Warning, "Session %ls: couldn't receive a request packet", *SessionInfo.SessionName);
				break;
			}

			// Processs the request
			typename IDisplayClusterSessionPacketHandler<TPacketType, bIsBidirectional>::ReturnType Response = GetPacketHandler().ProcessPacket(Request, GetSessionInfo());
			
			// Send a response (or not, it depends on the connection type)
			const bool bResult = HandleSendResponse(Response);
			if (!bResult)
			{
				UE_LOGF(LogDisplayClusterNetwork, Warning, "Session %ls: couldn't send a response packet", *SessionInfo.SessionName);
				break;
			}

			UE_LOGF(LogDisplayClusterNetwork, Verbose, "Session %ls has processed a packet", *SessionInfo.SessionName);
		}

		// Set session end time
		SessionInfo.TimeEnd = FPlatformTime::Seconds();

		UE_LOGF(LogDisplayClusterNetwork, Log, "Session thread %ls has finished", *SessionInfo.SessionName);

		// Since we left the cycle above, it means the session has been closed. We need to notify the owning server about that.
		GetOwner().NotifySessionClosed(SessionInfo);

		return 0;
	}

	virtual void Stop() override final
	{
		// Close the socket so the working thread will detect socket error and stop working
		CloseSocket();
	}

protected:
	IDisplayClusterServer& GetOwner() const
	{
		return OwningServer;
	}

	IDisplayClusterSessionPacketHandler<TPacketType, bIsBidirectional>& GetPacketHandler() const
	{
		return PacketHandler;
	}

private:
	template<typename TResponseType>
	bool HandleSendResponse(const TResponseType& Response)
	{
		unimplemented();
		return false;
	}

	template<>
	bool HandleSendResponse<typename IDisplayClusterSessionPacketHandler<TPacketType, false>::ReturnType>(const typename IDisplayClusterSessionPacketHandler<TPacketType, false>::ReturnType& Response)
	{
		// Nothing to do, no responses for unidirectional services
		return true;
	}

	template<>
	bool HandleSendResponse<typename IDisplayClusterSessionPacketHandler<TPacketType, true>::ReturnType>(const typename IDisplayClusterSessionPacketHandler<TPacketType, true>::ReturnType& Response)
	{
		if (Response.IsValid())
		{
			return FDisplayClusterSocketOperationsHelper<TPacketType>::SendPacket(Response);
		}

		return false;
	}

private:
	// Session info
	FDisplayClusterSessionInfo SessionInfo;

	// Session owner
	IDisplayClusterServer& OwningServer;
	// Session packets processor
	IDisplayClusterSessionPacketHandler<TPacketType, bIsBidirectional>& PacketHandler = nullptr;

	// Working thread priority
	const EThreadPriority ThreadPriority;

	// Session working thread
	TUniquePtr<FRunnableThread> ThreadObj;
};
