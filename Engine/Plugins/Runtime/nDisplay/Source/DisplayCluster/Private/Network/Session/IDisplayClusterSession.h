// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

struct FDisplayClusterSessionInfo;


/**
 * Session interface
 */
class IDisplayClusterSession
{
public:
	virtual ~IDisplayClusterSession() = default;

public:
	// Start processing of incoming events
	virtual bool StartSession() = 0;
	// Stop  processing of incoming events
	virtual void StopSession(bool bWaitForCompletion) = 0;
	// Wait unless session conmpletely stopped and the working thread is release
	virtual void WaitForCompletion() = 0;

	// Session info
	virtual const FDisplayClusterSessionInfo& GetSessionInfo() const = 0;
};
