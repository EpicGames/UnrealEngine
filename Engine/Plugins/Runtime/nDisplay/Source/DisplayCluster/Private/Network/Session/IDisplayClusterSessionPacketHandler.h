// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointerFwd.h"
#include <type_traits>

struct FDisplayClusterSessionInfo;

/**
 * Packet handler interface for all incoming packets
 */
template <typename TPacketType, bool bIsBidirectional>
class IDisplayClusterSessionPacketHandler
{
public:
	virtual ~IDisplayClusterSessionPacketHandler() = default;

public:
	// Kind of hack to avoid any problems with 'void' in the templates
	typedef struct{} NoResponse;
	// Different return types for unidirectional and bidirectional communication
	typedef typename std::conditional<bIsBidirectional, TSharedPtr<TPacketType>, NoResponse>::type ReturnType;

	// Process incoming packet and return response packet
	virtual typename IDisplayClusterSessionPacketHandler<TPacketType, bIsBidirectional>::ReturnType ProcessPacket(const TSharedPtr<TPacketType>& Request, const FDisplayClusterSessionInfo& SessionInfo) = 0;
};
