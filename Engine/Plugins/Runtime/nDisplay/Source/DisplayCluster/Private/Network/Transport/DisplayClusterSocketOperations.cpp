// Copyright Epic Games, Inc. All Rights Reserved.

#include "Network/Transport/DisplayClusterSocketOperations.h"

#include "Misc/DisplayClusterLog.h"
#include "Misc/ScopeLock.h"

#include "SocketSubsystem.h"


FDisplayClusterSocketOperations::FDisplayClusterSocketOperations(FSocket* InSocket, int32 PersistentBufferSize, const FString& InConnectionName, bool bReleaseSocketOnDestroy)
	: Socket(InSocket)
	, bReleaseSocket(bReleaseSocketOnDestroy)
	, ConnectionName(InConnectionName)
{
	checkSlow(InSocket);
	DataBuffer.AddUninitialized(PersistentBufferSize);
}


FDisplayClusterSocketOperations::~FDisplayClusterSocketOperations()
{
	// Release the socket only if requested on creation
	if (bReleaseSocket && Socket)
	{
		CloseSocket();
		ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->DestroySocket(Socket);
		Socket = nullptr;
	}
}

bool FDisplayClusterSocketOperations::RecvChunk(FSocket* const InSocket, TArray<uint8>& ChunkBuffer, const uint32 ChunkSize, const FString& ChunkName)
{
	checkSlow(InSocket);

	uint32 BytesRecvTotal = 0;
	int32  BytesRecvPass = 0;
	uint32 BytesRecvLeft = 0;

	// Make sure there is enough space for incoming data
	ChunkBuffer.Reset();
	ChunkBuffer.AddUninitialized(ChunkSize);

	// Read all requested bytes
	while (BytesRecvTotal < ChunkSize)
	{
		// Read data
		BytesRecvLeft = ChunkSize - BytesRecvTotal;
		if (!InSocket->Recv(ChunkBuffer.GetData() + BytesRecvTotal, BytesRecvLeft, BytesRecvPass))
		{
			UE_LOGF(LogDisplayClusterNetwork, Log, "%ls - %ls recv failed. It seems the client has disconnected.", *InSocket->GetDescription(), *ChunkName);
			return false;
		}

		// Check amount of read data
		if (BytesRecvPass <= 0 || static_cast<uint32>(BytesRecvPass) > BytesRecvLeft)
		{
			UE_LOGF(LogDisplayClusterNetwork, Error, "%ls - %ls recv failed - read wrong amount of bytes: %d", *InSocket->GetDescription(), *ChunkName, BytesRecvPass);
			return false;
		}

		BytesRecvTotal += BytesRecvPass;
		UE_LOGF(LogDisplayClusterNetwork, VeryVerbose, "%ls - %ls received %d bytes, left %u bytes", *InSocket->GetDescription(), *ChunkName, BytesRecvPass, ChunkSize - BytesRecvTotal);
	}

	// Operation succeeded
	return true;
}

bool FDisplayClusterSocketOperations::SendChunk(FSocket* const InSocket, const TArray<uint8>& ChunkBuffer, const uint32 ChunkSize, const FString& ChunkName)
{
	checkSlow(InSocket);

	uint32 BytesSentTotal = 0;
	int32  BytesSentNow = 0;
	uint32 BytesSendLeft = 0;

	// Write all bytes
	while (BytesSentTotal < ChunkSize)
	{
		BytesSendLeft = ChunkSize - BytesSentTotal;

		// Send data
		if (!InSocket->Send(ChunkBuffer.GetData() + BytesSentTotal, BytesSendLeft, BytesSentNow))
		{
			UE_LOGF(LogDisplayClusterNetwork, Log, "%ls - %ls send failed (length=%d)", *InSocket->GetDescription(), *ChunkName, ChunkSize);
			return false;
		}

		// Check amount of bytes sent
		if (BytesSentNow <= 0 || static_cast<uint32>(BytesSentNow) > BytesSendLeft)
		{
			UE_LOGF(LogDisplayClusterNetwork, Error, "%ls - %ls send failed: %d of %u left", *InSocket->GetDescription(), *ChunkName, BytesSentNow, BytesSendLeft);
			return false;
		}

		BytesSentTotal += BytesSentNow;
		UE_LOGF(LogDisplayClusterNetwork, VeryVerbose, "%ls - %ls sent %d bytes, %u bytes left", *InSocket->GetDescription(), *ChunkName, BytesSentNow, ChunkSize - BytesSentTotal);
	}

	UE_LOGF(LogDisplayClusterNetwork, Verbose, "%ls - %ls was sent", *InSocket->GetDescription(), *ChunkName);

	return true;
}

bool FDisplayClusterSocketOperations::RecvChunk(TArray<uint8>& ChunkBuffer, const uint32 ChunkSize, const FString& ChunkName)
{
	FScopeLock Lock(&CritSecInternals);
	return FDisplayClusterSocketOperations::RecvChunk(Socket, ChunkBuffer, ChunkSize, ChunkName);
}

bool FDisplayClusterSocketOperations::SendChunk(const TArray<uint8>& ChunkBuffer, const uint32 ChunkSize, const FString& ChunkName)
{
	FScopeLock Lock(&CritSecInternals);
	return FDisplayClusterSocketOperations::SendChunk(Socket, ChunkBuffer, ChunkSize, ChunkName);
}
