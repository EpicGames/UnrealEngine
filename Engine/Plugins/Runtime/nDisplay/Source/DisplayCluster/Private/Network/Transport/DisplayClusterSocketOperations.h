// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sockets.h"


/**
 * Low-level socket operations
 */
class FDisplayClusterSocketOperations
{
public:
	FDisplayClusterSocketOperations(FSocket* Socket, int32 PersistentBufferSize, const FString& ConnectionName, bool bReleaseSocketOnDestroy = true);
	virtual ~FDisplayClusterSocketOperations();

public:
	// Returns true if the socket is valid and connected
	inline bool IsOpen() const
	{
		return (Socket && (Socket->GetConnectionState() == ESocketConnectionState::SCS_Connected));
	}

	// Access to the socket synchronization object
	inline FCriticalSection& GetSyncObj()
	{
		return CritSecInternals;
	}

	// Connect
	inline bool ConnectSocket(const FInternetAddr& Addr)
	{
		// if(!IsOpen()) check was removed from here because of an issue in BSD sockets.
		if (Socket)
		{
			return Socket->Connect(Addr);
		}

		return false;
	}

	// Close socket
	inline void CloseSocket()
	{
		if (Socket)
		{
			Socket->Close();
		}
	}

	// Access to the connection name
	inline const FString& GetConnectionName() const
	{
		return ConnectionName;
	}

	// Access to the internal read/write buffer
	inline TArray<uint8>& GetPersistentBuffer()
	{
		return DataBuffer;
	}

public:
	// Receive specified amount of bytes to custom buffer (external socket object)
	static bool RecvChunk(FSocket* const InSocket, TArray<uint8>& ChunkBuffer, const uint32 ChunkSize, const FString& ChunkName = FString("ReadDataChunk"));
	// Send specified amount of bytes from specified buffer (external socket object)
	static bool SendChunk(FSocket* const InSocket, const TArray<uint8>& ChunkBuffer, const uint32 ChunkSize, const FString& ChunkName = FString("WriteDataChunk"));

	// Receive specified amount of bytes to custom buffer (internal socket object)
	bool RecvChunk(TArray<uint8>& ChunkBuffer, const uint32 ChunkSize, const FString& ChunkName = FString("ReadDataChunk"));
	// Send specified amount of bytes from specified buffer (internal socket object)
	bool SendChunk(const TArray<uint8>& ChankBuffer, const uint32 ChunkSize, const FString& ChunkName = FString("WriteDataChunk"));

private:
	// Socket
	FSocket* Socket = nullptr;
	// Should the socket be released in .dtor
	bool bReleaseSocket = false;
	// Data buffer
	TArray<uint8> DataBuffer;
	// Connection name (basically for nice logging)
	FString ConnectionName;
	// Sync access to internals
	mutable FCriticalSection CritSecInternals;
};
