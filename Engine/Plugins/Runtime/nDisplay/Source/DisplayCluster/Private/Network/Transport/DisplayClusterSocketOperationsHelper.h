// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/DisplayClusterLog.h"

#include "Network/Transport/DisplayClusterSocketOperations.h"
#include "Network/Packet/IDisplayClusterPacket.h"


/**
 * Socket operations helper. High level operations with specific packet types.
 */
template <typename TPacketType>
class FDisplayClusterSocketOperationsHelper
{
	static_assert(std::is_base_of<IDisplayClusterPacket, TPacketType>::value, "TPacketType is not derived from IDisplayClusterPacket");

public:

	FDisplayClusterSocketOperationsHelper(FDisplayClusterSocketOperations& InSocketOps, const FString& InLogHeader = FString())
		: SocketOps(InSocketOps)
		, LogHeader(InLogHeader)
	{ }

	virtual ~FDisplayClusterSocketOperationsHelper()
	{ }

public:

	TSharedPtr<TPacketType> SendRecvPacket(const TSharedPtr<TPacketType>& Request)
	{
		if (Request)
		{
			if (SendPacket(Request))
			{
				return ReceivePacket();
			}
		}

		return nullptr;
	}

	bool SendPacket(const TSharedPtr<IDisplayClusterPacket>& Packet)
	{
		if (Packet)
		{
			constexpr bool bDetailedLog = true;

			UE_LOGF(LogDisplayClusterNetwork, VeryVerbose, "%ls: sending packet - %ls",
				LogHeader.IsEmpty() ? *SocketOps.GetConnectionName() : *LogHeader,
				*Packet->ToLogString(bDetailedLog));

			return Packet->SendPacket(SocketOps);
		}

		return false;
	}

	bool SendPacket(const TSharedPtr<TPacketType>& Packet)
	{
		if (Packet)
		{
			return SendPacket(StaticCastSharedPtr<IDisplayClusterPacket>(Packet));
		}

		return false;
	}

	TSharedPtr<TPacketType> ReceivePacket()
	{
		TSharedPtr<TPacketType> Packet = MakeShared<TPacketType>();
		if (StaticCastSharedPtr<IDisplayClusterPacket>(Packet)->RecvPacket(SocketOps))
		{
			constexpr bool bDetailedLog = true;

			UE_LOGF(LogDisplayClusterNetwork, VeryVerbose, "%ls: received packet - %ls",
				LogHeader.IsEmpty() ? *SocketOps.GetConnectionName() : *LogHeader,
				*StaticCastSharedPtr<IDisplayClusterPacket>(Packet)->ToLogString(bDetailedLog));

			return Packet;
		}

		return nullptr;
	}

private:
	FDisplayClusterSocketOperations& SocketOps;
	FString LogHeader;
};
