// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Containers/DisplayClusterRender_MeshComponent.h"
#include "Render/Containers/DisplayClusterRender_MeshComponentProxy.h"
#include "Render/Containers/DisplayClusterRender_MeshComponentProxyData.h"
#include "Misc/DisplayClusterLog.h"

#include "Engine/StaticMesh.h"
#include "ProceduralMeshComponent.h"
#include "RenderingThread.h"
#include "StaticMeshResources.h"

namespace UE::DisplayCluster::MeshComponent
{
static void ImplUpdateMeshComponentProxyData(const TSharedPtr<FDisplayClusterRender_MeshComponentProxy, ESPMode::ThreadSafe>& InMeshComponentProxy, FDisplayClusterRender_MeshComponentProxyData* NewProxyData)
{
	if (InMeshComponentProxy.IsValid())
	{
		ENQUEUE_RENDER_COMMAND(DisplayClusterRender_MeshComponentProxy_Update)(
			[MeshComponentProxy = InMeshComponentProxy, NewProxyData](FRHICommandListImmediate& RHICmdList)
		{
			// Update RHI
			if (NewProxyData != nullptr)
			{
				MeshComponentProxy->UpdateRHI_RenderThread(RHICmdList, NewProxyData);
				delete NewProxyData;
			}
			else
			{
				MeshComponentProxy->Release_RenderThread();
			}
		});
	}
}

static void ImplDeleteMeshComponentProxy(const TSharedPtr<FDisplayClusterRender_MeshComponentProxy, ESPMode::ThreadSafe>& InMeshComponentProxy)
{
	if (InMeshComponentProxy.IsValid())
	{
		ENQUEUE_RENDER_COMMAND(DisplayClusterRender_MeshComponentProxy_Delete)(
			[MeshComponentProxy = InMeshComponentProxy](FRHICommandListImmediate& RHICmdList)
			{
				MeshComponentProxy->Release_RenderThread();
			});
	}
}

/** Returns true the first time InStaticMesh name is seen. */
static bool CanShowMsgOnce(const UStaticMesh* InStaticMesh)
{
	// Show log for mesh once.
	static TArray<FString> MeshNamesInLog;

	const FString InStaticMeshName = InStaticMesh->GetName();
	if (MeshNamesInLog.Find(InStaticMeshName) == INDEX_NONE)
	{
		MeshNamesInLog.Add(InStaticMeshName);

		return true;
	}

	return false;
}

static const FStaticMeshLODResources* ImplGetStaticMeshLODResources(const UStaticMesh* InStaticMesh, int32 LODIndex = 0)
{
	if (InStaticMesh != nullptr)
	{
		if (InStaticMesh->bAllowCPUAccess == false)
		{
			if (CanShowMsgOnce(InStaticMesh))
			{
				UE_LOGF(LogDisplayClusterRender, Warning, "If packaging this project, static mesh '%ls' requires its AllowCPUAccess flag to be enabled.", *InStaticMesh->GetName());
			}
#if !WITH_EDITOR
			// Can't access to cooked data from CPU without this flag
			return nullptr;
#endif
		}

		// GetLODForExport() asserts on null RenderData and indexes LODResources unchecked.
		const FStaticMeshRenderData* RenderData = InStaticMesh->GetRenderData();
		if (RenderData && !RenderData->LODResources.IsEmpty())
		{
			const FStaticMeshLODResources& StaticMeshLODResources = InStaticMesh->GetLODForExport(LODIndex);

			return &StaticMeshLODResources;
		}
		else if (CanShowMsgOnce(InStaticMesh))
		{
			UE_LOGF(LogDisplayClusterRender, Warning, "Skipping static mesh '%ls': render data is unavailable (mesh may be unbuilt, cooked-out, or still streaming).", *InStaticMesh->GetName());
		}
	}

	return nullptr;
}

static const FStaticMeshLODResources* ImplGetStaticMeshComponentLODResources(const UStaticMeshComponent* InStaticMeshComponent, int32 LODIndex = 0)
{
	if (InStaticMeshComponent != nullptr)
	{
		return ImplGetStaticMeshLODResources(InStaticMeshComponent->GetStaticMesh(), LODIndex);
	}

	return nullptr;
}

static const FProcMeshSection* ImplGetProceduralMeshComponentSection(UProceduralMeshComponent* InProceduralMeshComponent, const int32 InSectionIndex)
{
	if (InProceduralMeshComponent != nullptr && InSectionIndex >= 0)
	{
		return InProceduralMeshComponent->GetProcMeshSection(InSectionIndex);
	}

	return nullptr;
}
}; // namespace UE::DisplayCluster::MeshComponent

//-------------------------------------------------------------------------------
//        FDisplayClusterRender_MeshComponent
//-------------------------------------------------------------------------------
FDisplayClusterRender_MeshComponent::FDisplayClusterRender_MeshComponent()
{
	// Create render proxy object
	MeshComponentProxyPtr = MakeShared<FDisplayClusterRender_MeshComponentProxy, ESPMode::ThreadSafe>();
}

FDisplayClusterRender_MeshComponent::~FDisplayClusterRender_MeshComponent()
{
	using namespace UE::DisplayCluster::MeshComponent;

	// Delete proxy on RenderThread
	ImplDeleteMeshComponentProxy(MeshComponentProxyPtr);

	// Forget Ptr on GameThread
	MeshComponentProxyPtr.Reset();
}

USceneComponent* FDisplayClusterRender_MeshComponent::GetOriginComponent() const
{
	check(IsInGameThread());

	return OriginComponentRef.GetOrFindSceneComponent();
}

void FDisplayClusterRender_MeshComponent::SetGeometryFunc(const EDisplayClusterRender_MeshComponentProxyDataFunc InDataFunc)
{
	check(IsInGameThread());

	DataFunc = InDataFunc;
}

IDisplayClusterRender_MeshComponentProxy* FDisplayClusterRender_MeshComponent::GetMeshComponentProxy_RenderThread() const
{
	check(IsInRenderingThread());

	return MeshComponentProxyPtr.Get();
}

UMeshComponent* FDisplayClusterRender_MeshComponent::GetMeshComponent() const
{
	switch (GetGeometrySource())
	{
	case EDisplayClusterRender_MeshComponentGeometrySource::StaticMeshComponentRef:
		return GetStaticMeshComponent();

	case EDisplayClusterRender_MeshComponentGeometrySource::ProceduralMeshComponentRef:
		return GetProceduralMeshComponent();

	default:
		break;
	}

	return nullptr;
}

UStaticMeshComponent* FDisplayClusterRender_MeshComponent::GetStaticMeshComponent() const
{
	check(IsInGameThread());

	if(GeometrySource == EDisplayClusterRender_MeshComponentGeometrySource::StaticMeshComponentRef)
	{
		return StaticMeshComponentRef.GetOrFindStaticMeshComponent();
	}

	return nullptr;
}

const UStaticMesh* FDisplayClusterRender_MeshComponent::GetStaticMesh() const
{
	if (StaticMeshRef.IsValid() && !StaticMeshRef.IsStale())
	{
		return StaticMeshRef.Get();
	}

	return nullptr;
}

const FStaticMeshLODResources* FDisplayClusterRender_MeshComponent::GetStaticMeshComponentLODResources(int32 InLODIndex) const
{
	check(IsInGameThread());
	using namespace UE::DisplayCluster::MeshComponent;

	if(GeometrySource == EDisplayClusterRender_MeshComponentGeometrySource::StaticMeshComponentRef)
	{
		return ImplGetStaticMeshComponentLODResources(StaticMeshComponentRef.GetOrFindStaticMeshComponent(), InLODIndex);
	}

	return nullptr;
}

void FDisplayClusterRender_MeshComponent::AssignStaticMeshComponentRefs(UStaticMeshComponent* InStaticMeshComponent, const FDisplayClusterMeshUVs& InUVs, USceneComponent* InOriginComponent, int32 InLODIndex)
{
	check(IsInGameThread());
	using namespace UE::DisplayCluster::MeshComponent;

	// Set component refs
	OriginComponentRef.SetSceneComponent(InOriginComponent);
	StaticMeshComponentRef.SetStaticMeshComponentRef(InStaticMeshComponent);
	ProceduralMeshComponentRef.ResetProceduralMeshComponentRef();

	StaticMeshRef.Reset();

	// Set source geometry type
	GeometrySource = EDisplayClusterRender_MeshComponentGeometrySource::StaticMeshComponentRef;

	// Get geometry data
	const FStaticMeshLODResources* StaticMeshLODResources = ImplGetStaticMeshComponentLODResources(InStaticMeshComponent, InLODIndex);
	if (StaticMeshLODResources != nullptr)
	{
		const FString SourceGeometryName = InStaticMeshComponent->GetFName().ToString();
		// Send geometry to proxy
		ImplUpdateMeshComponentProxyData(MeshComponentProxyPtr, new FDisplayClusterRender_MeshComponentProxyData(SourceGeometryName, DataFunc, *StaticMeshLODResources, InUVs));
	}
	else
	{
		// no StaticMesh - release proxy geometry
		ImplUpdateMeshComponentProxyData(MeshComponentProxyPtr, nullptr);
	}
}

UProceduralMeshComponent* FDisplayClusterRender_MeshComponent::GetProceduralMeshComponent() const
{
	check(IsInGameThread());

	if(GeometrySource == EDisplayClusterRender_MeshComponentGeometrySource::ProceduralMeshComponentRef)
	{
		return ProceduralMeshComponentRef.GetOrFindProceduralMeshComponent();
	}

	return nullptr;
}

const FProcMeshSection* FDisplayClusterRender_MeshComponent::GetProceduralMeshComponentSection(const int32 InSectionIndex) const
{
	check(IsInGameThread());
	using namespace UE::DisplayCluster::MeshComponent;

	if(GeometrySource == EDisplayClusterRender_MeshComponentGeometrySource::ProceduralMeshComponentRef)
	{
		return ImplGetProceduralMeshComponentSection(ProceduralMeshComponentRef.GetOrFindProceduralMeshComponent(), InSectionIndex);
	}

	return nullptr;
}

void FDisplayClusterRender_MeshComponent::AssignProceduralMeshComponentRefs(UProceduralMeshComponent* InProceduralMeshComponent, const FDisplayClusterMeshUVs& InUVs, USceneComponent* InOriginComponent, const int32 InSectionIndex)
{
	check(IsInGameThread());
	using namespace UE::DisplayCluster::MeshComponent;

	// Set component refs
	OriginComponentRef.SetSceneComponent(InOriginComponent);
	StaticMeshComponentRef.ResetStaticMeshComponentRef();
	ProceduralMeshComponentRef.SetProceduralMeshComponentRef(InProceduralMeshComponent);

	StaticMeshRef.Reset();

	// Set source geometry type
	GeometrySource = EDisplayClusterRender_MeshComponentGeometrySource::ProceduralMeshComponentRef;

	// Get geometry data
	const FProcMeshSection* ProcMeshSection = ImplGetProceduralMeshComponentSection(InProceduralMeshComponent, InSectionIndex);
	if (ProcMeshSection != nullptr)
	{
		const FString SourceGeometryName = InProceduralMeshComponent->GetFName().ToString();
		// Send geometry to proxy
		ImplUpdateMeshComponentProxyData(MeshComponentProxyPtr, new FDisplayClusterRender_MeshComponentProxyData(SourceGeometryName, DataFunc, *ProcMeshSection, InUVs));
	}
	else
	{
		// no ProceduralMesh - release proxy geometry
		ImplUpdateMeshComponentProxyData(MeshComponentProxyPtr, nullptr);
	}
}

void FDisplayClusterRender_MeshComponent::AssignProceduralMeshSection(const FProcMeshSection& InProcMeshSection, const FDisplayClusterMeshUVs& InUVs)
{
	check(IsInGameThread());
	using namespace UE::DisplayCluster::MeshComponent;

	// Reset component refs
	OriginComponentRef.ResetSceneComponent();
	StaticMeshComponentRef.ResetStaticMeshComponentRef();
	ProceduralMeshComponentRef.ResetProceduralMeshComponentRef();

	StaticMeshRef.Reset();

	// Set source geometry type
	GeometrySource = EDisplayClusterRender_MeshComponentGeometrySource::ProceduralMeshSection;

	const FString SourceGeometryName(TEXT("ProcMeshSection"));
	// Send geometry to proxy
	ImplUpdateMeshComponentProxyData(MeshComponentProxyPtr, new FDisplayClusterRender_MeshComponentProxyData(SourceGeometryName, DataFunc, InProcMeshSection, InUVs));
}

void FDisplayClusterRender_MeshComponent::AssignStaticMesh(const UStaticMesh* InStaticMesh, const FDisplayClusterMeshUVs& InUVs, int32 InLODIndex)
{
	check(IsInGameThread());
	using namespace UE::DisplayCluster::MeshComponent;

	// Reset component refs
	OriginComponentRef.ResetSceneComponent();
	StaticMeshComponentRef.ResetStaticMeshComponentRef();
	ProceduralMeshComponentRef.ResetProceduralMeshComponentRef();

	StaticMeshRef = InStaticMesh;

	// Set source geometry type
	GeometrySource = EDisplayClusterRender_MeshComponentGeometrySource::StaticMeshAsset;

	// Get geometry data
	const FStaticMeshLODResources* StaticMeshLODResources = ImplGetStaticMeshLODResources(InStaticMesh, InLODIndex);
	if (StaticMeshLODResources != nullptr)
	{
		const FString SourceGeometryName = InStaticMesh->GetFName().ToString();
		// Send geometry to proxy
		ImplUpdateMeshComponentProxyData(MeshComponentProxyPtr, new FDisplayClusterRender_MeshComponentProxyData(SourceGeometryName, DataFunc, *StaticMeshLODResources, InUVs));
	}
	else
	{
		// no StaticMesh - release proxy geometry
		ImplUpdateMeshComponentProxyData(MeshComponentProxyPtr, nullptr);
	}
}

void FDisplayClusterRender_MeshComponent::AssignMeshGeometry(const FDisplayClusterRender_MeshGeometry* InMeshGeometry)
{
	check(IsInGameThread());
	using namespace UE::DisplayCluster::MeshComponent;

	// Reset component refs
	OriginComponentRef.ResetSceneComponent();
	StaticMeshComponentRef.ResetStaticMeshComponentRef();
	ProceduralMeshComponentRef.ResetProceduralMeshComponentRef();

	StaticMeshRef.Reset();

	// Set source geometry type
	GeometrySource = EDisplayClusterRender_MeshComponentGeometrySource::MeshGeometry;

	if (InMeshGeometry != nullptr)
	{
		const FString SourceGeometryName(TEXT("nDCRender_MeshGeometry"));
		// Send geometry to proxy
		ImplUpdateMeshComponentProxyData(MeshComponentProxyPtr, new FDisplayClusterRender_MeshComponentProxyData(SourceGeometryName, DataFunc, *InMeshGeometry));
	}
	else
	{
		// no InMeshGeometry- release proxy geometry
		ImplUpdateMeshComponentProxyData(MeshComponentProxyPtr, nullptr);
	}
}

void FDisplayClusterRender_MeshComponent::AssignMeshGeometry_RenderThread(const FDisplayClusterRender_MeshGeometry* InMeshGeometry, const EDisplayClusterRender_MeshComponentProxyDataFunc InDataFunc) const
{
	check(IsInRenderingThread());
	using namespace UE::DisplayCluster::MeshComponent;

	if (InMeshGeometry != nullptr)
	{
		const FString SourceGeometryName(TEXT("nDCRender_MeshGeometry_RenderThread"));
		// Send geometry to proxy
		ImplUpdateMeshComponentProxyData(MeshComponentProxyPtr, new FDisplayClusterRender_MeshComponentProxyData(SourceGeometryName, InDataFunc, *InMeshGeometry));
	}
}

void FDisplayClusterRender_MeshComponent::ReleaseMeshComponent()
{
	check(IsInGameThread());
	using namespace UE::DisplayCluster::MeshComponent;

	// Reset component refs
	OriginComponentRef.ResetSceneComponent();
	StaticMeshComponentRef.ResetStaticMeshComponentRef();
	ProceduralMeshComponentRef.ResetProceduralMeshComponentRef();

	StaticMeshRef.Reset();

	// Set source geometry type
	GeometrySource = EDisplayClusterRender_MeshComponentGeometrySource::Disabled;

	ImplUpdateMeshComponentProxyData(MeshComponentProxyPtr, nullptr);
}

void FDisplayClusterRender_MeshComponent::ReleaseProxyGeometry()
{
	check(IsInGameThread());
	using namespace UE::DisplayCluster::MeshComponent;

	// just release  geometry on proxy
	ImplUpdateMeshComponentProxyData(MeshComponentProxyPtr, nullptr);
}

bool FDisplayClusterRender_MeshComponent::EqualsMeshComponentName(const FName& InMeshComponentName) const
{
	check(IsInGameThread());

	switch (GeometrySource)
	{
	case EDisplayClusterRender_MeshComponentGeometrySource::StaticMeshComponentRef:
		return StaticMeshComponentRef.IsEqualsComponentName(InMeshComponentName);
	case EDisplayClusterRender_MeshComponentGeometrySource::ProceduralMeshComponentRef:
		return ProceduralMeshComponentRef.IsEqualsComponentName(InMeshComponentName);
	default:
		break;
	}

	return false;
}

bool FDisplayClusterRender_MeshComponent::IsMeshComponentRefGeometryDirty() const
{
	check(IsInGameThread());

	switch (GeometrySource)
	{
	case EDisplayClusterRender_MeshComponentGeometrySource::StaticMeshComponentRef:
		return StaticMeshComponentRef.IsStaticMeshGeometryDirty();
	case EDisplayClusterRender_MeshComponentGeometrySource::ProceduralMeshComponentRef:
		return ProceduralMeshComponentRef.IsProceduralMeshGeometryDirty();
	default:
		break;
	}

	return false;
}

void FDisplayClusterRender_MeshComponent::MarkMeshComponentRefGeometryDirty() const
{
	check(IsInGameThread());

	switch (GeometrySource)
	{
	case EDisplayClusterRender_MeshComponentGeometrySource::StaticMeshComponentRef:
		StaticMeshComponentRef.MarkStaticMeshGeometryDirty();
		break;
	case EDisplayClusterRender_MeshComponentGeometrySource::ProceduralMeshComponentRef:
		ProceduralMeshComponentRef.MarkProceduralMeshGeometryDirty();
		break;
	default:
		break;
	}
}

void FDisplayClusterRender_MeshComponent::ResetMeshComponentRefGeometryDirty() const
{
	check(IsInGameThread());

	switch (GeometrySource)
	{
	case EDisplayClusterRender_MeshComponentGeometrySource::StaticMeshComponentRef:
		StaticMeshComponentRef.ResetStaticMeshGeometryDirty();
		break;
	case EDisplayClusterRender_MeshComponentGeometrySource::ProceduralMeshComponentRef:
		ProceduralMeshComponentRef.ResetProceduralMeshGeometryDirty();
		break;
	default:
		break;
	}
}

EDisplayClusterRender_MeshComponentGeometrySource FDisplayClusterRender_MeshComponent::GetGeometrySource() const
{
	check(IsInGameThread());

	return GeometrySource;
}
