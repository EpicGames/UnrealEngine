// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Render/Containers/IDisplayClusterRender_MeshComponent.h"

#include "Render/Containers/DisplayClusterRender_StaticMeshComponentRef.h"
#include "Render/Containers/DisplayClusterRender_ProceduralMeshComponentRef.h"

class FDisplayClusterRender_MeshComponentProxy;

class FDisplayClusterRender_MeshComponent
	: public IDisplayClusterRender_MeshComponent
{
public:
	FDisplayClusterRender_MeshComponent();
	virtual ~FDisplayClusterRender_MeshComponent();

public:
	virtual void AssignStaticMeshComponentRefs(UStaticMeshComponent* InStaticMeshComponent, const FDisplayClusterMeshUVs& InUVs, USceneComponent* InOriginComponent = nullptr, int32 InLODIndex = 0) override;
	virtual void AssignProceduralMeshComponentRefs(UProceduralMeshComponent* InProceduralMeshComponent, const FDisplayClusterMeshUVs& InUVs, USceneComponent* InOriginComponent = nullptr, const int32 InSectionIndex = 0) override;
	virtual void AssignProceduralMeshSection(const FProcMeshSection& InProcMeshSection, const FDisplayClusterMeshUVs& InUVs) override;
	virtual void AssignStaticMesh(const UStaticMesh* InStaticMesh, const FDisplayClusterMeshUVs& InUVs, int32 InLODIndex = 0) override;
	virtual void AssignMeshGeometry(const FDisplayClusterRender_MeshGeometry* InMeshGeometry) override;
	
	virtual void AssignMeshGeometry_RenderThread(const FDisplayClusterRender_MeshGeometry* InMeshGeometry, const EDisplayClusterRender_MeshComponentProxyDataFunc InDataFunc) const override;

	virtual void ReleaseMeshComponent() override;
	virtual void ReleaseProxyGeometry() override;

	virtual USceneComponent* GetOriginComponent() const override;
	virtual UMeshComponent* GetMeshComponent() const override;
	virtual UStaticMeshComponent* GetStaticMeshComponent() const override;
	virtual const UStaticMesh* GetStaticMesh() const override;
	virtual UProceduralMeshComponent* GetProceduralMeshComponent() const override;
	virtual const FStaticMeshLODResources* GetStaticMeshComponentLODResources(int32 InLODIndex = 0) const override;
	virtual const FProcMeshSection* GetProceduralMeshComponentSection(const int32 SectionIndex = 0) const override;

	virtual IDisplayClusterRender_MeshComponentProxy* GetMeshComponentProxy_RenderThread() const override;

	virtual void SetGeometryFunc(const EDisplayClusterRender_MeshComponentProxyDataFunc InDataFunc) override;

	virtual bool IsMeshComponentRefGeometryDirty() const override;
	virtual void MarkMeshComponentRefGeometryDirty() const override;
	virtual void ResetMeshComponentRefGeometryDirty() const override;
	virtual bool EqualsMeshComponentName(const FName& InMeshComponentName) const override;

	virtual EDisplayClusterRender_MeshComponentGeometrySource GetGeometrySource() const override;

private:
	EDisplayClusterRender_MeshComponentGeometrySource GeometrySource = EDisplayClusterRender_MeshComponentGeometrySource::Disabled;
	EDisplayClusterRender_MeshComponentProxyDataFunc  DataFunc = EDisplayClusterRender_MeshComponentProxyDataFunc::Disabled;

	// Reference containers:
	FDisplayClusterSceneComponentRef                 OriginComponentRef;
	FDisplayClusterRender_StaticMeshComponentRef     StaticMeshComponentRef;
	FDisplayClusterRender_ProceduralMeshComponentRef ProceduralMeshComponentRef;

	// store ref to the static mesh
	TWeakObjectPtr<const UStaticMesh> StaticMeshRef;

private:
	TSharedPtr<FDisplayClusterRender_MeshComponentProxy, ESPMode::ThreadSafe> MeshComponentProxyPtr;
};
