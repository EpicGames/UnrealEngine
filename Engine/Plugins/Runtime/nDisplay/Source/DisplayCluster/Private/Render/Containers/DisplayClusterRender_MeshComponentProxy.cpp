// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Containers/DisplayClusterRender_MeshComponentProxy.h"
#include "Render/Containers/DisplayClusterRender_MeshComponentProxyData.h"
#include "Render/Containers/DisplayClusterRender_MeshResources.h"

#include "Misc/DisplayClusterLog.h"
#include "RHIResourceUtils.h"

TGlobalResource<FDisplayClusterMeshVertexDeclaration> GDisplayClusterMeshVertexDeclaration;

//*************************************************************************
//* FDisplayClusterRender_MeshComponentProxy
//*************************************************************************
FDisplayClusterRender_MeshComponentProxy::FDisplayClusterRender_MeshComponentProxy()
{ }

FDisplayClusterRender_MeshComponentProxy::~FDisplayClusterRender_MeshComponentProxy()
{
	ImplRelease();
}

void FDisplayClusterRender_MeshComponentProxy::Release_RenderThread()
{
	check(IsInRenderingThread());

	ImplRelease();
}

void FDisplayClusterRender_MeshComponentProxy::ImplRelease()
{
	VertexBufferRHI.SafeRelease();
	IndexBufferRHI.SafeRelease();

	NumTriangles = 0;
	NumVertices = 0;
}

bool FDisplayClusterRender_MeshComponentProxy::IsEnabled_RenderThread() const
{
	check(IsInRenderingThread());

	return NumTriangles > 0 && NumVertices > 0 && VertexBufferRHI.IsValid() && IndexBufferRHI.IsValid();
}

bool FDisplayClusterRender_MeshComponentProxy::BeginRender_RenderThread(FRHICommandListImmediate& RHICmdList, FGraphicsPipelineStateInitializer& GraphicsPSOInit) const
{
	if (IsEnabled_RenderThread())
	{
		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GDisplayClusterMeshVertexDeclaration.VertexDeclarationRHI;
		return true;
	}

	return false;
}

bool  FDisplayClusterRender_MeshComponentProxy::FinishRender_RenderThread(FRHICommandListImmediate& RHICmdList) const
{
	if (IsEnabled_RenderThread())
	{
		// Support update
		RHICmdList.SetStreamSource(0, VertexBufferRHI, 0);
		RHICmdList.DrawIndexedPrimitive(IndexBufferRHI, 0, 0, NumVertices, 0, NumTriangles, 1);
		return true;
	}

	return false;
}

void FDisplayClusterRender_MeshComponentProxy::UpdateRHI_RenderThread(FRHICommandListImmediate& RHICmdList, FDisplayClusterRender_MeshComponentProxyData* InMeshData)
{
	check(IsInRenderingThread());

	ImplRelease();

	if (InMeshData && InMeshData->IsValid())
	{
		NumTriangles = InMeshData->GetNumTriangles();
		NumVertices = InMeshData->GetNumVertices();

		EBufferUsageFlags Usage = BUF_ShaderResource | BUF_Static;

		// Create Vertex buffer RHI:
		{
			size_t VertexDataSize = sizeof(FDisplayClusterMeshVertexType) * NumVertices;
			if (VertexDataSize == 0)
			{
				UE_LOGF(LogDisplayClusterRender, Warning, "MeshComponent has a vertex size of 0, please make sure a mesh is assigned.")
				return;
			}

			const FRHIBufferCreateDesc CreateDesc =
				FRHIBufferCreateDesc::CreateVertex(TEXT("DisplayClusterRender_MeshComponentProxy_VertexBuffer"), VertexDataSize)
				.AddUsage(Usage)
				.SetInitActionInitializer()
				.DetermineInitialState();

			TRHIBufferInitializer<FDisplayClusterMeshVertexType> DestVertexData = RHICmdList.CreateBufferInitializer(CreateDesc);
			{
				const FDisplayClusterMeshVertex* SrcVertexData = InMeshData->GetVertexData().GetData();
				for (uint32 VertexIdx = 0; VertexIdx < NumVertices; VertexIdx++)
				{
					DestVertexData[VertexIdx].SetVertexData(SrcVertexData[VertexIdx]);
				}
			}

			VertexBufferRHI = DestVertexData.Finalize();
		}

		// Create Index buffer RHI:
		{
			IndexBufferRHI = UE::RHIResourceUtils::CreateIndexBufferFromArray(
				RHICmdList,
				TEXT("DisplayClusterRender_MeshComponentProxy_VertexBuffer"),
				Usage,
				MakeConstArrayView(InMeshData->GetIndexData())
			);
		}
	}
}
