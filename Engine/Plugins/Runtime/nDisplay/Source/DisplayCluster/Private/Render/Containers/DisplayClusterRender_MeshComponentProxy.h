// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "Render/Containers/IDisplayClusterRender_MeshComponentProxy.h"

class FDisplayClusterRender_MeshComponentProxy
	: public IDisplayClusterRender_MeshComponentProxy
{
public:
	FDisplayClusterRender_MeshComponentProxy();
	virtual ~FDisplayClusterRender_MeshComponentProxy();

public:
	virtual bool IsEnabled_RenderThread() const override;
	virtual bool BeginRender_RenderThread(FRHICommandListImmediate& RHICmdList, FGraphicsPipelineStateInitializer& GraphicsPSOInit) const override;
	virtual bool FinishRender_RenderThread(FRHICommandListImmediate& RHICmdList) const override;

	void UpdateRHI_RenderThread(FRHICommandListImmediate& RHICmdList, class FDisplayClusterRender_MeshComponentProxyData* InMeshData);
	void Release_RenderThread();

private:
	void ImplRelease();

private:
	/* RenderThread resources */
	FBufferRHIRef VertexBufferRHI;
	FBufferRHIRef IndexBufferRHI;

	uint32 NumTriangles = 0;
	uint32 NumVertices = 0;
};
