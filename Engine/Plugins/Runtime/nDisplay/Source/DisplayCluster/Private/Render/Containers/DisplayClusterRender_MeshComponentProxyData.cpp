// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Containers/DisplayClusterRender_MeshComponentProxyData.h"
#include "Render/Containers/DisplayClusterRender_MeshGeometry.h"

#include "Misc/DisplayClusterLog.h"

#include "Engine/StaticMesh.h"
#include "ProceduralMeshComponent.h"
#include "StaticMeshResources.h"

//*************************************************************************
//* FDisaplyClusterMeshComponentProxyData
//*************************************************************************
FDisplayClusterRender_MeshComponentProxyData::FDisplayClusterRender_MeshComponentProxyData(const FString& InSourceGeometryName, const EDisplayClusterRender_MeshComponentProxyDataFunc InDataFunc, const FStaticMeshLODResources& InStaticMeshLODResource, const FDisplayClusterMeshUVs& InUVs)
	: SourceGeometryName(InSourceGeometryName)
{
	check(IsInGameThread());

	const FPositionVertexBuffer& PositionBuffer = InStaticMeshLODResource.VertexBuffers.PositionVertexBuffer;
	const FStaticMeshVertexBuffer& VertexBuffer = InStaticMeshLODResource.VertexBuffers.StaticMeshVertexBuffer;
	const FRawStaticIndexBuffer& IndexBuffer = InStaticMeshLODResource.IndexBuffer;

	// Max tex coords used in StaticMesh
	const int32 NumTexCoords = InStaticMeshLODResource.GetNumTexCoords();

	// Remap BaseUVs
	// By default, StaticMesh uses index 0 for BaseUV
	const int32 BaseUVIndex = InUVs.GetSourceGeometryUVIndex(FDisplayClusterMeshUVs::ERemapTarget::Base, NumTexCoords, 0);

	// By default, StaticMesh uses index 1 for ChromakeyUV
	const int32 ChromakeyUVIndex = InUVs.GetSourceGeometryUVIndex(FDisplayClusterMeshUVs::ERemapTarget::Chromakey, NumTexCoords, 1);

	IndexBuffer.GetCopy(IndexData);

	VertexData.AddZeroed(PositionBuffer.GetNumVertices());
	for (int32 VertexIdx = 0; VertexIdx < VertexData.Num(); VertexIdx++)
	{
		VertexData[VertexIdx].Position = FVector4(FVector(PositionBuffer.VertexPosition(VertexIdx)));
		VertexData[VertexIdx].UV = FVector2D(VertexBuffer.GetVertexUV(VertexIdx, BaseUVIndex));
		VertexData[VertexIdx].UV_Chromakey = FVector2D(VertexBuffer.GetVertexUV(VertexIdx, ChromakeyUVIndex));
	}

	UpdateData(InDataFunc);
}

FDisplayClusterRender_MeshComponentProxyData::FDisplayClusterRender_MeshComponentProxyData(const FString& InSourceGeometryName, const EDisplayClusterRender_MeshComponentProxyDataFunc InDataFunc, const FProcMeshSection& InProcMeshSection, const FDisplayClusterMeshUVs& InUVs)
	: SourceGeometryName(InSourceGeometryName)
{
	if (InUVs[FDisplayClusterMeshUVs::ERemapTarget::Base] > 3)
	{
		// Show warning for values >3
		UE_LOGF(LogDisplayClusterRender, Warning, "MeshComponentProxyData('%ls'): Base UVIndex value '%d' is invalid - ProceduralMesh support range 0..3", *SourceGeometryName, InUVs[FDisplayClusterMeshUVs::ERemapTarget::Base]);
	}

	if (InUVs[FDisplayClusterMeshUVs::ERemapTarget::Chromakey] > 3)
	{
		// Show warning for values >3
		UE_LOGF(LogDisplayClusterRender, Warning, "MeshComponentProxyData('%ls'): Chromakey UVIndex value '%d' is invalid - ProceduralMesh support range 0..3", *SourceGeometryName, InUVs[FDisplayClusterMeshUVs::ERemapTarget::Chromakey]);
	}

	// Remap BaseUVs
	// Max tex coords used in ProceduralMesh now 4
	const int32 NumTexCoords = 4;

	// By default, ProceduralMesh uses index 0 for BaseUV
	const int32 BaseUVIndex = InUVs.GetSourceGeometryUVIndex(FDisplayClusterMeshUVs::ERemapTarget::Base, NumTexCoords, 0);

	// By default, ProceduralMesh uses index 0 for ChromakeyUV
	const int32 ChromakeyUVIndex = InUVs.GetSourceGeometryUVIndex(FDisplayClusterMeshUVs::ERemapTarget::Chromakey, NumTexCoords, 0);

	VertexData.AddZeroed(InProcMeshSection.ProcVertexBuffer.Num());
	for (int32 VertexIdx = 0; VertexIdx < VertexData.Num(); VertexIdx++)
	{
		const FProcMeshVertex& InVertex = InProcMeshSection.ProcVertexBuffer[VertexIdx];

		VertexData[VertexIdx].Position = InVertex.Position;

		// Remap source geometry UV for Base
		switch (BaseUVIndex)
		{
		default:
		case 0:
			VertexData[VertexIdx].UV = InVertex.UV0;
			break;
		case 1:
			VertexData[VertexIdx].UV = InVertex.UV1;
			break;
		case 2:
			VertexData[VertexIdx].UV = InVertex.UV2;
			break;
		case 3:
			VertexData[VertexIdx].UV = InVertex.UV3;
			break;
		}

		// Remap source geometry UV for Chromakey
		switch (ChromakeyUVIndex)
		{
		default:
		case 0:
			VertexData[VertexIdx].UV_Chromakey = InVertex.UV0;
			break;
		case 1:
			VertexData[VertexIdx].UV_Chromakey = InVertex.UV1;
			break;
		case 2:
			VertexData[VertexIdx].UV_Chromakey = InVertex.UV2;
			break;
		case 3:
			VertexData[VertexIdx].UV_Chromakey = InVertex.UV3;
			break;
		}
	}

	int32 IndexDataIt = 0;
	IndexData.AddZeroed(InProcMeshSection.ProcIndexBuffer.Num());
	for (const uint32& ProcIndexBufferIt : InProcMeshSection.ProcIndexBuffer)
	{
		IndexData[IndexDataIt++] = ProcIndexBufferIt;
	}

	UpdateData(InDataFunc);
}

FDisplayClusterRender_MeshComponentProxyData::FDisplayClusterRender_MeshComponentProxyData(const FString& InSourceGeometryName, const EDisplayClusterRender_MeshComponentProxyDataFunc InDataFunc, const FDisplayClusterRender_MeshGeometry& InMeshGeometry)
	: SourceGeometryName(InSourceGeometryName)
{
	bool bUseChromakeyUV = InMeshGeometry.ChromakeyUV.Num() > 0;

	VertexData.AddZeroed(InMeshGeometry.Vertices.Num());
	for (int32 VertexIdx = 0; VertexIdx < VertexData.Num(); VertexIdx++)
	{
		VertexData[VertexIdx].Position = InMeshGeometry.Vertices[VertexIdx];
		VertexData[VertexIdx].UV = InMeshGeometry.UV[VertexIdx];
		VertexData[VertexIdx].UV_Chromakey = bUseChromakeyUV ? InMeshGeometry.ChromakeyUV[VertexIdx] : InMeshGeometry.UV[VertexIdx];
	}

	int32 IndexDataIt = 0;
	IndexData.AddZeroed(InMeshGeometry.Triangles.Num());
	for (const int32& MeshGeometryIt : InMeshGeometry.Triangles)
	{
		IndexData[IndexDataIt++] = (uint32)MeshGeometryIt;
	}

	UpdateData(InDataFunc);
}

void FDisplayClusterRender_MeshComponentProxyData::UpdateData(const EDisplayClusterRender_MeshComponentProxyDataFunc InDataFunc)
{
	if (IndexData.Num() > 0 && VertexData.Num() > 0)
	{
		switch (InDataFunc)
		{
		case EDisplayClusterRender_MeshComponentProxyDataFunc::OutputRemapScreenSpace:
			// Output remap require normalize mesh to screen space coords:
			ImplNormalizeToScreenSpace();
			ImplRemoveInvisibleFaces();
			break;

		default:
		case EDisplayClusterRender_MeshComponentProxyDataFunc::Disabled:
			break;
		}
	}

	if (IndexData.Num() > 0 && VertexData.Num() > 0)
	{
		NumTriangles = IndexData.Num() / 3;
		NumVertices = VertexData.Num();
	}
	else
	{
		IndexData.Empty();
		VertexData.Empty();
		NumTriangles = 0;
		NumVertices = 0;
	}

	if (!IsValid())
	{
		UE_LOGF(LogDisplayClusterRender, Error, "MeshComponentProxyData::UpdateData('%ls') Invalid mesh - ignored", *SourceGeometryName);
	}
}

void FDisplayClusterRender_MeshComponentProxyData::ImplNormalizeToScreenSpace()
{
	FBox AABBox(FVector(FLT_MAX, FLT_MAX, FLT_MAX), FVector(-FLT_MAX, -FLT_MAX, -FLT_MAX));

	for (const FDisplayClusterMeshVertex& MeshVertexIt : VertexData)
	{
		const FVector4& Vertex = MeshVertexIt.Position;

		AABBox.Min.X = FMath::Min(AABBox.Min.X, Vertex.X);
		AABBox.Min.Y = FMath::Min(AABBox.Min.Y, Vertex.Y);
		AABBox.Min.Z = FMath::Min(AABBox.Min.Z, Vertex.Z);

		AABBox.Max.X = FMath::Max(AABBox.Max.X, Vertex.X);
		AABBox.Max.Y = FMath::Max(AABBox.Max.Y, Vertex.Y);
		AABBox.Max.Z = FMath::Max(AABBox.Max.Z, Vertex.Z);
	}

	//Normalize
	FVector Size(
		(AABBox.Max.X - AABBox.Min.X),
		(AABBox.Max.Y - AABBox.Min.Y),
		(AABBox.Max.Z - AABBox.Min.Z)
	);

	bool bIsValidMeshAxis = true;

	// Support axis rules: Z=UP(screen Y), Y=RIGHT(screen X), X=NotUsed
	if (FMath::Abs(Size.Y) <= KINDA_SMALL_NUMBER)
	{
		UE_LOGF(LogDisplayClusterRender, Error, "MeshComponentProxyData::NormalizeToScreenSpace('%ls'): The Y axis is used in screen space as 'x' and the distance cannot be zero.", *SourceGeometryName);
		bIsValidMeshAxis = false;
	}

	if (FMath::Abs(Size.Z) <= KINDA_SMALL_NUMBER)
	{
		UE_LOGF(LogDisplayClusterRender, Error, "MeshComponentProxyData::NormalizeToScreenSpace('%ls'): The Z axis is used in screen space as 'y' and the distance cannot be zero.", *SourceGeometryName);
		bIsValidMeshAxis = false;
	}

	if (bIsValidMeshAxis)
	{
		// Checking for strange aspect ratio
		FVector ScreenSize(FMath::Max(Size.Z, Size.Y), FMath::Min(Size.Z, Size.Y), 0);
		double AspectRatio = ScreenSize.X / ScreenSize.Y;
		if (AspectRatio > 10)
		{
			// just warning, experimental
			UE_LOGF(LogDisplayClusterRender, Warning, "MeshComponentProxyData::NormalizeToScreenSpace('%ls'): Aspect ratio is to big '%f'- <%f,%f>", *SourceGeometryName, AspectRatio, ScreenSize.X, ScreenSize.Y);
		}
	}

	if (!bIsValidMeshAxis)
	{
		// Dont use invalid mesh
		IndexData.Empty();
		VertexData.Empty();
		NumTriangles = 0;
		NumVertices = 0;

		UE_LOGF(LogDisplayClusterRender, Error, "MeshComponentProxyData::NormalizeToScreenSpace('%ls') Invalid mesh. Expected mesh axis: Z=UP(screen Y), Y=RIGHT(screen X), X=NotUsed", *SourceGeometryName);
		return;
	}

	FVector Scale(0, 1.f / Size.Y, 1.f / Size.Z);
	for (FDisplayClusterMeshVertex& MeshVertexIt : VertexData)
	{
		FVector4& Vertex = MeshVertexIt.Position;

		const float X = (Vertex.Y - AABBox.Min.Y) * Scale.Y;
		const float Y = (Vertex.Z - AABBox.Min.Z) * Scale.Z;

		Vertex = FVector4(X, Y, 0);
	}
}

void FDisplayClusterRender_MeshComponentProxyData::ImplRemoveInvisibleFaces()
{
	// The geometry is created by the 3D artist and is sometimes incorrect. 
	// For example, in the OutputRemap post-process, it is necessary that all UVs be in the range 0..1. 
	// For visual validation, all points outside the 0..1 range are excluded during geometry loading when called function RemoveInvisibleFaces().

	TArray<uint32> VisibleIndexData;
	int32 RemovedFacesNum = 0;
	const int32 FacesNum = IndexData.Num() / 3;
	for (int32 Face = 0; Face < FacesNum; ++Face)
	{
		if (IsFaceVisible(Face))
		{
			const uint32 FaceIdx0 = IndexData[Face * 3 + 0];
			const uint32 FaceIdx1 = IndexData[Face * 3 + 1];
			const uint32 FaceIdx2 = IndexData[Face * 3 + 2];

			VisibleIndexData.Add(FaceIdx0);
			VisibleIndexData.Add(FaceIdx1);
			VisibleIndexData.Add(FaceIdx2);
		}
		else
		{
			RemovedFacesNum++;
		}
	}

	IndexData.Empty();
	IndexData.Append(VisibleIndexData);

	if (IndexData.Num() == 0)
	{
		VertexData.Empty();
	}

	if (RemovedFacesNum)
	{
		UE_LOGF(LogDisplayClusterRender, Error, "MeshComponentProxyData::RemoveInvisibleFaces('%ls') Removed %d/%d faces", *SourceGeometryName, RemovedFacesNum, FacesNum);
	}
}

bool FDisplayClusterRender_MeshComponentProxyData::IsFaceVisible(int32 Face)
{
	const int32 FaceIdx0 = IndexData[Face * 3 + 0];
	const int32 FaceIdx1 = IndexData[Face * 3 + 1];
	const int32 FaceIdx2 = IndexData[Face * 3 + 2];

	return IsUVVisible(FaceIdx0) && IsUVVisible(FaceIdx1) && IsUVVisible(FaceIdx2);
}

bool FDisplayClusterRender_MeshComponentProxyData::IsUVVisible(int32 UVIndex)
{
	return (
		VertexData[UVIndex].UV.X >= 0.f && VertexData[UVIndex].UV.X <= 1.f &&
		VertexData[UVIndex].UV.Y >= 0.f && VertexData[UVIndex].UV.Y <= 1.f);
}
