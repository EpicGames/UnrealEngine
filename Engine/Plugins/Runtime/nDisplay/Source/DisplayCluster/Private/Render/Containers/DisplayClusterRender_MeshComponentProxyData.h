// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Render/Containers/DisplayClusterRender_MeshComponentTypes.h"


class FDisplayClusterRender_MeshGeometry;
struct FStaticMeshLODResources;
struct FProcMeshSection;

class FDisplayClusterRender_MeshComponentProxyData
{
public:
	/**
	* Initialize geometry data from static mesh geometry
	*
	* @param InDataFunc              - Geometry function type
	* @param InStaticMeshLODResource - Source of geometry
	* @param InUVs                   - Map source geometry UVs to DCMeshComponent UVs
	*/
	FDisplayClusterRender_MeshComponentProxyData(const FString& InSourceGeometryName, const EDisplayClusterRender_MeshComponentProxyDataFunc InDataFunc, const FStaticMeshLODResources& InStaticMeshLODResource, const FDisplayClusterMeshUVs& InUVs);

	/**
	* Initialize geometry data from procedural mesh geometry
	*
	* @param InDataFunc        - Geometry function type
	* @param InProcMeshSection - Source of geometry
	* @param InUVs             - Map source geometry UVs to DCMeshComponent UVs
	*/
	FDisplayClusterRender_MeshComponentProxyData(const FString& InSourceGeometryName, const EDisplayClusterRender_MeshComponentProxyDataFunc InDataFunc, const FProcMeshSection& InProcMeshSection, const FDisplayClusterMeshUVs& InUVs);

	/**
	* Initialize geometry data from nDisplay geometry container
	*
	* @param InDataFunc     - Geometry function type
	* @param InMeshGeometry - Source of geometry
	*/
	FDisplayClusterRender_MeshComponentProxyData(const FString& InSourceGeometryName, const EDisplayClusterRender_MeshComponentProxyDataFunc InDataFunc, const FDisplayClusterRender_MeshGeometry& InMeshGeometry);

	const TArray<uint32>& GetIndexData() const
	{ return IndexData; }

	const TArray<FDisplayClusterMeshVertex>& GetVertexData() const
	{ return VertexData; }

	uint32 GetNumTriangles() const
	{ return NumTriangles; }

	uint32 GetNumVertices() const
	{ return NumVertices; }

	bool IsValid() const
	{
		return NumTriangles > 0 && NumVertices > 0 && IndexData.Num() > 0 && VertexData.Num() > 0;
	}

private:
	/**
	* Update geometry data with specified func
	*
	* @param InDataFunc     - Geometry function type
	*/
	void UpdateData(const EDisplayClusterRender_MeshComponentProxyDataFunc InDataFunc);

	/**
	* Normalize geometry points to screen space
	* [OutputRemapScreenSpace]
	*/
	void ImplNormalizeToScreenSpace();

	/**
	* Visual check: remove faces with UV off-screen
	* [OutputRemapScreenSpace]
	*/
	void ImplRemoveInvisibleFaces();

private:
	bool IsFaceVisible(int32 Face);
	bool IsUVVisible(int32 UVIndex);

public:
	// Purpose of logging
	const FString SourceGeometryName;

private:

	TArray<uint32> IndexData;
	TArray<FDisplayClusterMeshVertex> VertexData;
	uint32 NumTriangles = 0;
	uint32 NumVertices = 0;
};
