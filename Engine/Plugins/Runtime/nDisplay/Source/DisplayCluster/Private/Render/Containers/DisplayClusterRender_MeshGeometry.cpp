// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Containers/DisplayClusterRender_MeshGeometry.h"

#include "Misc/ConfigCacheIni.h"
#include "Misc/Paths.h"
#include "Misc/FileHelper.h"
#include "Misc/DisplayClusterLog.h"

namespace UE::DisplayCluster::MeshGeometryStrings
{
	static constexpr auto Vertex = TEXT("v ");
	static constexpr auto VertexNormal = TEXT("vn ");
	static constexpr auto UV     = TEXT("vt ");
	static constexpr auto Face   = TEXT("f ");

	namespace delims
	{
		static constexpr auto Values = TEXT(" ");
		static constexpr auto Face = TEXT("/");
	}
};
using namespace UE::DisplayCluster;

/**
 * OBJ mesh loader class
 */
class FDisplayCluster_MeshGeometryLoaderOBJ
{
public:
	FDisplayCluster_MeshGeometryLoaderOBJ(FDisplayClusterRender_MeshGeometry& InTarget, bool bInImportVertexNormal = false)
		: bImportVertexNormal(bInImportVertexNormal)
		, Target(InTarget)
	{ }

	bool Load(const FString& FullPathFileName);

private:
	bool CreateFromFile(const FString& FullPathFileName);

private:
	// Obj file parser:
	bool ParseLine(const FString& Line);
	bool ExtractVertex(const FString& Line);
	bool ExtractVertexNormal(const FString& Line);
	bool ExtractUV(const FString& Line);
	bool ExtractFace(const FString& Line);
	int32 ExtractFaceVertex(const FString& Line);

	bool SaveToTarget();

private:
	const bool bImportVertexNormal;

	FDisplayClusterRender_MeshGeometry& Target;
	TArray<FVector> InVertex;
	TArray<FVector> InVertexNormal;
	TArray<FVector> InUV;


	struct FFaceIdx
	{
		int32 VertexIdx = -1;
		int32 VertexNormalIdx = -1;
		int32 UVIdx = -1;

		bool operator==(const FFaceIdx& In) const
		{
			return VertexIdx == In.VertexIdx && VertexNormalIdx == In.VertexNormalIdx && UVIdx == In.UVIdx;
		}
	};

	TArray<FFaceIdx> Faces;
	TArray<int32> Triangles;
};


//*************************************************************************
//* FDisplayClusterRender_MeshGeometry
//*************************************************************************
FDisplayClusterRender_MeshGeometry::FDisplayClusterRender_MeshGeometry(const FDisplayClusterRender_MeshGeometry& In)
	: Vertices(In.Vertices)
	, Normal(In.Normal)
	, UV(In.UV)
	, ChromakeyUV(In.ChromakeyUV)
	, Triangles(In.Triangles)
{ }


FDisplayClusterRender_MeshGeometry::FDisplayClusterRender_MeshGeometry(EDisplayClusterRender_MeshGeometryCreateType CreateType)
{
	switch (CreateType)
	{
	case EDisplayClusterRender_MeshGeometryCreateType::Passthrough:
		CreatePassthrough();
		break;
	}
}

// Load geometry from OBJ file
bool FDisplayClusterRender_MeshGeometry::LoadFromFile(const FString& FullPathFileName, EDisplayClusterRender_MeshGeometryFormat Format)
{
	switch (Format)
	{
	case EDisplayClusterRender_MeshGeometryFormat::OBJ:
	{
		FDisplayCluster_MeshGeometryLoaderOBJ LoaderOBJ(*this);
		return LoaderOBJ.Load(FullPathFileName);
	}
	default:
		break;
	}

	return false;
}

// Test purpose: create square geometry
void FDisplayClusterRender_MeshGeometry::CreatePassthrough()
{
	Vertices.Empty();
	Vertices.Add(FVector(0, 2.6f, 1));
	Vertices.Add(FVector(0, 2.6f, 2));
	Vertices.Add(FVector(0, 5.5f, 2));
	Vertices.Add(FVector(0, 5.5f, 1));

	UV.Empty();
	UV.Add(FVector2D(0, 1));
	UV.Add(FVector2D(0, 0));
	UV.Add(FVector2D(1, 0));
	UV.Add(FVector2D(1, 1));

	Triangles.Empty();
	Triangles.Add(0);
	Triangles.Add(1);
	Triangles.Add(2);

	Triangles.Add(3);
	Triangles.Add(0);
	Triangles.Add(2);
}

//*************************************************************************
//* FDisplayCluster_MeshGeometryLoaderOBJ
//*************************************************************************
bool FDisplayCluster_MeshGeometryLoaderOBJ::Load(const FString& FullPathFileName)
{
	return CreateFromFile(FullPathFileName);
}

bool FDisplayCluster_MeshGeometryLoaderOBJ::CreateFromFile(const FString& FullPathFileName)
{	
	// Load from obj file at runtime:
	if (FPaths::FileExists(FullPathFileName))
	{
		TArray<FString> data;
		if (FFileHelper::LoadANSITextFileToStrings(*FullPathFileName, nullptr, data) == true)
		{
			bool bResult = true;

			int32 LineIdx = 0;
			// Parse each line from config
			for (FString Line : data)
			{
				LineIdx++;
				Line.TrimStartAndEndInline();
				if (!ParseLine(Line))
				{
					UE_LOGF(LogDisplayClusterRender, Error, "MeshGeometryLoaderOBJ: Invalid line %d: '%ls'", LineIdx , *Line);
					bResult = false;
				}
			}

			if (bResult)
			{
				bResult = SaveToTarget();
			}

			if (!bResult)
			{
				UE_LOGF(LogDisplayClusterRender, Error, "MeshGeometryLoaderOBJ: Can't load mesh geometry from file '%ls'", *FullPathFileName);
			}

			return bResult;
		}
	}

	return false;
}

bool FDisplayCluster_MeshGeometryLoaderOBJ::SaveToTarget()
{
	for (FFaceIdx& It : Faces)
	{
		const FVector VertexPos = (It.VertexIdx < 0) ? FVector(0, 0, 0) : InVertex[It.VertexIdx];
		Target.Vertices.Add(VertexPos);

		const FVector VertexNormal = (It.VertexNormalIdx < 0) ? FVector(0, 0, 0) : InVertexNormal[It.VertexNormalIdx];
		Target.Normal.Add(VertexNormal);

		const FVector UVCoord = (It.UVIdx < 0) ? FVector(0, 0, 0) : InUV[It.UVIdx];
		Target.UV.Add(FVector2D(UVCoord.X, UVCoord.Y));
	}

	Target.Triangles.Empty();
	Target.Triangles.Append(Triangles);

	return true;
}

bool FDisplayCluster_MeshGeometryLoaderOBJ::ParseLine(const FString& Line)
{
	if (Line.StartsWith(MeshGeometryStrings::Vertex))
	{
		return ExtractVertex(Line);
	}
	else if (bImportVertexNormal && Line.StartsWith(MeshGeometryStrings::VertexNormal))
	{
		return ExtractVertexNormal(Line);
	}
	else if (Line.StartsWith(MeshGeometryStrings::UV))
	{
		return ExtractUV(Line);
	}
	else if (Line.StartsWith(MeshGeometryStrings::Face))
	{
		return ExtractFace(Line);
	}

	return true;
}

bool FDisplayCluster_MeshGeometryLoaderOBJ::ExtractVertex(const FString& Line)
{
	TArray<FString> Data;

	if (Line.ParseIntoArray(Data, MeshGeometryStrings::delims::Values) == 4)
	{
		const float X = FCString::Atof(*Data[1]);
		const float Y = FCString::Atof(*Data[2]);
		const float Z = FCString::Atof(*Data[3]);

		InVertex.Add(FVector(X, -Y, Z));

		return true;
	}

	return false;
}

bool FDisplayCluster_MeshGeometryLoaderOBJ::ExtractVertexNormal(const FString& Line)
{
	TArray<FString> Data;

	if (Line.ParseIntoArray(Data, MeshGeometryStrings::delims::Values) == 4)
	{
		const float X = FCString::Atof(*Data[1]);
		const float Y = FCString::Atof(*Data[2]);
		const float Z = FCString::Atof(*Data[3]);

		InVertexNormal.Add(FVector(X, -Y, Z));

		return true;
	}

	return false;
}

bool FDisplayCluster_MeshGeometryLoaderOBJ::ExtractUV(const FString& Line)
{
	TArray<FString> Data;
	int32 Count = Line.ParseIntoArray(Data, MeshGeometryStrings::delims::Values);
	if (Count > 2)
	{
		const float U = FCString::Atof(*Data[1]);
		const float V = FCString::Atof(*Data[2]);
		const float W = (Count > 3)?FCString::Atof(*Data[3]) : 0;

		InUV.Add(FVector(U, 1-V, W));

		return true;
	}

	return false;
}

bool FDisplayCluster_MeshGeometryLoaderOBJ::ExtractFace(const FString& Line)
{
	TArray<FString> Data;
	Line.ParseIntoArray(Data, MeshGeometryStrings::delims::Values);

	if (Data.Num() > 3)
	{
		TArray<int32> FaceIndices;
		for (int32 LineItem = 1; LineItem < Data.Num(); ++LineItem)
		{
			const int32 IdxC = ExtractFaceVertex(Data[LineItem]);
			if (IdxC >= 0)
			{
				if (FaceIndices.Num() > 2)
				{
					const int32 IdxA = FaceIndices[0];
					const int32 IdxB = FaceIndices.Last();

					FaceIndices.Add(IdxA);
					FaceIndices.Add(IdxB);
					FaceIndices.Add(IdxC);
				}
				else
				{
					FaceIndices.Add(IdxC);
				}
			}
			else
			{
				return false;
			}
		}

		Triangles.Append(FaceIndices);

		return true;
	}

	return false;
}

int32 FDisplayCluster_MeshGeometryLoaderOBJ::ExtractFaceVertex(const FString& Line)
{
	FFaceIdx OutFaceIdx;

	TArray<FString> Data;
	if (Line.ParseIntoArray(Data, MeshGeometryStrings::delims::Face) > 1)
	{
		const int32 InVertexIndex = FCString::Atoi(*Data[0]) - 1;
		if (InVertexIndex < 0 || InVertexIndex >= InVertex.Num())
		{
			UE_LOGF(LogDisplayClusterRender, Error, "MeshGeometryLoaderOBJ: broken vertex index. Line: '%ls'", *Line);
			OutFaceIdx.VertexIdx = -1;
		}
		else
		{
			OutFaceIdx.VertexIdx = InVertexIndex;
		}

		const int32 InUVIndex = FCString::Atoi(*Data[1]) - 1;
		if (InUVIndex < 0 || InUVIndex >= InUV.Num())
		{
			UE_LOGF(LogDisplayClusterRender, Error, "MeshGeometryLoaderOBJ: broken uv index. Line: '%ls'", *Line);
			OutFaceIdx.UVIdx = -1;
		}
		else
		{
			OutFaceIdx.UVIdx = InUVIndex;
		}

		if (bImportVertexNormal && Data.Num() > 2)
		{
			const int32 InNormalIndex = FCString::Atoi(*Data[2]) - 1;
			if (InNormalIndex < 0 || InNormalIndex >= InVertexNormal.Num())
			{
				UE_LOGF(LogDisplayClusterRender, Error, "MeshGeometryLoaderOBJ: broken vertex normal  index. Line: '%ls'", *Line);
				OutFaceIdx.VertexNormalIdx = -1;
			}
			else
			{
				OutFaceIdx.VertexNormalIdx = InNormalIndex;
			}
		}
	}
	else
	{
		return -1;
	}

	int32 OutIndex = Faces.Find(OutFaceIdx);
	if (OutIndex >= 0)
	{
		return OutIndex;
	}

	// Add new face vertex
	Faces.Add(OutFaceIdx);
	
	return Faces.Num() - 1;
}
