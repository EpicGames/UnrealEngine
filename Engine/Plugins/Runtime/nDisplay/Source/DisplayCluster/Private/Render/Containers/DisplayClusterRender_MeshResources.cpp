// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterRender_MeshResources.h"

#include "PipelineStateCache.h"
#include "RHI.h"

void FDisplayClusterMeshVertexDeclaration::InitRHI(FRHICommandListBase& RHICmdList)
{
	FVertexDeclarationElementList Elements;

	uint32 Stride = sizeof(FDisplayClusterMeshVertexType);

	Elements.Add(FVertexElement(0, STRUCT_OFFSET(FDisplayClusterMeshVertexType, Position), VET_Float4, 0, Stride));
	Elements.Add(FVertexElement(0, STRUCT_OFFSET(FDisplayClusterMeshVertexType, UV), VET_Float2, 1, Stride));
	Elements.Add(FVertexElement(0, STRUCT_OFFSET(FDisplayClusterMeshVertexType, UV_Chromakey), VET_Float2, 2, Stride));

	VertexDeclarationRHI = PipelineStateCache::GetOrCreateVertexDeclaration(Elements);
}

void FDisplayClusterMeshVertexDeclaration::ReleaseRHI()
{
	VertexDeclarationRHI.SafeRelease();
}
