// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "RenderResource.h"
#include "Render/Containers/DisplayClusterRender_MeshComponentTypes.h"


/** The vertex data used to filter a texture. */
struct FDisplayClusterMeshVertexType
{
	FVector4f Position;
	FVector2f UV;
	FVector2f UV_Chromakey;

	/** type conversion double -> float */
	FORCEINLINE void SetVertexData(const FDisplayClusterMeshVertex& In)
	{
		Position = FVector4f(In.Position); // LWC_TODO: precision loss
		UV = FVector2f(In.UV);
		UV_Chromakey = FVector2f(In.UV_Chromakey);
	}
};

/** The filter vertex declaration resource type. */
class FDisplayClusterMeshVertexDeclaration : public FRenderResource
{
public:
	FVertexDeclarationRHIRef VertexDeclarationRHI;

	//~ Begin FRenderResource interface
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	virtual void ReleaseRHI() override;
	//~ End FRenderResource interface
};
