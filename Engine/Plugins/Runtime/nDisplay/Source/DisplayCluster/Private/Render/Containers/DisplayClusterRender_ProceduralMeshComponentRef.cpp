// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Containers/DisplayClusterRender_ProceduralMeshComponentRef.h"

#include "ProceduralMeshComponent.h"

//-------------------------------------------------------------------------------
//        FDisplayClusterRender_ProceduralMeshComponentRef
//-------------------------------------------------------------------------------
UProceduralMeshComponent* FDisplayClusterRender_ProceduralMeshComponentRef::GetOrFindProceduralMeshComponent() const
{
	FScopeLock lock(&DataGuard);

	// When scene component of ProceduralMesh is re-created, etc - mark as dirty for update the geometry
	if (!IsSceneComponentValid())
	{
		MarkProceduralMeshGeometryDirty();
	}

	USceneComponent* SceneComponent = GetOrFindSceneComponent();
	if (SceneComponent != nullptr)
	{
		UProceduralMeshComponent* ProceduralMeshComponent = Cast<UProceduralMeshComponent>(SceneComponent);

		return ProceduralMeshComponent;
	}

	return nullptr;
}

bool FDisplayClusterRender_ProceduralMeshComponentRef::SetProceduralMeshComponentRef(UProceduralMeshComponent* InProceduralMeshComponent)
{
	FScopeLock lock(&DataGuard);

	ResetProceduralMeshGeometryDirty();

	return SetSceneComponent(InProceduralMeshComponent);
}

void FDisplayClusterRender_ProceduralMeshComponentRef::ResetProceduralMeshComponentRef()
{
	FScopeLock lock(&DataGuard);

	ResetProceduralMeshGeometryDirty();
	ResetSceneComponent();
}

bool FDisplayClusterRender_ProceduralMeshComponentRef::IsProceduralMeshGeometryDirty() const
{
	FScopeLock lock(&DataGuard);

	return bIsProceduralMeshGeometryDirty;
}

void FDisplayClusterRender_ProceduralMeshComponentRef::MarkProceduralMeshGeometryDirty() const
{
	FScopeLock lock(&DataGuard);

	bIsProceduralMeshGeometryDirty = true;
}

void FDisplayClusterRender_ProceduralMeshComponentRef::ResetProceduralMeshGeometryDirty() const
{
	FScopeLock lock(&DataGuard);

	bIsProceduralMeshGeometryDirty = false;
}
