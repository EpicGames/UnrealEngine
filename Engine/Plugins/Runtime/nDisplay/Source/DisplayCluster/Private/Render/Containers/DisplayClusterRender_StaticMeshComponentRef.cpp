// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Containers/DisplayClusterRender_StaticMeshComponentRef.h"
#include "Engine/StaticMesh.h"

//----------------------------------------------------------------------------------------
//
inline FName ImplGetStaticMeshGeometryName(const UStaticMeshComponent* StaticMeshComponent)
{
	if (StaticMeshComponent != nullptr)
	{
		const UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh();

		// When mesh geometry asset is not assigned to a component, return custom text
		if (StaticMesh == nullptr)
		{
			return FName(TEXT("nullptr"));
		}

		// Return assigned geometry asset name
		return StaticMesh->GetFName();
	}

	// Return empty FName for null component ptr
	return NAME_None;
}

//----------------------------------------------------------------------------------------
//      FDisplayClusterRender_StaticMeshComponentRef
//----------------------------------------------------------------------------------------
UStaticMeshComponent* FDisplayClusterRender_StaticMeshComponentRef::GetOrFindStaticMeshComponent() const
{
	FScopeLock lock(&DataGuard);

	USceneComponent* SceneComponent = GetOrFindSceneComponent();
	if (SceneComponent != nullptr)
	{
		UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(SceneComponent);
		if (StaticMeshComponent != nullptr)
		{
			// Handle static mesh component geometry asset reference changes runtime:
			if (!IsStaticMeshGeometryDirty())
			{
				const FName NewMeshGeometryName = ImplGetStaticMeshGeometryName(StaticMeshComponent);
				if(NewMeshGeometryName != MeshGeometryName)
				{
					MarkStaticMeshGeometryDirty();
				}
			}

			return StaticMeshComponent;
		}
	}

	return nullptr;
}

bool FDisplayClusterRender_StaticMeshComponentRef::SetStaticMeshComponentRef(UStaticMeshComponent* InStaticMeshComponent)
{
	FScopeLock lock(&DataGuard);

	ResetStaticMeshGeometryDirty();

	if (SetSceneComponent(InStaticMeshComponent))
	{
		// Update the name of the current mesh geometry asset
		MeshGeometryName = ImplGetStaticMeshGeometryName(InStaticMeshComponent);

		return true;
	}

	// Clear the name of the current mesh geometry asset
	MeshGeometryName = FName();

	return false;
}

void FDisplayClusterRender_StaticMeshComponentRef::ResetStaticMeshComponentRef()
{
	FScopeLock lock(&DataGuard);

	// Clear the name of the current mesh geometry asset
	MeshGeometryName = NAME_None;

	ResetStaticMeshGeometryDirty();
	ResetSceneComponent();
}

bool FDisplayClusterRender_StaticMeshComponentRef::IsStaticMeshGeometryDirty() const
{
	FScopeLock lock(&DataGuard);

	return bIsStaticMeshGeometryDirty;
}

void FDisplayClusterRender_StaticMeshComponentRef::MarkStaticMeshGeometryDirty() const
{
	FScopeLock lock(&DataGuard);

	bIsStaticMeshGeometryDirty = true;
}

void FDisplayClusterRender_StaticMeshComponentRef::ResetStaticMeshGeometryDirty() const
{
	FScopeLock lock(&DataGuard);

	bIsStaticMeshGeometryDirty = false;
}
