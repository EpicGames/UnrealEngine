// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Containers/DisplayClusterRender_Texture.h"
#include "Misc/App.h"
#include "Misc/DisplayClusterLog.h"
#include "RenderingThread.h"

int32 GDisplayClusterRender_TextureCacheEnable = 1;
static FAutoConsoleVariableRef CVarDisplayClusterRender_TextureCacheEnable(
	TEXT("nDisplay.cache.Texture.enable"),
	GDisplayClusterRender_TextureCacheEnable,
	TEXT("Enable the use of a named texture cache.\n"),
	ECVF_Default
);

int32 GDisplayClusterRender_TextureCacheTimeOutInFrames = 5 * 60 * 60; // Timeout is 5 minutes (for 60 frames per second)
static FAutoConsoleVariableRef CVarDisplayClusterRenderCached_TextureCacheTimeOutInFrames(
	TEXT("nDisplay.cache.Texture.TimeOutInFrames"),
	GDisplayClusterRender_TextureCacheTimeOutInFrames,
	TEXT("The timeout value in frames for the named texture cache.\n")
	TEXT("-1 - disable timeout.\n"),
	ECVF_Default
);

//---------------------------------------------------------------------------------------
// FDisplayClusterRender_Texture
//---------------------------------------------------------------------------------------
FDisplayClusterRender_Texture::FDisplayClusterRender_Texture(const FString& InUniqueTextureName)
	: UniqueTextureName(InUniqueTextureName)
{ }

FDisplayClusterRender_Texture::~FDisplayClusterRender_Texture()
{
	if (TextureResourcePtr.IsValid())
	{
		UE_LOGF(LogDisplayClusterRender, Error, "FDisplayClusterRender_Texture::SetTextureResource(nullptr) function must be called before the destructor.");

		// Forced release of a resource
		TextureResourcePtr->ReleaseRenderResource();
		TextureResourcePtr.Reset();

		TextureResourceProxyPtr->ReleaseRenderResource();
		TextureResourceProxyPtr.Reset();
	}
}

const TSharedPtr<FDisplayClusterRender_TextureResource, ESPMode::ThreadSafe>& FDisplayClusterRender_Texture::GetTextureResource() const
{
	if (IsInActualRenderingThread() || IsInRHIThread())
	{
		return TextureResourceProxyPtr;
	}

	return TextureResourcePtr;
}

void FDisplayClusterRender_Texture::SetTextureResource(const TSharedPtr<FDisplayClusterRender_TextureResource, ESPMode::ThreadSafe>& InTextureResourcePtr)
{
	check(!IsInActualRenderingThread() && !IsInRHIThread());

	if (InTextureResourcePtr == TextureResourcePtr)
	{
		// ignore the same resoures
		return;
	}

	// Each PrivateResource value must be updated in it's own thread because any
	// rendering code trying to access the Resource
	// crash if it suddenly sees nullptr or a new resource that has not had it's InitRHI called.

	if (TextureResourcePtr.IsValid())
	{
		// Always call ReleaseRenderResource before destructor
		TextureResourcePtr->ReleaseRenderResource();
		TextureResourcePtr.Reset();
	}
	TextureResourcePtr = InTextureResourcePtr;

	// Update render thread
	ENQUEUE_RENDER_COMMAND(FDisplayClusterRender_Texture_UpdateResource)([This = SharedThis(this), InTextureResourcePtr](FRHICommandListImmediate& RHICmdList)
	{
		if (This->TextureResourceProxyPtr.IsValid())
		{
			// Always call ReleaseRenderResource before destructor
			This->TextureResourceProxyPtr->ReleaseRenderResource();
			This->TextureResourceProxyPtr.Reset();
		}

		This->TextureResourceProxyPtr = InTextureResourcePtr;
	});
}

void FDisplayClusterRender_Texture::CreateTexture(const void* InTextureData, const uint32 InComponentDepth, const uint32 InBytesPerComponent, uint32_t InWidth, uint32_t InHeight, bool bInHasCPUAccess)
{
	//Dedicated servers have no texture internals
	if (FApp::CanEverRender())
	{
		// Create a new texture resource.
		TSharedPtr<FDisplayClusterRender_TextureResource, ESPMode::ThreadSafe> NewResource = MakeShared<FDisplayClusterRender_TextureResource, ESPMode::ThreadSafe>(InTextureData, InComponentDepth, InBytesPerComponent, InWidth, InHeight, bInHasCPUAccess);
		if (NewResource.IsValid())
		{
			NewResource->InitializeRenderResource();
			SetTextureResource(NewResource);
		}
		else
		{
			SetTextureResource(nullptr);
		}
	}
}

int32 FDisplayClusterRender_Texture::GetDataCacheTimeOutInFrames()
{
	return FMath::Max(0, GDisplayClusterRender_TextureCacheTimeOutInFrames);
}

bool FDisplayClusterRender_Texture::IsDataCacheEnabled()
{
	return GDisplayClusterRender_TextureCacheEnable != 0;
}
