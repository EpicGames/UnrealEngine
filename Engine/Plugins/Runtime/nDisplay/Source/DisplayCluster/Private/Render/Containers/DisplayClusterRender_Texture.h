// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Render/Containers/IDisplayClusterRender_Texture.h"
#include "Render/Containers/DisplayClusterRender_TextureResource.h"

/**
 * Implementation for runtime texture object
 * Can be cached with TDisplayClusterDataCache: Multiple objects can get the same texture by name
 */
class FDisplayClusterRender_Texture
	: public IDisplayClusterRender_Texture
	, public TSharedFromThis<FDisplayClusterRender_Texture, ESPMode::ThreadSafe>
{
public:
	FDisplayClusterRender_Texture(const FString& InUniqueTextureName);
	virtual ~FDisplayClusterRender_Texture();

public:
	virtual TSharedPtr<IDisplayClusterRender_Texture, ESPMode::ThreadSafe> ToSharedPtr() override
	{
		return AsShared();
	}

	virtual TSharedPtr<const IDisplayClusterRender_Texture, ESPMode::ThreadSafe> ToSharedPtr() const override
	{
		return AsShared();
	}

	virtual void CreateTexture(const void* InTextureData, const uint32 InComponentDepth, const uint32 InBitDepth, uint32_t InWidth, uint32_t InHeight, bool bInHasCPUAccess=false) override;

	virtual bool IsEnabled() const override
	{
		const TSharedPtr<FDisplayClusterRender_TextureResource, ESPMode::ThreadSafe>& TextureResource = GetTextureResource();
		return TextureResource.IsValid() && TextureResource->IsRenderResourceInitialized();
	}

	virtual void* GetData() const override
	{
		const TSharedPtr<FDisplayClusterRender_TextureResource, ESPMode::ThreadSafe>& TextureResource = GetTextureResource();
		return TextureResource ? TextureResource->GetTextureData() : nullptr;
	}

	virtual uint32_t GetWidth() const override
	{
		const TSharedPtr<FDisplayClusterRender_TextureResource, ESPMode::ThreadSafe>& TextureResource = GetTextureResource();
		return TextureResource.IsValid() ? TextureResource->GetWidth() : 0;
	}

	virtual uint32_t GetHeight() const override
	{
		const TSharedPtr<FDisplayClusterRender_TextureResource, ESPMode::ThreadSafe>& TextureResource = GetTextureResource();
		return TextureResource.IsValid() ? TextureResource->GetHeight() : 0;
	}

	virtual EPixelFormat GetPixelFormat() const override
	{
		const TSharedPtr<FDisplayClusterRender_TextureResource, ESPMode::ThreadSafe>& TextureResource = GetTextureResource();
		return TextureResource.IsValid() ? TextureResource->GetPixelFormat() : PF_Unknown;
	}

	virtual uint32 GetComponentDepth() const override
	{
		const TSharedPtr<FDisplayClusterRender_TextureResource, ESPMode::ThreadSafe>& TextureResource = GetTextureResource();
		return TextureResource.IsValid() ? TextureResource->GetComponentDepth() : 0;
	}

	virtual FRHITexture* GetRHITexture() const override
	{
		const TSharedPtr<FDisplayClusterRender_TextureResource, ESPMode::ThreadSafe>& TextureResource = GetTextureResource();
		return TextureResource.IsValid() ? TextureResource->TextureRHI : nullptr;
	}

	//~ Begin TDisplayClusterDataCache
	/** Return DataCache timeout in frames. */
	static int32 GetDataCacheTimeOutInFrames();

	/** Return true if DataCache is enabled. */
	static bool IsDataCacheEnabled();

	/** Returns the unique name of this texture for DataCache. */
	inline const FString& GetDataCacheName() const
	{
		return UniqueTextureName;
	}

	/** Method for releasing a cached data item, called before its destructor. */
	inline void ReleaseDataCacheItem()
	{
		SetTextureResource(nullptr);
	}

	// ~~ End TDisplayClusterDataCache

protected:
	/** Set new texture resource. */
	void SetTextureResource(const TSharedPtr<FDisplayClusterRender_TextureResource, ESPMode::ThreadSafe>& InTextureResourcePtr);

	/** Get texture resource: supports both the game thread and the rendering thread. */
	const TSharedPtr<FDisplayClusterRender_TextureResource, ESPMode::ThreadSafe>& GetTextureResource() const;

private:
	// The texture's resource, can be NULL
	TSharedPtr<FDisplayClusterRender_TextureResource, ESPMode::ThreadSafe> TextureResourcePtr;

	// Value updated and returned by the render-thread to allow
	// fenceless update from the game-thread without causing
	// potential crash in the render thread.
	TSharedPtr<FDisplayClusterRender_TextureResource, ESPMode::ThreadSafe> TextureResourceProxyPtr;

	// Unique texture name
	FString UniqueTextureName;
};
