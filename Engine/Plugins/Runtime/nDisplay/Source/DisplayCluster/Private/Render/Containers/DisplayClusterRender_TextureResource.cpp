// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Containers/DisplayClusterRender_TextureResource.h"
#include "Containers/ResourceArray.h"
#include "RenderingThread.h"

//---------------------------------------------------------------------------------------
// FDisplayClusterRender_TextureResource
//---------------------------------------------------------------------------------------
FDisplayClusterRender_TextureResource::FDisplayClusterRender_TextureResource(const void* InTextureData, const uint32 InComponentDepth, const uint32 InBitDepth, uint32_t InWidth, uint32_t InHeight, bool bInHasCPUAccess)
{
	check(InTextureData);
	check(InWidth > 0);
	check(InHeight > 0);

	check(InComponentDepth > 0 && InComponentDepth <= 4);
	check(InBitDepth == 8 || InBitDepth == 16 || InBitDepth == 32);

	bHasCPUAccess = bInHasCPUAccess;

	static const EPixelFormat DestPixelFormatMapping[4][4] =
	{
		{ PF_G8,       PF_G16,     PF_Unknown, PF_Unknown },
		{ PF_R8G8,     PF_G16R16,  PF_Unknown, PF_Unknown },
		{ PF_R8G8B8A8, PF_Unknown, PF_Unknown, PF_Unknown },
		{ PF_R8G8B8A8, PF_Unknown, PF_Unknown, PF_A32B32G32R32F}
	};

	const EPixelFormat DestPixelFormat = DestPixelFormatMapping[InComponentDepth - 1][(InBitDepth>>3) - 1];
	check(DestPixelFormat != PF_Unknown);

	// Source data not equal to dest
	const int32 DestNumComponents = GPixelFormats[DestPixelFormat].NumComponents;

	const uint32 DataSize = CalculateImageBytes(InWidth, InHeight, 1, DestPixelFormat);
	TextureData = FMemory::Malloc(DataSize);

	const size_t SrcPixelSize = InComponentDepth * (InBitDepth >> 3);
	const size_t DestPixelSize = DestNumComponents * (InBitDepth >> 3);

	if (SrcPixelSize == DestPixelSize)
	{
		FMemory::Memcpy(TextureData, InTextureData, DataSize);
	}
	else
	{
		check(SrcPixelSize < DestPixelSize);

		FMemory::Memset(TextureData, 0, DataSize);

		const uint32 TotalPixels = InWidth * InHeight;
		uint8* DestPixel = (uint8*)TextureData;
		uint8* SrcPixel = (uint8*)InTextureData;

		// adopt source data to dest texture format:
		for (uint32 PixelIndex = 0; PixelIndex < TotalPixels; PixelIndex++)
		{
			FMemory::Memcpy(DestPixel, SrcPixel, SrcPixelSize);

			DestPixel += DestPixelSize;
			SrcPixel += SrcPixelSize;
		}
	}

	// Create texture from data:
	PixelFormat = DestPixelFormat;
	Width = InWidth;
	Height = InHeight;
	ComponentDepth = InComponentDepth;
}

FDisplayClusterRender_TextureResource::~FDisplayClusterRender_TextureResource()
{
	ReleaseTextureData();
}


void FDisplayClusterRender_TextureResource::InitializeRenderResource()
{
	bRenderResourceInitialized = true;

	// Initialize on render thread
	ENQUEUE_RENDER_COMMAND(DisplayClusterRenderTextureResource_Initialize)([This = SharedThis(this)](FRHICommandListImmediate& RHICmdList)
	{
		This->InitResource(RHICmdList);
	});
}

void FDisplayClusterRender_TextureResource::ReleaseRenderResource()
{
	bRenderResourceInitialized = false;

	// Release on render thread
	ENQUEUE_RENDER_COMMAND(DisplayClusterRenderTextureResource_Release)([This = SharedThis(this)](FRHICommandListImmediate& RHICmdList)
	{
		This->ReleaseResource();
	});
}

void FDisplayClusterRender_TextureResource::ReleaseTextureData()
{
	if (TextureData != nullptr)
	{
		FMemory::Free(TextureData);
		TextureData = nullptr;

		bHasCPUAccess = false;
	}
}

void FDisplayClusterRender_TextureResource::InitRHI(FRHICommandListBase& RHICmdList)
{
	check(IsInRenderingThread());

	if (TextureData == nullptr)
	{
		// Texture data is needed
		return;
	}

	const uint32 DataSize = CalculateImageBytes(Width, Height, 1, PixelFormat);
	FResourceBulkDataArrayView BulkDataInterface(TextureData, DataSize);

	// Create a texture from the data in memory.
	const FRHITextureCreateDesc Desc =
		FRHITextureCreateDesc::Create2D(TEXT("DisplayClusterRender_TextureResource"), Width, Height, PixelFormat)
		.SetFlags(ETextureCreateFlags::ShaderResource)
		.SetInitActionBulkData(&BulkDataInterface);

	TextureRHI = RHICmdList.CreateTexture(Desc);

	// CPU access not required, release from memory
	if (bHasCPUAccess == false)
	{
		ReleaseTextureData();
	}
}
