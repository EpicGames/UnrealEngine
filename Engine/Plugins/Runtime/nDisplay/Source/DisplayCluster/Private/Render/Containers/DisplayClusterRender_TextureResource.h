// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "TextureResource.h"
#include "Templates/SharedPointer.h"

/**
 * Creating a texture resource from memory data
 * Also, this texture data will be stored in memory if bHasCPUAccess is true.
 */
class FDisplayClusterRender_TextureResource
	: public FTextureResource
	, public TSharedFromThis<FDisplayClusterRender_TextureResource, ESPMode::ThreadSafe>
{
public:
	FDisplayClusterRender_TextureResource(const void* InTextureData, const uint32 InComponentDepth, const uint32 InBitDepth, uint32_t InWidth, uint32_t InHeight, bool bInHasCPUAccess);
	virtual ~FDisplayClusterRender_TextureResource();

	//~FTextureResource
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	//~~ FTextureResource

	/** Release texture data from memory. */
	void ReleaseTextureData();

	/** Initializes the render resource. */
	void InitializeRenderResource();

	/** Prepares the resource for deletion. */
	void ReleaseRenderResource();

	/** Whether this texture resource has data in CPU memory. */
	bool HasCPUAccess() const
	{
		return bHasCPUAccess;
	}

	/** Get ptr to resource data in memory. The variable bHasCPUAccess must be true. */
	bool IsRenderResourceInitialized() const
	{
		return bRenderResourceInitialized;
	}

	/** Get ptr to resource data in memory. The variable bHasCPUAccess must be true. */
	void* GetTextureData() const
	{
		return TextureData;
	}

	/** Get texture width. */
	uint32_t GetWidth() const
	{
		return Width;
	}

	/** Get texture height. */
	uint32_t GetHeight() const
	{
		return Height;
	}

	/** Get texture pixel format. */
	EPixelFormat GetPixelFormat() const
	{
		return PixelFormat;
	}

	/** Get number of components. */
	uint32 GetComponentDepth() const
	{
		return ComponentDepth;
	}

private:
	// RAW data of the texture. Mips not supported now
	void* TextureData = nullptr;
	
	// Width of the texture.
	uint32_t Width = 0;
	
	// Height of the texture.
	uint32_t Height = 0;

	// Number of components
	uint32 ComponentDepth = 0;
	
	// Format in which mip data is stored.
	EPixelFormat PixelFormat = EPixelFormat::PF_Unknown;
	
	// has CPU access to texture data, dont release from memory
	bool bHasCPUAccess = false;

	// Is render resource initialized
	bool bRenderResourceInitialized = false;
};
