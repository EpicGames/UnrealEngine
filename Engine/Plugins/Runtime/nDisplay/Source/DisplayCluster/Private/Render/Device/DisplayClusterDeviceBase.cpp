// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Device/DisplayClusterDeviceBase.h"

#include "IPDisplayCluster.h"
#include "IDisplayClusterCallbacks.h"
#include "Cluster/IPDisplayClusterClusterManager.h"
#include "Cluster/Controller/IDisplayClusterClusterNodeController.h"
#include "Config/IPDisplayClusterConfigManager.h"
#include "Game/IPDisplayClusterGameManager.h"
#include "Render/IPDisplayClusterRenderManager.h"

#include "Misc/DisplayClusterGlobals.h"
#include "Misc/DisplayClusterHelpers.h"
#include "Misc/DisplayClusterStrings.h"
#include "Misc/DisplayClusterLog.h"

#include "DisplayClusterConfigurationTypes.h"

#include "DisplayClusterRootActor.h"
#include "Components/DisplayClusterCameraComponent.h"
#include "Components/DisplayClusterScreenComponent.h"

#include "HAL/IConsoleManager.h"

#include "RHIStaticStates.h"
#include "RenderGraphUtils.h"
#include "Slate/SceneViewport.h"
#include "Slate/SlateViewportProvider.h"
#include "Framework/Application/SlateApplication.h"

#include "Render/GUILayer/DisplayClusterGuiLayerController.h"
#include "Render/PostProcess/IDisplayClusterPostProcess.h"
#include "Render/Presentation/DisplayClusterPresentationBase.h"
#include "Render/Projection/IDisplayClusterProjectionPolicy.h"
#include "Render/Projection/IDisplayClusterProjectionPolicyFactory.h"
#include "Render/Synchronization/IDisplayClusterRenderSyncPolicy.h"


#include "Render/Viewport/DisplayClusterViewport.h"
#include "Render/Viewport/DisplayClusterViewportManager.h"
#include "Render/Viewport/DisplayClusterViewportManagerProxy.h"
#include "Render/Viewport/DisplayClusterViewportStereoscopicPass.h"
#include "Render/Viewport/IDisplayClusterViewportProxy.h"

#include "Render/Viewport/Configuration/DisplayClusterViewportConfigurationHelpers_Postprocess.h"

#include "RenderGraphUtils.h"
#include "RenderGraphBuilder.h"

#include <utility>

namespace UE::DisplayCluster::DeviceBaseHelpers
{
	static inline IDisplayCluster& GetDisplayClusterAPI()
	{
		static IDisplayCluster& DisplayClusterAPISingleton = IDisplayCluster::Get();

		return DisplayClusterAPISingleton;
	}
};
using namespace UE::DisplayCluster;

// Enable/Disable ClearTexture for RTT after resolving to the backbuffer
static TAutoConsoleVariable<int32> CVarClearTextureEnabled(
	TEXT("nDisplay.render.ClearTextureEnabled"),
	1,
	TEXT("Enables RTT cleaning for left / mono eye at end of frame.\n")
	TEXT("0 : disabled\n")
	TEXT("1 : enabled\n")
	,
	ECVF_RenderThreadSafe
);

FDisplayClusterDeviceBase::FDisplayClusterDeviceBase(EDisplayClusterRenderFrameMode InRenderFrameMode)
	: RenderFrameMode(InRenderFrameMode)
{
	UE_LOGF(LogDisplayClusterRender, Log, "Created DCRenderDevice");

	FSlateApplication::Get().GetRenderer()->OnBackBufferReadyToPresent().AddRaw(this, &FDisplayClusterDeviceBase::OnBackBufferReadyToPresent);
}

FDisplayClusterDeviceBase::~FDisplayClusterDeviceBase()
{
	//@todo: delete singleton object IDisplayClusterViewportManager

	FSlateApplication::Get().GetRenderer()->OnBackBufferReadyToPresent().RemoveAll(this);
}

FDisplayClusterViewportManager* FDisplayClusterDeviceBase::GetViewportManager() const
{
	return ViewportManagerWeakPtr.IsValid() ? ViewportManagerWeakPtr.Pin().Get() : nullptr;
}

FDisplayClusterViewportManagerProxy* FDisplayClusterDeviceBase::GetViewportManagerProxy_RenderThread() const
{
	return ViewportManagerProxyWeakPtr.IsValid() ? ViewportManagerProxyWeakPtr.Pin().Get() : nullptr;
}

//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterStereoDevice
//////////////////////////////////////////////////////////////////////////////////////////////

bool FDisplayClusterDeviceBase::Initialize()
{
	if (GDisplayCluster->GetOperationMode() == EDisplayClusterOperationMode::Disabled)
	{
		return false;
	}

	return true;
}

void FDisplayClusterDeviceBase::StartScene(UWorld* InWorld)
{
}

void FDisplayClusterDeviceBase::EndScene()
{
}

void FDisplayClusterDeviceBase::OnBackBufferReadyToPresent(SWindow& Window, ISlateViewportProvider& ViewportProvider)
{
	if (CustomPresentHandler)
	{
		return;
	}

	if (GEngine->GameViewport && GEngine->GameViewport->Viewport == Window.GetViewport().Get())
	{
		// Current sync policy
		TSharedPtr<IDisplayClusterRenderSyncPolicy> SyncPolicy = GDisplayCluster->GetRenderMgr()->GetCurrentSynchronizationPolicy();
		check(SyncPolicy.IsValid());

		// Create present handler
		CustomPresentHandler = CreatePresentationObject(GEngine->GameViewport->Viewport, SyncPolicy);
		check(CustomPresentHandler);

		ViewportProvider.SetCustomPresent(CustomPresentHandler);

		GDisplayCluster->GetCallbacks().OnDisplayClusterCustomPresentSet().Broadcast();
	}
}

IDisplayClusterPresentation* FDisplayClusterDeviceBase::GetPresentation() const
{
	return CustomPresentHandler;
}

//////////////////////////////////////////////////////////////////////////////////////////////
// IStereoRendering
//////////////////////////////////////////////////////////////////////////////////////////////
bool FDisplayClusterDeviceBase::IsStereoEnabled() const
{
	return true;
}

bool FDisplayClusterDeviceBase::IsStereoEnabledOnNextFrame() const
{
	return true;
}

bool FDisplayClusterDeviceBase::EnableStereo(bool stereo /*= true*/)
{
	return true;
}

EStereoscopicPass FDisplayClusterDeviceBase::GetViewPassForIndex(bool bStereoRequested, int32 ViewIndex) const
{
	if (bStereoRequested)
	{
		if (IsInRenderingThread())
		{
			if (IDisplayClusterViewportManagerProxy* ViewportManagerProxy = GetViewportManagerProxy_RenderThread())
			{
				uint32 ViewportContextNum = 0;
				IDisplayClusterViewportProxy* ViewportProxy = ViewportManagerProxy->FindViewport_RenderThread(ViewIndex, &ViewportContextNum);
				if (ViewportProxy)
				{
					const FDisplayClusterViewport_Context& Context = ViewportProxy->GetContexts_RenderThread()[ViewportContextNum];
					return Context.StereoscopicPass;
				}
			}
		}
		else
		{
			if (IDisplayClusterViewportManager* ViewportManager = GetViewportManager())
			{
				uint32 ViewportContextNum = 0;
				IDisplayClusterViewport* ViewportPtr = ViewportManager->FindViewport(ViewIndex, &ViewportContextNum);
				if (ViewportPtr)
				{
					const FDisplayClusterViewport_Context& Context = ViewportPtr->GetContexts()[ViewportContextNum];
					return Context.StereoscopicPass;
				}
			}
		}
	}

	return EStereoscopicPass::eSSP_FULL;
}

void FDisplayClusterDeviceBase::AdjustViewRect(int32 StereoViewIndex, int32& X, int32& Y, uint32& SizeX, uint32& SizeY) const
{
	check(IsInGameThread());

	if (!FDisplayClusterViewportStereoscopicPass::IsValidStereoViewIndex(StereoViewIndex))
	{
		return;
	}

	IDisplayClusterViewportManager* ViewportManager = GetViewportManager();
	if (ViewportManager == nullptr || ViewportManager->GetConfiguration().IsSceneOpened() == false)
	{
		return;
	}

	uint32 ViewportContextNum = 0;
	IDisplayClusterViewport* ViewportPtr = ViewportManager->FindViewport(StereoViewIndex, &ViewportContextNum);
	if (ViewportPtr == nullptr)
	{
		UE_LOGF(LogDisplayClusterRender, Warning, "Viewport StereoViewIndex='%i' not found", StereoViewIndex);
		return;
	}

	const FIntRect& ViewRect = ViewportPtr->GetContexts()[ViewportContextNum].RenderTargetRect;

	X = ViewRect.Min.X;
	Y = ViewRect.Min.Y;

	SizeX = ViewRect.Width();
	SizeY = ViewRect.Height();

	UE_LOGF(LogDisplayClusterRender, Verbose, "Adjusted view rect: Viewport='%ls', ViewIndex=%d, [%d,%d - %d,%d]", *ViewportPtr->GetId(), ViewportContextNum, ViewRect.Min.X, ViewRect.Min.Y, ViewRect.Max.X, ViewRect.Max.Y);
}

void FDisplayClusterDeviceBase::CalculateStereoViewOffset(const int32 StereoViewIndex, FRotator& ViewRotation, const float WorldToMeters, FVector& ViewLocation)
{
	check(IsInGameThread());
	check(WorldToMeters > 0.f);

	if (!FDisplayClusterViewportStereoscopicPass::IsValidStereoViewIndex(StereoViewIndex))
	{
		return;
	}

	IDisplayClusterViewportManager* ViewportManager = GetViewportManager();

	uint32 ViewportContextNum = 0;
	IDisplayClusterViewport* ViewportPtr = ViewportManager ? ViewportManager->FindViewport(StereoViewIndex, &ViewportContextNum) : nullptr;
	if (ViewportPtr == nullptr)
	{
		return;
	}

	// The camera position has already been determined from the SetupViewPoint() function
	// Obtaining the offset of the stereo eye and the values of the projection clipping plane for the given viewport was moved inside CalculateView().
	// Perform view calculations on a policy side
	if (ViewportPtr->CalculateView(ViewportContextNum, ViewLocation, ViewRotation, WorldToMeters) == false)
	{
#if WITH_EDITOR
		// Hide spam in logs when configuring VP in editor [UE-114493]
		static const bool bIsEditorOperationMode = DeviceBaseHelpers::GetDisplayClusterAPI().GetOperationMode() == EDisplayClusterOperationMode::Editor;
		if (!bIsEditorOperationMode)
#endif
		{
			UE_LOGF(LogDisplayClusterRender, Warning, "Couldn't compute view parameters for Viewport %ls, ViewIdx: %d", *ViewportPtr->GetId(), ViewportContextNum);
		}
	}

	UE_LOGF(LogDisplayClusterRender, VeryVerbose, "ViewLoc: %ls, ViewRot: %ls", *ViewLocation.ToString(), *ViewRotation.ToString());
}

FMatrix FDisplayClusterDeviceBase::GetStereoProjectionMatrix(const int32 StereoViewIndex) const
{
	check(IsInGameThread());

	FMatrix PrjMatrix = FMatrix::Identity;
	if (!FDisplayClusterViewportStereoscopicPass::IsValidStereoViewIndex(StereoViewIndex))
	{
		return PrjMatrix;
	}

	IDisplayClusterViewportManager* ViewportManager = GetViewportManager();
	if (ViewportManager && ViewportManager->GetConfiguration().IsSceneOpened())
	{
		uint32 ViewportContextNum = 0;
		IDisplayClusterViewport* ViewportPtr = ViewportManager->FindViewport(StereoViewIndex, &ViewportContextNum);
		if (ViewportPtr == nullptr)
		{
			UE_LOGF(LogDisplayClusterRender, Warning, "Viewport StereoViewIndex='%i' not found", StereoViewIndex);
		}
		else
		if (ViewportPtr->GetProjectionMatrix(ViewportContextNum, PrjMatrix) == false)
		{
			UE_LOGF(LogDisplayClusterRender, Warning, "Got invalid projection matrix: Viewport %ls, ViewIdx: %d", *ViewportPtr->GetId(), ViewportContextNum);
		}
	}
	
	return PrjMatrix;
}

bool FDisplayClusterDeviceBase::HasExternalViewState(const int32 StereoViewIndex) const
{
	check(IsInGameThread());

	// true if StereoViewIndex is in nDisplay range.
	return FDisplayClusterViewportStereoscopicPass::IsValidStereoViewIndex(StereoViewIndex);
}

FSceneViewStateInterface* FDisplayClusterDeviceBase::GetExternalViewState(const int32 StereoViewIndex)
{
	check(IsInGameThread());
	check(FDisplayClusterViewportStereoscopicPass::IsValidStereoViewIndex(StereoViewIndex));

	FDisplayClusterViewportManager* ViewportManager = GetViewportManager();
	if (ViewportManager && ViewportManager->GetConfiguration().IsSceneOpened())
	{
		uint32 ViewportContextNum = 0;
		if (FDisplayClusterViewport* ViewportPtr = ViewportManager->ImplFindViewport(StereoViewIndex, &ViewportContextNum))
		{
			TSharedPtr<FSceneViewStateReference, ESPMode::ThreadSafe> ViewState = ViewportPtr->GetOrCreateViewState(ViewportContextNum);
			return ViewState.IsValid() ? ViewState->GetReference() : nullptr;
		}

		UE_LOGF(LogDisplayClusterRender, Warning, "Viewport StereoViewIndex='%i' not found", StereoViewIndex);
	}

	return nullptr;
}

bool FDisplayClusterDeviceBase::BeginNewFrame(FViewport* InViewport, UWorld* InWorld, FDisplayClusterRenderFrame& OutRenderFrame)
{
	check(IsInGameThread());
	check(InViewport);

	if (ADisplayClusterRootActor* RootActor = DeviceBaseHelpers::GetDisplayClusterAPI().GetGameMgr()->GetRootActor())
	{
		if (IDisplayClusterViewportManager* ViewportManagerPtr = RootActor->GetOrCreateViewportManager())
		{
			const FString LocalNodeId = DeviceBaseHelpers::GetDisplayClusterAPI().GetConfigMgr()->GetLocalNodeId();

			// Get preview settings from RootActor properties
			FDisplayClusterViewport_PreviewSettings NewPreviewSettings = RootActor->GetPreviewSettings(true);
			NewPreviewSettings.bPreviewEnable = false;

			// Dont use preview setting on primary RootActor in game
			ViewportManagerPtr->GetConfiguration().SetPreviewSettings(NewPreviewSettings);

			// Update local node viewports (update\create\delete) and build new render frame
			if (ViewportManagerPtr->GetConfiguration().UpdateConfigurationForClusterNode(RenderFrameMode, InWorld, LocalNodeId))
			{
				if (ViewportManagerPtr->BeginNewFrame(InViewport, OutRenderFrame))
				{
					// update total number of views for this frame (in multiple families)
					DesiredNumberOfViews = OutRenderFrame.DesiredNumberOfViews;

					return true;
				}
			}
		}
	}

	return false;
}

void FDisplayClusterDeviceBase::InitializeNewFrame()
{
	check(IsInGameThread());

	if (ADisplayClusterRootActor* RootActor = DeviceBaseHelpers::GetDisplayClusterAPI().GetGameMgr()->GetRootActor())
	{
		if (IDisplayClusterViewportManager* ViewportManager = RootActor->GetOrCreateViewportManager())
		{
			// Begin use viewport manager for current frame
			ViewportManagerWeakPtr = ViewportManager->ToSharedRef();

			// Initialize frame for render
			ViewportManager->InitializeNewFrame();

			FDisplayClusterViewportManagerProxy* ViewportManagerProxyPtr = static_cast<FDisplayClusterViewportManagerProxy*>(ViewportManager->GetProxy());

			// Send viewport manager proxy on render thread
			ENQUEUE_RENDER_COMMAND(DisplayClusterDevice_SetViewportManagerProxy)(
				[DCRenderDevice = SharedThis(this), NewViewportManagerProxy = ViewportManagerProxyPtr->AsShared()](FRHICommandListImmediate& RHICmdList)
				{
					DCRenderDevice->ViewportManagerProxyWeakPtr = NewViewportManagerProxy;
				});
		}
	}
}

void FDisplayClusterDeviceBase::FinalizeNewFrame()
{
	IDisplayClusterViewportManager* ViewportManager = GetViewportManager();
	if (ViewportManager)
	{
		ViewportManager->FinalizeNewFrame();
	}

	// reset viewport manager ptr on game thread
	ViewportManagerWeakPtr.Reset();
}

DECLARE_GPU_STAT_NAMED(nDisplay_Device_RenderTexture, TEXT("nDisplay RenderDevice::RenderTexture"));

void FDisplayClusterDeviceBase::RenderTexture_RenderThread(class FRDGBuilder& GraphBuilder, FRDGTextureRef BackBuffer, FRDGTextureRef SrcTexture, FVector2f WindowSize) const
{
	const FIntVector SrcSize = SrcTexture->Desc.GetSize();
	const FIntVector DstSize = BackBuffer->Desc.GetSize();

	FRHICopyTextureInfo CopyInfo;
	CopyInfo.Size.X = FMath::Min(SrcSize.X, DstSize.X);
	CopyInfo.Size.Y = FMath::Min(SrcSize.Y, DstSize.Y);

	// Allow custom GUI related jobs to be done first. After that, we'll use the texture it returns (can be the same).
	SrcTexture = FDisplayClusterGuiLayerController::Get().ProcessFinalTexture_RenderThread(GraphBuilder, SrcTexture);

	if (SrcTexture->Desc.Format != BackBuffer->Desc.Format)
	{
		// If formats differ on Vulkan this requires an explicit conversion (e.g. A2B10G10R10 src vs RGBA8 backbuffer on Vulkan)
		FRDGDrawTextureInfo DrawInfo;
		DrawInfo.Size = FIntPoint(CopyInfo.Size.X, CopyInfo.Size.Y);
		AddDrawTexturePass(GraphBuilder, GetGlobalShaderMap(GMaxRHIFeatureLevel), SrcTexture, BackBuffer, DrawInfo);
	}
	else
	{
		AddCopyTexturePass(GraphBuilder, SrcTexture, BackBuffer, CopyInfo);
	}

	if (RenderFrameMode == EDisplayClusterRenderFrameMode::Stereo)
	{
		if (FDisplayClusterViewportManagerProxy* ViewportManagerProxy = GetViewportManagerProxy_RenderThread())
		{
			// QuadBufStereo: Copy RIGHT_EYE to backbuffer
			ViewportManagerProxy->ImplResolveFrameTargetToBackBuffer_RenderThread(GraphBuilder, 1, 1, BackBuffer, WindowSize);
		}
	}

	const bool bClearTextureEnabled = CVarClearTextureEnabled.GetValueOnRenderThread() != 0;
	if (bClearTextureEnabled)
	{
		// Clear render target before out frame resolving, help to make things look better visually for console/resize, etc.
		AddClearRenderTargetPass(GraphBuilder, SrcTexture);
	}
};

//////////////////////////////////////////////////////////////////////////////////////////////
// IStereoRenderTargetManager
//////////////////////////////////////////////////////////////////////////////////////////////

void FDisplayClusterDeviceBase::CalculateRenderTargetSize(const class FViewport& Viewport, uint32& InOutSizeX, uint32& InOutSizeY)
{
	InOutSizeX = FMath::Max(1, (int32)InOutSizeX);
	InOutSizeY = FMath::Max(1, (int32)InOutSizeY);
}

bool FDisplayClusterDeviceBase::NeedReAllocateViewportRenderTarget(const class FViewport& Viewport)
{
	check(IsInGameThread());

	// Get current RT size
	const FIntPoint rtSize = Viewport.GetRenderTargetTextureSizeXY();

	// Get desired RT size
	uint32 newSizeX = rtSize.X;
	uint32 newSizeY = rtSize.Y;

	CalculateRenderTargetSize(Viewport, newSizeX, newSizeY);

	// Here we conclude if need to re-allocate
	const bool Result = (newSizeX != rtSize.X || newSizeY != rtSize.Y);

	UE_LOGF(LogDisplayClusterRender, Verbose, "Is reallocate viewport render target needed: %d", Result ? 1 : 0);

	if (Result)
	{
		UE_LOGF(LogDisplayClusterRender, Log, "Need to re-allocate render target: cur %d:%d, new %d:%d", rtSize.X, rtSize.Y, newSizeX, newSizeY);
	}

	return Result;
}

//////////////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterDeviceBase
//////////////////////////////////////////////////////////////////////////////////////////////
void FDisplayClusterDeviceBase::StartFinalPostprocessSettings(FPostProcessSettings* StartPostProcessingSettings, const enum EStereoscopicPass StereoPassType, const int32 StereoViewIndex)
{
	check(IsInGameThread());

	// eSSP_FULL pass reserved for UE internal render
	if (StereoPassType != EStereoscopicPass::eSSP_FULL && StartPostProcessingSettings)
	{
		IDisplayClusterViewportManager* ViewportManager = GetViewportManager();
		uint32 ContextNum = 0;
		if (IDisplayClusterViewport* Viewport = ViewportManager ? ViewportManager->FindViewport(StereoViewIndex, &ContextNum) : nullptr)
		{
			Viewport->GetViewport_CustomPostProcessSettings().ApplyCustomPostProcess(Viewport, ContextNum, IDisplayClusterViewport_CustomPostProcessSettings ::ERenderPass::Start, *StartPostProcessingSettings);
		}
	}
}

bool FDisplayClusterDeviceBase::OverrideFinalPostprocessSettings(FPostProcessSettings* OverridePostProcessingSettings, const enum EStereoscopicPass StereoPassType, const int32 StereoViewIndex, float& BlendWeight)
{
	check(IsInGameThread());

	// eSSP_FULL pass reserved for UE internal render
	if (StereoPassType != EStereoscopicPass::eSSP_FULL)
	{
		IDisplayClusterViewportManager* ViewportManager = GetViewportManager();
		uint32 ContextNum = 0;
		if (IDisplayClusterViewport* Viewport = ViewportManager ? ViewportManager->FindViewport(StereoViewIndex, &ContextNum) : nullptr)
		{
			return Viewport->GetViewport_CustomPostProcessSettings().ApplyCustomPostProcess(Viewport, ContextNum, IDisplayClusterViewport_CustomPostProcessSettings::ERenderPass::Override, *OverridePostProcessingSettings, &BlendWeight);
		}
	}

	return false;
}

void FDisplayClusterDeviceBase::EndFinalPostprocessSettings(FPostProcessSettings* FinalPostProcessingSettings, const enum EStereoscopicPass StereoPassType, const int32 StereoViewIndex)
{
	check(IsInGameThread());

	// eSSP_FULL pass reserved for UE internal render
	if (StereoPassType != EStereoscopicPass::eSSP_FULL)
	{
		IDisplayClusterViewportManager* ViewportManager = GetViewportManager();
		uint32 ContextNum = 0;
		if (IDisplayClusterViewport* Viewport = ViewportManager ? ViewportManager->FindViewport(StereoViewIndex, &ContextNum) : nullptr)
		{
			Viewport->GetViewport_CustomPostProcessSettings().ApplyCustomPostProcess(Viewport, ContextNum, IDisplayClusterViewport_CustomPostProcessSettings::ERenderPass::Final, *FinalPostProcessingSettings);
		}
	}
}
