// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "RHI.h"
#include "RHIResources.h"
#include "Engine/Scene.h"
#include "Slate/SlateViewportProvider.h"
#include "StereoRenderTargetManager.h"

#include "Render/Device/IDisplayClusterRenderDevice.h"
#include "Render/IDisplayClusterRenderManager.h"

#include "Containers/Queue.h"
#include "Templates/SharedPointer.h"

class IDisplayClusterPostProcess;
class FDisplayClusterPresentationBase;
class FDisplayClusterViewportManager;
class FDisplayClusterViewportManagerProxy;
class FSceneView;

/**
 * Abstract render device
 */
class FDisplayClusterDeviceBase
	: public IStereoRenderTargetManager
	, public IDisplayClusterRenderDevice
	, public TSharedFromThis<FDisplayClusterDeviceBase, ESPMode::ThreadSafe>
{
public:
	FDisplayClusterDeviceBase() = delete;
	FDisplayClusterDeviceBase(EDisplayClusterRenderFrameMode InRenderFrameMode);
	virtual ~FDisplayClusterDeviceBase();

public:
	//////////////////////////////////////////////////////////////////////////////////////////////
	// IDisplayClusterRenderDevice
	//////////////////////////////////////////////////////////////////////////////////////////////
	virtual bool Initialize() override;
	virtual void StartScene(UWorld* InWorld) override;
	virtual void EndScene() override;

	virtual IDisplayClusterPresentation* GetPresentation() const override;

	virtual bool BeginNewFrame(FViewport* InViewport, UWorld* InWorld, FDisplayClusterRenderFrame& OutRenderFrame) override;

	virtual void InitializeNewFrame() override;
	virtual void FinalizeNewFrame() override;

public:
	//////////////////////////////////////////////////////////////////////////////////////////////
	// IStereoRendering
	//////////////////////////////////////////////////////////////////////////////////////////////
	virtual bool IsStereoEnabled() const override;
	virtual bool IsStereoEnabledOnNextFrame() const override;
	virtual bool EnableStereo(bool bStereoEnabled = true) override;
	virtual void AdjustViewRect(int32 ViewIndex, int32& X, int32& Y, uint32& SizeX, uint32& SizeY) const override;
	virtual void CalculateStereoViewOffset(const int32 ViewIndex, FRotator& ViewRotation, const float WorldToMeters, FVector& ViewLocation) override;
	virtual FMatrix GetStereoProjectionMatrix(const int32 ViewIndex) const override;
	
	virtual bool HasExternalViewState(const int32 ViewIndex) const override;
	virtual FSceneViewStateInterface* GetExternalViewState(const int32 ViewIndex) override;

	virtual void RenderTexture_RenderThread(class FRDGBuilder& GraphBuilder, FRDGTextureRef BackBuffer, FRDGTextureRef SrcTexture, FVector2f WindowSize) const override;

	// So far, we don’t need any region bounds
	virtual FVector2D GetTextSafeRegionBounds() const override
	{
		return FVector2D(1.f, 1.f);
	}

	virtual int32 GetDesiredNumberOfViews(bool bStereoRequested) const override
	{
		return FMath::Max(1, DesiredNumberOfViews);
	}

	virtual EStereoscopicPass GetViewPassForIndex(bool bStereoRequested, int32 ViewIndex) const override;

	virtual IStereoRenderTargetManager* GetRenderTargetManager() override
	{ return this; }

protected:
	//////////////////////////////////////////////////////////////////////////////////////////////
	// IStereoRenderTargetManager
	//////////////////////////////////////////////////////////////////////////////////////////////
	virtual bool ShouldUseSeparateRenderTarget() const override
	{ return true; }

	virtual void CalculateRenderTargetSize(const class FViewport& Viewport, uint32& InOutSizeX, uint32& InOutSizeY) override;
	virtual bool NeedReAllocateViewportRenderTarget(const class FViewport& Viewport) override;

protected:
	// Factory method to instantiate an output presentation class implementation
	virtual FDisplayClusterPresentationBase* CreatePresentationObject(FViewport* const Viewport, TSharedPtr<IDisplayClusterRenderSyncPolicy>& SyncPolicy) = 0;

	// Checks if custom post processing settings is assigned for specific viewport and assign them to be used
	virtual void StartFinalPostprocessSettings(struct FPostProcessSettings* StartPostProcessingSettings, const enum EStereoscopicPass StereoPassType, const int32 StereoViewIndex) override;
	virtual bool OverrideFinalPostprocessSettings(struct FPostProcessSettings* OverridePostProcessingSettings, const enum EStereoscopicPass StereoPassType, const int32 StereoViewIndex, float& BlendWeight) override;
	virtual void EndFinalPostprocessSettings(struct FPostProcessSettings* FinalPostProcessingSettings, const enum EStereoscopicPass StereoPassType, const int32 StereoViewIndex) override;

	/** Get a pointer to the DC ViewportManager if it still exists. */
	FDisplayClusterViewportManager* GetViewportManager() const;

	/** Get a pointer to the DC ViewportManagerProxy if it still exists. */
	FDisplayClusterViewportManagerProxy* GetViewportManagerProxy_RenderThread() const;

private:
	void OnBackBufferReadyToPresent(SWindow& Window, ISlateViewportProvider& ViewportProvider);

	// Pointer to the DC ViewportManager from the active DCRA that is currently being used for rendering.
	TWeakPtr<FDisplayClusterViewportManager, ESPMode::ThreadSafe> ViewportManagerWeakPtr;

	// Pointer to the DC ViewportManagerProxy from the active DCRA that is currently being used for rendering.
	mutable TWeakPtr<FDisplayClusterViewportManagerProxy, ESPMode::ThreadSafe> ViewportManagerProxyWeakPtr;

	EDisplayClusterRenderFrameMode RenderFrameMode = EDisplayClusterRenderFrameMode::Mono;
	int32 DesiredNumberOfViews = 0;

	// Data access synchronization
	mutable FCriticalSection InternalsSyncScope;

	// Pointer to the current presentation handler
	FDisplayClusterPresentationBase* CustomPresentHandler = nullptr;
};
