// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Device/DisplayClusterRenderDeviceFactoryInternal.h"

#if PLATFORM_WINDOWS
#include "Render/Device/Monoscopic/Windows/DisplayClusterDeviceMonoscopicDX11.h"
#include "Render/Device/Monoscopic/Windows/DisplayClusterDeviceMonoscopicDX12.h"
#include "Render/Device/QuadBufferStereo/Windows/DisplayClusterDeviceQuadBufferStereoDX11.h"
#include "Render/Device/QuadBufferStereo/Windows/DisplayClusterDeviceQuadBufferStereoDX12.h"
#include "Render/Device/SideBySide/Windows/DisplayClusterDeviceSideBySideDX11.h"
#include "Render/Device/SideBySide/Windows/DisplayClusterDeviceSideBySideDX12.h"
#include "Render/Device/TopBottom/Windows/DisplayClusterDeviceTopBottomDX11.h"
#include "Render/Device/TopBottom/Windows/DisplayClusterDeviceTopBottomDX12.h"
#endif

#include "Render/Device/Monoscopic/DisplayClusterDeviceMonoscopicVulkan.h"
#include "Render/Device/QuadBufferStereo/DisplayClusterDeviceQuadBufferStereoVulkan.h"
#include "Render/Device/SideBySide/DisplayClusterDeviceSideBySideVulkan.h"
#include "Render/Device/TopBottom/DisplayClusterDeviceTopBottomVulkan.h"

#include "Misc/DisplayClusterLog.h"
#include "Misc/DisplayClusterStrings.h"


TSharedPtr<IDisplayClusterRenderDevice, ESPMode::ThreadSafe> FDisplayClusterRenderDeviceFactoryInternal::Create(const FString& InDeviceType)
{
	const ERHIInterfaceType RHIType = RHIGetInterfaceType();

	// Monoscopic
	if (InDeviceType.Equals(DisplayClusterStrings::args::dev::Mono, ESearchCase::IgnoreCase))
	{
#if PLATFORM_WINDOWS
		if (RHIType == ERHIInterfaceType::D3D11)
		{
			UE_LOGF(LogDisplayClusterRender, Log, "Instantiating DX11 monoscopic device...");
			return MakeShared<FDisplayClusterDeviceMonoscopicDX11, ESPMode::ThreadSafe>();
		}
		else if (RHIType == ERHIInterfaceType::D3D12)
		{
			UE_LOGF(LogDisplayClusterRender, Log, "Instantiating DX12 monoscopic device...");
			return MakeShared<FDisplayClusterDeviceMonoscopicDX12, ESPMode::ThreadSafe>();
		}
		else
#endif
		if (RHIType == ERHIInterfaceType::Vulkan)
		{
			UE_LOGF(LogDisplayClusterRender, Log, "Instantiating Vulkan monoscopic device...");
			return MakeShared<FDisplayClusterDeviceMonoscopicVulkan, ESPMode::ThreadSafe>();
		}
	}
	// Quad buffer stereo
	else if (InDeviceType.Equals(DisplayClusterStrings::args::dev::QBS, ESearchCase::IgnoreCase))
	{
#if PLATFORM_WINDOWS
		if (RHIType == ERHIInterfaceType::D3D11)
		{
			UE_LOGF(LogDisplayClusterRender, Log, "Instantiating D3D11 quad buffer stereo device...");
			return MakeShared<FDisplayClusterDeviceQuadBufferStereoDX11, ESPMode::ThreadSafe>();
		}
		else if (RHIType == ERHIInterfaceType::D3D12)
		{
			UE_LOGF(LogDisplayClusterRender, Log, "Instantiating D3D12 quad buffer stereo device...");
			return MakeShared<FDisplayClusterDeviceQuadBufferStereoDX12, ESPMode::ThreadSafe>();
		}
		else
#endif
		if (RHIType == ERHIInterfaceType::Vulkan)
		{
			UE_LOGF(LogDisplayClusterRender, Log, "Instantiating Vulkan quad buffer stereo device...");
			return MakeShared<FDisplayClusterDeviceQuadBufferStereoVulkan, ESPMode::ThreadSafe>();
		}
	}
	// Side-by-side
	else if (InDeviceType.Equals(DisplayClusterStrings::args::dev::SbS, ESearchCase::IgnoreCase))
	{
#if PLATFORM_WINDOWS
		if (RHIType == ERHIInterfaceType::D3D11)
		{
			UE_LOGF(LogDisplayClusterRender, Log, "Instantiating D3D11 side-by-side stereo device...");
			return MakeShared<FDisplayClusterDeviceSideBySideDX11, ESPMode::ThreadSafe>();
		}
		else if (RHIType == ERHIInterfaceType::D3D12)
		{
			UE_LOGF(LogDisplayClusterRender, Log, "Instantiating D3D12 side-by-side stereo device...");
			return MakeShared<FDisplayClusterDeviceSideBySideDX12, ESPMode::ThreadSafe>();
		}
		else
#endif
		if (RHIType == ERHIInterfaceType::Vulkan)
		{
			UE_LOGF(LogDisplayClusterRender, Log, "Instantiating Vulkan side-by-side stereo device...");
			return MakeShared<FDisplayClusterDeviceSideBySideVulkan, ESPMode::ThreadSafe>();
		}
	}
	// Top-bottom
	else if (InDeviceType.Equals(DisplayClusterStrings::args::dev::TB, ESearchCase::IgnoreCase))
	{
#if PLATFORM_WINDOWS
		if (RHIType == ERHIInterfaceType::D3D11)
		{
			UE_LOGF(LogDisplayClusterRender, Log, "Instantiating D3D11 top-bottom stereo device...");
			return MakeShared<FDisplayClusterDeviceTopBottomDX11, ESPMode::ThreadSafe>();
		}
		else if (RHIType == ERHIInterfaceType::D3D12)
		{
			UE_LOGF(LogDisplayClusterRender, Log, "Instantiating D3D12 top-bottom stereo device...");
			return MakeShared<FDisplayClusterDeviceTopBottomDX12, ESPMode::ThreadSafe>();
		}
		else
#endif
		if (RHIType == ERHIInterfaceType::Vulkan)
		{
			UE_LOGF(LogDisplayClusterRender, Log, "Instantiating Vulkan top-bottom stereo device...");
			return MakeShared<FDisplayClusterDeviceTopBottomVulkan, ESPMode::ThreadSafe>();
		}
	}

	UE_LOGF(LogDisplayClusterRender, Warning, "An internal rendering device factory couldn't create a device %ls:%ls", GDynamicRHI->GetName(), *InDeviceType);

	return nullptr;
}
