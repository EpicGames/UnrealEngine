// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Render/Device/IDisplayClusterRenderDeviceFactory.h"


/**
 * Factory for internal rendering devices
 */
class FDisplayClusterRenderDeviceFactoryInternal
	: public IDisplayClusterRenderDeviceFactory
{
public:
	FDisplayClusterRenderDeviceFactoryInternal() = default;
	virtual ~FDisplayClusterRenderDeviceFactoryInternal() = default;

public:
	virtual TSharedPtr<IDisplayClusterRenderDevice, ESPMode::ThreadSafe> Create(const FString& InDeviceType) override;
};
