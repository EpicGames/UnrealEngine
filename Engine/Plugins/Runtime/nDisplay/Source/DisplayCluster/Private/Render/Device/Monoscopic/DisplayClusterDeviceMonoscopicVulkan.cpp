// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Device/Monoscopic/DisplayClusterDeviceMonoscopicVulkan.h"
#include "Render/Presentation/DisplayClusterPresentationVulkan.h"


FDisplayClusterDeviceMonoscopicVulkan::FDisplayClusterDeviceMonoscopicVulkan()
	: FDisplayClusterDeviceBase(EDisplayClusterRenderFrameMode::Mono)
{
}

FDisplayClusterPresentationBase* FDisplayClusterDeviceMonoscopicVulkan::CreatePresentationObject(FViewport* const Viewport, TSharedPtr<IDisplayClusterRenderSyncPolicy>& SyncPolicy)
{
	return new FDisplayClusterPresentationVulkan(Viewport, SyncPolicy);
}
