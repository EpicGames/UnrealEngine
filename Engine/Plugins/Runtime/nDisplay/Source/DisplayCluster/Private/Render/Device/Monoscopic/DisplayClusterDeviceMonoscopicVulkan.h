// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Render/Device/DisplayClusterDeviceBase.h"

/**
 * Monoscopic render device (Vulkan)
 */
class FDisplayClusterDeviceMonoscopicVulkan
	: public FDisplayClusterDeviceBase
{
public:
	FDisplayClusterDeviceMonoscopicVulkan();
	virtual ~FDisplayClusterDeviceMonoscopicVulkan() = default;

protected:
	virtual FDisplayClusterPresentationBase* CreatePresentationObject(FViewport* const Viewport, TSharedPtr<IDisplayClusterRenderSyncPolicy>& SyncPolicy) override;
};
