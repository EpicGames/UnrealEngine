// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Render/Device/DisplayClusterDeviceBase.h"

/**
 * Monoscopic render device (DirectX 11)
 */
class FDisplayClusterDeviceMonoscopicDX11
	: public FDisplayClusterDeviceBase
{
public:
	FDisplayClusterDeviceMonoscopicDX11();
	virtual ~FDisplayClusterDeviceMonoscopicDX11() = default;

protected:
	virtual FDisplayClusterPresentationBase* CreatePresentationObject(FViewport* const Viewport, TSharedPtr<IDisplayClusterRenderSyncPolicy>& SyncPolicy) override;
};
