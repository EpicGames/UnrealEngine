// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Device/Monoscopic/Windows/DisplayClusterDeviceMonoscopicDX12.h"
#include "Render/Presentation/Windows/DisplayClusterPresentationDX12.h"

#include "Misc/DisplayClusterLog.h"


FDisplayClusterDeviceMonoscopicDX12::FDisplayClusterDeviceMonoscopicDX12()
	: FDisplayClusterDeviceBase(EDisplayClusterRenderFrameMode::Mono)
{
}

FDisplayClusterPresentationBase* FDisplayClusterDeviceMonoscopicDX12::CreatePresentationObject(FViewport* const Viewport, TSharedPtr<IDisplayClusterRenderSyncPolicy>& SyncPolicy)
{
	return new FDisplayClusterPresentationDX12(Viewport, SyncPolicy);
}
