// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Render/Device/DisplayClusterDeviceBase.h"

/**
 * Monoscopic render device (DirectX 12)
 */
class FDisplayClusterDeviceMonoscopicDX12
	: public FDisplayClusterDeviceBase
{
public:
	FDisplayClusterDeviceMonoscopicDX12();
	virtual ~FDisplayClusterDeviceMonoscopicDX12() = default;

protected:
	virtual FDisplayClusterPresentationBase* CreatePresentationObject(FViewport* const Viewport, TSharedPtr<IDisplayClusterRenderSyncPolicy>& SyncPolicy) override;
};
