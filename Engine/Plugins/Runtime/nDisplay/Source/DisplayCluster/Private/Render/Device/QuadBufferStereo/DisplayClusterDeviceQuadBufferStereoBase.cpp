// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Device/QuadBufferStereo/DisplayClusterDeviceQuadBufferStereoBase.h"

#include "Misc/DisplayClusterLog.h"

#include "Render/Viewport/IDisplayClusterViewportManager.h"

#include <utility>


FDisplayClusterDeviceQuadBufferStereoBase::FDisplayClusterDeviceQuadBufferStereoBase()
	: FDisplayClusterDeviceBase(EDisplayClusterRenderFrameMode::Stereo)
{
}
