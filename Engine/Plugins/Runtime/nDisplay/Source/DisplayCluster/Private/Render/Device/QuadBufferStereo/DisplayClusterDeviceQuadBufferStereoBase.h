// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Render/Device/DisplayClusterDeviceBase.h"

/**
 * Abstract frame sequenced active stereo device
 */
class FDisplayClusterDeviceQuadBufferStereoBase :
	public FDisplayClusterDeviceBase
{
public:
	FDisplayClusterDeviceQuadBufferStereoBase();
	virtual ~FDisplayClusterDeviceQuadBufferStereoBase() = default;
};
