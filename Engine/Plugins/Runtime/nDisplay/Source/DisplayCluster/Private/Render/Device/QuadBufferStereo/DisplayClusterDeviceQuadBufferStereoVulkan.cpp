// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Device/QuadBufferStereo/DisplayClusterDeviceQuadBufferStereoVulkan.h"
#include "Render/Presentation/DisplayClusterPresentationVulkan.h"

#include "Misc/DisplayClusterLog.h"


FDisplayClusterDeviceQuadBufferStereoVulkan::FDisplayClusterDeviceQuadBufferStereoVulkan()
	: FDisplayClusterDeviceQuadBufferStereoBase()
{
}

FDisplayClusterPresentationBase* FDisplayClusterDeviceQuadBufferStereoVulkan::CreatePresentationObject(FViewport* const Viewport, TSharedPtr<IDisplayClusterRenderSyncPolicy>& SyncPolicy)
{
	return new FDisplayClusterPresentationVulkan(Viewport, SyncPolicy);
}
