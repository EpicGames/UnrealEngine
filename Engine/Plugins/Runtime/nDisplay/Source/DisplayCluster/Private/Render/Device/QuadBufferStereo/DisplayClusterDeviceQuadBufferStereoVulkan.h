// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Render/Device/QuadBufferStereo/DisplayClusterDeviceQuadBufferStereoBase.h"


/**
 * Frame sequenced active stereo (Vulkan)
 */
class FDisplayClusterDeviceQuadBufferStereoVulkan
	: public FDisplayClusterDeviceQuadBufferStereoBase
{
public:
	FDisplayClusterDeviceQuadBufferStereoVulkan();
	virtual ~FDisplayClusterDeviceQuadBufferStereoVulkan() = default;

protected:
	virtual FDisplayClusterPresentationBase* CreatePresentationObject(FViewport* const Viewport, TSharedPtr<IDisplayClusterRenderSyncPolicy>& SyncPolicy) override;
};
