// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Device/QuadBufferStereo/Windows/DisplayClusterDeviceQuadBufferStereoDX11.h"
#include "Render/Presentation/Windows/DisplayClusterPresentationDX11.h"

#include "Misc/DisplayClusterLog.h"


FDisplayClusterDeviceQuadBufferStereoDX11::FDisplayClusterDeviceQuadBufferStereoDX11()
	: FDisplayClusterDeviceQuadBufferStereoBase()
{
}

FDisplayClusterPresentationBase* FDisplayClusterDeviceQuadBufferStereoDX11::CreatePresentationObject(FViewport* const Viewport, TSharedPtr<IDisplayClusterRenderSyncPolicy>& SyncPolicy)
{
	return new FDisplayClusterPresentationDX11(Viewport, SyncPolicy);
}
