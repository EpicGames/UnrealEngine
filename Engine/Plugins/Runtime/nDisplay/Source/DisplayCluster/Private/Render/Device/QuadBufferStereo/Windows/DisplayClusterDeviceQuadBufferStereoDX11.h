// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Render/Device/QuadBufferStereo/DisplayClusterDeviceQuadBufferStereoBase.h"


/**
 * Frame sequenced active stereo (DirectX 11)
 */
class FDisplayClusterDeviceQuadBufferStereoDX11
	: public FDisplayClusterDeviceQuadBufferStereoBase
{
public:
	FDisplayClusterDeviceQuadBufferStereoDX11();
	virtual ~FDisplayClusterDeviceQuadBufferStereoDX11() = default;

protected:
	virtual FDisplayClusterPresentationBase* CreatePresentationObject(FViewport* const Viewport, TSharedPtr<IDisplayClusterRenderSyncPolicy>& SyncPolicy) override;
};
