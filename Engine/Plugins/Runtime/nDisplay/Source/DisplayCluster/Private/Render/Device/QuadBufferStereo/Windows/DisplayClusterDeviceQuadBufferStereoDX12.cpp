// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Device/QuadBufferStereo/Windows/DisplayClusterDeviceQuadBufferStereoDX12.h"
#include "Render/Presentation/Windows/DisplayClusterPresentationDX12.h"

#include "Misc/DisplayClusterLog.h"


FDisplayClusterDeviceQuadBufferStereoDX12::FDisplayClusterDeviceQuadBufferStereoDX12()
	: FDisplayClusterDeviceQuadBufferStereoBase()
{
}

FDisplayClusterPresentationBase* FDisplayClusterDeviceQuadBufferStereoDX12::CreatePresentationObject(FViewport* const Viewport, TSharedPtr<IDisplayClusterRenderSyncPolicy>& SyncPolicy)
{
	return new FDisplayClusterPresentationDX12(Viewport, SyncPolicy);
}
