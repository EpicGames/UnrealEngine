// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Render/Device/QuadBufferStereo/DisplayClusterDeviceQuadBufferStereoBase.h"


/**
 * Frame sequenced active stereo (DirectX 12)
 */
class FDisplayClusterDeviceQuadBufferStereoDX12
	: public FDisplayClusterDeviceQuadBufferStereoBase
{
public:
	FDisplayClusterDeviceQuadBufferStereoDX12();
	virtual ~FDisplayClusterDeviceQuadBufferStereoDX12() = default;

protected:
	virtual FDisplayClusterPresentationBase* CreatePresentationObject(FViewport* const Viewport, TSharedPtr<IDisplayClusterRenderSyncPolicy>& SyncPolicy) override;
};
