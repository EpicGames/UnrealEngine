// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Device/SideBySide/DisplayClusterDeviceSideBySideVulkan.h"
#include "Render/Presentation/DisplayClusterPresentationVulkan.h"


FDisplayClusterDeviceSideBySideVulkan::FDisplayClusterDeviceSideBySideVulkan()
	: FDisplayClusterDeviceBase(EDisplayClusterRenderFrameMode::SideBySide)
{
}

FDisplayClusterPresentationBase* FDisplayClusterDeviceSideBySideVulkan::CreatePresentationObject(FViewport* const Viewport, TSharedPtr<IDisplayClusterRenderSyncPolicy>& SyncPolicy)
{
	return new FDisplayClusterPresentationVulkan(Viewport, SyncPolicy);
}
