// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Device/SideBySide/Windows/DisplayClusterDeviceSideBySideDX11.h"
#include "Render/Presentation/Windows/DisplayClusterPresentationDX11.h"

#include "Misc/DisplayClusterLog.h"


FDisplayClusterDeviceSideBySideDX11::FDisplayClusterDeviceSideBySideDX11()
	: FDisplayClusterDeviceBase(EDisplayClusterRenderFrameMode::SideBySide)
{
}

FDisplayClusterPresentationBase* FDisplayClusterDeviceSideBySideDX11::CreatePresentationObject(FViewport* const Viewport, TSharedPtr<IDisplayClusterRenderSyncPolicy>& SyncPolicy)
{
	return new FDisplayClusterPresentationDX11(Viewport, SyncPolicy);
}
