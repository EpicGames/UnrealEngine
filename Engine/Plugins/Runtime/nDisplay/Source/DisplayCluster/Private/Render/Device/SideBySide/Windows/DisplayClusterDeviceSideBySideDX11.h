// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Render/Device/DisplayClusterDeviceBase.h"

/**
 * Side-by-side passive stereoscopic device
 */
class FDisplayClusterDeviceSideBySideDX11
	: public FDisplayClusterDeviceBase
{
public:
	FDisplayClusterDeviceSideBySideDX11();
	virtual ~FDisplayClusterDeviceSideBySideDX11() = default;

protected:
	virtual FDisplayClusterPresentationBase* CreatePresentationObject(FViewport* const Viewport, TSharedPtr<IDisplayClusterRenderSyncPolicy>& SyncPolicy) override;
};
