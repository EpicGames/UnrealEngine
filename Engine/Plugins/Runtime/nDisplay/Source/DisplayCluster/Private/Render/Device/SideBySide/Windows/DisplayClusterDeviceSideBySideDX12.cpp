// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Device/SideBySide/Windows/DisplayClusterDeviceSideBySideDX12.h"
#include "Render/Presentation/Windows/DisplayClusterPresentationDX12.h"

#include "Misc/DisplayClusterLog.h"


FDisplayClusterDeviceSideBySideDX12::FDisplayClusterDeviceSideBySideDX12()
	: FDisplayClusterDeviceBase(EDisplayClusterRenderFrameMode::SideBySide)
{
}

FDisplayClusterPresentationBase* FDisplayClusterDeviceSideBySideDX12::CreatePresentationObject(FViewport* const Viewport, TSharedPtr<IDisplayClusterRenderSyncPolicy>& SyncPolicy)
{
	return new FDisplayClusterPresentationDX12(Viewport, SyncPolicy);
}
