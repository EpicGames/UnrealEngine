// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Render/Device/DisplayClusterDeviceBase.h"

/**
 * Top-bottom passive stereoscopic device (Vulkan RHI)
 */
class FDisplayClusterDeviceTopBottomVulkan
	: public FDisplayClusterDeviceBase
{
public:
	FDisplayClusterDeviceTopBottomVulkan();
	virtual ~FDisplayClusterDeviceTopBottomVulkan() = default;

protected:
	virtual FDisplayClusterPresentationBase* CreatePresentationObject(FViewport* const Viewport, TSharedPtr<IDisplayClusterRenderSyncPolicy>& SyncPolicy) override;
};
