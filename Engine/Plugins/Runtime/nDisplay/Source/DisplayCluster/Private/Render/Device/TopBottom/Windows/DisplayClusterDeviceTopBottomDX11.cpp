// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Device/TopBottom/Windows/DisplayClusterDeviceTopBottomDX11.h"
#include "Render/Presentation/Windows/DisplayClusterPresentationDX11.h"

#include "Misc/DisplayClusterLog.h"


FDisplayClusterDeviceTopBottomDX11::FDisplayClusterDeviceTopBottomDX11()
	: FDisplayClusterDeviceBase(EDisplayClusterRenderFrameMode::TopBottom)
{
}

FDisplayClusterPresentationBase* FDisplayClusterDeviceTopBottomDX11::CreatePresentationObject(FViewport* const Viewport, TSharedPtr<IDisplayClusterRenderSyncPolicy>& SyncPolicy)
{
	return new FDisplayClusterPresentationDX11(Viewport, SyncPolicy);
}
