// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Render/Device/DisplayClusterDeviceBase.h"

/**
 * Top-bottom passive stereoscopic device
 */
class FDisplayClusterDeviceTopBottomDX11
	: public FDisplayClusterDeviceBase
{
public:
	FDisplayClusterDeviceTopBottomDX11();
	virtual ~FDisplayClusterDeviceTopBottomDX11() = default;

protected:
	virtual FDisplayClusterPresentationBase* CreatePresentationObject(FViewport* const Viewport, TSharedPtr<IDisplayClusterRenderSyncPolicy>& SyncPolicy) override;
};
