// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Device/TopBottom/Windows/DisplayClusterDeviceTopBottomDX12.h"
#include "Render/Presentation/Windows/DisplayClusterPresentationDX12.h"

#include "Misc/DisplayClusterLog.h"


FDisplayClusterDeviceTopBottomDX12::FDisplayClusterDeviceTopBottomDX12()
	: FDisplayClusterDeviceBase(EDisplayClusterRenderFrameMode::TopBottom)
{
}

FDisplayClusterPresentationBase* FDisplayClusterDeviceTopBottomDX12::CreatePresentationObject(FViewport* const Viewport, TSharedPtr<IDisplayClusterRenderSyncPolicy>& SyncPolicy)
{
	return new FDisplayClusterPresentationDX12(Viewport, SyncPolicy);
}
