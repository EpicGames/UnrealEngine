// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/DisplayClusterRenderManager.h"

#include "Components/DisplayClusterCameraComponent.h"
#include "Config/IPDisplayClusterConfigManager.h"

#include "Engine/GameEngine.h"
#include "Engine/GameViewportClient.h"
#include "Engine/Scene.h"

#include "Framework/Application/SlateApplication.h"
#include "Game/IPDisplayClusterGameManager.h"
#include "Kismet/GameplayStatics.h"

#include "Misc/DisplayClusterDataCache.h"
#include "Misc/DisplayClusterGlobals.h"
#include "Misc/DisplayClusterHelpers.h"
#include "Misc/DisplayClusterLog.h"
#include "Misc/DisplayClusterStrings.h"

#include "Render/Containers/DisplayClusterRender_MeshComponent.h"
#include "Render/Containers/DisplayClusterRender_Texture.h"
#include "Render/Device/DisplayClusterRenderDeviceFactoryInternal.h"
#include "Render/Device/IDisplayClusterRenderDevice.h"
#include "Render/Device/IDisplayClusterRenderDeviceFactory.h"
#include "Render/GUILayer/DisplayClusterGuiLayerController.h"
#include "Render/Monitoring/DisplayClusterVblankMonitor.h"
#include "Render/PostProcess/IDisplayClusterPostProcess.h"
#include "Render/Presentation/DisplayClusterPresentationNative.h"
#include "Render/Projection/IDisplayClusterProjectionPolicy.h"
#include "Render/Projection/IDisplayClusterProjectionPolicyFactory.h"
#include "Render/Synchronization/DisplayClusterRenderSyncPolicyFactoryInternal.h"
#include "Render/Synchronization/DisplayClusterRenderSyncPolicyNone.h"
#include "Render/Synchronization/IDisplayClusterRenderSyncPolicyFactory.h"

#include "Slate/SlateViewportProvider.h"

#include "CineCameraComponent.h"
#include "DisplayClusterConfigurationStrings.h"
#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterRootActor.h"
#include "DisplayClusterViewportClient.h"
#include "UnrealClient.h"


#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#include "Windows.h"
#include "Windows/HideWindowsPlatformTypes.h"
#endif


static TAutoConsoleVariable<int32> CVarSyncDiagnosticsVBlankMonitoring(
	TEXT("nDisplay.sync.diag.VBlankMonitoring"),
	0,
	TEXT("Sync diagnostics: V-blank monitoring\n")
	TEXT("0 : disabled\n")
	TEXT("1 : enabled (if policy supports only)\n")
	,
	ECVF_ReadOnly
);


/**
 * The cache for DC texture objects. (Singleton)
 * Allows you to reuse textures with the same unique name.
 */
class FDisplayClusterRenderTextureCache
	: public TDisplayClusterDataCache<FDisplayClusterRender_Texture>
{
public:
	static TSharedPtr<FDisplayClusterRender_Texture, ESPMode::ThreadSafe> GetOrCreateRenderTexture(const FString& InTextureName)
	{
		static FDisplayClusterRenderTextureCache TextureCacheSingleton;

		const FString UniqueName = HashString(InTextureName);

		TSharedPtr<FDisplayClusterRender_Texture, ESPMode::ThreadSafe> TextureRef = TextureCacheSingleton.Find(UniqueName);
		if (!TextureRef.IsValid())
		{
			TextureRef = MakeShared<FDisplayClusterRender_Texture, ESPMode::ThreadSafe>(UniqueName);
			TextureCacheSingleton.Add(TextureRef);
		}

		return TextureRef;
	}
};

//---------------------------------------------------
// FDisplayClusterRenderManager
//---------------------------------------------------
FDisplayClusterRenderManager::FDisplayClusterRenderManager()
{
	// Instantiate and register internal render device factory
	TSharedPtr<IDisplayClusterRenderDeviceFactory> NewRenderDeviceFactory(new FDisplayClusterRenderDeviceFactoryInternal);
	RegisterRenderDeviceFactory(DisplayClusterStrings::args::dev::Mono, NewRenderDeviceFactory);
	RegisterRenderDeviceFactory(DisplayClusterStrings::args::dev::QBS,  NewRenderDeviceFactory);
	RegisterRenderDeviceFactory(DisplayClusterStrings::args::dev::SbS,  NewRenderDeviceFactory);
	RegisterRenderDeviceFactory(DisplayClusterStrings::args::dev::TB,   NewRenderDeviceFactory);

	// Instantiate and register internal sync policy factory
	TSharedPtr<IDisplayClusterRenderSyncPolicyFactory> NewSyncPolicyFactory(new FDisplayClusterRenderSyncPolicyFactoryInternal);
	RegisterSynchronizationPolicyFactory(DisplayClusterConfigurationStrings::config::cluster::render_sync::None,            NewSyncPolicyFactory); // None
	RegisterSynchronizationPolicyFactory(DisplayClusterConfigurationStrings::config::cluster::render_sync::Ethernet,        NewSyncPolicyFactory); // Ethernet
	RegisterSynchronizationPolicyFactory(DisplayClusterConfigurationStrings::config::cluster::render_sync::EthernetBarrier, NewSyncPolicyFactory); // Ethernet_Simple
	RegisterSynchronizationPolicyFactory(DisplayClusterConfigurationStrings::config::cluster::render_sync::Nvidia,          NewSyncPolicyFactory); // NVIDIA

	// Instantiate V-blank monitor (it won't auto-start polling)
	VBlankMonitor = MakeShared<FDisplayClusterVBlankMonitor, ESPMode::ThreadSafe>();
}

FDisplayClusterRenderManager::~FDisplayClusterRenderManager()
{
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IPDisplayClusterManager
//////////////////////////////////////////////////////////////////////////////////////////////
bool FDisplayClusterRenderManager::Init(EDisplayClusterOperationMode OperationMode)
{
	CurrentOperationMode = OperationMode;

	return true;
}

void FDisplayClusterRenderManager::Release()
{
	//@note: No need to release our RenderDevice. It will be released in a safe way by TSharedPtr.
}

bool FDisplayClusterRenderManager::StartSession(UDisplayClusterConfigurationData* InConfigData, const FString& InNodeId)
{
	if (CurrentOperationMode == EDisplayClusterOperationMode::Disabled)
	{
		UE_LOGF(LogDisplayClusterRender, Log, "Operation mode is 'Disabled' so no initialization will be performed");
		return true;
	}

	// Set callback on viewport created. We want to make sure the DisplayClusterViewportClient is used.
	UGameViewportClient::OnViewportCreated().AddRaw(this, &FDisplayClusterRenderManager::OnViewportCreatedHandler_CheckViewportClass);

	// Create synchronization object
	UE_LOGF(LogDisplayClusterRender, Log, "Instantiating synchronization policy object...");
	SyncPolicy = CreateRenderSyncPolicy();
	if (SyncPolicy)
	{
		SyncPolicy->Initialize();
	}

	// Instantiate render device
	TSharedPtr<IDisplayClusterRenderDevice, ESPMode::ThreadSafe> NewRenderDevice;
	UE_LOGF(LogDisplayClusterRender, Log, "Instantiating stereo device...");
	NewRenderDevice = CreateRenderDevice();

	// Set new device as the engine's stereoscopic device
	if (GEngine && NewRenderDevice.IsValid())
	{
		GEngine->StereoRenderingDevice = StaticCastSharedPtr<IStereoRendering>(NewRenderDevice);
		RenderDevicePtr = NewRenderDevice.Get();
	}

	// Start v-blank monitoring if requested
	if (!!CVarSyncDiagnosticsVBlankMonitoring.GetValueOnGameThread())
	{
		VBlankMonitor->StartMonitoring();
	}

	// When session is starting in Editor the device won't be initialized so we avoid nullptr access here.
	//@todo Now we always have a device, even for Editor. Change the condition working on the EditorDevice.
	return (RenderDevicePtr ? RenderDevicePtr->Initialize() : true);
}

void FDisplayClusterRenderManager::EndSession()
{
#if WITH_EDITOR
	if (GIsEditor && RenderDevicePtr)
	{
		// Since we can run multiple PIE sessions we have to clean device before the next one.
		GEngine->StereoRenderingDevice.Reset();
		RenderDevicePtr = nullptr;
	}
#endif

	SyncPolicy.Reset();
}

bool FDisplayClusterRenderManager::StartScene(UWorld* InWorld)
{
	if (RenderDevicePtr)
	{
		RenderDevicePtr->StartScene(InWorld);
	}

	return true;
}

void FDisplayClusterRenderManager::EndScene()
{
	if (RenderDevicePtr)
	{
		RenderDevicePtr->EndScene();
	}
}

#if PLATFORM_WINDOWS
static bool SimulateMouseClick(HWND WindowHandle)
{
	RECT WindowRect; // Window rect in screen coordinates

	if (!GetWindowRect(WindowHandle, &WindowRect))
	{
		return false;
	}

	const double ScreenWidth = double(::GetSystemMetrics(SM_CXSCREEN));
	const double ScreenHeight = double(::GetSystemMetrics(SM_CYSCREEN));

	check(ScreenWidth > 0.5);
	check(ScreenHeight > 0.5);

	INPUT Inputs[3] = {{ 0 }};

	// Expected coordinates are normalized to screen dimensions of 65535x65535

	const double Left   = double(WindowRect.left);
	const double Right  = double(WindowRect.right);
	const double Top    = double(WindowRect.top);
	const double Bottom = double(WindowRect.bottom);

	Inputs[0].type = INPUT_MOUSE;
	Inputs[0].mi.dx = LONG((Left + (Right  - Left)/2.0) * (65535.0 / ScreenWidth));
	Inputs[0].mi.dy = LONG((Top  + (Bottom -  Top)/2.0) * (65535.0 / ScreenHeight));
	Inputs[0].mi.dwFlags = MOUSEEVENTF_ABSOLUTE | MOUSEEVENTF_MOVE;

	Inputs[1].type = INPUT_MOUSE;
	Inputs[1].mi.dwFlags = MOUSEEVENTF_LEFTDOWN;

	Inputs[2].type = INPUT_MOUSE;
	Inputs[2].mi.dwFlags = MOUSEEVENTF_LEFTUP;

	return !!::SendInput(3, Inputs, sizeof(INPUT));
}
#endif //PLATFORM_WINDOWS

void FDisplayClusterRenderManager::PreTick(float DeltaSeconds)
{
	static const bool bIsRenderingOffscreen = FParse::Param(FCommandLine::Get(), TEXT("RenderOffscreen"));

	if(!bIsRenderingOffscreen && !bWasWindowFocused)
	{
		if (UGameViewportClient* GameViewportClient = GEngine->GameViewport)
		{
			if (TSharedPtr<SWindow> Window = GameViewportClient->GetWindow())
			{
				if (TSharedPtr<const FGenericWindow> NativeWindow = Window->GetNativeWindow())
				{
					if (const void* WindowHandle = NativeWindow->GetOSWindowHandle())
					{
#if PLATFORM_WINDOWS
						const HWND GameHWND = (HWND)WindowHandle;

						::SetWindowPos(GameHWND, HWND_TOP, 0, 0, 0, 0, SWP_NOMOVE | SWP_NOSIZE);
						::SetForegroundWindow(GameHWND);
						::SetCapture(GameHWND);
						::SetFocus(GameHWND);
						::SetActiveWindow(GameHWND);

						SimulateMouseClick(GameHWND);
#endif
						FSlateApplication::Get().SetAllUserFocusToGameViewport();

						bWasWindowFocused = true;
					}
				}
			}
		}
	}

	if (RenderDevicePtr)
	{
		RenderDevicePtr->PreTick(DeltaSeconds);
	}
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterRenderManager
//////////////////////////////////////////////////////////////////////////////////////////////
IDisplayClusterRenderDevice* FDisplayClusterRenderManager::GetRenderDevice() const
{
	return RenderDevicePtr;
}

bool FDisplayClusterRenderManager::RegisterRenderDeviceFactory(const FString& InDeviceType, TSharedPtr<IDisplayClusterRenderDeviceFactory>& InFactory)
{
	UE_LOGF(LogDisplayClusterRender, Log, "Registering factory for rendering device type: %ls", *InDeviceType);

	if (!InFactory.IsValid())
	{
		UE_LOGF(LogDisplayClusterRender, Warning, "Invalid factory object");
		return false;
	}

	{
		FScopeLock Lock(&CritSecInternals);

		if (RenderDeviceFactories.Contains(InDeviceType))
		{
			UE_LOGF(LogDisplayClusterRender, Warning, "Setting a new factory for '%ls' rendering device type", *InDeviceType);
		}

		RenderDeviceFactories.Emplace(InDeviceType, InFactory);
	}

	UE_LOGF(LogDisplayClusterRender, Log, "Registered factory for rendering device type: %ls", *InDeviceType);

	return true;
}

bool FDisplayClusterRenderManager::UnregisterRenderDeviceFactory(const FString& InDeviceType)
{
	UE_LOGF(LogDisplayClusterRender, Log, "Unregistering factory for rendering device type: %ls", *InDeviceType);

	{
		FScopeLock Lock(&CritSecInternals);

		if (!RenderDeviceFactories.Contains(InDeviceType))
		{
			UE_LOGF(LogDisplayClusterRender, Warning, "A factory for '%ls' rendering device type not found", *InDeviceType);
			return false;
		}

		RenderDeviceFactories.Remove(InDeviceType);
	}

	UE_LOGF(LogDisplayClusterRender, Log, "Unregistered factory for rendering device type: %ls", *InDeviceType);

	return true;
}

bool FDisplayClusterRenderManager::RegisterSynchronizationPolicyFactory(const FString& InSyncPolicyType, TSharedPtr<IDisplayClusterRenderSyncPolicyFactory>& InFactory)
{
	UE_LOGF(LogDisplayClusterRender, Log, "Registering factory for synchronization policy: %ls", *InSyncPolicyType);

	if (!InFactory.IsValid())
	{
		UE_LOGF(LogDisplayClusterRender, Warning, "Invalid factory object");
		return false;
	}

	{
		FScopeLock Lock(&CritSecInternals);

		if (SyncPolicyFactories.Contains(InSyncPolicyType))
		{
			UE_LOGF(LogDisplayClusterRender, Warning, "A new factory for '%ls' synchronization policy was set", *InSyncPolicyType);
		}

		SyncPolicyFactories.Emplace(InSyncPolicyType, InFactory);
	}

	UE_LOGF(LogDisplayClusterRender, Log, "Registered factory for synchronization policy: %ls", *InSyncPolicyType);

	return true;
}

bool FDisplayClusterRenderManager::UnregisterSynchronizationPolicyFactory(const FString& InSyncPolicyType)
{
	UE_LOGF(LogDisplayClusterRender, Log, "Unregistering factory for synchronization policy: %ls", *InSyncPolicyType);

	{
		FScopeLock Lock(&CritSecInternals);

		if (!SyncPolicyFactories.Contains(InSyncPolicyType))
		{
			UE_LOGF(LogDisplayClusterRender, Warning, "A factory for '%ls' synchronization policy not found", *InSyncPolicyType);
			return false;
		}

		SyncPolicyFactories.Remove(InSyncPolicyType);
	}

	UE_LOGF(LogDisplayClusterRender, Log, "Unregistered factory for synchronization policy: %ls", *InSyncPolicyType);

	return true;
}

TSharedPtr<IDisplayClusterRenderSyncPolicy> FDisplayClusterRenderManager::GetCurrentSynchronizationPolicy()
{
	FScopeLock Lock(&CritSecInternals);
	return SyncPolicy;
}

//------------------------------------------------------------------------------------------------------------------------------
// Projection Policy
//------------------------------------------------------------------------------------------------------------------------------
bool FDisplayClusterRenderManager::RegisterProjectionPolicyFactory(const FString& InProjectionType, TSharedPtr<IDisplayClusterProjectionPolicyFactory>& InFactory)
{
	UE_LOGF(LogDisplayClusterRender, Log, "Registering factory for projection type: %ls", *InProjectionType);

	if (!InFactory.IsValid())
	{
		UE_LOGF(LogDisplayClusterRender, Warning, "Invalid factory object");
		return false;
	}

	{
		FScopeLock Lock(&CritSecInternals);

		if (ProjectionPolicyFactories.Contains(InProjectionType))
		{
			UE_LOGF(LogDisplayClusterRender, Warning, "A new factory for '%ls' projection policy was set", *InProjectionType);
		}

		ProjectionPolicyFactories.Emplace(InProjectionType, InFactory);
	}

	UE_LOGF(LogDisplayClusterRender, Log, "Registered factory for projection type: %ls", *InProjectionType);

	return true;
}

bool FDisplayClusterRenderManager::UnregisterProjectionPolicyFactory(const FString& InProjectionType)
{
	UE_LOGF(LogDisplayClusterRender, Log, "Unregistering factory for projection policy: %ls", *InProjectionType);

	{
		FScopeLock Lock(&CritSecInternals);

		if (!ProjectionPolicyFactories.Contains(InProjectionType))
		{
			UE_LOGF(LogDisplayClusterRender, Warning, "A handler for '%ls' projection type not found", *InProjectionType);
			return false;
		}

		ProjectionPolicyFactories.Remove(InProjectionType);
	}

	UE_LOGF(LogDisplayClusterRender, Log, "Unregistered factory for projection policy: %ls", *InProjectionType);

	return true;
}

TSharedPtr<IDisplayClusterProjectionPolicyFactory> FDisplayClusterRenderManager::GetProjectionPolicyFactory(const FString& InProjectionType)
{
	FScopeLock Lock(&CritSecInternals);

	TSharedPtr<IDisplayClusterProjectionPolicyFactory> Factory;
	if (!DisplayClusterHelpers::map::template ExtractValue(ProjectionPolicyFactories, InProjectionType, Factory))
	{
		UE_LOGF(LogDisplayClusterRender, Warning, "No factory found for projection policy: %ls", *InProjectionType);
	}

	return Factory;
}

void FDisplayClusterRenderManager::GetRegisteredProjectionPolicies(TArray<FString>& OutPolicyIDs) const
{
	FScopeLock Lock(&CritSecInternals);
	ProjectionPolicyFactories.GetKeys(OutPolicyIDs);
}

//------------------------------------------------------------------------------------------------------------------------------
// PostProcess
//------------------------------------------------------------------------------------------------------------------------------
bool FDisplayClusterRenderManager::RegisterPostProcessFactory(const FString& InPostProcessType, TSharedPtr<IDisplayClusterPostProcessFactory>& InFactory)
{
	UE_LOGF(LogDisplayClusterRender, Log, "Registering factory for postprocess type: %ls", *InPostProcessType);

	if (!InFactory.IsValid())
	{
		UE_LOGF(LogDisplayClusterRender, Warning, "Invalid factory object");
		return false;
	}

	{
		FScopeLock Lock(&CritSecInternals);

		if (PostProcessFactories.Contains(InPostProcessType))
		{
			UE_LOGF(LogDisplayClusterRender, Warning, "A new factory for '%ls' postprocess was set", *InPostProcessType);
		}

		PostProcessFactories.Emplace(InPostProcessType, InFactory);
	}

	UE_LOGF(LogDisplayClusterRender, Log, "Registered factory for postprocess type: %ls", *InPostProcessType);

	return true;
}

bool FDisplayClusterRenderManager::UnregisterPostProcessFactory(const FString& InPostProcessType)
{
	UE_LOGF(LogDisplayClusterRender, Log, "Unregistering factory for postprocess: %ls", *InPostProcessType);

	{
		FScopeLock Lock(&CritSecInternals);

		if (!PostProcessFactories.Contains(InPostProcessType))
		{
			UE_LOGF(LogDisplayClusterRender, Warning, "A handler for '%ls' postprocess type not found", *InPostProcessType);
			return false;
		}

		PostProcessFactories.Remove(InPostProcessType);
	}

	UE_LOGF(LogDisplayClusterRender, Log, "Unregistered factory for postprocess: %ls", *InPostProcessType);

	return true;
}

TSharedPtr<IDisplayClusterPostProcessFactory> FDisplayClusterRenderManager::GetPostProcessFactory(const FString& InPostProcessType)
{
	FScopeLock Lock(&CritSecInternals);

	TSharedPtr<IDisplayClusterPostProcessFactory> Factory;
	if (!DisplayClusterHelpers::map::template ExtractValue(PostProcessFactories, InPostProcessType, Factory))
	{
		UE_LOGF(LogDisplayClusterRender, Warning, "No factory found for postprocess: %ls", *InPostProcessType);
	}

	return Factory;
}

void FDisplayClusterRenderManager::GetRegisteredPostProcess(TArray<FString>& OutPostProcessIDs) const
{
	FScopeLock Lock(&CritSecInternals);
	PostProcessFactories.GetKeys(OutPostProcessIDs);
}

//------------------------------------------------------------------------------------------------------------------------------
// Warp Policy
//------------------------------------------------------------------------------------------------------------------------------
bool FDisplayClusterRenderManager::RegisterWarpPolicyFactory(const FString& InWarpPolicyType, TSharedPtr<IDisplayClusterWarpPolicyFactory>& InFactory)
{
	UE_LOGF(LogDisplayClusterRender, Log, "Registering factory for warp policy type: %ls", *InWarpPolicyType);

	if (!InFactory.IsValid())
	{
		UE_LOGF(LogDisplayClusterRender, Warning, "Invalid factory object");
		return false;
	}

	{
		FScopeLock Lock(&CritSecInternals);

		if (WarpPolicyFactories.Contains(InWarpPolicyType))
		{
			UE_LOGF(LogDisplayClusterRender, Warning, "A new factory for '%ls' warp policy was set", *InWarpPolicyType);
		}

		WarpPolicyFactories.Emplace(InWarpPolicyType, InFactory);
	}

	UE_LOGF(LogDisplayClusterRender, Log, "Registered factory for warp policy type: %ls", *InWarpPolicyType);

	return true;
}

bool FDisplayClusterRenderManager::UnregisterWarpPolicyFactory(const FString& InWarpPolicyType)
{
	UE_LOGF(LogDisplayClusterRender, Log, "Unregistering factory for warp policy: %ls", *InWarpPolicyType);

	{
		FScopeLock Lock(&CritSecInternals);

		if (!WarpPolicyFactories.Contains(InWarpPolicyType))
		{
			UE_LOGF(LogDisplayClusterRender, Warning, "A handler for '%ls' warp policy type not found", *InWarpPolicyType);
			return false;
		}

		WarpPolicyFactories.Remove(InWarpPolicyType);
	}

	UE_LOGF(LogDisplayClusterRender, Log, "Unregistered factory for warp policy: %ls", *InWarpPolicyType);

	return true;
}

TSharedPtr<IDisplayClusterWarpPolicyFactory> FDisplayClusterRenderManager::GetWarpPolicyFactory(const FString& InWarpPolicyType)
{
	FScopeLock Lock(&CritSecInternals);

	TSharedPtr<IDisplayClusterWarpPolicyFactory> Factory;
	if (!DisplayClusterHelpers::map::template ExtractValue(WarpPolicyFactories, InWarpPolicyType, Factory))
	{
		UE_LOGF(LogDisplayClusterRender, Warning, "No factory found for warp policy: %ls", *InWarpPolicyType);
	}

	return Factory;
}

void FDisplayClusterRenderManager::GetRegisteredWarpPolicies(TArray<FString>& OutWarpPolicyIDs) const
{
	FScopeLock Lock(&CritSecInternals);
	WarpPolicyFactories.GetKeys(OutWarpPolicyIDs);
}

//------------------------------------------------------------------------------------------------------------------------------
// Resources
//------------------------------------------------------------------------------------------------------------------------------
TSharedPtr<IDisplayClusterRender_MeshComponent, ESPMode::ThreadSafe> FDisplayClusterRenderManager::CreateMeshComponent() const
{
	return MakeShared<FDisplayClusterRender_MeshComponent, ESPMode::ThreadSafe>();
}

TSharedPtr<IDisplayClusterRender_Texture, ESPMode::ThreadSafe> FDisplayClusterRenderManager::GetOrCreateCachedTexture(const FString& InUniqueTextureName) const
{
	return FDisplayClusterRenderTextureCache::GetOrCreateRenderTexture(InUniqueTextureName);
}

IDisplayClusterGUILayerController& FDisplayClusterRenderManager::GetGuiLayerController() const
{
	return FDisplayClusterGuiLayerController::Get();
}

IDisplayClusterViewportManager* FDisplayClusterRenderManager::GetViewportManager() const
{
	ADisplayClusterRootActor* RootActor = GDisplayCluster->GetGameMgr()->GetRootActor();
	if (RootActor)
	{
		return RootActor->GetViewportManager();
	}

	return nullptr;
}

//////////////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterRenderManager
//////////////////////////////////////////////////////////////////////////////////////////////
TSharedPtr<IDisplayClusterRenderDevice, ESPMode::ThreadSafe> FDisplayClusterRenderManager::CreateRenderDevice() const
{
	TSharedPtr<IDisplayClusterRenderDevice, ESPMode::ThreadSafe> NewRenderDevice;

	if (CurrentOperationMode == EDisplayClusterOperationMode::Cluster)
	{
		if (GDynamicRHI == nullptr)
		{
			UE_LOGF(LogDisplayClusterRender, Error, "GDynamicRHI is null. Cannot detect RHI name.");
			return nullptr;
		}

		// Monoscopic
		if (FParse::Param(FCommandLine::Get(), DisplayClusterStrings::args::dev::Mono))
		{
			NewRenderDevice = RenderDeviceFactories[DisplayClusterStrings::args::dev::Mono]->Create(DisplayClusterStrings::args::dev::Mono);
		}
		// Quad buffer stereo
		else if (FParse::Param(FCommandLine::Get(), DisplayClusterStrings::args::dev::QBS))
		{
			NewRenderDevice = RenderDeviceFactories[DisplayClusterStrings::args::dev::QBS]->Create(DisplayClusterStrings::args::dev::QBS);
		}
		// Side-by-side
		else if (FParse::Param(FCommandLine::Get(), DisplayClusterStrings::args::dev::SbS))
		{
			NewRenderDevice = RenderDeviceFactories[DisplayClusterStrings::args::dev::SbS]->Create(DisplayClusterStrings::args::dev::SbS);
		}
		// Top-bottom
		else if (FParse::Param(FCommandLine::Get(), DisplayClusterStrings::args::dev::TB))
		{
			NewRenderDevice = RenderDeviceFactories[DisplayClusterStrings::args::dev::TB]->Create(DisplayClusterStrings::args::dev::TB);
		}
		// Leave native render but inject custom present for cluster synchronization
		else
		{
			UE_LOGF(LogDisplayClusterRender, Warning, "No rendering device specified! A native present handler will be instantiated when viewport is presented");
			FSlateApplication::Get().GetRenderer()->OnBackBufferReadyToPresent().AddRaw(this, &FDisplayClusterRenderManager::OnBackBufferReadyToPresent_SetCustomPresent);
		}
	}
	else if (CurrentOperationMode == EDisplayClusterOperationMode::Editor)
	{
#if 0
		UE_LOGF(LogDisplayClusterRender, Log, "Instantiating DX11 mono device for PIE");
		NewRenderDevice = MakeShared<FDisplayClusterDeviceMonoscopicDX11>();
#endif
	}
	else if (CurrentOperationMode == EDisplayClusterOperationMode::Disabled)
	{
		// Stereo device is not needed
		UE_LOGF(LogDisplayClusterRender, Log, "No need to instantiate stereo device");
	}
	else
	{
		UE_LOGF(LogDisplayClusterRender, Warning, "Unknown operation mode");
	}

	if (!NewRenderDevice.IsValid())
	{
		UE_LOGF(LogDisplayClusterRender, Log, "No stereo device created");
	}

	return NewRenderDevice;
}

TSharedPtr<IDisplayClusterRenderSyncPolicy> FDisplayClusterRenderManager::CreateRenderSyncPolicy() const
{
	if (CurrentOperationMode != EDisplayClusterOperationMode::Cluster)
	{
		UE_LOGF(LogDisplayClusterRender, Log, "Synchronization policy is not available for current operation mode");
		return nullptr;
	}

	if (GDynamicRHI == nullptr)
	{
		UE_LOGF(LogDisplayClusterRender, Error, "GDynamicRHI is null. Cannot detect RHI name.");
		return nullptr;
	}

	const UDisplayClusterConfigurationData* ConfigData = GDisplayCluster->GetPrivateConfigMgr()->GetConfig();
	if (!ConfigData)
	{
		UE_LOGF(LogDisplayClusterRender, Error, "Couldn't get configuration data");
		return nullptr;
	}

	FString SyncPolicyType;
	TMap<FString, FString>* SyncPolicyParams = nullptr;

	// Always use 'none' sync policy while rendering headless
	if (FParse::Param(FCommandLine::Get(), TEXT("RenderOffscreen")))
	{
		SyncPolicyType   = DisplayClusterConfigurationStrings::config::cluster::render_sync::HeadlessRenderingSyncPolicy;
		SyncPolicyParams = nullptr;
		UE_LOGF(LogDisplayClusterRender, Log, "Headless rendering requested. Using '%ls' sync policy.", *SyncPolicyType);
	}
	// Otherwise use sync policy from config
	else
	{
		SyncPolicyType   = ConfigData->Cluster->Sync.RenderSyncPolicy.Type;
		SyncPolicyParams = &ConfigData->Cluster->Sync.RenderSyncPolicy.Parameters;
		UE_LOGF(LogDisplayClusterRender, Log, "Requested sync policy is '%ls'", *SyncPolicyType);
	}

	// Instantiate policy instance
	TSharedPtr<IDisplayClusterRenderSyncPolicy> NewSyncPolicy;
	if (SyncPolicyFactories.Contains(SyncPolicyType))
	{
		UE_LOGF(LogDisplayClusterRender, Log, "A factory for the requested synchronization policy <%ls> was found", *SyncPolicyType);
		NewSyncPolicy = SyncPolicyFactories[SyncPolicyType]->Create(SyncPolicyType, SyncPolicyParams ? *SyncPolicyParams : TMap<FString, FString>());
	}
	// Fallback if requested policy is not available
	else
	{
		const FString DefaultPolicy = DisplayClusterConfigurationStrings::config::cluster::render_sync::EthernetBarrier;
		UE_LOGF(LogDisplayClusterRender, Log, "No factory found for the requested synchronization policy <%ls>. Default '%ls' policy will be used.", *SyncPolicyType, *DefaultPolicy);
		NewSyncPolicy = SyncPolicyFactories[DefaultPolicy]->Create(DefaultPolicy, TMap<FString, FString>());
	}

	return NewSyncPolicy;
}

void FDisplayClusterRenderManager::ResizeWindow(int32 WinX, int32 WinY, int32 ResX, int32 ResY)
{
	UGameEngine* Engine = Cast<UGameEngine>(GEngine);
	TSharedPtr<SWindow> Window = Engine->GameViewportWindow.Pin();
	check(Window.IsValid());

	UE_LOGF(LogDisplayClusterRender, Log, "Adjusting game window: pos [%d, %d],  size [%d x %d]", WinX, WinY, ResX, ResY);

	// Adjust window position/size
	Window->ReshapeWindow(FVector2D(WinX, WinY), FVector2D(ResX, ResY));
}

void FDisplayClusterRenderManager::OnViewportCreatedHandler_CheckViewportClass() const
{
	if (GEngine && GEngine->GameViewport)
	{
		UDisplayClusterViewportClient* const GameViewport = Cast<UDisplayClusterViewportClient>(GEngine->GameViewport);
		if (!GameViewport)
		{
			UE_LOGF(LogDisplayClusterRender, Warning, "DisplayClusterViewportClient is not set as a default GameViewport class");
		}
	}
}

void FDisplayClusterRenderManager::OnBackBufferReadyToPresent_SetCustomPresent(SWindow& Window, ISlateViewportProvider& ViewportProvider) const
{
	if (NativePresentHandler)
	{
		return;
	}

	if (GEngine->GameViewport && GEngine->GameViewport->Viewport == Window.GetViewport().Get())
	{
		NativePresentHandler = new FDisplayClusterPresentationNative(GEngine->GameViewport->Viewport, SyncPolicy);
		ViewportProvider.SetCustomPresent(NativePresentHandler);
	}
}
