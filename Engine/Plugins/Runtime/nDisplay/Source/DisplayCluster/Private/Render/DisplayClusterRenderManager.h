// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RHIFwd.h"
#include "Render/IPDisplayClusterRenderManager.h"

class IDisplayClusterGUILayerController;
class IDisplayClusterPostProcess;
class IDisplayClusterPostProcessFactory; 
class IDisplayClusterProjectionPolicy;
class IDisplayClusterProjectionPolicyFactory;
class IDisplayClusterRenderDeviceFactory;
class IDisplayClusterRenderSyncPolicy;
class IDisplayClusterRenderSyncPolicyFactory;
class IDisplayClusterVblankMonitor;
class UDisplayClusterCameraComponent;


/**
 * Render manager. Responsible for everything related to the visuals.
 */
class FDisplayClusterRenderManager :
	public IPDisplayClusterRenderManager
{
public:
	FDisplayClusterRenderManager();
	virtual ~FDisplayClusterRenderManager();

public:
	//////////////////////////////////////////////////////////////////////////////////////////////
	// IPDisplayClusterManager
	//////////////////////////////////////////////////////////////////////////////////////////////
	virtual bool Init(EDisplayClusterOperationMode OperationMode) override;
	virtual void Release() override;
	virtual bool StartSession(UDisplayClusterConfigurationData* InConfigData, const FString& InNodeId) override;
	virtual void EndSession() override;
	virtual bool StartScene(UWorld* InWorld) override;
	virtual void EndScene() override;
	virtual void PreTick(float DeltaSeconds) override;

public:
	//////////////////////////////////////////////////////////////////////////////////////////////
	// IDisplayClusterRenderManager
	//////////////////////////////////////////////////////////////////////////////////////////////
	// Device
	virtual IDisplayClusterRenderDevice* GetRenderDevice() const override;
	virtual bool RegisterRenderDeviceFactory(const FString& InDeviceType, TSharedPtr<IDisplayClusterRenderDeviceFactory>& InFactory) override;
	virtual bool UnregisterRenderDeviceFactory(const FString& InDeviceType) override;
	// Synchronization
	virtual bool RegisterSynchronizationPolicyFactory(const FString& InSyncPolicyType, TSharedPtr<IDisplayClusterRenderSyncPolicyFactory>& InFactory) override;
	virtual bool UnregisterSynchronizationPolicyFactory(const FString& InSyncPolicyType) override;
	virtual TSharedPtr<IDisplayClusterRenderSyncPolicy> GetCurrentSynchronizationPolicy() override;
	// Projection
	virtual bool RegisterProjectionPolicyFactory(const FString& InProjectionType, TSharedPtr<IDisplayClusterProjectionPolicyFactory>& InFactory) override;
	virtual bool UnregisterProjectionPolicyFactory(const FString& InProjectionType) override;
	virtual TSharedPtr<IDisplayClusterProjectionPolicyFactory> GetProjectionPolicyFactory(const FString& InProjectionType) override;
	virtual void GetRegisteredProjectionPolicies(TArray<FString>& OutPolicyIDs) const override;
	// Post-process
	virtual bool RegisterPostProcessFactory(const FString& InPostProcessType, TSharedPtr<IDisplayClusterPostProcessFactory>& InFactory) override;
	virtual bool UnregisterPostProcessFactory(const FString& InPostProcessType) override;
	virtual TSharedPtr<IDisplayClusterPostProcessFactory> GetPostProcessFactory(const FString& InPostProcessType) override;
	virtual void GetRegisteredPostProcess(TArray<FString>& OutPostProcessIDs) const override;
	// Warp policy
	virtual bool RegisterWarpPolicyFactory(const FString& InWarpPolicyType, TSharedPtr<IDisplayClusterWarpPolicyFactory>& Factory) override;
	virtual bool UnregisterWarpPolicyFactory(const FString& InWarpPolicyType) override;
	virtual TSharedPtr<IDisplayClusterWarpPolicyFactory> GetWarpPolicyFactory(const FString& InWarpPolicyType) override;
	virtual void GetRegisteredWarpPolicies(TArray<FString>& OutWarpPolicyIDs) const override;
	// Resources
	virtual TSharedPtr<IDisplayClusterRender_MeshComponent, ESPMode::ThreadSafe> CreateMeshComponent() const override;
	virtual TSharedPtr<IDisplayClusterRender_Texture, ESPMode::ThreadSafe> GetOrCreateCachedTexture(const FString& InUniqueTextureName) const override;
	virtual IDisplayClusterGUILayerController& GetGuiLayerController() const override;

	virtual TSharedPtr<IDisplayClusterVblankMonitor, ESPMode::ThreadSafe> GetVblankMonitor() override
	{
		return VBlankMonitor;
	}

	virtual IDisplayClusterViewportManager* GetViewportManager() const override;

public:
	//////////////////////////////////////////////////////////////////////////////////////////////
	// IPDisplayClusterRenderManager
	//////////////////////////////////////////////////////////////////////////////////////////////

private:
	//////////////////////////////////////////////////////////////////////////////////////////////
	// FDisplayClusterRenderManager
	//////////////////////////////////////////////////////////////////////////////////////////////
	void ResizeWindow(int32 WinX, int32 WinY, int32 ResX, int32 ResY);
	void OnBackBufferReadyToPresent_SetCustomPresent(class SWindow& Window, class ISlateViewportProvider& ViewportProvider) const;
	void OnViewportCreatedHandler_CheckViewportClass() const;

private:
	EDisplayClusterOperationMode CurrentOperationMode;

	// Interface pointer to avoid type casting
	IDisplayClusterRenderDevice* RenderDevicePtr = nullptr;

	mutable FCustomPresentRHIRef NativePresentHandler = nullptr;

private:
	// Rendering device factories
	TMap<FString, TSharedPtr<IDisplayClusterRenderDeviceFactory>> RenderDeviceFactories;
	// Helper function to instantiate a rendering device
	TSharedPtr<IDisplayClusterRenderDevice, ESPMode::ThreadSafe> CreateRenderDevice() const;

private:
	// Synchronization internals
	TMap<FString, TSharedPtr<IDisplayClusterRenderSyncPolicyFactory>> SyncPolicyFactories;
	TSharedPtr<IDisplayClusterRenderSyncPolicy> CreateRenderSyncPolicy() const;
	mutable TSharedPtr<IDisplayClusterRenderSyncPolicy> SyncPolicy;

private:
	// Projection internals
	TMap<FString, TSharedPtr<IDisplayClusterProjectionPolicyFactory>> ProjectionPolicyFactories;

	// Postprocess internals
	TMap<FString, TSharedPtr<IDisplayClusterPostProcessFactory>> PostProcessFactories;

	// Warp internals
	TMap<FString, TSharedPtr<IDisplayClusterWarpPolicyFactory>> WarpPolicyFactories;

private:
	// Internal data access synchronization
	mutable FCriticalSection CritSecInternals;

	// This flag is used to auto-focus the UE window once on start
	bool bWasWindowFocused = false;

private:
	// V-blank monitor
	TSharedPtr<IDisplayClusterVblankMonitor, ESPMode::ThreadSafe> VBlankMonitor;
};
