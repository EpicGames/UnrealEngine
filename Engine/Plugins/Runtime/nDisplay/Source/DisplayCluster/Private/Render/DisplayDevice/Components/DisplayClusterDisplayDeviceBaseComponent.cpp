// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/DisplayDevice/Components/DisplayClusterDisplayDeviceBaseComponent.h"
#include "Render/DisplayDevice/DisplayClusterDisplayDeviceStrings.h"

#include "DisplayClusterRootActor.h"

#include "Materials/Material.h"
#include "UObject/ConstructorHelpers.h"
#include "Engine/TextureRenderTarget2D.h"

namespace UE::DisplayCluster::DisplayDeviceBaseComponent
{
	static inline void ImplUpdatePreviewMeshTechvis(IDisplayClusterViewportPreview& InViewportPreview, UMeshComponent& InMeshComponent)
	{
		if (InViewportPreview.GetConfiguration().IsTechvisEnabled())
		{
			InMeshComponent.bAffectDynamicIndirectLighting = true;
			InMeshComponent.bAffectIndirectLightingWhileHidden = true;
		}
		else
		{
			// When disabling just revert to the archetype values. The original values at the time of enabling
			// techvis aren't saved, and techvis is enabled by default so they'll always be set to true on new meshes.
			if (const UMeshComponent* MeshArchetype = Cast<UMeshComponent>(InMeshComponent.GetArchetype()))
			{
				InMeshComponent.bAffectDynamicIndirectLighting = MeshArchetype->bAffectDynamicIndirectLighting;
				InMeshComponent.bAffectIndirectLightingWhileHidden = MeshArchetype->bAffectIndirectLightingWhileHidden;
			}
		}

		// For all preview meshes:
		InMeshComponent.SetCastShadow(false);
		InMeshComponent.SetHiddenInGame(false);
		InMeshComponent.SetVisibility(true);

		InMeshComponent.bVisibleInReflectionCaptures = false;
		InMeshComponent.bVisibleInRayTracing = false;
		InMeshComponent.bVisibleInRealTimeSkyCaptures = false;
	};

	static inline void ImplUpdatePreviewMeshMaterialInstanceParameters(IDisplayClusterViewportPreview& InViewportPreview, const EDisplayClusterDisplayDeviceMaterialType InMaterialType, UMaterialInstanceDynamic& InMaterialInstance)
	{
		// Update preview RTT parameter values:
		switch (InMaterialType)
		{
		case EDisplayClusterDisplayDeviceMaterialType::PreviewMeshMaterial:
		case EDisplayClusterDisplayDeviceMaterialType::PreviewMeshTechvisMaterial:
			if (InViewportPreview.HasAnyFlags(EDisplayClusterViewportPreviewFlags::HasChangedPreviewMeshMaterialInstance | EDisplayClusterViewportPreviewFlags::HasChangedPreviewEditableMeshMaterialInstance | EDisplayClusterViewportPreviewFlags::HasChangedPreviewRTT))
			{
				// Updates the RTT parameter for the material instance when the PreviewTexture or mesh material is changed.
				if (UTextureRenderTarget2D* PreviewTexture = InViewportPreview.GetPreviewTextureRenderTarget2D())
				{
					InMaterialInstance.SetTextureParameterValue(UE::DisplayClusterDisplayDeviceStrings::material::attr::Preview, PreviewTexture);
				}
				else
				{
					// Note: when preview texture not available, show default
					static TObjectPtr<UTexture2D> DefaultGridTexture = LoadObject<UTexture2D>(nullptr, TEXT("/Engine/EditorMaterials/T_1x1_Grid.T_1x1_Grid"));
					if (DefaultGridTexture)
					{
						InMaterialInstance.SetTextureParameterValue(UE::DisplayClusterDisplayDeviceStrings::material::attr::Preview, DefaultGridTexture);
					}
				}
			}
			break;

		default:
			break;
		}
	}
};

bool UDisplayClusterDisplayDeviceBaseComponent::ShouldUseDisplayDevice(IDisplayClusterViewportConfiguration& InConfiguration) const
{
	// Use this Display device only for preview
	if (InConfiguration.IsPreviewRendering())
	{
		return true;
	}

	return false;
}

TSharedPtr<IDisplayClusterDisplayDeviceProxy, ESPMode::ThreadSafe> UDisplayClusterDisplayDeviceBaseComponent::GetDisplayDeviceProxy(IDisplayClusterViewportConfiguration& InConfiguration)
{
	if (!ShouldUseDisplayDevice(InConfiguration))
	{
		return nullptr;
	}

	UpdateDisplayDeviceProxyImpl(InConfiguration);

	return DisplayDeviceProxy;
}

UDisplayClusterDisplayDeviceBaseComponent::UDisplayClusterDisplayDeviceBaseComponent()
{
	static ConstructorHelpers::FObjectFinder<UMaterial> MeshMaterialObj(UE::DisplayClusterDisplayDeviceStrings::material::asset::mesh);
	check(MeshMaterialObj.Object);
	MeshMaterial = MeshMaterialObj.Object;

	static ConstructorHelpers::FObjectFinder<UMaterial> PreviewMeshMaterialObj(UE::DisplayClusterDisplayDeviceStrings::material::asset::preview_mesh);
	check(PreviewMeshMaterialObj.Object);
	PreviewMeshMaterial = PreviewMeshMaterialObj.Object;

	static ConstructorHelpers::FObjectFinder<UMaterial> PreviewMeshTechvisMaterialObj(UE::DisplayClusterDisplayDeviceStrings::material::asset::preview_techvis_mesh);
	check(PreviewMeshTechvisMaterialObj.Object);
	PreviewMeshTechvisMaterial = PreviewMeshTechvisMaterialObj.Object;
}

TObjectPtr<UMaterial> UDisplayClusterDisplayDeviceBaseComponent::GetDisplayDeviceMaterial(const EDisplayClusterDisplayDeviceMeshType InMeshType, const EDisplayClusterDisplayDeviceMaterialType InMaterialType) const
{
	switch (InMeshType)
	{
	case EDisplayClusterDisplayDeviceMeshType::DefaultMesh:
		return MeshMaterial;

	case EDisplayClusterDisplayDeviceMeshType::PreviewMesh:
	case EDisplayClusterDisplayDeviceMeshType::PreviewEditableMesh:
		switch (InMaterialType)
		{
		case EDisplayClusterDisplayDeviceMaterialType::PreviewMeshMaterial:
			return PreviewMeshMaterial;

		case EDisplayClusterDisplayDeviceMaterialType::PreviewMeshTechvisMaterial:
			return PreviewMeshTechvisMaterial;

		default:
			break;
		}
		break;

	default:
		break;
	}

	return nullptr;
}

void UDisplayClusterDisplayDeviceBaseComponent::OnUpdateDisplayDeviceMeshAndMaterialInstance(IDisplayClusterViewportPreview& InViewportPreview, const EDisplayClusterDisplayDeviceMeshType InMeshType, const EDisplayClusterDisplayDeviceMaterialType InMaterialType, UMeshComponent* InMeshComponent, UMaterialInstanceDynamic* InMeshMaterialInstance) const
{
	using namespace UE::DisplayCluster::DisplayDeviceBaseComponent;

	if (!ShouldUseDisplayDevice(InViewportPreview.GetConfiguration()))
	{
		return;
	}

	switch (InMeshType)
	{
	case EDisplayClusterDisplayDeviceMeshType::PreviewMesh:
	case EDisplayClusterDisplayDeviceMeshType::PreviewEditableMesh:
		// Only some of the preview meshes are supported by Techvis
		if (InMeshComponent)
		{
			ImplUpdatePreviewMeshTechvis(InViewportPreview, *InMeshComponent);
		}

		// Only some of the preview meshes are supported by preview
		if (InMeshMaterialInstance)
		{
			ImplUpdatePreviewMeshMaterialInstanceParameters(InViewportPreview, InMaterialType, *InMeshMaterialInstance);
		}
		break;

	default:
		break;
	}
}

void UDisplayClusterDisplayDeviceBaseComponent::SetupSceneView(const IDisplayClusterViewportPreview& InViewportPreview, uint32 ContextNum, FSceneViewFamily& InOutViewFamily, FSceneView& InOutView) const
{
	// This component does not change rendering settings
}

void UDisplayClusterDisplayDeviceBaseComponent::UpdateDisplayDeviceProxyImpl(IDisplayClusterViewportConfiguration& InConfiguration)
{
	// This component does not interact with the rendering thread
}

#if WITH_EDITOR
void UDisplayClusterDisplayDeviceBaseComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif
