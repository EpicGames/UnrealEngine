// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/DisplayDevice/Components/DisplayClusterDisplayDeviceComponent.h"
#include "Render/Viewport/IDisplayClusterViewportConfiguration.h"

#include "Render/DisplayDevice/Proxy/DisplayClusterDisplayDeviceProxy_OpenColorIO.h"
#include "Render/DisplayDevice/DisplayClusterDisplayDeviceStrings.h"

#include "DisplayClusterRootActor.h"
#include "IDisplayCluster.h"

#include "Engine/TextureRenderTarget2D.h"
#include "Materials/MaterialInstanceDynamic.h"

UDisplayClusterDisplayDeviceComponent::UDisplayClusterDisplayDeviceComponent()
{ }

void UDisplayClusterDisplayDeviceComponent::OnUpdateDisplayDeviceMeshAndMaterialInstance(IDisplayClusterViewportPreview& InViewportPreview, const EDisplayClusterDisplayDeviceMeshType InMeshType, const EDisplayClusterDisplayDeviceMaterialType InMaterialType, UMeshComponent* InMeshComponent, UMaterialInstanceDynamic* InMeshMaterialInstance) const
{
	// Call a method of the base class
	Super::OnUpdateDisplayDeviceMeshAndMaterialInstance(InViewportPreview, InMeshType, InMaterialType, InMeshComponent, InMeshMaterialInstance);

	// Customizes and overrides material parameters after the base class
	if (InMeshMaterialInstance && ShouldUseDisplayDevice(InViewportPreview.GetConfiguration()))
	{
		InMeshMaterialInstance->SetScalarParameterValue(UE::DisplayClusterDisplayDeviceStrings::material::attr::Exposure, Exposure);
		InMeshMaterialInstance->SetScalarParameterValue(UE::DisplayClusterDisplayDeviceStrings::material::attr::Gamma, Gamma);
	}
}

void UDisplayClusterDisplayDeviceComponent::UpdateDisplayDeviceProxyImpl(IDisplayClusterViewportConfiguration& InConfiguration)
{
	if (!ShouldUseDisplayDevice(InConfiguration))
	{
		// Use this only for preview rendering
		DisplayDeviceProxy.Reset();

		return;
	}

	if (!bEnableRenderPass || !ColorConversionSettings.IsValid())
	{
		// OCIO for preview is disabled
		DisplayDeviceProxy.Reset();

		return;
	}

	// Add a single OCIO render pass at the rendering thread

	// If the settings have changed, create a new proxy object
	if(FDisplayClusterDisplayDeviceProxy_OpenColorIO* Proxy = static_cast<FDisplayClusterDisplayDeviceProxy_OpenColorIO*>(DisplayDeviceProxy.Get()))
	{
		const bool bIsOCIOEquals = Proxy->OCIOPassId == ColorConversionSettings.ToString();
		if (!bIsOCIOEquals)
		{
			DisplayDeviceProxy.Reset();
		}
	}

	if (!DisplayDeviceProxy.IsValid())
	{
		// OCIO shaders may not be ready at the moment, so we should check for them before creating our own proxy.
		FOpenColorIORenderPassResources OCIOPassResources = FOpenColorIORendering::GetRenderPassResources(ColorConversionSettings, GMaxRHIFeatureLevel);
		if(OCIOPassResources.IsValid())
		{
			// Create a DisplayDevice proxy only when OCIO render pass resources are available for use.
			DisplayDeviceProxy = MakeShared<FDisplayClusterDisplayDeviceProxy_OpenColorIO, ESPMode::ThreadSafe>(ColorConversionSettings.ToString(), OCIOPassResources);
		}
	}
}
