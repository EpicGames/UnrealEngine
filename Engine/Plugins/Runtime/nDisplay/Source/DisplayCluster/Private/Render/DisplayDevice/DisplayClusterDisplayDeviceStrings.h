// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace UE::DisplayClusterDisplayDeviceStrings
{
	namespace material
	{
		namespace asset
		{
			static constexpr const TCHAR* mesh = TEXT("/nDisplay/Materials/Preview/M_DisplayDeviceMesh");

			static constexpr const TCHAR* preview_mesh = TEXT("/nDisplay/Materials/Preview/M_DisplayDevicePreview");
			static constexpr const TCHAR* preview_techvis_mesh = TEXT("/nDisplay/Materials/Preview/M_DisplayDevicePreviewTechvis");
		}

		namespace attr
		{
			static constexpr const TCHAR* Preview = TEXT("Preview");
			static constexpr const TCHAR* Opacity = TEXT("Opacity");
			static constexpr const TCHAR* Exposure = TEXT("Exposure");
			static constexpr const TCHAR* Gamma = TEXT("Gamma");
		}
	}
};
