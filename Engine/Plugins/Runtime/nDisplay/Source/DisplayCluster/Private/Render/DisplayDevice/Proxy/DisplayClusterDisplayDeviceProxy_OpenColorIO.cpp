// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/DisplayDevice/Proxy/DisplayClusterDisplayDeviceProxy_OpenColorIO.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_Context.h"
#include "Render/Viewport/DisplayClusterViewport_OpenColorIO.h"

#include "IDisplayClusterShadersTextureUtils.h"

#include "Shader.h"
#include "SceneView.h"
#include "ScreenPass.h"
#include "CommonRenderResources.h"
#include "PostProcess/PostProcessMaterialInputs.h"
#include "Engine/TextureRenderTarget.h"

bool FDisplayClusterDisplayDeviceProxy_OpenColorIO::HasFinalPass_RenderThread() const
	{
	return true;
	}

bool FDisplayClusterDisplayDeviceProxy_OpenColorIO::AddFinalPass_RenderThread(
	const FDisplayClusterShadersTextureUtilsSettings& InTextureUtilsSettings,
	const TSharedRef<IDisplayClusterShadersTextureUtils, ESPMode::ThreadSafe>& InTextureUtils) const
{
	if (!OCIOPassResources.IsValid())
{
		// The OCIO shader is not ready at this point, use the default resolve method.
		InTextureUtils->Resolve(InTextureUtilsSettings);

	return true;
}

	// When requesting RDG from the TextureUtils API, it switches from RHI to RDG.
	FRDGBuilder& GraphBuilder = InTextureUtils->GetOrCreateRDGBuilder();

	bool bFinalPassResult = false;
	InTextureUtils->ForEachContextByPredicate([&](
		const FDisplayClusterShadersTextureViewportContext& Input,
		const FDisplayClusterShadersTextureViewportContext& Output)
		{
			// returns true if OCIO is applied.
			bFinalPassResult = true;

			// The OCIO output is in sRGB
			FOpenColorIORendering::AddPass_RenderThread(
				GraphBuilder,
				FScreenPassViewInfo(),
				GMaxRHIFeatureLevel,
				Input.ToScreenPassTexture(),
				FScreenPassRenderTarget(Output.ToScreenPassTexture(), ERenderTargetLoadAction::EClear),
				OCIOPassResources,
				FDisplayClusterViewport_OpenColorIO::GetGammaCorrection(Input.ColorEncoding),
				FDisplayClusterViewport_OpenColorIO::GetTransformAlpha(Input.ColorEncoding)
			);

			// Convert to output color space if necessary.
			if (Output.ColorEncoding.GetEqualEncoding() != EDisplayClusterColorEncoding::sRGB)
			{
				// Convert OutputTexture color encoding from sRGB to Linear(Holdout):
				InTextureUtils->ResolveTextureContext(
					// Customize the settings by requesting a temporary input texture cloned from the output texture.
					{ InTextureUtilsSettings , EDisplayClusterShaderTextureUtilsFlags::UseOutputTextureAsInput },
					// The input texture is empty, a temporary texture created from the output will be used instead.
					// Holdout uses the Liner->sRGB transform at the end, so use sRGB as the input encoding here even if it's different.
					{ {}, {EDisplayClusterColorEncoding::sRGB, Input.ColorEncoding.Premultiply } },
					// The output texture already contains the OCIO result and will be used as input in this pass.
					Output
				);
			}
		});

	return bFinalPassResult;
}
