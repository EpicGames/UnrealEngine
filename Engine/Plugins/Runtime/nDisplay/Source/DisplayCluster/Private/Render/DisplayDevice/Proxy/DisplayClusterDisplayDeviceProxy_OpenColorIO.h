// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Render/DisplayDevice/IDisplayClusterDisplayDeviceProxy.h"

#include "OpenColorIORendering.h"

/**
 * Display Device Proxy object (OCIO render pass)
 * [rendering thread]
 */
class FDisplayClusterDisplayDeviceProxy_OpenColorIO
	: public IDisplayClusterDisplayDeviceProxy
{
public:
	FDisplayClusterDisplayDeviceProxy_OpenColorIO(const FString& InOCIOPassId, FOpenColorIORenderPassResources& InOCIOPassResources)
		: OCIOPassId(InOCIOPassId)
		, OCIOPassResources(InOCIOPassResources)
	{ }

	virtual ~FDisplayClusterDisplayDeviceProxy_OpenColorIO() = default;

public:
	//~Begin IDisplayClusterDisplayDeviceProxy
	virtual bool HasFinalPass_RenderThread() const override;
	virtual bool AddFinalPass_RenderThread(const FDisplayClusterShadersTextureUtilsSettings& InTextureUtilsSettings, const TSharedRef<IDisplayClusterShadersTextureUtils, ESPMode::ThreadSafe>& InTextureUtils) const override;
	//~~End IDisplayClusterDisplayDeviceProxy

public:
	const FString OCIOPassId;

private:
	FOpenColorIORenderPassResources OCIOPassResources;
};
