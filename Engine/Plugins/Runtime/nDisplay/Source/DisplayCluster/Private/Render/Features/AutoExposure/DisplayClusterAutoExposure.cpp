// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Features/AutoExposure/DisplayClusterAutoExposure.h"

#include "Misc/DisplayClusterLog.h"

#include "Render/Viewport/Configuration/DisplayClusterViewportConfiguration.h"
#include "Render/Viewport/DisplayClusterViewport.h"
#include "Render/Viewport/DisplayClusterViewportManager.h"
#include "Render/Viewport/DisplayClusterViewportProxy.h"

#include "RenderUtils.h"

// When enabled, compiles in extra UE_LOG output for nDisplay auto-exposure diagnostics.
#define ENABLE_DISPLAYCLUSTER_AUTOEXPOSURE_LOG 0
#define ENABLE_DISPLAYCLUSTER_AUTOEXPOSURE_LOG_MATH 0

/** CPU version for functions from Engine\Shaders\Private\PostProcessHistogramCommon.ush */
namespace UE::DisplayClusterRender::Features::AutoExposure::Math
{
	/** CPU version of Eye Adaptation parameters.
	 * Based on structure from \Engine\Source\Runtime\Renderer\Private\PostProcess\PostProcessEyeAdaptation.h
	 * CPU dosn't support textures. Remove related parameters (Histogram, CompensationCurve, MeterMask).
	 */
	struct FEyeAdaptationParametersForCPU
	{
		float MinAverageLuminance = 0;
		float MaxAverageLuminance = 0;
		float DeltaWorldTime = 0;
		float ExposureSpeedUp = 3;
		float ExposureSpeedDown = 1;
		float ExponentialUpM = 0;
		float ExponentialDownM = 0;
		float StartDistance = 3;
		bool  ForceTarget = false;
	};

	// Logic based on ExponentialAdaption() from Engine\Shaders\Private\PostProcessHistogramCommon.ush
	static float ExponentialAdaption(float Current, float Target, float FrameTime, float AdaptionSpeed, float M)
	{
		const float Factor = 1.0f - FMath::Exp2(-FrameTime * AdaptionSpeed);
		const float Value = Current + (Target - Current) * Factor * M;

		return Value;
	}

	// Logic based on LinearAdaption() from Engine\Shaders\Private\PostProcessHistogramCommon.ush
	static float LinearAdaption(float Current, float Target, float FrameTime, float AdaptionSpeed)
	{
		const float Offset = FrameTime * AdaptionSpeed;

		const float Value = (Current < Target)
			? FMath::Min(Target, Current + Offset)
			: FMath::Max(Target, Current - Offset);

		return Value;
	}

	// Logic based on the ComputeEyeAdaptation() from Engine\Shaders\Private\PostProcessHistogramCommon.ush
	static float ComputeEyeAdaptation(const FEyeAdaptationParametersForCPU& EyeAdaptation, float OldExposure, float TargetExposure, float FrameTime)
	{
		// for manual mode or camera cuts, just lerp to the target
		if (EyeAdaptation.ForceTarget)
		{
			return TargetExposure;
		}

		// Avoid log2(0) -inf and later NaNs.
		const float SafeOld = FMath::Max(OldExposure, SMALL_NUMBER);
		const float SafeTarget = FMath::Max(TargetExposure, SMALL_NUMBER);

		const float LogTargetExposure = FMath::Log2(SafeTarget);
		const float LogOldExposure = FMath::Log2(SafeOld);

		const float LogDiff = LogTargetExposure - LogOldExposure;

		const bool bGoingUp = (LogDiff > 0.0f);
		const float AdaptionSpeed = bGoingUp ? EyeAdaptation.ExposureSpeedUp : EyeAdaptation.ExposureSpeedDown;
		const float M = bGoingUp ? EyeAdaptation.ExponentialUpM : EyeAdaptation.ExponentialDownM;

		const float AbsLogDiff = FMath::Abs(LogDiff);

		// blended exposure
		float LogAdaptedExposure = LogTargetExposure;
		if (AbsLogDiff > EyeAdaptation.StartDistance)
		{
			// Linear
			LogAdaptedExposure = LinearAdaption(LogOldExposure, LogTargetExposure, FrameTime, AdaptionSpeed);
		}
		else
		{
			// Exponential adaptation on the close distance
			LogAdaptedExposure = ExponentialAdaption(LogOldExposure, LogTargetExposure, FrameTime, AdaptionSpeed, M);
		}

#if ENABLE_DISPLAYCLUSTER_AUTOEXPOSURE_LOG_MATH
		// Debug: log auto-exposure adaptation per frame.
		UE_LOGF(LogDisplayClusterViewport, Log, "[%d] AutoExposure %.3f -> %.3f = %.3f (S=%.3f, M=%.3f, T=%.4f)",
			GFrameNumber, LogOldExposure, LogTargetExposure, LogAdaptedExposure, AdaptionSpeed, M, FrameTime);
#endif /** ENABLE_DISPLAYCLUSTER_AUTOEXPOSURE_LOG_MATH */

		// Note: no clamping here. Target exposure should be clamped upstream.
		const float AdaptedExposure = FMath::Exp2(LogAdaptedExposure);

		return AdaptedExposure;
	}

	/** Return eye adaptation changes over time
	* Logic based on the EyeAdaptationCommon() from '\\Engine\Shaders\Private\PostProcessEyeAdaptation.usf'
	*
	* return estimated luminance for current frame
	*/
	static float GetEstimatedLuminance(const FEyeAdaptationParametersForCPU& EyeAdaptation, float TargetLuminance, float PrevFrameLuminance)
	{
		const float TargetExposure = FMath::Clamp(TargetLuminance,    EyeAdaptation.MinAverageLuminance, EyeAdaptation.MaxAverageLuminance);
		const float OldExposure    = FMath::Clamp(PrevFrameLuminance, EyeAdaptation.MinAverageLuminance, EyeAdaptation.MaxAverageLuminance);

		// eye adaptation changes over time
		const float EstimatedExposure = ComputeEyeAdaptation(EyeAdaptation, OldExposure, TargetExposure, EyeAdaptation.DeltaWorldTime);
		const float FinalExposure = FMath::Clamp(EstimatedExposure, EyeAdaptation.MinAverageLuminance, EyeAdaptation.MaxAverageLuminance);

		return FinalExposure;
	}
};

namespace UE::DisplayClusterRender::Features::AutoExposure
{
	using namespace UE::DisplayClusterRender::Features::AutoExposure::Math;

	/**
	* Checks whether the extended luminance range (EV100) is enabled.
	*
	* When enabled, certain post-processing parameters are expressed in
	* exposure values (EV100). When disabled, they are expressed in pixel
	* luminance (cd/m²).
	*
	* @return true  if the extended luminance range (EV100) is enabled
	* @return false if parameters are expressed in pixel luminance (cd/m²)
	*/
	static bool IsExtendLuminanceRangeEnabled()
	{
		// Notes:
		//    CVar ("r.DefaultFeature.AutoExposure.ExtendDefaultLuminanceRange"),
		//    Whether the default values for AutoExposure should support an extended range of scene luminance.
		//    This also change the PostProcessSettings.Exposure.MinBrightness, MaxBrightness, HistogramLogMin and HisogramLogMax
		//    to be expressed in EV100 values instead of in Luminance and Log2 Luminance
		//    0: Legacy range (UE4 default)
		//    1: Extended range (UE5 default)
		static const TConsoleVariableData<int32>* const ExtendDefaultLuminanceRangeCVar =
			IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.DefaultFeature.AutoExposure.ExtendDefaultLuminanceRange"));
		if (ExtendDefaultLuminanceRangeCVar)
		{
			return ExtendDefaultLuminanceRangeCVar->GetValueOnGameThread() != 0;
		}

		// UE5 default use extended range
		return true;
	}

	// Function type used to calculate a per-viewport value with an associated weight.
	using FAutoExposureMetricFunc = TFunction<void(
		const FDisplayClusterViewportAutoExposureData& ViewportData,
		const double ViewportWeight)>;

	/**
	* Iterates over viewports and invokes a user-supplied function with each viewport and its normalized weight.
	* 
	* @param InViewports  Array of viewports to process.
	* @param MetricFunc   Function to execute per viewport with its normalized weight.
	*/
	static void ForEachViewportWeighted(
		const TArray<FDisplayClusterViewportAutoExposureData>& InViewports,
		FAutoExposureMetricFunc&& MetricFunc)
	{
		uint32 TotalPixels = 0;
		for (const FDisplayClusterViewportAutoExposureData& ViewportIt : InViewports)
		{
			const uint32 VisiblePixels = FMath::Max(ViewportIt.NumVisiblePixels, 0);

			// Prevent uint32 overflow when accumulating visible pixels across viewports.
			if (TotalPixels > UINT32_MAX - VisiblePixels)
			{
				// The weights of viewports cannot be calculated.
				return;
			}

			TotalPixels += VisiblePixels;
		}

		if (TotalPixels == 0)
		{
			// The weights of viewports cannot be calculated.
			return;
		}

		for (const FDisplayClusterViewportAutoExposureData& ViewportIt : InViewports)
		{
			// Viewport Weight in 0..1 range
			const double NumVisiblePixels = static_cast<double>(FMath::Max(ViewportIt.NumVisiblePixels, 0));
			const double NumTotalPixels = static_cast<double>(TotalPixels);
			const double ViewportWeight = NumVisiblePixels / NumTotalPixels;

			MetricFunc(ViewportIt, ViewportWeight);
		}
	}

	/**
	* Copied from \\Engine\Source\Runtime\Renderer\Private\PostProcess\PostProcessEyeAdaptation.cpp
	*/
	static float LuminanceMaxFromLensAttenuation()
	{
		const bool bExtendedLuminanceRange = IsExtendLuminanceRangeEnabled();

		static const auto VarEyeAdaptationLensAttenuation = IConsoleManager::Get().FindTConsoleVariableDataFloat(TEXT("r.EyeAdaptation.LensAttenuation"));
		check(VarEyeAdaptationLensAttenuation);
		float LensAttenuation = VarEyeAdaptationLensAttenuation->GetValueOnGameThread();

		// 78 is defined in the ISO 12232:2006 standard.
		const float kISOSaturationSpeedConstant = 0.78f;

		const float LuminanceMax = kISOSaturationSpeedConstant / FMath::Max<float>(LensAttenuation, .01f);

		// if we do not have luminance range extended, the math is hardcoded to 1.0 scale.
		return bExtendedLuminanceRange ? LuminanceMax : 1.0f;
	}

	/**
	* Logic from \\Engine\Source\Runtime\Renderer\Private\PostProcess\PostProcessEyeAdaptation.cpp -> GetEyeAdaptationParameters()
	*/
	static FEyeAdaptationParametersForCPU GetEyeAdaptationParameters(const FSceneView& View)
	{
		const bool bExtendedLuminanceRange = IsExtendLuminanceRangeEnabled();
		const float LuminanceMax = LuminanceMaxFromLensAttenuation();
		const float kMiddleGrey = 0.18f;

		const FPostProcessSettings& Settings = View.FinalPostProcessSettings;

		// These clamp the average luminance computed from the scene color. We are going to calculate the white point first, and then
		// figure out the average grey point later. I.e. if the white point is 1.0, the middle grey point should be 0.18.
		float MinWhitePointLuminance = 1.0f;
		float MaxWhitePointLuminance = 1.0f;
		{
			if (bExtendedLuminanceRange)
			{
				MinWhitePointLuminance = EV100ToLuminance(LuminanceMax, Settings.AutoExposureMinBrightness);
				MaxWhitePointLuminance = EV100ToLuminance(LuminanceMax, Settings.AutoExposureMaxBrightness);
			}
			else
			{
				MinWhitePointLuminance = Settings.AutoExposureMinBrightness;
				MaxWhitePointLuminance = Settings.AutoExposureMaxBrightness;
			}
			MinWhitePointLuminance = FMath::Min(MinWhitePointLuminance, MaxWhitePointLuminance);
		}

		static const auto CVarEyeAdaptationExponentialTransitionDistance =
			IConsoleManager::Get().FindTConsoleVariableDataFloat(TEXT("r.EyeAdaptation.ExponentialTransitionDistance"));
		check(CVarEyeAdaptationExponentialTransitionDistance);

		// Logic from PostProcessEyeAdaptation.cpp -> GetEyeAdaptationParameters()

		// The distance at which we switch from linear to exponential. I.e. at StartDistance=1.5, when linear is 1.5 f-stops away from hitting the 
		// target, we switch to exponential.
		const float StartDistance = FMath::Max(CVarEyeAdaptationExponentialTransitionDistance->GetValueOnGameThread(), 0.001f);
		const float StartTimeUp = StartDistance / FMath::Max(Settings.AutoExposureSpeedUp, 0.001f);
		const float StartTimeDown = StartDistance / FMath::Max(Settings.AutoExposureSpeedDown, 0.001f);

		// We want to ensure that at time=StartT, that the derivative of the exponential curve is the same as the derivative of the linear curve.
		// For the linear curve, the step will be AdaptationSpeed * FrameTime.
		// For the exponential curve, the step will be at t=StartT, M is slope modifier:
		//      slope(t) = M * (1.0f - exp2(-FrameTime * AdaptionSpeed)) * AdaptionSpeed * StartT
		//      AdaptionSpeed * FrameTime = M * (1.0f - exp2(-FrameTime * AdaptionSpeed)) * AdaptationSpeed * StartT
		//      M = FrameTime / (1.0f - exp2(-FrameTime * AdaptionSpeed)) * StartT
		//
		// To be technically correct, we should take the limit as FrameTime->0, but for simplicity we can make FrameTime a small number. So:
		const float kFrameTimeEps = 1.0f / 60.0f;
		const float ExponentialUpM = kFrameTimeEps / ((1.0f - exp2(-kFrameTimeEps * Settings.AutoExposureSpeedUp)) * StartTimeUp);
		const float ExponentialDownM = kFrameTimeEps / ((1.0f - exp2(-kFrameTimeEps * Settings.AutoExposureSpeedDown)) * StartTimeDown);

		// If the white point luminance is 1.0, then the middle grey luminance should be 0.18.
		const float MinAverageLuminance = MinWhitePointLuminance * kMiddleGrey;
		const float MaxAverageLuminance = MaxWhitePointLuminance * kMiddleGrey;

		const bool bValidRange = Settings.AutoExposureMinBrightness < Settings.AutoExposureMaxBrightness;
		const bool bValidSpeeds = Settings.AutoExposureSpeedDown >= 0.f && Settings.AutoExposureSpeedUp >= 0.f;

		// if it is a camera cut we force the exposure to go all the way to the target exposure without blending.
		// if it is manual mode, we also force the exposure to hit the target, which matters for HDR Visualization
		// if we don't have a valid range (AutoExposureMinBrightness == AutoExposureMaxBrightness) then force it like Manual as well.
		const bool ForceTarget = (View.bCameraCut || !bValidRange || !bValidSpeeds);

		FEyeAdaptationParametersForCPU Parameters;
		Parameters.MinAverageLuminance = MinAverageLuminance;
		Parameters.MaxAverageLuminance = MaxAverageLuminance;
		Parameters.DeltaWorldTime = View.Family->Time.GetDeltaWorldTimeSeconds();
		Parameters.ExposureSpeedUp = Settings.AutoExposureSpeedUp;
		Parameters.ExposureSpeedDown = Settings.AutoExposureSpeedDown;
		Parameters.ExponentialDownM = ExponentialDownM;
		Parameters.ExponentialUpM = ExponentialUpM;
		Parameters.StartDistance = StartDistance;
		Parameters.ForceTarget = ForceTarget;

		return Parameters;
	}

	/**
	 * Computes the Auto Exposure (AE) brightness value to override
	 * AutoExposureMinBrightness/MaxBrightness for a scene view.
	 *
	 * Uses the most recent eye-adaptation measurements from the specified
	 * metering viewports, together with the configured AE settings, to
	 * calculate the brightness.
	 *
	 * @param InMeteringViewports Viewports contributing to the average luminance used for metering.
	 * @return                    Brightness, suitable for AutoExposureMinBrightness/MaxBrightness override.
	 */
	static float ComputeDestBrightness(
		const TArray<FDisplayClusterViewportAutoExposureData>& InMeteringViewports)
	{
		// Average luminance
		double DestLuminance = 0;

		// Iterate over metering (dest) viewports
		ForEachViewportWeighted(InMeteringViewports,
			[&](const FDisplayClusterViewportAutoExposureData& ViewportData, const double ViewportWeight)
			{
				DestLuminance += ViewportData.LastAverageSceneLuminance * ViewportWeight;
			});

		const float LuminanceMax = LuminanceMaxFromLensAttenuation();

		const float DestLuminanceEV100 = LuminanceToEV100(LuminanceMax, DestLuminance);
		const float MidGrayLuminanceEV100 = LuminanceToEV100(LuminanceMax, 0.18f);

		const float FinalDestExposureEV100 = DestLuminanceEV100 - MidGrayLuminanceEV100;

		// All viewports should use same exposure
		const float DestBrightness = IsExtendLuminanceRangeEnabled()
			? FinalDestExposureEV100
			: EV100ToLuminance(LuminanceMax, FinalDestExposureEV100);

		return DestBrightness;
	}

	/**
	 * Computes the next eye-adaptation brightness value for this view.
	 *
	 * Moves the adaptation state from OldBrightness toward TargetBrightness using the view's
	 * exposure/eye-adaptation settings (e.g., speed up/down, clamping).
	 *
	 * @param View              Scene view providing exposure/adaptation parameters.
	 * @param TargetBrightness  Desired brightness after adaptation (the goal value).
	 * @param OldBrightness     Previous frame's adapted brightness (the current state).
	 * @return                  New adapted brightness for this frame.
	 */
	static float ComputeEyeAdaptationBrightness(const FSceneView & View, const float TargetBrightness, const float OldBrightness)
	{
		const bool bExtendLuminanceRangeEnabled = IsExtendLuminanceRangeEnabled();
		const float LuminanceMax = LuminanceMaxFromLensAttenuation();

		// Get EyeAdaptation parameters from View
		const FEyeAdaptationParametersForCPU EyeAdaptationParameters = GetEyeAdaptationParameters(View);

		const float TargetLuminance = bExtendLuminanceRangeEnabled
			? EV100ToLuminance(LuminanceMax, TargetBrightness)
			: TargetBrightness;

		const float OldLuminance = bExtendLuminanceRangeEnabled
			? EV100ToLuminance(LuminanceMax, OldBrightness)
			: OldBrightness;

		// Calculate new brightness value based on eye adaptation changes over time
		const float NewLuminance = GetEstimatedLuminance(EyeAdaptationParameters, TargetLuminance, OldLuminance);

		const float NewBrightness = bExtendLuminanceRangeEnabled
			? LuminanceToEV100(LuminanceMax, NewLuminance)
			: NewLuminance;

		return NewBrightness;
	}

	/**
	 * Checks whether the specified viewport proxy is allowed to contribute
	 * to auto-exposure metering on the render thread.
	 *
	 * @param InViewportProxy  The viewport proxy to test for AE metering eligibility.
	 * @return                 True if this viewport is allowed to contribute to AE metering.
	 */
	static bool IsViewportAllowedForMetering_RenderThread(
		const TSharedPtr<FDisplayClusterViewportProxy, ESPMode::ThreadSafe>& InViewportProxy)
	{
		check(IsInRenderingThread());

		const TArray<FDisplayClusterViewport_Context>& Contexts = InViewportProxy->GetContexts_RenderThread();
		if (Contexts.IsEmpty())
		{
			return false;
		}

		const FDisplayClusterViewport_Context& InContext = InViewportProxy->GetContexts_RenderThread()[0];
		const FDisplayClusterViewport_RenderSettings& InRenderSettings = InViewportProxy->GetRenderSettings_RenderThread();

		if (InContext.bDisableRender || InRenderSettings.bSkipRendering)
		{
			// Ignore viewports that are not used for rendering.
			return false;
		}

		if (!InContext.RenderThreadData.EyeAdaptationData.CanBeUsed())
		{
			// Ignore viewports without EyeAdaptation data
			return false;
		}

		return true;
	}

	/** Convert FDisplayClusterNodeAutoExposureData to text. */
	static FString ToString(const FDisplayClusterNodeAutoExposureData& InNodeAutoExposureData)
	{
		FString OutText;
		for (const FDisplayClusterViewportAutoExposureData& ViewportIt : InNodeAutoExposureData.Viewports)
		{
			OutText += FString::Printf(TEXT(" [%d, %.5f] "), ViewportIt.FrameNumber, ViewportIt.LastAverageSceneLuminance);
		}

		return OutText;
	}

	/** Convert TMap<FName, FDisplayClusterNodeAutoExposureData> to text. */
	static FString ToString(const TMap<FName, FDisplayClusterNodeAutoExposureData>& InPerNodeAutoExposureData)
	{
		FString OutText;

		// Sort nodes:
		TArray<FName> Keys;
		InPerNodeAutoExposureData.GetKeys(Keys);

		Keys.Sort([](const FName& A, const FName& B)
			{
				return A.LexicalLess(B); // alphabetical
			});

		for (const FName& Key : Keys)
		{
			OutText += FString::Printf(TEXT("%s={%s} "), *Key.ToString(), *ToString(InPerNodeAutoExposureData[Key]));
		}

		return OutText;
	}

	// The unique name for the FAutoExposureState object.
	static const FName AutoExposureStateName = TEXT("AutoExposure");
};

float FDisplayClusterAutoExposure::ComputeViewportAutoExposureBrightness(
	const FDisplayClusterViewport& InViewport,
	const uint32 InContextNum,
	FSceneView& View)
{
	// Finds an existing cache entry keyed by (InViewport, InContextNum) or creates one if missing.
	TOptional<float>* PrevFrameBrightness = nullptr;
	{

		// Opportunistic cleanup: drop entries whose viewport is gone.
		PrevFrameBrightnessCache.RemoveAllSwap([](const FPrevFrameBrightnessCacheEntry& Entry)
			{
				return !Entry.ViewportWeakPtr.IsValid();
			}, EAllowShrinking::No);

		// Find an existing entry keyed by (Viewport, Context).
		for (FPrevFrameBrightnessCacheEntry& Entry : PrevFrameBrightnessCache)
		{
			if (Entry.ContextIndex != InContextNum)
			{
				continue;
			}

			const TSharedPtr<const FDisplayClusterViewport, ESPMode::ThreadSafe> ViewportPinned = Entry.ViewportWeakPtr.Pin();
			if (!ViewportPinned.IsValid())
			{
				continue;
			}

			// Key match: compare actual viewport object identity.
			if (ViewportPinned.Get() == &InViewport)
			{
				PrevFrameBrightness = &Entry.PrevFrameBrightness;
				break;
			}
		}

		if (!PrevFrameBrightness)
		{
			// Not found: create a new cache entry.
			FPrevFrameBrightnessCacheEntry& NewEntry = PrevFrameBrightnessCache.AddDefaulted_GetRef();
			NewEntry.ViewportWeakPtr = InViewport.AsShared();
			NewEntry.ContextIndex = InContextNum;

			PrevFrameBrightness = &NewEntry.PrevFrameBrightness;
		}
	}

	using namespace UE::DisplayClusterRender::Features::AutoExposure;

	// Compute the brightness for this frame. If we have a previous value, apply eye adaptation,
	// otherwise fall back to the target brightness for the first frame.
	const float NewBrightness = PrevFrameBrightness && PrevFrameBrightness->IsSet()
		? ComputeEyeAdaptationBrightness(View, *TargetBrightness, PrevFrameBrightness->GetValue())
		: *TargetBrightness;

	// Update the cache with the brightness from the current frame.
	if (PrevFrameBrightness)
	{
		*PrevFrameBrightness = NewBrightness;
	}

	return NewBrightness;
}

void FDisplayClusterAutoExposure::ConfigureAutoExposureForSceneView(const FDisplayClusterViewport& InViewport, const uint32 InContextNum, FSceneView& View)
{

	if (!TargetBrightness.IsSet())
	{
		// Skip viewport if CurrentFrameBrightness isn't calculated.
		return;
	}

	if (!View.Family || !View.Family->EngineShowFlags.EyeAdaptation)
	{
		// Skip viewports that not use eye adptation.
		return;
	}

	// Ignore excluded nodes and viewports
	if(Settings->IsNodeExcludedFromAutoExposure(InViewport.ClusterNodeId)
	|| Settings->IsViewportExcludedFromAutoExposure(InViewport.GetBaseId(), InViewport.GetViewportFlags()))
	{
		return;
	}

	// Override brightness value for this viewport:
	FPostProcessSettings& PPSettings = View.FinalPostProcessSettings;

	// Only AEM_Basic and AEM_Histogram can be used to override the brightness value.
	if (View.FinalPostProcessSettings.AutoExposureMethod == EAutoExposureMethod::AEM_Manual)
	{
		// Override Manual method to AEM_Basic.
		PPSettings.bOverride_AutoExposureMethod = true;
		PPSettings.AutoExposureMethod = EAutoExposureMethod::AEM_Basic;
	}

	// Compute the brightness for this frame. If we have a previous value, apply eye adaptation,
	// otherwise fall back to the target brightness for the first frame.
	const float NewBrightness = ComputeViewportAutoExposureBrightness(InViewport, InContextNum, View);

	// Override Min/Max brightness value
	PPSettings.bOverride_AutoExposureMinBrightness = true;
	PPSettings.bOverride_AutoExposureMaxBrightness = true;
	PPSettings.AutoExposureMinBrightness = NewBrightness;
	PPSettings.AutoExposureMaxBrightness = NewBrightness;

	// Disable AE blending: apply actual brightness value in this frame.
	const float CustomAutoExposureSpeed = -1;
	PPSettings.bOverride_AutoExposureSpeedUp = true;
	PPSettings.bOverride_AutoExposureSpeedDown = true;
	PPSettings.AutoExposureSpeedUp = CustomAutoExposureSpeed;
	PPSettings.AutoExposureSpeedDown = CustomAutoExposureSpeed;	
}

void FDisplayClusterAutoExposure::ReleaseAutoExposureState()
{
	if (AutoExposureState.IsValid())
	{
#if ENABLE_DISPLAYCLUSTER_AUTOEXPOSURE_LOG
		UE_LOGF(LogDisplayClusterViewport, Log, "Delete AutoExposureState.");
#endif /*ENABLE_DISPLAYCLUSTER_AUTOEXPOSURE_LOG*/

		using namespace UE::DisplayClusterRender::Features::AutoExposure;

		// Unregister it from replication on this cluster node
		IDisplayCluster::Get().GetClusterMgr()->UnregisterCustomState(AutoExposureStateName);

		AutoExposureState.Reset();
	}
}

bool FDisplayClusterAutoExposure::EnsureAutoExposureStateUsable()
{
	if (!Configuration->IsClusterRendering())
	{
		// Use TDistributedCustomState<> only for cluster mode
		return false;
	}

	// Create TDistributedCustomState<>
	if (!AutoExposureState.IsValid())
	{
		if (!bDisableAutoExposureState)
		{
			using namespace UE::DisplayClusterRender::Features::AutoExposure;

			// Create TDistributedCustomState<> object for AutoExposure.
			AutoExposureState = FAutoExposureState::Create(AutoExposureStateName);

#if ENABLE_DISPLAYCLUSTER_AUTOEXPOSURE_LOG
			UE_LOGF(LogDisplayClusterViewport, Log, "Create AutoExposureState.");
#endif /*ENABLE_DISPLAYCLUSTER_AUTOEXPOSURE_LOG*/
		}

		// nullptr-check
		if (!AutoExposureState.IsValid())
		{
			// Call ::Create() only once
			bDisableAutoExposureState = true;

			return false;
		}
	}

	return true;
}

bool FDisplayClusterAutoExposure::UpdatePerNodeAutoExposureDataForCluster(const FDisplayClusterNodeAutoExposureData& CurrentNodeAutoExposureData)
{
	using namespace UE::DisplayClusterRender::Features::AutoExposure;

	// Reset the the data from prev frame.
	PerNodeAutoExposureData.Reset();

	if (!EnsureAutoExposureStateUsable())
	{
		return false;
	}

	// Receive AutoExposure data from all cluster nodes
	const TSet<FName> ClusterNodesIds = AutoExposureState->GetAvailableNodes();
	for (const FName& NodeId : ClusterNodesIds)
	{
		PerNodeAutoExposureData.Emplace(NodeId, AutoExposureState->GetData(NodeId));
	}

	// Send current node AutoExposure data to cluster
	if (Settings->IsNodeAllowedForMetering(Configuration->GetClusterNodeId()))
	{
		AutoExposureState->SetData(CurrentNodeAutoExposureData);
	}
	else
	{
		// Publish empty data for excluded node
		AutoExposureState->SetData(FDisplayClusterNodeAutoExposureData());
	}

	return true;
}

void FDisplayClusterAutoExposure::UpdatePerNodeAutoExposureDataForPreview(const FDisplayClusterNodeAutoExposureData& CurrentNodeAutoExposureData)
{
	// Get cluster node from configuration (DCRA preview rendering)
	const FName ClusterNodeId(Configuration->GetClusterNodeId());

	if (Settings->IsNodeAllowedForMetering(Configuration->GetClusterNodeId()))
	{
		// Add CurrentNodeAutoExposureData to the PerNodeAutoExposureData.
		PerNodeAutoExposureData.Emplace(ClusterNodeId, CurrentNodeAutoExposureData);
	}
	else
	{
		PerNodeAutoExposureData.Remove(ClusterNodeId);
	}
}

void FDisplayClusterAutoExposure::OnEntireClusterPreviewRendered()
{
	if (Configuration->IsPreviewRendering())
	{
		ComputeAutoExposure();
	}
}

void FDisplayClusterAutoExposure::SyncAndComputeAutoExposure()
{
	check(IsInGameThread());

	// Get CurrentNodeAutoExposureData from GT queue
	FDisplayClusterNodeAutoExposureData CurrentNodeAutoExposureData;
	{
		FScopeLock Lock(&DataQueueCS);
		if (!DataQueue.IsEmpty())
		{
			CurrentNodeAutoExposureData = DataQueue[0];
			DataQueue.RemoveAt(0);
		}
	}

	// nDisplay AutoExposure disabled in settings
	if (!Settings->bEnable)
	{
		Release();

		return;
	}

	// Sync NodeAutoExposureData
	if (Configuration->IsPreviewRendering())
	{
		// Note: Use TDistributedCustomState<> only for cluster
		ReleaseAutoExposureState();

		// Logic for preview rendering
		UpdatePerNodeAutoExposureDataForPreview(CurrentNodeAutoExposureData);

		// For preview, the ComputeAutoExposure() function must be called after the last cluster node has been rendered.
		// see OnEntireClusterPreviewRendered().
	}
	else if (Configuration->IsClusterRendering())
	{
		// Use TDistributedCustomState<> to exchange AutoExposure data between cluster nodes.
		if (!UpdatePerNodeAutoExposureDataForCluster(CurrentNodeAutoExposureData))
		{
			Release();

			return;
		}

		// In a cluster, each node calculates data separately.
		ComputeAutoExposure();
	}
	else
	{
		// Other rendering modes do not support nDisplay AutoExposure.
		Release();

		return;
	}	
}

void FDisplayClusterAutoExposure::ComputeAutoExposure()
{
	using namespace UE::DisplayClusterRender::Features::AutoExposure;

	// Reset target brightness value
	TargetBrightness.Reset();

	// Compute the global exposure by averaging contributions from the
	//    selected viewports(or all, depending on settings).
	TArray<FDisplayClusterViewportAutoExposureData> MeteringViewports;
	for (const TPair<FName, FDisplayClusterNodeAutoExposureData>& NodeIt : PerNodeAutoExposureData)
	{
		MeteringViewports.Append(NodeIt.Value.Viewports);
	}

	if (!MeteringViewports.IsEmpty())
	{
		// Calculate the new brightness value.
		TargetBrightness = ComputeDestBrightness(MeteringViewports);

#if ENABLE_DISPLAYCLUSTER_AUTOEXPOSURE_LOG
		UE_LOGF(LogDisplayClusterViewport, Log, "[%d] AutoExposure %.5f, Data %ls.",
			GFrameNumber,
			TargetBrightness.IsSet() ? *TargetBrightness : 0.0f,
			*ToString(PerNodeAutoExposureData));
#endif /*ENABLE_DISPLAYCLUSTER_AUTOEXPOSURE_LOG*/
	}
}

void FDisplayClusterAutoExposure::GatherMeterViewportsAutoExposureDataAndSync_RenderThread(const TArray<TSharedPtr<FDisplayClusterViewportProxy, ESPMode::ThreadSafe>>& InViewportProxies)
{
	TArray<FDisplayClusterViewportAutoExposureData> MeterViewports;

	if (SettingsProxy->bEnable
		&& !SettingsProxy->IsNodeExcludedFromAutoExposure(Configuration->GetClusterNodeId()))
	{
		using namespace UE::DisplayClusterRender::Features::AutoExposure;

		// Collect meter viewports
		for (const TSharedPtr<FDisplayClusterViewportProxy, ESPMode::ThreadSafe>& ViewportProxy : InViewportProxies)
		{
			if (!ViewportProxy.IsValid())
			{
				continue;
			}

			// Check if this viewport supports AutoExposure
			if (!IsViewportAllowedForMetering_RenderThread(ViewportProxy))
			{
				continue;
			}

			// Determine if current settings allow this viewport to contribute to AE metering
			if (!SettingsProxy->IsViewportEligibleForMetering(ViewportProxy->GetBaseId(), ViewportProxy->GetViewportFlags()))
			{
				continue;
			}

			const FDisplayClusterViewport_Context& InContext = ViewportProxy->GetContexts_RenderThread()[0];
			const FIntPoint VisibleSize = InContext.OverscanInnerRenderTargetRect.Size();
			const int32 NumVisiblePixels = VisibleSize.X * VisibleSize.Y;
			if (NumVisiblePixels == 0)
			{
				continue;
			}

			// Collect AutoExposure data of this viewport.
			MeterViewports.Add(
				FDisplayClusterViewportAutoExposureData(
					InContext.RenderThreadData.EyeAdaptationData,
					NumVisiblePixels,
					InContext.RenderThreadData.FrameNumber));
		}
	}

	// RT->GT: Send data to game thread
	{
		FScopeLock Lock(&DataQueueCS);
		DataQueue.Add(MeterViewports);
	}
}

void FDisplayClusterAutoExposure::ApplyAutoExposureSettings(
	const FDisplayClusterConfiguration_AutoExposureSettings& InAutoExposureSettings)
{
	using namespace UE::DisplayClusterRender::Features::AutoExposure;

	// Update GT settings
	Settings = MakeShared<FDisplayClusterAutoExposureSettings>(InAutoExposureSettings);
	
	// Update RT settings
	ENQUEUE_RENDER_COMMAND(DisplayClusterUpdateAutoExposureSettingsProxy)([
		This = AsShared(), NewSettings = Settings](FRHICommandListImmediate& RHICmdList)
		{
			This->SettingsProxy = NewSettings;
		});
}

void FDisplayClusterAutoExposure::Release()
{
#if ENABLE_DISPLAYCLUSTER_AUTOEXPOSURE_LOG
	UE_LOGF(LogDisplayClusterViewport, Log, "FDisplayClusterAutoExposure: Release internal data.");
#endif /*ENABLE_DISPLAYCLUSTER_AUTOEXPOSURE_LOG*/

	PrevFrameBrightnessCache.Reset();
	TargetBrightness.Reset();

	PerNodeAutoExposureData.Reset();

	ReleaseAutoExposureState();
	bDisableAutoExposureState = false;
}