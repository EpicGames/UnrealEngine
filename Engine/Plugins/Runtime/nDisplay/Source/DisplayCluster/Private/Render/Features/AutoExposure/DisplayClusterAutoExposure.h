// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Cluster/CustomStates/DisplayClusterCustomStateDistributed.h"

#include "Render/Viewport/Containers/DisplayClusterViewport_EyeAdaptation.h"
#include "Render/Features/AutoExposure/DisplayClusterAutoExposureSettings.h"

#include "Templates/SharedPointer.h"

class FDisplayClusterViewport;
class FDisplayClusterViewportProxy;

/**
 * Auto-exposure data for a single nDisplay viewport.
 */
struct FDisplayClusterViewportAutoExposureData
	: public FDisplayClusterViewport_EyeAdaptationData
{
	FDisplayClusterViewportAutoExposureData() = default;
	FDisplayClusterViewportAutoExposureData(
		const FDisplayClusterViewport_EyeAdaptationData& In,
		const int32 InNumVisiblePixels,
		const uint32 InFrameNumber)
		: FDisplayClusterViewport_EyeAdaptationData(In)
		, NumVisiblePixels(InNumVisiblePixels)
		, FrameNumber(InFrameNumber)
	{ }

	// Number of visible pixels covered by this viewport.
	// Used as an input when computing the eye-adaptation weight for this viewport.
	int32 NumVisiblePixels = 0;

	// RT frame number of this data
	uint32 FrameNumber = 0;
};

/** Auto-exposure data for all viewports on a single nDisplay node. */
struct FDisplayClusterNodeAutoExposureData
{
	FDisplayClusterNodeAutoExposureData() = default;

	FDisplayClusterNodeAutoExposureData(const TArray<FDisplayClusterViewportAutoExposureData>& InViewports)
		: Viewports(InViewports)
	{ }

	// Per-viewport auto-exposure data for this node.
	TArray<FDisplayClusterViewportAutoExposureData> Viewports;
};

/** Custom serialization for FDisplayClusterViewportAutoExposureData */
inline FArchive& operator<<(FArchive& Ar, FDisplayClusterViewportAutoExposureData& In)
{
	Ar << In.LastAverageSceneLuminance;
	Ar << In.NumVisiblePixels;
	Ar << In.FrameNumber;

	return Ar;
}

/** Custom serialization for FDisplayClusterNodeAutoExposureData */
inline FArchive& operator<<(FArchive& Ar, FDisplayClusterNodeAutoExposureData& In)
{
	Ar << In.Viewports;

	return Ar;
}

/**
* Implements the nDisplay multi-viewport AutoExposure feature.
* Collects exposure data from all viewports across the cluster,
* synchronizes the result via the cluster network,
* and applies a unified exposure state to each node’s viewports.
*/
class FDisplayClusterAutoExposure
	: public TSharedFromThis<FDisplayClusterAutoExposure, ESPMode::ThreadSafe>
{
public:
	FDisplayClusterAutoExposure(const TSharedRef<FDisplayClusterViewportConfiguration, ESPMode::ThreadSafe>& InConfiguration)
		: Configuration(InConfiguration)
		, Settings(MakeShared<FDisplayClusterAutoExposureSettings, ESPMode::ThreadSafe>())
		, SettingsProxy(MakeShared<FDisplayClusterAutoExposureSettings, ESPMode::ThreadSafe>())
	{ }

	~FDisplayClusterAutoExposure() = default;

public:
	/**
	* Apply new auto-exposure settings to the current viewport configuration.
	*
	* @param InAutoExposureSettings  The auto-exposure settings to apply.
	*/
	void ApplyAutoExposureSettings(const FDisplayClusterConfiguration_AutoExposureSettings& InAutoExposureSettings);

	/**
	 * Configures AutoExposure settings for a given viewport scene view.
	 *
	 * @param InViewport   The nDisplay viewport to configure.
	 * @param InContextNum View context index within the viewport.
	 * @param InOutView    Scene view to apply AutoExposure settings to.
	 */
	void ConfigureAutoExposureForSceneView(
		const FDisplayClusterViewport& InViewport,
		uint32 InContextNum,
		FSceneView& InOutView);

	/**
	 * Collects Auto Exposure (AE) metering data from the given viewport proxies
	 * on this node and forwards it to SyncAndComputeAutoExposure() for cluster-wide
	 * synchronization and global exposure computation.
	 *
	 * Executed on the render thread once per frame.
	 *
	 * @param InViewportProxies  The viewport proxies on this node that provide
	 *                           AE metering input.
	 */
	void GatherMeterViewportsAutoExposureDataAndSync_RenderThread(
		const TArray<TSharedPtr<FDisplayClusterViewportProxy, ESPMode::ThreadSafe>>& InViewportProxies);

	/**
	 * Synchronizes Auto Exposure (AE) data across the nDisplay cluster and computes
	 * a unified global exposure value.
	 *
	 * Process:
	 *   - For cluster: call UpdatePerNodeAutoExposureDataForCluster()
	 *   - For preview: call UpdatePerNodeAutoExposureDataForPreview()
	 * 
	 *   - Compute the global exposure by averaging contributions from the
	 *      selected viewports (or all, depending on settings).
	 *
	 * The resulting exposure value is consistent across the cluster, preventing
	 * brightness mismatches between displays.
	 */
	void SyncAndComputeAutoExposure();

	/** Release inner data. */
	void Release();

	/** Called every time previews for all cluster nodes are rendered. */
	void OnEntireClusterPreviewRendered();

protected:
	/**
	 * Compute the global exposure by averaging contributions from the
	 * selected viewports (or all, depending on settings).
	 */
	void ComputeAutoExposure();

	/**
	 * Synchronizes Auto Exposure (AE) data across the nDisplay cluster and update ClusterData.
	 *
	 * Process:
	 *   1. Publish this node's AE data (NodeId + viewports) to the cluster.
	 *   2. Receive AE data from other nodes.
	 *
	 * @param CurrentNodeAutoExposureData  The viewports on this node that contribute AE metrics.
	 */
	bool UpdatePerNodeAutoExposureDataForCluster(const FDisplayClusterNodeAutoExposureData& CurrentNodeAutoExposureData);

	/**
	 * Update ClusterData for preview rendering.
	 * 
	 * @param CurrentNodeAutoExposureData  The viewports on this node that contribute AE metrics.
	 */
	void UpdatePerNodeAutoExposureDataForPreview(const FDisplayClusterNodeAutoExposureData& CurrentNodeAutoExposureData);

	/**
	 * Ensures the AutoExposureState exists and is usable.
	 * Creates the AutoExposureState instance if it does not already exist.
	 *
	 * @return true if AutoExposureState is available after this call, false otherwise.
	 */
	bool EnsureAutoExposureStateUsable();

	/** Release AutoExposureState instance. */
	void ReleaseAutoExposureState();

private:
	// Cached brightness value from the previous frame, keyed by (Viewport, ContextIndex).
	struct FPrevFrameBrightnessCacheEntry
	{
		// Weak reference to the viewport this entry belongs to (no ownership).
		TWeakPtr<const FDisplayClusterViewport, ESPMode::ThreadSafe> ViewportWeakPtr;

		// Viewport context index within the viewport (e.g. stereo eye / context).
		uint32 ContextIndex = 0;

		// Previous frame brightness for this viewport/context. Unset until a value is produced.
		TOptional<float> PrevFrameBrightness;
	};

	// Cache of previous-frame brightness values per viewport/context.
	// This variable is exclusively used only by ComputeViewportAutoExposureBrightness()
	TArray<FPrevFrameBrightnessCacheEntry> PrevFrameBrightnessCache;

	/**
	 * Computes the viewport brightness.
	 * Call only once per frame per viewport context.
	 *
	 * @param InViewport    nDisplay viewport to evaluate.
	 * @param InContextNum  Viewport context index (eye/view pass) within the nDisplay viewport.
	 * @param View          Scene view associated with the specified viewport context.
	 *
	 * @return Computed brightness value used by the auto-exposure update.
	 */
	float ComputeViewportAutoExposureBrightness(
		const FDisplayClusterViewport& InViewport,
		const uint32 InContextNum,
		FSceneView& View);

private:
	// Configuration of the current cluster node
	const TSharedRef<FDisplayClusterViewportConfiguration, ESPMode::ThreadSafe> Configuration;

	// Target Brightness for the current frame.
	TOptional<float> TargetBrightness;

	// Per-node AutoExposure data collected from all cluster viewports.
	TMap<FName, FDisplayClusterNodeAutoExposureData> PerNodeAutoExposureData;

	// Runtime settings for Game thread
	TSharedRef<FDisplayClusterAutoExposureSettings, ESPMode::ThreadSafe> Settings;

	// Runtime settings for Rendering thread
	TSharedRef<FDisplayClusterAutoExposureSettings, ESPMode::ThreadSafe> SettingsProxy;

private:
	// RT->GT AE data queue
	TArray<FDisplayClusterNodeAutoExposureData> DataQueue;

	FCriticalSection DataQueueCS;

private:
	// Declare distribute state type
	using FAutoExposureState = TDistributedCustomState<FDisplayClusterNodeAutoExposureData>;

	// API to distribute auto exposure data between nodes.
	// Note: Use TDistributedCustomState<> only for cluster
	TSharedPtr<FAutoExposureState> AutoExposureState;

	// This flag disables the use of AutoExposureState.
	bool bDisableAutoExposureState = false;
};
