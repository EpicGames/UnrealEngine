// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Features/AutoExposure/DisplayClusterAutoExposureSettings.h"
#include "Render/Viewport/Configuration/DisplayClusterViewportConfiguration.h"
#include "DisplayClusterConfigurationTypes_AutoExposure.h"

FDisplayClusterAutoExposureSettings::FDisplayClusterAutoExposureSettings(
	const FDisplayClusterConfiguration_AutoExposureSettings& InAutoExposureSettings)
{
	bEnable = InAutoExposureSettings.bEnabled;

	OuterViewportsForMetering = InAutoExposureSettings.MeteringSettings.Viewports;
	ICVFXCamerasForMetering = InAutoExposureSettings.MeteringSettings.ICVFXCameras;
	ClusterNodesForMetering = InAutoExposureSettings.MeteringSettings.Nodes;

	ExcludedOuterViewports = InAutoExposureSettings.ExcludeSettings.Viewports;
	ExcludedICVFXCameras = InAutoExposureSettings.ExcludeSettings.ICVFXCameras;
	ExcludedClusterNodes = InAutoExposureSettings.ExcludeSettings.Nodes;
}

bool FDisplayClusterAutoExposureSettings::IsNodeExcludedFromAutoExposure(const FString& NodeId) const
{
	if (ExcludedClusterNodes.Contains(NodeId))
	{
		return true;
	}

	return false;
}

bool FDisplayClusterAutoExposureSettings::IsNodeAllowedForMetering(const FString& NodeId) const
{
	if (IsNodeExcludedFromAutoExposure(NodeId))
	{
		return false;
	}

	if (ClusterNodesForMetering.IsEmpty() || ClusterNodesForMetering.Contains(NodeId))
	{
		return true;
	}

	return false;
}

bool FDisplayClusterAutoExposureSettings::IsViewportExcludedFromAutoExposure(
	const FString& InViewportBaseId,
	const EDisplayClusterViewportFlags InViewportFlags) const
{
	// Outer viewport rules
	if (!EnumHasAnyFlags(InViewportFlags, EDisplayClusterViewportFlags::ICVFX_AnyRole))
	{
		if (ExcludedOuterViewports.Contains(InViewportBaseId))
		{
			return true;
		}
	}
	// Inner Frustum rules
	else if (EnumHasAnyFlags(InViewportFlags, EDisplayClusterViewportFlags::ICVFX_InnerFrustum))
	{
		if (ExcludedICVFXCameras.Contains(InViewportBaseId))
		{
			return true;
		}
	}
	// Exclude all other ICVFX viewports (uv/LC/CK)
	else if (EnumHasAnyFlags(InViewportFlags, EDisplayClusterViewportFlags::ICVFX_AnyRole))
	{
		return true;
	}

	return false;
}

bool FDisplayClusterAutoExposureSettings::IsViewportEligibleForMetering(
	const FString& InViewportBaseId,
	const EDisplayClusterViewportFlags InViewportFlags) const
{
	if (IsViewportExcludedFromAutoExposure(InViewportBaseId, InViewportFlags))
	{
		return false;
	}

	// Outer viewport rules
	if (!EnumHasAnyFlags(InViewportFlags, EDisplayClusterViewportFlags::ICVFX_AnyRole))
	{
		if (OuterViewportsForMetering.IsEmpty() || OuterViewportsForMetering.Contains(InViewportBaseId))
		{
			return true;
		}
	}
	// Inner Frustum rules
	else if (EnumHasAnyFlags(InViewportFlags, EDisplayClusterViewportFlags::ICVFX_InnerFrustum))
	{
		if (ICVFXCamerasForMetering.IsEmpty() || ICVFXCamerasForMetering.Contains(InViewportBaseId))
		{
			return true;
		}
	}

	return false;
}
