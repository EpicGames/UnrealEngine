// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Render/Viewport/Containers/DisplayClusterViewport_Enums.h"
#include "Render/Viewport/Configuration/DisplayClusterViewportConfiguration.h"

struct FDisplayClusterConfiguration_AutoExposureSettings;

/**
 * Runtime AutoExposure settings.
 */
struct FDisplayClusterAutoExposureSettings
{
	FDisplayClusterAutoExposureSettings() = default;
	FDisplayClusterAutoExposureSettings(
		const FDisplayClusterConfiguration_AutoExposureSettings& InAutoExposureSettings);

	/**
	 * Checks whether the specified cluster node is allowed to contribute its
	 * viewports to auto-exposure metering, based on the given settings.
	 *
	 * @param NodeId     The identifier of the cluster node being tested.
	 * @return           True if viewports from this node can be used for metering.
	 */
	bool IsNodeAllowedForMetering(const FString& NodeId) const;

	/**
	 * Checks whether the specified cluster node is excluded from auto-exposure.
	 *
	 * A node marked as excluded prevents all of its viewports from participating
	 * in auto-exposure metering and from receiving auto-exposure customization.
	 *
	 * @param NodeId  Identifier of the cluster node to check.
	 * @return        True if the node is excluded from auto-exposure.
	 */
	bool IsNodeExcludedFromAutoExposure(const FString& NodeId) const;

	/**
	* Checks whether the specified viewport is eligible to contribute to auto-exposure metering.
	*
	* @param InViewportBaseId  Base viewport identifier (shared across derived/tiled viewports).
	* @param InViewportFlags   Viewport usage flags (e.g. Regular, InCamera, Chromakey, LightCard).
	* @return                  True if this viewport is allowed to participate in AE metering.
	*/
	bool IsViewportEligibleForMetering(const FString& InViewportBaseId, const EDisplayClusterViewportFlags InViewportFlags) const;

	/**
	* Checks whether the specified viewport is excluded from auto-exposure.
	*
	* A viewport marked as excluded will not contribute to auto-exposure
	* metering and will not receive auto-exposure customization.
	*
	* @param InViewportBaseId  Base viewport identifier (shared across derived/tiled viewports).
	* @param InViewportFlags   Viewport usage flags (e.g. Regular, InCamera, Chromakey, LightCard).
	* @return                  True if the viewport is excluded from auto-exposure.
	*/
	bool IsViewportExcludedFromAutoExposure(const FString& InViewportBaseId, const EDisplayClusterViewportFlags InViewportFlags) const;

public:
	// Enables or disables the AutoExposure feature at runtime.
	bool bEnable = false;

	// Outer viewport names used for auto - exposure metering.
	TArray<FString> OuterViewportsForMetering;

	// ICVFX camera Id's used for auto-exposure metering.
	TArray<FString> ICVFXCamerasForMetering;

	// Only viewports from these nodes can be used for metering.
	TArray<FString> ClusterNodesForMetering;

	// Outer viewpots excluded from auto-exposure.
	TArray<FString> ExcludedOuterViewports;

	// ICVFX cameras excluded from auto-exposure.
	TArray<FString> ExcludedICVFXCameras;

	// Cluster nodes excluded from auto-exposure.
	TArray<FString> ExcludedClusterNodes;
};