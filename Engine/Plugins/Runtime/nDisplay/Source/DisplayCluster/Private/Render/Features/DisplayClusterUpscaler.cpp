// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Features/DisplayClusterUpscaler.h"
#include "Render/Viewport/DisplayClusterViewport.h"
#include "DisplayClusterSceneViewExtensions.h"

#include "IUpscalerModularFeature.h"
#include "Features/IModularFeatures.h"
#include "SceneViewExtensionContext.h"

#include "LegacyScreenPercentageDriver.h"
#include "SceneView.h"
#include "HAL/IConsoleManager.h"
#include "RHI.h"
#include "RenderUtils.h"

using namespace UE::VirtualProduction;

namespace UE::DisplayClusterRender::Features::Upscaler
{
	int32 GDisplayClusterRenderUpscaling_HideDisabledMethods = 1;
	static FAutoConsoleVariableRef CVarDisplayClusterRenderUpscaling_HideDisabledMethods(
		TEXT("nDisplay.render.upscaling.HideDisabledMethods"),
		GDisplayClusterRenderUpscaling_HideDisabledMethods,
		TEXT("Hide disabled upscaling methods (default = 1).\n")
		TEXT("0 - Show all upsccaling methods.\n"),
		ECVF_Default
	);

	int32 GDisplayClusterRenderUpscaling_Enable = 1;
	static FAutoConsoleVariableRef CVarDisplayClusterRenderUpscaling_Enable(
		TEXT("nDisplay.render.upscaling.Enable"),
		GDisplayClusterRenderUpscaling_Enable,
		TEXT("Allows to use the upscaling settings (default = 1).\n")
		TEXT("0 - Ignore upscaling settings.\n"),
		ECVF_Default
	);

	int32 GDisplayClusterRenderUpscaling_EnableCustomUpscalers = 1;
	static FAutoConsoleVariableRef CVarDisplayClusterRenderUpscaling_EnableCustomUpscalers(
		TEXT("nDisplay.render.upscaling.EnableCustomUpscalers"),
		GDisplayClusterRenderUpscaling_EnableCustomUpscalers,
		TEXT("Allows the use of custom upscalers (default = 1).\n")
		TEXT("0 - Don't use custom upscalers and render by default.\n"),
		ECVF_Default
	);

	/** A custom upscaler may be used. */
	static inline bool UseCustomUpscalers()
	{
		return GDisplayClusterRenderUpscaling_EnableCustomUpscalers && GDisplayClusterRenderUpscaling_Enable;
	}

	/** Iterate over all upscaler modular features. */
	static void ForeachUpscaler(TFunction<void(IUpscalerModularFeature&)> IteratorFunc)
	{
		static IModularFeatures& ModularFeatures = IModularFeatures::Get();
		ModularFeatures.LockModularFeatureList();
		ON_SCOPE_EXIT
		{
			ModularFeatures.UnlockModularFeatureList();
		};

		// Iterate over enabled upscalers
		const TArray<IUpscalerModularFeature*> UpscalerModularFeatures = ModularFeatures.GetModularFeatureImplementations<IUpscalerModularFeature>(IUpscalerModularFeature::ModularFeatureName);
		for (IUpscalerModularFeature* UpscalerModularFeature : UpscalerModularFeatures)
		{
			if (UpscalerModularFeature == nullptr)
			{
				continue;
			}

			if (UpscalerModularFeature->IsFeatureEnabled())
			{
				IteratorFunc(*UpscalerModularFeature);
			}
		}
	}

	/** Get global fraction. */
	static float GetGlobalScreenPercentage()
	{
		static const TConsoleVariableData<float>* CVarScreenPercentage = IConsoleManager::Get().FindTConsoleVariableDataFloat(TEXT("r.ScreenPercentage"));
		if (CVarScreenPercentage)
		{
			const float GlobalScreenPercentage = CVarScreenPercentage->GetValueOnGameThread() / 100.0f;
			if (GlobalScreenPercentage > 0)
			{
				return GlobalScreenPercentage;
			}
		}

		return 1.0f;
	}

	/** Get secondary screen percentage fraction. */
	static float GetSecondaryViewFraction(const float InDPIScale)
	{
		static IConsoleVariable* CVarCustomSecondaryScreenPercentage = IConsoleManager::Get().FindConsoleVariable(TEXT("r.SecondaryScreenPercentage.GameViewport"), false);
		if (CVarCustomSecondaryScreenPercentage)
		{
			const float CustomSecondaryScreenPercentage = CVarCustomSecondaryScreenPercentage->GetFloat();
			if (CustomSecondaryScreenPercentage > 0.0)
			{
				// Override secondary resolution fraction with CVar.
				return FMath::Min(CustomSecondaryScreenPercentage / 100.0f, 1.0f);
			}
		}

#if WITH_EDITOR
			// Automatically compute secondary resolution fraction from DPI.
			// When in high res screenshot do not modify screen percentage based on dpi scale
			if (GIsEditor && !GIsHighResScreenshot && InDPIScale > 0.0)
			{
				static IConsoleVariable* CVarEditorViewportHighDPIPtr = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Editor.Viewport.HighDPI"));
				if (CVarEditorViewportHighDPIPtr && CVarEditorViewportHighDPIPtr->GetInt() == 0)
				{

					const float DPIDerivedResolutionFraction = FMath::Min(1.0f / InDPIScale, 1.0f);

					return DPIDerivedResolutionFraction;
				}
			}
#endif

		return 1.0f;
	}


	/** Check if temporal upscaler can be used. */
	static bool CanUseTemporalUpscaler()
	{
		static const IConsoleVariable* CVarTemporalAAUpscaler = IConsoleManager::Get().FindConsoleVariable(TEXT("r.TemporalAA.Upscaler"));
		return CVarTemporalAAUpscaler && (CVarTemporalAAUpscaler->GetInt() != 0);
	}

	/** Check if MSAA can be used. */
	static bool CanUseMSAA(const FStaticFeatureLevel InFeatureLevel)
	{
		static auto* MSAACountCVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.MSAACount"));
		if (MSAACountCVar->GetValueOnAnyThread() <= 0)
		{
			return false;
		}

		EShaderPlatform ShaderPlatform = GetFeatureLevelShaderPlatform(InFeatureLevel);
		if ((InFeatureLevel >= ERHIFeatureLevel::SM5 && !IsForwardShadingEnabled(ShaderPlatform)))
		{
			// MSAA requires forward shading renderer
			return false;
		}
	
		return true;
	}

	/** Temporal upscaler interface requires special AA method. */
	static bool SceneViewSupportsTemporalUpscaler(const FSceneView& InSceneView)
	{
		// check if current AA method legitable for temporal upscaler
		switch (InSceneView.AntiAliasingMethod)
		{
		case EAntiAliasingMethod::AAM_TSR:
		case EAntiAliasingMethod::AAM_TemporalAA: //@TODO check AAM_TemporalAA
			return true;

		default:
			break;
		}

		return false;
	}

	/** Return default AntiAliasing method. */
	static EDisplayClusterUpscalerAntiAliasingMethod GetDefaultAntiAliasingMethod(const FStaticFeatureLevel InFeatureLevel)
	{
		static auto* DefaultAntiAliasingCvar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.AntiAliasingMethod"));
		static EAntiAliasingMethod DefaultAntiAliasingMethod = EAntiAliasingMethod(FMath::Clamp<int32>(DefaultAntiAliasingCvar->GetValueOnAnyThread(), 0, AAM_MAX));

		switch (DefaultAntiAliasingMethod)
		{
		case EAntiAliasingMethod::AAM_FXAA:
			return EDisplayClusterUpscalerAntiAliasingMethod::FXAA;

		case EAntiAliasingMethod::AAM_MSAA:
			if (CanUseMSAA(InFeatureLevel))
			{
				return EDisplayClusterUpscalerAntiAliasingMethod::MSAA;
			}
			break;

		case EAntiAliasingMethod::AAM_TemporalAA:
			return EDisplayClusterUpscalerAntiAliasingMethod::TAA;

		case EAntiAliasingMethod::AAM_TSR:
			{
				EShaderPlatform ShaderPlatform = GetFeatureLevelShaderPlatform(InFeatureLevel);
				if (!SupportsTSR(ShaderPlatform))
				{
					// Fallback to UE4's TAA if TSR isn't supported on that platform
					return EDisplayClusterUpscalerAntiAliasingMethod::TAA;
				}
			}
			return EDisplayClusterUpscalerAntiAliasingMethod::TSR;

		default:
			break;
		}

		return EDisplayClusterUpscalerAntiAliasingMethod::None;
	}

	/** 
	* Get the antialiasing method that can be used for the ViewFamily.
	* Note: the logic of this function is based on the FSceneView::SetupAntiAliasingMethod() function.
	* 
	* @param InViewport         - (in) the DC viewport
	* @param InUpscalerSettings - (in) the upscaler settings that we want to use.
	* @param InViewFamily       - (in) ViewFamily thyat own SceneView.
	* @param InSceneView        - (in) SceneView to which the upscaler settings should be applied.
	* 
	* @return AntiAliasing method that can be used.
	*/
	static EDisplayClusterUpscalerAntiAliasingMethod GetSuitableAntiAliasingMethod(
		const FDisplayClusterViewport& InViewport,
		const FDisplayClusterUpscalerSettings& InUpscalerSettings,
		const FSceneViewFamily& InViewFamily,
		const FSceneView& InSceneView)
	{
		const bool bWillApplyTemporalAA = InViewFamily.EngineShowFlags.PostProcessing || InSceneView.bIsPlanarReflection;

		if (!bWillApplyTemporalAA || !InViewFamily.EngineShowFlags.AntiAliasing)
		{
			return EDisplayClusterUpscalerAntiAliasingMethod::None;
		}

		// Get AntiAliasing method from the upscaler settings
		EDisplayClusterUpscalerAntiAliasingMethod AntiAliasingMethod = InUpscalerSettings.AntiAliasingMethod;
		
		// Use the default method if the upscaler settings is disabled
		if (!GDisplayClusterRenderUpscaling_Enable)
		{
			AntiAliasingMethod = EDisplayClusterUpscalerAntiAliasingMethod::Default;
		}

		// Back to default method if TemporalUpscaling not supported
		if (AntiAliasingMethod == EDisplayClusterUpscalerAntiAliasingMethod::TAA
			|| AntiAliasingMethod == EDisplayClusterUpscalerAntiAliasingMethod::TSR)
		{
			if (!CanUseTemporalUpscaler())
			{
				AntiAliasingMethod = EDisplayClusterUpscalerAntiAliasingMethod::Default;
			}
		}

		// Back to default method is MSAA not supported
		if (AntiAliasingMethod == EDisplayClusterUpscalerAntiAliasingMethod::MSAA && !CanUseMSAA(InSceneView.GetFeatureLevel()))
		{
			AntiAliasingMethod = EDisplayClusterUpscalerAntiAliasingMethod::Default;
		}

		// Get default method from the project settings
		if (AntiAliasingMethod == EDisplayClusterUpscalerAntiAliasingMethod::Default)
		{
			AntiAliasingMethod = GetDefaultAntiAliasingMethod(InSceneView.GetFeatureLevel());
		}

		if (AntiAliasingMethod == EDisplayClusterUpscalerAntiAliasingMethod::TAA)
		{
			// Downgrade TAA->FXAA
			if (!InViewFamily.EngineShowFlags.TemporalAA || !InViewFamily.bRealtimeUpdate || !SupportsGen4TAA(InSceneView.GetShaderPlatform()))
			{
				return EDisplayClusterUpscalerAntiAliasingMethod::FXAA;
			}
		}
		else if (AntiAliasingMethod == EDisplayClusterUpscalerAntiAliasingMethod::TSR)
		{
			// Downgrade TSR->FXAA
			if (!InViewFamily.EngineShowFlags.TemporalAA || !InViewFamily.bRealtimeUpdate || !SupportsTSR(InSceneView.GetShaderPlatform()))
			{
				return EDisplayClusterUpscalerAntiAliasingMethod::FXAA;
			}
		}

		if(AntiAliasingMethod == EDisplayClusterUpscalerAntiAliasingMethod::TAA
		|| AntiAliasingMethod == EDisplayClusterUpscalerAntiAliasingMethod::TSR)
		{
			// TemporalAA/TSR requires the view to have a valid state.
			if (InSceneView.State == nullptr)
			{	
				return EDisplayClusterUpscalerAntiAliasingMethod::FXAA;
			}

			// Disable TSR/TAA for preview
			if (!InViewport.Configuration->GetPreviewSettings().bPreviewEnableTSR)
			{
				if (InViewport.Configuration->IsPreviewRendering())
				{
					return EDisplayClusterUpscalerAntiAliasingMethod::FXAA;
				}
			}
		}

		return AntiAliasingMethod;
	}
};

void FDisplayClusterUpscaler::SetupSceneView(
	const FDisplayClusterViewport& InViewport,
	const FDisplayClusterUpscalerSettings& InUpscalerSettings,
	const FSceneViewFamily& InViewFamily,
	FSceneView& InOutView)
{
	using namespace UE::DisplayClusterRender::Features::Upscaler;

	// Get AntiAliasingMethod that can be used.
	const EDisplayClusterUpscalerAntiAliasingMethod SuitableAntiAliasingMethod = GetSuitableAntiAliasingMethod(InViewport, InUpscalerSettings, InViewFamily, InOutView);
	switch (SuitableAntiAliasingMethod)
	{
	case EDisplayClusterUpscalerAntiAliasingMethod::None:
		InOutView.AntiAliasingMethod = EAntiAliasingMethod::AAM_None;
		InOutView.PrimaryScreenPercentageMethod = EPrimaryScreenPercentageMethod::SpatialUpscale;
		break;

	case EDisplayClusterUpscalerAntiAliasingMethod::FXAA:
		InOutView.AntiAliasingMethod = EAntiAliasingMethod::AAM_FXAA;
		InOutView.PrimaryScreenPercentageMethod = EPrimaryScreenPercentageMethod::SpatialUpscale;
		break;

	case EDisplayClusterUpscalerAntiAliasingMethod::MSAA:
		InOutView.AntiAliasingMethod = EAntiAliasingMethod::AAM_MSAA;
		InOutView.PrimaryScreenPercentageMethod = EPrimaryScreenPercentageMethod::SpatialUpscale;
		break;

	case EDisplayClusterUpscalerAntiAliasingMethod::TAA:
		InOutView.AntiAliasingMethod = EAntiAliasingMethod::AAM_TemporalAA;
		InOutView.PrimaryScreenPercentageMethod = EPrimaryScreenPercentageMethod::TemporalUpscale;
		break;

	case EDisplayClusterUpscalerAntiAliasingMethod::TSR:
		InOutView.AntiAliasingMethod = EAntiAliasingMethod::AAM_TSR;
		InOutView.PrimaryScreenPercentageMethod = EPrimaryScreenPercentageMethod::TemporalUpscale;
		break;

	default:
		break;
	}

	// Custom upscaler should configure view for the settings
	if (!InUpscalerSettings.CustomUpscalerName.IsNone() && UseCustomUpscalers())
	{
		ForeachUpscaler(
			[&](IUpscalerModularFeature& Upscaler)
			{
				if (Upscaler.GetName() == InUpscalerSettings.CustomUpscalerName)
				{
					Upscaler.SetupSceneView(InUpscalerSettings.CustomUpscalerSettings, InOutView);
				}
			});
	}
}

FName FDisplayClusterUpscaler::PostConfigureViewFamily(
	const FDisplayClusterUpscalerSettings& InUpscalerSettings,
	const float InScreenPercentage,
	const float InDPIScale,
	FSceneViewFamilyContext& InOutViewFamily,
	const TArray<FSceneView*>& InOutViews)
{
	using namespace UE::DisplayClusterRender::Features::Upscaler;

	// One view per viewfamily
	check(InOutViewFamily.Views.Num() == 1);
	check(InOutViewFamily.Views[0]);

#if WITH_EDITOR
	if (GIsEditor)
	{
		// Force enable view family show flag for HighDPI derived's screen percentage.
		InOutViewFamily.EngineShowFlags.ScreenPercentage = true;
	}
#endif

	// Force screen percentage show flag to be turned off if not supported.
	if (!InOutViewFamily.SupportsScreenPercentage())
	{
		InOutViewFamily.EngineShowFlags.ScreenPercentage = false;
	}

	if (!InOutViewFamily.EngineShowFlags.ScreenPercentage)
	{
		return NAME_None;
	}

	// If a screen percentage interface was set by dynamic resolution, then do nothing
	if (InOutViewFamily.GetScreenPercentageInterface() != nullptr)
	{
		return NAME_None;
	}

	FUpscalerModularFeatureParameters UpscalerParam;
	UpscalerParam.UpscalerScreenPercentage = InScreenPercentage;
	UpscalerParam.SecondaryScreenPercentage = GetSecondaryViewFraction(InDPIScale);

	check(UpscalerParam.UpscalerScreenPercentage > 0.0f);
	check(UpscalerParam.SecondaryScreenPercentage > 0.0f);

	// Use custom upscaler
	if (!InUpscalerSettings.CustomUpscalerName.IsNone() && UseCustomUpscalers())
	{
		// Find  upscaler modular feature
		FName UpscalerName = NAME_None;
		ForeachUpscaler([&](IUpscalerModularFeature& Upscaler)
			{
				if (Upscaler.GetName() == InUpscalerSettings.CustomUpscalerName)
				{
					if (Upscaler.PostConfigureViewFamily(
						InUpscalerSettings.CustomUpscalerSettings, UpscalerParam,
						InOutViewFamily))
					{
						// Return upscaler name on success
						UpscalerName = Upscaler.GetName();
					}
				}
			});

		if (UpscalerName != NAME_None)
		{
			// Upscaler modular feature expected to set screen percentage interface
			check(InOutViewFamily.GetScreenPercentageInterface())

			return UpscalerName;
		}
	}

	// Validating that InOutViews is non-empty.
	if (InOutViews.IsEmpty() || InOutViews[0] == nullptr)
	{
		return NAME_None;
	}

	// Get the upscaler settings from the first SceneView.
	const EPrimaryScreenPercentageMethod PrimaryScreenPercentageMethod = InOutViews[0]->PrimaryScreenPercentageMethod;
	const EAntiAliasingMethod AntiAliasingMethod = InOutViews[0]->AntiAliasingMethod;

	// Get global view fraction set by r.ScreenPercentage.
	const float GlobalScreenPercentage = GetGlobalScreenPercentage();

	// Get TSR or TAA resolution fraction range
	FVector2f TemporalUpscaleResolutionFractionRange = FVector2f::ZeroVector;
	if (PrimaryScreenPercentageMethod == EPrimaryScreenPercentageMethod::TemporalUpscale)
	{
		if (AntiAliasingMethod == EAntiAliasingMethod::AAM_TemporalAA)
		{
			// Sets the minimal and maximal screen percentage for TAAU. [0.5 .. 2.0]
			TemporalUpscaleResolutionFractionRange = FVector2f(
				ISceneViewFamilyScreenPercentage::kMinTAAUpsampleResolutionFraction,
				ISceneViewFamilyScreenPercentage::kMaxTAAUpsampleResolutionFraction);
		}
		else if (AntiAliasingMethod == EAntiAliasingMethod::AAM_TSR)
		{
			// Sets the minimal and maximal screen percentage for TSR.  [0.25 .. 2.0]
			TemporalUpscaleResolutionFractionRange = FVector2f(
				ISceneViewFamilyScreenPercentage::kMinTSRResolutionFraction,
				ISceneViewFamilyScreenPercentage::kMaxTSRResolutionFraction);
		}
	}

	// We need to split the screen percentage value if it is outside the TSR/TAA range.
	// All TemporalUpscaleResolutionFractionRange values are greater than zero.
	if(TemporalUpscaleResolutionFractionRange.GetMin() > 0)
	{
		// Adjust for global screen% in math bellow
		const float FinalResolutionFraction = UpscalerParam.UpscalerScreenPercentage * GlobalScreenPercentage;
		if (FinalResolutionFraction < TemporalUpscaleResolutionFractionRange.X)
		{
			// Move the remaining ScreenPercentage to the spatial upscaler
			const float RemainingScreenPercentage = UpscalerParam.UpscalerScreenPercentage / TemporalUpscaleResolutionFractionRange.X;
			UpscalerParam.SecondaryScreenPercentage *= RemainingScreenPercentage;

			// Adjust min value (0.5f) for the global screen%. Expectations that the actual TSR_Fraction=0.5 
			UpscalerParam.UpscalerScreenPercentage = TemporalUpscaleResolutionFractionRange.X / GlobalScreenPercentage;
		}
		else if (FinalResolutionFraction > TemporalUpscaleResolutionFractionRange.Y)
		{
			// Move the remaining ScreenPercentage to the spatial upscaler
			const float RemainingScreenPercentage = UpscalerParam.UpscalerScreenPercentage / TemporalUpscaleResolutionFractionRange.Y;
			UpscalerParam.SecondaryScreenPercentage *= RemainingScreenPercentage;

			// Adjust min value (0.5f) for the global screen%. Expectations that the actual TSR_Fraction=0.5 
			UpscalerParam.UpscalerScreenPercentage = TemporalUpscaleResolutionFractionRange.Y / GlobalScreenPercentage;
		}
	}

	const float FinalResolutionFraction = UpscalerParam.UpscalerScreenPercentage * GlobalScreenPercentage;

	// Clamp final resolution fraction
	// Sets the minimal and max screen percentage to [0.01 ... 4.0]
	float ClampedFinalResolutionFraction = FMath::Clamp(
		FinalResolutionFraction,
		ISceneViewFamilyScreenPercentage::kMinResolutionFraction,
		ISceneViewFamilyScreenPercentage::kMaxResolutionFraction);

	UpscalerParam.UpscalerScreenPercentage = ClampedFinalResolutionFraction / GlobalScreenPercentage;

	// Setup screen percentage
	InOutViewFamily.SetScreenPercentageInterface(new FLegacyScreenPercentageDriver(
		InOutViewFamily, UpscalerParam.UpscalerScreenPercentage));

	InOutViewFamily.SecondaryViewFraction = UpscalerParam.SecondaryScreenPercentage;

	return NAME_None;
}

void FDisplayClusterUpscaler::InitializeNewFrame()
{
	using namespace UE::DisplayClusterRender::Features::Upscaler;

	// Store the upscaler name with function GUID
	static TMap<FName, FGuid> SceneViewExtensionIsActiveFunctorsMap;

	// If the modular function will be disabled at runtime, we should forget about it
	// And then restore again when it is enabled.
	TMap<FName, FGuid> NewSceneViewExtensionIsActiveFunctorsMap;

	ForeachUpscaler(
		[&](IUpscalerModularFeature& Upscaler)
		{
			// Register IsActive Functors
			const FName& UpscalerName = Upscaler.GetName();
			if (SceneViewExtensionIsActiveFunctorsMap.Contains(UpscalerName))
			{
				// Use exist Guid
				NewSceneViewExtensionIsActiveFunctorsMap.Emplace(UpscalerName, SceneViewExtensionIsActiveFunctorsMap[UpscalerName]);
			}
			else
			{
				FSceneViewExtensionIsActiveFunctor IsActiveFunctor;
				IsActiveFunctor.IsActiveFunction = [UpscalerName](const ISceneViewExtension* SceneViewExtension, const FSceneViewExtensionContext& Context)
					{
						// Note: SceneViewExtension this pointer is always passed to functors from FSceneViewExtensionBase::IsActiveThisFrame, we confirm this remains so.
						check(SceneViewExtension);

						// Is nDisplay context
						static const FDisplayClusterSceneViewExtensionContext DCViewExtensionContext;
						if (Context.IsA(MoveTempIfPossible(DCViewExtensionContext)))
						{
							// Is viewport exist
							const FDisplayClusterSceneViewExtensionContext& DisplayContext = static_cast<const FDisplayClusterSceneViewExtensionContext&>(Context);
							if (DisplayContext.DisplayClusterViewport.IsValid())
							{
								const FDisplayClusterUpscalerSettings& UpscalerSettings = DisplayContext.DisplayClusterViewport->GetRenderSettings().UpscalerSettings;
								if (!UpscalerSettings.bIsActive)
								{
									// Custom upscalers are disabled for this viewport
									return TOptional<bool>(false);
								}

								if (UpscalerSettings.CustomUpscalerName != UpscalerName)
								{
									// This viewport uses a different upscaler.
									return TOptional<bool>(false);
								}

								return TOptional<bool>(true);
							}
						}

						return TOptional<bool>();
					};

				if (Upscaler.AddSceneViewExtensionIsActiveFunctor(IsActiveFunctor))
				{
					NewSceneViewExtensionIsActiveFunctorsMap.Emplace(UpscalerName, IsActiveFunctor.GetGuid());
				}
			}
		});

	// Update
	SceneViewExtensionIsActiveFunctorsMap = NewSceneViewExtensionIsActiveFunctorsMap;
}
