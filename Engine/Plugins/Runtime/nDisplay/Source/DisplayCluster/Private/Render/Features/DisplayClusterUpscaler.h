// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Render/Upscaler/DisplayClusterUpscalerSettings.h"

class FDisplayClusterViewport;
class FSceneViewFamilyContext;
class FSceneViewFamily;
class FSceneView;

/**
 * nDisplay upscaling implementation.
 */
class FDisplayClusterUpscaler
{
public:
	/**
	 * Prepares all registered upscaler modular features for the new frame.
	 * Should be called at the start of each frame.
	 */
	static void InitializeNewFrame();

	/** Configure upscaler and viewfamily for ertain settings.
	* 
	* @param InUpscalerSettings - (in) the settings to apply for viewport
	* @param InScreenPercentage - (inj screen percentage of the viewport
	* @param InDPIScale         - (in) The DPI scale of the window
	* @param InOutViewFamily    - (in, out) The view family that will be configured.
	* @param InOutViews         - (in, out) The views in the family that will be configured.
	*/
	static FName PostConfigureViewFamily(
		const FDisplayClusterUpscalerSettings& InUpscalerSettings,
		const float InScreenPercentage,
		const float InDPIScale,
		FSceneViewFamilyContext& InOutViewFamily,
		const TArray<FSceneView*>& InOutViews);

	/**
	* Configure scene view for upscalers.
	* @param InViewport         - (in) the viewport who ask for this setup
	* @param InUpscalerSettings - (in) the settings to apply for view
	* @param InViewFamily       - (in) ViewFamily that own this view.
	* @param InOutView          - (in, out) The view that will be configured. 
	*/
	static void SetupSceneView(
		const FDisplayClusterViewport& InViewport,
		const FDisplayClusterUpscalerSettings& InUpscalerSettings,
		const FSceneViewFamily& InViewFamily,
		FSceneView& InOutView);
};
