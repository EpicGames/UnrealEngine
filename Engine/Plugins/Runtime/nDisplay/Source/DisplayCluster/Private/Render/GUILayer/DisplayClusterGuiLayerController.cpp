// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/GUILayer/DisplayClusterGuiLayerController.h"

#include "IDisplayClusterShaders.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"

#include "Async/Async.h"
#include "Framework/Application/SlateApplication.h"
#include "Engine/Console.h"
#include "Engine/Engine.h"
#include "Engine/GameViewportClient.h"
#include "Misc/DisplayClusterGlobals.h"
#include "Misc/DisplayClusterLog.h"
#include "Render/Viewport/IDisplayClusterViewportProxy.h"
#include "Render/Projection/IDisplayClusterProjectionPolicy.h"
#include "ShaderParameters/DisplayClusterShaderParameters_Overlay.h"
#include "ShaderParameters/DisplayClusterShaderParameters_TransformTexture.h"
#include "Slate/SceneViewport.h"

#define UE_NDISPLAY_USE_CUSTOM_SIZE_FOR_GUI 0


namespace UE::nDisplay::Core::GUILayer::Private
{
	/**
	 * GUI propagation mode
	 *
	 * 0: Disabled
	 * UI layer controller is completely disabled. Slate rendering is performed in its
	 * native way on top of the scene view. When disabled, UI texture cannot be observed
	 * by cluster monitors.
	 *
	 * 1: Hidden
	 * UI is rendered into an intermediate texture, but never rendered to the backbuffer.
	 * This option may be used when UI is observed via nDisplay monitor.
	 * 
	 * 2: Whole
	 * UI is renderd in the buffer space. This is very similar to how UE originally does
	 * that (option 0), but the UI is kept in an internal texture. This also makes it
	 * possible to observe the UI via nDisplay monitor.
	 * 
	 * 3: Propagated
	 * UI is drawn into an intermediate texture, then blended on top of every nDisplay viewport.
	 * In this mode, the following CVarGuiLayerTextureSize cvar is used to specify the size
	 * of an intermediate GUI texture.
	 */
	static TAutoConsoleVariable<int32> CVarPropagateGui(
		TEXT("nDisplay.GUI.Propagate"),
		2,
		TEXT("Show GUI on viewports\n")
		TEXT("0 : Disabled   - Feature is completely disabled\n")
		TEXT("1 : Hidden     - Never rendered to the output\n")
		TEXT("2 : Whole      - GUI is rendered in backbuffer space\n")
		TEXT("3 : Propagated - GUI is rendered to nDisplay viewports\n")
		TEXT("Other values are treated as '0'.\n"),
		ECVF_Default
	);

	namespace
	{
		/**
		 * Extracts texture size from string
		 * If something is wrong, returns {0, 0}
		 */
		FIntPoint GetGuilLayerTextureSizeFromCvar(const FString& InResolution)
		{
			FString StrWidth;
			FString StrHeight;

			// Split "AxB" into "A" and "B"
			const bool bWasSplit = InResolution.Split(TEXT("x"), &StrWidth, &StrHeight, ESearchCase::IgnoreCase);
			if (!bWasSplit)
			{
				return FIntPoint::ZeroValue;
			}

			// Get integers
			FIntPoint Output = FIntPoint::ZeroValue;
			Output.X = FCString::Atoi(*StrWidth);
			Output.Y = FCString::Atoi(*StrHeight);

			// Forbid unallowed values
			if (!(Output.X >= 1 && Output.Y >= 1))
			{
				return FIntPoint::ZeroValue;
			}

			return Output;
		}

		/** Default texture resolution cvar text */
		constexpr const TCHAR* DefaultResolutionTextValue = TEXT("2560x1440");
		/** Current texture resolution text */
		static FString CurrentResolutionTextValue = DefaultResolutionTextValue;
		/** Current texture resolution */
		static FIntPoint CurrentResolution = GetGuilLayerTextureSizeFromCvar(CurrentResolutionTextValue);
	}

	/**
	 * GUI texture size with multiple viewports
	 *
	 * For a single viewport, the intermediate GUI texture matches the viewport size.
	 * For multiple viewports, it's defined by this console variable.
	 */
	static TAutoConsoleVariable<FString> CVarGuiLayerTextureSize(
		TEXT("nDisplay.GUI.TextureResolution"),
		DefaultResolutionTextValue,
		TEXT("Intermediate UI texture resolution in format WidthxHeight (e.g. 2560x1440).")
		TEXT("This resolution is used when nDisplay renders multiple viewports."),
		FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InCvar)
			{
				if (InCvar)
				{
					// Get new string for the cvar
					const FString NewResolutionTextValue = InCvar->GetString();
					// Extract resolution
					const FIntPoint NewResolution = GetGuilLayerTextureSizeFromCvar(NewResolutionTextValue);

					// If everything is Ok, then update current state
					if (NewResolution != FIntPoint::ZeroValue)
					{
						CurrentResolution = NewResolution;
						CurrentResolutionTextValue = NewResolutionTextValue;
						return;
					}
					// Otherwise, reset to previous text
					else
					{
						InCvar->Set(*CurrentResolutionTextValue, ECVF_SetByConsole);
					}
				}
			}),
		ECVF_Default
	);
}


FDisplayClusterGuiLayerController::FDisplayClusterGuiLayerController()
	: bEnabled(GDisplayCluster ? GDisplayCluster->GetOperationMode() == EDisplayClusterOperationMode::Cluster : false)
{
	if (bEnabled)
	{
		// Initial output mode state
		OutputMode = ExtractOutputModeFromCvar();

		// GameThread only intialization
		AsyncTask(ENamedThreads::GameThread, [this]()
			{
				FSlateApplication::Get().OnPreTick().AddRaw(this, &FDisplayClusterGuiLayerController::HandleSlatePreTick);
			});
	}
}

FDisplayClusterGuiLayerController& FDisplayClusterGuiLayerController::Get()
{
	static FDisplayClusterGuiLayerController Instance;
	return Instance;
}

IDisplayClusterGUILayerController::EGuiOutputMode FDisplayClusterGuiLayerController::GetOutputModeFromInt(int32 InOutputMode)
{
	switch (InOutputMode)
	{
	case 0:  return EGuiOutputMode::Disabled;
	case 1:  return EGuiOutputMode::Hidden;
	case 2:  return EGuiOutputMode::Whole;
	case 3:  return EGuiOutputMode::Propagated;

	default: return EGuiOutputMode::Disabled;
	}
}

IDisplayClusterGUILayerController::EGuiOutputMode FDisplayClusterGuiLayerController::FDisplayClusterGuiLayerController::GetOutputMode() const
{
	return IsInRenderingThread() ? OutputModeRT : OutputMode;
}

bool FDisplayClusterGuiLayerController::IsEnabled() const
{
	return bEnabled && GetOutputMode() != EGuiOutputMode::Disabled;
}

FIntPoint FDisplayClusterGuiLayerController::GetGuiLayerTextureSize() const
{
	// Is completely disabled
	if (!IsEnabled())
	{
		return FIntPoint::ZeroValue;
	}

	// Game thread data
	if (IsInGameThread())
	{
		return TextureResolution;
	}
	// Render thread data
	else if (IsInRenderingThread())
	{
		return TextureResolutionRT;
	}
	// AnyThread is not expected
	else
	{
		checkf(false, TEXT("Not supposed to be called from AnyThread"));
	}

	return FIntPoint::ZeroValue;
}

FRDGTextureRef FDisplayClusterGuiLayerController::ProcessFinalTexture_RenderThread(FRDGBuilder& GraphBuilder, FRDGTextureRef FinalTexture)
{
	checkSlow(IsInRenderingThread());
	checkSlow(FinalTexture);

	// Nothing to do if not active
	if (!IsEnabled())
	{
		return FinalTexture;
	}

	// Make sure our buffer duplicate is valid
	if (!TextureViewport.IsValid())
	{
		return FinalTexture;
	}

	// Create an RDG texture reference to the original game viewport's texture that we stored before Slate rendering
	FRDGTextureRef Output = RegisterExternalTexture(GraphBuilder, TextureViewport.GetReference(), TEXT("nD.TextureViewport"));
	if (!Output)
	{
		return FinalTexture;
	}

	// Conditionally draw the GUI on top of the scene view
	if (OutputModeRT == EGuiOutputMode::Whole)
	{
		// Prepare textures for drawing
		FRDGTextureRef GUITexture  = FinalTexture;
		FRDGTextureRef TempTexture = CreateTextureFrom_RenderThread(GraphBuilder, Output, TEXT("nD.TextureTemp"), Output->Desc.Extent);

		// Make a copy to the input texture
		AddCopyTexturePass(GraphBuilder, Output, TempTexture, FRHICopyTextureInfo{});

		// Draw the GUI layer on top of the nD viewport
		IDisplayClusterShaders::Get().AddDrawOverlayPass(GraphBuilder,
			FDisplayClusterShaderParameters_Overlay
			{
				.BaseTexture    = TempTexture, // Scene color (temp copy)
				.OverlayTexture = GUITexture,  // GUI layer
				.OutputTexture  = Output,      // Scene color (original viewport)
			});
	}

	// If everything Ok, return the original viewport's buffer
	return Output;
}

void FDisplayClusterGuiLayerController::ProcessPostRenderViewFamily_RenderThread(FRDGBuilder& GraphBuilder, const FSceneViewFamily& ViewFamiliy, const IDisplayClusterViewportProxy* ViewportProxy)
{
	checkSlow(IsInRenderingThread());
	checkSlow(ViewportProxy);

	// Nothing to do if not active
	if (!IsEnabled())
	{
		return;
	}

	// Nothing to do if GUI layer propagation is not active
	if (OutputModeRT != EGuiOutputMode::Propagated)
	{
		return;
	}

	// Validate both input and internal data
	if (!ViewportProxy || !TextureGUI.IsValid())
	{
		return;
	}

	TArray<FRHITexture*> Textures;
	TArray<FIntRect>     Regions;

	// Get nD viewport's buffer
	if (ViewportProxy->GetResourcesWithRects_RenderThread(EDisplayClusterViewportResourceType::InternalRenderTargetEntireRectResource, Textures, Regions))
	{
		if (Textures.Num() > 0 && Regions.Num() > 0 && Textures[0])
		{
			FRDGTextureRef TexOverlayRotated = nullptr;

			// Is frustum currently rotated?
			bool bFrustumRotated = false;
			if (const TSharedPtr<IDisplayClusterProjectionPolicy> ProjectionPolicy = ViewportProxy->GetProjectionPolicy_RenderThread())
			{
				bFrustumRotated = ProjectionPolicy->IsFrustumRotatedToFitContextSize_RenderThread(ViewportProxy, 0);
			}

			// If this viewport uses a rotated frustum, the GUI texture must be
			// rotated accordingly. This is performed once per frame so the rotated
			// GUI texture can be reused by other rotated viewports.
			if (bFrustumRotated)
			{
				// If rotated GUI texture is not ready yet, then draw it
				if (!TextureRotatedGUI.IsValid())
				{
					// No output texture specified, it will be created automatically
					FDisplayClusterShaderParameters_TransformTexture RequestParams;
					RequestParams.InputTexture      = RegisterExternalTexture(GraphBuilder, TextureGUI, TEXT("nD.TextureGUI"));
					RequestParams.TranformationType = FDisplayClusterShaderParameters_TransformTexture::ETranformation::Rotation_90;

					// Rotate GUI
					IDisplayClusterShaders::Get().AddTransformTexturePass(GraphBuilder, RequestParams);

					// If everything succeeded, store this texture for future reuse
					if (RequestParams.OutputTexture)
					{
						TexOverlayRotated = RequestParams.OutputTexture;
					}
				}
				else
				{
					TexOverlayRotated = GraphBuilder.RegisterExternalTexture(TextureRotatedGUI);
				}
			}

			const FIntPoint DCViewportSize = Regions[0].Size();

			// Prepare other textures for drawing
			FRDGTextureRef TexBase    = CreateTextureFrom_RenderThread(GraphBuilder, Textures[0], TEXT("nD.TexBase"));
			FRDGTextureRef TexOutput  = RegisterExternalTexture(GraphBuilder, Textures[0], *Textures[0]->GetName().ToString());
			FRDGTextureRef TexOverlay = (TexOverlayRotated)
				? TexOverlayRotated // rotated GUI texture
				: RegisterExternalTexture(GraphBuilder, TextureGUI, TEXT("nD.TextureGUI")); // original GUI texture, no rotation

			// Make a copy so the original texture will be a render target
			AddCopyTexturePass(GraphBuilder, TexOutput, TexBase, FRHICopyTextureInfo{});

			FDisplayClusterShaderParameters_Overlay Parameters;
			Parameters.OverlayTexture = TexOverlay; // GUI layer
			Parameters.BaseTexture    = TexBase;    // Scene color (copy of nD viewport)
			Parameters.OutputTexture  = TexOutput;  // Scene color (original nD viewport)

			// Draw the GUI layer on top of the nD viewport
			IDisplayClusterShaders::Get().AddDrawOverlayPass(GraphBuilder, Parameters);

			// If shared texture was used, detach it
			if (TexOverlayRotated)
			{
				GraphBuilder.QueueTextureExtraction(TexOverlayRotated, &TextureRotatedGUI);
			}
		}
	}
}

IDisplayClusterGUILayerController::EGuiOutputMode FDisplayClusterGuiLayerController::ExtractOutputModeFromCvar() const
{
	// Get current mode
	const int32 Value = UE::nDisplay::Core::GUILayer::Private::CVarPropagateGui.GetValueOnGameThread();

	// Convert to enum
	return GetOutputModeFromInt(Value);
}

void FDisplayClusterGuiLayerController::HandleSlatePreTick(float InDeltaTime)
{
	checkSlow(IsInGameThread());

	// Nothing to do if disabled. bEnabled is a main switch that has nothing common with
	// the GUI output mode. This one allows the GUI controller to operate. So far,
	// it only allows to operate when nDisplay is running in 'cluster' mode.
	if (!bEnabled)
	{
		return;
	}

	// Read cvar values once per frame
	OutputMode = ExtractOutputModeFromCvar();
	TextureResolution = ComputeGuiTextureSize();

	ENQUEUE_RENDER_COMMAND(FlushCanvasRT)(
		[this, NewOutputMode = OutputMode, NewResolution = TextureResolution](FRHICommandListImmediate& RHICmdList)
		{
			// Update render thread data
			OutputModeRT = NewOutputMode;
			TextureResolutionRT = NewResolution;

			// Reset rotated GUI texture, a new one will be drawn on this frame
			TextureRotatedGUI.SafeRelease();

			// If disabled in runtime, release resources and leave
			if (OutputModeRT == EGuiOutputMode::Disabled)
			{
				TextureGUI.SafeRelease();
				return;
			}

			// Get current game viewport's render target
			FTextureRHIRef ViewportRTT = GetViewportRTT_RenderThread();
			if (!ViewportRTT.IsValid())
			{
				return;
			}

			// Create a buffer duplicate, but float16
			UpdateTempTexture_RenderThread(RHICmdList, TextureGUI, ViewportRTT, TEXT("nD.TextureGUI"), PF_FloatRGBA);
			if (!TextureGUI.IsValid())
			{
				return;
			}

			// Store the original viewport's buffer, and substitute is with our own buffer
			TextureViewport = ViewportRTT;
			SetViewportRTT_RenderThread(TextureGUI);

			// Clear out texture to transparent to let Slate renderer produce a clear GUI layer
			FRDGBuilder GraphBuilder(RHICmdList);
			FRDGTextureRef TexViewportGUILayerRDG = RegisterExternalTexture(GraphBuilder, TextureGUI.GetReference(), TEXT("nD.TextureGUI"));
			AddClearRenderTargetPass(GraphBuilder, TexViewportGUILayerRDG, FLinearColor::Transparent);

			GraphBuilder.Execute();
		});
}

FIntPoint FDisplayClusterGuiLayerController::ComputeGuiTextureSize() const
{
	checkSlow(IsInGameThread());

	// Get UE viewport size
	FIntPoint GameViewportSize = FIntPoint::ZeroValue;
	if (GEngine && GEngine->GameViewport && GEngine->GameViewport->Viewport)
	{
		GameViewportSize = GEngine->GameViewport->Viewport->GetSize();
	}

	// Ensure the texture is large enough. Its size must not be smaller than the user-requested value (CVar).
	const FIntPoint GuiTextureSize =
	{
		FMath::Max(GameViewportSize.X, UE::nDisplay::Core::GUILayer::Private::CurrentResolution.X),
		FMath::Max(GameViewportSize.Y, UE::nDisplay::Core::GUILayer::Private::CurrentResolution.Y)
	};

	return GuiTextureSize;
}

FTextureRHIRef FDisplayClusterGuiLayerController::GetViewportRTT_RenderThread()
{
	checkSlow(IsInRenderingThread());

	if (GEngine && GEngine->GameViewport && GEngine->GameViewport->Viewport)
	{
		return GEngine->GameViewport->Viewport->GetRenderTargetTexture();
	}

	return nullptr;
}

void FDisplayClusterGuiLayerController::SetViewportRTT_RenderThread(FTextureRHIRef& NewRTT)
{
	checkSlow(IsInRenderingThread());
	checkSlow(NewRTT.IsValid());

	if (NewRTT.IsValid())
	{
		if (GEngine && GEngine->GameViewport && GEngine->GameViewport->Viewport)
		{
			static_cast<FSceneViewport*>(GEngine->GameViewport->Viewport)->SetRenderTargetTextureRenderThread(NewRTT);
		}
	}
}

void FDisplayClusterGuiLayerController::UpdateTempTexture_RenderThread(
	FRHICommandListImmediate& RHICmdList,
	FTextureRHIRef& StorageTexture,
	FTextureRHIRef& FromTexture,
	const TCHAR* Name,
	EPixelFormat PixelFormat
)
{
	// Must always be valid
	if (!FromTexture.IsValid())
	{
		return;
	}

	// Don't allow invalid texture size
	if (TextureResolution.GetMin() < 1)
	{
		return;
	}

	// Create new one if not exists
	if (!StorageTexture.IsValid())
	{
		StorageTexture = CreateTextureFrom_RenderThread(RHICmdList, Name, PixelFormat, TextureResolution);
	}

	// Re-create if different size
	if (StorageTexture->GetDesc().Extent != TextureResolution)
	{
		StorageTexture = CreateTextureFrom_RenderThread(RHICmdList, Name, PixelFormat, TextureResolution);
	}
}

FTextureRHIRef FDisplayClusterGuiLayerController::CreateTextureFrom_RenderThread(FRHICommandListImmediate& RHICmdList, const TCHAR* Name, EPixelFormat PixelFormat, const FIntPoint& InSize)
{
	// Prepare description
	FRHITextureCreateDesc Desc =
		FRHITextureCreateDesc::Create2D(Name, InSize.X, InSize.Y, PixelFormat)
		.SetClearValue(FClearValueBinding::Transparent)
		.SetNumMips(1)
		.SetFlags(ETextureCreateFlags::ShaderResource | ETextureCreateFlags::RenderTargetable);

	// Create texture
	return RHICmdList.CreateTexture(Desc);
}

FRDGTextureRef FDisplayClusterGuiLayerController::CreateTextureFrom_RenderThread(
	FRDGBuilder& GraphBuilder,
	const TCHAR* Name,
	const FIntPoint& Size,
	EPixelFormat PixelFormat,
	ETextureCreateFlags CreateFlags = TexCreate_ShaderResource | TexCreate_RenderTargetable,
	const FClearValueBinding& ClearValueBinding = FClearValueBinding::Transparent)
{
	const FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(Size, PixelFormat, ClearValueBinding, CreateFlags);
	return GraphBuilder.CreateTexture(Desc, Name);
}

FRDGTextureRef FDisplayClusterGuiLayerController::CreateTextureFrom_RenderThread(FRDGBuilder& GraphBuilder, FRDGTextureRef& FromTexture, const TCHAR* Name, const FIntPoint& Size)
{
	if (!FromTexture)
	{
		return nullptr;
	}

	const FIntPoint NewSize = (Size == FIntPoint::ZeroValue ? FromTexture->Desc.Extent : Size);
	return CreateTextureFrom_RenderThread(GraphBuilder, Name, NewSize, FromTexture->Desc.Format);
}

FRDGTextureRef FDisplayClusterGuiLayerController::CreateTextureFrom_RenderThread(FRDGBuilder& GraphBuilder, FRHITexture* FromTexture, const TCHAR* Name, const FIntPoint& Size)
{
	if (!FromTexture)
	{
		return nullptr;
	}

	const FIntPoint NewSize = (Size == FIntPoint::ZeroValue ? FromTexture->GetDesc().Extent : Size);
	return CreateTextureFrom_RenderThread(GraphBuilder, Name, NewSize, FromTexture->GetFormat());
}
