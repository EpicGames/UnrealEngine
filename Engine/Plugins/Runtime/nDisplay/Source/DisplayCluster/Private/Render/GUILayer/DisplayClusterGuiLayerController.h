// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Render/GUILayer/IDisplayClusterGUILayerController.h"

#include "Templates/RefCounting.h"

#include "PixelFormat.h"
#include "RHIResources.h"
#include "UnrealClient.h"

class FRDGBuilder;
class FSceneViewFamily;
class IDisplayClusterViewportProxy;
struct IPooledRenderTarget;


/**
 * GUI layer controller singleton
 * 
 * This class is responsible for propagation of the GUI layer
 * (includes canvases + UMG) across all nDisplay viewports.
 * It also exposes the GUI layer texture to external users.
 */
class FDisplayClusterGuiLayerController
	: public IDisplayClusterGUILayerController
{
protected:

	FDisplayClusterGuiLayerController();
	~FDisplayClusterGuiLayerController() = default;

public:

	/** Singleton access */
	static FDisplayClusterGuiLayerController& Get();

	/** Converts output mode index to the enum value */
	static IDisplayClusterGUILayerController::EGuiOutputMode GetOutputModeFromInt(int32 InOutputMode);

public:

	//~ Begin IDisplayClusterGUILayerController interface

	/** Return current operation mode */
	virtual EGuiOutputMode GetOutputMode() const override;

	/** Returns true if GUI controller is currently enabled */
	virtual bool IsEnabled() const override;

	/** Returns current GUI layer texture size or {0, 0} if disabled */
	virtual FIntPoint GetGuiLayerTextureSize() const override;

	/** Returns current GUI layer texture if available, otherwise invalid reference */
	virtual FTextureRHIRef GetGuiLayerTexture_RenderThread() override
	{
		return TextureGUI;
	}

	//~ End IDisplayClusterGUILayerController interface

public:

	/**
	 * Called before copying the final output to the backbuffer. It's used to substitute the current
	 * viewport's buffer to its original one stored on SlatePreTick (refer corresponding method).
	 * 
	 * @param FinalTexture - The final Slate outcome that is passed for copying to the backbuffer
	 */
	FRDGTextureRef ProcessFinalTexture_RenderThread(FRDGBuilder& GraphBuilder, FRDGTextureRef FinalTexture);

	/**
	 * Called for each nDisplay view family once rendering is finished. Responsible for drawing
	 * the GUI layer on top of every nDisplay viewport.
	 * 
	 * @param ViewFamily - View family that was just rendered
	 * @param ViewProxy  - nDisplay viewport (render thread obj)
	 */
	void ProcessPostRenderViewFamily_RenderThread(FRDGBuilder& GraphBuilder, const FSceneViewFamily& ViewFamiliy, const IDisplayClusterViewportProxy* ViewportProxy);

private:

	/** Extracts operation mode from the cvar */
	EGuiOutputMode ExtractOutputModeFromCvar() const;

	/**
	 * Slate pre-Tick callback, used to substitute original game viewport buffer with our custom texture
	 * to store whole Slate rendering output separately, and with high-precision alpha.
	 */
	void HandleSlatePreTick(float InDeltaTime);

	/** Computes the size of the UI layer texture */
	FIntPoint ComputeGuiTextureSize() const;

private:

	/** Returns current game viewport buffer */
	FTextureRHIRef GetViewportRTT_RenderThread();

	/** Sets up new game viewport bufer (stays there until end of the frame) */
	void SetViewportRTT_RenderThread(FTextureRHIRef& NewRTT);

	/**
	 * Creates or re-creates a texture duplicate from a referenced texture. Allows to override some texture parameters.
	 * If some parameters of the referenced texture are different, a new internal texture will be created.
	 * 
	 * @param StorageTexture - An internal "duplicate" of the referenced texture
	 * @param FromTexture    - The referenced texture
	 * @param Name           - Name of the new texture
	 * @param PixelFormat    - Pixel format of the new texture
	 */
	void UpdateTempTexture_RenderThread(FRHICommandListImmediate& RHICmdList, FTextureRHIRef& StorageTexture, FTextureRHIRef& FromTexture, const TCHAR* Name, EPixelFormat PixelFormat);

	/** Auxiliary function to create an RHI texture. */
	FTextureRHIRef CreateTextureFrom_RenderThread(FRHICommandListImmediate& RHICmdList, const TCHAR* Name, EPixelFormat PixelFormat, const FIntPoint& InSize);

	/** Auxiliary function to create an RDG texture. */
	FRDGTextureRef CreateTextureFrom_RenderThread(FRDGBuilder& GraphBuilder, const TCHAR* Name, const FIntPoint& Size, EPixelFormat PixelFormat, ETextureCreateFlags CreateFlags, const FClearValueBinding& ClearValueBinding);

	/** Auxiliary function to create an RDG texture from an RDG texture template, overriding some of the reference parameters. */
	FRDGTextureRef CreateTextureFrom_RenderThread(FRDGBuilder& GraphBuilder, FRDGTextureRef& FromTexture, const TCHAR* Name, const FIntPoint& Size = FIntPoint::ZeroValue);

	/** Auxiliary function to create an RDG texture from an RHI texture template, overriding some of the reference parameters. */
	FRDGTextureRef CreateTextureFrom_RenderThread(FRDGBuilder& GraphBuilder, FRHITexture* FromTexture, const TCHAR* Name, const FIntPoint& Size = FIntPoint::ZeroValue);

private:

	/** Whether GUI layer operation is allowed */
	const bool bEnabled;

	/** Current GUI output mode (game thread) */
	EGuiOutputMode OutputMode = EGuiOutputMode::Whole;

	/** Current GUI output mode (render thread) */
	EGuiOutputMode OutputModeRT = EGuiOutputMode::Whole;

	/** Current GUI texture resolution (game thread) */
	FIntPoint TextureResolution = FIntPoint::ZeroValue;

	/** Current GUI texture resolution (render thread) */
	FIntPoint TextureResolutionRT = FIntPoint::ZeroValue;

	/** The RHI texture reference to the original game viewport buffer */
	FTextureRHIRef TextureViewport;

	/** Internal RHI texture used to redirect GUI rendering to */
	FTextureRHIRef TextureGUI;

	/** Internal RDG texture used to share the rotated GUI among different graph builders (post-render callbacks) */
	TRefCountPtr<IPooledRenderTarget> TextureRotatedGUI;
};
