// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Render/IDisplayClusterRenderManager.h"
#include "IPDisplayClusterManager.h"


/**
 * Render manager interface
 */
class IPDisplayClusterRenderManager
	: public IDisplayClusterRenderManager
	, public IPDisplayClusterManager
{
public:
	virtual ~IPDisplayClusterRenderManager() = default;
};
