// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Monitoring/DisplayClusterVblankMonitor.h"

#include "Render/Synchronization/DisplayClusterRenderSyncHelper.h"
#include "Misc/DisplayClusterLog.h"
#include "Misc/ScopeLock.h"

#include "HAL/RunnableThread.h"


FDisplayClusterVBlankMonitor::FDisplayClusterVBlankMonitor()
{
}

FDisplayClusterVBlankMonitor::~FDisplayClusterVBlankMonitor()
{
	StopMonitoring();
}


uint32 FDisplayClusterVBlankMonitor::Run()
{
	UE_LOGF(LogDisplayClusterRenderSync, Log, "V-blank monitoring thread has started");

	while (!bThreadExitRequested)
	{
		// Get sync helper based on the current RHI
		IDisplayClusterRenderSyncHelper& SyncHelper = FDisplayClusterRenderSyncHelper::Get();

		// Polling...
		if (SyncHelper.IsWaitForVBlankSupported())
		{
			// Wait for V-blank
			if (!SyncHelper.WaitForVBlank(nullptr))
			{
				// Nothing to do if the underlying output subsystem is not ready for monitoring yet
				FPlatformProcess::SleepNoStats(1e-3f);
				continue;
			}

			// Get current timestamp
			const double CurrentTime = FPlatformTime::Seconds();

			TRACE_BOOKMARK(TEXT("VBLANK %llu"), VBlankCounter);

			{
				FScopeLock Lock(&DataCS);

				++VBlankCounter;

				if (!bTimedataAvailable)
				{
					// Here we store last V-blank timestamp only
					if (FMath::IsNearlyZero(LastVblankTime))
					{
						LastVblankTime = CurrentTime;
					}
					// Here we store both V-blank timestamp and V-blank period
					else
					{
						VblankPeriod = CurrentTime - LastVblankTime;
						LastVblankTime = CurrentTime;
						bTimedataAvailable = true;
					}
				}
				else
				{
					// Update V-blank timestamp and period (always average for better precision)
					VblankPeriod = (VblankPeriod + (CurrentTime - LastVblankTime)) / 2;
					LastVblankTime = CurrentTime;

					UE_LOGF(LogDisplayClusterRenderSync, VeryVerbose, "V-blank monitoring: vblank=%lf, period=%lf", LastVblankTime, VblankPeriod);
				}
			}
		}
		else
		{
			// No need to keep the monitoring thread alive as v-blank monitoring is not supported
			break;
		}
	}

	UE_LOGF(LogDisplayClusterRenderSync, Log, "V-blank monitoring thread has finished");

	return 0;
}

bool FDisplayClusterVBlankMonitor::StartMonitoring()
{
	FScopeLock Lock(&ControlCS);

	// Check if running already
	if (bIsRunning)
	{
		return false;
	}

	// Reset internals
	ResetData();

	// Update controls
	bIsRunning = true;
	bThreadExitRequested = false;

	// Start monitoring thread
	WorkingThread.Reset(FRunnableThread::Create(this, TEXT("VBLANK_Monitoring_thread"), 32 * 1024, TPri_AboveNormal));

	return true;
}

bool FDisplayClusterVBlankMonitor::StopMonitoring()
{
	FScopeLock Lock(&ControlCS);

	// Check if running
	if (!bIsRunning)
	{
		return false;
	}

	// Update controls
	bThreadExitRequested = true;
	bIsRunning = false;

	// Wait until working thread is finished
	if (WorkingThread)
	{
		WorkingThread->WaitForCompletion();
		WorkingThread.Reset();
	}

	// Reset internals
	ResetData();

	return true;
}

bool FDisplayClusterVBlankMonitor::IsMonitoring()
{
	FScopeLock Lock(&ControlCS);
	return bIsRunning;
}

bool FDisplayClusterVBlankMonitor::IsVblankTimeDataAvailable()
{
	FScopeLock Lock(&DataCS);
	return bTimedataAvailable;
}

double FDisplayClusterVBlankMonitor::GetLastVblankTime()
{
	FScopeLock Lock(&DataCS);
	return LastVblankTime;
}

double FDisplayClusterVBlankMonitor::GetNextVblankTime()
{
	FScopeLock Lock(&DataCS);
	return LastVblankTime + VblankPeriod;
}

void FDisplayClusterVBlankMonitor::ResetData()
{
	FScopeLock Lock(&DataCS);

	VBlankCounter = 0;
	bTimedataAvailable = false;
	LastVblankTime = 0;
	VblankPeriod = 0;
}
