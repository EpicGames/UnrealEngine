// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Render/Monitoring/IDisplayClusterVblankMonitor.h"
#include "HAL/Runnable.h"


/**
 * V-blank monitor. It continuously polls the current sync policy
 * for V-blanks and reports the time data to utraces.
 */
class FDisplayClusterVBlankMonitor
	: public IDisplayClusterVblankMonitor
	, protected FRunnable
{
public:
	FDisplayClusterVBlankMonitor();
	virtual ~FDisplayClusterVBlankMonitor();

public:
	//~ Begin IDisplayClusterVblankMonitor interface
	virtual bool StartMonitoring() override;
	virtual bool StopMonitoring() override;
	virtual bool IsMonitoring() override;
	virtual bool IsVblankTimeDataAvailable() override;
	virtual double GetLastVblankTime() override;
	virtual double GetNextVblankTime() override;
	//~ End IDisplayClusterVblankMonitor interface

protected:
	//~ Begin FRunnable interface
	virtual uint32 Run() override;
	//~ End FRunnable interface

private:
	/** Resets internal data */
	void ResetData();

private:
	/** Holds the thread object */
	TUniquePtr<FRunnableThread> WorkingThread;

	/** Keeps monitoring status (true if running) */
	bool bIsRunning = false;

	/** V - blank counter */
	uint64 VBlankCounter = 0;

	/** Flag to ask working thread to terminate */
	bool bThreadExitRequested = false;

	/* It's set to true when time data is available (last vblank, period) */
	bool bTimedataAvailable = false;

	/* A timestamp of the last V-blank */
	double LastVblankTime = 0;

	/* V-blank period (approximate) */
	double VblankPeriod = 0;

	/** CritSec for control */
	mutable FCriticalSection ControlCS;
	
	/** CritSec for data */
	mutable FCriticalSection DataCS;
};
