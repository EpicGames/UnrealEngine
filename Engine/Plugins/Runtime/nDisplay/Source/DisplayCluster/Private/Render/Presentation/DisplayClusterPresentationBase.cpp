// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Presentation/DisplayClusterPresentationBase.h"
#include "Render/Synchronization/IDisplayClusterRenderSyncPolicy.h"

#include "HAL/IConsoleManager.h"
#include "IPDisplayCluster.h"
#include "IDisplayClusterCallbacks.h"

#include "Misc/DisplayClusterGlobals.h"
#include "Misc/DisplayClusterLog.h"


// Custom VSync interval control
static TAutoConsoleVariable<int32>  CVarVSyncInterval(
	TEXT("nDisplay.render.VSyncInterval"),
	1,
	TEXT("VSync interval"),
	ECVF_RenderThreadSafe
);


FDisplayClusterPresentationBase::FDisplayClusterPresentationBase(FViewport* const InViewport, TSharedPtr<IDisplayClusterRenderSyncPolicy>& InSyncPolicy)
	: FRHICustomPresent()
	, Viewport(InViewport)
	, SyncPolicy(InSyncPolicy)
{
}

FDisplayClusterPresentationBase::~FDisplayClusterPresentationBase()
{
}

uint32 FDisplayClusterPresentationBase::GetSwapInt() const
{
	const uint32 SyncInterval = static_cast<uint32>(CVarVSyncInterval.GetValueOnAnyThread());
	return (SyncInterval);
}

void FDisplayClusterPresentationBase::OnBackBufferResize()
{
}

bool FDisplayClusterPresentationBase::Present(FRHIViewport* ViewportRHI, IRHICommandContext& RHICmdContext, int32& InOutSyncInterval)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(nDisplay PresentationBase);

	bool bNeedPresent = true;

	// Get sync policy instance
	if (SyncPolicy.IsValid())
	{
		// Update sync value with nDisplay value
		InOutSyncInterval = GetSwapInt();

		GDisplayCluster->GetCallbacks().OnDisplayClusterPresentationPreSynchronization_RHIThread().Broadcast(ViewportRHI);

		// False results means we don't need to present current frame, the sync object already presented it
		bNeedPresent = SyncPolicy->SynchronizeClusterRendering(ViewportRHI, InOutSyncInterval);

		GDisplayCluster->GetCallbacks().OnDisplayClusterPresentationPostSynchronization_RHIThread().Broadcast(ViewportRHI);
	}

	return bNeedPresent;
}

bool FDisplayClusterPresentationBase::PresentOnSubmissionThread(int32& InOutSyncInterval)
{
	checkNoEntry(); // @todo not currently supported - fix this
	return true;
}

void FDisplayClusterPresentationBase::PostPresent()
{
	// Notify native frame presentation has been done
	GDisplayCluster->GetCallbacks().OnDisplayClusterFramePresented_RHIThread().Broadcast(true);
}
