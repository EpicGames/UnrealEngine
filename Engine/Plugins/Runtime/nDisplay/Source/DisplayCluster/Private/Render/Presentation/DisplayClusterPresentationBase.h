// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Render/Presentation/IDisplayClusterPresentation.h"
#include "RHI.h"
#include "RHIResources.h"

class FViewport;
class IDisplayClusterRenderSyncPolicy;


class FDisplayClusterPresentationBase
	: public FRHICustomPresent
	, public IDisplayClusterPresentation
{
public:
	FDisplayClusterPresentationBase(FViewport* const InViewport, TSharedPtr<IDisplayClusterRenderSyncPolicy>& InSyncPolicy);
	virtual ~FDisplayClusterPresentationBase();

public:
	// Returns internal swap interval
	uint32 GetSwapInt() const;

protected:
	//////////////////////////////////////////////////////////////////////////////////////////////
	// FRHICustomPresent
	//////////////////////////////////////////////////////////////////////////////////////////////
	virtual bool NeedsNativePresent() override
	{
		return true;
	}

	virtual bool NeedsAdvanceBackbuffer() override
	{
		return true;
	}

	virtual bool Present(FRHIViewport* RHIViewport, IRHICommandContext& RHICmdContext, int32& InOutSyncInterval) override;
	virtual bool PresentOnSubmissionThread(int32& InOutSyncInterval) override;
	virtual void PostPresent() override;
	virtual void OnBackBufferResize() override;

protected:
	FViewport* const GetViewport() const
	{
		return Viewport;
	}

	const TSharedPtr<IDisplayClusterRenderSyncPolicy>& GetSyncPolicyObject() const
	{
		return SyncPolicy;
	}

private:
	FViewport* const Viewport;
	TSharedPtr<IDisplayClusterRenderSyncPolicy> SyncPolicy;
};
