// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Render/Presentation/DisplayClusterPresentationBase.h"


/**
 * Present stub to allow to synchronize a cluster with native rendering pipeline (no nDisplay stereo devices used)
 */
class FDisplayClusterPresentationNative : public FDisplayClusterPresentationBase
{
public:
	FDisplayClusterPresentationNative(FViewport* const Viewport, TSharedPtr<IDisplayClusterRenderSyncPolicy>& SyncPolicy)
		: FDisplayClusterPresentationBase(Viewport, SyncPolicy)
	{ }

	virtual ~FDisplayClusterPresentationNative() = default;
};
