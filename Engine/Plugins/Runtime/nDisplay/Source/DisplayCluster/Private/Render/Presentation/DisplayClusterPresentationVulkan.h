// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Render/Presentation/DisplayClusterPresentationBase.h"


/**
 * Helper class to encapsulate Vulkan frame presentation
 */
class FDisplayClusterPresentationVulkan : public FDisplayClusterPresentationBase
{
public:
	FDisplayClusterPresentationVulkan(FViewport* const InViewport, TSharedPtr<IDisplayClusterRenderSyncPolicy>& InSyncPolicy)
		: FDisplayClusterPresentationBase(InViewport, InSyncPolicy)
	{ }

	virtual ~FDisplayClusterPresentationVulkan() = default;
};
