// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Render/Presentation/DisplayClusterPresentationBase.h"


/**
 * Helper class to encapsulate DX11 frame presentation
 */
class FDisplayClusterPresentationDX11 : public FDisplayClusterPresentationBase
{
public:
	FDisplayClusterPresentationDX11(FViewport* const InViewport, TSharedPtr<IDisplayClusterRenderSyncPolicy>& InSyncPolicy)
		: FDisplayClusterPresentationBase(InViewport, InSyncPolicy)
	{ }

	virtual ~FDisplayClusterPresentationDX11() = default;
};
