// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Render/Presentation/DisplayClusterPresentationBase.h"


/**
 * Helper class to encapsulate DX12 frame presentation
 */
class FDisplayClusterPresentationDX12 : public FDisplayClusterPresentationBase
{
public:
	FDisplayClusterPresentationDX12(FViewport* const InViewport, TSharedPtr<IDisplayClusterRenderSyncPolicy>& InSyncPolicy)
		: FDisplayClusterPresentationBase(InViewport, InSyncPolicy)
	{ }

	virtual ~FDisplayClusterPresentationDX12() = default;
};
