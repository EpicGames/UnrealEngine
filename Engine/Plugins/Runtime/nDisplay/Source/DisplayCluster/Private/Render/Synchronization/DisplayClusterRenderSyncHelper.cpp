// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Synchronization/DisplayClusterRenderSyncHelper.h"

#include "Misc/ScopeLock.h"


// Singleton instance
TUniquePtr<IDisplayClusterRenderSyncHelper> FDisplayClusterRenderSyncHelper::Instance;

// Singleton access mutex
FCriticalSection FDisplayClusterRenderSyncHelper::CritSecInternals;


IDisplayClusterRenderSyncHelper& FDisplayClusterRenderSyncHelper::Get()
{
	if (!Instance)
	{
		FScopeLock Lock(&FDisplayClusterRenderSyncHelper::CritSecInternals);
		if (!Instance)
		{
			Instance = CreateHelper();
		}
	}

	return *Instance.Get();
}
