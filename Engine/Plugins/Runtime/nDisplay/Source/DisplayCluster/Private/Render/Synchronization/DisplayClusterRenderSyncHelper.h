// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RHIFwd.h"

/**
 * RHI agnostic helper interface
 */
class IDisplayClusterRenderSyncHelper
{
public:
	virtual ~IDisplayClusterRenderSyncHelper() = default;

public:
	virtual bool WaitForVBlank(FRHIViewport* ViewportRHI)
	{
		return false;
	}

	virtual bool IsWaitForVBlankSupported()
	{
		return false;
	}

	virtual bool GetMaximumFrameLatency(FRHIViewport* ViewportRHI, uint32& OutMaximumFrameLatency)
	{
		return false;
	}

	virtual bool SetMaximumFrameLatency(FRHIViewport* ViewportRHI, uint32 MaximumFrameLatency)
	{
		return false;
	}
};


/**
 * Synchronization helper implementation. The main purpose of this class
 * is to solve the combinatoric complexity Platform/RHI/SyncPolicy without
 * making the whole sync policies hierarchy overcomplicated.
 */
class FDisplayClusterRenderSyncHelper
{
private:
	FDisplayClusterRenderSyncHelper();

public:
	// Singleton access
	static IDisplayClusterRenderSyncHelper& Get();

protected:
	// Factory method to instantiate helper instance of proper type
	static TUniquePtr<IDisplayClusterRenderSyncHelper> CreateHelper();

#if PLATFORM_WINDOWS
private:
	// DX helper implementation
	struct FDCRSHelperDX : public IDisplayClusterRenderSyncHelper
	{
		virtual bool WaitForVBlank(FRHIViewport* ViewportRHI) override;
		virtual bool IsWaitForVBlankSupported() override;
		virtual bool GetMaximumFrameLatency(FRHIViewport* ViewportRHI, uint32& OutMaximumFrameLatency) override;
		virtual bool SetMaximumFrameLatency(FRHIViewport* ViewportRHI, uint32 MaximumFrameLatency) override;
	};
#endif

private:
	// Vulkan helper implementation
	struct FDCRSHelperVulkan : public IDisplayClusterRenderSyncHelper
	{
		virtual bool WaitForVBlank(FRHIViewport* ViewportRHI) override;
		virtual bool IsWaitForVBlankSupported() override;
		virtual bool GetMaximumFrameLatency(FRHIViewport* ViewportRHI, uint32& OutMaximumFrameLatency) override;
		virtual bool SetMaximumFrameLatency(FRHIViewport* ViewportRHI, uint32 MaximumFrameLatency) override;
	};

private:
	// Stub implementation for unknown RHI
	struct FDCRSHelperNull : public IDisplayClusterRenderSyncHelper
	{
	};

private:
	// Helper instance
	static TUniquePtr<IDisplayClusterRenderSyncHelper> Instance;

	// Singleton access mutex
	static FCriticalSection CritSecInternals;
};
