// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Synchronization/DisplayClusterRenderSyncPolicyBase.h"
#include "Components/Viewport.h"
#include "Render/Synchronization/DisplayClusterRenderSyncHelper.h"

#include "Cluster/IPDisplayClusterClusterManager.h"
#include "Cluster/Controller/IDisplayClusterClusterNodeController.h"
#include "Cluster/NetAPI/DisplayClusterNetApiFacade.h"

#include "Misc/DisplayClusterGlobals.h"
#include "Misc/DisplayClusterLog.h"

#include "Engine/Engine.h"
#include "Engine/GameViewportClient.h"

#include "RHI.h"
#include "RHIResources.h"


bool FDisplayClusterRenderSyncPolicyBase::Initialize()
{
	return true;
}

void FDisplayClusterRenderSyncPolicyBase::SyncOnBarrier()
{
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(nDisplay SyncOnBarrier);
		GDisplayCluster->GetPrivateClusterMgr()->GetNetApi().GetRenderSyncAPI()->SynchronizeOnBarrier();
	}
}

void FDisplayClusterRenderSyncPolicyBase::WaitForFrameCompletion(FRHIViewport* ViewportRHI)
{
	ViewportRHI->IssueFrameEvent();
	ViewportRHI->WaitForFrameEventCompletion();
}

bool FDisplayClusterRenderSyncPolicyBase::GetMaximumFrameLatency(FRHIViewport* ViewportRHI, uint32& OutMaximumFrameLatency)
{
	return FDisplayClusterRenderSyncHelper::Get().GetMaximumFrameLatency(ViewportRHI, OutMaximumFrameLatency);
}

bool FDisplayClusterRenderSyncPolicyBase::SetMaximumFrameLatency(FRHIViewport* ViewportRHI, uint32 MaximumFrameLatency)
{
	return FDisplayClusterRenderSyncHelper::Get().SetMaximumFrameLatency(ViewportRHI, MaximumFrameLatency);
}

bool FDisplayClusterRenderSyncPolicyBase::IsWaitForVBlankFeatureSupported()
{
	return FDisplayClusterRenderSyncHelper::Get().IsWaitForVBlankSupported();
}

bool FDisplayClusterRenderSyncPolicyBase::WaitForVBlank(FRHIViewport* ViewportRHI)
{
	return FDisplayClusterRenderSyncHelper::Get().WaitForVBlank(ViewportRHI);
}
