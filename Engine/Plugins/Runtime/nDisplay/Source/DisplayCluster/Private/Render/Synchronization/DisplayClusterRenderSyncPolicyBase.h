// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Render/Synchronization/IDisplayClusterRenderSyncPolicy.h"


/**
 * Base synchronization policy
 */
class FDisplayClusterRenderSyncPolicyBase
	: public IDisplayClusterRenderSyncPolicy
{
public:
	FDisplayClusterRenderSyncPolicyBase(const TMap<FString, FString>& InParameters)
		: Parameters(InParameters)
	{ }

	virtual ~FDisplayClusterRenderSyncPolicyBase() = default;

public:
	//~ Begin IDisplayClusterRenderSyncPolicy interface

	/** Non-constructor initialization */
	virtual bool Initialize() override;

	/** VBlank related functions */
	virtual bool IsWaitForVBlankFeatureSupported() override;
	virtual bool WaitForVBlank(FRHIViewport* ViewportRHI) override;

	//~ End IDisplayClusterRenderSyncPolicy interface

public:
	/** Performs Ethernet barrier synchronization with other nodes in the cluster */
	void SyncOnBarrier();

	/** Returns policy configuration parameters */
	const TMap<FString, FString>& GetParameters() const
	{
		return Parameters;
	}

protected:
	/** Wait for frame rendering completion using a GPU fence */
	virtual void WaitForFrameCompletion(FRHIViewport* ViewportRHI);

	/** Get maximum frame latency */
	bool GetMaximumFrameLatency(FRHIViewport* ViewportRHI, uint32& OutMaximumFrameLatency);

	/** Set maximum frame latency */
	bool SetMaximumFrameLatency(FRHIViewport* ViewportRHI, uint32 MaximumFrameLatency);

private:
	/** Sync policy configuration parameters */
	TMap<FString, FString> Parameters;
};
