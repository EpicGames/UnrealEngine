// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Synchronization/DisplayClusterRenderSyncPolicyEthernet.h"
#include "HAL/IConsoleManager.h"


static TAutoConsoleVariable<int32>  CVarBarrierSyncOnly(
	TEXT("nDisplay.render.softsync.BarrierSyncOnly"),
	0,
	TEXT("Simple synchronization, ethernet barrier only (0 = disabled)"),
	ECVF_ReadOnly
);

static TAutoConsoleVariable<int32> CVarUseCustomRefreshRate(
	TEXT("nDisplay.render.softsync.UseCustomRefreshRate"),
	0,
	TEXT("Force custom refresh rate to be used in synchronization math (0 = disabled)"),
	ECVF_ReadOnly
);

static TAutoConsoleVariable<float> CVarCustomRefreshRate(
	TEXT("nDisplay.render.softsync.CustomRefreshRate"),
	60.f,
	TEXT("Custom refresh rate for synchronization math (Hz)"),
	ECVF_ReadOnly
);

static TAutoConsoleVariable<float> CVarVBlankFrontEdgeThreshold(
	TEXT("nDisplay.render.softsync.VBlankFrontEdgeThreshold"),
	0.003f,
	TEXT("Unsafe period of time before conventional V-blank pulse"),
	ECVF_ReadOnly
);

static TAutoConsoleVariable<float> CVarVBlankBackEdgeThreshold(
	TEXT("nDisplay.render.softsync.VBlankBackEdgeThreshold"),
	0.002f,
	TEXT("Unsafe period of time after conventional V-blank pulse"),
	ECVF_ReadOnly
);

static TAutoConsoleVariable<float> CVarVBlankThresholdSleepMultiplier(
	TEXT("nDisplay.render.softsync.VBlankThresholdSleepMultipler"),
	1.5f,
	TEXT("Multiplier applied to a VBlank threshold to compute sleep time for safely leaving an unsafe zone"),
	ECVF_ReadOnly
);

static TAutoConsoleVariable<int32>  CVarVBlankBasisUpdate(
	TEXT("nDisplay.render.softsync.VBlankBasisUpdate"),
	0,
	TEXT("Update VBlank basis periodically to avoid time drifting (0 = disabled)"),
	ECVF_ReadOnly
);

static TAutoConsoleVariable<float>  CVarVBlankBasisUpdatePeriod(
	TEXT("nDisplay.render.softsync.VBlankBasisUpdatePeriod"),
	120.f,
	TEXT("VBlank basis update period in seconds"),
	ECVF_ReadOnly
);

static TAutoConsoleVariable<int32>  CVarRisePresentationThreadPriority(
	TEXT("nDisplay.render.softsync.RisePresentationThreadPriority"),
	0,
	TEXT("Set higher priority for the presentation thread (0 = disabled)"),
	ECVF_ReadOnly
);


FDisplayClusterRenderSyncPolicyEthernet::FDisplayClusterRenderSyncPolicyEthernet(const TMap<FString, FString>& Parameters)
	: FDisplayClusterRenderSyncPolicyBase(Parameters)

	, bSimpleSync(!!CVarBarrierSyncOnly.GetValueOnGameThread())

	, bUseCustomRefreshRate(!!CVarUseCustomRefreshRate.GetValueOnGameThread())
	, CustomRefreshRate(CVarCustomRefreshRate.GetValueOnGameThread())

	, VBlankFrontEdgeThreshold(CVarVBlankFrontEdgeThreshold.GetValueOnGameThread())
	, VBlankBackEdgeThreshold(CVarVBlankBackEdgeThreshold.GetValueOnGameThread())
	, VBlankThresholdSleepMultiplier(CVarVBlankThresholdSleepMultiplier.GetValueOnGameThread())
	, VBlankBasisUpdate(!!CVarVBlankBasisUpdate.GetValueOnGameThread())
	, VBlankBasisUpdatePeriod(CVarVBlankBasisUpdatePeriod.GetValueOnGameThread())

	, bRiseThreadPriority(!!CVarRisePresentationThreadPriority.GetValueOnGameThread())
{
}
