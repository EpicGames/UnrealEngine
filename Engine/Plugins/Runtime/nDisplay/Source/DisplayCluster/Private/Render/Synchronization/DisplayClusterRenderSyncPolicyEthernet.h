// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Render/Synchronization/DisplayClusterRenderSyncPolicyBase.h"
#include "DisplayClusterConfigurationStrings.h"


/**
 * Base Ethernet synchronization policy
 */
class FDisplayClusterRenderSyncPolicyEthernet
	: public FDisplayClusterRenderSyncPolicyBase
{
public:
	FDisplayClusterRenderSyncPolicyEthernet(const TMap<FString, FString>& Parameters);

	virtual ~FDisplayClusterRenderSyncPolicyEthernet() = default;

public:
	//////////////////////////////////////////////////////////////////////////////////////////////
	// IDisplayClusterRenderSyncPolicy
	//////////////////////////////////////////////////////////////////////////////////////////////
	virtual FName GetName() const override
	{
		static const FName Policy = DisplayClusterConfigurationStrings::config::cluster::render_sync::Ethernet;
		return Policy;
	}

	virtual bool SynchronizeClusterRendering(FRHIViewport* ViewportRHI, int32& InOutSyncInterval) override;

protected:
	virtual void Procedure_SynchronizePresent(FRHIViewport* ViewportRHI);
	virtual void Step_InitializeFrameSynchronization(FRHIViewport* ViewportRHI);
	virtual void Step_WaitForFrameCompletion(FRHIViewport* ViewportRHI);
	virtual void Step_WaitForEthernetBarrierSignal_1();
	virtual void Step_SkipPresentationOnClosestVBlank();
	virtual void Step_WaitForEthernetBarrierSignal_2();
	virtual void Step_Present(FRHIViewport* ViewportRHI);
	virtual void Step_FinalizeFrameSynchronization();

	double GetVBlankTimestamp(FRHIViewport* ViewportRHI);
	double GetRefreshPeriod();

private:
	// Sync math
	double B1B = 0.f;  // Barrier 1 before
	double B1A = 0.f;  // Barrier 1 after
	double TToB = 0.f; // Time to VBlank
	double SB = 0.f;   // Sleep before
	double SA = 0.f;   // Sleep after
	double B2B = 0.f;  // Barrier 2 before
	double B2A = 0.f;  // Barrier 2 after
	double PB = 0.f;   // Present before
	double PA = 0.f;   // Present after

	// Sync logic (cvars)
	const bool  bSimpleSync = false;
	const bool  bUseCustomRefreshRate = false;
	const float CustomRefreshRate = 0.f;
	const float VBlankFrontEdgeThreshold = 0.f;
	const float VBlankBackEdgeThreshold = 0.f;
	const float VBlankThresholdSleepMultiplier = 0.f;
	const bool  VBlankBasisUpdate = false;
	const float VBlankBasisUpdatePeriod = 0.f;
	const bool  bRiseThreadPriority = false;

	// Sync internals
	int SyncIntervalToUse = 1;
	bool bInternalsInitialized = false;
	double VBlankBasis = 0;
	double RefreshPeriod = 0;
	uint32 FrameCounter = 0;
};
