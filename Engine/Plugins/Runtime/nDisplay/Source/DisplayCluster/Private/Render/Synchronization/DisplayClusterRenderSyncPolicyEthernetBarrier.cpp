// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Synchronization/DisplayClusterRenderSyncPolicyEthernetBarrier.h"
#include "DisplayClusterConfigurationStrings.h"


FName FDisplayClusterRenderSyncPolicyEthernetBarrier::GetName() const
{
	static const FName Policy = DisplayClusterConfigurationStrings::config::cluster::render_sync::EthernetBarrier;
	return Policy;
}

bool FDisplayClusterRenderSyncPolicyEthernetBarrier::SynchronizeClusterRendering(FRHIViewport* ViewportRHI, int32& InOutSyncInterval)
{
	// Wait unless the frame is rendered
	WaitForFrameCompletion(ViewportRHI);
	// Sync on the network barrier
	SyncOnBarrier();
	// Ask engine to present the frame
	return true;
}
