// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Render/Synchronization/DisplayClusterRenderSyncPolicyBase.h"


/**
 * Simple version of Ethernet based synchronization (barrier only)
 */
class FDisplayClusterRenderSyncPolicyEthernetBarrier
	: public FDisplayClusterRenderSyncPolicyBase
{
public:
	FDisplayClusterRenderSyncPolicyEthernetBarrier(const TMap<FString, FString>& Parameters)
		: FDisplayClusterRenderSyncPolicyBase(Parameters)
	{ }

	virtual ~FDisplayClusterRenderSyncPolicyEthernetBarrier() = default;

public:
	//////////////////////////////////////////////////////////////////////////////////////////////
	// IDisplayClusterRenderSyncPolicy
	//////////////////////////////////////////////////////////////////////////////////////////////
	virtual FName GetName() const override;
	virtual bool SynchronizeClusterRendering(FRHIViewport* ViewportRHI, int32& InOutSyncInterval) override;
};
