// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Synchronization/DisplayClusterRenderSyncPolicyFactoryInternal.h"

#include "Render/Synchronization/DisplayClusterRenderSyncPolicyNone.h"
#include "Render/Synchronization/DisplayClusterRenderSyncPolicyEthernet.h"
#include "Render/Synchronization/DisplayClusterRenderSyncPolicyEthernetBarrier.h"
#include "Render/Synchronization/DisplayClusterRenderSyncPolicyNvidiaSwapBarrier.h"
#include "Render/Synchronization/DisplayClusterRenderSyncPolicyNvidiaPresentBarrier.h"

#include "Misc/DisplayClusterLog.h"
#include "Misc/DisplayClusterStrings.h"
#include "DisplayClusterConfigurationStrings.h"
#include "RHI.h"
#include "RHIDefinitions.h"


// This cvar allows to replace the original NVIDIA sync policy with the new Present Barrier (DX12 only).
// While in testing, it's controled by this cvar. Later on, Present Barrier policy will be exposed to
// the GUI configurator as well as other policies are.
static TAutoConsoleVariable<bool> CVarUseNvidiaPresentBarrierPolicy(
	TEXT("nDisplay.sync.nvidia.UsePresentBarrierPolicy"),
	true,
	TEXT("NVIDIA Present Barrier synchronization policy (experimental)"),
	ECVF_ReadOnly
);

TSharedPtr<IDisplayClusterRenderSyncPolicy> FDisplayClusterRenderSyncPolicyFactoryInternal::Create(const FString& InPolicyType, const TMap<FString, FString>& Parameters)
{
	const ERHIInterfaceType RHIType = RHIGetInterfaceType();

	// None
	if (InPolicyType.Equals(DisplayClusterConfigurationStrings::config::cluster::render_sync::None, ESearchCase::IgnoreCase))
	{
		return MakeShared<FDisplayClusterRenderSyncPolicyNone>(Parameters);
	}
	// EthernetBarrier
	else if (InPolicyType.Equals(DisplayClusterConfigurationStrings::config::cluster::render_sync::EthernetBarrier, ESearchCase::IgnoreCase))
	{
		return MakeShared<FDisplayClusterRenderSyncPolicyEthernetBarrier>(Parameters);
	}
	// Ethernet
	else if (InPolicyType.Equals(DisplayClusterConfigurationStrings::config::cluster::render_sync::Ethernet, ESearchCase::IgnoreCase))
	{
#if PLATFORM_WINDOWS
		if (RHIType == ERHIInterfaceType::D3D11 || RHIType == ERHIInterfaceType::D3D12)
		{
			return MakeShared<FDisplayClusterRenderSyncPolicyEthernet>(Parameters);
		}
		else if (RHIType == ERHIInterfaceType::Vulkan)
		{
			return MakeShared<FDisplayClusterRenderSyncPolicyEthernetBarrier>(Parameters);
		}
#elif PLATFORM_LINUX
		if (RHIType == ERHIInterfaceType::Vulkan)
		{
			return MakeShared<FDisplayClusterRenderSyncPolicyEthernet>(Parameters);
		}
#endif
	}
	// NVIDIA
	else if (InPolicyType.Equals(DisplayClusterConfigurationStrings::config::cluster::render_sync::Nvidia, ESearchCase::IgnoreCase))
	{
#if PLATFORM_WINDOWS
		if (RHIType == ERHIInterfaceType::D3D12 && CVarUseNvidiaPresentBarrierPolicy.GetValueOnGameThread() && WITH_NVAPI)
		{
			UE_LOGF(LogDisplayClusterRenderSync, Log, "NVIDIA Present Barrier (new) synchronization approach will be used!");
			return MakeShared<FDisplayClusterRenderSyncPolicyNvidiaPresentBarrier>(Parameters);
		}
		else if ((RHIType == ERHIInterfaceType::D3D11 || RHIType == ERHIInterfaceType::D3D12) && WITH_NVAPI)
		{
			UE_LOGF(LogDisplayClusterRenderSync, Log, "NVIDIA Swap Barrier (old) synchronization approach will be used!");
			return MakeShared<FDisplayClusterRenderSyncPolicyNvidiaSwapBarrier>(Parameters);
		}
		else if (RHIType == ERHIInterfaceType::Vulkan)
		{
			UE_LOGF(LogDisplayClusterRenderSync, Warning, "Sync policy '%ls' has not been implemented for 'Vulkan' RHI. Default '%ls' will be used.",
				DisplayClusterConfigurationStrings::config::cluster::render_sync::Nvidia,
				DisplayClusterConfigurationStrings::config::cluster::render_sync::EthernetBarrier);

			return MakeShared<FDisplayClusterRenderSyncPolicyEthernetBarrier>(Parameters);
		}
#elif PLATFORM_LINUX
		if (RHIType == ERHIInterfaceType::Vulkan)
		{
			UE_LOGF(LogDisplayClusterRenderSync, Warning, "Sync policy '%ls' has not been implemented for 'Vulkan' RHI. Default '%ls' will be used.",
				DisplayClusterConfigurationStrings::config::cluster::render_sync::Nvidia,
				DisplayClusterConfigurationStrings::config::cluster::render_sync::EthernetBarrier);

			return MakeShared<FDisplayClusterRenderSyncPolicyEthernetBarrier>(Parameters);
		}
#endif
	}

	return nullptr;
}
