// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Render/Synchronization/IDisplayClusterRenderSyncPolicyFactory.h"


/**
 * Factory for internal synchronization policies
 */
class FDisplayClusterRenderSyncPolicyFactoryInternal
	: public IDisplayClusterRenderSyncPolicyFactory
{
public:
	FDisplayClusterRenderSyncPolicyFactoryInternal() = default;
	virtual ~FDisplayClusterRenderSyncPolicyFactoryInternal() = default;

public:
	virtual TSharedPtr<IDisplayClusterRenderSyncPolicy> Create(const FString& InPolicyType, const TMap<FString, FString>& Parameters) override;
};
