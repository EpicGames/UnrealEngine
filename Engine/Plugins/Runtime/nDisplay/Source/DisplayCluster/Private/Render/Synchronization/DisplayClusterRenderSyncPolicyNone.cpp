// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Synchronization/DisplayClusterRenderSyncPolicyNone.h"

#include "Misc/DisplayClusterLog.h"
#include "DisplayClusterConfigurationStrings.h"


bool FDisplayClusterRenderSyncPolicyNone::SynchronizeClusterRendering(FRHIViewport* ViewportRHI, int32& InOutSyncInterval)
{
	// Override sync interval with 0 to show a frame ASAP. We don't care about tearing in this policy.
	InOutSyncInterval = 0;
	// Tell a caller that it still needs to present a frame
	return true;
}

FName FDisplayClusterRenderSyncPolicyNone::GetName() const
{
	static const FName NonePolicy = FName(DisplayClusterConfigurationStrings::config::cluster::render_sync::None);
	return NonePolicy;
}
