// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Render/Synchronization/DisplayClusterRenderSyncPolicyBase.h"


/**
 * Synchronization policy - None (no synchronization)
 */
class FDisplayClusterRenderSyncPolicyNone
	: public FDisplayClusterRenderSyncPolicyBase
{
public:
	FDisplayClusterRenderSyncPolicyNone(const TMap<FString, FString>& Parameters)
		: FDisplayClusterRenderSyncPolicyBase(Parameters)
	{ }

	virtual ~FDisplayClusterRenderSyncPolicyNone()
	{ }

public:
	//////////////////////////////////////////////////////////////////////////////////////////////
	// IDisplayClusterRenderSyncPolicy
	//////////////////////////////////////////////////////////////////////////////////////////////
	virtual bool SynchronizeClusterRendering(FRHIViewport* ViewportRHI, int32& InOutSyncInterval) override;
	virtual FName GetName() const override;
};
