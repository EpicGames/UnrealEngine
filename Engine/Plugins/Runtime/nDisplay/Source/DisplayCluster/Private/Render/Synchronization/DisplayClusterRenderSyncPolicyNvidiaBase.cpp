// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Synchronization/DisplayClusterRenderSyncPolicyNvidiaBase.h"

#include "Misc/DisplayClusterLog.h"

#if WITH_NVAPI
	#include "Windows/AllowWindowsPlatformTypes.h"
	THIRD_PARTY_INCLUDES_START
	#include "dxgi1_3.h"
	#include "d3d12.h"
	// NOTE: The two previous includes prevent issues in (stress) unity builds where our usage of nvapi expects __d3d12_h__ to be defined.
	#include "nvapi.h"
	THIRD_PARTY_INCLUDES_END
	#include "Windows/HideWindowsPlatformTypes.h"
#endif // WITH_NVAPI


// Enables aligning of the RHI threads on an Ethernet barrier before initializing
// NVIDIA synchronization subsystem.
static TAutoConsoleVariable<bool> CVarNvidiaSyncPreInitAlignment(
	TEXT("nDisplay.sync.nvidia.PreInitAlignment"),
	true,
	TEXT("Aligns RHI threads before initializing synchronization subsystem."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe
);

// Forces delay after joining the barrier. The idea is to give nvidia sync clients some time
// to switch to sync state before calling present. 
static TAutoConsoleVariable<float> CVarNvidiaSyncPostBarrierJoinSleep(
	TEXT("nDisplay.sync.nvidia.PostBarrierJoinSleep"),
	0.f,
	TEXT("Forces thread delay after joining the barrier\n")
	TEXT("    N <= 0 : Disabled\n")
	TEXT("    N  > 0 : Sleep time (seconds)\n"),
	ECVF_ReadOnly | ECVF_RenderThreadSafe
);

// Used to set the number of presentation cluster sync barriers before getting disabled.
// It is meant to compensate for the fact that the node may not enter the Swap Group barrier on the first frame,
// so an additional barrier may be necessary to ensure sync across the cluster.
// This CVar may become obsolete when the driver can be queried to distinguish between pending and actually
// having joined the swap group, which is not possible at the time, and the only indication of this is by looking at the
// HUD that option 8 in ConfigureDriver.exe provides.
static TAutoConsoleVariable<int32> CVarNvidiaSyncPrePresentAlignmentLimit(
	TEXT("nDisplay.sync.nvidia.PrePresentAlignmentLimit"),
	120,
	TEXT("Sets a limit to the number of times (frames) a sync barrier is enforced before calling nvidia presentation.\n")
	TEXT("Defaults to 120, which corresponds to 5s@24fps or 2s@60fps\n")
	TEXT("    N < 0 : No limit\n")
	TEXT("    N = 0 : Disabled\n")
	TEXT("    N > 0 : The presentation sync barrier will only happen N times (frames)\n"),
	ECVF_ReadOnly | ECVF_RenderThreadSafe
);

// Used to help the cluster start up in sync at the expense of performance. It forces the synchronization procedure
// to happen after frame is done on the GPU side. But it stops doing that after the specified number of frames
// to that it can run without the performance hit.
static TAutoConsoleVariable<int32> CVarNvidiaSyncFrameCompletionLimit(
	TEXT("nDisplay.sync.nvidia.FrameCompletionLimit"),
	120,
	TEXT("Sets a limit to the number of frames it waits for GPU completion.\n")
	TEXT("Defaults to 120, which corresponds to 5s@24fps or 2s@60fps\n")
	TEXT("    N < 0 : No limit\n")
	TEXT("    N = 0 : Disabled\n")
	TEXT("    N > 0 : The completion will only happen for N times (frames)\n"),
	ECVF_ReadOnly | ECVF_RenderThreadSafe
);

// Sometimes we see a weird pattern of NvAPI_D3D1x_Present() handling in the utraces. Those patterns
// visualize something that we would not expect to see if synchronization works properly.
// NvAPI_D3D1x_Present() may return in between of the vblanks, or it may return asynchronously to other nodes.
// This cvar enables additional barrier synchronization step right after NvAPI_D3D1x_Present() call.
// This should help keeping RHI threads aligned.
static TAutoConsoleVariable<bool> CVarNvidiaSyncPostPresentAlignment(
	TEXT("nDisplay.sync.nvidia.PostPresentAlignment"),
	false,
	TEXT("Sync nodes on a network barrier after frame presentation\n"),
	ECVF_ReadOnly | ECVF_RenderThreadSafe
);

// In the recent logs (Nov-Dec, 2023), we see GetMaximumFrameLatency() and SetMaximumFrameLatency() don't work
// properly. This is something weird and may indicate there is an issue outside of UE. Those methods are called
// once during the policy initialization. To better understand the issue and probably see some correlation,
// this cvar allows to focribly call set/get frame latency every frame, and log the results.
TAutoConsoleVariable<bool> CVarNvidiaSyncForceLatencyUpdateEveryFrame(
	TEXT("nDisplay.sync.nvidia.ForceLatencyUpdateEveryFrame"),
	false,
	TEXT("Performs get/set latency every frame, and logs the results"),
	ECVF_RenderThreadSafe
);

// Allows nvidia sync policy implementations to print their own statistics every frame.
TAutoConsoleVariable<bool> CVarNvidiaSyncPrintStatsEveryFrame(
	TEXT("nDisplay.sync.nvidia.ForceStatsEveryFrame"),
	false,
	TEXT("Enables sync stats every frame"),
	ECVF_RenderThreadSafe
);



FDisplayClusterRenderSyncPolicyNvidiaBase::FDisplayClusterRenderSyncPolicyNvidiaBase(const TMap<FString, FString>& Parameters)
	: Super(Parameters)
	, bCfgPreInitAlignment(CVarNvidiaSyncPreInitAlignment.GetValueOnAnyThread())
	, CfgPostBarrierJoinSleep(CVarNvidiaSyncPostBarrierJoinSleep.GetValueOnAnyThread())
	, CfgPrePresentAlignmentLimit(CVarNvidiaSyncPrePresentAlignmentLimit.GetValueOnAnyThread())
	, CfgFrameCompletionLimit(CVarNvidiaSyncFrameCompletionLimit.GetValueOnAnyThread())
	, bCfgPostPresentAlignment(CVarNvidiaSyncPostPresentAlignment.GetValueOnAnyThread())
{
#if WITH_NVAPI
	// Initialize NVIDIA library
	const NvAPI_Status NvApiResult = NvAPI_Initialize();
	if (NvApiResult == NVAPI_OK)
	{
		bNvLibraryInitialized = true;
	}
	else
	{
		UE_LOGF(LogDisplayClusterRenderSync, Error, "NvAPI_Initialize() failed, error 0x%x", NvApiResult);
	}
#endif // WITH_NVAPI

	UE_LOGF(LogDisplayClusterRenderSync, Log,
		"NVIDIA sync configuration: PreInitAlignment=%d, PostBarrierJoinSleep=%f, PrePresentAlignmentLimit=%d, FrameCompletionLimit=%d, PostPresentAlignment=%d",
		bCfgPreInitAlignment ? 1 : 0,
		CfgPostBarrierJoinSleep,
		CfgPrePresentAlignmentLimit,
		CfgFrameCompletionLimit,
		bCfgPostPresentAlignment ? 1 : 0
	);
}

FDisplayClusterRenderSyncPolicyNvidiaBase::~FDisplayClusterRenderSyncPolicyNvidiaBase()
{
	// Release NVIDIA library
	if (bNvLibraryInitialized)
	{
#if WITH_NVAPI
		NvAPI_Unload();
#endif // WITH_NVAPI
	}
}

bool FDisplayClusterRenderSyncPolicyNvidiaBase::Initialize()
{
	return bNvLibraryInitialized;
}
