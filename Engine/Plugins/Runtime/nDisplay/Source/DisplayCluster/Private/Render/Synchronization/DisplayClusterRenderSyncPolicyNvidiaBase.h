// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Render/Synchronization/DisplayClusterRenderSyncPolicyBase.h"

#include "HAL/IConsoleManager.h"


// CVars available for change in runtime
extern TAutoConsoleVariable<bool> CVarNvidiaSyncForceLatencyUpdateEveryFrame;
extern TAutoConsoleVariable<bool> CVarNvidiaSyncPrintStatsEveryFrame;


/**
 * Abstract NVIDIA synchronization policy.
 * 
 * It's used as a base class for all NVIDIA sync policies. Encapsulates common NVIDIA synchronization features.
 */
class FDisplayClusterRenderSyncPolicyNvidiaBase
	: public FDisplayClusterRenderSyncPolicyBase
{
private:
	using Super = FDisplayClusterRenderSyncPolicyBase;

public:
	FDisplayClusterRenderSyncPolicyNvidiaBase(const TMap<FString, FString>& Parameters);
	virtual ~FDisplayClusterRenderSyncPolicyNvidiaBase();

public:
	//~ Begin IDisplayClusterRenderSyncPolicy interface

	/** Initialization */
	virtual bool Initialize() override;

	//~ End IDisplayClusterRenderSyncPolicy interface

protected:
	/** Whether NVIDIA API library was initialized */
	bool bNvLibraryInitialized = false;

	/** Whether NVIDIA sync initialization function was called (to prevent multiple initialization calls) */
	bool bNvSyncInitializationCalled = false;

	/** Whether NVIDIA sync initialization succeeded */
	bool bNvSyncInitializedSuccessfully = false;

protected:
	/** CVar config: synchronous initialization (alignment on a barrier before initializing NVIDIA synchronization) */
	const bool bCfgPreInitAlignment;

	/** CVar config: Force delay after joining the barrier */
	const float CfgPostBarrierJoinSleep;

	/** CVar config: pre-present threads alignment on a network barrier */
	const int32 CfgPrePresentAlignmentLimit;
	/** Pre-present sync counter */
	int32 PrePresentAlignmentCounter = 0;

	/** CVar config: frame completion limit */
	const int32 CfgFrameCompletionLimit;
	/** Frame completion counter */
	int32 FrameCompletionCounter = 0;

	/** CVar config: post-present threads alignment on a network barrier */
	const bool bCfgPostPresentAlignment;
};
