// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Render/Synchronization/DisplayClusterRenderSyncPolicyNvidiaBase.h"
#include "DisplayClusterConfigurationStrings.h"
#include "Templates/PimplPtr.h"

/**
 * NVIDIA Present Barrier synchronization policy (new NVDIA sync approach)
 */
class FDisplayClusterRenderSyncPolicyNvidiaPresentBarrier
	: public FDisplayClusterRenderSyncPolicyNvidiaBase
{
private:
	using Super = FDisplayClusterRenderSyncPolicyNvidiaBase;

public:
	FDisplayClusterRenderSyncPolicyNvidiaPresentBarrier(const TMap<FString, FString>& Parameters);
	virtual ~FDisplayClusterRenderSyncPolicyNvidiaPresentBarrier();

	//~ Begin IDisplayClusterRenderSyncPolicy interface
public:

	/** Returns name of this policy implementation */
	virtual FName GetName() const override
	{
		return FName(DisplayClusterConfigurationStrings::config::cluster::render_sync::NvidiaPresentBarrier);
	}

	/** Performs rendering and frame output synchronization */
	virtual bool SynchronizeClusterRendering(FRHIViewport* ViewportRHI, int32& InOutSyncInterval) override;

	//~ End IDisplayClusterRenderSyncPolicy interface

private:
	/** Handles frame presentation events */
	void OnFramePresented(bool bNativePresent);

	/** Prints present barrier stats to the log */
	void LogPresentBarrierStats();

private:
	/** Initializes NVIDIA present barrier */
	bool InitializePresentBarrier(FRHIViewport* ViewportRHI);

	struct FData;
	TPimplPtr<FData> Data;
};
