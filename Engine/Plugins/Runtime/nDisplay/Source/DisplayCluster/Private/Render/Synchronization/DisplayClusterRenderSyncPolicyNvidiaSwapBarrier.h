// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Render/Synchronization/DisplayClusterRenderSyncPolicyNvidiaBase.h"
#include "DisplayClusterConfigurationStrings.h"
#include "Templates/PimplPtr.h"

/**
 * NVIDIA Swap Barrier synchronization policy (old NVDIA sync approach)
 */
class FDisplayClusterRenderSyncPolicyNvidiaSwapBarrier
	: public FDisplayClusterRenderSyncPolicyNvidiaBase
{
private:
	using Super = FDisplayClusterRenderSyncPolicyNvidiaBase;

public:
	FDisplayClusterRenderSyncPolicyNvidiaSwapBarrier(const TMap<FString, FString>& Parameters);
	virtual ~FDisplayClusterRenderSyncPolicyNvidiaSwapBarrier();

	//~ Begin IDisplayClusterRenderSyncPolicy interface
public:

	/** Returns name of this policy implementation */
	virtual FName GetName() const override
	{
		return FName(DisplayClusterConfigurationStrings::config::cluster::render_sync::Nvidia);
	}

	/** Performs rendering and frame output synchronization */
	virtual bool SynchronizeClusterRendering(FRHIViewport* ViewportRHI, int32& InOutSyncInterval) override;

	//~ End IDisplayClusterRenderSyncPolicy interface

private:
	/** Initializes NVIDIA internals */
	bool InitializeNvidiaSwapLock(FRHIViewport* ViewportRHI);

private:
	/** Policy config parameter: Swap group ID */
	uint32 RequestedGroup   = 1;

	/** Policy config parameter: Swap barrier ID */
	uint32 RequestedBarrier = 1;

	struct FData;
	TPimplPtr<FData> Data;
};
