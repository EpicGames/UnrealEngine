// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Synchronization/DisplayClusterRenderSyncHelper.h"

#include "Misc/DisplayClusterStrings.h"

#include "Engine/Engine.h"
#include "Engine/GameViewportClient.h"
#include "RHI.h"
#include "DynamicRHI.h"



TUniquePtr<IDisplayClusterRenderSyncHelper> FDisplayClusterRenderSyncHelper::CreateHelper()
{
	if (GDynamicRHI)
	{
		// Instantiate Vulkan helper
		if (RHIGetInterfaceType() == ERHIInterfaceType::Vulkan)
		{
			return MakeUnique<FDisplayClusterRenderSyncHelper::FDCRSHelperVulkan>();
		}
	}

	// Null-helper stub as fallback
	return MakeUnique<FDisplayClusterRenderSyncHelper::FDCRSHelperNull>();
}


bool FDisplayClusterRenderSyncHelper::FDCRSHelperVulkan::IsWaitForVBlankSupported()
{
	return false;
}

bool FDisplayClusterRenderSyncHelper::FDCRSHelperVulkan::WaitForVBlank(FRHIViewport* ViewportRHI)
{
	return false;
}

bool FDisplayClusterRenderSyncHelper::FDCRSHelperVulkan::GetMaximumFrameLatency(FRHIViewport* ViewportRHI, uint32& OutMaximumFrameLatency)
{
	return false;
}

bool FDisplayClusterRenderSyncHelper::FDCRSHelperVulkan::SetMaximumFrameLatency(FRHIViewport* ViewportRHI, uint32 FrameLatency)
{
	return false;
}
