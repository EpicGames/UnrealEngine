// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Synchronization/DisplayClusterRenderSyncPolicyEthernet.h"


bool FDisplayClusterRenderSyncPolicyEthernet::SynchronizeClusterRendering(FRHIViewport* ViewportRHI, int32& InOutSyncInterval)
{
	// Wait unless the frame is rendered
	WaitForFrameCompletion(ViewportRHI);
	// Sync on the network barrier
	SyncOnBarrier();
	// Ask engine to present the frame
	return true;
}

void FDisplayClusterRenderSyncPolicyEthernet::Procedure_SynchronizePresent(FRHIViewport* ViewportRHI)
{
}

void FDisplayClusterRenderSyncPolicyEthernet::Step_InitializeFrameSynchronization(FRHIViewport* ViewportRHI)
{
}

void FDisplayClusterRenderSyncPolicyEthernet::Step_WaitForFrameCompletion(FRHIViewport* ViewportRHI)
{
}

void FDisplayClusterRenderSyncPolicyEthernet::Step_WaitForEthernetBarrierSignal_1()
{
}

void FDisplayClusterRenderSyncPolicyEthernet::Step_SkipPresentationOnClosestVBlank()
{
}

void FDisplayClusterRenderSyncPolicyEthernet::Step_WaitForEthernetBarrierSignal_2()
{
}

void FDisplayClusterRenderSyncPolicyEthernet::Step_Present(FRHIViewport* ViewportRHI)
{
}

void FDisplayClusterRenderSyncPolicyEthernet::Step_FinalizeFrameSynchronization()
{
}

double FDisplayClusterRenderSyncPolicyEthernet::GetVBlankTimestamp(FRHIViewport* ViewportRHI)
{
	return 0.l;
}

double FDisplayClusterRenderSyncPolicyEthernet::GetRefreshPeriod()
{
	return 0.l;
}
