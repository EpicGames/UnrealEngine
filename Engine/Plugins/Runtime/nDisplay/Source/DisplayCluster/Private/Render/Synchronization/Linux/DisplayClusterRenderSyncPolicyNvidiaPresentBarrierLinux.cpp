// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Synchronization/DisplayClusterRenderSyncPolicyNvidiaPresentBarrier.h"


FDisplayClusterRenderSyncPolicyNvidiaPresentBarrier::FDisplayClusterRenderSyncPolicyNvidiaPresentBarrier(const TMap<FString, FString>& Parameters)
	: Super(Parameters)
{
}

FDisplayClusterRenderSyncPolicyNvidiaPresentBarrier::~FDisplayClusterRenderSyncPolicyNvidiaPresentBarrier()
{
}

bool FDisplayClusterRenderSyncPolicyNvidiaPresentBarrier::InitializePresentBarrier(FRHIViewport* ViewportRHI)
{
	return true;
}

bool FDisplayClusterRenderSyncPolicyNvidiaPresentBarrier::SynchronizeClusterRendering(FRHIViewport* ViewportRHI, int32& InOutSyncInterval)
{
	// Wait unless the frame is rendered
	WaitForFrameCompletion(ViewportRHI);
	// Sync on the network barrier
	SyncOnBarrier();
	// Ask engine to present the frame
	return true;
}

void FDisplayClusterRenderSyncPolicyNvidiaPresentBarrier::OnFramePresented(bool bNativePresent)
{
}

void FDisplayClusterRenderSyncPolicyNvidiaPresentBarrier::LogPresentBarrierStats()
{
}
