// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Synchronization/DisplayClusterRenderSyncPolicyNvidiaSwapBarrier.h"


FDisplayClusterRenderSyncPolicyNvidiaSwapBarrier::FDisplayClusterRenderSyncPolicyNvidiaSwapBarrier(const TMap<FString, FString>& Parameters)
	: Super(Parameters)
{
}

FDisplayClusterRenderSyncPolicyNvidiaSwapBarrier::~FDisplayClusterRenderSyncPolicyNvidiaSwapBarrier()
{
}

bool FDisplayClusterRenderSyncPolicyNvidiaSwapBarrier::InitializeNvidiaSwapLock(FRHIViewport* ViewportRHI)
{
	return true;
}

bool FDisplayClusterRenderSyncPolicyNvidiaSwapBarrier::SynchronizeClusterRendering(FRHIViewport* ViewportRHI, int32& InOutSyncInterval)
{
	// Wait unless the frame is rendered
	WaitForFrameCompletion(ViewportRHI);
	// Sync on the network barrier
	SyncOnBarrier();
	// Ask engine to present the frame
	return true;
}
