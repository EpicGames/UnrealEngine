// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Synchronization/DisplayClusterRenderSyncHelper.h"

#include "Misc/DisplayClusterLog.h"
#include "Misc/DisplayClusterStrings.h"

#include "Engine/Engine.h"
#include "Engine/GameViewportClient.h"
#include "RHI.h"
#include "DynamicRHI.h"
#include "UnrealClient.h"

#include "ID3D11DynamicRHI.h"
#include "ID3D12DynamicRHI.h"

#include "Windows/AllowWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include "dxgi1_3.h"
THIRD_PARTY_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"



TUniquePtr<IDisplayClusterRenderSyncHelper> FDisplayClusterRenderSyncHelper::CreateHelper()
{
	if (GDynamicRHI)
	{
		const ERHIInterfaceType RHIType = RHIGetInterfaceType();

		// Instantiate DX helper
		if (RHIType == ERHIInterfaceType::D3D11 || RHIType == ERHIInterfaceType::D3D12)
		{
			return MakeUnique<FDisplayClusterRenderSyncHelper::FDCRSHelperDX>();
		}
		// Instantiate Vulkan helper
		else if (RHIType == ERHIInterfaceType::Vulkan)
		{
			return MakeUnique<FDisplayClusterRenderSyncHelper::FDCRSHelperVulkan>();
		}
	}

	// Null-helper stub as fallback
	return MakeUnique<FDisplayClusterRenderSyncHelper::FDCRSHelperNull>();
}


bool FDisplayClusterRenderSyncHelper::FDCRSHelperDX::IsWaitForVBlankSupported()
{
	return true;
}

bool FDisplayClusterRenderSyncHelper::FDCRSHelperDX::WaitForVBlank(FRHIViewport* ViewportRHI)
{
	if (ViewportRHI == nullptr)
	{
		// Special case for FDisplayClusterVBlankMonitor until this can be refactored
		switch (RHIGetInterfaceType())
		{
		case ERHIInterfaceType::D3D11: ViewportRHI = GetID3D11DynamicRHI()->RHIGetSingletonViewport(); break;
		case ERHIInterfaceType::D3D12: ViewportRHI = GetID3D12DynamicRHI()->RHIGetSingletonViewport(); break;
		}
	}

	if (ViewportRHI)
	{
		if (IDXGISwapChain* DXGISwapChain = static_cast<IDXGISwapChain*>(ViewportRHI->GetNativeSwapChain()))
		{
			IDXGIOutput* DXOutput = nullptr;
			DXGISwapChain->GetContainingOutput(&DXOutput);

			if (DXOutput)
			{
				DXOutput->WaitForVBlank();
				DXOutput->Release();

				// Return true to notify about successful v-blank awaiting
				return true;
			}
		}
	}

	// Something went wrong. We have to let the caller know that
	// we are not at the v-blank now.
	return false;
}

bool FDisplayClusterRenderSyncHelper::FDCRSHelperDX::GetMaximumFrameLatency(FRHIViewport* ViewportRHI, uint32& OutMaximumFrameLatency)
{
	if (IDXGISwapChain2* DXGISwapChain = static_cast<IDXGISwapChain2*>(ViewportRHI->GetNativeSwapChain()))
	{
		const HRESULT Result = DXGISwapChain->GetMaximumFrameLatency(&OutMaximumFrameLatency);

		if (Result == S_OK)
		{
			UE_LOGF(LogDisplayClusterRenderSync, Verbose, "Swapchain frame latency: %u", OutMaximumFrameLatency);
		}
		else
		{
			UE_LOGF(LogDisplayClusterRenderSync, Warning, "Couldn't get maximum frame latency. Error: %x", Result);
		}

		return Result == S_OK;
	}

	return false;
}

bool FDisplayClusterRenderSyncHelper::FDCRSHelperDX::SetMaximumFrameLatency(FRHIViewport* ViewportRHI, uint32 MaximumFrameLatency)
{
	if (IDXGISwapChain2* DXGISwapChain = static_cast<IDXGISwapChain2*>(ViewportRHI->GetNativeSwapChain()))
	{
		const HRESULT Result = DXGISwapChain->SetMaximumFrameLatency(MaximumFrameLatency);

		if (Result == S_OK)
		{
			UE_LOGF(LogDisplayClusterRenderSync, Verbose, "Swapchain frame latency was set to: %u", MaximumFrameLatency);
		}
		else
		{
			UE_LOGF(LogDisplayClusterRenderSync, Warning, "Couldn't set maximum frame latency to %u. Error: %x", MaximumFrameLatency, Result);
		}

		return Result == S_OK;
	}

	return false;
}

bool FDisplayClusterRenderSyncHelper::FDCRSHelperVulkan::IsWaitForVBlankSupported()
{
	return false;
}

bool FDisplayClusterRenderSyncHelper::FDCRSHelperVulkan::WaitForVBlank(FRHIViewport* ViewportRHI)
{
	return false;
}

bool FDisplayClusterRenderSyncHelper::FDCRSHelperVulkan::GetMaximumFrameLatency(FRHIViewport* ViewportRHI, uint32& OutMaximumFrameLatency)
{
	return false;
}

bool FDisplayClusterRenderSyncHelper::FDCRSHelperVulkan::SetMaximumFrameLatency(FRHIViewport* ViewportRHI, uint32 MaximumFrameLatency)
{
	return false;
}
