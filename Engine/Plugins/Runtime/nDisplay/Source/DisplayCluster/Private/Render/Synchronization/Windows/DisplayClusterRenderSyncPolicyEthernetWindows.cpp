// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Synchronization/DisplayClusterRenderSyncPolicyEthernet.h"

#include "Misc/DisplayClusterGlobals.h"
#include "Misc/DisplayClusterLog.h"
#include "DisplayClusterCallbacks.h"
#include "DisplayClusterConfigurationStrings.h"

#include "Engine/Engine.h"
#include "Engine/GameViewportClient.h"

#include "Math/UnrealMathUtility.h"

#include "UnrealClient.h"
#include "Windows/AllowWindowsPlatformTypes.h"
#include <windows.h>
#include "DXGI.h"
#include "dwmapi.h"
#include "Windows/HideWindowsPlatformTypes.h"

#include "RHI.h"
#include "RHIResources.h"


static TAutoConsoleVariable<int32> CVarLogDwmStats(
	TEXT("nDisplay.render.softsync.LogDwmStats"),
	0,
	TEXT("Print DWM stats to log (0 = disabled)"),
	ECVF_RenderThreadSafe | ECVF_ReadOnly
);

void PrintDwmStats(uint32 FrameNum);

bool FDisplayClusterRenderSyncPolicyEthernet::SynchronizeClusterRendering(FRHIViewport* ViewportRHI, int32& InOutSyncInterval)
{
	bool bNeedEnginePresent = false;

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(nDisplay SYNC);

		if (bSimpleSync)
		{
			// Wait unless the frame is rendered
			WaitForFrameCompletion(ViewportRHI);
			// Barrier sync only
			SyncOnBarrier();
			// Let the engine present this frame
			bNeedEnginePresent = true;
		}
		else
		{
			// Run advanced synchronization procedure
			Procedure_SynchronizePresent(ViewportRHI);
			// No need to present frame, we already did that
			bNeedEnginePresent = false;
		}
	}

	if (!bNeedEnginePresent)
	{
		// Notify custom presentation was done
		GDisplayCluster->GetCallbacks().OnDisplayClusterFramePresented_RHIThread().Broadcast(false);
	}

	++FrameCounter;

	return bNeedEnginePresent;
}

void FDisplayClusterRenderSyncPolicyEthernet::Procedure_SynchronizePresent(FRHIViewport* ViewportRHI)
{
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(nDisplay SYNC: Init);

		// Initialize some internals before start sync procedure for current frame
		Step_InitializeFrameSynchronization(ViewportRHI);
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(nDisplay SYNC: frame completion);

		// Wait for all render commands to be completed. We don't want the Present() function
		// to be queued with an undefined waiting time. When a frame is rendered already,
		// the behavior of Present() depends on the frame latency and back buffers amount only.
		Step_WaitForFrameCompletion(ViewportRHI);
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(nDisplay SYNC: ethernet barrier 1);

		// At this point we know our particular node has finished rendering. But we don't know
		// if other cluster nodes have finished rendering either. To make sure all the nodes
		// have finished rendering, we need to use cluster barrier.
		Step_WaitForEthernetBarrierSignal_1();

		// At this point we 100% sure all the nodes have finished current frame rendering. Since
		// all cluster nodes have finished their frames, we're in a more safe state in terms
		// of timings before calling Present() or branching the logic.
		// However, we don't know if it's safe to call Present() because all the nodes leave
		// the barrier asynchronously, not in the same time. The main reasons of that are
		// the following:
		// 1. With TCP protocol used for cluster barriers, it's not possible to free the nodes with
		//    a broadcast packet. Because of serial networking, the cluster nodes leave the barrier
		//    ony by one in serial manner. The duration between first and last nodes left can be up
		//    to several hundred microseconds. More nodes in a cluster, the bigger duration would be.
		// 2. The duration between the moments [a socket got a message to leave cluster barrier] and
		//    [the barrier awaiting thread got CPU resource from OS task scheduler and has started
		//    running] is non-deterministic.
		// As a result, it's possible that some cluster nodes leave the barrier like several
		// microseconds before Vblank period has started, and the other ones leave the barrier during
		// or after the V-blank period. In this case we'll have a glitch.
		//
		//        V-blank (N)               V-blank(N+1)              V-blank(N+2)              V-blank(N+3)
		// _______|_________________________|_________________________|_________________________|_____
		// Timeline                         |
		//                                  | Node 1 and Node 2 are framelocked or genlocked to the same
		//                                  | source signal so they are sharing the V-blank timeline
		// _________________________________|_________________________________________________________
		// Node 1 sync thread            ^  | 
		//                                  | Node 1 left barrier K microseconds before V-blank interval
		// _________________________________|_________________________________________________________
		// Node 2 sync thread               | ^
		//                                  | Node 2 left barrier during (or after) V-blank interval
		//
		// In the example above, the node 1 will display a new frame during the V-blank N+1, while
		// the node 2 will do that on V-blank N+2 only.
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(nDisplay SYNC: skip Vblank);

		// Since it's not 100% safe to present frame now, we need to decide either we present now or
		// postpone presentation until next V-blank. Depending on the timings based math, we might
		// sleep here for some small time to skip V-blank and present frame after it.
		Step_SkipPresentationOnClosestVBlank();
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(nDisplay SYNC: ethernet barrier 2);

		// Align render threads again. Similar to the situation explained above, it's possible that one
		// cluster node is in unsafe zone (threshold) while another one is still in safe zone. To handle
		// such situation we have to synchronize the nodes on cluster barrier again. After that, all
		// render threads we'll be either before VBlank or after. The threshold, sleep, this barrier,
		// MaxFrameLatency==1 (for blocking Present calls) and pretty fast barrier related networking
		// make it very-very likely all the nodes will be on the same side of V-blank.
		Step_WaitForEthernetBarrierSignal_2();
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(nDisplay SYNC: sync present);
		// Regardless of where we are, it's safe to present a frame now.
		Step_Present(ViewportRHI);
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(nDisplay SYNC: sync finalization);
		// Finalization, logs, cleanup
		Step_FinalizeFrameSynchronization();
	}
}

void FDisplayClusterRenderSyncPolicyEthernet::Step_InitializeFrameSynchronization(FRHIViewport* ViewportRHI)
{
	// Reset to default
	SyncIntervalToUse = 1;

	if (!bInternalsInitialized)
	{
		VBlankBasis = GetVBlankTimestamp(ViewportRHI);
		RefreshPeriod = GetRefreshPeriod();

		UE_LOGF(LogDisplayClusterRenderSync, Log, "nDisplay SYNC: Refresh period:      %lf (custom=%d)", RefreshPeriod, bUseCustomRefreshRate ? 1 : 0);
		UE_LOGF(LogDisplayClusterRenderSync, Log, "nDisplay SYNC: VBlank basis:        %lf", VBlankBasis);
		UE_LOGF(LogDisplayClusterRenderSync, Log, "nDisplay SYNC: VBlank FE threshold: %lf", VBlankFrontEdgeThreshold);
		UE_LOGF(LogDisplayClusterRenderSync, Log, "nDisplay SYNC: VBlank BE threshold: %lf", VBlankBackEdgeThreshold);
		UE_LOGF(LogDisplayClusterRenderSync, Log, "nDisplay SYNC: VBlank sleep mult:   %lf", VBlankThresholdSleepMultiplier);
		UE_LOGF(LogDisplayClusterRenderSync, Log, "nDisplay SYNC: VBlank sleep:        %lf", VBlankFrontEdgeThreshold * VBlankThresholdSleepMultiplier);

		const int32 SamplesNum = 10;
		double Samples[SamplesNum] = { 0.f };

		for (int32 SampleIdx = 0; SampleIdx < SamplesNum; ++SampleIdx)
		{
			Samples[SampleIdx] = GetVBlankTimestamp(ViewportRHI);
			UE_LOGF(LogDisplayClusterRenderSync, Log, "nDisplay SYNC: VBlank Sample #%2d: %lf", SampleIdx, Samples[SampleIdx]);
		}

		for (int32 SampleIdx = 1; SampleIdx < SamplesNum; ++SampleIdx)
		{
			double Diff = Samples[SampleIdx] - Samples[SampleIdx - 1];
			UE_LOGF(LogDisplayClusterRenderSync, Log, "nDisplay SYNC: Frame time #%2d: %lfsec == %lffps", SampleIdx, Diff, 1 / Diff);
		}

		// Rise the thread priority if requested
		if (bRiseThreadPriority)
		{
			::SetThreadPriority(GetCurrentThread(), THREAD_PRIORITY_TIME_CRITICAL);
		}

		bInternalsInitialized = true;
	}
}

void FDisplayClusterRenderSyncPolicyEthernet::Step_WaitForFrameCompletion(FRHIViewport* ViewportRHI)
{
	WaitForFrameCompletion(ViewportRHI);
}

void FDisplayClusterRenderSyncPolicyEthernet::Step_WaitForEthernetBarrierSignal_1()
{
	// Align render threads with a barrier
	B1B = FPlatformTime::Seconds();
	SyncOnBarrier();
	B1A = FPlatformTime::Seconds();
}

void FDisplayClusterRenderSyncPolicyEthernet::Step_SkipPresentationOnClosestVBlank()
{
	// Here we calculate how much time left before the next VBlank
	const double CurTime = FPlatformTime::Seconds();
	const double DiffTime = CurTime - VBlankBasis;
	const double TimeRemainder = ::fmodl(DiffTime, RefreshPeriod);
	const double TimeLeftToVBlank = RefreshPeriod - TimeRemainder;
	const double TimePassedAfterVBlank = TimeRemainder;

	TToB = TimeLeftToVBlank;

	// Skip upcoming VBlank if we're in red zone
	SB = FPlatformTime::Seconds();
	if (TimeLeftToVBlank < VBlankFrontEdgeThreshold || TimePassedAfterVBlank < VBlankBackEdgeThreshold)
	{
		const double SleepTime = VBlankFrontEdgeThreshold * VBlankThresholdSleepMultiplier;
		UE_LOGF(LogDisplayClusterRenderSync, Verbose, "nDisplay SYNC: Skipping VBlank, sleeping for %f seconds", SleepTime);
		FPlatformProcess::Sleep(SleepTime);
	}
	SA = FPlatformTime::Seconds();
}

void FDisplayClusterRenderSyncPolicyEthernet::Step_WaitForEthernetBarrierSignal_2()
{
	B2B = FPlatformTime::Seconds();
	SyncOnBarrier();
	B2A = FPlatformTime::Seconds();
}

void FDisplayClusterRenderSyncPolicyEthernet::Step_Present(FRHIViewport* ViewportRHI)
{
	// If we need to update the VBlank basis, we have to wait for a VBlank and store the time. We don't want
	// to miss a frame presentation so we present it with swap interval 0 right after VBlank signal.
	SyncIntervalToUse = 1;
	if (VBlankBasisUpdate && (B2A - VBlankBasis) > VBlankBasisUpdatePeriod)
	{
		VBlankBasis = GetVBlankTimestamp(ViewportRHI);
		UE_LOGF(LogDisplayClusterRenderSync, Log, "nDisplay SYNC: - VBlank basis update. New timestamp: %lf", VBlankBasis);
		SyncIntervalToUse = 0;
	}

	IDXGISwapChain* SwapChain = static_cast<IDXGISwapChain*>(ViewportRHI->GetNativeSwapChain());

	PB = FPlatformTime::Seconds();
	SwapChain->Present(SyncIntervalToUse, 0);
	PA = FPlatformTime::Seconds();
}

void FDisplayClusterRenderSyncPolicyEthernet::Step_FinalizeFrameSynchronization()
{
	UE_LOGF(LogDisplayClusterRenderSync, Verbose, "nDisplay SYNC: - %d:%lf:%lf:%lf:%lf:%lf:%lf:%lf:%lf:%lf", FrameCounter, B1B, B1A, TToB, SB, SA, B2B, B2A, PB, PA);

	const bool LogDwmStats = (CVarLogDwmStats.GetValueOnRenderThread() != 0);
	if (LogDwmStats)
	{
		PrintDwmStats(FrameCounter);
	}
}

double FDisplayClusterRenderSyncPolicyEthernet::GetVBlankTimestamp(FRHIViewport* ViewportRHI)
{
	WaitForVBlank(ViewportRHI);

	return FPlatformTime::Seconds();
}

double FDisplayClusterRenderSyncPolicyEthernet::GetRefreshPeriod()
{
	// Sometimes the DWM returns a refresh rate value that doesn't correspond to the real system.
	// Use custom refresh rate for the synchronization algorithm below if required.
	if (bUseCustomRefreshRate)
	{
		return 1.L / FMath::Abs(CustomRefreshRate);
	}
	else
	{
		// Obtain frame interval from the DWM
		DWM_TIMING_INFO TimingInfo;
		FMemory::Memzero(TimingInfo);
		TimingInfo.cbSize = sizeof(DWM_TIMING_INFO);
		::DwmGetCompositionTimingInfo(nullptr, &TimingInfo);

		return FPlatformTime::ToSeconds(TimingInfo.qpcRefreshPeriod);
	}
}

void PrintDwmStats(uint32 FrameNum)
{
	DWM_TIMING_INFO TimingInfo;
	FMemory::Memzero(TimingInfo);
	TimingInfo.cbSize = sizeof(DWM_TIMING_INFO);
	::DwmGetCompositionTimingInfo(nullptr, &TimingInfo);

	UE_LOGF(LogDisplayClusterRenderSync, Log, "nDisplay SYNC: DWM(%d) ----------------------- DWM START", FrameNum);
	UE_LOGF(LogDisplayClusterRenderSync, Log, "nDisplay SYNC: DWM(%d) cRefresh:               %llu", FrameNum, TimingInfo.cRefresh);
	UE_LOGF(LogDisplayClusterRenderSync, Log, "nDisplay SYNC: DWM(%d) cDXRefresh:             %u", FrameNum, TimingInfo.cDXRefresh);
	UE_LOGF(LogDisplayClusterRenderSync, Log, "nDisplay SYNC: DWM(%d) qpcRefreshPeriod:       %lf", FrameNum, FPlatformTime::ToSeconds(TimingInfo.qpcRefreshPeriod));
	UE_LOGF(LogDisplayClusterRenderSync, Log, "nDisplay SYNC: DWM(%d) qpcVBlank:              %lf", FrameNum, FPlatformTime::ToSeconds(TimingInfo.qpcVBlank));
	UE_LOGF(LogDisplayClusterRenderSync, Log, "nDisplay SYNC: DWM(%d) cFrame:                 %llu", FrameNum, TimingInfo.cFrame);
	UE_LOGF(LogDisplayClusterRenderSync, Log, "nDisplay SYNC: DWM(%d) cDXPresent:             %u", FrameNum, TimingInfo.cDXPresent);
	UE_LOGF(LogDisplayClusterRenderSync, Log, "nDisplay SYNC: DWM(%d) cRefreshFrame:          %llu", FrameNum, TimingInfo.cRefreshFrame);
	UE_LOGF(LogDisplayClusterRenderSync, Log, "nDisplay SYNC: DWM(%d) cDXRefreshConfirmed:    %u", FrameNum, TimingInfo.cDXRefreshConfirmed);
	UE_LOGF(LogDisplayClusterRenderSync, Log, "nDisplay SYNC: DWM(%d) cFramesLate:            %llu", FrameNum, TimingInfo.cFramesLate);
	UE_LOGF(LogDisplayClusterRenderSync, Log, "nDisplay SYNC: DWM(%d) cFramesOutstanding:     %u", FrameNum, TimingInfo.cFramesOutstanding);
	UE_LOGF(LogDisplayClusterRenderSync, Log, "nDisplay SYNC: DWM(%d) cFrameDisplayed:        %llu", FrameNum, TimingInfo.cFrameDisplayed);
	UE_LOGF(LogDisplayClusterRenderSync, Log, "nDisplay SYNC: DWM(%d) cRefreshFrameDisplayed: %llu", FrameNum, TimingInfo.cRefreshFrameDisplayed);
	UE_LOGF(LogDisplayClusterRenderSync, Log, "nDisplay SYNC: DWM(%d) cFrameComplete:         %llu", FrameNum, TimingInfo.cFrameComplete);
	UE_LOGF(LogDisplayClusterRenderSync, Log, "nDisplay SYNC: DWM(%d) qpcFrameComplete:       %lf", FrameNum, FPlatformTime::ToSeconds(TimingInfo.qpcFrameComplete));
	UE_LOGF(LogDisplayClusterRenderSync, Log, "nDisplay SYNC: DWM(%d) cFramePending:          %llu", FrameNum, TimingInfo.cFramePending);
	UE_LOGF(LogDisplayClusterRenderSync, Log, "nDisplay SYNC: DWM(%d) qpcFramePending:        %lf", FrameNum, FPlatformTime::ToSeconds(TimingInfo.qpcFramePending));
	UE_LOGF(LogDisplayClusterRenderSync, Log, "nDisplay SYNC: DWM(%d) cFramesDisplayed:       %llu", FrameNum, TimingInfo.cFramesDisplayed);
	UE_LOGF(LogDisplayClusterRenderSync, Log, "nDisplay SYNC: DWM(%d) cFramesComplete:        %llu", FrameNum, TimingInfo.cFramesComplete);
	UE_LOGF(LogDisplayClusterRenderSync, Log, "nDisplay SYNC: DWM(%d) cFramesPending:         %llu", FrameNum, TimingInfo.cFramesPending);
	UE_LOGF(LogDisplayClusterRenderSync, Log, "nDisplay SYNC: DWM(%d) cFramesDropped:         %llu", FrameNum, TimingInfo.cFramesDropped);
	UE_LOGF(LogDisplayClusterRenderSync, Log, "nDisplay SYNC: DWM(%d) cFramesMissed:          %llu", FrameNum, TimingInfo.cFramesMissed);
	UE_LOGF(LogDisplayClusterRenderSync, Log, "nDisplay SYNC: DWM(%d) ----------------------- DWM END", FrameNum);
}
