// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Synchronization/DisplayClusterRenderSyncPolicyNvidiaPresentBarrier.h"

#include "DisplayClusterCallbacks.h"

#include "Misc/DisplayClusterGlobals.h"
#include "Misc/DisplayClusterLog.h"

#include "Engine/Engine.h"
#include "Engine/GameViewportClient.h"
#include "UnrealClient.h"

#include "Windows/AllowWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
	#include "dxgi1_3.h"
	#include "d3d12.h"
#if WITH_NVAPI
	#include "nvapi.h"
#endif
	#include <combaseapi.h>
THIRD_PARTY_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"

#include "ID3D12DynamicRHI.h"

struct FDisplayClusterRenderSyncPolicyNvidiaPresentBarrier::FData
{
	// Active output device
	ID3D12Device* D3DDevice = nullptr;

	// Active swapchain
	IDXGISwapChain2* DXGISwapChain = nullptr;

	// Synchronization fence
	TRefCountPtr<ID3D12Fence> PresentBarrierFence = nullptr;

#if WITH_NVAPI
	// Barrier client handle
	NvPresentBarrierClientHandle NvBarrierClientHandle = nullptr;
#endif

	// Backbuffer references to keep the resources alive
	TArray<TRefCountPtr<ID3D12Resource>> BackBuffers;
};

FDisplayClusterRenderSyncPolicyNvidiaPresentBarrier::FDisplayClusterRenderSyncPolicyNvidiaPresentBarrier(const TMap<FString, FString>& Parameters)
	: Super(Parameters)
	, Data(MakePimpl<FData>())
{
	GDisplayCluster->GetCallbacks().OnDisplayClusterFramePresented_RHIThread().AddRaw(this, &FDisplayClusterRenderSyncPolicyNvidiaPresentBarrier::OnFramePresented);
}

FDisplayClusterRenderSyncPolicyNvidiaPresentBarrier::~FDisplayClusterRenderSyncPolicyNvidiaPresentBarrier()
{
	GDisplayCluster->GetCallbacks().OnDisplayClusterFramePresented_RHIThread().RemoveAll(this);

#if WITH_NVAPI
	if (bNvLibraryInitialized)
	{
		// Release present barrier client
		if (Data->NvBarrierClientHandle)
		{
			NvAPI_LeavePresentBarrier(Data->NvBarrierClientHandle);
			NvAPI_DestroyPresentBarrierClient(Data->NvBarrierClientHandle);
		}

		// Release backbuffer references if there are any
		Data->BackBuffers.Empty();
	}
#endif // WITH_NVAPI
}

bool FDisplayClusterRenderSyncPolicyNvidiaPresentBarrier::SynchronizeClusterRendering(FRHIViewport* ViewportRHI, int32& InOutSyncInterval)
{
	// Initialize barriers at first call
	if (!bNvSyncInitializationCalled)
	{
		// Initialzie NVIDIA present barrier
		bNvSyncInitializedSuccessfully = InitializePresentBarrier(ViewportRHI);

		// Print initial stats before presenting any frames
		if (bNvSyncInitializedSuccessfully)
		{
			LogPresentBarrierStats();
		}
	}

	// Update maximum frame latency every if requested
	if (CVarNvidiaSyncForceLatencyUpdateEveryFrame.GetValueOnAnyThread())
	{
		SetMaximumFrameLatency(ViewportRHI, 1);
	}

	// Wait until frame rendering is finished
	if (CfgFrameCompletionLimit != 0 && FrameCompletionCounter < CfgFrameCompletionLimit)
	{
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(nDisplay FrameCompletion);
			WaitForFrameCompletion(ViewportRHI);
		}

		if (++FrameCompletionCounter == CfgFrameCompletionLimit)
		{
			UE_LOGF(LogDisplayClusterRenderSync, Log, "NVS_PB: Disabled completion after %d frames", FrameCompletionCounter);
		}
	}

	// Align all threads on the timescale before calling NvAPI_D3D1x_Present
	// As a side-effect, it should avoid NvAPI_D3D1x_Present stuck on application kill
	if (CfgPrePresentAlignmentLimit != 0 && PrePresentAlignmentCounter < CfgPrePresentAlignmentLimit)
	{
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(nDisplay PrePresentAlignment);
			SyncOnBarrier();
		}

		if (++PrePresentAlignmentCounter == CfgPrePresentAlignmentLimit)
		{
			UE_LOGF(LogDisplayClusterRenderSync, Log, "NVS_PB: Disabled PrePresent sync after %d frames", PrePresentAlignmentCounter);
		}
	}

	// This synchronization approach automatically hooks for present calls so we don't need
	// to do anything special over here. By returning true, we ask engine to present frame.
	// Present barrier works transparently.
	return true;
}

bool FDisplayClusterRenderSyncPolicyNvidiaPresentBarrier::InitializePresentBarrier(FRHIViewport* ViewportRHI)
{
	// Remember the fact this function has been called
	bNvSyncInitializationCalled = true;

	// No sense to proceed if NVIDIA library has not been initialized
	if (!bNvLibraryInitialized)
	{
		UE_LOGF(LogDisplayClusterRenderSync, Error, "NVS_PB: Couldn't initialize present barrier because NVIDIA library has not been initialized.");
		return false;
	}

	// Get D3D1XDevice
	Data->D3DDevice = GetID3D12DynamicRHI()->RHIGetDevice_NoMGPU();
	check(Data->D3DDevice);

	// Get IDXGISwapChain
	IDXGISwapChain2* DXGISwapChain = static_cast<IDXGISwapChain2*>(ViewportRHI->GetNativeSwapChain());
	check(DXGISwapChain);

	if (!Data->D3DDevice || !DXGISwapChain)
	{
		UE_LOGF(LogDisplayClusterRenderSync, Error, "NVS_PB: Couldn't get DX device or swap chain while initializing present barrier sync policy");
		return false;
	}

	// Set frame latency
	SetMaximumFrameLatency(ViewportRHI, 1);

#if WITH_NVAPI
	NvAPI_Status NVResult = NVAPI_ERROR;

	// Check if present barrier is supported
	{
		bool bSupported = false;
		NVResult = NvAPI_D3D12_QueryPresentBarrierSupport(Data->D3DDevice, &bSupported);
		if (NVResult != NVAPI_OK || !bSupported)
		{
			UE_LOGF(LogDisplayClusterRenderSync, Error, "NVS_PB: Present barrier is not supported on current device. Error=%d.", NVResult);
			return false;
		}
		else
		{
			UE_LOGF(LogDisplayClusterRenderSync, Log, "NVS_PB: Present barrier is supported");
		}
	}

	// Create present barrier client handle
	{
		NVResult = NvAPI_D3D12_CreatePresentBarrierClient(Data->D3DDevice, DXGISwapChain, &Data->NvBarrierClientHandle);
		if (NVResult != NVAPI_OK || !Data->NvBarrierClientHandle)
		{
			UE_LOGF(LogDisplayClusterRenderSync, Error, "NVS_PB: Couldn't initialize present barrier client. Error=%d.", NVResult);
			return false;
		}
		else
		{
			UE_LOGF(LogDisplayClusterRenderSync, Log, "NVS_PB: Present barrier client has been initialized");
		}
	}

	// Create sync fence
	{
		HRESULT HResult = Data->D3DDevice->CreateFence(0, D3D12_FENCE_FLAG_NONE, IID_PPV_ARGS(Data->PresentBarrierFence.GetInitReference()));
		if (FAILED(HResult))
		{
			UE_LOGF(LogDisplayClusterRenderSync, Error, "NVS_PB: Couldn't create ID3D12Fence instance. Error=%d.", HResult);
			return false;
		}
		else
		{
			UE_LOGF(LogDisplayClusterRenderSync, Log, "NVS_PB: Successfully obtained an instance of ID3D12Fence");
		}
	}

	// Register present barrier resources
	{
		// Get swap chain info (we need backbuffers amount)
		DXGI_SWAP_CHAIN_DESC SwapChainDesc;
		HRESULT HResult = DXGISwapChain->GetDesc(&SwapChainDesc);
		if (FAILED(HResult))
		{
			UE_LOGF(LogDisplayClusterRenderSync, Error, "NVS_PB: Couldn't create SwapChain info. Error=%d.", HResult);
			return false;
		}

		// Get backbuffer references
		TArray<ID3D12Resource*> BackBuffersRaw;
		BackBuffersRaw.SetNumZeroed(SwapChainDesc.BufferCount);
		Data->BackBuffers.SetNumZeroed(SwapChainDesc.BufferCount);
		for (uint32 BufferIdx = 0; BufferIdx < SwapChainDesc.BufferCount; ++BufferIdx)
		{
			DXGISwapChain->GetBuffer(BufferIdx, IID_PPV_ARGS(Data->BackBuffers[BufferIdx].GetInitReference()));
			BackBuffersRaw[BufferIdx] = Data->BackBuffers[BufferIdx];
		}

		// Register backbuffers
		NVResult = NvAPI_D3D12_RegisterPresentBarrierResources(Data->NvBarrierClientHandle, Data->PresentBarrierFence, BackBuffersRaw.GetData(), SwapChainDesc.BufferCount);
		if (NVResult != NVAPI_OK)
		{
			UE_LOGF(LogDisplayClusterRenderSync, Error, "NVS_PB: Couldn't register barrier resources. Error=%d.", NVResult);
			return false;
		}
		else
		{
			UE_LOGF(LogDisplayClusterRenderSync, Log, "NVS_PB: Barrier resources have been registered");
		}
	}

	{
		// Here we initialize NVIDIA sync on the same frame interval on the timescale
		if (bCfgPreInitAlignment)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(nDisplay NVIDIA Sync Init);

			// Align all the threads on the same timeline. It's very likely all the nodes got free on the same side of a potential
			// upcoming V-blank. Normally, the barrier synchronization takes up to 500 microseconds which is about 3% of frame time.
			// So the probability of a situation where some nodes left the barrier before V-blank while others after is really low.
			// However, it's still possible. And cluster restart should likely be successfull.
			SyncOnBarrier();

			// Assuming all the nodes in the right place and have some time before V-blank. Let's wait untill it happend.
			WaitForVBlank(ViewportRHI);

			// We don't really know either we're at the very beginning of V-blank interval or in the end. Let's spend some time
			// on the barrier to let V-blank interval over.
			SyncOnBarrier();

			// Yes, the task scheduler can break the logic above by allocating CPU resources to this thread like 20ms later which
			// makes us behind of the next V-blank for 60Hz output. And we probably should play around thread priorities to reduce
			// probability of that. But hope we're good here. Usually the systems that use NVIDIA sync approach have a lot of CPU
			// cores and don't run any other applications that compete for CPU.
			// Moreover, we will initialize NVIDIA sync for all cluster nodes before presenting first frame. So the initialization
			// process should be fine anyway.
		}
	}

	// Join the barrier
	{
		NV_JOIN_PRESENT_BARRIER_PARAMS Params = { };
		Params.dwVersion = NV_JOIN_PRESENT_BARRIER_PARAMS_VER1;
		NVResult = NvAPI_JoinPresentBarrier(Data->NvBarrierClientHandle, &Params);
		if (NVResult != NVAPI_OK)
		{
			UE_LOGF(LogDisplayClusterRenderSync, Error, "NVS_PB: Couldn't join the barrier. Error=%d.", NVResult);
			return false;
		}
		else
		{
			UE_LOGF(LogDisplayClusterRenderSync, Log, "NVS_PB: Joined the barrier successfully");
		}
	}

	// Force sleep if requested
	if(CfgPostBarrierJoinSleep > 0.f)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(nDisplay PostBarrierJoinSleep);
		FPlatformProcess::SleepNoStats(CfgPostBarrierJoinSleep);
	}

	UE_LOGF(LogDisplayClusterRenderSync, Log, "NVS_PB: Initialized successfully");

	return true;
#else
	return false;
#endif // WITH_NVAPI
}

void FDisplayClusterRenderSyncPolicyNvidiaPresentBarrier::OnFramePresented(bool bNativePresent)
{
	// Post-present alignment
	if (bCfgPostPresentAlignment)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(nDisplay PostPresentAlignment);
		SyncOnBarrier();
	}

	// Log stats
	if (CVarNvidiaSyncPrintStatsEveryFrame.GetValueOnAnyThread())
	{
		LogPresentBarrierStats();
	}
}

// A helper function to convert ID->TEXT for friendly logging
#if WITH_NVAPI
static constexpr const TCHAR* GetSyncModeName(NV_PRESENT_BARRIER_SYNC_MODE SyncMode)
{
	switch (SyncMode)
	{
	case PRESENT_BARRIER_NOT_JOINED:
		// The client hasn't joined presentBarrier
		return TEXT("PRESENT_BARRIER_NOT_JOINED");

	case PRESENT_BARRIER_SYNC_CLIENT:
		// The client joined the presentBarrier, but is not synchronized with
		// any other presentBarrier clients. This happens if the back buffers
		// of this client are composited instead of being flipped out to screen
		return TEXT("PRESENT_BARRIER_SYNC_CLIENT");

	case PRESENT_BARRIER_SYNC_SYSTEM:
		// The client joined the presentBarrier, and is synchronized with other
		// presentBarrier clients within the system
		return TEXT("PRESENT_BARRIER_SYNC_SYSTEM");

	case PRESENT_BARRIER_SYNC_CLUSTER:
		// The client joined the presentBarrier, and is synchronized with other
		// clients within the system and across systems through QSync devices
		return TEXT("PRESENT_BARRIER_SYNC_CLUSTER");

	default:
		return TEXT("UNKNOWN_STATE");
	}
}
#endif //WITH_NVAPI

void FDisplayClusterRenderSyncPolicyNvidiaPresentBarrier::LogPresentBarrierStats()
{
#if WITH_NVAPI
	if (bNvLibraryInitialized && bNvSyncInitializedSuccessfully)
	{
		NV_PRESENT_BARRIER_FRAME_STATISTICS PBStats = { };
		PBStats.dwVersion = NV_PRESENT_BARRIER_FRAME_STATICS_VER1;

		const NvAPI_Status NVResult = NvAPI_QueryPresentBarrierFrameStatistics(Data->NvBarrierClientHandle, &PBStats);
		if (NVResult != NVAPI_OK)
		{
			UE_LOGF(LogDisplayClusterRenderSync, Warning, "NVS_PB: Couldn't obtain present barrier stats");
			return;
		}

		UE_LOGF(LogDisplayClusterRenderSync, Log, "NVS_PB: Stats: SyncMode=%ls, PresentCount=%u, PresentInSyncCount=%u, FlipInSyncCount=%u, RefreshCount=%u"
			, GetSyncModeName(PBStats.SyncMode)
			, PBStats.PresentCount
			, PBStats.PresentInSyncCount
			, PBStats.FlipInSyncCount
			, PBStats.RefreshCount
			);
	}
#endif // WITH_NVAPI
}
