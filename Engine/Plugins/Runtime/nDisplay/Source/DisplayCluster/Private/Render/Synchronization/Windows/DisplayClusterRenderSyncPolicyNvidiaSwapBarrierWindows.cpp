// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Synchronization/DisplayClusterRenderSyncPolicyNvidiaSwapBarrier.h"

#include "DisplayClusterCallbacks.h"

#include "Misc/DisplayClusterGlobals.h"
#include "Misc/DisplayClusterHelpers.h"
#include "Misc/DisplayClusterLog.h"
#include "Misc/DisplayClusterTypesConverter.h"

#include "Engine/Engine.h"
#include "Engine/GameViewportClient.h"
#include "UnrealClient.h"

#include "Windows/AllowWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
	#include "dxgi1_3.h"
	#include "d3d12.h"
#if WITH_NVAPI
	#include "nvapi.h"
#endif
THIRD_PARTY_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"

#include "ID3D12DynamicRHI.h"
#include "ID3D11DynamicRHI.h"

struct FDisplayClusterRenderSyncPolicyNvidiaSwapBarrier::FData
{
	IUnknown* D3DDevice = nullptr;
	IDXGISwapChain2* DXGISwapChain = nullptr;
};

FDisplayClusterRenderSyncPolicyNvidiaSwapBarrier::FDisplayClusterRenderSyncPolicyNvidiaSwapBarrier(const TMap<FString, FString>& Parameters)
	: Super(Parameters)
	, Data(MakePimpl<FData>())
{
}

FDisplayClusterRenderSyncPolicyNvidiaSwapBarrier::~FDisplayClusterRenderSyncPolicyNvidiaSwapBarrier()
{
#if WITH_NVAPI
	if (bNvLibraryInitialized)
	{
		if (bNvSyncInitializedSuccessfully)
		{
			// Unbind from swap barrier
			NvAPI_D3D1x_BindSwapBarrier(Data->D3DDevice, RequestedGroup, 0);
			// Leave swap group
			NvAPI_D3D1x_JoinSwapGroup(Data->D3DDevice, Data->DXGISwapChain, 0, false);
		}
	}
#endif // WITH_NVAPI
}

bool FDisplayClusterRenderSyncPolicyNvidiaSwapBarrier::SynchronizeClusterRendering(FRHIViewport* ViewportRHI, int32& InOutSyncInterval)
{
	// Initialize barriers at first call
	if (!bNvSyncInitializationCalled)
	{
		// Set up NVIDIA swap barrier
		bNvSyncInitializedSuccessfully = InitializeNvidiaSwapLock(ViewportRHI);
	}

	// Check if all required objects are available
	if (!Data->D3DDevice || !Data->DXGISwapChain)
	{
		UE_LOGF(LogDisplayClusterRenderSync, Warning, "NVS_SB: Couldn't get DX resources, no swap synchronization will be performed");
		// Present frame on a higher level
		return true;
	}

	// Update maximum frame latency every if requested
	if(CVarNvidiaSyncForceLatencyUpdateEveryFrame.GetValueOnAnyThread())
	{
		SetMaximumFrameLatency(ViewportRHI, 1);
	}

	// Wait until frame rendering is finished
	if (CfgFrameCompletionLimit != 0 && FrameCompletionCounter < CfgFrameCompletionLimit)
	{
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(nDisplay FrameCompletion);
			WaitForFrameCompletion(ViewportRHI);
		}

		if (++FrameCompletionCounter == CfgFrameCompletionLimit)
		{
			UE_LOGF(LogDisplayClusterRenderSync, Log, "NVS_SB: Disabled completion after %d frames", FrameCompletionCounter);
		}
	}

	// Align all threads on the timescale before calling NvAPI_D3D1x_Present
	// As a side-effect, it should avoid NvAPI_D3D1x_Present stuck on application kill
	if (CfgPrePresentAlignmentLimit != 0 && PrePresentAlignmentCounter < CfgPrePresentAlignmentLimit)
	{
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(nDisplay PrePresentAlignment);
			SyncOnBarrier();
		}

		if (++PrePresentAlignmentCounter == CfgPrePresentAlignmentLimit)
		{
			UE_LOGF(LogDisplayClusterRenderSync, Log, "NVS_SB: Disabled PrePresent sync after %d frames", PrePresentAlignmentCounter);
		}
	}

#if WITH_NVAPI
	if (bNvSyncInitializedSuccessfully && Data->D3DDevice && Data->DXGISwapChain)
	{
		UE_LOGF(LogDisplayClusterRenderSync, VeryVerbose, "NVS_SB: presenting the frame with sync...");

		{
			NvAPI_Status NvApiResult = NVAPI_ERROR;

			{
				TRACE_CPUPROFILER_EVENT_SCOPE(nDisplay NvAPI_D3D1x_Present);

				// Present frame via NVIDIA API
				NvApiResult = NvAPI_D3D1x_Present(Data->D3DDevice, Data->DXGISwapChain, (UINT)InOutSyncInterval, (UINT)0);
			}

			// Notify custom presentation was done
			if (NvApiResult == NVAPI_OK)
			{
				GDisplayCluster->GetCallbacks().OnDisplayClusterFramePresented_RHIThread().Broadcast(false);
			}

			// Regardless of NvAPI_D3D1x_Present() call result, synchronize clients on the barrier if requested. The following
			// if-block may theoretically branch the execution paths of the nodes, so we might lose threads alignment.
			if (bCfgPostPresentAlignment)
			{
				SyncOnBarrier();
			}

			if (NvApiResult != NVAPI_OK)
			{
				UE_LOGF(LogDisplayClusterRenderSync, Warning, "NVS_SB: An error occurred during frame presentation, error code 0x%x", NvApiResult);
				// Present frame on a higher level
				return true;
			}
		}

		UE_LOGF(LogDisplayClusterRenderSync, VeryVerbose, "NVS_SB: the frame has been presented successfully");
	}
	else
	{
		UE_LOGF(LogDisplayClusterRenderSync, Warning, "NVS_SB: Can't synchronize frame presentation");
		// Something went wrong, let the upper level present this frame
		return true;
	}

	// We presented current frame so no need to present it on higher level
	return false;
#else
	// NVAPI isn't available. Ask engine to present.
	return true;
#endif // WITH_NVAPI
}

bool FDisplayClusterRenderSyncPolicyNvidiaSwapBarrier::InitializeNvidiaSwapLock(FRHIViewport* ViewportRHI)
{
	bNvSyncInitializationCalled = true;
#if WITH_NVAPI

	// Get D3D1XDevice
	Data->D3DDevice = nullptr;
	switch (RHIGetInterfaceType())
	{
	case ERHIInterfaceType::D3D12: Data->D3DDevice = GetID3D12DynamicRHI()->RHIGetDevice_NoMGPU(); break;
	case ERHIInterfaceType::D3D11: Data->D3DDevice = GetID3D11DynamicRHI()->RHIGetDevice(); break;
	}
	check(Data->D3DDevice);
	
	// Get IDXGISwapChain
	Data->DXGISwapChain = static_cast<IDXGISwapChain2*>(ViewportRHI->GetNativeSwapChain());
	check(Data->DXGISwapChain);

	if (!Data->D3DDevice || !Data->DXGISwapChain)
	{
		UE_LOGF(LogDisplayClusterRenderSync, Error, "NVS_SB: Couldn't get DX context data for NVIDIA swap barrier initialization");
		return false;
	}

	// Set frame latency
	SetMaximumFrameLatency(ViewportRHI, 1);

	NvU32 MaxGroups = 0;
	NvU32 MaxBarriers = 0;

	// Get amount of available groups and barriers
	NvAPI_Status NvApiResult = NvAPI_D3D1x_QueryMaxSwapGroup(Data->D3DDevice, &MaxGroups, &MaxBarriers);
	if (NvApiResult != NVAPI_OK)
	{
		UE_LOGF(LogDisplayClusterRenderSync, Error, "NVS_SB: Couldn't query group/barrier limits, error code 0x%x", NvApiResult);
		return false;
	}

	// Make sure resources are available
	UE_LOGF(LogDisplayClusterRenderSync, Log, "NVS_SB: max_groups=%d max_barriers=%d", (int)MaxGroups, (int)MaxBarriers);
	if (!(MaxGroups > 0 && MaxBarriers > 0))
	{
		UE_LOGF(LogDisplayClusterRenderSync, Error, "NVS_SB: No available groups or barriers");
		return false;
	}

	// Get requested barrier/group from config
	DisplayClusterHelpers::map::template ExtractValueFromString(GetParameters(), FString("SyncGroup"),   RequestedGroup);
	DisplayClusterHelpers::map::template ExtractValueFromString(GetParameters(), FString("SyncBarrier"), RequestedBarrier);

	// Here we initialize NVIDIA sync on the same frame interval on the timescale
	if (bCfgPreInitAlignment)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(nDisplay NVIDIA Sync Init);

		// Align all the threads on the same timeline. It's very likely all the nodes got free on the same side of a potential
		// upcoming V-blank. Normally, the barrier synchronization takes up to 500 microseconds which is about 3% of frame time.
		// So the probability of a situation where some nodes left the barrier before V-blank while others after is really low.
		// However, it's still possible. And cluster restart should likely be successfull.
		SyncOnBarrier();

		// Assuming all the nodes in the right place and have some time before V-blank. Let's wait untill it happend.
		WaitForVBlank(ViewportRHI);

		// We don't really know either we're at the very beginning of V-blank interval or in the end. Let's spend some time
		// on the barrier to let V-blank interval over.
		SyncOnBarrier();

		// Yes, the task scheduler can break the logic above by allocating CPU resources to this thread like 20ms later which
		// makes us behind of the next V-blank for 60Hz output. And we probably should play around thread priorities to reduce
		// probability of that. But hope we're good here. Usually the systems that use NVIDIA sync approach have a lot of CPU
		// cores and don't run any other applications that compete for CPU.
		// Moreover, we will initialize NVIDIA sync for all cluster nodes before presenting first frame. So the initialization
		// process should be fine anyway.
	}

	// Join swap group
	NvApiResult = NvAPI_D3D1x_JoinSwapGroup(Data->D3DDevice, Data->DXGISwapChain, RequestedGroup, true);
	if (NvApiResult != NVAPI_OK)
	{
		UE_LOGF(LogDisplayClusterRenderSync, Error, "NVS_SB: Couldn't join swap group %d, error code 0x%x", RequestedGroup, NvApiResult);
		return false;
	}
	else
	{
		UE_LOGF(LogDisplayClusterRenderSync, Log, "NVS_SB: Successfully joined the swap group %d", RequestedGroup);
	}

	// Bind to sync barrier
	NvApiResult = NvAPI_D3D1x_BindSwapBarrier(Data->D3DDevice, RequestedGroup, RequestedBarrier);
	if (NvApiResult != NVAPI_OK)
	{
		UE_LOGF(LogDisplayClusterRenderSync, Error, "NVS_SB: Couldn't bind group %d to swap barrier %d, error code 0x%x", RequestedGroup, RequestedBarrier, NvApiResult);
		return false;
	}
	else
	{
		UE_LOGF(LogDisplayClusterRenderSync, Log, "NVS_SB: Successfully bound group %d to the swap barrier %d", RequestedGroup, RequestedBarrier);
	}

	// Force sleep if requested
	if (CfgPostBarrierJoinSleep > 0.f)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(nDisplay PostBarrierJoinSleep);
		FPlatformProcess::SleepNoStats(CfgPostBarrierJoinSleep);
	}

	UE_LOGF(LogDisplayClusterRenderSync, Log, "NVS_SB: Initialized successfully");

	return true;
#else
	return false;
#endif // WITH_NVAPI
}
