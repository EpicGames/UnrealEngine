// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterViewportConfiguration.h"

#include "DisplayClusterViewportConfiguration_Viewport.h"
#include "DisplayClusterViewportConfiguration_ViewportManager.h"
#include "DisplayClusterViewportConfiguration_Postprocess.h"
#include "DisplayClusterViewportConfiguration_ProjectionPolicy.h"
#include "DisplayClusterViewportConfiguration_ICVFX.h"
#include "DisplayClusterViewportConfiguration_ICVFXCamera.h"
#include "DisplayClusterViewportConfiguration_Tile.h"

#include "DisplayClusterViewportConfigurationHelpers_RenderFrameSettings.h"

#include "DisplayClusterConfigurationStrings.h"
#include "DisplayClusterProjectionStrings.h"

#include "DisplayClusterRootActor.h"

#include "Render/Features/AutoExposure/DisplayClusterAutoExposure.h"

#include "Render/Viewport/DisplayClusterViewport.h"
#include "Render/Viewport/DisplayClusterViewportManager.h"
#include "Render/Viewport/DisplayClusterViewportManagerProxy.h"
#include "Render/Viewport/Postprocess/DisplayClusterViewportPostProcessManager.h"

#include "Render/Viewport/RenderFrame/DisplayClusterRenderFrameSettings.h"

#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterConfigurationTypes_Viewport.h"

#include "Render/IPDisplayClusterRenderManager.h"
#include "Misc/DisplayClusterLog.h"

#include "Engine/RendererSettings.h"

#include "Misc/DisplayClusterGlobals.h"

static bool GDisplayClusterOptimizeResourcesOnOffscreenNodes = true;
static FAutoConsoleVariableRef CVarDisplayClusterOptimizeResourcesOnOffscreenNodes(
	TEXT("nDisplay.OffscreenNodes.OptimizeResources"),
	GDisplayClusterOptimizeResourcesOnOffscreenNodes,
	TEXT("If true, optimizes resources for offscreen nDisplay nodes.\n")
	TEXT("Unused viewports will not allocate resources and will not be rendered.\n"),
	ECVF_RenderThreadSafe
);

namespace UE::DisplayCluster::Configuration
{
	static inline ADisplayClusterRootActor* ImplGetRootActor(const FDisplayClusterActorRef& InConfigurationRootActorRef)
	{
		AActor* const ActorPtr = InConfigurationRootActorRef.GetOrFindSceneActor();
		return IsValid(ActorPtr) ? Cast<ADisplayClusterRootActor>(ActorPtr) : nullptr;
	}

	static inline bool ImplIsChangedRootActor(const ADisplayClusterRootActor* InRootActor, FDisplayClusterActorRef& InOutConfigurationRootActorRef)
	{
		const bool bIsDefined = InOutConfigurationRootActorRef.IsDefinedSceneActor();
		if (InRootActor == nullptr)
		{
			return bIsDefined;
		}

		if (!bIsDefined || ImplGetRootActor(InOutConfigurationRootActorRef) != InRootActor)
		{
			return true;
		}

		return false;
	}

	static inline void ImplSetRootActor(const ADisplayClusterRootActor* InRootActor, FDisplayClusterActorRef& InOutConfigurationRootActorRef)
	{
		if (InRootActor == nullptr)
		{
			InOutConfigurationRootActorRef.ResetSceneActor();
		}
		else
		{
			InOutConfigurationRootActorRef.SetSceneActor(InRootActor);
		}
	}
};

///////////////////////////////////////////////////////////////////
// FDisplayClusterViewportConfiguration
///////////////////////////////////////////////////////////////////
FDisplayClusterViewportConfiguration::FDisplayClusterViewportConfiguration()
	: Proxy(MakeShared<FDisplayClusterViewportConfigurationProxy, ESPMode::ThreadSafe>())
{ }

FDisplayClusterViewportConfiguration::~FDisplayClusterViewportConfiguration()
{ }

void FDisplayClusterViewportConfiguration::Initialize(FDisplayClusterViewportManager& InViewportManager)
{
	// Set weak refs to viewport manager and proxy
	ViewportManagerWeakPtr = InViewportManager.AsShared();
	Proxy->Initialize_GameThread(InViewportManager.GetViewportManagerProxy());
}

void FDisplayClusterViewportConfiguration::SetRootActor(const EDisplayClusterRootActorType InRootActorType, const ADisplayClusterRootActor* InRootActor)
{
	check(IsInGameThread());

	using namespace UE::DisplayCluster::Configuration;

	// Gather all RootActor refs changes
	EDisplayClusterRootActorType RootActorChanges = (EDisplayClusterRootActorType)0;
	if (EnumHasAnyFlags(InRootActorType, EDisplayClusterRootActorType::Preview) && ImplIsChangedRootActor(InRootActor, PreviewRootActorRef))
	{
		EnumAddFlags(RootActorChanges, EDisplayClusterRootActorType::Preview);
	}
	if (EnumHasAnyFlags(InRootActorType, EDisplayClusterRootActorType::Scene) && ImplIsChangedRootActor(InRootActor, SceneRootActorRef))
	{
		EnumAddFlags(RootActorChanges, EDisplayClusterRootActorType::Scene);
	}
	if (EnumHasAnyFlags(InRootActorType, EDisplayClusterRootActorType::Configuration) && ImplIsChangedRootActor(InRootActor, ConfigurationRootActorRef))
	{
		EnumAddFlags(RootActorChanges, EDisplayClusterRootActorType::Configuration);
	}

	// Handle changes
	if (RootActorChanges != (EDisplayClusterRootActorType)0)
	{
		// Handle EndScene for current RootActor
		if (IsSceneOpened())
		{
			if (FDisplayClusterViewportManager* ViewportManager = GetViewportManagerImpl())
			{
				ViewportManager->HandleEndScene();

				// Always reset entore cluster preview rendering when RootActor chagned
				ViewportManager->GetViewportManagerPreview().ResetEntireClusterPreviewRendering();
			}
		}

		// Apply RootActor changes
		if (EnumHasAnyFlags(RootActorChanges, EDisplayClusterRootActorType::Preview))
		{
			ImplSetRootActor(InRootActor, PreviewRootActorRef);
		}
		if (EnumHasAnyFlags(RootActorChanges, EDisplayClusterRootActorType::Scene))
		{
			ImplSetRootActor(InRootActor, SceneRootActorRef);
		}
		if (EnumHasAnyFlags(RootActorChanges, EDisplayClusterRootActorType::Configuration))
		{
			ImplSetRootActor(InRootActor, ConfigurationRootActorRef);
		}
	}

	// Sync exclusive lock on RootActors
	UpdateExclusiveLockOnRootActors();
}

ADisplayClusterRootActor* FDisplayClusterViewportConfiguration::GetRootActor(const EDisplayClusterRootActorType InRootActorType) const
{
	using namespace UE::DisplayCluster::Configuration;

	if (EnumHasAnyFlags(InRootActorType, EDisplayClusterRootActorType::Preview))
	{
		if (ADisplayClusterRootActor* OutRootActor = ImplGetRootActor(PreviewRootActorRef))
		{
			// Sync exclusive lock on RootActors
			UpdateExclusiveLockOnRootActors();

			return OutRootActor;
		}
	}

	if (EnumHasAnyFlags(InRootActorType, EDisplayClusterRootActorType::Scene))
	{
		if (ADisplayClusterRootActor* OutRootActor = ImplGetRootActor(SceneRootActorRef))
		{
			// Sync exclusive lock on RootActors
			UpdateExclusiveLockOnRootActors();

			return OutRootActor;
		}
	}

	if (EnumHasAnyFlags(InRootActorType, EDisplayClusterRootActorType::Configuration))
	{
		if (ADisplayClusterRootActor* OutRootActor = ImplGetRootActor(ConfigurationRootActorRef))
		{
			// Sync exclusive lock on RootActors
			UpdateExclusiveLockOnRootActors();

			return OutRootActor;
		}
	}

	return nullptr;
}

void FDisplayClusterViewportConfiguration::UpdateExclusiveLockOnRootActorsImpl(const bool bExclusivelyLocked) const
{
	FDisplayClusterViewportManager* ViewportManagerPtr = GetViewportManagerImpl();
	if (!ViewportManagerPtr)
	{
		return;
	}

	const TSet<ADisplayClusterRootActor*> CurrentActors = GatherRootActorsImpl();
	bool bNeedsUpdate = CurrentActors.Num() != LockedRootActorsCache.Num();

	// Unlock previously locked DCRAs that are no longer used.
	for (const TWeakObjectPtr<ADisplayClusterRootActor>& CachedIt : LockedRootActorsCache)
	{
		ADisplayClusterRootActor* CachedRootActorIt = CachedIt.Get();
		if (CachedRootActorIt && CurrentActors.Contains(CachedRootActorIt))
		{
			continue;
		}

		bNeedsUpdate = true;

		if (CachedRootActorIt)
		{
			// DCRA no more used, - unlock
			CachedIt->RemoveRootActorExclusiveLockOwner(ViewportManagerPtr->ToSharedRef());
		}
	}

	if (!bNeedsUpdate && bExclusivelyLocked)
	{
		// Actor set is unchanged and we are locking: actors are already locked, nothing to do.
		return;
	}

	LockedRootActorsCache.Reset();

	for (ADisplayClusterRootActor* RootActorIt : CurrentActors)
	{
		if (bExclusivelyLocked)
		{
			// Exclusively lock the DCRA: disabling preview and PIE rendering.
			RootActorIt->AddRootActorExclusiveLockOwner(ViewportManagerPtr->ToSharedRef());
			LockedRootActorsCache.Add(RootActorIt);
		}
		else
		{
			// Release the exclusive lock so preview and PIE rendering can resume.
			RootActorIt->RemoveRootActorExclusiveLockOwner(ViewportManagerPtr->ToSharedRef());
		}
	}
}

TSet<ADisplayClusterRootActor*> FDisplayClusterViewportConfiguration::GatherRootActorsImpl() const
{
	using namespace UE::DisplayCluster::Configuration;

	// Collect all live DCRAs referenced by this configuration. TSet deduplicates in case
	// the same actor is assigned to multiple slots (Preview, Scene, Configuration).
	TSet<ADisplayClusterRootActor*> ReferencedRootActors;
	if (ADisplayClusterRootActor* RootActorPtr = ImplGetRootActor(PreviewRootActorRef))
	{
		ReferencedRootActors.Add(RootActorPtr);
	}
	if (ADisplayClusterRootActor* RootActorPtr = ImplGetRootActor(SceneRootActorRef))
	{
		ReferencedRootActors.Add(RootActorPtr);
	}
	if (ADisplayClusterRootActor* RootActorPtr = ImplGetRootActor(ConfigurationRootActorRef))
	{
		ReferencedRootActors.Add(RootActorPtr);
	}

	return ReferencedRootActors;
}

FString FDisplayClusterViewportConfiguration::GetRootActorName() const
{
	if (ADisplayClusterRootActor* RootActor = GetRootActor(EDisplayClusterRootActorType::Scene))
	{
		return RootActor->GetName();
	}

	return FString();
}

void FDisplayClusterViewportConfiguration::OnHandleStartScene()
{
	bCurrentSceneActive = true;
}

void FDisplayClusterViewportConfiguration::OnHandleEndScene()
{
	bCurrentSceneActive = false;
	MarkCurrentRenderFrameViewportsOutOfDate();

	// Always reset world ptr at the end of the scene
	CurrentWorldRef.Reset();
}

void FDisplayClusterViewportConfiguration::SetCurrentWorldImpl(const UWorld* InWorld)
{
	if (GetCurrentWorld() != InWorld)
	{
		if (IsSceneOpened())
		{
			if (FDisplayClusterViewportManager* ViewportManager = GetViewportManagerImpl())
			{
				ViewportManager->HandleEndScene();
			}
		}

		// Ignore const UWorld
		CurrentWorldRef = (UWorld*)InWorld;
	}
}

UWorld* FDisplayClusterViewportConfiguration::GetCurrentWorld() const
{
	check(IsInGameThread());

	if (!CurrentWorldRef.IsValid() || CurrentWorldRef.IsStale())
	{
		return nullptr;
	}

	return CurrentWorldRef.Get();
}

float FDisplayClusterViewportConfiguration::GetRootActorWorldDeltaSeconds(const EDisplayClusterRootActorType InRootActorType) const
{
	if (ADisplayClusterRootActor* RootActor = GetRootActor(InRootActorType))
	{
		return RootActor->GetWorldDeltaSeconds();
	}

	return 0.0f;
}

const UDisplayClusterConfigurationData* FDisplayClusterViewportConfiguration::GetConfigurationData() const
{
	ADisplayClusterRootActor* ConfigurationRootActor = GetRootActor(EDisplayClusterRootActorType::Configuration);

	return ConfigurationRootActor ? ConfigurationRootActor->GetConfigData() : nullptr;
}

const FDisplayClusterConfigurationICVFX_StageSettings* FDisplayClusterViewportConfiguration::GetStageSettings() const
{
	if (const UDisplayClusterConfigurationData* ConfigurationData = GetConfigurationData())
	{
		return &ConfigurationData->StageSettings;
	}

	return nullptr;
}

const FDisplayClusterConfigurationRenderFrame* FDisplayClusterViewportConfiguration::GetConfigurationRenderFrameSettings() const
{
	if (const UDisplayClusterConfigurationData* ConfigurationData = GetConfigurationData())
	{
		return &ConfigurationData->RenderFrameSettings;
	}

	return nullptr;
}

bool FDisplayClusterViewportConfiguration::IsCurrentWorldHasAnyType(const EWorldType::Type InWorldType1, const EWorldType::Type InWorldType2, const EWorldType::Type InWorldType3) const
{
	if (UWorld* CurrentWorld = GetCurrentWorld())
	{
		return (CurrentWorld->WorldType == InWorldType1 && InWorldType1 != EWorldType::None)
			|| (CurrentWorld->WorldType == InWorldType2 && InWorldType2 != EWorldType::None)
			|| (CurrentWorld->WorldType == InWorldType3 && InWorldType3 != EWorldType::None);
	}

	return false;
}

bool FDisplayClusterViewportConfiguration::IsRootActorWorldHasAnyType(const EDisplayClusterRootActorType InRootActorType, const EWorldType::Type InWorldType1, const EWorldType::Type InWorldType2, const EWorldType::Type InWorldType3) const
{
	if (ADisplayClusterRootActor* RootActor = GetRootActor(InRootActorType))
	{
		if (UWorld* CurrentWorld = RootActor->GetWorld())
		{
			return (CurrentWorld->WorldType == InWorldType1 && InWorldType1 != EWorldType::None)
				|| (CurrentWorld->WorldType == InWorldType2 && InWorldType2 != EWorldType::None)
				|| (CurrentWorld->WorldType == InWorldType3 && InWorldType3 != EWorldType::None);
		}
	}

	return false;
}

const float FDisplayClusterViewportConfiguration::GetWorldToMeters() const
{
	// Get world scale
	float OutWorldToMeters = 100.f;
	if (UWorld* World = GetCurrentWorld())
	{
		if (AWorldSettings* WorldSettings = World->GetWorldSettings())
		{
			OutWorldToMeters = WorldSettings->WorldToMeters;
		}
	}

	return OutWorldToMeters;
}

EDisplayClusterRenderFrameMode FDisplayClusterViewportConfiguration::GetRenderModeForPIE() const
{
#if WITH_EDITOR
	if (ADisplayClusterRootActor* PreviewRootActor = GetRootActor(EDisplayClusterRootActorType::Preview))
	{
		switch (PreviewRootActor->RenderMode)
		{
		case EDisplayClusterConfigurationRenderMode::SideBySide:
			return EDisplayClusterRenderFrameMode::PIE_SideBySide;

		case EDisplayClusterConfigurationRenderMode::TopBottom:
			return EDisplayClusterRenderFrameMode::PIE_TopBottom;

		default:
			break;
		}
	}
#endif

	return EDisplayClusterRenderFrameMode::PIE_Mono;
}

bool FDisplayClusterViewportConfiguration::IsRootActorExclusivelyLocked() const
{
#if WITH_EDITOR
	// Use the DCRA in scene to ensure that all proxies comply with its rules.
	// (ICVFX Panel, EpicStageApp, etc.)
	if (ADisplayClusterRootActor* SceneRootActor = GetRootActor(EDisplayClusterRootActorType::Scene))
	{
		// MRQ requires exclusivity: preview and proxies cannot use or modify the DCRA in scene.
		if (SceneRootActor->IsRootActorExclusivelyLocked())
		{
			return true;
		}
	}
#endif

	// MoviePipeline instances can't use preview-in-scene and PIE.
	switch (RenderFrameSettings.RenderMode)
	{
	case EDisplayClusterRenderFrameMode::MRG_Mono:
	case EDisplayClusterRenderFrameMode::MRG_Stereo:
	case EDisplayClusterRenderFrameMode::MRQ_Mono:
	case EDisplayClusterRenderFrameMode::MRQ_Stereo:
		return true;
	}

	return false;
}

bool FDisplayClusterViewportConfiguration::IsMediaAvailable() const
{
	// Media is not available in DCRA preview.
	if (IsPreviewRendering())
	{
		return false;
	}

	// Media is available only when running in cluster mode.
	const bool bIsClusterMode = GDisplayCluster->GetOperationMode() == EDisplayClusterOperationMode::Cluster;

	return bIsClusterMode;
}

bool FDisplayClusterViewportConfiguration::IsClusterNodeRenderingOffscreen() const
{
	// DCRA Preview: Determined by UDisplayClusterConfigurationClusterNode::bRenderHeadless
	if (IsPreviewRendering())
	{
		return RenderFrameSettings.CurrentNode.bRenderHeadless;
	}

	// Cluster Runtime: Determined by the "RenderOffscreen" command - line argument
	static const bool bIsRenderingOffscreen = FParse::Param(FCommandLine::Get(), TEXT("RenderOffscreen"));
	
	return bIsRenderingOffscreen;
}

bool FDisplayClusterViewportConfiguration::ShouldOptimizeOutOffscreenNodeResources() const
{
	return GDisplayClusterOptimizeResourcesOnOffscreenNodes
		// Exception: DCRA preview should ignore off-screen node optimizations.
		&& !IsPreviewRendering();
}

IDisplayClusterViewportManager* FDisplayClusterViewportConfiguration::GetViewportManager() const
{
	return GetViewportManagerImpl();
}

void FDisplayClusterViewportConfiguration::ReleaseConfiguration()
{
	if (FDisplayClusterViewportManager* ViewportManager = GetViewportManagerImpl())
	{
		ViewportManager->HandleEndScene();
		ViewportManager->ReleaseAllResources();
	}

	RenderFrameSettings = FDisplayClusterRenderFrameSettings();
}

bool FDisplayClusterViewportConfiguration::ImplUpdateConfiguration(
	EDisplayClusterRenderFrameMode InRenderMode,
	const UWorld* InWorld,
	const FString& InClusterNodeId,
	const TArray<FString>* InViewportNames,
	const uint32* InFrameNumberOverride)
{
	check(IsInGameThread());

	if (InViewportNames == nullptr && InClusterNodeId.IsEmpty())
	{
		// Requires either a cluster node ID or defined viewport names.
		return false;
	}

	if (InRenderMode == EDisplayClusterRenderFrameMode::Unknown)
	{
		// Do not initialize for unknown rendering type
		return false;
	}

	FDisplayClusterViewportManager* ViewportManager = GetViewportManagerImpl();
	if (!InWorld || !ViewportManager)
	{
		// The world is required
		return false;
	}

	// Update base settings
	if(!FDisplayClusterViewportConfigurationHelpers_RenderFrameSettings::UpdateRenderFrameConfiguration(InRenderMode, InClusterNodeId, *this))
	{
		return false;
	}

	// Override frame number
	if (InFrameNumberOverride)
	{
		RenderFrameSettings.FrameNumberOverride = *InFrameNumberOverride;
	}
	else
	{
		// by default use GFrameNumber
		RenderFrameSettings.FrameNumberOverride.Reset();
	}

	// The current world should always be initialized immediately before updating the configuration, since there is a dependency (e.g., visibility lists, etc.)
	SetCurrentWorldImpl(InWorld);

	FDisplayClusterViewportConfiguration_ViewportManager  ConfigurationViewportManager(*this);
	FDisplayClusterViewportConfiguration_Postprocess      ConfigurationPostprocess(*this);
	FDisplayClusterViewportConfiguration_ProjectionPolicy ConfigurationProjectionPolicy(*this);
	FDisplayClusterViewportConfiguration_ICVFX            ConfigurationICVFX(*this);
	FDisplayClusterViewportConfiguration_Tile             ConfigurationTile(*this);

	if (InViewportNames)
	{
		ConfigurationViewportManager.UpdateCustomViewports(*InViewportNames);
	}
	else
	{
		ConfigurationViewportManager.UpdateClusterNodeViewports(InClusterNodeId);
	}

	ConfigurationICVFX.Update();
	ConfigurationProjectionPolicy.Update();
	ConfigurationICVFX.PostUpdate();

	ImplUpdateConfigurationVisibility();

	// Tiled viewports should be created and updated at the very end, when all base viewports are already set up.
	// Because tile viewports copy configuration data from the base viewport that is used to create the tile.
	ConfigurationTile.Update();

	// Viewports created at this point, so lets handle start scene.
	if (!IsSceneOpened())
	{
		// Before render we need to start scene
		ViewportManager->HandleStartScene();
	}

	if (!InViewportNames)
	{
		// Update postprocess for current cluster node
		ConfigurationPostprocess.UpdateClusterNodePostProcess(InClusterNodeId);
	}

	FDisplayClusterViewportConfigurationHelpers_RenderFrameSettings::PostUpdateRenderFrameConfiguration(*this);

	// Apply Auto Exposure new settings
	if (const FDisplayClusterConfigurationICVFX_StageSettings* StageSettings = GetStageSettings())
	{
		ViewportManager->AutoExposure->ApplyAutoExposureSettings(StageSettings->AutoExposureSettings);
	}

	return true;
}

void FDisplayClusterViewportConfiguration::ImplUpdateConfigurationVisibility()
{
	RootActorHidePrimitivesList.Reset();

	// Hide root actor components for all viewports
	if (ADisplayClusterRootActor* SceneRootActor = GetRootActor(EDisplayClusterRootActorType::Scene))
	{
		SceneRootActor->GetHiddenInGamePrimitives(RootActorHidePrimitivesList);
	}
}

FString FDisplayClusterViewportConfiguration::BuildViewportName(
	const FString& InViewportBaseId,
	const EDisplayClusterViewportFlags InViewportFlags,
	const FIntPoint* InOptTilePos)
{
	// Regular viewport use same name
	FString UniqueName = InViewportBaseId;

	// Build unique ICVFX name
	if (EnumHasAnyFlags(InViewportFlags, EDisplayClusterViewportFlags::ICVFX_AnyRole))
	{
		if (EnumHasAllFlags(InViewportFlags, EDisplayClusterViewportFlags::ICVFX_InnerFrustum))
		{
			UniqueName = FString::Printf(TEXT("%s_icvfx_%s_incamera"), *RenderFrameSettings.CurrentNode.Id, *InViewportBaseId);
		}
		else if (EnumHasAllFlags(InViewportFlags, EDisplayClusterViewportFlags::ICVFX_ChromaKey))
		{
			UniqueName = FString::Printf(TEXT("%s_icvfx_%s_chromakey"), *RenderFrameSettings.CurrentNode.Id, *InViewportBaseId);
		}
		else if (EnumHasAnyFlags(InViewportFlags, EDisplayClusterViewportFlags::ICVFX_LightCard))
		{
			if (EnumHasAllFlags(InViewportFlags, EDisplayClusterViewportFlags::ICVFX_LightCard | EDisplayClusterViewportFlags::ICVFX_Modifier_RenderUnderInnerFrustum))
			{
				UniqueName = FString::Printf(TEXT("%s_icvfx_%s_lightcard_under"), *RenderFrameSettings.CurrentNode.Id, *InViewportBaseId);

			}
			else if (EnumHasAllFlags(InViewportFlags, EDisplayClusterViewportFlags::ICVFX_LightCard | EDisplayClusterViewportFlags::ICVFX_Modifier_RenderOverInnerFrustum))
			{
				UniqueName = FString::Printf(TEXT("%s_icvfx_%s_lightcard_over"), *RenderFrameSettings.CurrentNode.Id, *InViewportBaseId);
			}
			else
			{
				check(false); // Invalid EDisplayClusterViewportFlags flags combination: ICVFX_LightCard requires ICVFX_Modifier_RenderUnderInnerFrustum or ICVFX_Modifier_RenderOverInnerFrustum.

				UniqueName = FString::Printf(TEXT("%s_icvfx_%s_lightcard_invalid"), *RenderFrameSettings.CurrentNode.Id, *InViewportBaseId);
			}
		}
		else if (EnumHasAnyFlags(InViewportFlags, EDisplayClusterViewportFlags::ICVFX_UVLightCard))
		{
			if (EnumHasAllFlags(InViewportFlags, EDisplayClusterViewportFlags::ICVFX_UVLightCard | EDisplayClusterViewportFlags::ICVFX_Modifier_RenderUnderInnerFrustum))
			{
				UniqueName = FString::Printf(TEXT("%s_icvfx_%s_uvlightcard_under"), *RenderFrameSettings.CurrentNode.Id, *InViewportBaseId);
			}
			else if (EnumHasAllFlags(InViewportFlags, EDisplayClusterViewportFlags::ICVFX_UVLightCard | EDisplayClusterViewportFlags::ICVFX_Modifier_RenderOverInnerFrustum))
			{
				UniqueName = FString::Printf(TEXT("%s_icvfx_%s_uvlightcard_over"), *RenderFrameSettings.CurrentNode.Id, *InViewportBaseId);
			}
			else
			{
				check(false); // Invalid EDisplayClusterViewportFlags flags combination: ICVFX_UVLightCard requires ICVFX_Modifier_RenderUnderInnerFrustum or ICVFX_Modifier_RenderOverInnerFrustum.

				UniqueName = FString::Printf(TEXT("%s_icvfx_%s_uvlightcard_invalid"), *RenderFrameSettings.CurrentNode.Id, *InViewportBaseId);
			}
		}
		else
		{
			check(false); // Invalid EDisplayClusterViewportFlags flags combination: Undefined viewport type - one of LC/CK/InCamera is expected.

			UniqueName = FString::Printf(TEXT("%s_icvfx_%s_invalid"), *RenderFrameSettings.CurrentNode.Id, *InViewportBaseId);
		}
	}

	// Build the tile viewport's final unique name
	if (EnumHasAnyFlags(InViewportFlags, EDisplayClusterViewportFlags::TileViewport))
	{
		if (!InOptTilePos)
		{
			check(false); // Invalid args combination: when using EDisplayClusterViewportFlags::TileViewport, InOptTilePos must not be nullptr.

			return FString::Printf(TEXT("%s_tile_invalid"), *UniqueName);
		}
		else
		{
			return FString::Printf(TEXT("%s_tile_%d_%d"), *UniqueName, InOptTilePos->X, InOptTilePos->Y);
		}
	}

	return UniqueName;
}

TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe> FDisplayClusterViewportConfiguration::CreateProjectionPolicy_ICVFXViewport(
	const FString& InViewportBaseId,
	const EDisplayClusterViewportFlags InViewportFlags)
{
	if (!EnumHasAnyFlags(InViewportFlags, EDisplayClusterViewportFlags::ICVFX_AnyRole)
		|| EnumHasAnyFlags(InViewportFlags, EDisplayClusterViewportFlags::TileViewport))
	{
		// Create only projection policy only for ICVFX viewports
		return nullptr;
	}

	const FString& ClusterNodeId = GetClusterNodeId();
	if (ClusterNodeId.IsEmpty())
	{
		// The node name is requried
		return nullptr;
	}

	FDisplayClusterConfigurationProjection ProjectionPolicyConfig;

	// Only InCamera viewports use the 'camera' projection policy.
	// All other viewport types inherit/copy the projection policy data (use type 'link') from their parent viewport.
	ProjectionPolicyConfig.Type = EnumHasAnyFlags(InViewportFlags, EDisplayClusterViewportFlags::ICVFX_InnerFrustum)
		? DisplayClusterProjectionStrings::projection::Camera
		: DisplayClusterProjectionStrings::projection::Link;

	const FString ProjectionPolicyId = BuildViewportName(InViewportBaseId, InViewportFlags);

	// Create new projection policy for the ICVFX viewport
	const TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe>& NewProjectionPolicy =
		FDisplayClusterViewportManager::CreateProjectionPolicy(ProjectionPolicyId, &ProjectionPolicyConfig);

	if (!NewProjectionPolicy.IsValid())
	{
		UE_LOGF(LogDisplayClusterViewport, Error, "Can't create ICVFX projection policy '%ls' for node '%ls'.", *ProjectionPolicyId, *ClusterNodeId);

		return nullptr;
	}

	return NewProjectionPolicy;
}

TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe> FDisplayClusterViewportConfiguration::CreateProjectionPolicy_TileViewport(
	const FString& InViewportBaseId,
	const EDisplayClusterViewportFlags InViewportFlags,
	const FIntPoint& InTilePos)
{
	if (!EnumHasAnyFlags(InViewportFlags, EDisplayClusterViewportFlags::TileViewport))
	{
		// Create projection policy only for Tile viewports
		return nullptr;
	}

	const FString& ClusterNodeId = GetClusterNodeId();
	if (ClusterNodeId.IsEmpty())
	{
		// The node name is requried
		return nullptr;
	}

	FDisplayClusterConfigurationProjection ProjectionPolicyConfig;

	// tile viewports always use prj policy with type "link"
	ProjectionPolicyConfig.Type = DisplayClusterProjectionStrings::projection::Link;

	const FString ProjectionPolicyId = BuildViewportName(InViewportBaseId, InViewportFlags, &InTilePos);

	// Create new projection policy for the tile viewport
	const TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe>& NewProjectionPolicy =
		FDisplayClusterViewportManager::CreateProjectionPolicy(ProjectionPolicyId, &ProjectionPolicyConfig);

	if (!NewProjectionPolicy.IsValid())
	{
		UE_LOGF(LogDisplayClusterViewport, Error, "Can't create Tile viewport projection policy '%ls' for node '%ls'.", *ProjectionPolicyId, *ClusterNodeId);

		return nullptr;
	}

	return NewProjectionPolicy;
}

FDisplayClusterViewport* FDisplayClusterViewportConfiguration::CreateViewport(
	const FString& InViewportBaseId,
	const TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe>& InProjectionPolicy,
	const EDisplayClusterViewportFlags InViewportFlags,
	const FIntPoint* InOptTilePos)
{
	if (!InProjectionPolicy.IsValid())
	{
		return nullptr;
	}

	if (FDisplayClusterViewportManager* ViewportManager = GetViewportManagerImpl())
	{
		// Create new viewport
		return ViewportManager->ImplCreateViewport(InViewportBaseId, InProjectionPolicy, InViewportFlags, InOptTilePos).Get();
	}

	return nullptr;
}

FDisplayClusterViewport* FDisplayClusterViewportConfiguration::FindViewport(
	const FString& InViewportBaseId,
	const EDisplayClusterViewportFlags InViewportFlags,
	const FIntPoint* InOptTilePos) const
{
	if (InViewportBaseId.IsEmpty())
	{
		// The base ID of the viewport is mandatory.
		return nullptr;
	}

	if (FDisplayClusterViewportManager* ViewportManager = GetViewportManagerImpl())
	{
		return ViewportManager->ImplFindViewport(InViewportBaseId, InViewportFlags, InOptTilePos).Get();
	}

	return nullptr;
}
