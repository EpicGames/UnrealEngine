// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Render/Viewport/IDisplayClusterViewportConfiguration.h"
#include "Render/Viewport/Configuration/DisplayClusterViewportConfigurationProxy.h"
#include "Render/Viewport/RenderFrame/DisplayClusterRenderFrameSettings.h"

#include "Render/Viewport/Containers/DisplayClusterViewport_InternalEnums.h"

#include "SceneTypes.h"
#include "Misc/DisplayClusterObjectRef.h"
#include "Templates/SharedPointer.h"

class FDisplayClusterViewportConfigurationProxy;
class FDisplayClusterViewportManager;
class FDisplayClusterViewportManagerProxy;
class FDisplayClusterViewport; 
class IDisplayClusterProjectionPolicy;
class ADisplayClusterRootActor;
class UDisplayClusterConfigurationData;
struct FDisplayClusterConfigurationRenderFrame;

/**
* Implementation of the viewport manager configuration.
*/
class FDisplayClusterViewportConfiguration
	: public IDisplayClusterViewportConfiguration
	, public TSharedFromThis<FDisplayClusterViewportConfiguration, ESPMode::ThreadSafe>
{
public:
	FDisplayClusterViewportConfiguration();
	virtual ~FDisplayClusterViewportConfiguration();

	void Initialize(FDisplayClusterViewportManager& InViewportManager);

public:
	//~ BEGIN IDisplayClusterViewportConfiguration
	virtual void SetExclusiveLockOnRootActors(const bool bLocked) override	{ SetExclusiveLockOnRootActorsImpl(bLocked); }
	virtual bool IsRootActorLockedByThis(const ADisplayClusterRootActor* InRootActor) const override { return IsRootActorLockedByThisImpl(InRootActor); };

	virtual void SetRootActor(const EDisplayClusterRootActorType InRootActorType, const ADisplayClusterRootActor* InRootActor) override;
	
	virtual void SetPreviewSettings(const FDisplayClusterViewport_PreviewSettings& InPreviewSettings) override
	{
		RenderFrameSettings.PreviewSettings = InPreviewSettings;
	}

	virtual const FDisplayClusterViewport_PreviewSettings& GetPreviewSettings() const override
	{
		return RenderFrameSettings.PreviewSettings;
	}

	virtual void SetExternalOverscan(bool bInEnable, const float* InOverscanOverride = nullptr) override
	{
		RenderFrameSettings.bUseExternalOverscan = bInEnable;
		RenderFrameSettings.ExternalOverscanOverride = InOverscanOverride ? TOptional<float>(*InOverscanOverride) : TOptional<float>();
	}

	virtual void SetRenderResolutionScale(const float InScale) override
	{
		RenderFrameSettings.RenderResolutionScale = FMath::Clamp(InScale, 0.f, 1.f);
	}

	virtual bool UpdateConfigurationForClusterNode(EDisplayClusterRenderFrameMode InRenderMode, const UWorld* InWorld, const FString& InClusterNodeId, const uint32* InFrameNumberOverride = nullptr) override
	{
		return ImplUpdateConfiguration(InRenderMode, InWorld, InClusterNodeId, nullptr, InFrameNumberOverride);
	}

	virtual bool UpdateConfigurationForViewportsList(EDisplayClusterRenderFrameMode InRenderMode, const UWorld* InWorld, const TArray<FString>& InViewportNames, const uint32* InFrameNumberOverride = nullptr) override
	{
		return ImplUpdateConfiguration(InRenderMode, InWorld, TEXT(""), &InViewportNames, InFrameNumberOverride);
	}

	virtual void ReleaseConfiguration() override;

	virtual UWorld* GetCurrentWorld() const override;
	virtual float GetRootActorWorldDeltaSeconds(const EDisplayClusterRootActorType InRootActorType = EDisplayClusterRootActorType::Scene) const override;
	virtual ADisplayClusterRootActor* GetRootActor(const EDisplayClusterRootActorType InRootActorType) const override;
	virtual IDisplayClusterViewportManager* GetViewportManager() const override;
	virtual const UDisplayClusterConfigurationData* GetConfigurationData() const override;
	virtual const FDisplayClusterConfigurationICVFX_StageSettings* GetStageSettings() const override;
	virtual const FDisplayClusterConfigurationRenderFrame* GetConfigurationRenderFrameSettings() const  override;

	virtual EDisplayClusterRenderFrameMode GetRenderModeForPIE() const override;
	virtual bool IsCurrentWorldHasAnyType(const EWorldType::Type InWorldType1, const EWorldType::Type InWorldType2 = EWorldType::None, const EWorldType::Type InWorldType3 = EWorldType::None) const override;
	virtual bool IsRootActorWorldHasAnyType(const EDisplayClusterRootActorType InRootActorType, const EWorldType::Type InWorldType1, const EWorldType::Type InWorldType2 = EWorldType::None, const EWorldType::Type InWorldType3 = EWorldType::None) const override;

	virtual const IDisplayClusterViewportConfigurationProxy& GetProxy() const override
	{
		return *Proxy;
	}

	virtual bool IsSceneOpened() const override
	{
		return bCurrentSceneActive && GetCurrentWorld();
	}

	virtual bool IsICVFXEnabled() const override
	{
		return RenderFrameSettings.IsICVFXEnabled();
	}

	virtual bool IsClusterNodeRenderingOffscreen() const override;
	virtual bool IsMediaAvailable() const override;

	virtual const FString& GetClusterNodeId() const override
	{
		return RenderFrameSettings.CurrentNode.Id;
	}

	virtual bool IsPreviewRendering() const override
	{
		return RenderFrameSettings.IsPreviewRendering();
	}

	virtual bool IsClusterRendering() const override
	{
		return RenderFrameSettings.IsClusterRendering();
	}

	virtual bool IsMoviePipelineRendering() const override
	{
		switch (RenderFrameSettings.RenderMode)
		{
		case EDisplayClusterRenderFrameMode::MRQ_Mono:
		case EDisplayClusterRenderFrameMode::MRG_Mono:
		case EDisplayClusterRenderFrameMode::MRQ_Stereo:
		case EDisplayClusterRenderFrameMode::MRG_Stereo:
			return true;

		default:
			break;
		}

		return false;
	}

	virtual bool IsTechvisEnabled() const override
	{
		return RenderFrameSettings.IsTechvisEnabled();
	}

	virtual bool IsPreviewInGameEnabled() const override
	{
		return RenderFrameSettings.IsPreviewInGameEnabled();
	}

	virtual bool IsRootActorExclusivelyLocked() const override;

	virtual const float GetWorldToMeters() const override;

	virtual int32 GetViewPerViewportAmount() const override
	{
		return RenderFrameSettings.GetViewPerViewportAmount();
	}

	// ~~END IDisplayClusterViewportConfiguration

public:
	/**
	* Builds a viewport name that is unique within the cluster.
	*
	* The name is composed from the current cluster node ID(from this configuration),
	* the base viewport ID, the viewport usage type, and optionally a tile position
	* to disambiguate tiled viewports.
	*
	* @param InViewportBaseId Base viewport identifier.
	* @param InViewportFlags  Usage flags of the viewport (e.g. InCamera/LightCard/Tile, etc.).
	* @param InOptTilePos     Optional parameter for tile viewport position
	*
	* @return A viewport name string unique within the cluster.
	*/
	FString BuildViewportName(
		const FString& InViewportBaseId,
		const EDisplayClusterViewportFlags InViewportFlags,
		const FIntPoint* InOptTilePos = nullptr);

	/**
	* Creates a projection policy instance with type "link" or "camera" for ICVFX viewport.
	*
	* @param InViewportBaseId The base viewport identifier (before ICVFX unique naming).
	* @param InViewportFlags  Usage flags of the viewport (e.g. InCamera/LightCard/Tile, etc.).
	*
	* @return A thread-safe shared pointer to the created projection policy.
	*/
	TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe> CreateProjectionPolicy_ICVFXViewport(
		const FString& InViewportBaseId,
		const EDisplayClusterViewportFlags InViewportFlags);

	/**
	* Creates a projection policy instance with type "link" for Tile viewport.
	*
	* @param InViewportBaseId The base viewport identifier (before ICVFX unique naming).
	* @param InViewportFlags  Usage flags of the viewport (e.g. InCamera/LightCard/Tile, etc.).
	* @param InTilePos        Tile viewport position
	*
	* @return A thread-safe shared pointer to the created projection policy.
	*/
	TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe> CreateProjectionPolicy_TileViewport(
		const FString& InViewportBaseId,
		const EDisplayClusterViewportFlags InViewportFlags,
		const FIntPoint& InTilePos);

	/**
	*Creates and initializes a new nDisplay viewport.
	*
	* @param InViewportBaseId   The base viewport identifier (before ICVFX unique naming).
	* @param InProjectionPolicy Projection policy assigned to the viewport
	* @param InViewportFlags    Usage flags of the viewport (e.g. InCamera/LightCard/Tile, etc.).
	* @param InOptTilePos       Optional parameter for tile viewport position
	*
	* @return A raw pointer to the created FDisplayClusterViewport or nullptr in case of failure.
	*/
	FDisplayClusterViewport* CreateViewport(
		const FString& InViewportBaseId,
		const TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe>& InProjectionPolicy,
		const EDisplayClusterViewportFlags InViewportFlags,
		const FIntPoint* InOptTilePos = nullptr);

	/**
	* Finds a viewport instance by identifier and usage type.
	*
	* @param InViewportBaseId The base viewport identifier (before ICVFX unique naming).
	* @param InViewportFlags  Usage flags of the viewport (e.g. InCamera/LightCard/Tile, etc.).
	* @param InOptTilePos     Optional parameter for tile viewport position
	*
	* @return A pointer to the matching FDisplayClusterViewport,
	*         or nullptr if no viewport matches the given criteria.
	*/
	FDisplayClusterViewport* FindViewport(
		const FString& InViewportBaseId,
		const EDisplayClusterViewportFlags InViewportFlags = EDisplayClusterViewportFlags::None,
		const FIntPoint* InOptTilePos = nullptr) const;

public:
	/** Get a pointer to the DC ViewportManager if it still exists. */
	FDisplayClusterViewportManager* GetViewportManagerImpl() const
	{
		return ViewportManagerWeakPtr.IsValid() ? ViewportManagerWeakPtr.Pin().Get() : nullptr;
	}

	/** Gets the rendering frame settings. */
	const FDisplayClusterRenderFrameSettings& GetRenderFrameSettings() const
	{ 
		check(IsInGameThread());

		return RenderFrameSettings;
	}

	/**
	* Returns true if unused resources on offscreen nodes should be optimized.
	*/
	bool ShouldOptimizeOutOffscreenNodeResources() const;


	/** Sets the rendering frame settings. */
	void SetRenderFrameSettings(const FDisplayClusterRenderFrameSettings& InRenderFrameSettings)
	{
		check(IsInGameThread());

		// Process cluster node name changes
		SetClusterNodeId(InRenderFrameSettings.CurrentNode.Id);

		RenderFrameSettings = InRenderFrameSettings;
	}

	// Returns true if the current frame's viewport list is out of date and needs to be refreshed.
	inline bool IsCurrentRenderFrameViewportsOutOfDate() const
	{
		return bCurrentRenderFrameViewportsOutOfDate;
	}

	// Clears the stale flag, marking the current frame's viewport list as up to date.
	inline void ClearCurrentRenderFrameViewportsOutOfDate()
	{
		bCurrentRenderFrameViewportsOutOfDate = 0;
	}

	// Raises the stale flag, marking the current frame's viewport list as needing an update.
	inline void MarkCurrentRenderFrameViewportsOutOfDate()
	{
		bCurrentRenderFrameViewportsOutOfDate = 1;
	}

	/** Set the name of the current cluster node. */
	void SetClusterNodeId(const FString& InClusterNodeId)
	{
		// Tell ViewportManager to update the viewport list:
		// 1. Cluster node name changed (each node has its own viewport list)
		// 2. Empty cluster node name (use custom viewport list instead: MRQ, etc)
		if (RenderFrameSettings.CurrentNode.Id != InClusterNodeId
		|| InClusterNodeId.IsEmpty())
		{
			RenderFrameSettings.CurrentNode.Id = InClusterNodeId;
			MarkCurrentRenderFrameViewportsOutOfDate();
		}
	}

	void OnHandleStartScene();
	void OnHandleEndScene();

	/** Returns the DCRA name or an empty string. */
	FString GetRootActorName() const;

	/** Returns a list of hidden RootActor primitives. */
	const TSet<FPrimitiveComponentId>& GetRootActorHidePrimitivesList() const
	{
		return RootActorHidePrimitivesList;
	}

private:
	/** Update configuration implementation.
	* 
	* @param InRenderMode          - rendering mode.
	* @param InWorld               - ptr to the world to be rendered
	* @param InClusterNodeId       - (opt) Configuring rendering for a cluster node.
	* @param InViewportNames       - (opt) Configuring rendering for a list of viewports.
	* @param InFrameNumberOverride - (opt) If set, use this value instead of GFrameNumber.
	*/
	bool ImplUpdateConfiguration(
		EDisplayClusterRenderFrameMode InRenderMode,
		const UWorld* InWorld,
		const FString& InClusterNodeId,
		const TArray<FString>* InViewportNames,
		const uint32* InFrameNumberOverride);

	/** Hide DCRA components for nDisplay rendering.*/
	void ImplUpdateConfigurationVisibility();

	/** Set current world.*/
	void SetCurrentWorldImpl(const UWorld* InWorld);

private:
	/** Returns all DCRAs currently referenced by this configuration. */
	TSet<ADisplayClusterRootActor*> GatherRootActorsImpl() const;

	/** Applies or releases the exclusive lock on all referenced DCRAs to match bExclusiveLockOnRootActors. */
	void UpdateExclusiveLockOnRootActorsImpl(const bool bExclusivelyLocked) const;

	/**
	 * Applies the current lock state to any new or recreated DCRAs.
	 * Called from SetRootActor/GetRootActor so the lock follows actor recreation automatically.
	 *
	 * Re-applies if: (a) any previously locked actor's weak pointer expired (DCRA was destroyed/recreated),
	 * or (b) the gathered actor set contains an actor not present in the last locked set.
	 */
	inline void UpdateExclusiveLockOnRootActors() const
	{
		if (bExclusiveLockOnRootActors.IsSet())
		{
			const bool bExclusivelyLocked = bExclusiveLockOnRootActors.GetValue();
			UpdateExclusiveLockOnRootActorsImpl(bExclusivelyLocked);

			if (!bExclusivelyLocked)
			{
				// Lock released: clear the optional so UpdateExclusiveLockOnRootActors() becomes a no-op
				// for any DCRAs assigned later via SetRootActor(), until SetExclusiveLockOnRootActors() is called again.
				bExclusiveLockOnRootActors.Reset();
			}
		}
	}

	/**
	 * Applies or releases the exclusive lock on all referenced root actors.
	 * No-op if the lock state is already set to the requested value.
	 *
	 * @param bExclusivelyLocked  true to acquire the exclusive lock; false to release it.
	 */
	inline void SetExclusiveLockOnRootActorsImpl(const bool bExclusivelyLocked)
	{
		// Skip only when the state is already set to exactly the requested value.
		// Treat unset as distinct from false: an unlock request (false) while unset must still
		// call UpdateExclusiveLockOnRootActors() to release any stale locks.
		if (bExclusiveLockOnRootActors.IsSet() && bExclusiveLockOnRootActors.GetValue() == bExclusivelyLocked)
		{
			return;
		}

		bExclusiveLockOnRootActors = bExclusivelyLocked;
		UpdateExclusiveLockOnRootActors();
	}

	/**
	 * Returns true if this configuration currently holds an exclusive lock on InRootActor.
	 * Counterpart to ADisplayClusterRootActor::IsRootActorExclusivelyLocked(), which queries all its holders.
	 *
	 * @param InRootActor  The root actor to test.
	 */
	inline bool IsRootActorLockedByThisImpl(const ADisplayClusterRootActor* InRootActor) const
	{
		const bool bExclusivelyLocked = bExclusiveLockOnRootActors.IsSet() && bExclusiveLockOnRootActors.GetValue();

		return bExclusivelyLocked && GatherRootActorsImpl().Contains(InRootActor);
	}

private:
	/** Desired exclusive lock state for all referenced DCRAs.
	 *  Unset means SetExclusiveLockOnRootActors() has never been called;
	 *  true/false is applied to each DCRA via UpdateExclusiveLockOnRootActors(). */
	mutable TOptional<bool> bExclusiveLockOnRootActors;

	/** Weak pointers to the DCRAs that were locked on the last UpdateExclusiveLockOnRootActorsImpl() call.
	 *  An expired entry means the DCRA was destroyed and a re-lock is needed for its successor. */
	mutable TSet<TWeakObjectPtr<ADisplayClusterRootActor>> LockedRootActorsCache;

public:
	// Reference to configuration proxy object
	const TSharedRef<FDisplayClusterViewportConfigurationProxy, ESPMode::ThreadSafe> Proxy;

private:
	// Whether the list of viewports of the current frame needs to be updated
	uint8 bCurrentRenderFrameViewportsOutOfDate : 1 = 0;

	// Is current scene started
	uint8 bCurrentSceneActive : 1 = 0;

	// Current render frame settings
	FDisplayClusterRenderFrameSettings RenderFrameSettings;

	// A reference to the owning viewport manager
	TWeakPtr<FDisplayClusterViewportManager, ESPMode::ThreadSafe> ViewportManagerWeakPtr;

	// FDisplayClusterActorRef is not a raw pointer: it resolves the current live actor by name/reference,
	// so GatherRootActorsImpl() always returns the effective instance even if the DCRA was recreated.
	// This is what makes the symmetric lock architecture correct: UpdateExclusiveLockOnRootActors()
	// (called from SetRootActor/GetRootActor) re-applies the lock to the new instance automatically.

	// This DCRA will be used to render previews. The meshes and preview materials are created at runtime.
	FDisplayClusterActorRef PreviewRootActorRef;

	// (Optional) A reference to DCRA in the scene, used as a source for math calculations and references.
	// Locations in the scene and math data are taken from this DCRA.
	FDisplayClusterActorRef SceneRootActorRef;

	// (Optional) Reference to DCRA, used as a source of configuration data from DCRA and its components.
	FDisplayClusterActorRef ConfigurationRootActorRef;

	// Pointer to the current world to be rendered in.
	TWeakObjectPtr<UWorld> CurrentWorldRef;

	// Additional hide primitives list from root actor
	TSet<FPrimitiveComponentId> RootActorHidePrimitivesList;
};
