// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterViewportConfigurationHelpers.h"
#include "DisplayClusterViewportConfigurationHelpers_OpenColorIO.h"
#include "DisplayClusterViewportConfigurationHelpers_Postprocess.h"

#include "Render/Viewport/Configuration/DisplayClusterViewportConfiguration.h"

#include "Render/Viewport/DisplayClusterViewport.h"
#include "Render/Viewport/DisplayClusterViewportManager.h"
#include "Render/Viewport/DisplayClusterViewportHelpers.h"

#include "DisplayClusterRootActor.h"

#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterConfigurationTypes_Viewport.h"
#include "DisplayClusterConfigurationTypes_ICVFX.h"
#include "DisplayClusterConfigurationTypes_PostRender.h"

#include "IDisplayCluster.h"
#include "Cluster/IDisplayClusterClusterManager.h"

#include "ShaderParameters/DisplayClusterShaderParameters_PostprocessBlur.h"
#include "ShaderParameters/DisplayClusterShaderParameters_GenerateMips.h"
#include "ShaderParameters/DisplayClusterShaderParameters_Override.h"
#include "ShaderParameters/DisplayClusterShaderParameters_ICVFX.h"

#include "Render/Viewport/Containers/DisplayClusterViewport_RenderSettings.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_RenderSettingsICVFX.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_PostRenderSettings.h"

#include "IDisplayClusterProjection.h"
#include "Render/Projection/IDisplayClusterProjectionPolicy.h"

#include "DisplayClusterSceneViewExtensions.h"

#include "Misc/DisplayClusterLog.h"
#include "TextureResource.h"

#include "HAL/IConsoleManager.h"

bool FDisplayClusterViewportConfigurationHelpers::IsForceMonoscopicRendering(const EDisplayClusterConfigurationViewport_StereoMode StereoMode)
{
	return StereoMode == EDisplayClusterConfigurationViewport_StereoMode::ForceMono;
}

FDisplayClusterViewport_OverscanSettings FDisplayClusterViewportConfigurationHelpers::GetViewportOverscanSettings(const FDisplayClusterConfigurationViewport_Overscan& InOverscan)
{
	FDisplayClusterViewport_OverscanSettings OutOverscanSettings;

	OutOverscanSettings.bEnabled = false;
	OutOverscanSettings.bOversize = InOverscan.bOversize;

	if (InOverscan.bEnabled)
	{
		const FMarginSet OverscanMargins(
			InOverscan.Left,
			InOverscan.Right,
			InOverscan.Top,
			InOverscan.Bottom
		);

		switch (InOverscan.Mode)
		{
		case EDisplayClusterConfigurationViewportOverscanMode::Percent:
			OutOverscanSettings.bEnabled = InOverscan.bEnabled;
			OutOverscanSettings.Unit = EDisplayClusterViewport_FrustumUnit::Percent;
			
			// Scale 0..100% to 0..1 range
			OutOverscanSettings.AssignMargins(OverscanMargins * .01f);

			break;

		case EDisplayClusterConfigurationViewportOverscanMode::Pixels:
			OutOverscanSettings.bEnabled = InOverscan.bEnabled;
			OutOverscanSettings.Unit = EDisplayClusterViewport_FrustumUnit::Pixels;

			OutOverscanSettings.AssignMargins(OverscanMargins);
			break;

		default:
			break;
		}
	}

	return OutOverscanSettings;
}

void FDisplayClusterViewportConfigurationHelpers::UpdateBaseViewportSetting(FDisplayClusterViewport& DstViewport, const UDisplayClusterConfigurationViewport& InConfigurationViewport)
{
	// Gain direct access to internal settings of the viewport:
	FDisplayClusterViewport_RenderSettings&           InOutRenderSettings = DstViewport.GetRenderSettingsImpl();

	// UDisplayClusterConfigurationViewport
	{
		if (InConfigurationViewport.bAllowRendering == false)
		{
			InOutRenderSettings.bEnable = false;
		}

		InOutRenderSettings.DisplayDeviceId = InConfigurationViewport.DisplayDeviceName;

		InOutRenderSettings.CameraId = InConfigurationViewport.Camera;
		InOutRenderSettings.Rect = FDisplayClusterViewportHelpers::GetValidViewportRect(InConfigurationViewport.Region.ToRect(), DstViewport.GetId(), TEXT("Configuration Region"));

		InOutRenderSettings.bEnableCrossGPUTransfer = InConfigurationViewport.RenderSettings.bEnableCrossGPUTransfer;

		InOutRenderSettings.GPUIndex = InConfigurationViewport.GPUIndex;
		InOutRenderSettings.OverlapOrder = InConfigurationViewport.OverlapOrder;

		if (DstViewport.Configuration->GetRenderFrameSettings().bEnableAlphaOutput)
		{
			InOutRenderSettings.bWarpBlendRenderAlphaChannel = true;
		}

		// update viewport remap data
		DstViewport.UpdateConfiguration_ViewportRemap(InConfigurationViewport.ViewportRemap); 
	}

	const FDisplayClusterConfigurationViewport_RenderSettings& InRenderSettings = InConfigurationViewport.RenderSettings;

	// Update OCIO for Viewport
	FDisplayClusterViewportConfigurationHelpers_OpenColorIO::UpdateBaseViewportOCIO(DstViewport, InConfigurationViewport);

	// Additional per-viewport PostProcess
	FDisplayClusterViewportConfigurationHelpers_Postprocess::UpdateCustomPostProcessSettings(DstViewport, InRenderSettings.CustomPostprocess);
	FDisplayClusterViewportConfigurationHelpers_Postprocess::UpdatePerViewportPostProcessSettings(DstViewport);

	{
		DstViewport.SetViewportBufferRatio(InRenderSettings.BufferRatio);

		// Get default upscaler settings
		const FDisplayClusterConfigurationICVFX_StageSettings* StageSettings = DstViewport.Configuration->GetStageSettings();
		const FDisplayClusterConfigurationUpscalerSettings* DefaultUpscalerSettings = StageSettings ? &StageSettings->OuterViewportUpscalerSettings : nullptr;

		// Update viewport Upscaler method
		InRenderSettings.UpscalerSettings.GetUpscalerSettings(DefaultUpscalerSettings, InOutRenderSettings.UpscalerSettings);

		// UE-290901: Temporary fix to avoid API changes by moving this out of GetUpscalerSettings so that it only applies to outer viewport render settings, not ICVFX settings
		// TODO: Fold back into GetUpscalerSettings
		if (DefaultUpscalerSettings)
		{
			InOutRenderSettings.UpscalerSettings.CustomUpscalerSettings.MigrateToNewBagInstanceWithOverrides(DefaultUpscalerSettings->EditingData, InRenderSettings.UpscalerSettings.ParameterOverrideGuids);
		}
		
		DstViewport.UpdateConfiguration_Overscan(GetViewportOverscanSettings(InRenderSettings.Overscan));

		DstViewport.UpdateConfiguration_PostRenderOverride(InRenderSettings.Replace);
		DstViewport.UpdateConfiguration_PostRenderBlur(InRenderSettings.PostprocessBlur);
		DstViewport.UpdateConfiguration_PostRenderGenerateMips(InRenderSettings.GenerateMips);

		InOutRenderSettings.bForceMono = FDisplayClusterViewportConfigurationHelpers::IsForceMonoscopicRendering(InRenderSettings.StereoMode);

		InOutRenderSettings.StereoGPUIndex = InRenderSettings.StereoGPUIndex;
		InOutRenderSettings.RenderTargetRatio = InRenderSettings.RenderTargetRatio;
	}
}
