// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DisplayClusterConfigurationTypes_Enums.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_OverscanSettings.h"

class FDisplayClusterViewport;
class UDisplayClusterConfigurationViewport;
struct FDisplayClusterConfigurationViewport_Overscan;

/**
* Base configuration helper class.
*/
class FDisplayClusterViewportConfigurationHelpers
{
public:
	/** Returns true if this viewport should be rendered in monoscopic mode. */
	static bool IsForceMonoscopicRendering(const EDisplayClusterConfigurationViewport_StereoMode StereoMode);

	/** Update base settings of DC viewport.*/
	static void UpdateBaseViewportSetting(FDisplayClusterViewport& DstViewport, const UDisplayClusterConfigurationViewport& InConfigurationViewport);

	/** Get viewport overscan settings. */
	static FDisplayClusterViewport_OverscanSettings GetViewportOverscanSettings(const FDisplayClusterConfigurationViewport_Overscan& InOverscan);
};
