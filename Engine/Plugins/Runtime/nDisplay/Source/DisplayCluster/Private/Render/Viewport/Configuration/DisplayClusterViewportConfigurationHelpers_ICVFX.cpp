// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Viewport/Configuration/DisplayClusterViewportConfigurationHelpers_ICVFX.h"

#include "Render/Viewport/Configuration/DisplayClusterViewportConfiguration.h"
#include "Render/Viewport/Configuration/DisplayClusterViewportConfigurationHelpers.h"
#include "Render/Viewport/Configuration/DisplayClusterViewportConfigurationHelpers_OpenColorIO.h"
#include "Render/Viewport/Configuration/DisplayClusterViewportConfigurationHelpers_Postprocess.h"
#include "Render/Viewport/Configuration/DisplayClusterViewportConfigurationHelpers_Visibility.h"
#include "Render/Viewport/Configuration/DisplayClusterViewportConfigurationHelpers_Tile.h"

#include "Render/Viewport/DisplayClusterViewport.h"
#include "Render/Viewport/DisplayClusterViewportManager.h"
#include "Render/Viewport/DisplayClusterViewportHelpers.h"

#include "DisplayClusterRootActor.h"

#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterConfigurationTypes_Viewport.h"
#include "DisplayClusterConfigurationTypes_ICVFX.h"
#include "DisplayClusterConfigurationTypes_PostRender.h"

#include "IDisplayCluster.h"
#include "Cluster/IDisplayClusterClusterManager.h"

#include "ShaderParameters/DisplayClusterShaderParameters_PostprocessBlur.h"
#include "ShaderParameters/DisplayClusterShaderParameters_GenerateMips.h"
#include "ShaderParameters/DisplayClusterShaderParameters_Override.h"
#include "ShaderParameters/DisplayClusterShaderParameters_ICVFX.h"

#include "Render/Viewport/Containers/DisplayClusterViewport_RenderSettings.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_RenderSettingsICVFX.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_PostRenderSettings.h"

#include "Render/Viewport/RenderFrame/DisplayClusterRenderFrameSettings.h"
#include "Render/Viewport/LightCard/DisplayClusterViewportLightCardManager.h"

#include "Containers/DisplayClusterProjectionCameraPolicySettings.h"
#include "DisplayClusterProjectionStrings.h"

#include "Components/DisplayClusterICVFXCameraComponent.h"

#include "HAL/IConsoleManager.h"
#include "Misc/DisplayClusterLog.h"
#include "TextureResource.h"

////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterViewportConfigurationHelpers_ICVFX
////////////////////////////////////////////////////////////////////////////////
void FDisplayClusterViewportConfigurationHelpers_ICVFX::UpdateLegacyRuntimeICVFXFlagsFromViewportFlags(FDisplayClusterViewport& InOutViewport)
{
	// TODO: Deprecate and remove the duplicate flags from EDisplayClusterViewportRuntimeICVFXFlags.
	// They have been moved to EDisplayClusterViewportFlags.

	// Temporary bridge: mirror legacy ICVFX flags from EDisplayClusterViewportFlags into
	// EDisplayClusterViewportRuntimeICVFXFlags until the old flags are removed.
	FDisplayClusterViewport_RenderSettingsICVFX& RenderSettingsICVFX = InOutViewport.GetRenderSettingsICVFXImpl();
	if (EnumHasAnyFlags(InOutViewport.ViewportFlags, EDisplayClusterViewportFlags::ICVFX_InnerFrustum))
	{
		// Add viewport ICVFX usage as Incamera
		EnumAddFlags(RenderSettingsICVFX.RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::InCamera);
	}
	else if (EnumHasAnyFlags(InOutViewport.ViewportFlags, EDisplayClusterViewportFlags::ICVFX_ChromaKey))
	{
		// Add viewport ICVFX usage as Chromakey
		EnumAddFlags(RenderSettingsICVFX.RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::Chromakey);
	}
	else if (EnumHasAnyFlags(InOutViewport.ViewportFlags, EDisplayClusterViewportFlags::ICVFX_LightCard))
	{
		// Add viewport ICVFX usage as Lightcard
		EnumAddFlags(RenderSettingsICVFX.RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::Lightcard);

		// Add LC viewport position flag
		if (EnumHasAnyFlags(InOutViewport.ViewportFlags, EDisplayClusterViewportFlags::ICVFX_Modifier_RenderOverInnerFrustum))
		{
			EnumAddFlags(RenderSettingsICVFX.RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::OverInFrustum);
		}
		else if (EnumHasAnyFlags(InOutViewport.ViewportFlags, EDisplayClusterViewportFlags::ICVFX_Modifier_RenderUnderInnerFrustum))
		{
			EnumAddFlags(RenderSettingsICVFX.RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::UnderInFrustum);
		}
		else
		{
			check(false); // invalid flag settings
		}
	}
	else if (EnumHasAnyFlags(InOutViewport.ViewportFlags, EDisplayClusterViewportFlags::ICVFX_UVLightCard))
	{
		// Add viewport ICVFX usage as UVLightcard
		EnumAddFlags(RenderSettingsICVFX.RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::UVLightcard);

		// Add LC viewport position flag
		if (EnumHasAnyFlags(InOutViewport.ViewportFlags, EDisplayClusterViewportFlags::ICVFX_Modifier_RenderOverInnerFrustum))
		{
			EnumAddFlags(RenderSettingsICVFX.RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::OverInFrustum);
		}
		else if (EnumHasAnyFlags(InOutViewport.ViewportFlags, EDisplayClusterViewportFlags::ICVFX_Modifier_RenderUnderInnerFrustum))
		{
			EnumAddFlags(RenderSettingsICVFX.RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::UnderInFrustum);
		}
		else
		{
			check(false); // invalid flag settings
		}
	}
	else
	{
		check(false); // invalid flag settings
	}
}

bool FDisplayClusterViewportConfigurationHelpers_ICVFX::UpdateCameraProjectionSettingsICVFX(FDisplayClusterViewportConfiguration& InConfiguration, UDisplayClusterICVFXCameraComponent& InCameraComponent, const FDisplayClusterConfigurationICVFX_CameraSettings& InCameraSettings, const TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe>& InProjectionPolicy)
{
	const FDisplayClusterConfigurationICVFX_StageSettings* StageSettings = InConfiguration.GetStageSettings();
	if (!StageSettings)
	{
		return false;
	}

	// Initialize camera policy with camera component and settings
	FDisplayClusterProjectionCameraPolicySettings PolicyCameraSettings;
	PolicyCameraSettings.FOVMultiplier = InCameraSettings.CustomFrustum.GetCameraFieldOfViewMultiplier(*StageSettings);

	// Lens correction
	PolicyCameraSettings.FrustumRotation = InCameraSettings.FrustumRotation;
	PolicyCameraSettings.FrustumOffset = InCameraSettings.FrustumOffset;
	PolicyCameraSettings.OffCenterProjectionOffset = InCameraSettings.OffCenterProjectionOffset;

	static IDisplayClusterProjection& DisplayClusterProjectionAPI = IDisplayClusterProjection::Get();

	// Initialize camera policy with camera component and settings
	return DisplayClusterProjectionAPI.CameraPolicySetCamera(InProjectionPolicy, &InCameraComponent, PolicyCameraSettings);
}

TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe> FDisplayClusterViewportConfigurationHelpers_ICVFX::CreateProjectionPolicyCameraICVFX(FDisplayClusterViewportConfiguration& InConfiguration, UDisplayClusterICVFXCameraComponent& InCameraComponent, const FDisplayClusterConfigurationICVFX_CameraSettings& InCameraSettings)
{
	TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe> OutProjPolicy =
		InConfiguration.CreateProjectionPolicy_ICVFXViewport(InCameraComponent.GetCameraUniqueId(), EDisplayClusterViewportFlags::ICVFX_InnerFrustum);

	if (OutProjPolicy.IsValid() && UpdateCameraProjectionSettingsICVFX(InConfiguration, InCameraComponent, InCameraSettings, OutProjPolicy))
	{
		return OutProjPolicy;
	}

	return nullptr;
}

FDisplayClusterViewport* FDisplayClusterViewportConfigurationHelpers_ICVFX::GetOrCreateCameraViewport(FDisplayClusterViewportConfiguration& InConfiguration, UDisplayClusterICVFXCameraComponent& InCameraComponent, const FDisplayClusterConfigurationICVFX_CameraSettings& InCameraSettings)
{
	FDisplayClusterViewport* CameraViewport = InConfiguration.FindViewport(InCameraComponent.GetCameraUniqueId(), EDisplayClusterViewportFlags::ICVFX_InnerFrustum);
	if (!CameraViewport)
	{
		// Create viewport instance
		CameraViewport = InConfiguration.CreateViewport(
			InCameraComponent.GetCameraUniqueId(),
			CreateProjectionPolicyCameraICVFX(InConfiguration, InCameraComponent, InCameraSettings),
			EDisplayClusterViewportFlags::ICVFX_InnerFrustum);
		if(!CameraViewport)
		{
			return nullptr;
		}
	}

	// Reset runtime flags from prev frame:
	CameraViewport->ResetRuntimeParameters();

	// Gain direct access to internal resources of the CameraViewport:
	FDisplayClusterViewport_RenderSettingsICVFX& InOutRenderSettingsICVFX = CameraViewport->GetRenderSettingsICVFXImpl();

	// Mark viewport as used
	EnumRemoveFlags(InOutRenderSettingsICVFX.RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::Unused);


	return CameraViewport;
}

void FDisplayClusterViewportConfigurationHelpers_ICVFX::ReuseUVLightCardViewportWithinClusterNode(FDisplayClusterViewport& InUVLightCardViewport, const bool bOverInFrustum)
{
	if (FDisplayClusterViewportManager* ViewportManager = InUVLightCardViewport.Configuration->GetViewportManagerImpl())
	{
		// Filter lightcard viewport type
		EDisplayClusterViewportRuntimeICVFXFlags UVLightcardViewportFlags = EDisplayClusterViewportRuntimeICVFXFlags::UVLightcard;
		UVLightcardViewportFlags |= bOverInFrustum ? EDisplayClusterViewportRuntimeICVFXFlags::OverInFrustum : EDisplayClusterViewportRuntimeICVFXFlags::UnderInFrustum;

		for (const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& ViewportIt : ViewportManager->ImplGetEntireClusterViewports())
		{
			if (ViewportIt.IsValid() && ViewportIt != InUVLightCardViewport.AsShared() && !ViewportIt->GetRenderSettings().IsViewportOverridden()
				&& EnumHasAllFlags(ViewportIt->GetRenderSettingsICVFX().RuntimeFlags, UVLightcardViewportFlags))
			{
				if (ViewportIt->IsOpenColorIOEquals(InUVLightCardViewport))
				{
					// Gain direct access to internal resources of the viewport:
					FDisplayClusterViewport_RenderSettings& InOutRenderSettings = InUVLightCardViewport.GetRenderSettingsImpl();

					// Reuse exist viewport:
					InOutRenderSettings.SetViewportOverride(ViewportIt->GetId(), EDisplayClusterViewportOverrideMode::All);
					break;
				}
			}
		}
	}
}

TArray<TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>> FDisplayClusterViewportConfigurationHelpers_ICVFX::GetAllVisibleInnerCameraViewports(FDisplayClusterViewportConfiguration& InConfiguration, bool bGetChromakey)
{
	TArray<TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>> OutViewports;

	ADisplayClusterRootActor* ConfigurationRootActor = InConfiguration.GetRootActor(EDisplayClusterRootActorType::Configuration);
	const UDisplayClusterConfigurationData* ConfigurationData = InConfiguration.GetConfigurationData();
	if (ConfigurationRootActor && ConfigurationData)
	{
		TArray<UDisplayClusterICVFXCameraComponent*> ExistsICVFXCameraComponents;
		ConfigurationRootActor->GetComponents<UDisplayClusterICVFXCameraComponent>(ExistsICVFXCameraComponents);

		for (UDisplayClusterICVFXCameraComponent* CfgCineCameraComponent : ExistsICVFXCameraComponents)
		{
			if (CfgCineCameraComponent && CfgCineCameraComponent->GetCameraSettingsICVFX().IsICVFXEnabled(*ConfigurationData, InConfiguration.GetClusterNodeId()))
			{
				const FString& CameraName = CfgCineCameraComponent->GetCameraUniqueId();

				OutViewports.Append(FDisplayClusterViewportConfigurationHelpers_ICVFX::PreviewGetRenderedInCameraViewports(InConfiguration, CameraName, bGetChromakey));
			}
		}
	}

	return OutViewports;
}

TArray<TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>> FDisplayClusterViewportConfigurationHelpers_ICVFX::PreviewGetRenderedInCameraViewports(FDisplayClusterViewportConfiguration& InConfiguration, const FString& InICVFXCameraId, bool bGetChromakey)
{
	TArray<TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>> OutViewports;

	// Filter viewports by usage flags
	const EDisplayClusterViewportFlags ExpectedViewportFlags = bGetChromakey
		? EDisplayClusterViewportFlags::ICVFX_ChromaKey
		: EDisplayClusterViewportFlags::ICVFX_InnerFrustum;

	if (FDisplayClusterViewportManager* ViewportManager = InConfiguration.GetViewportManagerImpl())
	{
		for (const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& ViewportIt : ViewportManager->ImplGetEntireClusterViewports())
		{
			if (ViewportIt.IsValid()
				&& EnumHasAnyFlags(ViewportIt->ViewportFlags, ExpectedViewportFlags)
				&& ( ViewportIt->GetViewportResources(EDisplayClusterViewportResource::InputShaderResources).Num() > 0
					&& ViewportIt->GetViewportResources(EDisplayClusterViewportResource::InputShaderResources)[0] != nullptr
					&& ViewportIt->GetContexts().Num() > 0)
				&& (!ViewportIt->GetRenderSettings().IsViewportOverridden()))
			{
				// This is InCamera based viewport. Check by the base name.
				if (ViewportIt->ViewportBaseId == InICVFXCameraId)
				{
					OutViewports.Add(ViewportIt);
				}
			}
		}
	}

	return OutViewports;
}

bool FDisplayClusterViewportConfigurationHelpers_ICVFX::PreviewReuseInnerFrustumViewportWithinClusterNodes(FDisplayClusterViewport& InCameraViewport, UDisplayClusterICVFXCameraComponent& InCameraComponent, const FDisplayClusterConfigurationICVFX_CameraSettings& InCameraSettings)
{
	if (!InCameraViewport.Configuration->GetRenderFrameSettings().CanReuseViewportWithinClusterNodes())
	{
		return false;
	}

	// If the InCamera viewport has already been rendered for another node,
	// use it as a source to avoid rendering for each node in the cluster.
	if (FDisplayClusterViewportManager* ViewportManager = InCameraViewport.Configuration->GetViewportManagerImpl())
	{
		for (const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& ViewportIt : PreviewGetRenderedInCameraViewports(*InCameraViewport.Configuration, InCameraComponent.GetCameraUniqueId(), false))
		{
			if (ViewportIt.IsValid() && ViewportIt != InCameraViewport.AsShared() && ViewportIt->GetClusterNodeId() != InCameraViewport.GetClusterNodeId()
				&& FDisplayClusterViewportConfigurationHelpers_Postprocess::IsInnerFrustumViewportSettingsEqual(*ViewportIt, InCameraViewport, InCameraSettings))
			{
				const EDisplayClusterViewportOverrideMode ViewportOverrideMode =
					ViewportIt->IsOpenColorIOEquals(InCameraViewport)
					? EDisplayClusterViewportOverrideMode::All
					: EDisplayClusterViewportOverrideMode::InternalRTT;

				// Gain direct access to internal resources of the viewport:
				FDisplayClusterViewport_RenderSettings& InOutRenderSettings = InCameraViewport.GetRenderSettingsImpl();

				// Reuse exist viewport:
				InOutRenderSettings.SetViewportOverride(ViewportIt->GetId(), ViewportOverrideMode);

				return true;
			}
		}
	}

	return false;
}

bool FDisplayClusterViewportConfigurationHelpers_ICVFX::PreviewReuseChromakeyViewportWithinClusterNodes(FDisplayClusterViewport& InChromakeyViewport, const FString& InICVFXCameraId)
{
	if (!InChromakeyViewport.Configuration->GetRenderFrameSettings().CanReuseViewportWithinClusterNodes())
	{
		return false;
	}
	
	// If the InCamera chromakey viewport has already been rendered for another node,
	// use it as a source to avoid rendering for each node in the cluster.
	if (FDisplayClusterViewportManager* ViewportManager = InChromakeyViewport.Configuration->GetViewportManagerImpl())
	{
		for (const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& ViewportIt : PreviewGetRenderedInCameraViewports(*InChromakeyViewport.Configuration, InICVFXCameraId, true))
		{
			if (ViewportIt.IsValid() && ViewportIt != InChromakeyViewport.AsShared() && ViewportIt->GetClusterNodeId() != InChromakeyViewport.GetClusterNodeId())
			{
				// Chromakey support OCIO
				const EDisplayClusterViewportOverrideMode ViewportOverrideMode = 
					ViewportIt->IsOpenColorIOEquals(InChromakeyViewport)
					? EDisplayClusterViewportOverrideMode::All
					: EDisplayClusterViewportOverrideMode::InternalRTT;

				// Gain direct access to internal resources of the viewport:
				FDisplayClusterViewport_RenderSettings& InOutRenderSettings = InChromakeyViewport.GetRenderSettingsImpl();

				// Reuse exist viewport from other node
				InOutRenderSettings.SetViewportOverride(ViewportIt->GetId(), ViewportOverrideMode);

				return true;
			}
		}
	}

	return false;
}

FDisplayClusterViewport* FDisplayClusterViewportConfigurationHelpers_ICVFX::GetOrCreateChromakeyViewport(FDisplayClusterViewportConfiguration& InConfiguration, UDisplayClusterICVFXCameraComponent& InCameraComponent, const FDisplayClusterConfigurationICVFX_CameraSettings& InCameraSettings)
{
	const FString InICVFXCameraId = InCameraComponent.GetCameraUniqueId();
	FDisplayClusterViewport* ChromakeyViewport = InConfiguration.FindViewport(InICVFXCameraId, EDisplayClusterViewportFlags::ICVFX_ChromaKey);
	if (ChromakeyViewport == nullptr)
	{
		ChromakeyViewport = InConfiguration.CreateViewport(
			InICVFXCameraId,
			InConfiguration.CreateProjectionPolicy_ICVFXViewport(InICVFXCameraId, EDisplayClusterViewportFlags::ICVFX_ChromaKey),
			EDisplayClusterViewportFlags::ICVFX_ChromaKey);
		if (ChromakeyViewport == nullptr)
		{
			return nullptr;
		}
	}

	// Reset runtime flags from prev frame:
	ChromakeyViewport->ResetRuntimeParameters();

	// Gain direct access to internal resources of the viewport:
	FDisplayClusterViewport_RenderSettingsICVFX& InOutRenderSettingsICVFX = ChromakeyViewport->GetRenderSettingsICVFXImpl();

	// Mark viewport as used
	EnumRemoveFlags(InOutRenderSettingsICVFX.RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::Unused);

	return ChromakeyViewport;
}

FDisplayClusterViewport* FDisplayClusterViewportConfigurationHelpers_ICVFX::GetOrCreateLightcardViewport(FDisplayClusterViewport& BaseViewport, const bool bOverInFrustum)
{
	// Create new lightcard viewport
	const EDisplayClusterViewportFlags ViewportFlags= bOverInFrustum
		? EDisplayClusterViewportFlags::ICVFX_LightCard | EDisplayClusterViewportFlags::ICVFX_Modifier_RenderOverInnerFrustum
		: EDisplayClusterViewportFlags::ICVFX_LightCard | EDisplayClusterViewportFlags::ICVFX_Modifier_RenderUnderInnerFrustum;

	FDisplayClusterViewport* LightcardViewport = BaseViewport.Configuration->FindViewport(BaseViewport.GetId(), ViewportFlags);
	if (LightcardViewport == nullptr)
	{
		LightcardViewport = BaseViewport.Configuration->CreateViewport(
			BaseViewport.GetId(),
			BaseViewport.Configuration->CreateProjectionPolicy_ICVFXViewport(BaseViewport.GetId(), ViewportFlags),
			ViewportFlags);

		if (LightcardViewport == nullptr)
		{
			return nullptr;
		}
	}

	// Reset runtime flags from prev frame:
	LightcardViewport->ResetRuntimeParameters(BaseViewport.GetViewportConfigurationData());

	// Also updates the reference to the configuration data of the base viewport.
	LightcardViewport->SetViewportConfigurationData(BaseViewport.GetViewportConfigurationData());

	// Gain direct access to internal resources of the viewport:
	FDisplayClusterViewport_RenderSettingsICVFX& InOutRenderSettingsICVFX = LightcardViewport->GetRenderSettingsICVFXImpl();

	// Mark viewport as used
	EnumRemoveFlags(InOutRenderSettingsICVFX.RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::Unused);

	return LightcardViewport;
}

FDisplayClusterViewport* FDisplayClusterViewportConfigurationHelpers_ICVFX::GetOrCreateUVLightcardViewport(FDisplayClusterViewport& BaseViewport, const bool bOverInFrustum)
{
	// Create new UV lightcard viewport
	const EDisplayClusterViewportFlags ViewportFlags = bOverInFrustum
		? EDisplayClusterViewportFlags::ICVFX_UVLightCard | EDisplayClusterViewportFlags::ICVFX_Modifier_RenderOverInnerFrustum
		: EDisplayClusterViewportFlags::ICVFX_UVLightCard | EDisplayClusterViewportFlags::ICVFX_Modifier_RenderUnderInnerFrustum;

	FDisplayClusterViewport* UVLightcardViewport = BaseViewport.Configuration->FindViewport(BaseViewport.GetId(), ViewportFlags);
	if (UVLightcardViewport == nullptr)
	{
		UVLightcardViewport = BaseViewport.Configuration->CreateViewport(
			BaseViewport.GetId(),
			BaseViewport.Configuration->CreateProjectionPolicy_ICVFXViewport(BaseViewport.GetId(), ViewportFlags),
			ViewportFlags);
		
		if (UVLightcardViewport == nullptr)
		{
			return nullptr;
		}
	}

	// Also updates the reference to the configuration data of the base viewport.
	UVLightcardViewport->SetViewportConfigurationData(BaseViewport.GetViewportConfigurationData());

	// Gain direct access to internal resources of the viewport:
	FDisplayClusterViewport_RenderSettingsICVFX& InOutRenderSettingsICVFX = UVLightcardViewport->GetRenderSettingsICVFXImpl();

	// Mark viewport as used
	EnumRemoveFlags(InOutRenderSettingsICVFX.RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::Unused);

	return UVLightcardViewport;
}

bool FDisplayClusterViewportConfigurationHelpers_ICVFX::IsCameraUsed(const FDisplayClusterConfigurationICVFX_CameraSettings& InCameraSettings)
{
	// Check rules for camera settings:
	if (InCameraSettings.bEnable == false)
	{
		// don't use camera if disabled
		return false;
	}

	if (InCameraSettings.RenderSettings.Replace.bAllowReplace && InCameraSettings.RenderSettings.Replace.SourceTexture == nullptr)
	{
		// RenderSettings.Override require source texture
		return false;
	}

	return true;
}

void FDisplayClusterViewportConfigurationHelpers_ICVFX::UpdateCameraViewportSettings(FDisplayClusterViewport& DstViewport, UDisplayClusterICVFXCameraComponent& InCameraComponent, const FDisplayClusterConfigurationICVFX_CameraSettings& InCameraSettings)
{
	const FDisplayClusterConfigurationICVFX_StageSettings* StageSettings = DstViewport.Configuration->GetStageSettings();
	if (!StageSettings)
	{
		return;
	}

	// Gain direct access to internal resources of the viewport:
	FDisplayClusterViewport_RenderSettings& InOutRenderSettings = DstViewport.GetRenderSettingsImpl();

	// The InCamera viewport is used via an overlay (not displayed directly on the backbuffer).
	InOutRenderSettings.bVisible = false;

	// Update OCIO for Camera Viewport
	FDisplayClusterViewportConfigurationHelpers_OpenColorIO::UpdateCameraViewportOCIO(DstViewport, InCameraSettings);

	// FDisplayClusterConfigurationICVFX_CameraSettings
	InOutRenderSettings.CameraId.Empty();

	// UDisplayClusterConfigurationICVFX_CameraRenderSettings
	const FIntPoint DesiredSize = InCameraComponent.GetICVFXCameraFrameSize(*StageSettings, InCameraSettings);

	InOutRenderSettings.Rect = FDisplayClusterViewportHelpers::GetValidViewportRect(FIntRect(FIntPoint(0, 0), DesiredSize), DstViewport.GetId(), TEXT("Configuration Camera Frame Size"));

	// Apply postprocess for ICVFX camera
	InCameraComponent.ApplyICVFXCameraPostProcessesToViewport(&DstViewport, EDisplayClusterViewportCameraPostProcessFlags::All);

	DstViewport.UpdateConfiguration_PostRenderOverride(InCameraSettings.RenderSettings.Replace);
	DstViewport.UpdateConfiguration_PostRenderBlur(InCameraSettings.RenderSettings.PostprocessBlur);
	DstViewport.UpdateConfiguration_PostRenderGenerateMips(InCameraSettings.RenderSettings.GenerateMips);

	// UDisplayClusterConfigurationICVFX_CameraAdvancedRenderSettings
	const FDisplayClusterConfigurationICVFX_CameraAdvancedRenderSettings& InAdvancedRS = InCameraSettings.RenderSettings.AdvancedRenderSettings;
	{
		InOutRenderSettings.RenderTargetRatio = InAdvancedRS.RenderTargetRatio;
		InOutRenderSettings.GPUIndex = InAdvancedRS.GPUIndex;
		InOutRenderSettings.StereoGPUIndex = InAdvancedRS.StereoGPUIndex;
		InOutRenderSettings.bForceMono = FDisplayClusterViewportConfigurationHelpers::IsForceMonoscopicRendering(InAdvancedRS.StereoMode);
	}

	// Support inner camera custom frustum
	FDisplayClusterViewportConfigurationHelpers_ICVFX::UpdateCameraCustomFrustum(InCameraSettings.CustomFrustum, InOutRenderSettings.CustomFrustumSettings);

	// Set RenderTargetAdaptRatio
	InOutRenderSettings.RenderTargetAdaptRatio = InCameraSettings.CustomFrustum.GetCameraAdaptResolutionRatio(*StageSettings);

	// Set viewport buffer ratio
	DstViewport.SetViewportBufferRatio(InCameraSettings.GetCameraBufferRatio(*StageSettings));

	// Update viewport Upscaler method
	InCameraSettings.GetCameraUpscalerSettings(*StageSettings, InOutRenderSettings.UpscalerSettings);

	// InCamera tile rendering.
	FDisplayClusterViewportConfigurationHelpers_Tile::UpdateICVFXCameraViewportTileSettings(DstViewport, InCameraSettings.RenderSettings.Media);
}

void FDisplayClusterViewportConfigurationHelpers_ICVFX::UpdateChromakeyViewportSettings(FDisplayClusterViewport& DstViewport, FDisplayClusterViewport& InCameraViewport, const FDisplayClusterConfigurationICVFX_CameraSettings& InCameraSettings)
{
	const FDisplayClusterConfigurationICVFX_StageSettings* StageSettings = DstViewport.Configuration->GetStageSettings();
	if (!StageSettings)
	{
		return;
	}

	const FDisplayClusterConfigurationICVFX_ChromakeySettings& InChromakeySettings = InCameraSettings.Chromakey;

	// Gain direct access to internal resources of the viewport:
	FDisplayClusterViewport_RenderSettings&               InOutRenderSettings = DstViewport.GetRenderSettingsImpl();
	FDisplayClusterViewport_RenderSettings& InOutCameraViewportRenderSettings = InCameraViewport.GetRenderSettingsImpl();

	// Chromakey used as overlay
	InOutRenderSettings.bVisible = false;

	// Use special capture mode (this change RTT format and render flags)
	InOutRenderSettings.CaptureMode = EDisplayClusterViewportCaptureMode::Chromakey;

	// UDisplayClusterConfigurationICVFX_ChromakeyRenderSettings
	if (const FDisplayClusterConfigurationICVFX_ChromakeyRenderSettings* InChromakeyRenderSettings = InChromakeySettings.GetChromakeyRenderSettings(*StageSettings))
	{
		DstViewport.UpdateConfiguration_PostRenderOverride(InChromakeyRenderSettings->Replace);
		DstViewport.UpdateConfiguration_PostRenderBlur(InChromakeyRenderSettings->PostprocessBlur);
		DstViewport.UpdateConfiguration_PostRenderGenerateMips(InChromakeyRenderSettings->GenerateMips);

		// Update visibility settings only for rendered viewports
		if (!DstViewport.GetPostRenderSettings().Replace.IsEnabled())
		{
			check(InChromakeyRenderSettings->ShowOnlyList.IsVisibilityListValid());

			FDisplayClusterViewportConfigurationHelpers_Visibility::UpdateShowOnlyList_ICVFX(DstViewport, InChromakeyRenderSettings->ShowOnlyList, &InCameraSettings);
		}

		DstViewport.UpdateConfiguration_OverlayRenderSettings(InChromakeyRenderSettings->AdvancedRenderSettings);

		// Use the size of the ICVFX camera viewport as a base value, then multiply it by ChromakeySizeMult.
		const FIntPoint ChromakeyTextureSize = InOutCameraViewportRenderSettings.Rect.Size() * InChromakeyRenderSettings->ChromakeySizeMult;

		// Check that the size of the CK viewport is zero and show the log message once
		if (ChromakeyTextureSize.GetMin() <= 0)
		{
			if (DstViewport.CanShowLogMsgOnce(EDisplayClusterViewportShowLogMsgOnce::ChromakeyViewportHasZeroSize))
			{
				UE_LOGF(LogDisplayClusterViewport, Error, "ICVFX Chromakey Viewport '%ls': Has zero size. (ChromakeySizeMult=%.2f)", *DstViewport.GetId(), InChromakeyRenderSettings->ChromakeySizeMult);
			}
			else
			{
				DstViewport.ResetShowLogMsgOnce(EDisplayClusterViewportShowLogMsgOnce::ChromakeyViewportHasZeroSize);
			}
		}

		// Check the obtained value to be valid and use as the chromakey RTT size.
		InOutRenderSettings.Rect = FDisplayClusterViewportHelpers::GetValidViewportRect(FIntRect(FIntPoint(0, 0), ChromakeyTextureSize), DstViewport.GetId(), TEXT("Configuration custom chromakey Frame Size"));

		// Debug: override the texture of the target viewport from this chromakeyRTT
		if (InChromakeyRenderSettings->bReplaceCameraViewport)
		{
			InOutCameraViewportRenderSettings.SetViewportOverride(DstViewport.GetId());
		}
	}

	// Update OCIO for Chromakey Viewport
	// Note: Chromakey OCIO is temporarily disabled
	// FDisplayClusterViewportConfigurationHelpers_OpenColorIO::UpdateChromakeyViewportOCIO(DstViewport, RootActor, InCameraComponent);

	// Support inner camera custom frustum
	FDisplayClusterViewportConfigurationHelpers_ICVFX::UpdateCameraCustomFrustum(InCameraSettings.CustomFrustum, InOutRenderSettings.CustomFrustumSettings);

	// Attach to parent viewport
	InOutRenderSettings.AssignParentViewport(InCameraViewport.GetId(), InOutCameraViewportRenderSettings);
}

void FDisplayClusterViewportConfigurationHelpers_ICVFX::UpdateCameraSettings_Chromakey(FDisplayClusterShaderParameters_ICVFX::FCameraSettings& InOutCameraSettings,
	const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings,
	const FDisplayClusterConfigurationICVFX_CameraSettings& InCameraSettings,
	const bool bEnableChromakey,
	const bool bEnableChromakeyMarkers,
	const FString& InChromakeyViewportId)
{
	// Set chromakey color
	InOutCameraSettings.ChromakeyColor = InCameraSettings.Chromakey.GetChromakeyColor(InStageSettings);

	// Set chromakey source
	switch (InOutCameraSettings.ChromakeySource)
	{
	case EDisplayClusterShaderParametersICVFX_ChromakeySource::ChromakeyLayers:
		if (InChromakeyViewportId.IsEmpty())
		{
			// Disable chromakey: CK viewport required
			InOutCameraSettings.ChromakeySource = EDisplayClusterShaderParametersICVFX_ChromakeySource::Disabled;
			return;
		}

		if (const FDisplayClusterConfigurationICVFX_ChromakeyRenderSettings* ChromakeyRenderSettings = InCameraSettings.Chromakey.GetChromakeyRenderSettings(InStageSettings))
		{
			if (ChromakeyRenderSettings->bReplaceCameraViewport)
			{
				// Do not show Chromakey layers in this in-camera viewport, because they will be replaced by the Chromakey image (for debugging purposes).
				InOutCameraSettings.ChromakeySource = EDisplayClusterShaderParametersICVFX_ChromakeySource::Disabled;
				return;
			}
		}

		// Set the chromakey viewport name in the ICVFX shader params for current camera
		InOutCameraSettings.Chromakey.ViewportId = InChromakeyViewportId;

		break;

	case EDisplayClusterShaderParametersICVFX_ChromakeySource::Disabled:
		return;

	default:
		break;
	}

	//Setup chromakey markers
	if (bEnableChromakeyMarkers)
	{
		// Setup chromakey markers
		if (const FDisplayClusterConfigurationICVFX_ChromakeyMarkers* ChromakeyMarkers = InCameraSettings.Chromakey.GetChromakeyMarkers(InStageSettings))
		{
			InOutCameraSettings.ChromakeyMarkersColor = ChromakeyMarkers->MarkerColor;
			InOutCameraSettings.ChromakeyMarkersScale = ChromakeyMarkers->MarkerSizeScale;
			InOutCameraSettings.ChromakeyMarkersDistance = ChromakeyMarkers->MarkerTileDistance;
			InOutCameraSettings.ChromakeyMarkersOffset = ChromakeyMarkers->MarkerTileOffset;

			// Assign texture RHI ref
			if (FTextureResource* MarkersResource = ChromakeyMarkers->MarkerTileRGBA->GetResource())
			{
				InOutCameraSettings.ChromakeMarkerTextureRHI = MarkersResource->TextureRHI;
			}
		}
	}
}

void FDisplayClusterViewportConfigurationHelpers_ICVFX::UpdateCameraSettings_OverlapChromakey(FDisplayClusterShaderParameters_ICVFX::FCameraSettings& InOutCameraSettings, const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings, const FDisplayClusterConfigurationICVFX_CameraSettings& InCameraSettings, bool bEnableChromakeyMarkers)
{
	// Setup overlap chromakey color
	InOutCameraSettings.OverlapChromakeyColor = InCameraSettings.Chromakey.GetOverlapChromakeyColor(InStageSettings);

	// Setup overlap chromakey markers
	if (bEnableChromakeyMarkers)
	{
		if (const FDisplayClusterConfigurationICVFX_ChromakeyMarkers* OverlapChromakeyMarkers = InCameraSettings.Chromakey.GetOverlapChromakeyMarkers(InStageSettings))
		{
			InOutCameraSettings.OverlapChromakeyMarkersColor = OverlapChromakeyMarkers->MarkerColor;
			InOutCameraSettings.OverlapChromakeyMarkersScale = OverlapChromakeyMarkers->MarkerSizeScale;
			InOutCameraSettings.OverlapChromakeyMarkersDistance = OverlapChromakeyMarkers->MarkerTileDistance;
			InOutCameraSettings.OverlapChromakeyMarkersOffset = OverlapChromakeyMarkers->MarkerTileOffset;

			// Assign texture RHI ref
			if (FTextureResource* OverlapMarkersResource = OverlapChromakeyMarkers->MarkerTileRGBA->GetResource())
			{
				InOutCameraSettings.OverlapChromakeyMarkerTextureRHI = OverlapMarkersResource->TextureRHI;
			}
		}
	}
}

void FDisplayClusterViewportConfigurationHelpers_ICVFX::UpdateLightcardViewportSetting(FDisplayClusterViewport& DstViewport, FDisplayClusterViewport& BaseViewport)
{
	const FDisplayClusterConfigurationICVFX_StageSettings* InStageSettings = DstViewport.Configuration->GetStageSettings();
	if (!InStageSettings)
	{
		return;
	}

	const FDisplayClusterConfigurationICVFX_LightcardSettings& InLightcardSettings = InStageSettings->Lightcard;

	// Gain direct access to internal settings of the viewport:
	FDisplayClusterViewport_RenderSettings&                       InOutRenderSettings = DstViewport.GetRenderSettingsImpl();
	FDisplayClusterViewport_RenderSettings&           InOutBaseViewportRenderSettings = BaseViewport.GetRenderSettingsImpl();
	FDisplayClusterViewport_RenderSettingsICVFX& InOutBaseViewportRenderSettingsICVFX = BaseViewport.GetRenderSettingsICVFXImpl();

	// LIghtcard texture used as overlay
	InOutRenderSettings.bVisible = false;

	if (!InLightcardSettings.bEnable)
	{
		// Disable this viewport
		InOutRenderSettings.bEnable = false;
		return;
	}

	FDisplayClusterViewportConfigurationHelpers_Postprocess::UpdateLightcardPostProcessSettings(DstViewport, BaseViewport);

	// Update OCIO for Lightcard Viewport
	FDisplayClusterViewportConfigurationHelpers_OpenColorIO::UpdateLightcardViewportOCIO(DstViewport, BaseViewport);

	InOutRenderSettings.CaptureMode = EDisplayClusterViewportCaptureMode::Lightcard;

	const FDisplayClusterConfigurationICVFX_LightcardRenderSettings& InRenderSettings = InLightcardSettings.RenderSettings;
	{
		DstViewport.UpdateConfiguration_PostRenderOverride(InRenderSettings.Replace);
		DstViewport.UpdateConfiguration_PostRenderBlur(InRenderSettings.PostprocessBlur);
		DstViewport.UpdateConfiguration_PostRenderGenerateMips(InRenderSettings.GenerateMips);

		// Update visibility settings only for rendered viewports
		if (!DstViewport.GetPostRenderSettings().Replace.IsEnabled())
		{
			check(InLightcardSettings.ShowOnlyList.IsVisibilityListValid());

			FDisplayClusterViewportConfigurationHelpers_Visibility::UpdateShowOnlyList_ICVFX(DstViewport, InLightcardSettings.ShowOnlyList);
		}

		DstViewport.UpdateConfiguration_OverlayRenderSettings(InRenderSettings.AdvancedRenderSettings);
	}

	FString ParentViewportId = BaseViewport.GetId();
	if (!BaseViewport.GetRenderSettings().GetViewportOverrideId().IsEmpty())
	{
		// If the base viewport is already overridden (e.g. by the 'Reference' projection policy),
		// use it as the parent viewport.
		ParentViewportId = BaseViewport.GetRenderSettings().GetViewportOverrideId();
	}

	// Attach to parent viewport
	InOutRenderSettings.AssignParentViewport(ParentViewportId, BaseViewport.GetRenderSettings());

	if (EnumHasAnyFlags(DstViewport.GetRenderSettingsICVFX().RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::Lightcard))
	{
		// Debug: override the texture of the target viewport from this lightcard RTT
		if (InRenderSettings.bReplaceViewport)
		{
			InOutBaseViewportRenderSettings.SetViewportOverride(DstViewport.GetId());
		}
		else
		{
			if (EnumHasAnyFlags(DstViewport.GetRenderSettingsICVFX().RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::UnderInFrustum))
			{
				InOutBaseViewportRenderSettingsICVFX.ICVFX.LightCardUnder.ViewportId = DstViewport.GetId();
			}

			if (EnumHasAnyFlags(DstViewport.GetRenderSettingsICVFX().RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::OverInFrustum))
			{
				InOutBaseViewportRenderSettingsICVFX.ICVFX.LightCardOver.ViewportId = DstViewport.GetId();
			}

		}
	}

	if (EnumHasAnyFlags(DstViewport.GetRenderSettingsICVFX().RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::UVLightcard))
	{
		if (EnumHasAnyFlags(DstViewport.GetRenderSettingsICVFX().RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::UnderInFrustum))
		{
			InOutBaseViewportRenderSettingsICVFX.ICVFX.UVLightCardUnder.ViewportId = DstViewport.GetId();
		}

		if (EnumHasAnyFlags(DstViewport.GetRenderSettingsICVFX().RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::OverInFrustum))
		{
			InOutBaseViewportRenderSettingsICVFX.ICVFX.UVLightCardOver.ViewportId = DstViewport.GetId();
		}
	}

	if (EnumHasAnyFlags(DstViewport.GetRenderSettingsICVFX().RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::Lightcard | EDisplayClusterViewportRuntimeICVFXFlags::UVLightcard))
	{
		if (EnumHasAnyFlags(InOutBaseViewportRenderSettingsICVFX.Flags, EDisplayClusterViewportICVFXFlags::LightcardAlwaysUnder))
		{
			InOutBaseViewportRenderSettingsICVFX.ICVFX.OverrideUVLightCardRenderMode = EDisplayClusterShaderParametersICVFX_LightCardRenderMode::Under;
		}
		else if (EnumHasAnyFlags(InOutBaseViewportRenderSettingsICVFX.Flags, EDisplayClusterViewportICVFXFlags::LightcardAlwaysOver))
		{
			InOutBaseViewportRenderSettingsICVFX.ICVFX.OverrideUVLightCardRenderMode = EDisplayClusterShaderParametersICVFX_LightCardRenderMode::Over;
		}
	}
}

void FDisplayClusterViewportConfigurationHelpers_ICVFX::UpdateCameraCustomFrustum(
	const FDisplayClusterConfigurationICVFX_CameraCustomFrustum& InCameraCustomFrustumConfiguration,
	FDisplayClusterViewport_CustomFrustumSettings& OutCustomFrustumSettings)
{
	OutCustomFrustumSettings.bEnabled = false;

	if (InCameraCustomFrustumConfiguration.bEnable)
	{
		const FMarginSet CustomFrustumMargins(
			InCameraCustomFrustumConfiguration.Left,
			InCameraCustomFrustumConfiguration.Right,
			InCameraCustomFrustumConfiguration.Top,
			InCameraCustomFrustumConfiguration.Bottom
		);

		OutCustomFrustumSettings.bAdaptResolution = InCameraCustomFrustumConfiguration.bAdaptResolution;

		switch (InCameraCustomFrustumConfiguration.Mode)
		{
		case EDisplayClusterConfigurationViewportCustomFrustumMode::Percent:
			OutCustomFrustumSettings.bEnabled = true;
			OutCustomFrustumSettings.Unit = EDisplayClusterViewport_FrustumUnit::Percent;

			// Scale 0..100% to 0..1 range
			OutCustomFrustumSettings.AssignMargins(CustomFrustumMargins * .01f);

			break;

		case EDisplayClusterConfigurationViewportCustomFrustumMode::Pixels:
			OutCustomFrustumSettings.bEnabled = true;
			OutCustomFrustumSettings.Unit = EDisplayClusterViewport_FrustumUnit::Pixels;

			OutCustomFrustumSettings.AssignMargins(CustomFrustumMargins);

			break;

		default:
			break;
		}
	}
};
