// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "ShaderParameters/DisplayClusterShaderParameters_ICVFX.h"
#include "IDisplayClusterProjection.h"
#include "Render/Projection/IDisplayClusterProjectionPolicy.h"

class FDisplayClusterViewportConfiguration;
class FDisplayClusterViewport;
class UDisplayClusterICVFXCameraComponent;

struct FDisplayClusterConfigurationICVFX_StageSettings;
struct FDisplayClusterConfigurationICVFX_CameraCustomFrustum;
struct FDisplayClusterConfigurationICVFX_CameraSettings;
struct FDisplayClusterViewport_CustomFrustumSettings;

/**
* ICVFX configuration helper class.
*/
class FDisplayClusterViewportConfigurationHelpers_ICVFX
{
public:
	/* Return the new or existing InnerFrustum viewport. */
	static FDisplayClusterViewport* GetOrCreateCameraViewport(FDisplayClusterViewportConfiguration& InConfiguration, UDisplayClusterICVFXCameraComponent& InCameraComponent, const FDisplayClusterConfigurationICVFX_CameraSettings& InCameraSettings);

	/* Return the new or existing Chromakey viewport. */
	static FDisplayClusterViewport* GetOrCreateChromakeyViewport(FDisplayClusterViewportConfiguration& InConfiguration, UDisplayClusterICVFXCameraComponent& InCameraComponent, const FDisplayClusterConfigurationICVFX_CameraSettings& InCameraSettings);
	
	/* Return the new or existing LightCard viewport. */
	static FDisplayClusterViewport* GetOrCreateLightcardViewport(FDisplayClusterViewport& BaseViewport, const bool bOverInFrustum);

	/* Return the new or existing UVLightCard viewport. */
	static FDisplayClusterViewport* GetOrCreateUVLightcardViewport(FDisplayClusterViewport& BaseViewport, const bool bOverInFrustum);

	/* Because UVLightCard viewports share the same outer texture (from LightCardManager), they can clone each other.
	 * Except when the OCIO settings are not equal.
	 * This function sets the InUVLightCardViewport rendering settings to minimize rendering costs.
	 */
	static void ReuseUVLightCardViewportWithinClusterNode(FDisplayClusterViewport& InUVLightCardViewport, const bool bOverInFrustum);

	/** Returns all visible InnerCamera viewports. */
	static TArray<TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>> GetAllVisibleInnerCameraViewports(FDisplayClusterViewportConfiguration& InConfiguration, bool bGetChromakey = false);

	static TArray<TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>> PreviewGetRenderedInCameraViewports(FDisplayClusterViewportConfiguration& InConfiguration, const FString& InICVFXCameraId, bool bGetChromakey = false);
	static bool PreviewReuseInnerFrustumViewportWithinClusterNodes(FDisplayClusterViewport& InCameraViewport, UDisplayClusterICVFXCameraComponent& InCameraComponent, const FDisplayClusterConfigurationICVFX_CameraSettings& InCameraSettings);
	static bool PreviewReuseChromakeyViewportWithinClusterNodes(FDisplayClusterViewport& InChromakeyViewport, const FString& InICVFXCameraId);

	static void UpdateCameraViewportSettings(FDisplayClusterViewport& DstViewport, UDisplayClusterICVFXCameraComponent& InCameraComponent, const FDisplayClusterConfigurationICVFX_CameraSettings& InCameraSettings);
	static void UpdateChromakeyViewportSettings(FDisplayClusterViewport& DstViewport, FDisplayClusterViewport& InCameraViewport, const FDisplayClusterConfigurationICVFX_CameraSettings& InCameraSettings);
	static void UpdateLightcardViewportSetting(FDisplayClusterViewport& DstViewport, FDisplayClusterViewport& BaseViewport);

	static bool IsCameraUsed(const FDisplayClusterConfigurationICVFX_CameraSettings& InCameraSettings);

	static TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe> CreateProjectionPolicyCameraICVFX(FDisplayClusterViewportConfiguration& InConfiguration, UDisplayClusterICVFXCameraComponent& CameraComponent, const FDisplayClusterConfigurationICVFX_CameraSettings& InCameraSettings);
	static bool UpdateCameraProjectionSettingsICVFX(FDisplayClusterViewportConfiguration& InConfiguration, UDisplayClusterICVFXCameraComponent& CameraComponent, const FDisplayClusterConfigurationICVFX_CameraSettings& InCameraSettings, const TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe>& InProjectionPolicy);

	/** Configures the camera's chromakey render settings to match the specified configuration settings  */
	static void UpdateCameraSettings_Chromakey(
		FDisplayClusterShaderParameters_ICVFX::FCameraSettings& InOutCameraSettings,
		const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings,
		const FDisplayClusterConfigurationICVFX_CameraSettings& InCameraSettings,
		const bool bEnableChromakey,
		const bool bEnableChromakeyMarkers,
		const FString& InChromakeyViewportId);
	static void UpdateCameraSettings_OverlapChromakey(FDisplayClusterShaderParameters_ICVFX::FCameraSettings& InOutCameraSettings, const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings, const FDisplayClusterConfigurationICVFX_CameraSettings& InCameraSettings, bool bEnableChromakeyMarkers);

	static void UpdateCameraCustomFrustum(const FDisplayClusterConfigurationICVFX_CameraCustomFrustum& InCameraCustomFrustumConfiguration, FDisplayClusterViewport_CustomFrustumSettings& OutCustomFrustumSettings);

	/**
	* Backward-compatibility bridge: derive legacy EDisplayClusterViewportRuntimeICVFXFlags
	* from the new EDisplayClusterViewportFlags until the legacy runtime enum is removed.
	*/ 
	static void UpdateLegacyRuntimeICVFXFlagsFromViewportFlags(FDisplayClusterViewport& InOutViewport);
};
