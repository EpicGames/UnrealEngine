// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterViewportConfigurationHelpers_OpenColorIO.h"
#include "DisplayClusterViewportConfiguration.h"

#include "DisplayClusterConfigurationTypes_Viewport.h"
#include "DisplayClusterSceneViewExtensions.h"
#include "DisplayClusterRootActor.h"

#include "Render/Viewport/DisplayClusterViewport.h"
#include "Render/Viewport/DisplayClusterViewportManager.h"

#include "Components/DisplayClusterICVFXCameraComponent.h"

///////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterViewportConfigurationHelpers_OpenColorIO
///////////////////////////////////////////////////////////////////////////////////////
bool FDisplayClusterViewportConfigurationHelpers_OpenColorIO::UpdateBaseViewportOCIO(FDisplayClusterViewport& DstViewport, const UDisplayClusterConfigurationViewport& InViewportConfiguration)
{
	if (!EnumHasAnyFlags(DstViewport.GetRenderSettingsICVFX().RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::InCamera | EDisplayClusterViewportRuntimeICVFXFlags::Chromakey | EDisplayClusterViewportRuntimeICVFXFlags::Lightcard | EDisplayClusterViewportRuntimeICVFXFlags::UVLightcard))
	{
		const FDisplayClusterConfigurationICVFX_StageSettings* StageSettings =DstViewport.Configuration->GetStageSettings();
		if (const FOpenColorIOColorConversionSettings* OCIOConfiguration = StageSettings ? StageSettings->FindViewportOCIOConfiguration(DstViewport.GetId()) : nullptr)
		{
			ApplyOCIOConfiguration(DstViewport, *OCIOConfiguration);

			return true;
		}

		DisableOCIOConfiguration(DstViewport);
	}

	return false;
}

bool FDisplayClusterViewportConfigurationHelpers_OpenColorIO::UpdateLightcardViewportOCIO(FDisplayClusterViewport& DstViewport, FDisplayClusterViewport& BaseViewport)
{
	if (EnumHasAnyFlags(DstViewport.GetRenderSettingsICVFX().RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::Lightcard | EDisplayClusterViewportRuntimeICVFXFlags::UVLightcard))
	{
		const FDisplayClusterConfigurationICVFX_StageSettings* StageSettings = DstViewport.Configuration->GetStageSettings();
		if (const FOpenColorIOColorConversionSettings* OCIOConfiguration = StageSettings ? StageSettings->FindLightcardOCIOConfiguration(BaseViewport.GetId()): nullptr)
		{
			ApplyOCIOConfiguration(DstViewport, *OCIOConfiguration);

			return true;
		}

		// No OCIO defined for this lightcard viewport, disabled
		DisableOCIOConfiguration(DstViewport);
	}

	return false;
}

bool FDisplayClusterViewportConfigurationHelpers_OpenColorIO::UpdateCameraViewportOCIO(FDisplayClusterViewport& DstViewport, const FDisplayClusterConfigurationICVFX_CameraSettings& InCameraSettings)
{
	if (EnumHasAllFlags(DstViewport.GetRenderSettingsICVFX().RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::InCamera))
	{
		if (const FOpenColorIOColorConversionSettings* OCIOConfiguration = InCameraSettings.FindInnerFrustumOCIOConfiguration(DstViewport.GetClusterNodeId()))
		{
			ApplyOCIOConfiguration(DstViewport, *OCIOConfiguration);

			return true;
		}

		DisableOCIOConfiguration(DstViewport);
	}

	return false;
}

bool FDisplayClusterViewportConfigurationHelpers_OpenColorIO::UpdateChromakeyViewportOCIO(FDisplayClusterViewport& DstViewport, const FDisplayClusterConfigurationICVFX_CameraSettings& InCameraSettings)
{
	if (EnumHasAllFlags(DstViewport.GetRenderSettingsICVFX().RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::Chromakey))
	{
		if (const FOpenColorIOColorConversionSettings* OCIOConfiguration = InCameraSettings.FindChromakeyOCIOConfiguration(DstViewport.GetClusterNodeId()))
		{
			ApplyOCIOConfiguration(DstViewport, *OCIOConfiguration);

			return true;
		}

		DisableOCIOConfiguration(DstViewport);
	}

	return false;
}

void FDisplayClusterViewportConfigurationHelpers_OpenColorIO::ApplyOCIOConfiguration(FDisplayClusterViewport& DstViewport, const FOpenColorIOColorConversionSettings& InConversionSettings)
{
	if (!DstViewport.Configuration->GetRenderFrameSettings().IsEnabledOpenColorIO())
	{
		// OCIO is disabled for nDisplay.
		DisableOCIOConfiguration(DstViewport);

		return;
	}

	if (DstViewport.GetOpenColorIO().IsValid() && DstViewport.GetOpenColorIO()->IsConversionSettingsEqual(InConversionSettings))
	{
		// Already assigned
		return;
	}

	DstViewport.SetOpenColorIO(MakeShared<FDisplayClusterViewport_OpenColorIO>(InConversionSettings));
}

void FDisplayClusterViewportConfigurationHelpers_OpenColorIO::DisableOCIOConfiguration(FDisplayClusterViewport& DstViewport)
{
	// Remove OICO ref
	DstViewport.SetOpenColorIO(nullptr);
}

bool FDisplayClusterViewportConfigurationHelpers_OpenColorIO::IsInnerFrustumViewportOCIOSettingsEqual(const FDisplayClusterViewport& InViewport1, const FDisplayClusterViewport& InViewport2, const FDisplayClusterConfigurationICVFX_CameraSettings& InCameraSettings)
{
	return InCameraSettings.IsInnerFrustumViewportSettingsEqual(InViewport1.GetClusterNodeId(), InViewport2.GetClusterNodeId());
}

bool FDisplayClusterViewportConfigurationHelpers_OpenColorIO::IsChromakeyViewportOCIOSettingsEqual(const FDisplayClusterViewport& InViewport1, const FDisplayClusterViewport& InViewport2, const FDisplayClusterConfigurationICVFX_CameraSettings& InCameraSettings)
{
	return InCameraSettings.IsChromakeyViewportSettingsEqual(InViewport1.GetClusterNodeId(), InViewport2.GetClusterNodeId());
}
