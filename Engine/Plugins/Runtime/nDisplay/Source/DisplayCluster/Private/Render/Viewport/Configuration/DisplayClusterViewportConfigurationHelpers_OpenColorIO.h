// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FDisplayClusterViewport;
class UDisplayClusterConfigurationViewport;

struct FDisplayClusterConfigurationICVFX_CameraSettings;
struct FOpenColorIOColorConversionSettings;

/**
* OCIO configuration helper class.
*/
class FDisplayClusterViewportConfigurationHelpers_OpenColorIO
{
public:
	/** Update OCIO for base viewport (Outer).
	 *
	 * @param DstViewport             - Dest viewport object.
	 * @param RootActor               - DCRA object.
	 * @param InViewportConfiguration - Actual viewport configuration.
	 *
	 * @return - true if success.
	 */
	static bool UpdateBaseViewportOCIO(FDisplayClusterViewport& DstViewport, const UDisplayClusterConfigurationViewport& InViewportConfiguration);

	/** Update OCIO for LightCard viewport.
	 *
	 * @param DstViewport  - Dest LightCard viewport object.
	 * @param BaseViewport - Base (Outer) viewport object.
	 * @param RootActor    - DCRA object.
	 *
	 * @return - true if success.
	 */
	static bool UpdateLightcardViewportOCIO(FDisplayClusterViewport& DstViewport, FDisplayClusterViewport& BaseViewport);

	/** Update OCIO for InnerFrustum(InCamera) viewport.
	 *
	 * @param DstViewport       - Dest InCamera viewport object
	 * @param RootActor         - DCRA object
	 * @param InCameraComponent - InCamera component with configuration
	 *
	 * @return - true if success
	 */
	static bool UpdateCameraViewportOCIO(FDisplayClusterViewport& DstViewport, const FDisplayClusterConfigurationICVFX_CameraSettings& InCameraSettings);

	/** Update OCIO for Chromakey viewport.
	 *
	 * @param DstViewport       - Dest InCamera viewport object
	 * @param RootActor         - DCRA object
	 * @param InCameraComponent - InCamera component with configuration
	 *
	 * @return - true if success
	 */
	static bool UpdateChromakeyViewportOCIO(FDisplayClusterViewport& DstViewport, const FDisplayClusterConfigurationICVFX_CameraSettings& InCameraSettings);

	/** Compare InnerFrustum OCIO configuration for input viewports
	 *
	 * @param InViewport1       - Viewport #1
	 * @param InViewport2       - Viewport #2
	 * @param InCameraComponent - InCamera component with configuration
	 *
	 * @return - true if OCIO is equals
	 */
	static bool IsInnerFrustumViewportOCIOSettingsEqual(const FDisplayClusterViewport& InViewport1, const FDisplayClusterViewport& InViewport2, const FDisplayClusterConfigurationICVFX_CameraSettings& InCameraSettings);

	/** Compare Chromakey OCIO configuration for input viewports
	 *
	 * @param InViewport1       - Viewport #1
	 * @param InViewport2       - Viewport #2
	 * @param InCameraComponent - InCamera component with configuration
	 *
	 * @return - true if OCIO is equals
	 */
	static bool IsChromakeyViewportOCIOSettingsEqual(const FDisplayClusterViewport& InViewport1, const FDisplayClusterViewport& InViewport2, const FDisplayClusterConfigurationICVFX_CameraSettings& InCameraSettings);

private:
	/** Apply OCIO to viewport.
	 *
	 * @param DstViewport       - Dest viewport object
	 * @param InConversionSettings   - OCIO conversion settings
	 *
	 * @return - none
	 */
	static void ApplyOCIOConfiguration(FDisplayClusterViewport& DstViewport, const FOpenColorIOColorConversionSettings& InConversionSettings);

	/** Reset OCIO for viewport.
	 *
	 * @param DstViewport       - Dest viewport object
	 *
	 * @return - none
	 */
	static void DisableOCIOConfiguration(FDisplayClusterViewport& DstViewport);
};
