// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterViewportConfigurationHelpers_Postprocess.h"
#include "DisplayClusterViewportConfiguration.h"
#include "DisplayClusterConfigurationTypes.h"
#include "Render/Viewport/DisplayClusterViewport.h"
#include "Render/Viewport/DisplayClusterViewportManager.h"

#include "DisplayClusterRootActor.h"
#include "Components/DisplayClusterICVFXCameraComponent.h"
#include "Components/DisplayClusterCameraComponent.h"

namespace UE::DisplayCluster::Configuration::PostprocessHelpers
{
	static inline void ImplUpdateCustomPostprocess(FDisplayClusterViewport& DstViewport, bool bEnabled, const FDisplayClusterConfigurationViewport_CustomPostprocessSettings& InCustomPostprocess, IDisplayClusterViewport_CustomPostProcessSettings::ERenderPass RenderPass)
	{
		if (bEnabled)
		{
			DstViewport.GetViewport_CustomPostProcessSettings().AddCustomPostProcess(RenderPass, InCustomPostprocess.PostProcessSettings, InCustomPostprocess.BlendWeight, InCustomPostprocess.bIsOneFrame);
		}
		else
		{
			DstViewport.GetViewport_CustomPostProcessSettings().RemoveCustomPostProcess(RenderPass);
		}
	}

	static inline void ImplRemoveCustomPostprocess(FDisplayClusterViewport& DstViewport, IDisplayClusterViewport_CustomPostProcessSettings::ERenderPass RenderPass)
	{
		DstViewport.GetViewport_CustomPostProcessSettings().RemoveCustomPostProcess(RenderPass);
	}

// Note that skipped parameters in macro definitions will just evaluate to nothing
// This is intentional to get around the inconsistent naming in the color grading fields in FPostProcessSettings
#define PP_CONDITIONAL_BLEND(BLENDOP, COLOR, OUTGROUP, INGROUP, NAME, OFFSETOP, OFFSETVALUE) \
	{ \
		bool bOverridePPSettings0 = PPSettings0.INGROUP bOverride_##NAME; \
		bool bOverridePPSettings1 = (PPSettings1 != nullptr) && PPSettings1->INGROUP bOverride_##NAME; \
		bool bOverridePPSettings2 = (PPSettings2 != nullptr) && PPSettings2->INGROUP bOverride_##NAME; \
		bool bOverridePPSettings3 = (PPSettings3 != nullptr) && PPSettings3->INGROUP bOverride_##NAME; \
		 \
		if (bOverridePPSettings0 && bOverridePPSettings1 && bOverridePPSettings2 && bOverridePPSettings3) \
		{ \
			OutputPP.COLOR##NAME##OUTGROUP = PPSettings0.INGROUP NAME BLENDOP PPSettings1->INGROUP NAME BLENDOP PPSettings2->INGROUP NAME BLENDOP PPSettings3->INGROUP NAME OFFSETOP OFFSETVALUE OFFSETOP OFFSETVALUE; \
			OutputPP.bOverride_##COLOR##NAME##OUTGROUP = true; \
		} \
		else if (bOverridePPSettings0 && bOverridePPSettings1 && bOverridePPSettings2) \
		{ \
			OutputPP.COLOR##NAME##OUTGROUP = PPSettings0.INGROUP NAME BLENDOP PPSettings1->INGROUP NAME BLENDOP PPSettings2->INGROUP NAME OFFSETOP OFFSETVALUE OFFSETOP OFFSETVALUE; \
			OutputPP.bOverride_##COLOR##NAME##OUTGROUP = true; \
		} \
		else if (bOverridePPSettings0 && bOverridePPSettings1) \
		{ \
			OutputPP.COLOR##NAME##OUTGROUP = PPSettings0.INGROUP NAME BLENDOP PPSettings1->INGROUP NAME OFFSETOP OFFSETVALUE; \
			OutputPP.bOverride_##COLOR##NAME##OUTGROUP = true; \
		} \
		else if (bOverridePPSettings0 && bOverridePPSettings2) \
		{ \
			OutputPP.COLOR##NAME##OUTGROUP = PPSettings0.INGROUP NAME BLENDOP PPSettings2->INGROUP NAME OFFSETOP OFFSETVALUE; \
			OutputPP.bOverride_##COLOR##NAME##OUTGROUP = true; \
		} \
		else if (bOverridePPSettings1 && bOverridePPSettings2) \
		{ \
			OutputPP.COLOR##NAME##OUTGROUP = PPSettings1->INGROUP NAME BLENDOP PPSettings2->INGROUP NAME OFFSETOP OFFSETVALUE; \
			OutputPP.bOverride_##COLOR##NAME##OUTGROUP = true; \
		} \
		else if (bOverridePPSettings2) \
		{ \
			OutputPP.COLOR##NAME##OUTGROUP = PPSettings2->INGROUP NAME; \
			OutputPP.bOverride_##COLOR##NAME##OUTGROUP = true; \
		} \
		else if (bOverridePPSettings1) \
		{ \
			OutputPP.COLOR##NAME##OUTGROUP = PPSettings1->INGROUP NAME; \
			OutputPP.bOverride_##COLOR##NAME##OUTGROUP = true; \
		} \
		else if (bOverridePPSettings0) \
		{ \
			OutputPP.COLOR##NAME##OUTGROUP = PPSettings0.INGROUP NAME; \
			OutputPP.bOverride_##COLOR##NAME##OUTGROUP = true; \
		} \
	} \

/*
* This will override the settings using the priority.
* bOverridePPSettings2 (any additional settings specified by the user) will be of highest priority.
* bOverridePPSettings1 (which is nDisplay override settings) will be of highest priority.
* following by Cumulative settings (bOverridePPSettings0).
*/
#define PP_CONDITIONAL_OVERRIDE(COLOR, OUTGROUP, INGROUP, NAME) \
	{ \
		bool bOverridePPSettings0 = PPSettings0.INGROUP bOverride_##NAME; \
		bool bOverridePPSettings1 = PPSettings1 && PPSettings1->INGROUP bOverride_##NAME; \
		bool bOverridePPSettings2 = PPSettings2 && PPSettings2->INGROUP bOverride_##NAME; \
		bool bOverridePPSettings3 = PPSettings3 && PPSettings3->INGROUP bOverride_##NAME; \
		if (bOverridePPSettings0) \
		{ \
			OutputPP.COLOR##NAME##OUTGROUP = PPSettings0.INGROUP NAME; \
			OutputPP.bOverride_##COLOR##NAME##OUTGROUP = true; \
		} \
		if (bOverridePPSettings1) \
		{ \
			OutputPP.COLOR##NAME##OUTGROUP = PPSettings1->INGROUP NAME; \
			OutputPP.bOverride_##COLOR##NAME##OUTGROUP = true; \
		} \
		if (bOverridePPSettings2) \
		{ \
			OutputPP.COLOR##NAME##OUTGROUP = PPSettings2->INGROUP NAME; \
			OutputPP.bOverride_##COLOR##NAME##OUTGROUP = true; \
		} \
		if (bOverridePPSettings3) \
		{ \
			OutputPP.COLOR##NAME##OUTGROUP = PPSettings3->INGROUP NAME; \
			OutputPP.bOverride_##COLOR##NAME##OUTGROUP = true; \
		} \
	} \

	static inline void ImplBlendPostProcessSettings(
		FPostProcessSettings& OutputPP,
		const FDisplayClusterConfigurationViewport_ColorGradingRenderingSettings& PPSettings0,
		const FDisplayClusterConfigurationViewport_ColorGradingRenderingSettings* PPSettings1,
		const FDisplayClusterConfigurationViewport_ColorGradingRenderingSettings* PPSettings2,
		const FDisplayClusterConfigurationViewport_ColorGradingRenderingSettings* PPSettings3)
	{
		PP_CONDITIONAL_BLEND(+, , , , AutoExposureBias, , );
		PP_CONDITIONAL_BLEND(+, , , , ColorCorrectionHighlightsMin, , );
		PP_CONDITIONAL_BLEND(+, , , , ColorCorrectionHighlightsMax, , );
		PP_CONDITIONAL_BLEND(+, , , , ColorCorrectionShadowsMax, , );

		PP_CONDITIONAL_OVERRIDE(, , WhiteBalance., TemperatureType);
		PP_CONDITIONAL_BLEND(+, , , WhiteBalance., WhiteTemp, +, -6500.0f);
		PP_CONDITIONAL_BLEND(+, , , WhiteBalance., WhiteTint, , );

		PP_CONDITIONAL_BLEND(*, Color, , Global., Saturation, , );
		PP_CONDITIONAL_BLEND(*, Color, , Global., Contrast, , );
		PP_CONDITIONAL_BLEND(*, Color, , Global., Gamma, , );
		PP_CONDITIONAL_BLEND(*, Color, , Global., Gain, , );
		PP_CONDITIONAL_BLEND(+, Color, , Global., Offset, , );

		PP_CONDITIONAL_BLEND(*, Color, Shadows, Shadows., Saturation, , );
		PP_CONDITIONAL_BLEND(*, Color, Shadows, Shadows., Contrast, , );
		PP_CONDITIONAL_BLEND(*, Color, Shadows, Shadows., Gamma, , );
		PP_CONDITIONAL_BLEND(*, Color, Shadows, Shadows., Gain, , );
		PP_CONDITIONAL_BLEND(+, Color, Shadows, Shadows., Offset, , );

		PP_CONDITIONAL_BLEND(*, Color, Midtones, Midtones., Saturation, , );
		PP_CONDITIONAL_BLEND(*, Color, Midtones, Midtones., Contrast, , );
		PP_CONDITIONAL_BLEND(*, Color, Midtones, Midtones., Gamma, , );
		PP_CONDITIONAL_BLEND(*, Color, Midtones, Midtones., Gain, , );
		PP_CONDITIONAL_BLEND(+, Color, Midtones, Midtones., Offset, , );

		PP_CONDITIONAL_BLEND(*, Color, Highlights, Highlights., Saturation, , );
		PP_CONDITIONAL_BLEND(*, Color, Highlights, Highlights., Contrast, , );
		PP_CONDITIONAL_BLEND(*, Color, Highlights, Highlights., Gamma, , );
		PP_CONDITIONAL_BLEND(*, Color, Highlights, Highlights., Gain, , );
		PP_CONDITIONAL_BLEND(+, Color, Highlights, Highlights., Offset, , );

		PP_CONDITIONAL_BLEND(+, , , Misc., BlueCorrection, , );
		PP_CONDITIONAL_BLEND(+, , , Misc., ExpandGamut, , );
		PP_CONDITIONAL_BLEND(+, , , Misc., SceneColorTint, , );
	}

#define PP_CONDITIONAL_COPY(COLOR, OUTGROUP, INGROUP, NAME) \
		if (!bIsConditionalCopy || InPPS->bOverride_##COLOR##NAME##INGROUP) \
		{ \
			OutViewportPPSettings->OUTGROUP NAME = InPPS->COLOR##NAME##INGROUP; \
			OutViewportPPSettings->OUTGROUP bOverride_##NAME = true; \
		}

	static inline void ImplCopyPPSStruct(bool bIsConditionalCopy, FDisplayClusterConfigurationViewport_ColorGradingRenderingSettings* OutViewportPPSettings, FPostProcessSettings* InPPS)
	{
		if ((OutViewportPPSettings != nullptr) && (InPPS != nullptr))
		{
			PP_CONDITIONAL_COPY(, , , AutoExposureBias);
			PP_CONDITIONAL_COPY(, , , ColorCorrectionHighlightsMin);
			PP_CONDITIONAL_COPY(, , , ColorCorrectionHighlightsMax);
			PP_CONDITIONAL_COPY(, , , ColorCorrectionShadowsMax);

			PP_CONDITIONAL_COPY(, WhiteBalance., , TemperatureType);
			PP_CONDITIONAL_COPY(, WhiteBalance., , WhiteTemp);
			PP_CONDITIONAL_COPY(, WhiteBalance., , WhiteTint);

			PP_CONDITIONAL_COPY(Color, Global., , Saturation);
			PP_CONDITIONAL_COPY(Color, Global., , Contrast);
			PP_CONDITIONAL_COPY(Color, Global., , Gamma);
			PP_CONDITIONAL_COPY(Color, Global., , Gain);
			PP_CONDITIONAL_COPY(Color, Global., , Offset);

			PP_CONDITIONAL_COPY(Color, Shadows., Shadows, Saturation);
			PP_CONDITIONAL_COPY(Color, Shadows., Shadows, Contrast);
			PP_CONDITIONAL_COPY(Color, Shadows., Shadows, Gamma);
			PP_CONDITIONAL_COPY(Color, Shadows., Shadows, Gain);
			PP_CONDITIONAL_COPY(Color, Shadows., Shadows, Offset);

			PP_CONDITIONAL_COPY(Color, Midtones., Midtones, Saturation);
			PP_CONDITIONAL_COPY(Color, Midtones., Midtones, Contrast);
			PP_CONDITIONAL_COPY(Color, Midtones., Midtones, Gamma);
			PP_CONDITIONAL_COPY(Color, Midtones., Midtones, Gain);
			PP_CONDITIONAL_COPY(Color, Midtones., Midtones, Offset);

			PP_CONDITIONAL_COPY(Color, Highlights., Highlights, Saturation);
			PP_CONDITIONAL_COPY(Color, Highlights., Highlights, Contrast);
			PP_CONDITIONAL_COPY(Color, Highlights., Highlights, Gamma);
			PP_CONDITIONAL_COPY(Color, Highlights., Highlights, Gain);
			PP_CONDITIONAL_COPY(Color, Highlights., Highlights, Offset);

			PP_CONDITIONAL_COPY(, Misc., , BlueCorrection);
			PP_CONDITIONAL_COPY(, Misc., , ExpandGamut);
			PP_CONDITIONAL_COPY(, Misc., , SceneColorTint);
		}
	}
};
using namespace UE::DisplayCluster::Configuration;

// return true when same settings used for both viewports
bool FDisplayClusterViewportConfigurationHelpers_Postprocess::IsInnerFrustumViewportSettingsEqual(const FDisplayClusterViewport& InViewport1, const FDisplayClusterViewport& InViewport2, const FDisplayClusterConfigurationICVFX_CameraSettings& InCameraSettings)
{
	for (const FDisplayClusterConfigurationViewport_PerNodeColorGrading& ColorGradingProfileIt : InCameraSettings.PerNodeColorGrading)
	{
		if (ColorGradingProfileIt.bIsEnabled)
		{
			const FString* CustomNode1 = ColorGradingProfileIt.ApplyPostProcessToObjects.FindByPredicate([ClusterNodeId = InViewport1.GetClusterNodeId()](const FString& InClusterNodeId)
			{
				return ClusterNodeId.Equals(InClusterNodeId, ESearchCase::IgnoreCase);
			});

			const FString* CustomNode2 = ColorGradingProfileIt.ApplyPostProcessToObjects.FindByPredicate([ClusterNodeId = InViewport2.GetClusterNodeId()](const FString& InClusterNodeId)
			{
				return ClusterNodeId.Equals(InClusterNodeId, ESearchCase::IgnoreCase);
			});

			if (CustomNode1 && CustomNode2)
			{
				// equal custom settings
				return true;
			}

			if (CustomNode1 || CustomNode2)
			{
				// one of node has custom settings
				return false;
			}
		}
	}

	return true;
}

bool FDisplayClusterViewportConfigurationHelpers_Postprocess::ImplUpdateInnerFrustumColorGrading(FDisplayClusterViewport& DstViewport, const FDisplayClusterConfigurationICVFX_CameraSettings& InCameraSettings)
{
	// per node color grading first (it includes all nodes blending too)
	const FString& ClusterNodeId = DstViewport.GetClusterNodeId();
	check(!ClusterNodeId.IsEmpty());

	const FDisplayClusterConfigurationICVFX_StageSettings* StageSettings = DstViewport.Configuration->GetStageSettings();
	if (!StageSettings)
	{
		return false;
	}

	// Collect all used color grading settings into this array.
	TArray<const FDisplayClusterConfigurationViewport_ColorGradingRenderingSettings*> PostProcessList;

	const FDisplayClusterConfigurationViewport_PerNodeColorGrading* ExistPerNodeColorGrading = InCameraSettings.PerNodeColorGrading.FindByPredicate([ClusterNodeId](const FDisplayClusterConfigurationViewport_PerNodeColorGrading& ColorGradingProfileIt)
		{
			// Only allowed profiles
			return ColorGradingProfileIt.bIsEnabled && ColorGradingProfileIt.ApplyPostProcessToObjects.ContainsByPredicate([ClusterNodeId](const FString& ClusterNodeIt)
				{
					return ClusterNodeId.Compare(ClusterNodeIt, ESearchCase::IgnoreCase) == 0;
				});
		});

	if (ExistPerNodeColorGrading)
	{
		const bool bIncludeUseEntireClusterPostProcess = StageSettings->EntireClusterColorGrading.bEnableEntireClusterColorGrading && ExistPerNodeColorGrading->bEntireClusterColorGrading;
		const bool bIncludeAllNodesColorGrading = InCameraSettings.AllNodesColorGrading.bEnableInnerFrustumAllNodesColorGrading && ExistPerNodeColorGrading->bAllNodesColorGrading;

		// Cluster
		if (bIncludeUseEntireClusterPostProcess)
		{
			PostProcessList.Add(&StageSettings->EntireClusterColorGrading.ColorGradingSettings);
		}

		// All Nodes
		if (bIncludeAllNodesColorGrading)
		{
			PostProcessList.Add(&InCameraSettings.AllNodesColorGrading.ColorGradingSettings);
		}

		// Per-Node
		PostProcessList.Add(&ExistPerNodeColorGrading->ColorGradingSettings);
	}
	else if (InCameraSettings.AllNodesColorGrading.bEnableInnerFrustumAllNodesColorGrading)
	{
		const bool bEnableEntireClusterColorGrading = StageSettings->EntireClusterColorGrading.bEnableEntireClusterColorGrading && InCameraSettings.AllNodesColorGrading.bEnableEntireClusterColorGrading;

		// Cluster
		if (bEnableEntireClusterColorGrading)
		{
			PostProcessList.Add(&StageSettings->EntireClusterColorGrading.ColorGradingSettings);
		}

		// All Nodes
		PostProcessList.Add(&InCameraSettings.AllNodesColorGrading.ColorGradingSettings);
	}

	return ImplUpdateFinalPerViewportPostProcessList(DstViewport, PostProcessList);
}

bool FDisplayClusterViewportConfigurationHelpers_Postprocess::UpdateLightcardPostProcessSettings(FDisplayClusterViewport& DstViewport, FDisplayClusterViewport& BaseViewport)
{
	const FDisplayClusterConfigurationICVFX_StageSettings* StageSettings = DstViewport.Configuration->GetStageSettings();
	if (!StageSettings)
	{
		return false;
	}

	const FDisplayClusterConfigurationICVFX_LightcardSettings& LightcardSettings = StageSettings->Lightcard;

	// First try use global OCIO from stage settings
	if (LightcardSettings.bEnableOuterViewportColorGrading)
	{
		if (ImplUpdateViewportColorGrading(DstViewport, BaseViewport.GetId()))
		{
			return true;
		}
	}

	// This viewport doesn't use PP
	PostprocessHelpers::ImplRemoveCustomPostprocess(DstViewport, IDisplayClusterViewport_CustomPostProcessSettings::ERenderPass::FinalPerViewport);

	return false;
}

bool FDisplayClusterViewportConfigurationHelpers_Postprocess::ImplUpdateViewportColorGrading(FDisplayClusterViewport& DstViewport, const FString& InClusterViewportId)
{
	const FDisplayClusterConfigurationICVFX_StageSettings* StageSettings = DstViewport.Configuration->GetStageSettings();
	if (!StageSettings || StageSettings->EnableColorGrading == false)
	{
		return false;
	}

	// Collect all used color grading settings into this array.
	TArray<const FDisplayClusterConfigurationViewport_ColorGradingRenderingSettings*> PostProcessList;

	const FDisplayClusterConfigurationViewport_PerViewportColorGrading* ExistPerViewportColorGrading = StageSettings->PerViewportColorGrading.FindByPredicate([InClusterViewportId](const FDisplayClusterConfigurationViewport_PerViewportColorGrading& ColorGradingProfileIt)
		{
			return ColorGradingProfileIt.bIsEnabled && ColorGradingProfileIt.ApplyPostProcessToObjects.ContainsByPredicate([InClusterViewportId](const FString& ViewportNameIt)
				{
					return InClusterViewportId.Compare(ViewportNameIt, ESearchCase::IgnoreCase) == 0;
				});
		});

	// enable entire cluster only when global settings is on
	const bool bUseEntireClusterPostProcess = StageSettings->EntireClusterColorGrading.bEnableEntireClusterColorGrading && (!ExistPerViewportColorGrading || ExistPerViewportColorGrading->bIsEntireClusterEnabled);
	
	// Cluster
	if (bUseEntireClusterPostProcess)
	{
		PostProcessList.Add(&StageSettings->EntireClusterColorGrading.ColorGradingSettings);
	}

	// Per-Viewport
	if (ExistPerViewportColorGrading)
	{
		PostProcessList.Add(&ExistPerViewportColorGrading->ColorGradingSettings);
	}

	return ImplUpdateFinalPerViewportPostProcessList(DstViewport, PostProcessList);
}

bool FDisplayClusterViewportConfigurationHelpers_Postprocess::ImplUpdateInnerFrustumColorGradingForOuterViewport(FDisplayClusterViewport& DstViewport, const FDisplayClusterConfigurationICVFX_CameraSettings& InCameraSettings)
{
	// per node color grading first (it includes all nodes blending too)
	const FString& ClusterNodeId = DstViewport.GetClusterNodeId();
	const FString& ClusterViewportId = DstViewport.GetId();

	check(!ClusterNodeId.IsEmpty());

	const FDisplayClusterConfigurationICVFX_StageSettings* StageSettings = DstViewport.Configuration->GetStageSettings();
	if (!StageSettings)
	{
		return false;
	}

	// Collect all used color grading settings into this array.
	TArray<const FDisplayClusterConfigurationViewport_ColorGradingRenderingSettings*> PostProcessList;

	const FDisplayClusterConfigurationViewport_PerNodeColorGrading* ExistPerNodeColorGrading = InCameraSettings.PerNodeColorGrading.FindByPredicate([ClusterNodeId](const FDisplayClusterConfigurationViewport_PerNodeColorGrading& ColorGradingProfileIt)
		{
			// Only allowed profiles
			return ColorGradingProfileIt.bIsEnabled && ColorGradingProfileIt.ApplyPostProcessToObjects.ContainsByPredicate([ClusterNodeId](const FString& ClusterNodeIt)
				{
					return ClusterNodeId.Compare(ClusterNodeIt, ESearchCase::IgnoreCase) == 0;
				});
		});

	const FDisplayClusterConfigurationViewport_PerViewportColorGrading* ExistPerViewportColorGrading = StageSettings->PerViewportColorGrading.FindByPredicate([ClusterViewportId](const FDisplayClusterConfigurationViewport_PerViewportColorGrading& ColorGradingProfileIt)
		{
			return ColorGradingProfileIt.bIsEnabled && ColorGradingProfileIt.ApplyPostProcessToObjects.ContainsByPredicate([ClusterViewportId](const FString& ViewportNameIt)
				{
					return ClusterViewportId.Compare(ViewportNameIt, ESearchCase::IgnoreCase) == 0;
				});
		});

	if (ExistPerNodeColorGrading)
	{
		const bool bIncludeUseEntireClusterPostProcess = StageSettings->EntireClusterColorGrading.bEnableEntireClusterColorGrading && ExistPerNodeColorGrading->bEntireClusterColorGrading;
		const bool bIncludeAllNodesColorGrading = InCameraSettings.AllNodesColorGrading.bEnableInnerFrustumAllNodesColorGrading && ExistPerNodeColorGrading->bAllNodesColorGrading;

		// Cluster
		if (bIncludeUseEntireClusterPostProcess)
		{
			// enable entire cluster only when global settings is on
			const bool bUseEntireClusterPostProcess = StageSettings->EntireClusterColorGrading.bEnableEntireClusterColorGrading && (!ExistPerViewportColorGrading || ExistPerViewportColorGrading->bIsEntireClusterEnabled);

			// Cluster
			if (bUseEntireClusterPostProcess)
			{
				PostProcessList.Add(&StageSettings->EntireClusterColorGrading.ColorGradingSettings);
			}

			// Per-Viewport
			if (ExistPerViewportColorGrading)
			{
				PostProcessList.Add(&ExistPerViewportColorGrading->ColorGradingSettings);
			}
		}

		// All Nodes
		if (bIncludeAllNodesColorGrading)
		{
			PostProcessList.Add(&InCameraSettings.AllNodesColorGrading.ColorGradingSettings);
		}

		// Per-Node
		PostProcessList.Add(&ExistPerNodeColorGrading->ColorGradingSettings);
	}
	else if (InCameraSettings.AllNodesColorGrading.bEnableInnerFrustumAllNodesColorGrading)
	{
		const bool bEnableEntireClusterColorGrading = StageSettings->EntireClusterColorGrading.bEnableEntireClusterColorGrading && InCameraSettings.AllNodesColorGrading.bEnableEntireClusterColorGrading;

		// Cluster
		if (bEnableEntireClusterColorGrading)
		{
			// enable entire cluster only when global settings is on
			const bool bUseEntireClusterPostProcess = StageSettings->EntireClusterColorGrading.bEnableEntireClusterColorGrading && (!ExistPerViewportColorGrading || ExistPerViewportColorGrading->bIsEntireClusterEnabled);

			// Cluster
			if (bUseEntireClusterPostProcess)
			{
				PostProcessList.Add(&StageSettings->EntireClusterColorGrading.ColorGradingSettings);
			}

			// Per-Viewport
			if (ExistPerViewportColorGrading)
			{
				PostProcessList.Add(&ExistPerViewportColorGrading->ColorGradingSettings);
			}
		}

		// All Nodes
		PostProcessList.Add(&InCameraSettings.AllNodesColorGrading.ColorGradingSettings);
	}

	return ImplUpdateFinalPerViewportPostProcessList(DstViewport, PostProcessList);
}

void FDisplayClusterViewportConfigurationHelpers_Postprocess::ImplApplyICVFXCameraPostProcessesToViewport(FDisplayClusterViewport& DstViewport, UDisplayClusterICVFXCameraComponent& InSceneCameraComponent, const FDisplayClusterConfigurationICVFX_CameraSettings& InCfgCameraSettings, const EDisplayClusterViewportCameraPostProcessFlags InPostProcessingFlags)
{
	const FDisplayClusterConfigurationICVFX_StageSettings* StageSettings = DstViewport.Configuration->GetStageSettings();
	if (!(StageSettings))
	{
		return;
	}

	const bool bIsLightcardViewport = EnumHasAnyFlags(DstViewport.GetRenderSettingsICVFX().RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::Lightcard);
	if (bIsLightcardViewport)
	{
		// LC viewports should not use settings from the ICVFX camera.
		// Note: This use case needs to be clarified.
		return;
	}

	// This function should only be used for InCamera and Outer viewports.
	const bool bIsInCameraViewport = EnumHasAnyFlags(DstViewport.GetRenderSettingsICVFX().RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::InCamera);
	const bool bIsOuterViewport = !EnumHasAnyFlags(DstViewport.GetRenderSettingsICVFX().RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::InternalResource);
	if (!bIsInCameraViewport && !bIsOuterViewport)
	{
		return;
	}

	// All ICVFX camera post-process settings should be moved here.
	
	// Motion blur:
	if (EnumHasAnyFlags(InPostProcessingFlags, EDisplayClusterViewportCameraPostProcessFlags::EnableICVFXMotionBlur))
	{
		DstViewport.UpdateConfiguration_CameraMotionBlur(GetICVFXCameraMotionBlurParameters(*StageSettings, InSceneCameraComponent, InCfgCameraSettings));
	}

	// Depth of field
	if (EnumHasAnyFlags(InPostProcessingFlags, EDisplayClusterViewportCameraPostProcessFlags::EnableICVFXDepthOfFieldCompensation))
	{
		DstViewport.UpdateConfiguration_CameraDepthOfField(GetICVFXCameraDepthOfFieldParameters(*StageSettings, InSceneCameraComponent, InCfgCameraSettings));
	}

	// Always use postprocess from the actual camera
	if (EnumHasAnyFlags(InPostProcessingFlags, EDisplayClusterViewportCameraPostProcessFlags::EnablePostProcess))
	{
		const bool bUseCameraPostprocess = true; // use internal rules of UDisplayClusterICVFXCameraComponent

		// All logic was moved to the UDisplayClusterICVFXCameraComponent::GetCameraView() virtual function.
		FMinimalViewInfo DesiredView;

		// PP is now always derived from the actual CineCamera component (ICVFXCameraComponent or from an external CineCameraActor).
		UCineCameraComponent* ActualCineCameraComponent = InSceneCameraComponent.GetActualCineCameraComponent();
		if (IDisplayClusterViewport::GetCameraComponentView(ActualCineCameraComponent, DstViewport.GetConfiguration().GetRootActorWorldDeltaSeconds(), bUseCameraPostprocess, DesiredView) && DesiredView.PostProcessBlendWeight > 0)
		{
			// Applies a filter to the post-processing settings.
			FilterPostProcessSettings(DesiredView.PostProcessSettings, InPostProcessingFlags);

			// Send camera postprocess to override
			DstViewport.GetViewport_CustomPostProcessSettings().AddCustomPostProcess(IDisplayClusterViewport_CustomPostProcessSettings::ERenderPass::Override, DesiredView.PostProcessSettings, DesiredView.PostProcessBlendWeight, true);
		}
	}

	// check if frustum color grading is enabled
	if (EnumHasAnyFlags(InPostProcessingFlags, EDisplayClusterViewportCameraPostProcessFlags::EnableICVFXColorGrading))
	{
		if (StageSettings->EnableColorGrading && InCfgCameraSettings.EnableInnerFrustumColorGrading)
		{
			if (bIsInCameraViewport)
			{
				// Use this function for all InCamera viewports
				ImplUpdateInnerFrustumColorGrading(DstViewport, InCfgCameraSettings);
			}
			else if (bIsOuterViewport)
			{
				// Use this function for all Outer viewports
				ImplUpdateInnerFrustumColorGradingForOuterViewport(DstViewport, InCfgCameraSettings);
			}
		}
	}
}

void FDisplayClusterViewportConfigurationHelpers_Postprocess::FilterPostProcessSettings(FPostProcessSettings& InOutPostProcessSettings, const EDisplayClusterViewportCameraPostProcessFlags InPostProcessingFlags)
{
	if (!EnumHasAnyFlags(InPostProcessingFlags, EDisplayClusterViewportCameraPostProcessFlags::EnableDepthOfField))
	{
		// Do not override DoF PP settings from the CineCamera
		InOutPostProcessSettings.bOverride_DepthOfFieldFstop = false;
		InOutPostProcessSettings.bOverride_DepthOfFieldMinFstop = false;
		InOutPostProcessSettings.bOverride_DepthOfFieldBladeCount = false;
		InOutPostProcessSettings.bOverride_DepthOfFieldFocalDistance = false;
		InOutPostProcessSettings.bOverride_DepthOfFieldSensorWidth = false;
		InOutPostProcessSettings.bOverride_DepthOfFieldSqueezeFactor = false;
	}
}

FDisplayClusterViewport_CameraMotionBlur FDisplayClusterViewportConfigurationHelpers_Postprocess::GetICVFXCameraMotionBlurParameters(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings, UDisplayClusterICVFXCameraComponent& InSceneCameraComponent, const FDisplayClusterConfigurationICVFX_CameraSettings& InCfgCameraSettings)
{
	FDisplayClusterViewport_CameraMotionBlur OutParameters;
	OutParameters.Mode = EDisplayClusterViewport_CameraMotionBlur::Undefined;

	switch (InCfgCameraSettings.CameraMotionBlur.MotionBlurMode)
	{
	case EDisplayClusterConfigurationCameraMotionBlurMode::Override:
		if (ADisplayClusterRootActor* SceneRootActor = Cast<ADisplayClusterRootActor>(InSceneCameraComponent.GetOwner()))
		{
			UDisplayClusterCameraComponent* OuterCamera = SceneRootActor->GetDefaultCamera();
			if (OuterCamera)
			{
				OutParameters.Mode = EDisplayClusterViewport_CameraMotionBlur::Override;

				OutParameters.CameraLocation = OuterCamera->GetComponentLocation();
				OutParameters.CameraRotation = OuterCamera->GetComponentRotation();

				OutParameters.TranslationScale = InCfgCameraSettings.CameraMotionBlur.TranslationScale;
			}
		}
		break;

	case EDisplayClusterConfigurationCameraMotionBlurMode::On:
		OutParameters.Mode = EDisplayClusterViewport_CameraMotionBlur::On;
		break;

	case EDisplayClusterConfigurationCameraMotionBlurMode::Off:
	default:
		OutParameters.Mode = EDisplayClusterViewport_CameraMotionBlur::Off;
		break;
	}

	return OutParameters;
}

FDisplayClusterViewport_CameraDepthOfField FDisplayClusterViewportConfigurationHelpers_Postprocess::GetICVFXCameraDepthOfFieldParameters(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings, UDisplayClusterICVFXCameraComponent& InSceneCameraComponent, const FDisplayClusterConfigurationICVFX_CameraSettings& InCfgCameraSettings)
{
	FDisplayClusterViewport_CameraDepthOfField OutParameters;

	OutParameters.bEnableDepthOfFieldCompensation = InCfgCameraSettings.CameraDepthOfField.bEnableDepthOfFieldCompensation;
	OutParameters.DistanceToWall = InCfgCameraSettings.CameraDepthOfField.DistanceToWall;
	OutParameters.DistanceToWallOffset = InCfgCameraSettings.CameraDepthOfField.DistanceToWallOffset;
	OutParameters.CompensationLUT = InCfgCameraSettings.CameraDepthOfField.GetCompensationLUT(InStageSettings);

	return OutParameters;
}

void FDisplayClusterViewportConfigurationHelpers_Postprocess::UpdateCustomPostProcessSettings(FDisplayClusterViewport& DstViewport, const FDisplayClusterConfigurationViewport_CustomPostprocess& InCustomPostprocessConfiguration)
{
	// update postprocess settings (Start, Override, Final)
	PostprocessHelpers::ImplUpdateCustomPostprocess(DstViewport, InCustomPostprocessConfiguration.Start.bIsEnabled, InCustomPostprocessConfiguration.Start, IDisplayClusterViewport_CustomPostProcessSettings::ERenderPass::Start);
	PostprocessHelpers::ImplUpdateCustomPostprocess(DstViewport, InCustomPostprocessConfiguration.Override.bIsEnabled, InCustomPostprocessConfiguration.Override, IDisplayClusterViewport_CustomPostProcessSettings::ERenderPass::Override);
	PostprocessHelpers::ImplUpdateCustomPostprocess(DstViewport, InCustomPostprocessConfiguration.Final.bIsEnabled, InCustomPostprocessConfiguration.Final, IDisplayClusterViewport_CustomPostProcessSettings::ERenderPass::Final);
}

void FDisplayClusterViewportConfigurationHelpers_Postprocess::UpdatePerViewportPostProcessSettings(FDisplayClusterViewport& DstViewport)
{
	if (!ImplUpdateViewportColorGrading(DstViewport, DstViewport.GetId()))
	{
		// This viewport doesn't use PP
		PostprocessHelpers::ImplRemoveCustomPostprocess(DstViewport, IDisplayClusterViewport_CustomPostProcessSettings::ERenderPass::FinalPerViewport);
	}
}

void FDisplayClusterViewportConfigurationHelpers_Postprocess::CopyBlendPostProcessSettings(FPostProcessSettings& OutputPP, const FDisplayClusterConfigurationViewport_ColorGradingRenderingSettings& InPPSettings)
{
	PostprocessHelpers::ImplBlendPostProcessSettings(OutputPP, InPPSettings, nullptr, nullptr, nullptr);
}

void FDisplayClusterViewportConfigurationHelpers_Postprocess::PerNodeBlendPostProcessSettings(FPostProcessSettings& OutputPP, const FDisplayClusterConfigurationViewport_ColorGradingRenderingSettings& ClusterPPSettings, const FDisplayClusterConfigurationViewport_ColorGradingRenderingSettings& ViewportPPSettings, const FDisplayClusterConfigurationViewport_ColorGradingRenderingSettings& PerNodePPSettings)
{
	PostprocessHelpers::ImplBlendPostProcessSettings(OutputPP, ClusterPPSettings, &ViewportPPSettings, &PerNodePPSettings, nullptr);
}

void FDisplayClusterViewportConfigurationHelpers_Postprocess::BlendPostProcessSettings(FPostProcessSettings& OutputPP, const FDisplayClusterConfigurationViewport_ColorGradingRenderingSettings& ClusterPPSettings, const FDisplayClusterConfigurationViewport_ColorGradingRenderingSettings& ViewportPPSettings)
{
	PostprocessHelpers::ImplBlendPostProcessSettings(OutputPP, ClusterPPSettings, &ViewportPPSettings, nullptr, nullptr);
}

void FDisplayClusterViewportConfigurationHelpers_Postprocess::CopyPPSStructConditional(FDisplayClusterConfigurationViewport_ColorGradingRenderingSettings* OutViewportPPSettings, FPostProcessSettings* InPPS)
{
	PostprocessHelpers::ImplCopyPPSStruct(true, OutViewportPPSettings, InPPS);
}

void FDisplayClusterViewportConfigurationHelpers_Postprocess::CopyPPSStruct(FDisplayClusterConfigurationViewport_ColorGradingRenderingSettings* OutViewportPPSettings, FPostProcessSettings* InPPS)
{
	PostprocessHelpers::ImplCopyPPSStruct(false, OutViewportPPSettings, InPPS);
}

bool FDisplayClusterViewportConfigurationHelpers_Postprocess::ImplUpdateFinalPerViewportPostProcessList(FDisplayClusterViewport& DstViewport, const TArray<const FDisplayClusterConfigurationViewport_ColorGradingRenderingSettings*>& InPostProcessList)
{
	FPostProcessSettings FinalPostProcessSettings;
	float BlendWeight = 0;

	const TArray<const FDisplayClusterConfigurationViewport_ColorGradingRenderingSettings*> PPList = InPostProcessList.FilterByPredicate([](const FDisplayClusterConfigurationViewport_ColorGradingRenderingSettings* ColorGradingPtr)
		{
			// Skip PostProcess with zero weight
			return ColorGradingPtr && ColorGradingPtr->BlendWeight > 0;
		});

	// Now blend only up to 4 PP
	check(PPList.Num() < 4);

	switch (PPList.Num())
	{
	case 1:
		PostprocessHelpers::ImplBlendPostProcessSettings(FinalPostProcessSettings, *PPList[0], nullptr, nullptr, nullptr);
		BlendWeight = PPList[0]->BlendWeight;
		break;

	case 2:
		PostprocessHelpers::ImplBlendPostProcessSettings(FinalPostProcessSettings, *PPList[0], PPList[1], nullptr, nullptr);
		BlendWeight = PPList[0]->BlendWeight * PPList[1]->BlendWeight;
		break;

	case 3:
		PostprocessHelpers::ImplBlendPostProcessSettings(FinalPostProcessSettings, *PPList[0], PPList[1], PPList[2], nullptr);
		BlendWeight = PPList[0]->BlendWeight * PPList[1]->BlendWeight * PPList[2]->BlendWeight;
		break;

	case 4:
		PostprocessHelpers::ImplBlendPostProcessSettings(FinalPostProcessSettings, *PPList[0], PPList[1], PPList[2], PPList[3]);
		BlendWeight = PPList[0]->BlendWeight * PPList[1]->BlendWeight * PPList[2]->BlendWeight * PPList[3]->BlendWeight;
		break;

	default:
	case 0:
		return false;
	}

	if (BlendWeight > 0)
	{
		DstViewport.GetViewport_CustomPostProcessSettings().AddCustomPostProcess(IDisplayClusterViewport_CustomPostProcessSettings::ERenderPass::FinalPerViewport, FinalPostProcessSettings, BlendWeight, true);

		return true;
	}

	return false;
}
