// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Render/Viewport/Containers/DisplayClusterViewport_Enums.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_CameraMotionBlur.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_CameraDepthOfField.h"

#include "DisplayClusterConfigurationTypes_Enums.h"

class FDisplayClusterViewport;
class UDisplayClusterICVFXCameraComponent;

struct FPostProcessSettings;
struct FDisplayClusterConfigurationViewport_ColorGradingRenderingSettings;
struct FDisplayClusterConfigurationViewport_CustomPostprocess;
struct FDisplayClusterConfigurationICVFX_CameraSettings;
struct FDisplayClusterConfigurationICVFX_StageSettings;

/**
* Postprocess configuration helper class.
*/
class FDisplayClusterViewportConfigurationHelpers_Postprocess
{
public:
	static void UpdateCustomPostProcessSettings(FDisplayClusterViewport& DstViewport, const FDisplayClusterConfigurationViewport_CustomPostprocess& InCustomPostprocessConfiguration);
	static void UpdatePerViewportPostProcessSettings(FDisplayClusterViewport& DstViewport);

	static bool UpdateLightcardPostProcessSettings(FDisplayClusterViewport& DstViewport, FDisplayClusterViewport& BaseViewport);

	// return true when same settings used for both viewports
	static bool IsInnerFrustumViewportSettingsEqual(const FDisplayClusterViewport& InViewport1, const FDisplayClusterViewport& InViewport2, const FDisplayClusterConfigurationICVFX_CameraSettings& InCameraSettings);

public:
	static void BlendPostProcessSettings(FPostProcessSettings& OutputPP, const FDisplayClusterConfigurationViewport_ColorGradingRenderingSettings& ClusterPPSettings, const FDisplayClusterConfigurationViewport_ColorGradingRenderingSettings& ViewportPPSettings);
	static void CopyPPSStructConditional(FDisplayClusterConfigurationViewport_ColorGradingRenderingSettings* OutViewportPPSettings, FPostProcessSettings* InPPS);
	static void CopyPPSStruct(FDisplayClusterConfigurationViewport_ColorGradingRenderingSettings* OutViewportPPSettings, FPostProcessSettings* InPPS);

	static void CopyBlendPostProcessSettings(FPostProcessSettings& OutputPP, const FDisplayClusterConfigurationViewport_ColorGradingRenderingSettings& InPPSettings);
	static void PerNodeBlendPostProcessSettings(FPostProcessSettings& OutputPP, const FDisplayClusterConfigurationViewport_ColorGradingRenderingSettings& ClusterPPSettings, const FDisplayClusterConfigurationViewport_ColorGradingRenderingSettings& ViewportPPSettings, const FDisplayClusterConfigurationViewport_ColorGradingRenderingSettings& PerNodePPSettings);

public:
	/** Initialize PP settings for the viewport from the ICVFX camera component. */
	static void ImplApplyICVFXCameraPostProcessesToViewport(FDisplayClusterViewport& DstViewport, UDisplayClusterICVFXCameraComponent& InSceneCameraComponent, const FDisplayClusterConfigurationICVFX_CameraSettings& InCfgCameraSettings, const EDisplayClusterViewportCameraPostProcessFlags InPostProcessingFlags);

	/** Applies a filter to the post-processing settings.
	* Note: If DoF is disabled in InPostProcessingFlags, this causes it to be removed from the PP settings as well.
	*/
	static void FilterPostProcessSettings(FPostProcessSettings& InOutPostProcessSettings, const EDisplayClusterViewportCameraPostProcessFlags InPostProcessingFlags);

private:
	static bool ImplUpdateInnerFrustumColorGrading(FDisplayClusterViewport& DstViewport, const FDisplayClusterConfigurationICVFX_CameraSettings& InCameraSettings);
	static bool ImplUpdateInnerFrustumColorGradingForOuterViewport(FDisplayClusterViewport& DstViewport, const FDisplayClusterConfigurationICVFX_CameraSettings& InCameraSettings);
	static bool ImplUpdateViewportColorGrading(FDisplayClusterViewport& DstViewport, const FString& InClusterViewportId);

	/** Gets the MotionBlur parameters of camera*/
	static FDisplayClusterViewport_CameraMotionBlur GetICVFXCameraMotionBlurParameters(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings, UDisplayClusterICVFXCameraComponent& InSceneCameraComponent, const FDisplayClusterConfigurationICVFX_CameraSettings& InCfgCameraSettings);

	/** Gets the depth of field parameters to store on the display cluster viewport */
	static FDisplayClusterViewport_CameraDepthOfField GetICVFXCameraDepthOfFieldParameters(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings, UDisplayClusterICVFXCameraComponent& InSceneCameraComponent, const FDisplayClusterConfigurationICVFX_CameraSettings& InCfgCameraSettings);

	/** Blend input list of postprocess and set as a FinalPerViewport. */
	static bool ImplUpdateFinalPerViewportPostProcessList(FDisplayClusterViewport& DstViewport, const TArray<const FDisplayClusterConfigurationViewport_ColorGradingRenderingSettings*>& InPostProcessList);
};
