// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Viewport/Configuration/DisplayClusterViewportConfigurationHelpers_RenderFrameSettings.h"
#include "Render/Viewport/Configuration/DisplayClusterViewportConfiguration.h"
#include "Render/Viewport/RenderFrame/DisplayClusterRenderFrameSettings.h"
#include "Render/Viewport/DisplayClusterViewportManager.h"
#include "Render/Viewport/Preview/DisplayClusterViewportManagerPreview.h"
#include "Render/Viewport/Postprocess/DisplayClusterViewportPostProcessManager.h"

#include "DisplayClusterRootActor.h"
#include "IDisplayCluster.h"

#include "DisplayClusterConfigurationTypes_Viewport.h"
#include "Engine/RendererSettings.h"

//////////////////////////////////////////////////////////////////////////////////////////////
int32 GDisplayClusterPreviewAllowMultiGPURendering = 0;
static FAutoConsoleVariableRef CVarDisplayClusterPreviewAllowMultiGPURendering(
	TEXT("nDisplay.render.preview.AllowMultiGPURendering"),
	GDisplayClusterPreviewAllowMultiGPURendering,
	TEXT("Allow mGPU for preview rendering (0 == disabled)"),
	ECVF_RenderThreadSafe
);

int32 GDisplayClusterPreviewMultiGPURenderingMinIndex = 0;
static FAutoConsoleVariableRef CVarDisplayClusterPreviewMultiGPURenderingMinIndex(
	TEXT("nDisplay.render.preview.MultiGPURenderingMinIndex"),
	GDisplayClusterPreviewMultiGPURenderingMinIndex,
	TEXT("Distribute mGPU render on GPU from #min to #max indices"),
	ECVF_RenderThreadSafe
);

int32 GDisplayClusterPreviewMultiGPURenderingMaxIndex = 0;
static FAutoConsoleVariableRef CVarDisplayClusterPreviewMultiGPURenderingMaxIndex(
	TEXT("nDisplay.render.preview.MultiGPURenderingMaxIndex"),
	GDisplayClusterPreviewMultiGPURenderingMaxIndex,
	TEXT("Distribute mGPU render on GPU from #min to #max indices"),
	ECVF_RenderThreadSafe
);

int32 GDisplayClusterCrossGPUTransferEnable = 0;
static FAutoConsoleVariableRef CDisplayClusterCrossGPUTransferEnable(
	TEXT("nDisplay.render.CrossGPUTransfer.Enable"),
	GDisplayClusterCrossGPUTransferEnable,
	TEXT("Enable cross-GPU transfers using nDisplay implementation (0 - disable, default) \n")
	TEXT("That replaces the default cross-GPU transfers using UE Core for the nDisplay viewports viewfamilies.\n"),
	ECVF_RenderThreadSafe
);

int32 GDisplayClusterCrossGPUTransferLockSteps = 1;
static FAutoConsoleVariableRef CDisplayClusterCrossGPUTransferLockSteps(
	TEXT("nDisplay.render.CrossGPUTransfer.LockSteps"),
	GDisplayClusterCrossGPUTransferLockSteps,
	TEXT("The bLockSteps parameter is simply passed to the FTransferResourceParams structure. (0 - disable)\n")
	TEXT("Whether the GPUs must handshake before and after the transfer. Required if the texture rect is being written to in several render passes.\n")
	TEXT("Otherwise, minimal synchronization will be used.\n"),
	ECVF_RenderThreadSafe
);

int32 GDisplayClusterCrossGPUTransferPullData = 1;
static FAutoConsoleVariableRef CVarDisplayClusterCrossGPUTransferPullData(
	TEXT("nDisplay.render.CrossGPUTransfer.PullData"),
	GDisplayClusterCrossGPUTransferPullData,
	TEXT("The bPullData parameter is simply passed to the FTransferResourceParams structure. (0 - disable)\n")
	TEXT("Whether the data is read by the dest GPU, or written by the src GPU (not allowed if the texture is a backbuffer)\n"),
	ECVF_RenderThreadSafe
);

// Choose method to preserve alpha channel
int32 GDisplayClusterAlphaChannelCaptureMode = (uint8)ECVarDisplayClusterAlphaChannelCaptureMode::FXAA;
static FAutoConsoleVariableRef CVarDisplayClusterAlphaChannelCaptureMode(
	TEXT("nDisplay.render.AlphaChannelCaptureMode"),
	GDisplayClusterAlphaChannelCaptureMode,
	TEXT("Alpha channel capture mode (FXAA - default)\n")
	TEXT("0 - Disabled\n")
	TEXT("1 - ThroughTonemapper\n")
	TEXT("2 - FXAA\n")
	TEXT("3 - Copy [experimental]\n")
	TEXT("4 - CopyAA [experimental]\n"),
	ECVF_RenderThreadSafe
);

int32 GDisplayClusterEnableAlphaChannelRendering = 0;
static FAutoConsoleVariableRef CVarDisplayClusterEnableAlphaChannelRendering(
	TEXT("DC.EnableAlphaChannelRendering"),
	GDisplayClusterEnableAlphaChannelRendering,
	TEXT("Enable alpha output for all viewports (0 = disabled, 1 = enabled)\n")
	TEXT("Requires CVar 'r.PostProcessing.PropagateAlpha' to be enabled.\n"),
	ECVF_RenderThreadSafe
);

namespace UE::DisplayClusterViewport::Configuration::RenderFrameSettings
{
	/**
	* Returns true if "Alpha Output" is enabled in Project Settings
	* (backed by the CVar r.PostProcessing.PropagateAlpha).
	*/
	static bool IsProjectSettingAlphaOutputEnabled()
	{
		static const auto CVarPropagateAlpha = IConsoleManager::Get().FindConsoleVariable(TEXT("r.PostProcessing.PropagateAlpha"));
		if (CVarPropagateAlpha)
		{
			const bool bPropagateAlpha = CVarPropagateAlpha->GetBool();

			return bPropagateAlpha;
		}

		return false;
	}
};

///////////////////////////////////////////////////////////////////
// FDisplayClusterViewportConfiguration
///////////////////////////////////////////////////////////////////
EDisplayClusterRenderFrameAlphaChannelCaptureMode FDisplayClusterViewportConfigurationHelpers_RenderFrameSettings::GetAlphaChannelCaptureMode()
{
	using namespace UE::DisplayClusterViewport::Configuration::RenderFrameSettings;

	const ECVarDisplayClusterAlphaChannelCaptureMode AlphaChannelCaptureMode = (ECVarDisplayClusterAlphaChannelCaptureMode)FMath::Clamp(GDisplayClusterAlphaChannelCaptureMode, 0, (int32)ECVarDisplayClusterAlphaChannelCaptureMode::COUNT - 1);
	switch (AlphaChannelCaptureMode)
	{
	case ECVarDisplayClusterAlphaChannelCaptureMode::ThroughTonemapper:
		return IsProjectSettingAlphaOutputEnabled()
			// Capture alpha as processed through the tonemapper.
			? EDisplayClusterRenderFrameAlphaChannelCaptureMode::ThroughTonemapper
			// Disable alpha capture when r.PostProcessing.PropagateAlpha is not enabled.
			: EDisplayClusterRenderFrameAlphaChannelCaptureMode::None;

	case ECVarDisplayClusterAlphaChannelCaptureMode::FXAA:
		return EDisplayClusterRenderFrameAlphaChannelCaptureMode::FXAA;

	case ECVarDisplayClusterAlphaChannelCaptureMode::Copy:
		return EDisplayClusterRenderFrameAlphaChannelCaptureMode::Copy;

	case ECVarDisplayClusterAlphaChannelCaptureMode::CopyAA:
		return EDisplayClusterRenderFrameAlphaChannelCaptureMode::CopyAA;

	case ECVarDisplayClusterAlphaChannelCaptureMode::Disabled:
	default:
		break;
	}

	return EDisplayClusterRenderFrameAlphaChannelCaptureMode::None;
}

bool FDisplayClusterViewportConfigurationHelpers_RenderFrameSettings::UpdateRenderFrameConfiguration(
	EDisplayClusterRenderFrameMode InRenderMode,
	const FString& InClusterNodeId,
	FDisplayClusterViewportConfiguration& InOutConfiguration)
{
	using namespace UE::DisplayClusterViewport::Configuration::RenderFrameSettings;

	ADisplayClusterRootActor* ConfigurationRootActor = InOutConfiguration.GetRootActor(EDisplayClusterRootActorType::Configuration);
	if (!(ConfigurationRootActor))
	{
		// If the ConfigurationRootActor is not defined, initialization cannot be performed.
		return false;
	}

	// Use settings from the previous frame as base values.
	// Note: Reset to default values manually if necessary.
	FDisplayClusterRenderFrameSettings NewRenderFrameSettings = InOutConfiguration.GetRenderFrameSettings();

	// Set current rendering mode
	NewRenderFrameSettings.RenderMode = InRenderMode;

	// Preview Settings : experimental mGPU feature
	NewRenderFrameSettings.PreviewMultiGPURendering.Reset();
	if (GDisplayClusterPreviewAllowMultiGPURendering)
	{
		const int32 MinGPUIndex = FMath::Max(0, GDisplayClusterPreviewMultiGPURenderingMinIndex);
		const int32 MaxGPUIndex = FMath::Max(MinGPUIndex, GDisplayClusterPreviewMultiGPURenderingMaxIndex);

		NewRenderFrameSettings.PreviewMultiGPURendering = FIntPoint(MinGPUIndex, MaxGPUIndex);
	}

	// Support alpha channel capture
	NewRenderFrameSettings.AlphaChannelCaptureMode = GetAlphaChannelCaptureMode();

	// True if alpha channel should be written to all viewport outputs.
	// Note: Controlled by "DC.EnableAlphaChannelRendering" and "r.PostProcessing.PropagateAlpha".
	NewRenderFrameSettings.bEnableAlphaOutput =
		GDisplayClusterEnableAlphaChannelRendering
		&& IsProjectSettingAlphaOutputEnabled();

	// Update RenderFrame configuration
	const FDisplayClusterConfigurationRenderFrame& InRenderFrameConfiguration = ConfigurationRootActor->GetRenderFrameSettings();
	{
		// Global RTT sizes mults
		NewRenderFrameSettings.ClusterRenderTargetRatioMult = InRenderFrameConfiguration.ClusterRenderTargetRatioMult;
		NewRenderFrameSettings.ClusterICVFXInnerViewportRenderTargetRatioMult = InRenderFrameConfiguration.ClusterICVFXInnerViewportRenderTargetRatioMult;
		NewRenderFrameSettings.ClusterICVFXOuterViewportRenderTargetRatioMult = InRenderFrameConfiguration.ClusterICVFXOuterViewportRenderTargetRatioMult;

		// Global Buffer ratio mults
		NewRenderFrameSettings.ClusterBufferRatioMult = InRenderFrameConfiguration.ClusterBufferRatioMult;
		NewRenderFrameSettings.ClusterICVFXInnerFrustumBufferRatioMult = InRenderFrameConfiguration.ClusterICVFXInnerFrustumBufferRatioMult;
		NewRenderFrameSettings.ClusterICVFXOuterViewportBufferRatioMult = InRenderFrameConfiguration.ClusterICVFXOuterViewportBufferRatioMult;

		// Allow warpblend render
		NewRenderFrameSettings.bAllowWarpBlend = InRenderFrameConfiguration.bAllowWarpBlend;

		// Performance: nDisplay has its own implementation of cross-GPU transfer.
		NewRenderFrameSettings.CrossGPUTransfer.bEnable = GDisplayClusterCrossGPUTransferEnable != 0;
		NewRenderFrameSettings.CrossGPUTransfer.bLockSteps = GDisplayClusterCrossGPUTransferLockSteps != 0;
		NewRenderFrameSettings.CrossGPUTransfer.bPullData = GDisplayClusterCrossGPUTransferPullData != 0;
	}

	if (NewRenderFrameSettings.IsPreviewRendering())
	{
		// Preview use its own rendering pipeline
		NewRenderFrameSettings.bUseDisplayClusterRenderDevice = false;
	}
	else
	{
		// Configuring the use of DC RenderDevice:
		static IDisplayCluster& DisplayClusterAPI = IDisplayCluster::Get();
		NewRenderFrameSettings.bUseDisplayClusterRenderDevice = (GEngine->StereoRenderingDevice.IsValid() && DisplayClusterAPI.GetOperationMode() == EDisplayClusterOperationMode::Cluster);
	}

	// Settings related to the cluster node.
	{
		// Reset to default values
		NewRenderFrameSettings.CurrentNode = FDisplayClusterRenderFrameSettings::FCurrentClusterNodeSettings();

		// Assign the current node Id.
		// Note: SetRenderFrameSettings() later also calls SetClusterNodeId() to raise the required flags.
		NewRenderFrameSettings.CurrentNode.Id = InClusterNodeId;

		// Configure for current node
		if (!InClusterNodeId.IsEmpty())
		{
			const UDisplayClusterConfigurationData* CfgData = ConfigurationRootActor->GetConfigData();
			if (const UDisplayClusterConfigurationClusterNode* CfgNodeData = CfgData ? CfgData->GetNode(InClusterNodeId) : nullptr)
			{
				// Headless node
				NewRenderFrameSettings.CurrentNode.bRenderHeadless = CfgNodeData->bRenderHeadless;

				// Backbuffer media output
				NewRenderFrameSettings.CurrentNode.bHasBackbufferMediaOutput =
					InOutConfiguration.IsMediaAvailable()
					&& CfgNodeData->MediaSettings.bEnable
					&& CfgNodeData->MediaSettings.IsMediaOutputAssigned();
			}
		}
	}

	// Applies new render frame settings to the configuration.
	InOutConfiguration.SetRenderFrameSettings(NewRenderFrameSettings);

	return true;
}

void FDisplayClusterViewportConfigurationHelpers_RenderFrameSettings::PostUpdateRenderFrameConfiguration(FDisplayClusterViewportConfiguration& InOutConfiguration)
{
	if (const FDisplayClusterViewportManager* ViewportManager = InOutConfiguration.GetViewportManagerImpl())
	{
		FDisplayClusterRenderFrameSettings NewRenderFrameSettings = InOutConfiguration.GetRenderFrameSettings();

		// Postprocess flags
		NewRenderFrameSettings.bUseAdditionalFrameTargetableForPostprocess = ViewportManager->PostProcessManager->ShouldUseAdditionalFrameTargetableResource();
		NewRenderFrameSettings.bUseFullSizeFrameTargetableForPostprocess = ViewportManager->PostProcessManager->ShouldUseFullSizeFrameTargetableResource();

		// Update immediately, since other ViewportManager functions will use this value
		InOutConfiguration.SetRenderFrameSettings(NewRenderFrameSettings);

		// Update global flags that control viewport resources
		NewRenderFrameSettings.bShouldUseOutputFrameTargetableResources = ViewportManager->ShouldUseOutputFrameTargetableResources();
		NewRenderFrameSettings.bShouldUseAdditionalFrameTargetableResource = ViewportManager->ShouldUseAdditionalFrameTargetableResource();
		NewRenderFrameSettings.bShouldUseFullSizeFrameTargetableResource = ViewportManager->ShouldUseFullSizeFrameTargetableResource();

		// Apply new settings:
		InOutConfiguration.SetRenderFrameSettings(NewRenderFrameSettings);
	}
}
