// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Viewport/Configuration/DisplayClusterViewportConfigurationHelpers_Tile.h"
#include "Render/Viewport/Configuration/DisplayClusterViewportConfigurationHelpers.h"

#include "Misc/DisplayClusterGlobals.h"
#include "Misc/DisplayClusterLog.h"

#include "Render/Viewport/DisplayClusterViewport.h"
#include "Render/Viewport/DisplayClusterViewportManager.h"
#include "Render/Viewport/DisplayClusterViewportHelpers.h"
#include "Render/Viewport/Configuration/DisplayClusterViewportConfiguration.h"

#include "DisplayClusterEnums.h"
#include "DisplayClusterConfigurationTypes_ICVFX.h"
#include "DisplayClusterConfigurationTypes_Media.h"
#include "DisplayClusterConfigurationTypes_Tile.h"
#include "DisplayClusterProjectionStrings.h"

void FDisplayClusterViewportConfigurationHelpers_Tile::UpdateICVFXCameraViewportTileSettings(FDisplayClusterViewport& InSourceViewport, const FDisplayClusterConfigurationMediaICVFX& InCameraMediaSettings)
{
	if (!InSourceViewport.CanSplitIntoTiles() || !InCameraMediaSettings.ShouldMediaICVFXSplitIntoTiles())
	{
		// Disable tile splittings for this viewport.
		InSourceViewport.GetRenderSettingsImpl().TileSettings = { };
		return;
	}

	// Generate overscan settings
	const FDisplayClusterViewport_OverscanSettings OverscanSettings = FDisplayClusterViewportConfigurationHelpers_Tile::GetTileOverscanSettings(InCameraMediaSettings.TileOverscan);

	// Get additional tile flags
	EDisplayClusterViewportTileFlags TileFlags = InCameraMediaSettings.GetMediaICVFXTileFlags(InSourceViewport.GetClusterNodeId());

	if (InCameraMediaSettings.TileOverscan.bOptimizeTileOverscan)
	{
		TileFlags |= EDisplayClusterViewportTileFlags::OptimizeTileOverscan;
	}

	// Set this viewport as a source for tile rendering.
	FDisplayClusterViewport_TileSettings& OutTileSettings = InSourceViewport.GetRenderSettingsImpl().TileSettings;
	OutTileSettings = FDisplayClusterViewport_TileSettings(InCameraMediaSettings.TiledSplitLayout, OverscanSettings, TileFlags);
}

FIntRect FDisplayClusterViewportConfigurationHelpers_Tile::GetDestRect(const FDisplayClusterViewport_TileSettings& InTileSettings, const FIntRect& InSourceRect)
{
	check(InTileSettings.GetType() == EDisplayClusterViewportTileType::Tile);

	const FIntPoint SourceSize(InSourceRect.Width(), InSourceRect.Height());
	const FIntPoint TileSize(FMath::RoundToZero(float(SourceSize.X) / InTileSettings.GetSize().X), FMath::RoundToZero(float(SourceSize.Y) / InTileSettings.GetSize().Y));
	const FIntPoint TilePos(InTileSettings.GetPos().X * TileSize.X, InTileSettings.GetPos().Y * TileSize.Y);

	// Dest rect min value:
	const FIntPoint DestPos = InSourceRect.Min + TilePos;

	// Dest rect size
	FIntPoint DestSize(TileSize);

	// The source size may not be divisible, and some pixels may be lost; then they will need to be restored to the edge tiles.
	if ((InTileSettings.GetPos().X + 1) == InTileSettings.GetSize().X)
	{
		DestSize.X = SourceSize.X - TilePos.X;
	}
	if ((InTileSettings.GetPos().Y + 1) == InTileSettings.GetSize().Y)
	{
		DestSize.Y = SourceSize.Y - TilePos.Y;
	}

	return FIntRect(DestPos, DestPos + DestSize);
}

FDisplayClusterViewport* FDisplayClusterViewportConfigurationHelpers_Tile::GetOrCreateTileViewport(FDisplayClusterViewport& InSourceViewport, const FIntPoint& InTilePos)
{
	// Note: At this point, the viewports should already be configured.

	const FDisplayClusterViewport_RenderSettings& SourceRenderSettings = InSourceViewport.GetRenderSettings();
	if (SourceRenderSettings.TileSettings.GetType() != EDisplayClusterViewportTileType::Source)
	{
		return nullptr;
	}

	const FIntPoint& InTileSize = SourceRenderSettings.TileSettings.GetSize();
	const EDisplayClusterViewportFlags TileViewportUsage = InSourceViewport.ViewportFlags | EDisplayClusterViewportFlags::TileViewport; // Combine flags on top;
	const FString& TileViewportBaseId = InSourceViewport.ViewportBaseId;

	FDisplayClusterViewport* TileViewport = InSourceViewport.Configuration->FindViewport(
		TileViewportBaseId, TileViewportUsage, &InTilePos);
	if (!TileViewport)
	{
		TileViewport = InSourceViewport.Configuration->CreateViewport(
			TileViewportBaseId,
			InSourceViewport.Configuration->CreateProjectionPolicy_TileViewport(TileViewportBaseId, TileViewportUsage, InTilePos),
			TileViewportUsage,
			&InTilePos);
	}

	// Update tile viewport settings from the source viewport
	// Note: The source viewport must already be configured.
	if (TileViewport)
	{
		// Reset runtime flags from prev frame.
		// Also this function update media states.
		// Note: Save the new media states after calling ResetRuntimeParameters().
		TileViewport->ResetRuntimeParameters(InSourceViewport.GetViewportConfigurationData());
		const EDisplayClusterViewportMediaState SavedTileViewportMediaStates = TileViewport->GetRenderSettings().GetMediaStates();

		// Gain direct access to internal resources of the NewViewport:
		FDisplayClusterViewport_RenderSettings& InOutRenderSettings = TileViewport->GetRenderSettingsImpl();

			// Copy all the settings from the source viewport, but some of them still need to be overridden.
			InOutRenderSettings = SourceRenderSettings;

		// Restore media states for the tile viewport.
		InOutRenderSettings.AssignMediaStates(SavedTileViewportMediaStates);

		// Don't show Tile composing viewports on frame target
		InOutRenderSettings.bVisible = false;

		// Override custom frustum settings and override overscan settings.
		// The size of the original viewport after overscan, custom frustum, etc. is used as
		// the base size for tiling. (see the FDisplayClusterViewport::UpdateFrameContexts() function)

		// Disables the custom frustum for the tile viewport (it has already been applied to the original viewport).
		InOutRenderSettings.CustomFrustumSettings = FDisplayClusterViewport_CustomFrustumSettings();

		// Initialize from the original viewport tile overscan settings, then apply per-tile adjustments.
		InOutRenderSettings.OverscanSettings = SourceRenderSettings.TileSettings.GetOverscanSettings();

		// OverscanSettings Per-tile adjustments: Optimize overscan values for edge tiles.
		// Overscan is not applied on the outer edges of the image — only where tiles meet.
		// Example layout (tiles with overscan overlap marked as "++"):
		//
		//  +------------+-----------+
		//  | Tile0_0  ++|++ Tile1_0 |
		//  |     ++     |     ++    |
		//  +------------+-----------+
		//  |     ++     |     ++    |
		//  | Tile0_1  ++|++ Tile1_1 |
		//  +------------+-----------+
		{
			using EMarginSide = UE::Math::EMarginSetSideIndex;
			auto OptimizeSize = [&](const EMarginSide InSideIndex)
				{
					if (SourceRenderSettings.TileSettings.HasAnyTileFlags(EDisplayClusterViewportTileFlags::OptimizeTileOverscan))
					{
						InOutRenderSettings.OverscanSettings[InSideIndex] = 0;
					}
				};

			if (InTilePos.X == 0)
			{
				OptimizeSize(EMarginSide::Left);
			}
			if (InTilePos.Y == 0)
			{
				OptimizeSize(EMarginSide::Top);
			}
			if (InTilePos.X == (InTileSize.X - 1))
			{
				OptimizeSize(EMarginSide::Right);
			}
			if (InTilePos.Y == (InTileSize.Y - 1))
			{
				OptimizeSize(EMarginSide::Bottom);
			}
		}

		// Inherit tile flags from the source viewport
		const EDisplayClusterViewportTileFlags TileFlags = InSourceViewport.GetRenderSettings().TileSettings.GetTileFlags();

		// Setup as tile.
		InOutRenderSettings.TileSettings = FDisplayClusterViewport_TileSettings(InSourceViewport.GetId(), InTilePos, InTileSize, TileFlags);

		// Copy the other viewport settings:
{
			// Use the OCIO of the original viewport.
			// In some cases, OCIO may be applied during the post-processing phase of rendering.
			TileViewport->SetOpenColorIO(InSourceViewport.GetOpenColorIO());

			// The tile viewport uses the same post-processing settings as the original viewport.
		TileViewport->GetCustomPostProcessSettings() = InSourceViewport.GetCustomPostProcessSettings();

			// The tile viewport uses the same visibility settings as the original viewport.
		TileViewport->GetVisibilitySettingsImpl() = InSourceViewport.GetVisibilitySettingsImpl();

			// Use the same settings for camera motion blur and DoF.
		TileViewport->GetCameraMotionBlurImpl() = InSourceViewport.GetCameraMotionBlurImpl();
		TileViewport->GetCameraDepthOfFieldImpl() = InSourceViewport.GetCameraDepthOfFieldImpl();

			// Ignore `OverscanRuntimeSettings` and `CustomFrustumRuntimeSettings`:
			// These values are calculated later, in FDisplayClusterViewport::UpdateFrameContexts() function
			// from RenderSettings.CustomFrustumSettings and RenderSettings.OverscanSettings

			// Ignore `RenderSettingsICVFX`: tile viewport exists outside of the ICVFX architecture.

			// Ignore `ViewportRemap` and`PostRenderSettings`: it only applies to the final source viewport.
	}
	}

	return TileViewport;
}

FDisplayClusterViewport_OverscanSettings FDisplayClusterViewportConfigurationHelpers_Tile::GetTileOverscanSettings(const FDisplayClusterConfigurationTile_Overscan& InTileOverscan)
{
	FDisplayClusterViewport_OverscanSettings OutOverscanSettings;

	if (InTileOverscan.bOversize)
	{
		OutOverscanSettings.bOversize = true;
	}

	if (InTileOverscan.bEnabled)
	{
		switch (InTileOverscan.Mode)
		{
		case EDisplayClusterConfigurationViewportOverscanMode::Percent:
			OutOverscanSettings.bEnabled = true;
			OutOverscanSettings.Unit = EDisplayClusterViewport_FrustumUnit::Percent;

			// Scale 0..100% to 0..1 range
			OutOverscanSettings.AssignMargins(FMarginSet(.01f * InTileOverscan.AllSides));

			break;

		case EDisplayClusterConfigurationViewportOverscanMode::Pixels:
			OutOverscanSettings.bEnabled = true;
			OutOverscanSettings.Unit = EDisplayClusterViewport_FrustumUnit::Pixels;

			OutOverscanSettings.AssignMargins(FMarginSet(InTileOverscan.AllSides));

			break;

		default:
			break;
		}
	}

	// The tile edge blending feature now requires overscan to be enabled.
	// This logic should be updated if FDisplayClusterConfigurationTile_Overscan rules change.
	if (!OutOverscanSettings.bEnabled)
	{
		return OutOverscanSettings;
	}

	switch (InTileOverscan.OverscanBlendMode)
	{
	case EDisplayClusterConfigurationViewportOverscanBlendMode::Percent25:
		OutOverscanSettings.bApplyTileEdgeBlend = true;
		OutOverscanSettings.TileEdgeBlendPercentage = 0.25f;
		break;

	case EDisplayClusterConfigurationViewportOverscanBlendMode::Percent50:
		OutOverscanSettings.bApplyTileEdgeBlend = true;
		OutOverscanSettings.TileEdgeBlendPercentage = 0.50f;
		break;

	case EDisplayClusterConfigurationViewportOverscanBlendMode::Percent75:
		OutOverscanSettings.bApplyTileEdgeBlend = true;
		OutOverscanSettings.TileEdgeBlendPercentage = 0.75f;
		break;

	case EDisplayClusterConfigurationViewportOverscanBlendMode::Disable:
	default:
		OutOverscanSettings.bApplyTileEdgeBlend = false;
		OutOverscanSettings.TileEdgeBlendPercentage = 0.0f;
		break;

	}

	return OutOverscanSettings;
}
