// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_TileSettings.h"

class FDisplayClusterViewport;
class IDisplayClusterProjectionPolicy;
struct FDisplayClusterConfigurationMediaICVFX;
struct FDisplayClusterConfigurationTile_Overscan;
struct FDisplayClusterViewport_TileSettings;

/**
* Tile configuration helper class.
*/
class FDisplayClusterViewportConfigurationHelpers_Tile
{
public:
	/** Get or create a new tile viewport from source. */
	static FDisplayClusterViewport* GetOrCreateTileViewport(FDisplayClusterViewport& InSourceViewport, const FIntPoint& TilePos);

public:
	/** Get tile settings for the camera viewport. */
	static void UpdateICVFXCameraViewportTileSettings(FDisplayClusterViewport& InSourceViewport, const FDisplayClusterConfigurationMediaICVFX& InCameraMediaSettings);

	/** Get a tile viewport rect inside the rect of the source viewport. */
	static FIntRect GetDestRect(const FDisplayClusterViewport_TileSettings& InTileSettings, const FIntRect& InSourceRect);

	/** Get tile overscan settings. */
	static FDisplayClusterViewport_OverscanSettings GetTileOverscanSettings(const FDisplayClusterConfigurationTile_Overscan& InTileOverscan);
};
