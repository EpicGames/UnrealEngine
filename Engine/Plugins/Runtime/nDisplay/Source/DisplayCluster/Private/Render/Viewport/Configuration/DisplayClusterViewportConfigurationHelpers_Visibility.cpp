// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Viewport/Configuration/DisplayClusterViewportConfigurationHelpers_Visibility.h"
#include "Render/Viewport/Configuration/DisplayClusterViewportConfiguration.h"

#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterConfigurationTypes_Viewport.h"
#include "DisplayClusterConfigurationTypes_ICVFX.h"

#include "DisplayClusterRootActor.h"
#include "Components/DisplayClusterICVFXCameraComponent.h"

#include "Render/Viewport/DisplayClusterViewport.h"
#include "Render/Viewport/DisplayClusterViewportManager.h"
#include "DisplayClusterConfigurationTypes_Viewport.h"

#include "DisplayClusterLightCardActor.h"

namespace UE::DisplayCluster::Configuration::VisibilityHelpers
{
	static inline void ImplCollectActorComponents(AActor& InActor, TSet<FPrimitiveComponentId>& OutComponentsList, const FDisplayClusterViewport* InShowOnlyViewport)
	{
		if (InShowOnlyViewport)
		{
			if (!FDisplayClusterViewportConfigurationHelpers_Visibility::IsActorVisibleForViewport(*InShowOnlyViewport, InActor))
			{
				// Ignore actors that not visible for this viewport.
				return;
			}
		}

		for (const UActorComponent* Component : InActor.GetComponents())
		{
			if (const UPrimitiveComponent* PrimComp = Cast<UPrimitiveComponent>(Component))
			{
				OutComponentsList.Add(PrimComp->GetPrimitiveSceneId());
			}
		}
	}

	/** Collects actors from the layers of the current world. */
	static inline void ImplCollectActorsFromLayers(UWorld* InCurrentWorld, const TArray<FActorLayer>& InActorLayers, TArray<TSoftObjectPtr<AActor>>& OutActorsList)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(DisplayClusterViewport_ImplCollectActorsFromLayers);

		TSet<FName> ActorLayerNames;
		ActorLayerNames.Reserve(InActorLayers.Num());

		// Remove empty names
		for (const FActorLayer& ActorLayerIt : InActorLayers)
		{
			if (!ActorLayerIt.Name.IsNone())
			{
				ActorLayerNames.Add(ActorLayerIt.Name);
			}
		}

		if (!ActorLayerNames.IsEmpty())
		{
			// Iterate over all actors, looking for actors in the specified layers.
			for (const TWeakObjectPtr<AActor> ActorWeakPtr : FActorRange(InCurrentWorld))
			{
				if (ActorWeakPtr.IsValid())
				{
					// Search actor on source layers
					bool bActorFoundOnSourceLayers = false;
					for (const FName& ActorLayerNameIt : ActorLayerNames)
					{
						if (!ActorLayerNameIt.IsNone() && ActorWeakPtr->Layers.Contains(ActorLayerNameIt))
						{
							OutActorsList.Add(ActorWeakPtr.Get());
							break;
						}
					}
				}
			}
		}
	}

	static inline void ImplCollectComponentsFromVisibilityList(FDisplayClusterViewportConfiguration& InConfiguration, const FDisplayClusterConfigurationICVFX_VisibilityList& InVisibilityList, TSet<FPrimitiveComponentId>& OutAdditionalComponentsList, const FDisplayClusterViewport* InShowOnlyViewport = nullptr)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(DisplayCluster_ImplCollectComponentsFromVisibilityList);

		// Collect components from the DCRA
		if (!InVisibilityList.RootActorComponentNames.IsEmpty())
		{
			bool bCanRenderPrimitives = true;
			if (InShowOnlyViewport)
			{
				// If the InShowOnlyViewport argument is defined, check if it is of type lightcard and can be used.
				if (!FDisplayClusterViewportConfigurationHelpers_Visibility::IsLightcardViewportRenderable(*InShowOnlyViewport))
				{
					// Optimization: do not add LC components from DCRA if this viewport is of type lightcard and can be skipped.
					bCanRenderPrimitives = false;
				}
			}

			if (bCanRenderPrimitives)
			{
				if (ADisplayClusterRootActor* SceneRootActor = InConfiguration.GetRootActor(EDisplayClusterRootActorType::Scene))
				{
					// All DCRA components from the list need to be collected.
					SceneRootActor->FindPrimitivesByName(InVisibilityList.RootActorComponentNames, OutAdditionalComponentsList, true);
				}
			}
		}

		auto CollectActorRefs = [&](UWorld* CurrentWorld, const TArray<TSoftObjectPtr<AActor>>& InActorRefs)
			{
				for (const TSoftObjectPtr<AActor>& ActorSOPtrIt : InActorRefs)
				{
					if (ActorSOPtrIt.IsValid())
					{
						if (ActorSOPtrIt->GetWorld() == CurrentWorld)
						{
							ImplCollectActorComponents(*ActorSOPtrIt.Get(), OutAdditionalComponentsList, InShowOnlyViewport);
						}
						else if(CurrentWorld)
						{
							// re-reference to the current world
							// Not implemented.
							//!
						}
					}
				}
			};

		UWorld* CurrentWorld = InConfiguration.GetCurrentWorld();

		// Collect actors from the layers.
		TArray<TSoftObjectPtr<AActor>> ActorsFromLayers;
		ImplCollectActorsFromLayers(CurrentWorld, InVisibilityList.ActorLayers, ActorsFromLayers);

		// Collect Actors refs
		CollectActorRefs(CurrentWorld, InVisibilityList.Actors);
		CollectActorRefs(CurrentWorld, InVisibilityList.AutoAddedActors);
		CollectActorRefs(CurrentWorld, ActorsFromLayers);
	}
};

void FDisplayClusterViewportConfigurationHelpers_Visibility::UpdateShowOnlyList_ICVFX(
	FDisplayClusterViewport& DstViewport,
	const FDisplayClusterConfigurationICVFX_VisibilityList& InVisibilityList,
	const FDisplayClusterConfigurationICVFX_CameraSettings* InCameraSettings)
{
	TSet<FPrimitiveComponentId> ComponentsList;
	UE::DisplayCluster::Configuration::VisibilityHelpers::ImplCollectComponentsFromVisibilityList(*DstViewport.Configuration, InVisibilityList, ComponentsList, &DstViewport);

	// Create show-only list for 3D-LC/CK viewports
	DstViewport.GetVisibilitySettingsImpl().SetVisibilityModeAndComponentsList(EDisplayClusterViewport_VisibilityMode::ShowOnly, ComponentsList);

	const FDisplayClusterConfigurationICVFX_StageSettings* StageSettings = DstViewport.Configuration->GetStageSettings();
	ADisplayClusterRootActor* ConfigurationRootActor = DstViewport.Configuration->GetRootActor(EDisplayClusterRootActorType::Configuration);
	if (!StageSettings || !ConfigurationRootActor)
	{
		return;
	}

	// Exclude entire cluster HideList
	TSet<FPrimitiveComponentId> EntireClusterHiddenComponentsList;
	UE::DisplayCluster::Configuration::VisibilityHelpers::ImplCollectComponentsFromVisibilityList(
		DstViewport.Configuration.Get(), StageSettings->HideList, EntireClusterHiddenComponentsList);
	DstViewport.GetVisibilitySettingsImpl().AddExcludedComponents(EntireClusterHiddenComponentsList);

	// Update exclude list for 3D-LC viewports
	const FString& ParentViewportId = DstViewport.GetRenderSettings().GetParentViewportId();
	const bool bLightcardViewport = EnumHasAnyFlags(DstViewport.GetRenderSettingsICVFX().RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::Lightcard);
	if (bLightcardViewport && !ParentViewportId.IsEmpty())
	{
		// Exclude Outer viewport HideList
		TSet<FPrimitiveComponentId> OuterViewportHiddenComponentsList;
		UE::DisplayCluster::Configuration::VisibilityHelpers::ImplCollectComponentsFromVisibilityList(
			DstViewport.Configuration.Get(), StageSettings->OuterViewportHideList, OuterViewportHiddenComponentsList);
		DstViewport.GetVisibilitySettingsImpl().AddExcludedComponents(OuterViewportHiddenComponentsList);

		// Exclude Per-viewport HideList
		const UDisplayClusterConfigurationData* ConfigData = ConfigurationRootActor->GetConfigData();
		if (const UDisplayClusterConfigurationViewport* ViewportCfg = ConfigData
			? ConfigData->GetViewport(DstViewport.ClusterNodeId, ParentViewportId) : nullptr)
		{
			TSet<FPrimitiveComponentId> PerViewportHiddenComponentsList;
			UE::DisplayCluster::Configuration::VisibilityHelpers::ImplCollectComponentsFromVisibilityList(
				DstViewport.Configuration.Get(), ViewportCfg->RenderSettings.HiddenContent, PerViewportHiddenComponentsList);

			DstViewport.GetVisibilitySettingsImpl().AddExcludedComponents(PerViewportHiddenComponentsList);
		}
	}

	// Update exclude list for CK viewports
	const bool bChromakeyViewport = EnumHasAnyFlags(DstViewport.GetRenderSettingsICVFX().RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::Chromakey);
	if (bChromakeyViewport && InCameraSettings)
	{
		// Exclude camera HideList
		TSet<FPrimitiveComponentId> CameraHiddenComponentsList;
		UE::DisplayCluster::Configuration::VisibilityHelpers::ImplCollectComponentsFromVisibilityList(
			DstViewport.Configuration.Get(), InCameraSettings->CameraHideList, CameraHiddenComponentsList);
		DstViewport.GetVisibilitySettingsImpl().AddExcludedComponents(CameraHiddenComponentsList);
	}
}

void FDisplayClusterViewportConfigurationHelpers_Visibility::AppendHideList_ICVFX(FDisplayClusterViewport& DstViewport, const FDisplayClusterConfigurationICVFX_VisibilityList& InHideList)
{
	TSet<FPrimitiveComponentId> ComponentsList;
	UE::DisplayCluster::Configuration::VisibilityHelpers::ImplCollectComponentsFromVisibilityList(*DstViewport.Configuration, InHideList, ComponentsList);

	DstViewport.GetVisibilitySettingsImpl().AppendVisibilityComponentsList(EDisplayClusterViewport_VisibilityMode::Hide, ComponentsList);
}

void FDisplayClusterViewportConfigurationHelpers_Visibility::UpdateHideList_ICVFX(FDisplayClusterViewportConfiguration& InConfiguration, TArray<TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>>& DstViewports)
{
	ADisplayClusterRootActor* ConfigurationRootActor = InConfiguration.GetRootActor(EDisplayClusterRootActorType::Configuration);
	const FDisplayClusterConfigurationICVFX_StageSettings* StageSettings = InConfiguration.GetStageSettings();

	if (!DstViewports.IsEmpty() && ConfigurationRootActor && StageSettings)
	{
		TSet<FPrimitiveComponentId> ComponentsList;

		// Entire cluster HideList
		UE::DisplayCluster::Configuration::VisibilityHelpers::ImplCollectComponentsFromVisibilityList(InConfiguration, StageSettings->HideList, ComponentsList);

		// Hide lightcard
		UE::DisplayCluster::Configuration::VisibilityHelpers::ImplCollectComponentsFromVisibilityList(InConfiguration, StageSettings->Lightcard.ShowOnlyList, ComponentsList);

		// Also hide chromakeys for all cameras
		TArray<UDisplayClusterICVFXCameraComponent*> ConfigurationRootActorCameras;
		ConfigurationRootActor->GetComponents(ConfigurationRootActorCameras);
		for (const UDisplayClusterICVFXCameraComponent* ConfigurationCameraIt : ConfigurationRootActorCameras)
		{
			if (const FDisplayClusterConfigurationICVFX_ChromakeyRenderSettings* ChromakeyRenderSettings = ConfigurationCameraIt ?
				ConfigurationCameraIt->GetCameraSettingsICVFX().Chromakey.GetChromakeyRenderSettings(*StageSettings) : nullptr)
			{
				UE::DisplayCluster::Configuration::VisibilityHelpers::ImplCollectComponentsFromVisibilityList(InConfiguration, ChromakeyRenderSettings->ShowOnlyList, ComponentsList);
			}
		}

		TSet<FPrimitiveComponentId> OuterComponentsList;
		UE::DisplayCluster::Configuration::VisibilityHelpers::ImplCollectComponentsFromVisibilityList(InConfiguration, StageSettings->OuterViewportHideList, OuterComponentsList);

		// Update hide list for all desired viewports:
		for (TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& ViewportIt : DstViewports)
		{
			if (ViewportIt.IsValid())
			{
				ViewportIt->GetVisibilitySettingsImpl().SetVisibilityModeAndComponentsList(EDisplayClusterViewport_VisibilityMode::Hide, ComponentsList);

				// Support additional hide list for outer viewports
				if (EnumHasAllFlags(ViewportIt->GetRenderSettingsICVFX().RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::Target))
				{
					ViewportIt->GetVisibilitySettingsImpl().AppendVisibilityComponentsList(EDisplayClusterViewport_VisibilityMode::Hide, OuterComponentsList);
				}

				if (const UDisplayClusterConfigurationData* ConfigData = ConfigurationRootActor->GetConfigData())
				{
					// Hide actors from specific viewports.
					if (const UDisplayClusterConfigurationViewport* SourceViewport = ConfigData->GetViewport(ViewportIt->ClusterNodeId, ViewportIt->ViewportId))
					{
						TSet<FPrimitiveComponentId> ViewportSpecificComponentList;
						UE::DisplayCluster::Configuration::VisibilityHelpers::ImplCollectComponentsFromVisibilityList(InConfiguration, SourceViewport->RenderSettings.HiddenContent, ViewportSpecificComponentList);
						ViewportIt->GetVisibilitySettingsImpl().AppendVisibilityComponentsList(EDisplayClusterViewport_VisibilityMode::Hide, ViewportSpecificComponentList);
					}
				}
			}
		}
	}
}

bool FDisplayClusterViewportConfigurationHelpers_Visibility::IsLightcardViewportRenderable(const FDisplayClusterViewport& InViewport, const EDisplayClusterConfigurationICVFX_PerLightcardRenderMode PerLightcardRenderMode)
{
	const EDisplayClusterViewportRuntimeICVFXFlags RuntimeFlags = InViewport.GetRenderSettingsICVFX().RuntimeFlags;
	if (EnumHasAnyFlags(RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::Lightcard | EDisplayClusterViewportRuntimeICVFXFlags::UVLightcard))
	{
		const FDisplayClusterConfigurationICVFX_StageSettings* StageSettings = InViewport.Configuration->GetStageSettings();

		const UDisplayClusterConfigurationViewport* ViewportConfiguration = nullptr;
		if (EnumHasAnyFlags(RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::Lightcard))
		{
			// The UV-light card is rendered once for all viewports.
			// This means that here we cannot use the per-viewport LC rules here. These rules are implemented in the ICVFX shader.
			ViewportConfiguration = InViewport.GetViewportConfigurationData();
		}

		if (StageSettings)
		{
			// Renders all primitives only into the default LC viewport.
			const EDisplayClusterShaderParametersICVFX_LightCardRenderMode LightCardRenderMode = StageSettings->Lightcard.GetLightCardRenderMode(PerLightcardRenderMode, ViewportConfiguration);
			switch (LightCardRenderMode)
			{
			case EDisplayClusterShaderParametersICVFX_LightCardRenderMode::Under:
				return EnumHasAnyFlags(RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::UnderInFrustum);

			case EDisplayClusterShaderParametersICVFX_LightCardRenderMode::Over:
				return EnumHasAnyFlags(RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::OverInFrustum);

			default:
				break;
			}

			return false;
		}
	}

	return true;
}

bool FDisplayClusterViewportConfigurationHelpers_Visibility::IsActorVisibleForViewport(const FDisplayClusterViewport& InViewport, AActor& InActor)
{
	using namespace UE::DisplayCluster::Configuration::VisibilityHelpers;

	// Get special rules from the lightcard actor.
	EDisplayClusterConfigurationICVFX_PerLightcardRenderMode PerLightcardRenderMode = EDisplayClusterConfigurationICVFX_PerLightcardRenderMode::Default;
	if (InActor.IsA<ADisplayClusterLightCardActor>())
	{
		if (const ADisplayClusterLightCardActor* LightCardActor = Cast<ADisplayClusterLightCardActor>(&InActor))
		{
			PerLightcardRenderMode = LightCardActor->PerLightcardRenderMode;
		}
	}

	return IsLightcardViewportRenderable(InViewport, PerLightcardRenderMode);
}
