// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DisplayClusterConfigurationTypes_Enums.h"

class FDisplayClusterViewportConfiguration;
class FDisplayClusterViewport;
class AActor;
struct FDisplayClusterConfigurationICVFX_VisibilityList;
struct FDisplayClusterConfigurationICVFX_CameraSettings;

/**
* Visibility configuration helper class.
*/
class FDisplayClusterViewportConfigurationHelpers_Visibility
{
public:
	// Update ShowOnly list for 3D-LC and CK viewports
	static void UpdateShowOnlyList_ICVFX(FDisplayClusterViewport& DstViewport, const FDisplayClusterConfigurationICVFX_VisibilityList& InVisibilityList, const FDisplayClusterConfigurationICVFX_CameraSettings* InCameraSettings = nullptr);

	// Update hide lists for DstViewports (lightcards, chromakey, stage settings hide list, outer viewports hide list, etc)
	static void UpdateHideList_ICVFX(FDisplayClusterViewportConfiguration& InConfiguration, TArray<TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>>& DstViewports);
	
	// Append to exist hide list (must be call after UpdateHideList_ICVFX)
	static void AppendHideList_ICVFX(FDisplayClusterViewport& DstViewport, const FDisplayClusterConfigurationICVFX_VisibilityList& InHideList);

	/** Returns true if this actor can be rendered for the viewport.
	* For Example:
	*   The LightCard actor has a property that determines which of the two LightCard viewports it should be rendered in.
	*/
	static bool IsActorVisibleForViewport(const FDisplayClusterViewport& InViewport, AActor& InActor);

	/** Returns true if the viewport is of type lightcard and can be rendered. */
	static bool IsLightcardViewportRenderable(const FDisplayClusterViewport& InViewport, const EDisplayClusterConfigurationICVFX_PerLightcardRenderMode PerLightcardRenderMode = EDisplayClusterConfigurationICVFX_PerLightcardRenderMode::Default);
};

