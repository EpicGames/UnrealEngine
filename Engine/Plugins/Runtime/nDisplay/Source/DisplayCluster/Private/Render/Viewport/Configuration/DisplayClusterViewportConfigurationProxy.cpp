// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterViewportConfigurationProxy.h"

#include "DisplayClusterViewportConfiguration.h"
#include "Render/Viewport/DisplayClusterViewportManagerProxy.h"

#include "RHI.h"
#include "RHICommandList.h"
#include "RHIContext.h"

///////////////////////////////////////////////////////////////////
// FDisplayClusterViewportConfigurationProxy
///////////////////////////////////////////////////////////////////
IDisplayClusterViewportManagerProxy* FDisplayClusterViewportConfigurationProxy::GetViewportManagerProxy_RenderThread() const
{
	return GetViewportManagerProxyImpl();
}


DECLARE_GPU_STAT_NAMED(nDisplay_ViewportConfiguration_UpdateConfigurationProxy, TEXT("nDisplay UpdateConfigurationProxy"));
void FDisplayClusterViewportConfigurationProxy::UpdateConfigurationProxy_GameThread(const FDisplayClusterViewportConfiguration& InConfiguration)
{
	// Send configuration data to proxy
	ENQUEUE_RENDER_COMMAND(DisplayClusterUpdateConfigurationProxy)(
		[ConfigurationProxy = SharedThis(this)
		, RenderFrameSettingsCopy = new FDisplayClusterRenderFrameSettings(InConfiguration.GetRenderFrameSettings())
		](FRHICommandListImmediate& RHICmdList)
		{
			RHI_BREADCRUMB_EVENT_STAT(RHICmdList, nDisplay_ViewportConfiguration_UpdateConfigurationProxy, "nDisplay_ViewportConfiguration_UpdateConfigurationProxy");

			if (RenderFrameSettingsCopy)
			{
				ConfigurationProxy->RenderFrameSettings = *RenderFrameSettingsCopy;
				delete RenderFrameSettingsCopy;
			}
		});
}
