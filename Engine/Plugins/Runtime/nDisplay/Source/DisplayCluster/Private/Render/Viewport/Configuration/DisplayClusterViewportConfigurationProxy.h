// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Render/Viewport/IDisplayClusterViewportConfigurationProxy.h"
#include "Render/Viewport/RenderFrame/DisplayClusterRenderFrameSettings.h"
#include "Templates/SharedPointer.h"

class FDisplayClusterViewportManagerProxy;

/**
* Implementation of the IDisplayClusterViewportConfigurationProxy
*/
class FDisplayClusterViewportConfigurationProxy
	: public IDisplayClusterViewportConfigurationProxy
	, public TSharedFromThis<FDisplayClusterViewportConfigurationProxy, ESPMode::ThreadSafe>
{
public:
	FDisplayClusterViewportConfigurationProxy() = default;
	virtual ~FDisplayClusterViewportConfigurationProxy() = default;

public:
	//~ BEGIN IDisplayClusterViewportConfigurationProxy
	virtual IDisplayClusterViewportManagerProxy* GetViewportManagerProxy_RenderThread() const override;

	virtual bool IsPreviewRendering_RenderThread() const override
	{
		check(IsInRenderingThread());

		return RenderFrameSettings.IsPreviewRendering();
	}
	
	virtual bool IsICVFXEnabled_RenderThread() const override
	{
		check(IsInRenderingThread());

		return RenderFrameSettings.IsICVFXEnabled();
	}

	virtual EDisplayClusterRenderFrameMode GetRenderMode_RenderThread() const override
	{
		check(IsInRenderingThread());

		return RenderFrameSettings.RenderMode;
	}

	virtual const FString& GetClusterNodeId_RenderThread() const override
	{
		check(IsInRenderingThread());

		return RenderFrameSettings.CurrentNode.Id;
	}
	// ~~END IDisplayClusterViewportConfigurationProxy

public:
	/** One-time callable initializer. */
	void Initialize_GameThread(const TSharedPtr<FDisplayClusterViewportManagerProxy, ESPMode::ThreadSafe>& InViewportManagerProxy)
	{
		ViewportManagerProxyWeakPtr = InViewportManagerProxy;
	}

	/** Send data to proxy object on rendering thread. */
	void UpdateConfigurationProxy_GameThread(const class FDisplayClusterViewportConfiguration& InConfiguration);

	/** Get a pointer to the DC ViewportManagerProxy if it still exists. */
	FDisplayClusterViewportManagerProxy* GetViewportManagerProxyImpl() const
	{
		return ViewportManagerProxyWeakPtr.IsValid() ? ViewportManagerProxyWeakPtr.Pin().Get() : nullptr;
	}

	/** Get render frame settings. */
	const FDisplayClusterRenderFrameSettings& GetRenderFrameSettings() const
	{
		return RenderFrameSettings;
	}

private:
	// A reference to the owning viewport manager proxy
	TWeakPtr<FDisplayClusterViewportManagerProxy, ESPMode::ThreadSafe> ViewportManagerProxyWeakPtr;

	// The render frame settings (Copy from game thread)
	FDisplayClusterRenderFrameSettings RenderFrameSettings;
};
