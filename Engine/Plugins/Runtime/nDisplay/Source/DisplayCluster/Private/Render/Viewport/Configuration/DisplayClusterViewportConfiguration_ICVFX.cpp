// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterViewportConfiguration_ICVFX.h"

#include "DisplayClusterEnums.h"
#include "DisplayClusterConfigurationTypes_Viewport.h"
#include "DisplayClusterProjectionStrings.h"
#include "DisplayClusterRootActor.h"
#include "IDisplayClusterProjection.h"
#include "IPDisplayCluster.h"

#include "Cluster/IPDisplayClusterClusterManager.h"
#include "Components/DisplayClusterICVFXCameraComponent.h"
#include "Containers/DisplayClusterProjectionCameraPolicySettings.h"

#include "Misc/DisplayClusterLog.h"
#include "Misc/DisplayClusterGlobals.h"
#include "Misc/Parse.h"

#include "Render/Projection/IDisplayClusterProjectionPolicy.h"
#include "Render/Viewport/Configuration/DisplayClusterViewportConfiguration.h"
#include "Render/Viewport/Configuration/DisplayClusterViewportConfigurationHelpers.h"
#include "Render/Viewport/Configuration/DisplayClusterViewportConfigurationHelpers_ICVFX.h"
#include "Render/Viewport/Configuration/DisplayClusterViewportConfigurationHelpers_Visibility.h"
#include "Render/Viewport/DisplayClusterViewport.h"
#include "Render/Viewport/DisplayClusterViewportManager.h"
#include "Render/Viewport/LightCard/DisplayClusterViewportLightCardManager.h"
#include "Render/Viewport/RenderFrame/DisplayClusterRenderFrameSettings.h"

///////////////////////////////////////////////////////////////////
// FDisplayClusterViewportConfiguration_ICVFX
///////////////////////////////////////////////////////////////////
bool FDisplayClusterViewportConfiguration_ICVFX::CreateLightcardViewport(FDisplayClusterViewport& BaseViewport, const bool bOverInFrustum)
{
	if (FDisplayClusterViewport* LightcardViewport = FDisplayClusterViewportConfigurationHelpers_ICVFX::GetOrCreateLightcardViewport(BaseViewport, bOverInFrustum))
	{
		// Update lightcard viewport settings
		FDisplayClusterViewportConfigurationHelpers_ICVFX::UpdateLightcardViewportSetting(*LightcardViewport, BaseViewport);

		// Support projection policy update
		LightcardViewport->UpdateConfiguration_ProjectionPolicy();

		return true;
	}

	return false;
}

bool FDisplayClusterViewportConfiguration_ICVFX::CreateUVLightcardViewport(FDisplayClusterViewport& BaseViewport, const bool bOverInFrustum)
{
	if (FDisplayClusterViewport* UVLightCardViewport = FDisplayClusterViewportConfigurationHelpers_ICVFX::GetOrCreateUVLightcardViewport(BaseViewport, bOverInFrustum))
	{
		// Update UV LightCard viewport settings
		FDisplayClusterViewportConfigurationHelpers_ICVFX::UpdateLightcardViewportSetting(*UVLightCardViewport, BaseViewport);

		// Support projection policy update
		UVLightCardViewport->UpdateConfiguration_ProjectionPolicy();

		// Optimize: re-use UVLightCard viewports with equals OCIO
		FDisplayClusterViewportConfigurationHelpers_ICVFX::ReuseUVLightCardViewportWithinClusterNode(*UVLightCardViewport, bOverInFrustum);

		return true;
	}

	return false;
}

void FDisplayClusterViewportConfiguration_ICVFX::Update()
{
	ImplBeginReallocateViewports();

	const FDisplayClusterRenderFrameSettings& RenderFrameSettings = Configuration.GetRenderFrameSettings();
	const FDisplayClusterConfigurationICVFX_StageSettings* StageSettings = Configuration.GetStageSettings();
	FDisplayClusterViewportManager* ViewportManager = Configuration.GetViewportManagerImpl();

	// ICVFX is node-based; requires a defined cluster node to create viewports
	if (Configuration.GetClusterNodeId().IsEmpty() || !ViewportManager || !StageSettings)
	{
		// The nDisplay viewport should now always have the name of the cluster node.
		// When rendering MRQ viewports, the list of viewports is used without cluster node names.
		ImplFinishReallocateViewports();

		return;
	}

	// Find ICVFX target viewports
	TArray<TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>> TargetViewports;
	EDisplayClusterViewportICVFXFlags TargetViewportsFlags;
	if (!ImplGetTargetViewports(TargetViewports, TargetViewportsFlags))
	{
		// We need to be able to render inner views (ICVFX camera view) without any outers. Current pipeline has
		// a bunch of optimizations that block inner rendering if no outers available. To avoid the limitation
		// we force inner data initialization if no outers found.
		if(Configuration.IsClusterNodeRenderingOffscreen())
		{
			GetAndUpdateStageCameras();
		}

		ImplFinishReallocateViewports();

		return;
	}

	// If not all of the viewports disable camera:
	// Collect all ICVFX cameras from stage
	if (!EnumHasAnyFlags(TargetViewportsFlags, EDisplayClusterViewportICVFXFlags::DisableCamera))
	{
		GetAndUpdateStageCameras(&TargetViewports);
	}

	// If not all viewports disable lightcard
	if (!EnumHasAnyFlags(TargetViewportsFlags, EDisplayClusterViewportICVFXFlags::DisableLightcard))
	{
		// UVLightCard must be enabled in LC manager
		const bool bUVLightCardOverEnabled = ViewportManager->LightCardManager->IsUVLightCardEnabled(EDisplayClusterUVLightCardType::Over);
		const bool bUVLightCardUnderEnabled = ViewportManager->LightCardManager->IsUVLightCardEnabled(EDisplayClusterUVLightCardType::Under);

		// per-viewport lightcard use-case
		const EDisplayClusterViewportICVFXFlags LightcardRenderModeFlags = TargetViewportsFlags & EDisplayClusterViewportICVFXFlags::LightcardRenderModeMask;
		const bool bLightCardOverEnabled  = LightcardRenderModeFlags != EDisplayClusterViewportICVFXFlags::LightcardAlwaysUnder;
		const bool bLightCardUnderEnabled = LightcardRenderModeFlags != EDisplayClusterViewportICVFXFlags::LightcardAlwaysOver;

		// Allocate and assign lightcard resources
		const bool bUseLightCard = StageSettings->Lightcard.ShouldUseLightCard(*StageSettings);
		const bool bUseUVLightCard = StageSettings->Lightcard.ShouldUseUVLightCard(*StageSettings);

		for (const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& TargetIt : TargetViewports)
		{
			// only for support targets
			if (TargetIt.IsValid() && !EnumHasAnyFlags(TargetIt->GetRenderSettingsICVFX().Flags, EDisplayClusterViewportICVFXFlags::DisableLightcard))
			{
				if (bLightCardOverEnabled && bUseLightCard)
				{
					CreateLightcardViewport(*TargetIt, true);
				}
				
				if (bLightCardUnderEnabled && bUseLightCard)
				{
					CreateLightcardViewport(*TargetIt, false);
				}

				if (bUVLightCardOverEnabled && bUseUVLightCard)
				{
					CreateUVLightcardViewport(*TargetIt, true);
				}

				if (bUVLightCardUnderEnabled && bUseUVLightCard)
				{
					CreateUVLightcardViewport(*TargetIt, false);
				}
			}
		}
	}


	for (const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& TargetIt : TargetViewports)
	{
		if (TargetIt.IsValid())
		{
			// Gain direct access to internal settings of the viewport:
			FDisplayClusterViewport_RenderSettingsICVFX& InOutRenderSettingsICVFX = TargetIt->GetRenderSettingsICVFXImpl();

			// Clear enforced camera order each frame to avoid stale data
			InOutRenderSettingsICVFX.ICVFX.EnforcedCameraOrder.Empty();

			// Build enforced camera order by resolving camera names to viewport IDs
			if (const UDisplayClusterConfigurationViewport* ViewportCfg = TargetIt->GetViewportConfigurationData())
			{

				if (!ViewportCfg->ICVFX.EnforcedCameraOrder.IsEmpty())
				{
					// Build full viewport IDs from camera names using helper
					for (const FString& CameraName : ViewportCfg->ICVFX.EnforcedCameraOrder)
					{
						if (!CameraName.IsEmpty())
						{
							const FString InCameraViewportId = Configuration.BuildViewportName(CameraName, EDisplayClusterViewportFlags::ICVFX_InnerFrustum);
							InOutRenderSettingsICVFX.ICVFX.EnforcedCameraOrder.Add(InCameraViewportId);
						}
					}
				}
			}

			// Sort cameras by render order for each target
			InOutRenderSettingsICVFX.ICVFX.SortCamerasRenderOrder(EnumHasAnyFlags(InOutRenderSettingsICVFX.Flags, EDisplayClusterViewportICVFXFlags::ReverseCameraPriority));

			// Setup incamera overlap mode
			InOutRenderSettingsICVFX.ICVFX.CameraOverlappingRenderMode = StageSettings->GetCameraOverlappingRenderMode();
		}
	}

	ImplFinishReallocateViewports();
}

void FDisplayClusterViewportConfiguration_ICVFX::PostUpdate()
{
		// Update visibility for icvfx viewports and cameras
		ImplUpdateVisibility();

	// Note: Alpha-output configuration for rendering (unrelated to ICVFX)
	//       has been moved to FDisplayClusterRenderFrameSettings::bEnableAlphaOutput.
	// see FDisplayClusterViewportConfigurationHelpers::UpdateBaseViewportSetting().
}

void FDisplayClusterViewportConfiguration_ICVFX::ImplUpdateVisibility()
{
	// Collect viewports, that use ICVFX hide list
	if (FDisplayClusterViewportManager* ViewportManager = Configuration.GetViewportManagerImpl())
	{
		TArray<TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>> ICVFXViewports;

		for (const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& ViewportIt : ViewportManager->ImplGetCurrentRenderFrameViewports())
		{
			if (ViewportIt.IsValid())
			{
				const bool bInternalResource = EnumHasAnyFlags(ViewportIt->GetRenderSettingsICVFX().RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::InternalResource);
				const bool bIsInCamera = EnumHasAnyFlags(ViewportIt->GetRenderSettingsICVFX().RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::InCamera);
				const bool bICVFX_Enable = EnumHasAnyFlags(ViewportIt->GetRenderSettingsICVFX().Flags, EDisplayClusterViewportICVFXFlags::Enable);

				if ((bICVFX_Enable && !bInternalResource) || (bInternalResource && bIsInCamera))
				{
					ICVFXViewports.Add(ViewportIt);
				}
			}
		}

		// Update hide list for all icvfx viewports
		FDisplayClusterViewportConfigurationHelpers_Visibility::UpdateHideList_ICVFX(Configuration, ICVFXViewports);
	}

	// Support additional hide list for icvfx cameras:
	for (const FDisplayClusterViewportConfiguration_ICVFXCamera& CameraIt : StageCameras)
	{
		if (CameraIt.CameraViewport.IsValid())
		{
			FDisplayClusterViewportConfigurationHelpers_Visibility::AppendHideList_ICVFX(*CameraIt.CameraViewport, CameraIt.GetCameraSettings().CameraHideList);
		}
	}
}

void FDisplayClusterViewportConfiguration_ICVFX::ImplBeginReallocateViewports()
{
	if (FDisplayClusterViewportManager* ViewportManager = Configuration.GetViewportManagerImpl())
	{
		for (const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& ViewportIt : ViewportManager->ImplGetCurrentRenderFrameViewports())
		{
			// Runtime icvfx viewport support reallocate feature:
			if (ViewportIt.IsValid()
				// Exclude tile viewports: they handled in FDisplayClusterViewportConfiguration_Tile::Update()
				&& !EnumHasAnyFlags(ViewportIt->GetViewportFlags(), EDisplayClusterViewportFlags::TileViewport)
				// Process only ICVFX viewports
				&& EnumHasAnyFlags(ViewportIt->GetViewportFlags(), EDisplayClusterViewportFlags::ICVFX_AnyRole))
			{
				// Gain direct access to internal settings of the viewport:
				FDisplayClusterViewport_RenderSettingsICVFX& InOutRenderSettingsICVFX = ViewportIt->GetRenderSettingsICVFXImpl();

				// Mark this ICVFX viewport as unused. This flag is cleared later if the viewport is actually used.
				// Otherwise, this viewport will be removed in ImplFinishReallocateViewports().
				EnumAddFlags(InOutRenderSettingsICVFX.RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::Unused);
			}
		}
	}
}

void FDisplayClusterViewportConfiguration_ICVFX::ImplFinishReallocateViewports()
{
	if (FDisplayClusterViewportManager* ViewportManager = Configuration.GetViewportManagerImpl())
	{
		TArray<TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>> UnusedViewports;

		// Collect all unused viewports for remove
		for (const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& ViewportIt : ViewportManager->ImplGetCurrentRenderFrameViewports())
		{
			if (ViewportIt.IsValid() && EnumHasAllFlags(ViewportIt->GetRenderSettingsICVFX().RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::Unused))
			{
				UnusedViewports.Add(ViewportIt);
			}
		}

		// Delete unused viewports:
		for (TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& RemoveViewportIt : UnusedViewports)
		{
			if (RemoveViewportIt.IsValid())
			{
				ViewportManager->DeleteViewport(RemoveViewportIt->GetId());
			}
		}

		UnusedViewports.Empty();
	}
}

void FDisplayClusterViewportConfiguration_ICVFX::GetAndUpdateStageCameras(const TArray<TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>>* InTargetViewports)
{
	// Allocate and assign camera resources
	if (ImplGetStageCameras())
	{
		const FDisplayClusterConfigurationICVFX_StageSettings* StageSettings = Configuration.GetStageSettings();

		// Collect visible targets for cameras:
		if (InTargetViewports && StageSettings)
		{
			for (const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& TargetIt : *InTargetViewports)
			{
				// Target viewpot must support camera render:
				if (TargetIt.IsValid() && !EnumHasAnyFlags(TargetIt->GetRenderSettingsICVFX().Flags, EDisplayClusterViewportICVFXFlags::DisableCamera))
				{
					// Add this target to all cameras visible on it
					for (FDisplayClusterViewportConfiguration_ICVFXCamera& StageCameraIt : StageCameras)
					{
						const bool bCameraProjectionVisible  = StageCameraIt.IsCameraProjectionVisibleOnViewport(TargetIt.Get());
						const bool bIsCameraHiddenOnViewport = StageCameraIt.GetCameraSettings().HiddenICVFXViewports.ItemNames.Contains(TargetIt->GetId());
						
						if (bCameraProjectionVisible && !bIsCameraHiddenOnViewport)
						{
							// This ICVXF camera is visible in this viewport.
							FDisplayClusterViewportConfiguration_ICVFXCamera::FTargetViewport TargetViewport(TargetIt.ToSharedRef());

							// Update per-viewport chromakey for this ICVFX camera
							if (const UDisplayClusterConfigurationViewport* ViewportCfg = TargetIt->GetViewportConfigurationData())
							{
								TargetViewport.ChromakeySource =
									ViewportCfg->GetViewportChromakeyType(*StageSettings, StageCameraIt.GetCameraUniqueId(), StageCameraIt.GetCameraSettings());
							}

							StageCameraIt.TargetViewports.Add(TargetViewport);
						}
					}
				}
			}
		}

		// Create camera resources and initialize target ICVFX viewports
		for (FDisplayClusterViewportConfiguration_ICVFXCamera& CameraIt : StageCameras)
		{
			// Ignore cameras that are invisible and have no media assigned.
			// ("Invisible" = not used on any target (outer) viewport.)
			if (CameraIt.TargetViewports.IsEmpty() && !CameraIt.HasAnyMediaBindings())
			{
				continue;
			}

			// Update ICVFX camera viewports.
			CameraIt.Update();
		}
	}
}

bool FDisplayClusterViewportConfiguration_ICVFX::ImplGetStageCameras()
{
	// Clear stage cameras unconditionally to prevent stale data
	StageCameras.Empty();

	ADisplayClusterRootActor* SceneRootActor = Configuration.GetRootActor(EDisplayClusterRootActorType::Scene);
	ADisplayClusterRootActor* ConfigurationRootActor = Configuration.GetRootActor(EDisplayClusterRootActorType::Configuration);
	const UDisplayClusterConfigurationData* ConfigurationData = Configuration.GetConfigurationData();

	if (!(SceneRootActor && ConfigurationRootActor && ConfigurationData))
	{
		return false;
	}

	// Get all ICVFX camera components
	TArray<UDisplayClusterICVFXCameraComponent*> SceneCameraComponents;
	SceneRootActor->GetComponents(SceneCameraComponents);

	TArray<UDisplayClusterICVFXCameraComponent*> ConfigurationCameraComponents;
	ConfigurationRootActor->GetComponents(ConfigurationCameraComponents);

	// Filter active cameras only
	for (UDisplayClusterICVFXCameraComponent* SceneCameraComponentIt : SceneCameraComponents)
	{
		if (SceneCameraComponentIt)
		{
			UDisplayClusterICVFXCameraComponent** ConfigurationCameraPtr = ConfigurationCameraComponents.FindByPredicate([CameraId = SceneCameraComponentIt->GetCameraUniqueId()](const UDisplayClusterICVFXCameraComponent* CameraComponent)
				{
					return CameraComponent && (CameraComponent->GetCameraUniqueId() == CameraId);
				});

			if (ConfigurationCameraPtr && *ConfigurationCameraPtr)
			{
				const FDisplayClusterConfigurationICVFX_CameraSettings& CameraSettings = (*ConfigurationCameraPtr)->GetCameraSettingsICVFX();
				if (CameraSettings.IsICVFXEnabled(*ConfigurationData, Configuration.GetClusterNodeId()))
				{
					const FString InnerFrustumID = (*ConfigurationCameraPtr)->GetCameraUniqueId();
					if (ConfigurationRootActor->IsInnerFrustumEnabled(InnerFrustumID))
					{
						FDisplayClusterViewportConfiguration_ICVFXCamera NewCamera(Configuration, *SceneCameraComponentIt , *(*ConfigurationCameraPtr));
						if (NewCamera.Initialize())
						{
							StageCameras.Add(NewCamera);
						}
					}
				}
			}
		}
	}

	return StageCameras.Num() > 0;
}

bool FDisplayClusterViewportConfiguration_ICVFX::ImplGetTargetViewports(TArray<TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>>& OutTargets, EDisplayClusterViewportICVFXFlags& OutMergedICVFXFlags)
{
	EDisplayClusterViewportICVFXFlags InvFlags = EDisplayClusterViewportICVFXFlags::None;

	if (FDisplayClusterViewportManager* ViewportManager = Configuration.GetViewportManagerImpl())
	{
		// Collect invertet disable flags from all target viewports
		for (const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& ViewportIt : ViewportManager->ImplGetCurrentRenderFrameViewports())
		{
			// Process only external viewports:
			if (ViewportIt.IsValid() && !ViewportIt->IsInternalViewport())
			{
				//Raise new projection target if possible
				if (ViewportIt->GetRenderSettings().bEnable && EnumHasAnyFlags(ViewportIt->GetRenderSettingsICVFX().Flags, EDisplayClusterViewportICVFXFlags::Enable))
				{
					if (ViewportIt->GetProjectionPolicy().IsValid() && ViewportIt->GetProjectionPolicy()->ShouldSupportICVFX(ViewportIt.Get()))
					{
						// Collect this viewport ICVFX target
						OutTargets.Add(ViewportIt);

						// Gain direct access to internal settings of the viewport:
						FDisplayClusterViewport_RenderSettingsICVFX& InOutRenderSettingsICVFX = ViewportIt->GetRenderSettingsICVFXImpl();

						// proj policy support ICVFX, Use this viewport as icvfx target
						EnumAddFlags(InOutRenderSettingsICVFX.RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::Target);

						// Update targets use flags:
						InvFlags |= ~(InOutRenderSettingsICVFX.Flags);
					}
				}
			}
		}
	}

	// Collect all targets disable flags
	OutMergedICVFXFlags = ~(InvFlags);

	return OutTargets.Num() > 0;
}
