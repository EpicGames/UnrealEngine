// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Render/Viewport/Configuration/DisplayClusterViewportConfiguration_ICVFXCamera.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_RenderSettingsICVFX.h"

/**
 * A helper class that configure ICVFX.
 */
class FDisplayClusterViewportConfiguration_ICVFX
{
public:
	FDisplayClusterViewportConfiguration_ICVFX(FDisplayClusterViewportConfiguration& InConfiguration)
		: Configuration(InConfiguration)
	{ }

	~FDisplayClusterViewportConfiguration_ICVFX() = default;

public:
	/** Update ICVFX viewports for a new frame. */
	void Update();

	/** Post-Update ICVFX viewports for a new frame. */
	void PostUpdate();

private:
	/** Get and Update stage cameras. */
	void GetAndUpdateStageCameras(const TArray<TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>>* InTargetViewports = nullptr);

	/** Mark all ICVFX viewports as unused before upgrading. */
	void ImplBeginReallocateViewports();

	/** Delete unused ICVFX viewports. */
	void ImplFinishReallocateViewports();

	/** Search for ICVFX cameras inside DCRA and populate the StageCameras variable.*/
	bool ImplGetStageCameras();


	/** Searches for OuterViewports within DCRA and returns them to the OutTargets variable. */
	bool ImplGetTargetViewports(TArray<TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>>& OutTargets, EDisplayClusterViewportICVFXFlags& OutMergedICVFXFlags);

	/** Create a LightCard viewport for the  BaseViewport. */
	bool CreateLightcardViewport(FDisplayClusterViewport& BaseViewport, const bool bOverInFrustum);

	/** Create an UVLightCard viewport for the BaseViewport. */
	bool CreateUVLightcardViewport(FDisplayClusterViewport& BaseViewport, const bool bOverInFrustum);

	/** Update a visibility for the ICVFX viewports and cameras. */
	void ImplUpdateVisibility();

private:
	// Viewport configuration API
	FDisplayClusterViewportConfiguration& Configuration;

	// ICVFX cameras
	TArray<FDisplayClusterViewportConfiguration_ICVFXCamera> StageCameras;
};
