// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterViewportConfiguration_ICVFXCamera.h"

#include "DisplayClusterViewportConfiguration.h"
#include "DisplayClusterViewportConfigurationHelpers.h"
#include "DisplayClusterViewportConfigurationHelpers_ICVFX.h"
#include "DisplayClusterViewportConfigurationHelpers_Visibility.h"

#include "DisplayClusterRootActor.h"
#include "DisplayClusterConfigurationTypes_Viewport.h"
#include "DisplayClusterConfigurationTypes_ICVFX.h"

#include "IDisplayClusterProjection.h"
#include "DisplayClusterProjectionStrings.h"

#include "Containers/DisplayClusterProjectionCameraPolicySettings.h"
#include "Render/Projection/IDisplayClusterProjectionPolicy.h"

#include "Render/Viewport/DisplayClusterViewport.h"
#include "Render/Viewport/DisplayClusterViewportManager.h"
#include "Render/Viewport/RenderFrame/DisplayClusterRenderFrameSettings.h"
#include "Render/Viewport/LightCard/DisplayClusterViewportLightCardManager.h"

#include "Components/DisplayClusterICVFXCameraComponent.h"

#include "Misc/DisplayClusterLog.h"
#include "Misc/Parse.h"

////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterViewportConfiguration_ICVFXCamera
////////////////////////////////////////////////////////////////////////////////
bool FDisplayClusterViewportConfiguration_ICVFXCamera::HasAnyMediaBindings() const
{
	const FDisplayClusterConfigurationMediaICVFX& MediaSettings = GetCameraSettings().RenderSettings.Media;
	if (Configuration.IsMediaAvailable() && MediaSettings.bEnable)
	{
		const bool bMediaInputAssigned = MediaSettings.HasAnyMediaInputAssigned(Configuration.GetClusterNodeId(), MediaSettings.SplitType);
		const bool MediaOutputAssigned = MediaSettings.HasAnyMediaOutputAssigned(Configuration.GetClusterNodeId(), MediaSettings.SplitType);

		return bMediaInputAssigned || MediaOutputAssigned;
	}

	return false;
}

bool FDisplayClusterViewportConfiguration_ICVFXCamera::GetOrCreateAndSetupInnerCameraViewport()
{
	if (FDisplayClusterViewport* NewOrExistCameraViewport = FDisplayClusterViewportConfigurationHelpers_ICVFX::GetOrCreateCameraViewport(Configuration, CameraComponent, GetCameraSettings()))
	{
		CameraViewport = NewOrExistCameraViewport->AsShared();

		// If this In-Camera viewport is not visible on the outer viewports, set the ICVFX flag.
		if (TargetViewports.IsEmpty())
		{
			// Set flag that indicates that the In-Camera viewport is not visible because no target viewports are assigned.
			// Note: Flags are reset at the beginning of each new frame by FDisplayClusterViewport_RenderSettingsICVFX::BeginUpdateSettings().
			EnumAddFlags(NewOrExistCameraViewport->GetRenderSettingsICVFXImpl().Flags, EDisplayClusterViewportICVFXFlags::CameraHasNoTargetViewports);
		}

		auto DoesAnyTargetViewportSupportInCameraRendering = [&]()
			{
				if (HasAnyMediaBindings())
				{	
					// Can render the camera if it has any media bindings.
					// Note: Input media bindings disable rendering in FDisplayClusterViewport::IsRenderEnabledByMedia().
					return true;
				}

				if (TargetViewports.IsEmpty())
				{
					// Do not render ICVFX cameras that are invisible.
					return false;
				}

				for (const FTargetViewport& TargetViewportIt : TargetViewports)
				{
					if (TargetViewportIt.ChromakeySource != EDisplayClusterShaderParametersICVFX_ChromakeySource::FrameColor)
					{
						// At least one target viewport is configured with chromakey that does not override the In-Camera viewport.
						return true;
					}
				}

				// Performance: If all chromakey sources have a ‘FrameColor’ value for all viewports on the current cluster node,
				// we can skip rendering the InnerFrustum viewport
				return false;
			};

		// Do not render an In-Camera viewport if it is not visible on the outers 
		// (unless a MediaOutput is assigned).
		// In this case, the camera viewport is used only as a source of projection data, not textures.
		const bool bUseCameraForProjectionOnly = !DoesAnyTargetViewportSupportInCameraRendering();

		// Note: Ensure bSkipRendering is set beforehand; otherwise, the InCamera viewport may be split into tiles
		CameraViewport->GetRenderSettingsImpl().bSkipRendering = bUseCameraForProjectionOnly;

		// overlay rendered only for enabled incamera
		check(GetCameraSettings().bEnable);

		// Update camera viewport settings
		FDisplayClusterViewportConfigurationHelpers_ICVFX::UpdateCameraViewportSettings(*CameraViewport, CameraComponent, GetCameraSettings());

		// Support projection policy update
		CameraViewport->UpdateConfiguration_ProjectionPolicy();

		// Update camera viewport projection policy settings (Use settings from the UDisplayClusterICVFXCameraComponent)
		FDisplayClusterViewportConfigurationHelpers_ICVFX::UpdateCameraProjectionSettingsICVFX(*CameraViewport->Configuration, CameraComponent, GetCameraSettings(), CameraViewport->GetProjectionPolicy());

		// Preview performance optimization: the InCamera viewport should only be rendered on one node and then used on all other nodes in the cluster.
		FDisplayClusterViewportConfigurationHelpers_ICVFX::PreviewReuseInnerFrustumViewportWithinClusterNodes(*CameraViewport, CameraComponent, GetCameraSettings());

		return true;
	}

	return false;
}

bool FDisplayClusterViewportConfiguration_ICVFXCamera::IsCameraProjectionVisibleOnViewport(FDisplayClusterViewport* TargetViewport)
{
	if (TargetViewport && TargetViewport->GetProjectionPolicy().IsValid())
	{
		// Currently, only mono context is supported to check the visibility of the inner camera.
		if (TargetViewport->GetProjectionPolicy()->IsCameraProjectionVisible(CameraContext.ViewRotation, CameraContext.ViewLocation, CameraContext.PrjMatrix))
		{
			return true;
		}
	}

	// do not use camera for this viewport
	return false;
}

void FDisplayClusterViewportConfiguration_ICVFXCamera::Update()
{
	// Clears the CK viewports references
	CameraViewport.Reset();
	ChromakeyViewport.Reset();

	if (GetOrCreateAndSetupInnerCameraViewport())
	{
		bool bShouldUseChromakeyViewport =
			// CK only applies as composition over the rendered InnerFrustum image.
			!CameraViewport->GetRenderSettings().bSkipRendering
			// CK composition requires target viewports.
			&& !TargetViewports.IsEmpty();

		// Optimize CK viewport resources for offscreen nodes
		if (Configuration.IsClusterNodeRenderingOffscreen()
			&& Configuration.ShouldOptimizeOutOffscreenNodeResources()
			// Optimize if backbuffer not used by media
			&& !Configuration.GetRenderFrameSettings().CurrentNode.bHasBackbufferMediaOutput)
		{
			bShouldUseChromakeyViewport = false;
		}

		if (bShouldUseChromakeyViewport)
		{
			// Performance: Do not render CK if InnerFrustum is not visible.
			// Updates the Chromakey viewport and configures it for use in the ICVFX rendering stack.
			UpdateChromakeyViewport();
		}

		// Updates all target viewports.
		UpdateICVFXSettingsForTargetViewports();
	}
}

bool FDisplayClusterViewportConfiguration_ICVFXCamera::Initialize()
{
	// Create new camera projection policy for camera viewport
	TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe> CameraProjectionPolicy =
		FDisplayClusterViewportConfigurationHelpers_ICVFX::CreateProjectionPolicyCameraICVFX(Configuration, CameraComponent, GetCameraSettings());
	if (!CameraProjectionPolicy.IsValid())
	{
		return false;
	}

	// Applying the correct sequence of steps to use the projection policy math:
	// SetupProjectionViewPoint()->CalculateView()->GetProjectionMatrix()
	FMinimalViewInfo CameraViewInfo;
	float CustomNearClippingPlane = -1; // a value less than zero means ignoring.
	CameraProjectionPolicy->SetupProjectionViewPoint(nullptr, Configuration.GetRootActorWorldDeltaSeconds(), CameraViewInfo, &CustomNearClippingPlane);

	CameraContext.ViewLocation = CameraViewInfo.Location;
	CameraContext.ViewRotation = CameraViewInfo.Rotation;

	// Todo: Here we need to calculate the correct ViewOffset so that ICVFX can support stereo rendering.
	const FVector ViewOffset = FVector::ZeroVector;

	// Get world scale
	const float WorldToMeters = Configuration.GetWorldToMeters();
	// Supports custom near clipping plane
	const float NCP = (CustomNearClippingPlane >= 0) ? CustomNearClippingPlane : GNearClippingPlane;

	if(CameraProjectionPolicy->CalculateView(nullptr, 0, CameraContext.ViewLocation, CameraContext.ViewRotation, ViewOffset, WorldToMeters, NCP, NCP)
	&& CameraProjectionPolicy->GetProjectionMatrix(nullptr, 0, CameraContext.PrjMatrix))
	{
		return true;
	}

	return false;
}

const FDisplayClusterConfigurationICVFX_CameraSettings& FDisplayClusterViewportConfiguration_ICVFXCamera::GetCameraSettings() const
{
	return ConfigurationCameraComponent.GetCameraSettingsICVFX();
}

FString FDisplayClusterViewportConfiguration_ICVFXCamera::GetCameraUniqueId() const
{
	return CameraComponent.GetCameraUniqueId();
}

bool FDisplayClusterViewportConfiguration_ICVFXCamera::UpdateChromakeyViewport()
{
	const FDisplayClusterConfigurationICVFX_StageSettings* StageSettings = Configuration.GetStageSettings();
	if (!StageSettings)
		{
		return false;
	}

	// Check the configuration rules to see if chromakey viewport rendering is allowed.
	const FDisplayClusterConfigurationICVFX_ChromakeyRenderSettings* ChromakeyRenderSettings = GetCameraSettings().Chromakey.GetChromakeyRenderSettings(*StageSettings);
	const bool bShouldUseChromakeyViewport = ChromakeyRenderSettings && ChromakeyRenderSettings->ShouldUseChromakeyViewport(*StageSettings);

	// Returns true if any target viewport is configured to use chromakey viewport rendering.
	auto IsChromakeyUsedByAnyTargetViewport = [&]()
	{
			// Performance: Render CK only when it is in use.
	for (const FTargetViewport& TargetViewportIt : TargetViewports)
	{
				if (TargetViewportIt.ChromakeySource == EDisplayClusterShaderParametersICVFX_ChromakeySource::ChromakeyLayers)
		{
			return true;
		}
	}

	return false;
		};

	// Update CK viewport reference
	if (CameraViewport.IsValid() // CK requires the InCamera
		&& bShouldUseChromakeyViewport
		&& IsChromakeyUsedByAnyTargetViewport())
	{
	// Create new chromakey viewport
		if (FDisplayClusterViewport* NewOrExistChromakeyViewport = FDisplayClusterViewportConfigurationHelpers_ICVFX::GetOrCreateChromakeyViewport(Configuration, CameraComponent, GetCameraSettings()))
	{
			ChromakeyViewport = NewOrExistChromakeyViewport->AsShared();

		// Update chromakey viewport settings
		FDisplayClusterViewportConfigurationHelpers_ICVFX::UpdateChromakeyViewportSettings(*ChromakeyViewport, *CameraViewport, GetCameraSettings());

		// Support projection policy update
		ChromakeyViewport->UpdateConfiguration_ProjectionPolicy();

		// reuse for EditorPreview
			const FString ICVFXCameraId = CameraComponent.GetCameraUniqueId();
		FDisplayClusterViewportConfigurationHelpers_ICVFX::PreviewReuseChromakeyViewportWithinClusterNodes(*ChromakeyViewport, ICVFXCameraId);

		return true;
	}
	}

	return false;
};

void FDisplayClusterViewportConfiguration_ICVFXCamera::UpdateICVFXSettingsForTargetViewports()
{
	check(CameraViewport.IsValid());

	const FDisplayClusterConfigurationICVFX_StageSettings* StageSettings = Configuration.GetStageSettings();
	ADisplayClusterRootActor* SceneRootActor = Configuration.GetRootActor(EDisplayClusterRootActorType::Scene);
	ADisplayClusterRootActor* ConfigurationRootActor = Configuration.GetRootActor(EDisplayClusterRootActorType::Configuration);
	if (StageSettings == nullptr || SceneRootActor == nullptr || ConfigurationRootActor == nullptr)
		{
		return;
	}

	// Chromakey viewport name with alpha channel
	const FString ChromakeyViewportId(ChromakeyViewport.IsValid() ? ChromakeyViewport->GetId() : TEXT(""));

	const FDisplayClusterConfigurationICVFX_CameraSettings& CameraSettings = GetCameraSettings();

	FDisplayClusterShaderParameters_ICVFX::FCameraSettings ShaderParametersCameraSettings =
		CameraComponent.GetICVFXCameraShaderParameters(*StageSettings, CameraSettings);
	{
		// Assign the camera viewport even if it isn’t rendered; math data is still required.
		ShaderParametersCameraSettings.Resource.ViewportId = CameraViewport->GetId();

		// Rendering order for camera overlap
		const FString InnerFrustumID = CameraComponent.GetCameraUniqueId();
		const int32 CameraRenderOrder = ConfigurationRootActor->GetInnerFrustumPriority(InnerFrustumID);
		ShaderParametersCameraSettings.RenderOrder = (CameraRenderOrder < 0) ? CameraSettings.RenderSettings.RenderOrder : CameraRenderOrder;
	}

	// Update ICVFX settings for the all target viewports 
	for (const FTargetViewport& TargetViewportIt : TargetViewports)
	{
		// Per-viewport CK
		ShaderParametersCameraSettings.ChromakeySource = TargetViewportIt.ChromakeySource;

		// Adds shader parameters from this camera to the camera list of the target viewport.
		TargetViewportIt.Viewport->GetRenderSettingsICVFXImpl().ICVFX.Cameras.Add(ShaderParametersCameraSettings);

		// Assign this chromakey to all supported targets
		const bool bEnableChromakey = TargetViewportIt.ChromakeySource != EDisplayClusterShaderParametersICVFX_ChromakeySource::Disabled;
		const bool bEnableChromakeyMarkers = bEnableChromakey && !EnumHasAnyFlags(TargetViewportIt.Viewport->GetRenderSettingsICVFX().Flags, EDisplayClusterViewportICVFXFlags::DisableChromakeyMarkers);

		// Gain direct access to internal settings of the viewport:
		FDisplayClusterViewport_RenderSettingsICVFX& InOutOuterViewportRenderSettingsICVFX = TargetViewportIt.Viewport->GetRenderSettingsICVFXImpl();
		FDisplayClusterShaderParameters_ICVFX::FCameraSettings& DstCameraData = InOutOuterViewportRenderSettingsICVFX.ICVFX.Cameras.Last();

		// Setup chromakey with markers
		FDisplayClusterViewportConfigurationHelpers_ICVFX::UpdateCameraSettings_Chromakey(DstCameraData, *StageSettings, GetCameraSettings(), bEnableChromakey, bEnableChromakeyMarkers, ChromakeyViewportId);

		// Setup overlap chromakey with markers
		FDisplayClusterViewportConfigurationHelpers_ICVFX::UpdateCameraSettings_OverlapChromakey(DstCameraData, *StageSettings, GetCameraSettings(), bEnableChromakeyMarkers);
	}
}
