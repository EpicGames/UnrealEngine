// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Render/Viewport/Configuration/DisplayClusterViewportConfigurationHelpers_ICVFX.h"
#include "ShaderParameters/DisplayClusterShaderParameters_ICVFX.h"

class UDisplayClusterICVFXCameraComponent;
struct FDisplayClusterConfigurationICVFX_CameraSettings;
class FDisplayClusterViewport;
class FDisplayClusterViewportConfiguration;

/**
* ICVFX Configurator: InCamera instance
*/
class FDisplayClusterViewportConfiguration_ICVFXCamera
{
public:
	FDisplayClusterViewportConfiguration_ICVFXCamera(FDisplayClusterViewportConfiguration& InConfiguration, UDisplayClusterICVFXCameraComponent& InCameraComponent, UDisplayClusterICVFXCameraComponent& InConfigurationCameraComponent)
		: Configuration(InConfiguration), CameraComponent(InCameraComponent), ConfigurationCameraComponent(InConfigurationCameraComponent)
	{ }

	/** Initialize CameraContext. */
	bool Initialize();

	/** Creates camera and chromakey viewports and initializes their target OuterViewports. */
	void Update();

	/** Returns true if the camera frustum is visible on the TargetViewport geometry. */
	bool IsCameraProjectionVisibleOnViewport(FDisplayClusterViewport* TargetViewport);

	/** Gets a ICVFX camera settings. */
	const FDisplayClusterConfigurationICVFX_CameraSettings& GetCameraSettings() const;

	/** Gets the name of the ICVFX camera. */
	FString GetCameraUniqueId() const;

	/**
	* Returns true if this camera has media bindings
	* (assigned to a media input or output).
	*
	* Note: Media is available only in cluster mode.
	*/
	bool HasAnyMediaBindings() const;

protected:
	/** Gets or creates an In-Camera viewport and configures it for use in the ICVFX rendering stack. */
	bool GetOrCreateAndSetupInnerCameraViewport();

	/** Updates the Chromakey viewport and configures it for use in the ICVFX rendering stack. */
	bool UpdateChromakeyViewport();

	/**
	 * Updates ICVFX settings for all target viewports.
	 *
	 * Applies the necessary ICVFX configuration (e.g., In-Camera, Chromakey, and related
	 * parameters) so that target viewports are correctly prepared for the ICVFX rendering stack.
	 */
	void UpdateICVFXSettingsForTargetViewports();

public:
	struct FICVFXCameraContext
	{
		//@todo: add stereo context support
		FRotator ViewRotation;
		FVector  ViewLocation;
		FMatrix  PrjMatrix;
	};

	// Camera context, used for visibility test vs outer
	FICVFXCameraContext CameraContext;

	// The inner camera viewport ref
	TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe> CameraViewport;

	// The inner camera chromakey viewport ref
	TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe> ChromakeyViewport;

	struct FTargetViewport
	{
		FTargetViewport(const TSharedRef<FDisplayClusterViewport, ESPMode::ThreadSafe>& InViewport)
			: Viewport(InViewport)
		{ }

		// The reference to the viewport
		const TSharedRef<FDisplayClusterViewport, ESPMode::ThreadSafe> Viewport;

		// The camera chromakey type from this viewport.
		EDisplayClusterShaderParametersICVFX_ChromakeySource ChromakeySource = EDisplayClusterShaderParametersICVFX_ChromakeySource::Disabled;
	};

	// List of OuterViewports for this camera
	TArray<FTargetViewport> TargetViewports;

private:
	FDisplayClusterViewportConfiguration& Configuration;

	// Camera component in scene DCRA
	UDisplayClusterICVFXCameraComponent& CameraComponent;

	// Camera component in configuration DCRA
	UDisplayClusterICVFXCameraComponent& ConfigurationCameraComponent;
};
