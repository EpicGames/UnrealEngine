// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterViewportConfiguration_Postprocess.h"

#include "Render/Viewport/DisplayClusterViewportManager.h"
#include "Render/Viewport/Postprocess/DisplayClusterViewportPostProcessManager.h"
#include "Render/Viewport/Postprocess/DisplayClusterViewportPostProcessOutputRemap.h"
#include "Render/Viewport/RenderFrame/DisplayClusterRenderFrameSettings.h"

#include "DisplayClusterRootActor.h"
#include "Engine/StaticMesh.h"
#include "ProceduralMeshComponent.h"
#include "Misc/DisplayClusterLog.h"

///////////////////////////////////////////////////////////////////
// Copied from "TextureShareDisplayCluster/Misc/TextureShareDisplayClusterStrings.h"
namespace TextureShareDisplayClusterStrings
{
	namespace Postprocess
	{
		static constexpr auto TextureShare = TEXT("TextureShare");
	}

	namespace Projection
	{
		static constexpr auto TextureShare = TEXT("textureshare");
	}
};

TArray<FString> FDisplayClusterViewportConfiguration_Postprocess::DisabledPostprocessNames;

///////////////////////////////////////////////////////////////////
// FDisplayClusterViewportConfiguration_Postprocess
///////////////////////////////////////////////////////////////////
void FDisplayClusterViewportConfiguration_Postprocess::AddInternalPostprocess(const FString& InPostprocessName)
{
	InternalPostprocessNames.AddUnique(InPostprocessName);
}

void FDisplayClusterViewportConfiguration_Postprocess::UpdateClusterNodePostProcess(const FString& InClusterNodeId)
{
	if (InClusterNodeId.IsEmpty())
	{
		// this function expects a exists cluster node name
		return;
	}

	FDisplayClusterViewportManager* ViewportManager = Configuration.GetViewportManagerImpl();
	if (!ViewportManager)
	{
		return;
	}

	const UDisplayClusterConfigurationData* ConfigurationData = Configuration.GetConfigurationData();;
	if (const UDisplayClusterConfigurationClusterNode* ClusterNode = (ConfigurationData && ConfigurationData->Cluster) ? ConfigurationData->Cluster->GetNode(InClusterNodeId) : nullptr)
	{
		TSharedPtr<FDisplayClusterViewportPostProcessManager, ESPMode::ThreadSafe> PPManager = ViewportManager->PostProcessManager;
		if (PPManager.IsValid())
		{
			// Add TextureShare postprocess:
			if (ClusterNode->bEnableTextureShare && !Configuration.GetRenderFrameSettings().IsPreviewRendering())
			{
				AddInternalPostprocess(TextureShareDisplayClusterStrings::Postprocess::TextureShare);
			}

			{
				// Find unused PP:
				TArray<FString> UnusedPP;
				for (const FString& It : PPManager->GetPostprocess())
				{
					// Leave defined postprocess (dynamic reconf)
					bool IsDefinedPostProcess = ClusterNode->Postprocess.Contains(It);

					// Support InternalPostprocess dynamic reconf
					if (InternalPostprocessNames.Find(It) != INDEX_NONE)
					{
						IsDefinedPostProcess = true;
					}

					if (!IsDefinedPostProcess)
					{
						UnusedPP.Add(It);
					}
				}

				// Delete unused PP:
				for (const FString& It : UnusedPP)
				{
					PPManager->RemovePostprocess(It);
				}
			}

			// Create InternalPostprocess
			for (const FString& InternalPostprocessId : InternalPostprocessNames)
			{
				if (DisabledPostprocessNames.Find(InternalPostprocessId) != INDEX_NONE)
				{
					// Always skip a postprocess that cannot be initialized
					continue;
				}

				TSharedPtr<IDisplayClusterPostProcess, ESPMode::ThreadSafe> ExistPostProcess = PPManager->FindPostProcess(InternalPostprocessId);
				if (!ExistPostProcess.IsValid())
				{
					// Create postprocess instance
					FDisplayClusterConfigurationPostprocess ConfigurationPostprocess;
					ConfigurationPostprocess.Type = InternalPostprocessId;

					if (PPManager->CanBeCreated(&ConfigurationPostprocess) && !PPManager->CreatePostprocess(InternalPostprocessId, &ConfigurationPostprocess))
					{
						// Can't create... Disable this postprocess
						DisabledPostprocessNames.AddUnique(InternalPostprocessId);

						UE_LOGF(LogDisplayClusterViewport, Error, "Can't create postprocess '%ls' required by cluster node '%ls': Disabled", *InternalPostprocessId, *InClusterNodeId);
					}
				}
			}

			// Create and update PP
			for (const TPair<FString, FDisplayClusterConfigurationPostprocess>& It : ClusterNode->Postprocess)
			{
				if (DisabledPostprocessNames.Find(It.Value.Type) != INDEX_NONE)
				{
					// Always skip a postprocess that cannot be initialized
					continue;
				}

				TSharedPtr<IDisplayClusterPostProcess, ESPMode::ThreadSafe> ExistPostProcess = PPManager->FindPostProcess(It.Key);
				if (ExistPostProcess.IsValid())
				{
					if (ExistPostProcess->IsConfigurationChanged(&It.Value))
					{
						PPManager->UpdatePostprocess(It.Key, &It.Value);
					}
				}
				else
				{
					if (PPManager->CanBeCreated(&It.Value) && !PPManager->CreatePostprocess(It.Key, &It.Value))
					{
						// Can't create... Disable this postprocess
						DisabledPostprocessNames.AddUnique(It.Value.Type);

						UE_LOGF(LogDisplayClusterViewport, Error, "Can't create postprocess '%ls' on cluster node '%ls': Disabled", *It.Value.Type, *InClusterNodeId);
					}
				}
			}

			// Update OutputRemap PP
			{
				const struct FDisplayClusterConfigurationFramePostProcess_OutputRemap& OutputRemapCfg = ClusterNode->OutputRemap;
				if (OutputRemapCfg.bEnable)
				{
					switch (OutputRemapCfg.DataSource)
					{
					case EDisplayClusterConfigurationFramePostProcess_OutputRemapSource::StaticMesh:
						PPManager->GetOutputRemap()->UpdateConfiguration_StaticMesh(OutputRemapCfg.StaticMesh);
						break;

					case EDisplayClusterConfigurationFramePostProcess_OutputRemapSource::MeshComponent:
						if (!OutputRemapCfg.MeshComponentName.IsEmpty())
						{
							if (ADisplayClusterRootActor* SceneRootActor = Configuration.GetRootActor(EDisplayClusterRootActorType::Scene))
							{
								// Get the StaticMeshComponent
								UStaticMeshComponent* StaticMeshComponent = SceneRootActor->GetComponentByName<UStaticMeshComponent>(OutputRemapCfg.MeshComponentName);
								if (StaticMeshComponent != nullptr)
								{
									PPManager->GetOutputRemap()->UpdateConfiguration_StaticMeshComponent(StaticMeshComponent);
								}
								else
								{
									// Get the procedural mesh component
									UProceduralMeshComponent* ProceduralMeshComponent = SceneRootActor->GetComponentByName<UProceduralMeshComponent>(OutputRemapCfg.MeshComponentName);
									PPManager->GetOutputRemap()->UpdateConfiguration_ProceduralMeshComponent(ProceduralMeshComponent);
								}
							}
						}
						else
						{
							PPManager->GetOutputRemap()->UpdateConfiguration_Disabled();
						}
						break;

					case EDisplayClusterConfigurationFramePostProcess_OutputRemapSource::ExternalFile:
						PPManager->GetOutputRemap()->UpdateConfiguration_ExternalFile(OutputRemapCfg.ExternalFile);
						break;

					default:
						PPManager->GetOutputRemap()->UpdateConfiguration_Disabled();
					}
				}
				else
				{
					PPManager->GetOutputRemap()->UpdateConfiguration_Disabled();
				}
			}
		}
	}
}
