// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Render/Viewport/Configuration/DisplayClusterViewportConfiguration.h"

/**
 * A helper class that configure postprocess.
 */
struct FDisplayClusterViewportConfiguration_Postprocess
{
public:
	FDisplayClusterViewportConfiguration_Postprocess(FDisplayClusterViewportConfiguration& InConfiguration)
		: Configuration(InConfiguration)
	{ }

public:
	/** Update postprocess for cluster node.
	 * 
	 * @param ClusterNodeId - cluster node name (special names are supported)
	 */
	void UpdateClusterNodePostProcess(const FString& ClusterNodeId);

protected:
	/**
	 * By default, all postprocess parameters are defined in the FDisplayClusterConfigurationPostprocess structure.
	 * But some post-processes may use their own logic to enable or disable.
	 * 
	 * For example, TextureSharePP will only be enabled with a special condition:
	 * 1. The EnableTextureShare checkbox is checked in the cluster node settings
	 * 2. The TextureShare plugin is enabled for this project.
	 */
	void AddInternalPostprocess(const FString& InPostprocessName);

private:
	// The configuration api
	FDisplayClusterViewportConfiguration& Configuration;

	/**
	 * The PP is updated from the configuration every frame.
	 * If the post process name is unknown or initialization fails, an error message appears in the log.
	 * To prevent duplicate log error messages in subsequent frames, this PP name is added to the DisabledPostprocessNames list.
	 *
	 * All PPs with names from this list will be ignored.
	 */
	static TArray<FString> DisabledPostprocessNames;

	/**
	 * This post-processing list is updated at runtime. See AddInternalPostprocess()
	 */
	TArray<FString> InternalPostprocessNames;
};
