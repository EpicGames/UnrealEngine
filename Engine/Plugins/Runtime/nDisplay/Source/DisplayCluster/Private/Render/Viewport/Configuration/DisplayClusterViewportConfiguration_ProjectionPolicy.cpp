// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterViewportConfiguration_ProjectionPolicy.h"
#include "DisplayClusterViewportConfigurationHelpers_ICVFX.h"

#include "Render/Viewport/DisplayClusterViewport.h"
#include "Render/Viewport/DisplayClusterViewportManager.h"

#include "DisplayClusterViewportConfigurationHelpers.h"
#include "DisplayClusterViewportConfigurationHelpers_Postprocess.h"
#include "DisplayClusterConfigurationTypes.h"

#include "IDisplayClusterProjection.h"
#include "Render/Projection/IDisplayClusterProjectionPolicy.h"
#include "DisplayClusterProjectionStrings.h"

#include "DisplayClusterRootActor.h"
#include "Misc/DisplayClusterHelpers.h"
#include "Misc/DisplayClusterLog.h"

#include "Components/DisplayClusterICVFXCameraComponent.h"
#include "Components/DisplayClusterCameraComponent.h"
#include "Containers/DisplayClusterProjectionCameraPolicySettings.h"

namespace UE::DisplayCluster::Configuration::ProjectionPolicyHelpers
{
	template<class T>
	T* FindComponentByName(ADisplayClusterRootActor& InRootActor, const FString& InComponentName)
	{
		TArray<T*> ActorComps;
		InRootActor.GetComponents(ActorComps);
		for (T* CompIt : ActorComps)
		{
			if (CompIt->GetName() == InComponentName)
			{
				return CompIt;
			}
		}

		return nullptr;
	}
};
using namespace UE::DisplayCluster::Configuration;

///////////////////////////////////////////////////////////////////
// FDisplayClusterViewportConfiguration_ProjectionPolicy
///////////////////////////////////////////////////////////////////
void FDisplayClusterViewportConfiguration_ProjectionPolicy::Update()
{
	if (FDisplayClusterViewportManager* ViewportManager = Configuration.GetViewportManagerImpl())
	{
		for (const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& ViewportIt : ViewportManager->ImplGetCurrentRenderFrameViewports())
		{
			// ignore internal viewport
			if (ViewportIt.IsValid() && !ViewportIt->IsInternalViewport())
			{
				if (ViewportIt->GetProjectionPolicy().IsValid())
				{
					// Support advanced logic for 'camera' projection policy
					if (ViewportIt->GetProjectionPolicy()->GetType().Compare(DisplayClusterProjectionStrings::projection::Camera) == 0)
					{
						UpdateCameraPolicy(*ViewportIt);
					}

					// Support postprocesses from the ViewPoint component
					if (ViewportIt->GetProjectionPolicy()->ShouldUseViewPointComponentPostProcesses(ViewportIt.Get()))
					{
						// ViewPoint can use its own postprocess, so we apply it before anything else in case others override it.
						if (UDisplayClusterCameraComponent* SceneViewPointCameraComponent = ViewportIt->GetViewPointCameraComponent(EDisplayClusterRootActorType::Scene))
						{
							// Apply the PP of the cameras referenced by the ViewPoint to the viewport.
							SceneViewPointCameraComponent->ApplyViewPointComponentPostProcessesToViewport(ViewportIt.Get());
						}
					}

					// Projection policies can override postprocess settings
					// The camera PP override code has been moved from FDisplayClusterViewportConfiguration_ProjectionPolicy::UpdateCameraPolicyForCameraComponent()
					// to the projection policy new function
					ViewportIt->GetProjectionPolicy()->UpdatePostProcessSettings(ViewportIt.Get());
				}
			}
		}
	}
}

bool FDisplayClusterViewportConfiguration_ProjectionPolicy::UpdateCameraPolicy(FDisplayClusterViewport& DstViewport)
{
	const TMap<FString, FString>& CameraPolicyParameters = DstViewport.GetProjectionPolicy()->GetParameters();

	FString CameraComponentId;
	// Get assigned camera ID
	if (!DisplayClusterHelpers::map::template ExtractValue(CameraPolicyParameters, DisplayClusterProjectionStrings::cfg::camera::Component, CameraComponentId))
	{
		// use default cameras
		DstViewport.ResetShowLogMsgOnce(EDisplayClusterViewportShowLogMsgOnce::UpdateCameraPolicy);

		return true;
	}

	if (CameraComponentId.IsEmpty())
	{
		if (DstViewport.CanShowLogMsgOnce(EDisplayClusterViewportShowLogMsgOnce::UpdateCameraPolicy_ReferencedCameraNameIsEmpty))
		{
			UE_LOGF(LogDisplayClusterViewport, Verbose, "Viewport '%ls': referenced camera '' (empty name).", *DstViewport.GetId());
		}

		return false;
	}


	// ICVFX InCamera viewport already configured in the FDisplayClusterViewportConfiguration_ICVFXCamera::GetOrCreateAndSetupInnerCameraViewport().
	if (EnumHasAnyFlags(DstViewport.GetRenderSettingsICVFX().RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::InCamera))
	{
		DstViewport.ResetShowLogMsgOnce(EDisplayClusterViewportShowLogMsgOnce::UpdateCameraPolicy);

		return true;
	}

	// Update projection policy 'camera' for the ICVFXCameraComponent.
	if (UpdateCameraPolicyForICVFXCameraComponent(DstViewport, CameraComponentId))
	{
		DstViewport.ResetShowLogMsgOnce(EDisplayClusterViewportShowLogMsgOnce::UpdateCameraPolicy);

		return true;
	}

	// Update projection policy 'camera' for the CameraComponent.
	if (UpdateCameraPolicyForCameraComponent(DstViewport, CameraComponentId))
	{
		DstViewport.ResetShowLogMsgOnce(EDisplayClusterViewportShowLogMsgOnce::UpdateCameraPolicy);

		return true;
	}

	// Report an error to the log once.
	if (DstViewport.CanShowLogMsgOnce(EDisplayClusterViewportShowLogMsgOnce::UpdateCameraPolicy_ReferencedCameraNotFound))
	{
		UE_LOGF(LogDisplayClusterViewport, Error, "Viewport '%ls': referenced camera '%ls' not found.", *DstViewport.GetId(), *CameraComponentId);
	}

	return false;
}

bool FDisplayClusterViewportConfiguration_ProjectionPolicy::UpdateCameraPolicyForCameraComponent(FDisplayClusterViewport& DstViewport, const FString& CameraComponentId)
{
	if (ADisplayClusterRootActor* SceneRootActor = Configuration.GetRootActor(EDisplayClusterRootActorType::Scene))
	{
		if (UCameraComponent* SceneCameraComp = ProjectionPolicyHelpers::FindComponentByName<UCameraComponent>(*SceneRootActor, CameraComponentId))
		{
			// Now this parameter is always 1 for the standard camera component.
			const float FOVMultiplier = 1.f;

			// add camera's post processing materials
			// Moved to IDisplayClusterProjectionPolicy::UpdatePostProcessSettings()

			FDisplayClusterProjectionCameraPolicySettings PolicyCameraSettings;
			PolicyCameraSettings.FOVMultiplier = FOVMultiplier;

			// Initialize camera policy with camera component and settings
			return IDisplayClusterProjection::Get().CameraPolicySetCamera(DstViewport.GetProjectionPolicy(), SceneCameraComp, PolicyCameraSettings);
		}
	}

	return false;
}

bool FDisplayClusterViewportConfiguration_ProjectionPolicy::UpdateCameraPolicyForICVFXCameraComponent(FDisplayClusterViewport& DstViewport, const FString& CameraComponentId)
{
	// If a regular viewport window references ICVFXCameraComponent, it should be configured using parameters from UDisplayClusterICVFXCameraComponent.
	ADisplayClusterRootActor*         SceneRootActor = Configuration.GetRootActor(EDisplayClusterRootActorType::Scene);
	UDisplayClusterICVFXCameraComponent* SceneCameraComp = SceneRootActor ?
		ProjectionPolicyHelpers::FindComponentByName<UDisplayClusterICVFXCameraComponent>(*SceneRootActor, CameraComponentId) : nullptr;
	if (!SceneCameraComp)
	{
		// This is not the projection policy of the ICVFX camera.
		return false;
	}

	ADisplayClusterRootActor* ConfigurationRootActor = Configuration.GetRootActor(EDisplayClusterRootActorType::Configuration);
	UDisplayClusterICVFXCameraComponent* ConfigurationCameraComp = ConfigurationRootActor
		? ProjectionPolicyHelpers::FindComponentByName<UDisplayClusterICVFXCameraComponent>(*ConfigurationRootActor, CameraComponentId)
		: nullptr;

	if (!ConfigurationCameraComp)
	{
		ConfigurationCameraComp = SceneCameraComp;
	}

	// Note: Use parameters from FDisplayClusterConfigurationICVFX_CameraSettings:
	// @TODO: Add additional parameters here. (see FDisplayClusterViewportConfigurationHelpers_ICVFX::UpdateCameraViewportSettings)

	// 1. [-] OCIO: use viewport setings	
	// 2. [-] Viewport Rect: use viewport settings	
	// 3. Apply postprocess for ICVFX camera
	SceneCameraComp->ApplyICVFXCameraPostProcessesToViewport(&DstViewport, EDisplayClusterViewportCameraPostProcessFlags::All);

	// 4. [-] Post-render (Override, Blur, Mips): use viewport settings
	// 5. [-] Custom frustum: None
	// 6. [-] RenderTargetAdaptRatio: None
	// 7. [-] BufferRatio (ScreenPercentage): use viewport settings
	// 8. [-] Upscaler settings: Use viewport settings
	// 9. [-] Tile rendering: use viewport settings

	// FIN. Update camera viewport projection policy settings (Use settings from the UDisplayClusterICVFXCameraComponent->FDisplayClusterConfigurationICVFX_CameraSettings):
	// * FOVMultiplier
	// * FrustumRotation
	// * FrustumOffset
	// * OffCenterProjectionOffset
	return FDisplayClusterViewportConfigurationHelpers_ICVFX::UpdateCameraProjectionSettingsICVFX(
		*DstViewport.Configuration,
		*SceneCameraComp,
		ConfigurationCameraComp->GetCameraSettingsICVFX(),
		DstViewport.GetProjectionPolicy());
}
