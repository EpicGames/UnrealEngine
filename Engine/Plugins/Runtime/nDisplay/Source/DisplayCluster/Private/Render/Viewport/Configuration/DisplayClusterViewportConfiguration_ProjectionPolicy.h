// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Render/Viewport/Configuration/DisplayClusterViewportConfiguration.h"

class FDisplayClusterViewport;
class UCameraComponent;
class UDisplayClusterICVFXCameraComponent;

/**
 * A helper class that configure projection policyes for viewports.
 */
struct FDisplayClusterViewportConfiguration_ProjectionPolicy
{
public:
	FDisplayClusterViewportConfiguration_ProjectionPolicy(FDisplayClusterViewportConfiguration& InConfiguration)
		: Configuration(InConfiguration)
	{ }

public:
	/** Special logic to additionally perform some updates within projection policies.
	* Currently, only the advanced logic for the "camera" projection policy is implemented.
	*/
	void Update();

private:
	/** Basic implementation of extended logic for projection policy "camera */
	bool UpdateCameraPolicy(FDisplayClusterViewport& DstViewport);

	/**
	  * Updates the projection policy of a regular (outer) viewport using settings
	  * from the specified CameraComponent.
	 *
	 * @param DstViewport       The outer (regular) viewport to update.
	 * @param CameraComponentId The identifier of the CameraComponent whose settings will be applied.
	 *
	 * @return true if the camera policy was successfully updated, false otherwise.
	 */
	bool UpdateCameraPolicyForCameraComponent(FDisplayClusterViewport& DstViewport, const FString& CameraComponentId);

	 /**
	  * Updates the projection policy of a regular (outer) viewport using settings
	  * from the specified ICVFX CameraComponent.
	  *
	  * This is used when the input viewport has the 'camera' projection policy
	  * and must be configured to match the ICVFX camera component.
	  *
	  * @param DstViewport              The outer (regular) viewport to update.
	  * @param ICVFXCameraComponentId   The identifier of the ICVFX CameraComponent whose settings will be applied.
	  *
	  * @return true if the projection policy was successfully updated, false otherwise.
	  */
	bool UpdateCameraPolicyForICVFXCameraComponent(FDisplayClusterViewport& DstViewport, const FString& ICVFXCameraComponentId);

private:
	FDisplayClusterViewportConfiguration& Configuration;
};
