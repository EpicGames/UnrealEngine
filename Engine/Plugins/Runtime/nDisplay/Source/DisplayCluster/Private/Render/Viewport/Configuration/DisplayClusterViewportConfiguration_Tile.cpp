// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterViewportConfiguration_Tile.h"

#include "DisplayClusterViewportConfiguration.h"
#include "DisplayClusterViewportConfigurationHelpers.h"
#include "DisplayClusterViewportConfigurationHelpers_Tile.h"

#include "Render/Viewport/DisplayClusterViewport.h"
#include "Render/Viewport/DisplayClusterViewportManager.h"
#include "Render/Viewport/RenderFrame/DisplayClusterRenderFrameSettings.h"

#include "DisplayClusterRootActor.h"
#include "DisplayClusterConfigurationTypes_Tile.h"

#include "Misc/DisplayClusterLog.h"
#include "Misc/Parse.h"

/**
 * Iterate over tile source viewports that are ready to split.
 *
 * @param Configuration Viewport configuration.
 * @param Pred          Callable invoked as Pred(Viewport, TilePosition) for each matching viewport.
 */
template <typename Predicate>
void ForEachReadyToSplitTileSourceViewport(const FDisplayClusterViewportConfiguration& Configuration, Predicate Pred)
{
	if (FDisplayClusterViewportManager* ViewportManager = Configuration.GetViewportManagerImpl())
	{
		for (const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& ViewportIt : ViewportManager->ImplGetCurrentRenderFrameViewports())
		{
			if (ViewportIt.IsValid() && ViewportIt->GetRenderSettings().TileSettings.GetType() == EDisplayClusterViewportTileType::Source)
			{
				// Check if the viewport can be split into tiles.
				if (!ViewportIt->CanSplitIntoTiles())
				{
					continue;
				}

				const FIntPoint& TileSize = ViewportIt->GetRenderSettings().TileSettings.GetSize();

				// Iterate over all tile viewports
				for (int32 PosX = 0; PosX < TileSize.X; PosX++)
				{
					for (int32 PosY = 0; PosY < TileSize.Y; PosY++)
					{
						::Invoke(Pred, *ViewportIt, FIntPoint(PosX, PosY));
					}
				}
			}
		}
	}
	else
	{
		UE_LOGF(LogDisplayClusterViewport, Warning, "ForEachReadyToSplitTileSourceViewport() - ViewportManager not found. ");
	}
}

void FDisplayClusterViewportConfiguration_Tile::Update()
{
	ImplBeginReallocateViewports();

	ForEachReadyToSplitTileSourceViewport(Configuration, [](FDisplayClusterViewport& InSourceViewport, const FIntPoint& InTilePos)
	{
		// Split the source viewports into multiple tiles.
		FDisplayClusterViewportConfigurationHelpers_Tile::GetOrCreateTileViewport(InSourceViewport, InTilePos);
	});

	ImplFinishReallocateViewports();
}

void FDisplayClusterViewportConfiguration_Tile::ImplBeginReallocateViewports() const
{
	if (FDisplayClusterViewportManager* ViewportManager = Configuration.GetViewportManagerImpl())
	{
		for (const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& ViewportIt : ViewportManager->ImplGetCurrentRenderFrameViewports())
		{
			if (ViewportIt.IsValid())
			{
				ViewportIt->GetRenderSettingsImpl().TileSettings.SetTileStateToBeUsed(false);
			}
		}
	}
	else
	{
		UE_LOGF(LogDisplayClusterViewport, Warning, "Tile:BeginReallocate - ViewportManager not found. ");
	}
}

void FDisplayClusterViewportConfiguration_Tile::ImplFinishReallocateViewports() const
{
	if (FDisplayClusterViewportManager* ViewportManager = Configuration.GetViewportManagerImpl())
	{
		TArray<TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>> UnusedViewports;

		// Collect all unused viewports for remove
		for (const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& ViewportIt : ViewportManager->ImplGetCurrentRenderFrameViewports())
		{
			if (ViewportIt.IsValid() && ViewportIt->GetRenderSettings().TileSettings.GetType() == EDisplayClusterViewportTileType::UnusedTile)
			{
				UnusedViewports.Add(ViewportIt);
			}
		}

		// Delete unused viewports:
		for (TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& RemoveViewportIt : UnusedViewports)
		{
			if (RemoveViewportIt.IsValid())
			{
				ViewportManager->DeleteViewport(RemoveViewportIt->GetId());
			}
		}

		UnusedViewports.Empty();
	}
	else
	{
		UE_LOGF(LogDisplayClusterViewport, Warning, "Tile:FinishReallocate - ViewportManager not found. ");
	}
}
