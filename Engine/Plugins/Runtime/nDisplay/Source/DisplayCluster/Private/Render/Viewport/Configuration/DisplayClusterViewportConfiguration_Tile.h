// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_RenderSettingsICVFX.h"

class FDisplayClusterViewportConfiguration;

/**
 * A helper class that configure Tile render.
 */
class FDisplayClusterViewportConfiguration_Tile
{
public:
	FDisplayClusterViewportConfiguration_Tile(const FDisplayClusterViewportConfiguration& InConfiguration)
		: Configuration(InConfiguration)
	{ }

	~FDisplayClusterViewportConfiguration_Tile() = default;

public:
	/** Update Tile viewports for a new frame. */
	void Update();

private:
	/** Mark all Tile viewports as unused before updating. */
	void ImplBeginReallocateViewports() const;

	/** Delete unused Tile viewports. */
	void ImplFinishReallocateViewports() const;

private:
	// Viewport configuration API
	const FDisplayClusterViewportConfiguration& Configuration;
};
