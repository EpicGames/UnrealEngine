// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterViewportConfiguration_Viewport.h"

#include "Render/Viewport/Configuration/DisplayClusterViewportConfiguration.h"
#include "Render/Viewport/Configuration/DisplayClusterViewportConfigurationHelpers.h"
#include "Render/Viewport/RenderFrame/DisplayClusterRenderFrameSettings.h"
#include "Render/Viewport/DisplayClusterViewport.h"
#include "Render/Viewport/DisplayClusterViewportManager.h"

#include "Render/Projection/IDisplayClusterProjectionPolicy.h"

#include "DisplayClusterConfigurationTypes_Viewport.h"

#include "DisplayClusterRootActor.h"

void FDisplayClusterViewportConfiguration_Viewport::CreateOrUpdateViewportInstance(FDisplayClusterViewportConfiguration& InOutConfiguration)
{
	FDisplayClusterViewportManager* ViewportManager = InOutConfiguration.GetViewportManagerImpl();
	if (!ViewportManager)
	{
		return;
	}

	const FString CurrentClusterNodeId = InOutConfiguration.GetClusterNodeId();
	// When creating a viewport for another cluster node, we need to temporarily override the cluster node name
	InOutConfiguration.SetClusterNodeId(ClusterNodeId);
	
	const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe> ExistViewport = ViewportManager->ImplFindViewport(ViewportId);
	if (ExistViewport.IsValid())
	{
		UpdateViewportConfiguration(*ExistViewport, ConfigurationViewport);

		// Store viewpoint of exist viewport
		ViewPointCameraComponent = ExistViewport->GetViewPointCameraComponent(EDisplayClusterRootActorType::Any);

		// Store Viewport
		Viewport = ExistViewport;
	}
	else
	{
		if (FDisplayClusterViewport* NewViewport = ViewportManager->CreateViewport(ViewportId, ConfigurationViewport))
		{
			// Store viewpoint of new viewport
			ViewPointCameraComponent = NewViewport->GetViewPointCameraComponent(EDisplayClusterRootActorType::Any);

			// Store Viewport
			Viewport = NewViewport->AsShared();
		}
	}

	// Restore current render frame settings
	InOutConfiguration.SetClusterNodeId(CurrentClusterNodeId);
}

void FDisplayClusterViewportConfiguration_Viewport::PostUpdateViewportInstance(FDisplayClusterViewportConfiguration& InOutConfiguration)
{
	// Call the post-update event for the projection policy
	if (Viewport.IsValid() && Viewport->GetProjectionPolicy().IsValid())
	{
		Viewport->GetProjectionPolicy()->PostUpdateBaseConfiguration(Viewport.Get());
	}
}

bool FDisplayClusterViewportConfiguration_Viewport::UpdateViewportConfiguration(FDisplayClusterViewport& DstViewport, const UDisplayClusterConfigurationViewport& ConfigurationViewport)
{
	check(IsInGameThread());

	// Reset runtime flags from prev frame:
	DstViewport.ResetRuntimeParameters(&ConfigurationViewport);

	// Store viewport configuration data
	DstViewport.SetViewportConfigurationData(&ConfigurationViewport);

	FDisplayClusterViewportConfigurationHelpers::UpdateBaseViewportSetting(DstViewport, ConfigurationViewport);
	DstViewport.UpdateConfiguration_ProjectionPolicy(&ConfigurationViewport.ProjectionPolicy);

	return true;
}

void FDisplayClusterViewportConfiguration_Viewport::SetViewportWarpPolicy(const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& InViewport, IDisplayClusterWarpPolicy* InWarpPolicy)
{
	// ignore internal viewports
	if (InViewport.IsValid() && !InViewport->IsInternalViewport())
	{
		if (InViewport->GetProjectionPolicy().IsValid())
		{
			InViewport->GetProjectionPolicy()->SetWarpPolicy(InWarpPolicy);
		}
	}
}
