// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class UDisplayClusterConfigurationViewport;
class UDisplayClusterCameraComponent;

class FDisplayClusterViewportConfiguration;
class FDisplayClusterViewport;
class IDisplayClusterWarpPolicy;

struct FDisplayClusterRenderFrameSettings;

/**
 * Container with information related to viewport: configuration and references to some class instances.
 */
struct FDisplayClusterViewportConfiguration_Viewport
{
	FDisplayClusterViewportConfiguration_Viewport(const FString& InClusterNodeId, const FString& InViewportId, const UDisplayClusterConfigurationViewport& InConfigurationViewport)
		: ClusterNodeId(InClusterNodeId)
		, ViewportId(InViewportId)
		, ConfigurationViewport(InConfigurationViewport)
	{ }

	/**
	 * Create or update DC viewport instance from configuration
	 */
	void CreateOrUpdateViewportInstance(FDisplayClusterViewportConfiguration& InOutConfiguration);

	/** Post update viewport. */
	void PostUpdateViewportInstance(FDisplayClusterViewportConfiguration& InOutConfiguration);

	/**
	 * Update exists viewport runtime settings from configuration
	 */
	static bool UpdateViewportConfiguration(FDisplayClusterViewport& DstViewport, const UDisplayClusterConfigurationViewport& InConfigurationViewport);

	/**
	 * Assign warp policy instance to given viewport
	 */
	static void SetViewportWarpPolicy(const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& InViewport, IDisplayClusterWarpPolicy* InWarpPolicy);

public:
	// Cluster name, who own this viewport
	const FString& ClusterNodeId;

	// Viewport unique name
	const FString& ViewportId;

	// Reference to the configuration used to create or update the viewport instance
	const UDisplayClusterConfigurationViewport& ConfigurationViewport;

	// The viewport instance
	TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe> Viewport;

	// Pointer to the view point component, used by this viewport instance.
	UDisplayClusterCameraComponent* ViewPointCameraComponent = nullptr;
};
