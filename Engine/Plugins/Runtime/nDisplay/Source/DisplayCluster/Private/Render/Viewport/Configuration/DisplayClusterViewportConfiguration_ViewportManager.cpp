// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterViewportConfiguration_ViewportManager.h"

#include "Render/Projection/IDisplayClusterProjectionPolicy.h"

#include "Render/Viewport/DisplayClusterViewport.h"
#include "Render/Viewport/DisplayClusterViewportManager.h"
#include "Render/Viewport/RenderFrame/DisplayClusterRenderFrameSettings.h"

#include "DisplayClusterViewportConfigurationHelpers.h"
#include "DisplayClusterConfigurationTypes.h"

#include "DisplayClusterRootActor.h"

#include "Components/DisplayClusterCameraComponent.h"

#include "Misc/DisplayClusterLog.h"

///////////////////////////////////////////////////////////////////
// FDisplayClusterViewportConfiguration_ViewportManager
///////////////////////////////////////////////////////////////////
void FDisplayClusterViewportConfiguration_ViewportManager::UpdateClusterNodeViewports(const FString& InClusterNodeId)
{
	// Initialize variable EntireClusterViewports from configuration
	ImplInitializeEntireClusterViewportsList();

	if (InClusterNodeId.IsEmpty())
	{
		// If the cluster node name is empty, we use the viewports of the entire cluster 
		CurrentFrameViewports = EntireClusterViewports;
	}
	else
	{
		// Otherwise, we only use the viewports for one cluster node
		CurrentFrameViewports.Reset();
		for (const FDisplayClusterViewportConfiguration_Viewport& ViewportData : EntireClusterViewports)
		{
			if (ViewportData.ClusterNodeId == InClusterNodeId)
			{
				CurrentFrameViewports.Add(ViewportData);
			}
		}
	}

	// Create or update viewports from CurrentFrameViewports
	ImplUpdateViewports();

	// Updates warp policies only if the cluster node name is not empty.
	if (!InClusterNodeId.IsEmpty())
	{
		// when the cluster node name is empty, it means that the DCRA preview right now is initializing for the entire cluster.
		ImplUpdateViewportsWarpPolicy();
	}
}

void FDisplayClusterViewportConfiguration_ViewportManager::UpdateCustomViewports(const TArray<FString>& InViewportNames)
{
	// Initialize variable EntireClusterViewports from configuration
	ImplInitializeEntireClusterViewportsList();

	// Get the list of viewport instance data for the specified InViewportNames
	for (const FDisplayClusterViewportConfiguration_Viewport& ViewportData : EntireClusterViewports)
	{
		if (InViewportNames.Contains(ViewportData.ViewportId))
		{
			CurrentFrameViewports.Add(ViewportData);
		}
	}

	// Create or update viewports from CurrentFrameViewports
	ImplUpdateViewports();
}

void FDisplayClusterViewportConfiguration_ViewportManager::ImplUpdateViewports()
{
	FDisplayClusterViewportManager* ViewportManager = Configuration.GetViewportManagerImpl();
	if (!ViewportManager)
	{
		return;
	}

	// Delete an existing viewport when it is removed from the configuration or its configuration state is set as disabled.
	const TArray<TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>> EntireClusterDCViewports = ViewportManager->ImplGetEntireClusterViewports();
	for (const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& Viewport : EntireClusterDCViewports)
	{
		// ignore internal viewports
		if (Viewport.IsValid() && !Viewport->IsInternalViewport())
		{
			// Delete an existing viewport when it is removed from the configuration
			if (!ImplFindViewportInEntireCluster(Viewport->GetId()))
			{
				// we can safely remove viewports in a loop, because we use our own local array
				ViewportManager->ImplDeleteViewport(Viewport);
			}
		}
	}

	// Create or update viewports for current rendering frame
	for (FDisplayClusterViewportConfiguration_Viewport& ViewportData : CurrentFrameViewports)
	{
		ViewportData.CreateOrUpdateViewportInstance(Configuration);
	}

	// Post update viewports for current rendering frame
	for (FDisplayClusterViewportConfiguration_Viewport& ViewportData : CurrentFrameViewports)
	{
		ViewportData.PostUpdateViewportInstance(Configuration);
	}

	// Clear viewports warp policies for entire cluster
	for (const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& Viewport : ViewportManager->ImplGetEntireClusterViewports())
	{
		FDisplayClusterViewportConfiguration_Viewport::SetViewportWarpPolicy(Viewport, nullptr);
	}
}

void FDisplayClusterViewportConfiguration_ViewportManager::ImplUpdateViewportsWarpPolicy()
{
	FDisplayClusterViewportManager* ViewportManager = Configuration.GetViewportManagerImpl();
	if (!ViewportManager)
	{
		return;
	}

	// Collect DC origin components used in current frame
	TArray<UDisplayClusterCameraComponent*> CurrentFrameViewPointCameraComponent;
	for (const FDisplayClusterViewportConfiguration_Viewport& ViewportData : CurrentFrameViewports)
	{
		if (ViewportData.ViewPointCameraComponent)
		{
			CurrentFrameViewPointCameraComponent.AddUnique(ViewportData.ViewPointCameraComponent);
		}
	}

	// Create reffered viewports from other cluster nodes and update warp policy on it
	for (UDisplayClusterCameraComponent* ViewPointComponent : CurrentFrameViewPointCameraComponent)
	{
		// The viewpoint camera component expects all viewports (from the entire cluster) that refer to them to be exists (created)  and updated.
		// this is for cluster rendering
		if (ViewPointComponent->ShouldUseEntireClusterViewports(ViewportManager))
		{
			if (IDisplayClusterWarpPolicy* WarpPolicy = ViewPointComponent->GetWarpPolicy(ViewportManager))
			{
				// Find the viewports that use this ViewPointComponent in the entire cluster.
				TArray<FDisplayClusterViewportConfiguration_Viewport> ViewPointViewports;
				const FString ViewPointComponentId = ViewPointComponent->GetName();
				for (const FDisplayClusterViewportConfiguration_Viewport& ViewportData : EntireClusterViewports)
				{
					if (ViewportData.ConfigurationViewport.Camera == ViewPointComponentId)
					{
						ViewPointViewports.Add(ViewportData);
					}
				}

				// Initialize the warp policy for all found viewports:
				for (FDisplayClusterViewportConfiguration_Viewport& ViewportData : ViewPointViewports)
				{
					// If the viewport already exists in the current rendering frame, we simply assign a warp policy
					if (FDisplayClusterViewportConfiguration_Viewport const* CurrentFrameViewportData = ImplFindCurrentFrameViewports(ViewportData.ViewportId))
					{
						// Set the warp projection for the viewport from the current cluster node
						FDisplayClusterViewportConfiguration_Viewport::SetViewportWarpPolicy(CurrentFrameViewportData->Viewport, WarpPolicy);
					}
					else
					// when the viewport is on another cluster node, we must create it, because the warp policy requires viewports from the whole cluster.
					{
						// Create or update this viewport from another cluster node
						ViewportData.CreateOrUpdateViewportInstance(Configuration);

						// Set the warp projection for the viewport from another cluster node
						FDisplayClusterViewportConfiguration_Viewport::SetViewportWarpPolicy(ViewportData.Viewport, WarpPolicy);
					}
				}
			}
		}
	}
}

void FDisplayClusterViewportConfiguration_ViewportManager::ImplInitializeEntireClusterViewportsList()
{
	const UDisplayClusterConfigurationData* ConfigurationData = Configuration.GetConfigurationData();;
	if (ConfigurationData && ConfigurationData->Cluster)
	{
		// Update and Create new viewports
		for (const TPair<FString, TObjectPtr<UDisplayClusterConfigurationClusterNode>>& ClusterNodeConfigurationIt : ConfigurationData->Cluster->Nodes)
		{
			if (const UDisplayClusterConfigurationClusterNode* ClusterNodeConfiguration = ClusterNodeConfigurationIt.Value)
			{
				for (const TPair<FString, TObjectPtr<UDisplayClusterConfigurationViewport>>& ViewportIt : ClusterNodeConfiguration->Viewports)
				{
					if (const UDisplayClusterConfigurationViewport* ConfigurationViewport = ViewportIt.Value)
					{
						const FString& ClusterNodeId = ClusterNodeConfigurationIt.Key;
						const FString& ViewportId = ViewportIt.Key;
						if (!ClusterNodeId.IsEmpty() && !ViewportId.IsEmpty() && ConfigurationViewport->IsViewportEnabled())
						{
							EntireClusterViewports.Add(FDisplayClusterViewportConfiguration_Viewport(ClusterNodeId, ViewportId, *ConfigurationViewport));
						}
					}
				}
			}
		}
	}
}

FDisplayClusterViewportConfiguration_Viewport const* FDisplayClusterViewportConfiguration_ViewportManager::ImplFindViewportInEntireCluster(const FString& InViewportId) const
{
	return EntireClusterViewports.FindByPredicate([InViewportId](const FDisplayClusterViewportConfiguration_Viewport& ViewportItem)
		{
			return ViewportItem.ViewportId == InViewportId;
		});
}

FDisplayClusterViewportConfiguration_Viewport const* FDisplayClusterViewportConfiguration_ViewportManager::ImplFindCurrentFrameViewports(const FString& InViewportId) const
{
	return CurrentFrameViewports.FindByPredicate([InViewportId](const FDisplayClusterViewportConfiguration_Viewport& ViewportItem)
		{
			return ViewportItem.ViewportId == InViewportId;
		});
}
