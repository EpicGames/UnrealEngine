// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Render/Viewport/Configuration/DisplayClusterViewportConfiguration_Viewport.h"
#include "Render/Viewport/Configuration/DisplayClusterViewportConfiguration.h"

/**
 * A helper class that creates/updates/deletes viewports for the current rendering frame.
 */
struct FDisplayClusterViewportConfiguration_ViewportManager
{
public:
	FDisplayClusterViewportConfiguration_ViewportManager(FDisplayClusterViewportConfiguration& InConfiguration)
		: Configuration(InConfiguration)
	{ }

public:
	/** Updates the list of viewports for the specified cluster node name, using ConfigurationData.
	 * 
	 * @param ClusterNodeId - cluster node name (special names are supported)
	 */
	void UpdateClusterNodeViewports(const FString& ClusterNodeId);

	/** Updates only the viewports for the specified list with the names of the viewports using ConfigurationData.
	 * 
	 * @param InViewportNames - a list of viewport names to create or update.
	 */
	void UpdateCustomViewports(const TArray<FString>& InViewportNames);

private:
	/** Update viewports instances in ViewportManager. */
	void ImplUpdateViewports();

	/** Update all viewports warp policies. */
	void ImplUpdateViewportsWarpPolicy();

	/** Initialize variable EntireClusterViewports from configuration. */
	void ImplInitializeEntireClusterViewportsList();

	/** Find the data of the viewport instance by name in the entire cluster. */
	FDisplayClusterViewportConfiguration_Viewport const* ImplFindViewportInEntireCluster(const FString& InViewportId) const;

	/** Find the viewport instance data by name in the current rendering frame. */
	FDisplayClusterViewportConfiguration_Viewport const* ImplFindCurrentFrameViewports(const FString& InViewportId) const;

private:
	// The configuration api
	FDisplayClusterViewportConfiguration& Configuration;

	// The entire cluster viewports
	TArray<FDisplayClusterViewportConfiguration_Viewport> EntireClusterViewports;

	// the viewports of the current rendering frame (determined from the cluster node name or from the user's list of viewport names)
	TArray<FDisplayClusterViewportConfiguration_Viewport> CurrentFrameViewports;
};
