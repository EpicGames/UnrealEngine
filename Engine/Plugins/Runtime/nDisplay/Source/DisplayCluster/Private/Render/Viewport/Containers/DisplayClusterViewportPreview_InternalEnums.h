// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * Special flags for nDisplay preview log
 */

enum class EDisplayClusterViewportPreviewShowLogMsgOnce : uint8
{
	None = 0,

	// When CalculateStereoViewOffset() function has an error
	CalculateViewIsFailed = 1 << 0,

	StereoProjectionMatrixIsInvalid = 1 << 1,

};
ENUM_CLASS_FLAGS(EDisplayClusterViewportPreviewShowLogMsgOnce);
