// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Render/DisplayDevice/IDisplayClusterDisplayDeviceProxy.h"
#include "Render/Viewport/DisplayClusterViewportResources.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_Context.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_RenderSettings.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_RenderSettingsICVFX.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_PostRenderSettings.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_OverscanRuntimeSettings.h"
#include "Render/Viewport/Containers/DisplayClusterViewportFrameData.h"

class IDisplayClusterProjectionPolicy;
class FDisplayClusterViewportResource;
class FDisplayClusterViewport;
class FDisplayClusterViewportProxy;
class FDisplayClusterViewport_Context;
class IDisplayClusterRender_MeshComponent;
class FDisplayClusterViewport_OpenColorIO;
class FSceneViewStateReference;

/**
 *  Container for data exchange game->render threads
 */
class FDisplayClusterViewportProxyData
{
public:
	FDisplayClusterViewportProxyData(const TSharedRef<FDisplayClusterViewportProxy, ESPMode::ThreadSafe>& InDestViewportProxy)
		: DestViewportProxy(InDestViewportProxy)
	{ }
	~FDisplayClusterViewportProxyData() = default;

public:
	// Dest viewport proxy
	const TSharedRef<FDisplayClusterViewportProxy, ESPMode::ThreadSafe> DestViewportProxy;

	// This data is stored and copied from the game thread to the rendering thread:
public:
	TSharedPtr<IDisplayClusterRender_MeshComponent, ESPMode::ThreadSafe> RemapMesh;

	FDisplayClusterViewport_OverscanRuntimeSettings OverscanRuntimeSettings;

	// Viewport render params
	FDisplayClusterViewport_RenderSettings       RenderSettings;
	FDisplayClusterViewport_RenderSettingsICVFX  RenderSettingsICVFX;
	FDisplayClusterViewport_PostRenderSettings   PostRenderSettings;

	// OpenColorIO
	TSharedPtr<FDisplayClusterViewport_OpenColorIO, ESPMode::ThreadSafe> OpenColorIO;

	// Display Device
	TSharedPtr<IDisplayClusterDisplayDeviceProxy, ESPMode::ThreadSafe> DisplayDeviceProxy;

	// Projection policy instance that serves this viewport
	TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe> ProjectionPolicy;

	/** Frame data snapshot for the current frame (contexts, cached view info). */
	FDisplayClusterViewportFrameData CurrentFrameData;

	// Unified repository of viewport resources
	FDisplayClusterViewportResources Resources;

	// Used ViewStates
	TArray<TSharedPtr<FSceneViewStateReference, ESPMode::ThreadSafe>> ViewStates;
};
