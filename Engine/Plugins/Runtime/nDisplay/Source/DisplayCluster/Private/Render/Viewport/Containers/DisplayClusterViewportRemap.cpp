// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Viewport/Containers/DisplayClusterViewportRemap.h"
#include "DisplayClusterConfigurationTypes_ViewportRemap.h"

#include "Misc/DisplayClusterGlobals.h"
#include "Render/IDisplayClusterRenderManager.h"

#include "Render/Viewport/DisplayClusterViewport.h"
#include "Render/Viewport/DisplayClusterViewportManager.h"
#include "Render/Viewport/RenderFrame/DisplayClusterRenderFrameSettings.h"
#include "Render/Containers/DisplayClusterRender_MeshGeometry.h"

bool FDisplayClusterViewportRemap::UpdateViewportRemap(const FDisplayClusterViewport& InViewport, const FDisplayClusterConfigurationViewport_Remap& InRemapConfiguration)
{
	if (InRemapConfiguration.bEnable)
	{
		TArray<FDisplayClusterViewportRemapData> NewRemapRegions;

		if (InRemapConfiguration.BaseRemap.IsValid())
		{
			NewRemapRegions.Add(CreateRemapData(InRemapConfiguration.BaseRemap));
		}

		for (const auto& RemapRegion : InRemapConfiguration.RemapRegions)
		{
			if (RemapRegion.IsValid())
			{
				NewRemapRegions.Add(CreateRemapData(RemapRegion));
			}
		}

		if (RemapRegions.Num() != NewRemapRegions.Num())
		{
			bIsNeedUpdateRemapMesh = true;
		}
		else
		{
			for (int32 RegionIt = 0; RegionIt < RemapRegions.Num(); RegionIt++)
			{
				if (NewRemapRegions[RegionIt] != RemapRegions[RegionIt])
				{
					bIsNeedUpdateRemapMesh = true;
					break;
				}
			}
		}

		if (bIsNeedUpdateRemapMesh)
		{
			RemapRegions.Empty();
			RemapRegions.Append(NewRemapRegions);

			return true;
		}

		return false;
	}

	Empty();

	return true;
}

void FDisplayClusterViewportRemap::Update(const FDisplayClusterViewport& InViewport, const FIntPoint& InRenderFrameSize)
{
	// Preview in editor not support this feature
	if (InViewport.Configuration->IsPreviewRendering())
	{
		return;
	}

	const FDisplayClusterViewport_RenderSettings& RenderSettings = InViewport.GetRenderSettings();

	if (ViewportRegion != RenderSettings.Rect || RenderFrameSize != InRenderFrameSize)
	{
		bIsNeedUpdateRemapMesh = true;
	}

	if (bIsNeedUpdateRemapMesh)
	{
		// Save new viewport remap settings
		ViewportRegion  = RenderSettings.Rect;
		RenderFrameSize = InRenderFrameSize;

		ImplUpdateRemapMesh(InViewport);
	}
}

void FDisplayClusterViewportRemap::Empty()
{
	RemapRegions.Empty();
	RemapMesh.Reset();
}

TSharedPtr<IDisplayClusterRender_MeshComponent, ESPMode::ThreadSafe> FDisplayClusterViewportRemap::GetRemapMesh() const
{
	check(IsInGameThread());

	return RemapMesh;
}

void FDisplayClusterViewportRemap::ImplUpdateRemapMesh(const FDisplayClusterViewport& InViewport)
{
	check(IsInGameThread());

	bIsNeedUpdateRemapMesh = false;

	// Skip build geometry
	if(!IsUsed())
	{
		RemapMesh.Reset();
		return;
	}

	// Build new remap geometry for current data
	FDisplayClusterRender_MeshGeometry RemapGeometry;
	if (ImplBuildGeometry(RemapGeometry))
	{
		if (!RemapMesh.IsValid())
		{
			// Create new viewport remap mesh component
			RemapMesh = IDisplayCluster::Get().GetRenderMgr()->CreateMeshComponent();
		}

		RemapMesh->AssignMeshGeometry(&RemapGeometry);
	}
	else
	{
		RemapMesh.Reset();
	}
}

bool FDisplayClusterViewportRemap::ImplBuildGeometry(FDisplayClusterRender_MeshGeometry& OutGeometry) const
{
	const FIntPoint ViewportSize = ViewportRegion.Size();

	for (const FDisplayClusterViewportRemapData& RegionIt : RemapRegions)
	{
		TArray<FVector>   OutVertices;
		TArray<FVector2D> OutUV;

		// Build rect
		//  0----3
		//  |    |
		//  1----2

		// Build Dest rect verts
		OutVertices.Add(FVector(RegionIt.DstRect.Min.X, RegionIt.DstRect.Min.Y, 0));
		OutVertices.Add(FVector(RegionIt.DstRect.Min.X, RegionIt.DstRect.Max.Y, 0));
		OutVertices.Add(FVector(RegionIt.DstRect.Max.X, RegionIt.DstRect.Max.Y, 0));
		OutVertices.Add(FVector(RegionIt.DstRect.Max.X, RegionIt.DstRect.Min.Y, 0));

		const FIntPoint DstRectSize = RegionIt.DstRect.Size();
		const FVector RectCenter = FVector(RegionIt.DstRect.Min) + 0.5f * FVector(DstRectSize);

		float SinAngle, CosAngle;
		FMath::SinCos(&SinAngle, &CosAngle, FMath::DegreesToRadians(RegionIt.DstAngle));

		FVector RotatedSize = FVector::ZeroVector;
		RotatedSize.X = DstRectSize.X * FMath::Abs(CosAngle) + DstRectSize.Y * FMath::Abs(SinAngle);
		RotatedSize.Y = DstRectSize.X * FMath::Abs(SinAngle) + DstRectSize.Y * FMath::Abs(CosAngle);
		FVector Offset = 0.5f * RotatedSize - 0.5f * FVector(DstRectSize);

		for (FVector& PtsIt : OutVertices)
		{
			// Rotate 2D vector from center around axis Z
			FVector LocalPts = PtsIt - RectCenter;
			PtsIt = LocalPts.RotateAngleAxis(RegionIt.DstAngle, FVector(0,0,1)) + RectCenter + Offset;

			// Transform Dest rect verts to screen space:
			PtsIt.X /= RenderFrameSize.X;
			PtsIt.Y /= RenderFrameSize.Y;
			PtsIt.Y = 1 - PtsIt.Y;
		}

		// Source region from viewport always inside viewport
		FIntRect SrcRect = RegionIt.SrcRect;
		SrcRect.Min.X = FMath::Max(SrcRect.Min.X, 0);
		SrcRect.Min.Y = FMath::Max(SrcRect.Min.Y, 0);
		SrcRect.Max.X = FMath::Min(SrcRect.Max.X, ViewportSize.X);
		SrcRect.Max.Y = FMath::Min(SrcRect.Max.Y, ViewportSize.Y);

		// Build UVs:
		float U0 = float(((!RegionIt.bSrcFlipH) ? SrcRect.Min.X : SrcRect.Max.X) + ViewportRegion.Min.X) / float(RenderFrameSize.X);
		float U1 = float(((!RegionIt.bSrcFlipH) ? SrcRect.Max.X : SrcRect.Min.X) + ViewportRegion.Min.X) / float(RenderFrameSize.X);

		float V0 = float(((RegionIt.bSrcFlipV) ? SrcRect.Min.Y : SrcRect.Max.Y) + ViewportRegion.Min.Y) / float(RenderFrameSize.Y);
		float V1 = float(((RegionIt.bSrcFlipV) ? SrcRect.Max.Y : SrcRect.Min.Y) + ViewportRegion.Min.Y) / float(RenderFrameSize.Y);

		OutUV.Add(FVector2D(U0, V1));
		OutUV.Add(FVector2D(U0, V0));
		OutUV.Add(FVector2D(U1, V0));
		OutUV.Add(FVector2D(U1, V1));

		// Add to geometry
		int32 Index = OutGeometry.Vertices.Num();

		OutGeometry.Vertices.Append(OutVertices);
		OutGeometry.UV.Append(OutUV);

		OutGeometry.Triangles.Add(Index + 0);
		OutGeometry.Triangles.Add(Index + 1);
		OutGeometry.Triangles.Add(Index + 2);

		OutGeometry.Triangles.Add(Index + 3);
		OutGeometry.Triangles.Add(Index + 0);
		OutGeometry.Triangles.Add(Index + 2);
	}

	return OutGeometry.Triangles.Num() > 0;
}

FDisplayClusterViewportRemapData FDisplayClusterViewportRemap::CreateRemapData(const FDisplayClusterConfigurationViewport_RemapData& CfgRemapData) const
{
	FDisplayClusterViewportRemapData NewRegion;

	// If the source rectangle is not defined, use the entire viewport for the remap
	NewRegion.SrcRect = CfgRemapData.ViewportRegion.ToRect();
	if (NewRegion.SrcRect.IsEmpty())
	{
		NewRegion.SrcRect = FIntRect(0, 0, ViewportRegion.Width(), ViewportRegion.Height());
	}

	NewRegion.bSrcFlipH = CfgRemapData.bFlipH;
	NewRegion.bSrcFlipV = CfgRemapData.bFlipV;

	// If the destination rectangle is not defined, use a destination rect that matches where the source rectangle is located on the viewport
	NewRegion.DstRect = CfgRemapData.OutputRegion.ToRect();
	if (NewRegion.DstRect.IsEmpty())
	{
		NewRegion.DstRect = NewRegion.SrcRect + ViewportRegion.Min;
	}

	NewRegion.DstAngle = CfgRemapData.Angle;

	return NewRegion;
}