// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Render/Containers/IDisplayClusterRender_MeshComponent.h"
#include "Render/Viewport/Containers/DisplayClusterViewportRemapData.h"

class FDisplayClusterViewport;
struct FDisplayClusterConfigurationViewport_Remap;
struct FDisplayClusterConfigurationViewport_RemapData;

/** Helper class that computes the render geometry needed to display a viewport's output remapping */
class FDisplayClusterViewportRemap
{
public:
	/**
	 * Updates the configuration to match the specified viewport's output remapping configuration
	 * @param InViewport - The viewport whose remapping configuration is being updated
	 * @param InRemapConfiguration - The remapping configuration to update to
	 * @returns true if the render geometry needs updating after the configuration has been updated; otherwise, false
	 */
	bool UpdateViewportRemap(const FDisplayClusterViewport& InViewport, const FDisplayClusterConfigurationViewport_Remap& InRemapConfiguration);

	/** Gets a pointer to the mesh geometry to use for rendering the remapped regions */
	TSharedPtr<IDisplayClusterRender_MeshComponent, ESPMode::ThreadSafe> GetRemapMesh() const;

	/** Gets whether there are any remapped regions to render */
	bool IsUsed() const { return RemapRegions.Num() > 0; }

	/**
	 * Updates the render geometry used to render the remapped regions for the specified viewport
	 * @param InViewport - The viewport whose remapping configuration is being updated
	 * @param InRenderFrameSize - The size of the render frame the viewports are being rendered to
	 */
	void Update(const FDisplayClusterViewport& InViewport, const FIntPoint& InRenderFrameSize);

protected:
	/** Empies all configured remapping regions and resets the mesh geometry */
	void Empty();

private:
	/** Updates the mesh used to render the remapped regions of the specified viewport  */
	void ImplUpdateRemapMesh(const FDisplayClusterViewport& InViewport);

	/** Builds the mesh geometry needed to display the remapped regions */
	bool ImplBuildGeometry(FDisplayClusterRender_MeshGeometry& OutGeometry) const;

	/** Creates a data structure to store a remapped region from the configuration data structure */
	FDisplayClusterViewportRemapData CreateRemapData(const FDisplayClusterConfigurationViewport_RemapData& CfgRemapData) const;

private:
	/** The mesh used to render the remapped regions */
	TSharedPtr<IDisplayClusterRender_MeshComponent, ESPMode::ThreadSafe> RemapMesh;
	
	/** A list of all the remapped regions to render */
	TArray<FDisplayClusterViewportRemapData> RemapRegions;

	/** The native size of the viewport whose remapped regions are being managed */
	FIntRect  ViewportRegion;

	/** The size of the render frame the viewports are being rendered to */
	FIntPoint RenderFrameSize;

	/** Indicates if the mesh needs to be updated in the next update pass */
	bool bIsNeedUpdateRemapMesh = false;
};
