// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

struct FDisplayClusterViewportRemapData
{
	// The viewport source region.
	FIntRect SrcRect;

	// flip source region horizontal
	bool bSrcFlipH = false;

	// flip source region vertical
	bool bSrcFlipV = false;

	// The backbuffer destination region
	FIntRect DstRect;

	// Rotate angle in degree, Rotate origin is center of DstRect
	float DstAngle = 0;

	FORCEINLINE bool operator==(const FDisplayClusterViewportRemapData& In) const
	{
		return SrcRect == In.SrcRect
			&& DstRect == In.DstRect
			&& DstAngle == In.DstAngle
			&& bSrcFlipH == In.bSrcFlipH
			&& bSrcFlipV == In.bSrcFlipV;
	}

	FORCEINLINE bool operator!=(const FDisplayClusterViewportRemapData& In) const
	{
		return (operator==(In)) == false;
	}
};
