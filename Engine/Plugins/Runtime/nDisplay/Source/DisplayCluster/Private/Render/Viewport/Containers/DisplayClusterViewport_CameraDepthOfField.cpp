// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Viewport/Containers/DisplayClusterViewport_CameraDepthOfField.h"

#include "Engine/Texture2D.h"
#include "SceneView.h"
#include "TextureResource.h"

void FDisplayClusterViewport_CameraDepthOfField::SetupSceneView(FSceneView& InOutView) const
{
	InOutView.bEnableDynamicCocOffset = bEnableDepthOfFieldCompensation;
	InOutView.InFocusDistance = DistanceToWall + DistanceToWallOffset;

	if (CompensationLUT.IsValid())
	{
		InOutView.DynamicCocOffsetLUT = CompensationLUT->GetResource() ? CompensationLUT->GetResource()->GetTextureRHI() : nullptr;
	}
}