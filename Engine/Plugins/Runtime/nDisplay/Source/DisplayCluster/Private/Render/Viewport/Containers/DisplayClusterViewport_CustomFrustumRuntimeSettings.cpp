// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Viewport/Containers/DisplayClusterViewport_CustomFrustumRuntimeSettings.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_CustomFrustumSettings.h"
#include "Render/Viewport/DisplayClusterViewportHelpers.h"

#include "HAL/IConsoleManager.h"

int32 GDisplayClusterRenderCustomFrustumEnable = 1;
static FAutoConsoleVariableRef CVarDisplayClusterRenderCustomFrustumEnable(
	TEXT("nDisplay.render.custom_frustum.enable"),
	GDisplayClusterRenderCustomFrustumEnable,
	TEXT("Enable custom frustum feature.\n")
	TEXT(" 0 - to disable.\n"),
	ECVF_Default
);

int32 GDisplayClusterRenderCustomFrustumMaxValue = 50;
static FAutoConsoleVariableRef CVarDisplayClusterRenderCustomFrustumMaxValue(
	TEXT("nDisplay.render.custom_frustum.max_percent"),
	GDisplayClusterRenderCustomFrustumMaxValue,
	TEXT("Max percent for custom frustum (default 50).\n"),
	ECVF_Default
);

int32 GDisplayClusterRenderCustomFrustumMinSizeValue = 1;
static FAutoConsoleVariableRef CVarDisplayClusterRenderCustomFrustumMinSizeValue(
	TEXT("nDisplay.render.custom_frustum.min_percent_size"),
	GDisplayClusterRenderCustomFrustumMinSizeValue,
	TEXT("Min size in percent for custom frustum (default 1).\n"),
	ECVF_Default
);

namespace UE::DisplayCluster::Viewport::CustomFrustumHelpers
{
	/** Clamp percent for a custom frustum settings. */
	static inline double ClampPercent(double InValue)
	{
		const double MaxCustomFrustumValue = double(GDisplayClusterRenderCustomFrustumMaxValue) / 100;

		return FMath::Clamp(InValue, -MaxCustomFrustumValue, MaxCustomFrustumValue);
	}

	/** Adjusts the overscan values so that the final size is larger than the minimum size.
	* 
	* @param InOutFromValue - (in, out) FromValue, where 1 means 100%
	* @param InOutToValue   - (in, out) ToValue, where 1 means 100%
	*/
	static inline void AdjustOverscanValuesToEnforceMinimumSize(double& InOutFromValue, double& InOutToValue)
	{
		const double MinSize = FMath::Max(GDisplayClusterRenderCustomFrustumMinSizeValue * 0.01, 0);

		// Expected size in percentage = 100 % + From + To 
		// 1 means 100 % because all percentages were multiplied by 0.01.
		const double ExpectedSize = InOutFromValue + InOutToValue + 1;

		// If negative values are used, the size may decrease to zero or less.
		// The MinSize value is used as a limit.
		if (ExpectedSize < MinSize)
		{
			// Gets the negative values only
			const double NegativeFromValue = -FMath::Min(InOutFromValue, 0);
			const double NegativeToValue   = -FMath::Min(InOutToValue, 0);

			// When using negative values, check that the result size is larger than the minimum.
			const double NegativeRange = NegativeFromValue + NegativeToValue;
			if (NegativeRange > 0)
			{
				// Adjust From and To values to the minimum size.
				const double AdjustMult = (MinSize - ExpectedSize) / NegativeRange;

				InOutFromValue += NegativeFromValue * AdjustMult;
				InOutToValue   += NegativeToValue * AdjustMult;
			}
		}
	}
};

///////////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterViewport_CustomFrustumRuntimeSettings
///////////////////////////////////////////////////////////////////////////////////////////
bool FDisplayClusterViewport_CustomFrustumRuntimeSettings::UpdateProjectionAngles(
	const FDisplayClusterViewport_CustomFrustumRuntimeSettings& InRuntimeSettings,
	double& InOutLeft,
	double& InOutRight,
	double& InOutTop,
	double& InOutBottom)
{
	if (InRuntimeSettings.bIsEnabled)
	{
		const FVector2D SizeFOV(InOutRight - InOutLeft, InOutTop - InOutBottom);
		const FMarginSet FovMargins = InRuntimeSettings.CustomFrustumPercent * SizeFOV;

		InOutLeft   -= FovMargins.Left;
		InOutRight  += FovMargins.Right;
		InOutBottom -= FovMargins.Bottom;
		InOutTop    += FovMargins.Top;

		return true;
	}

	return false;
}

void FDisplayClusterViewport_CustomFrustumRuntimeSettings::UpdateCustomFrustumSettings(
	const FString& InViewportId,
	const FDisplayClusterViewport_CustomFrustumSettings& InCustomFrustumSettings,
	FDisplayClusterViewport_CustomFrustumRuntimeSettings& InOutRuntimeSettings,
	FIntRect& InOutRenderTargetRect,
	const TCHAR* InCustomResourceName)
{
	using namespace UE::DisplayCluster::Viewport;

	const FIntPoint Size = InOutRenderTargetRect.Size();

	// Disable viewport CustomFrustum feature
	if (!GDisplayClusterRenderCustomFrustumEnable || !InCustomFrustumSettings.bEnabled)
	{
		return;
	}

	const double MaxCustomFrustumValue = double(GDisplayClusterRenderCustomFrustumMaxValue) / 100;
	switch (InCustomFrustumSettings.Unit)
	{
	case EDisplayClusterViewport_FrustumUnit::Percent:
	{
		InOutRuntimeSettings.bIsEnabled = true;
		InOutRuntimeSettings.CustomFrustumPercent.AssignMargins(InCustomFrustumSettings.GetClamped(-MaxCustomFrustumValue, MaxCustomFrustumValue));
		break;
	}

	case EDisplayClusterViewport_FrustumUnit::Pixels:
	{
		InOutRuntimeSettings.bIsEnabled = true;
		InOutRuntimeSettings.CustomFrustumPercent.AssignMargins((InCustomFrustumSettings / Size).GetClamped(-MaxCustomFrustumValue, MaxCustomFrustumValue));
		break;
	}

	default:
		return;
	}

	// Clamp minimum size of frustum
	CustomFrustumHelpers::AdjustOverscanValuesToEnforceMinimumSize(InOutRuntimeSettings.CustomFrustumPercent.Left,   InOutRuntimeSettings.CustomFrustumPercent.Right);
	CustomFrustumHelpers::AdjustOverscanValuesToEnforceMinimumSize(InOutRuntimeSettings.CustomFrustumPercent.Bottom, InOutRuntimeSettings.CustomFrustumPercent.Top);

	// Calc pixels from percent
		InOutRuntimeSettings.CustomFrustumPixels.AssignMargins((InOutRuntimeSettings.CustomFrustumPercent * Size).GetRoundToInt());
	
	// Quantize the percentage to exactly fit the number of pixels.
	InOutRuntimeSettings.CustomFrustumPercent.AssignMargins(FMarginSet(InOutRuntimeSettings.CustomFrustumPixels) / Size);

	// Update RTT size for CustomFrustum when we need to scale target resolution
	if (InCustomFrustumSettings.bAdaptResolution)
	{
		const FIntRect NewCustomFrustumRect(
			FIntPoint(0, 0),
			Size + InOutRuntimeSettings.CustomFrustumPixels.Size()
		);

		// Return new custom frustum rect
		InOutRenderTargetRect = FDisplayClusterViewportHelpers::GetValidViewportRect(
			NewCustomFrustumRect,
			InViewportId,
			InCustomResourceName ? InCustomResourceName : TEXT("CustomFrustum"));
	}
}
