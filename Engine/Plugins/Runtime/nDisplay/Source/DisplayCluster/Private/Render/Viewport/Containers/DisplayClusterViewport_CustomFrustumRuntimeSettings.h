// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_CustomFrustumSettings.h"

/**
* Runtime custom frustum settings
*/
struct FDisplayClusterViewport_CustomFrustumRuntimeSettings
{
	/** Update custom frustum settings
	*
	* @param InViewportId            - owner viewport name
	* @param InCustomFrustumSettings - CustomFrustum settings
	* @param InOutRuntimeSettings    - CustomFrustum runtime settings.
	* @param InOutRenderTargetRect   - Viewport rect, changeable during CustomFrustum
	* @param InCustomResourceName    - (opt) The unique resource name used for log messages.
	*                                        Use a new custom resource name for each new location from which this function is called.
	*/
	static void UpdateCustomFrustumSettings(
		const FString& InViewportId,
		const FDisplayClusterViewport_CustomFrustumSettings& InCustomFrustumSettings,
		FDisplayClusterViewport_CustomFrustumRuntimeSettings& InOutRuntimeSettings,
		FIntRect& InOutRenderTargetRect,
		const TCHAR* InCustomResourceName = nullptr);
	
	/** Update projection angles by custom frustum
	*
	* @param InCustomFrustumSettings   - the CustomFrustum runtime settings.
	* @param InOutLeft                 - the value of the left projection plane that you want to change
	* @param InOutRight                - the value of the right projection plane that you want to change
	* @param InOutTop                  - the value of the top projection plane that you want to change
	* @param InOutBottom               - the value of the bottom projection plane that you want to change
	*/
	static bool UpdateProjectionAngles(
		const FDisplayClusterViewport_CustomFrustumRuntimeSettings& InCustomFrustumSettings,
		double& InOutLeft,
		double& InOutRight,
		double& InOutTop,
		double& InOutBottom);

	/**
	* These are frustum modifiers, where 1 means 100% of the frustum size.
	* 
	* Positive values mean frustum expansion:
	*   FrustumAngles.Left   -= FrustumAngles.Width * Left.
	*   FrustumAngles.Right  += FrustumAngles.Width * Right.
	*   FrustumAngles.Top    += FrustumAngles.Height * Top.
	*   FrustumAngles.Bottom -= FrustumAngles.Height * Bottom.
	*/
	struct FCustomFrustumPercent
		: public FMarginSet
	{
		/** Returns the aspect ratio multiplier. */
		inline double GetAspectRatioMult() const
		{
			// Positive values mean frustum expansion
			const double WidthMult = 1 + Left + Right;
			const double HeightMult = 1 + Top + Bottom;

			if (WidthMult <= 0.f || HeightMult <= 0.f)
			{
				// If extreme values are used, 1 is returned, which means that the aspect ratio remains unchanged.
				return 1;
			}

			return WidthMult / HeightMult;
		}
	};

	/**
	* Values in pixels
	*/
	struct FCustomFrustumPixels
		: public FIntMarginSet
	{
		inline FIntRect GetInnerRect(const FIntRect& InRect) const
		{
			const FIntPoint InnerSize = InRect.Size() - Size();
			const FIntPoint InnerPos = FIntPoint(Left, Top);

			return FIntRect(InnerPos, InnerPos + InnerSize);
		}

		inline FIntPoint Size() const
		{
			return FIntPoint(Left + Right, Top + Bottom);
		}
	};

	// Enable custom frustum
	bool bIsEnabled = false;

	// CustomFrustum sides in percent
	FCustomFrustumPercent CustomFrustumPercent;

	// CustomFrustum sides in pixels
	FCustomFrustumPixels  CustomFrustumPixels;
};
