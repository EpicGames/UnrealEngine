// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * Special flags for nDisplay viewport log
 */
enum class EDisplayClusterViewportShowLogMsgOnce : uint16
{
	None = 0,

	// No projection policy assigned for Viewports
	HandleStartScene_InvalidProjectionPolicy = 1 << 0,

	UpdateFrameContexts_FrameTargetRectHasZeroSize = 1 << 1,
	UpdateFrameContexts_RenderTargetRectHasZeroSize = 1 << 2,
	UpdateFrameContexts_TileSizeNotEqualContextSize = 1 << 3,
	UpdateFrameContexts =UpdateFrameContexts_FrameTargetRectHasZeroSize
	| UpdateFrameContexts_RenderTargetRectHasZeroSize
	| UpdateFrameContexts_TileSizeNotEqualContextSize,

	UpdateCameraPolicy_ReferencedCameraNameIsEmpty = 1 << 7,
	UpdateCameraPolicy_ReferencedCameraNotFound = 1 << 8,
	UpdateCameraPolicy = UpdateCameraPolicy_ReferencedCameraNameIsEmpty | UpdateCameraPolicy_ReferencedCameraNotFound,

	GetViewPointCameraComponent_NoRootActorFound = 1 << 9,
	GetViewPointCameraComponent_HasAssignedViewPoint = 1 << 10,
	GetViewPointCameraComponent_NotFound = 1 << 11,
	GetViewPointCameraComponent = GetViewPointCameraComponent_NoRootActorFound | GetViewPointCameraComponent_NotFound,

	ChromakeyViewportHasZeroSize = 1 << 15,
};
ENUM_CLASS_FLAGS(EDisplayClusterViewportShowLogMsgOnce);
