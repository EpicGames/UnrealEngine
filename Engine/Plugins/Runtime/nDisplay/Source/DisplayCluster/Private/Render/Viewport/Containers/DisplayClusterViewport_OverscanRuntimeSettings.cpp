// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Viewport/Containers/DisplayClusterViewport_OverscanRuntimeSettings.h"
#include "Render/Viewport/DisplayClusterViewportHelpers.h"

#include "HAL/IConsoleManager.h"

int32 GDisplayClusterRenderOverscanEnable = 1;
static FAutoConsoleVariableRef CVarDisplayClusterRenderOverscanEnable(
	TEXT("nDisplay.render.overscan.enable"),
	GDisplayClusterRenderOverscanEnable,
	TEXT("Enable overscan feature.\n")
	TEXT(" 0 - to disable.\n"),
	ECVF_Default
);

int32 GDisplayClusterRenderOverscanMaxValue = 50;
static FAutoConsoleVariableRef CVarDisplayClusterRenderOverscanMaxValue(
	TEXT("nDisplay.render.overscan.max_percent"),
	GDisplayClusterRenderOverscanMaxValue,
	TEXT("Max percent for overscan (default 50).\n"),
	ECVF_Default
);

///////////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterViewport_OverscanRuntimeSettings
///////////////////////////////////////////////////////////////////////////////////////////
bool FDisplayClusterViewport_OverscanRuntimeSettings::UpdateProjectionAngles(
	const FDisplayClusterViewport_OverscanRuntimeSettings& InOverscanRuntimeSettings,
	double& InOutLeft,
	double& InOutRight,
	double& InOutTop,
	double& InOutBottom)
{
	if (InOverscanRuntimeSettings.bIsEnabled)
	{
		const FVector2D SizeFOV(InOutRight - InOutLeft, InOutTop - InOutBottom);
		const FMarginSet FovMargins = InOverscanRuntimeSettings.OverscanPercent * SizeFOV;

		InOutLeft   -= FovMargins.Left;
		InOutRight  += FovMargins.Right;
		InOutBottom -= FovMargins.Bottom;
		InOutTop    += FovMargins.Top;

		return true;
	}

	return false;
}

void FDisplayClusterViewport_OverscanRuntimeSettings::UpdateOverscanSettings(
	const FString& InViewportId,
	const FDisplayClusterViewport_OverscanSettings& InOverscanSettings,
	FDisplayClusterViewport_OverscanRuntimeSettings& InOutOverscanRuntimeSettings,
	FIntRect& InOutRenderTargetRect)
{
	// Disable viewport overscan feature
	if (!GDisplayClusterRenderOverscanEnable || !InOverscanSettings.bEnabled)
	{
		return;
	}

	const FIntPoint Size = InOutRenderTargetRect.Size();
	if(Size.GetMin() <= 0)
	{
		// If the Size is invalid (on of values are zero or negative), overscan is disabled.
		InOutOverscanRuntimeSettings = FDisplayClusterViewport_OverscanRuntimeSettings();

		return;
	}

	/**
	* We can't use negative overscan values.
	* The idea behind the overscan is to add extra space on the sides of the RTT.
	* Note: this only applies to regular viewports (Outers, etc.).
	*
	* The inner frustum viewport has its own implementation for the overscan feature called "CustomFrustum".
	* (see FDisplayClusterViewport_CustomFrustumRuntimeSettings)
	*/
	const double OverscanMaxValue = FMath::Max(0, double(GDisplayClusterRenderOverscanMaxValue) / 100);

	switch (InOverscanSettings.Unit)
	{
	case EDisplayClusterViewport_FrustumUnit::Percent:
	{
		InOutOverscanRuntimeSettings.bIsEnabled = true;
		InOutOverscanRuntimeSettings.OverscanPercent = InOverscanSettings.GetClamped(0, OverscanMaxValue);
		break;
	}

	case EDisplayClusterViewport_FrustumUnit::Pixels:
	{
		InOutOverscanRuntimeSettings.bIsEnabled = true;
		InOutOverscanRuntimeSettings.OverscanPercent = (InOverscanSettings / Size).GetClamped(0, OverscanMaxValue);
		break;
	}

	default:
		break;
	}

	// Update RTT size for overscan
	if (InOutOverscanRuntimeSettings.bIsEnabled)
	{
		// Calc pixels from percent
		const FIntMarginSet OverscanPixels = (InOutOverscanRuntimeSettings.OverscanPercent * Size).GetRoundToInt();

		// Update values
		InOutOverscanRuntimeSettings.OverscanPixels.AssignMargins(OverscanPixels);
		InOutOverscanRuntimeSettings.BaseOverscanPixels.AssignMargins(OverscanPixels);

		// Quantize the overscan percentage to exactly fit the number of pixels.
		// This will avoid a mismatch between the overscanned frustum calculated in UpdateProjectionAngles and the pixel crop in GetFinalContextRect.
		InOutOverscanRuntimeSettings.OverscanPercent = FMarginSet(InOutOverscanRuntimeSettings.OverscanPixels) / Size;

		const FIntPoint OverscanSize = Size + InOutOverscanRuntimeSettings.OverscanPixels.Size();
		InOutOverscanRuntimeSettings.bOversize = InOverscanSettings.bOversize;

		// Check if new RTT size is supported
		if (InOutOverscanRuntimeSettings.bOversize)
		{
			const FIntPoint ValidOverscanSize = FDisplayClusterViewportHelpers::GetValidViewportRect(FIntRect(FIntPoint(0, 0), OverscanSize), InViewportId, TEXT("Overscan")).Size();
			if (OverscanSize != ValidOverscanSize)
			{
				// can't use overscan with extra size, disable oversize
				InOutOverscanRuntimeSettings.bOversize = false;
			}
		}

		// Increase the RTT size if possible
		if (InOutOverscanRuntimeSettings.bOversize)
		{
			InOutRenderTargetRect.Max = OverscanSize;

			return;
		}

		// Configure overcan rendering within the existing RTT size.
		{
			// Calculate the effective overscan margins relative to the original texture size.
			//
			// Definitions:
			//   Size                  = <W, H> (original texture resolution)
			//   OverscanPixels        = {L, R, T, B} (absolute overscan in pixels)
			//   OverscanPixels.Size() = <L + R, T + B>
			//
			// Steps:
			//   1. OverscanSize = Size + OverscanPixels.Size()
			//        "overscan-adjusted" total size, used only as a math reference
			//
			//   2. AdjustedOverscanPercent = OverscanPixels / OverscanSize
			//        fraction of each margin relative to overscan-adjusted size
			//
			//   3. AdjustedOverscanPixels = Size * AdjustedOverscanPercent
			//        overscan margins rescaled to match the original Size (in pixels)
			//
			//   4. InnerRegionSize = Size - AdjustedOverscanPixels.Size()
			//        the effective inner region after overscan is applied
			// 
			//   5. Final OverscanPercent = AdjustedOverscanPixels / InnerRegionSize
			//        overscan margins expressed as a fraction of the final usable region
			//
			const FMarginSet AdjustedOverscanPercent = FMarginSet(InOutOverscanRuntimeSettings.OverscanPixels) / OverscanSize;
			const FOverscanPixels AdjustedOverscanPixels((AdjustedOverscanPercent * Size).GetRoundToInt());

			// Compute the scaled inner region size based on overscan margins.
			const FIntPoint InnerRegionSize = Size - AdjustedOverscanPixels.Size();

			// Mathematically, InnerRegionSize should be to be positive for valid input
			// This check is a safeguard for unexpected or corrupted input values.
			if (InnerRegionSize.GetMin() <= 0)
			{
				// If it evaluates to zero or negative (invalid case), disable overscan.
				InOutOverscanRuntimeSettings = FDisplayClusterViewport_OverscanRuntimeSettings();

				return;
			}

			// Quantize the overscan percentage to exactly fit the number of pixels.
			InOutOverscanRuntimeSettings.OverscanPercent = FMarginSet(AdjustedOverscanPixels) / InnerRegionSize;

			// Update OverscanPixels with new value
			InOutOverscanRuntimeSettings.OverscanPixels = AdjustedOverscanPixels;
		}
	}
}
