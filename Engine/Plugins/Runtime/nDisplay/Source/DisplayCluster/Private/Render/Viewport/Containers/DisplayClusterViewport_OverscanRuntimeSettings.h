// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_OverscanSettings.h"

#include "Math/MarginSet.h"

/**
* Runtime overscan settings
*/
struct FDisplayClusterViewport_OverscanRuntimeSettings
	: public FMarginSet
{
	/** Update overscan settings
	* 
	* @param InViewport            - owner viewport
	* @param InOutRuntimeSettings  - the overscan runtime settings.
	* @param InOutRenderTargetRect - Viewport rect, changeable during overscanning
	*/
	static void UpdateOverscanSettings(const FString& InViewportId, const FDisplayClusterViewport_OverscanSettings& InOverscanSettings, FDisplayClusterViewport_OverscanRuntimeSettings& InOutOverscanRuntimeSettings, FIntRect& InOutRenderTargetRect);

	/** Update projection angles by overscan
	* 
	* @param InOverscanRuntimeSettings - the overscan runtime settings.
	* @param InOutLeft                 - the value of the left projection plane that you want to change
	* @param InOutRight                - the value of the right projection plane that you want to change
	* @param InOutTop                  - the value of the top projection plane that you want to change
	* @param InOutBottom               - the value of the bottom projection plane that you want to change
	*/
	static bool UpdateProjectionAngles(const FDisplayClusterViewport_OverscanRuntimeSettings& InOverscanRuntimeSettings, double& InOutLeft, double& InOutRight, double& InOutTop, double& InOutBottom);

	/**
	* Overscan values in pixels:
	*    +--------------------------+
	*    |           Top            |
	*    |        +-------+         |
	*    |   Left |       |  Right  |
	*    |        +-------+         |
	*    |         Bottom           |
	*    +--------------------------+
	*/
	struct FOverscanPixels
		: public FIntMarginSet
	{
		inline FIntRect GetInnerRect(const FIntRect& InRect) const
		{
			const FIntPoint InnerSize = InRect.Size() - Size();
			const FIntPoint InnerPos = FIntPoint(Left, Top);

			return FIntRect(InnerPos, InnerPos + InnerSize);
		}

		inline FIntPoint Size() const
		{
			return FIntPoint(Left + Right, Top + Bottom);
		}
	};

public:
	// Enable overscan
	uint8 bIsEnabled : 1 = 0;

	// Set to True to render at the overscan resolution, set to false to render at the resolution in the configuration and scale for overscan
	uint8 bOversize : 1 = 0;

public:
	// Overscan size per side, expressed as a percentage of the viewport size.
	// 0.0 = no overscan, 1.0 = 100% overscan (equal to the full viewport dimension).
	FMarginSet OverscanPercent;

	// Overscan size per side (in pixels) after applying resolution adjustments to match the overscan settings.
	// See bOversize for details on how resolution adjustments are applied.
	FOverscanPixels OverscanPixels;

	// The tile viewport may render at a lower resolution but will be stretched to its final size in the original viewport.
	// Stores the original values before any size adjustments, serving as the reference for overscan size modifiers.

	// Original overscan size per side (in pixels) before any resolution adjustments.
	FOverscanPixels BaseOverscanPixels;
};
