// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "SceneView.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_CameraMotionBlur.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_Context.h"

class FImplDisplayClusterViewport_CameraMotionBlur
{
public:
	inline void SetupSceneView(const FDisplayClusterViewport_Context& ViewportContext, FSceneView& InOutView) const
	{
		switch (BlurSetup.Mode)
		{
		case EDisplayClusterViewport_CameraMotionBlur::Off:
			//The value in shader maps to this: CAMERA_MOTION_BLUR_MODE = 0
			InOutView.bCameraMotionBlur = false;
			break;

		case EDisplayClusterViewport_CameraMotionBlur::On:
			//The value in shader maps to this: CAMERA_MOTION_BLUR_MODE = 1
			InOutView.bCameraMotionBlur = true;
			break;

		case EDisplayClusterViewport_CameraMotionBlur::Override:
			//The value in shader maps to this: CAMERA_MOTION_BLUR_MODE = 2
			InOutView.bCameraMotionBlur = true;
			{
				FViewMatrices::FMinimalInitializer Initializer;
				Initializer.ViewRotationMatrix = FInverseRotationMatrix(BlurSetup.CameraRotation) * FMatrix(
					FPlane(0, 0, 1, 0),
					FPlane(1, 0, 0, 0),
					FPlane(0, 1, 0, 0),
					FPlane(0, 0, 0, 1));
				Initializer.ViewOrigin = BlurSetup.CameraLocation;
				Initializer.ProjectionMatrix = ViewportContext.ProjectionMatrix;
				FViewMatrices ViewMatrices(Initializer);

				// Initialize blur for all contexts
				if (ViewportContext.ContextNum >= (uint32)CameraMatrices.Num())
				{
					CameraMatrices.AddZeroed(ViewportContext.ContextNum + 1);
					for (FViewMatrices& It : CameraMatrices)
					{
						It = ViewMatrices;
					}
				}

				FViewMatrices& PrevViewMatrices = CameraMatrices[ViewportContext.ContextNum];

				FVector DeltaTranslation = (PrevViewMatrices.GetPreViewTranslation() - ViewMatrices.GetPreViewTranslation());

				DeltaTranslation.X *= BlurSetup.TranslationScale;
				DeltaTranslation.Y *= BlurSetup.TranslationScale;
				DeltaTranslation.Z *= BlurSetup.TranslationScale;

				FMatrix InvViewProj = ViewMatrices.ComputeInvProjectionNoAAMatrix() * ViewMatrices.GetTranslatedViewMatrix().GetTransposed();
				FMatrix PrevViewProj = FTranslationMatrix(DeltaTranslation) * PrevViewMatrices.GetTranslatedViewMatrix() * PrevViewMatrices.ComputeProjectionNoAAMatrix();

				InOutView.ClipToPrevClipOverride = InvViewProj * PrevViewProj;

				// Save new camera matrices
				PrevViewMatrices = ViewMatrices;
			}
			break;

		case EDisplayClusterViewport_CameraMotionBlur::Undefined:
		default:
			break;
		}
	}

	void BeginUpdateSettings()
	{
		BlurSetup = FDisplayClusterViewport_CameraMotionBlur();
	}

public:
	FDisplayClusterViewport_CameraMotionBlur BlurSetup;

private:
	mutable TArray<FViewMatrices> CameraMatrices;
};

