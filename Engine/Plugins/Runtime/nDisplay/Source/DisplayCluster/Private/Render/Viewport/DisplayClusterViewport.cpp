// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Viewport/DisplayClusterViewport.h"
#include "Render/Viewport/DisplayClusterViewportHelpers.h"
#include "Render/Viewport/DisplayClusterViewportManager.h"
#include "Render/Viewport/DisplayClusterViewportProxy.h"
#include "Render/Viewport/DisplayClusterViewportManagerProxy.h"
#include "Render/Viewport/DisplayClusterViewport_OpenColorIO.h"
#include "Render/Viewport/DisplayClusterViewportManagerViewExtension.h"
#include "Render/Viewport/DisplayClusterViewportManagerViewPointExtension.h"

#include "Render/Viewport/Configuration/DisplayClusterViewportConfiguration.h"
#include "Render/Viewport/Configuration/DisplayClusterViewportConfigurationProxy.h"
#include "Render/Viewport/Configuration/DisplayClusterViewportConfigurationHelpers_Tile.h"
#include "Render/Viewport/Configuration/DisplayClusterViewportConfigurationHelpers_ICVFX.h"

#include "Render/Viewport/LightCard/DisplayClusterViewportLightCardManager.h"

#include "Render/Projection/IDisplayClusterProjectionPolicy.h"

#include "Render/Viewport/DisplayClusterViewportStereoscopicPass.h"
#include "Render/Viewport/RenderFrame/DisplayClusterRenderFrame.h"
#include "Render/Viewport/RenderFrame/DisplayClusterRenderFrameSettings.h"
#include "Render/Viewport/RenderTarget/DisplayClusterRenderTargetResource.h"

#include "Render/Viewport/Containers/DisplayClusterViewport_PostRenderSettings.h"
#include "Render/Viewport/Containers/DisplayClusterViewportProxyData.h"

#include "Render/Viewport/Preview/DisplayClusterViewportPreview.h"

#include "Render/DisplayDevice/Components/DisplayClusterDisplayDeviceBaseComponent.h"

#include "Render/Features/AutoExposure/DisplayClusterAutoExposure.h"
#include "Render/Features/DisplayClusterUpscaler.h"

#include "EngineUtils.h"
#include "OpenColorIORendering.h"
#include "PrimitiveDrawingUtils.h"
#include "SceneManagement.h"
#include "SceneView.h"
#include "UnrealClient.h"

#include "DisplayClusterSceneViewExtensions.h"

#include "LegacyScreenPercentageDriver.h"

#include "Misc/DisplayClusterLog.h"

#include "Templates/Greater.h"

int32 GDisplayClusterMultiGPUEnable = 1;
static FAutoConsoleVariableRef CVarDisplayClusterMultiGPUEnable(
	TEXT("DC.MultiGPU"),
	GDisplayClusterMultiGPUEnable,
	TEXT("Enable MultiGPU for Display Cluster rendering.  Useful to disable for debugging.  (Default = 1)"),
	ECVF_Default
);

///////////////////////////////////////////////////////////////////////////////////////
//          FDisplayClusterViewport
///////////////////////////////////////////////////////////////////////////////////////
FDisplayClusterViewport::FDisplayClusterViewport(
	const TSharedRef<FDisplayClusterViewportConfiguration, ESPMode::ThreadSafe>& InConfiguration,
	const FString& InViewportBaseId,
	const EDisplayClusterViewportFlags InViewportFlags,
	const TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe>& InProjectionPolicy,
	const FIntPoint* InOptTilePos)
	: Configuration(InConfiguration)
	, ViewportBaseId(InViewportBaseId)
	, ViewportFlags(InViewportFlags)
	, ViewportId(InConfiguration->BuildViewportName(InViewportBaseId, InViewportFlags, InOptTilePos))
	, ClusterNodeId(InConfiguration->GetClusterNodeId())
	, ViewportProxy(MakeShared<FDisplayClusterViewportProxy, ESPMode::ThreadSafe>(
		InConfiguration, ViewportBaseId, InViewportFlags, ViewportId, InProjectionPolicy))
	, ViewportPreview(MakeShared< FDisplayClusterViewportPreview, ESPMode::ThreadSafe>(InConfiguration, ViewportId))
	, UninitializedProjectionPolicy(InProjectionPolicy)
	, Contexts(CurrentFrameData.Contexts)
{
	check(!ClusterNodeId.IsEmpty());
	check(!ViewportBaseId.IsEmpty());
	check(UninitializedProjectionPolicy.IsValid());

	// Initialize ICVFX viewport
	if (EnumHasAnyFlags(InViewportFlags, EDisplayClusterViewportFlags::ICVFX_AnyRole))
	{
		// ICVFX viewports are not directly visible on the final frame texture (back buffer).
		// They are composed using ICVFX shaders.
		RenderSettings.bVisible = false;

		// All ICVFX viewports should have a flag to manage it properly
		// Mark as internal resource
		EnumAddFlags(RenderSettingsICVFX.RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::InternalResource);

		// Backward-compatibility bridge: derive legacy EDisplayClusterViewportRuntimeICVFXFlags
		// from the new EDisplayClusterViewportFlags until the legacy runtime enum is removed.
		FDisplayClusterViewportConfigurationHelpers_ICVFX::UpdateLegacyRuntimeICVFXFlagsFromViewportFlags(*this);
	}

	if (FDisplayClusterViewportManagerProxy* ViewportManagerProxy = Configuration->Proxy->GetViewportManagerProxyImpl())
	{
		// Add viewport proxy on renderthread
		ENQUEUE_RENDER_COMMAND(CreateDisplayClusterViewportProxy)(
			[ ViewportManagerProxy = ViewportManagerProxy->AsShared()
			, ViewportProxy = ViewportProxy
			](FRHICommandListImmediate& RHICmdList)
			{
				ViewportManagerProxy->CreateViewport_RenderThread(ViewportProxy);
			}
		);
	}
}

FDisplayClusterViewport::~FDisplayClusterViewport()
{
	if (FDisplayClusterViewportManagerProxy* ViewportManagerProxy = Configuration->Proxy->GetViewportManagerProxyImpl())
	{
		// Remove viewport proxy on render_thread
		ENQUEUE_RENDER_COMMAND(DeleteDisplayClusterViewportProxy)(
			[ ViewportManagerProxy = ViewportManagerProxy->AsShared()
			, ViewportProxy = ViewportProxy
			](FRHICommandListImmediate& RHICmdList)
			{
				ViewportManagerProxy->DeleteViewport_RenderThread(ViewportProxy);
			}
		);
	}

	OpenColorIO.Reset();

	// Handle projection policy EndScene event
	OnHandleEndScene();

	// Handle projection policy event
	ProjectionPolicy.Reset();
	UninitializedProjectionPolicy.Reset();

	if (FDisplayClusterViewportManager* ViewportManager = Configuration->GetViewportManagerImpl())
	{
		// Reset RTT size after viewport delete
		ViewportManager->ResetSceneRenderTargetSize();
	}
}

IDisplayClusterViewportPreview& FDisplayClusterViewport::GetViewportPreview() const
{
	return ViewportPreview.Get();
}

TSharedRef<IDisplayClusterViewportProxy, ESPMode::ThreadSafe> FDisplayClusterViewport::GetViewportProxy() const
{
	return ViewportProxy;
}

FDisplayClusterViewportProxyData* FDisplayClusterViewport::CreateViewportProxyData()
{
	FDisplayClusterViewportProxyData* OutViewportProxyData = new FDisplayClusterViewportProxyData(ViewportProxy);

	OutViewportProxyData->OpenColorIO = OpenColorIO;

	// Get Display Device proxy object
	if (UDisplayClusterDisplayDeviceBaseComponent* DisplayDevice = GetDisplayDeviceComponent(Configuration->GetPreviewSettings().DisplayDeviceRootActorType))
	{
		OutViewportProxyData->DisplayDeviceProxy = DisplayDevice->GetDisplayDeviceProxy(GetConfiguration());
	}

	OutViewportProxyData->RenderSettings = RenderSettings;
	OutViewportProxyData->RenderSettingsICVFX.SetParameters(RenderSettingsICVFX);
	OutViewportProxyData->PostRenderSettings.SetParameters(PostRenderSettings);

	// Additional parameters
	OutViewportProxyData->OverscanRuntimeSettings = OverscanRuntimeSettings;

	OutViewportProxyData->RemapMesh = ViewportRemap.GetRemapMesh();

	OutViewportProxyData->ProjectionPolicy = ProjectionPolicy;
	
	OutViewportProxyData->CurrentFrameData = CurrentFrameData;

	OutViewportProxyData->Resources = Resources;
	OutViewportProxyData->ViewStates = ViewStates;

	return OutViewportProxyData;
}

bool FDisplayClusterViewport::UseSameOCIO(const IDisplayClusterViewport* InViewportPtr) const
{
	if (const FDisplayClusterViewport* InViewport = static_cast<const FDisplayClusterViewport*>(InViewportPtr))
	{
		return IsOpenColorIOEquals(*InViewport);
	}

	return false;
}

void FDisplayClusterViewport::BeginNewFrame(const FIntPoint& InRenderFrameSize)
{
	check(IsInGameThread());

	// Update ViewportRemap geometry
	ViewportRemap.Update(*this, InRenderFrameSize);
}

void FDisplayClusterViewport::FinalizeNewFrame()
{
	check(IsInGameThread());

	// When all viewports processed, we remove all single frame custom postprocess
	CustomPostProcessSettings.FinalizeFrame();

	// Update projection policy proxy data
	if (ProjectionPolicy.IsValid())
	{
		ProjectionPolicy->UpdateProxyData(this);
	}

	RenderSettings.FinishUpdateSettings();
}

const TArray<FSceneViewExtensionRef> FDisplayClusterViewport::GatherActiveExtensions(const int32 ViewIndex, FViewport* InViewport) const
{
	FDisplayClusterViewportManager* ViewportManager = Configuration->GetViewportManagerImpl();
	if (ViewportManager && Contexts.IsValidIndex(ViewIndex))
	{
		// Configure FDisplayClusterViewportManagerViewPointExtension for this viewport.
		ViewportManager->SetCurrentStereoViewIndexForViewPointExtension(Contexts[ViewIndex].StereoViewIndex);
	}

	TArray<FSceneViewExtensionRef> OutExtensions;
	ImplGatherActiveExtensions(ViewIndex, InViewport, OutExtensions);

	if (ViewportManager)
	{
		ViewportManager->SetCurrentStereoViewIndexForViewPointExtension(INDEX_NONE);
	}

	// Sort extensions in order of priority (copied from FSceneViewExtensions::GatherActiveExtensions)
	Algo::SortBy(OutExtensions, &ISceneViewExtension::GetPriority, TGreater<>());

	return OutExtensions;
}

void FDisplayClusterViewport::ImplGatherActiveExtensions(const int32 ViewIndex, FViewport* InViewport, TArray<FSceneViewExtensionRef>& OutExtensions) const
{
	// Use VE from engine for default render and MRQ:
	switch (RenderSettings.CaptureMode)
	{
	case EDisplayClusterViewportCaptureMode::Default:
	case EDisplayClusterViewportCaptureMode::MoviePipeline:
		if (InViewport)
		{
			FDisplayClusterSceneViewExtensionContext ViewExtensionContext(InViewport, AsShared());
			OutExtensions = GEngine->ViewExtensions->GatherActiveExtensions(ViewExtensionContext);
			return;
		}
		else
		{
			if (UWorld* CurrentWorld = Configuration->GetCurrentWorld())
			{
				FDisplayClusterSceneViewExtensionContext ViewExtensionContext(CurrentWorld->Scene, AsShared());
				OutExtensions = GEngine->ViewExtensions->GatherActiveExtensions(ViewExtensionContext);
				return;
			}
		}

		// No extension found.
		return;
		
	case EDisplayClusterViewportCaptureMode::Chromakey:
	case EDisplayClusterViewportCaptureMode::Lightcard:
	{
		// Chromakey and LightCard only use some internal nDisplay ViewExtensions:
		if (FDisplayClusterViewportManager* ViewportManager = Configuration->GetViewportManagerImpl())
		{
			if (ViewportManager->GetViewportManagerViewPointExtension().IsValid())
			{
				// To call SetupView() as early as possible: integration with other ViewExtensions
				OutExtensions.Add(ViewportManager->GetViewportManagerViewPointExtension()->AsShared());
			}
			if (ViewportManager->GetViewportManagerViewExtension().IsValid())
			{
				// For callback purposes (preserving alpha channel, etc.)
				OutExtensions.Add(ViewportManager->GetViewportManagerViewExtension()->AsShared());
			}
		}
		break;
	}
	default:
		break;
	}
}

void FDisplayClusterViewport::Initialize()
{
	// Initialize a reference to this viewport for the preview API
	ViewportPreview->Initialize(*this);
}

void FDisplayClusterViewport::ReleaseAllResources()
{
	// Release viewport-owned render resources (textures, RTT).
	Resources.ReleaseAllResources();

	// Release material/view-state references on the game thread.
	CleanupViewState();

	// Drop game-thread view state references held by this viewport.
	// The viewport proxy keeps its own copy of these references.
	ViewStates.Reset();
}

void FDisplayClusterViewport::OnHandleStartScene()
{
	if (UninitializedProjectionPolicy.IsValid())
	{
		if (UninitializedProjectionPolicy->HandleStartScene(this))
		{
			ProjectionPolicy = UninitializedProjectionPolicy;
			UninitializedProjectionPolicy.Reset();

			ResetShowLogMsgOnce(EDisplayClusterViewportShowLogMsgOnce::HandleStartScene_InvalidProjectionPolicy);
		}
	}
	else
	{
		// Already Initialized
		if (!ProjectionPolicy.IsValid())
		{
			// No projection policy for this viewport
			if (CanShowLogMsgOnce(EDisplayClusterViewportShowLogMsgOnce::HandleStartScene_InvalidProjectionPolicy))
			{
				UE_LOGF(LogDisplayClusterViewport, Error, "No projection policy assigned for Viewports '%ls'.", *GetId());
			}
		}
	}
}

void FDisplayClusterViewport::OnHandleEndScene()
{
	if (ProjectionPolicy.IsValid())
	{
		ProjectionPolicy->HandleEndScene(this);
		UninitializedProjectionPolicy = ProjectionPolicy;
		ProjectionPolicy.Reset();
	}

	CleanupViewState();
}

void FDisplayClusterViewport::AddReferencedObjects(FReferenceCollector& Collector)
{
	// ViewStates released on rendering thread from viewport proxy object
}


void FDisplayClusterViewport::SetViewportBufferRatio(const float InBufferRatio)
{
	const float BufferRatio = FLegacyScreenPercentageDriver::GetCVarResolutionFraction() * InBufferRatio;
	if (RenderSettings.BufferRatio > BufferRatio)
	{
		if (FDisplayClusterViewportManager* ViewportManager = Configuration->GetViewportManagerImpl())
		{
			// Reset scene RTT when buffer ratio changed down
			ViewportManager->ResetSceneRenderTargetSize();
		}
	}

	RenderSettings.BufferRatio = BufferRatio;
}

void FDisplayClusterViewport::SetupSceneView(uint32 ContextNum, class UWorld* World, FSceneViewFamily& InOutViewFamily, FSceneView& InOutView) const
{
	check(IsInGameThread());
	check(Contexts.IsValidIndex(ContextNum));

	// Configure the capture mode to be used.
	IDisplayClusterViewportManager::SetupSceneView(RenderSettings.CaptureMode, InOutViewFamily, InOutView);

	// MRQ only uses viewport visibility settings
	if(RenderSettings.CaptureMode == EDisplayClusterViewportCaptureMode::MoviePipeline)
	{
		// Apply visibility settigns to view
		VisibilitySettings.SetupSceneView(*this, InOutView);

		return;
	}

	// Always modify rendering parameters if valid OCIO transformation is configured
	if (OpenColorIO.IsValid() && OpenColorIO->GetConversionSettings().IsValid())
	{
		OpenColorIO->SetupSceneView(InOutViewFamily, InOutView);
	}
	// When capturing with late OCIO enabled, we still need to modify the OCIO related
	// rendering parameters even though OCIO is not set. The receivers might have valid OCIO
	// transformations configured therefore should get a proper input texture.
	else if (RenderSettings.HasAnyMediaStates(EDisplayClusterViewportMediaState::CaptureLateOCIO))
	{
		FOpenColorIORendering::PrepareView(InOutViewFamily, InOutView);
	}

	if(Contexts[ContextNum].GPUIndex >= 0)
	{
		// Use custom GPUIndex for render
		InOutView.bOverrideGPUMask = true;
		InOutView.GPUMask = FRHIGPUMask::FromIndex((uint32)Contexts[ContextNum].GPUIndex);
	}

	if (Contexts[ContextNum].bOverrideCrossGPUTransfer || !RenderSettings.bEnableCrossGPUTransfer)
	{
		// Disable native cross-GPU transfers inside Renderer.
		InOutView.bAllowCrossGPUTransfer = false;
	}

	// Apply visibility settigns to view
	VisibilitySettings.SetupSceneView(*this, InOutView);

	// Handle Motion blur parameters
	CameraMotionBlur.SetupSceneView(Contexts[ContextNum], InOutView);

	// Handle depth of field parameters
	CameraDepthOfField.SetupSceneView(InOutView);

	// Handle AutoExposure for this viewport
	if (FDisplayClusterViewportManager* ViewportManager = Configuration->GetViewportManagerImpl())
	{
		ViewportManager->AutoExposure->ConfigureAutoExposureForSceneView(*this, ContextNum, InOutView);
	}

	// Handle DisplayDevice
	if (UDisplayClusterDisplayDeviceBaseComponent* InDisplayDeviceComponent = GetDisplayDeviceComponent(Configuration->GetPreviewSettings().DisplayDeviceRootActorType))
	{
		InDisplayDeviceComponent->SetupSceneView(*ViewportPreview, ContextNum, InOutViewFamily, InOutView);
	}

	// Handle upscalers
	FDisplayClusterUpscaler::SetupSceneView(*this, RenderSettings.UpscalerSettings, InOutViewFamily, InOutView);


	if (InOutView.State && World && World->Scene)
	{
		if (ShouldUseLumenPerView())
		{
			InOutView.State->AddLumenSceneData(World->Scene);
		}
		else
		{
			InOutView.State->RemoveLumenSceneData(World->Scene);
		}
	}
}

float FDisplayClusterViewport::GetClusterRenderTargetRatioMult(const FDisplayClusterRenderFrameSettings& InFrameSettings) const
{
	float ClusterRenderTargetRatioMult = InFrameSettings.ClusterRenderTargetRatioMult;

	// Support Outer viewport cluster rtt multiplier
	if (EnumHasAllFlags(RenderSettingsICVFX.RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::Target))
	{
		ClusterRenderTargetRatioMult *= InFrameSettings.ClusterICVFXOuterViewportRenderTargetRatioMult;
	}
	else if (EnumHasAllFlags(RenderSettingsICVFX.RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::InCamera))
	{
		ClusterRenderTargetRatioMult *= InFrameSettings.ClusterICVFXInnerViewportRenderTargetRatioMult;
	}

	// Cluster mult downscale in range 0..1
	return FMath::Clamp(ClusterRenderTargetRatioMult, 0.f, 1.f);
}

FIntPoint FDisplayClusterViewport::GetDesiredContextSize(const FIntPoint& InContextSize, const FDisplayClusterRenderFrameSettings& InFrameSettings) const
{
	// Overrides the base size of the RenderTarget texture for all viewport contexts.
	// The rest of the RTT size modifiers are applied after this.
	FIntPoint CustomCustomRenderTargetSize;
	const FIntPoint InSize = (ProjectionPolicy.IsValid() && ProjectionPolicy->GetCustomRenderTargetSize(this, CustomCustomRenderTargetSize)) ? CustomCustomRenderTargetSize : InContextSize;

	const float ClusterRenderTargetRatioMult = GetClusterRenderTargetRatioMult(InFrameSettings);

	// Check size multipliers in order bellow:
	const float RenderTargetAdaptRatio = FDisplayClusterViewportHelpers::GetValidSizeMultiplier(InSize, RenderSettings.RenderTargetAdaptRatio, ClusterRenderTargetRatioMult * RenderSettings.RenderTargetRatio);
	const float RenderTargetRatio = FDisplayClusterViewportHelpers::GetValidSizeMultiplier(InSize, RenderSettings.RenderTargetRatio, ClusterRenderTargetRatioMult * RenderTargetAdaptRatio);
	const float ClusterMult = FDisplayClusterViewportHelpers::GetValidSizeMultiplier(InSize, ClusterRenderTargetRatioMult, RenderTargetRatio * RenderTargetAdaptRatio);

	const float FinalRenderTargetMult = FMath::Max(RenderTargetAdaptRatio * RenderTargetRatio * ClusterMult, 0.f);

	// Scale RTT size
	FIntPoint DesiredContextSize = (ProjectionPolicy.IsValid() && !ProjectionPolicy->ShouldUseAnySizeScaleForRenderTarget(this))
		? InSize // Use original RenderTarget size.
		: FDisplayClusterViewportHelpers::ScaleTextureSize(InSize, FinalRenderTargetMult);

	return InFrameSettings.ApplyViewportSizeConstraint(*this, DesiredContextSize);
}

float FDisplayClusterViewport::GetCustomBufferRatio(const FDisplayClusterRenderFrameSettings& InFrameSettings) const
{
	float CustomBufferRatio = RenderSettings.BufferRatio;

	// Global multiplier
	CustomBufferRatio *= InFrameSettings.ClusterBufferRatioMult;

	if (EnumHasAllFlags(RenderSettingsICVFX.RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::Target))
	{
		// Outer viewport
		CustomBufferRatio *= InFrameSettings.ClusterICVFXOuterViewportBufferRatioMult;
	}
	else if (EnumHasAllFlags(RenderSettingsICVFX.RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::InCamera))
	{
		// Inner Frustum
		CustomBufferRatio *= InFrameSettings.ClusterICVFXInnerFrustumBufferRatioMult;
	}

	return CustomBufferRatio;
}

void FDisplayClusterViewport::ResetFrameContexts()
{
	Resources.ReleaseAllResources();

	// Reset means this viewport is no longer rendering. Both current and previous frame data are cleared
	// because if rendering resumes in a future frame it has no continuity with the old frame history
	// (e.g. temporal effects such as motion blur must not bleed across the gap).
	CurrentFrameData.Reset();
	PreviousFrameData.Reset();
}

bool FDisplayClusterViewport::UpdateFrameContexts(const uint32 InStereoViewIndex)
{
	check(IsInGameThread());

	const FDisplayClusterRenderFrameSettings& InFrameSettings = Configuration->GetRenderFrameSettings();
	const uint32 FrameTargetsAmount = InFrameSettings.GetViewPerViewportAmount();
	if (FrameTargetsAmount == 0)
	{
		ResetFrameContexts();

		return false;
	}

	// Scale desired frame rect
	const FVector2D DesiredFrameMult = InFrameSettings.GetDesiredFrameMult() * InFrameSettings.RenderResolutionScale;
	FIntRect DesiredFrameTargetRect = FDisplayClusterViewportHelpers::ScaleViewportRect(RenderSettings.Rect, DesiredFrameMult);

	// Support preview in scene rendering
	if (InFrameSettings.IsPreviewRendering())
	{
		// Preview renders each viewport into a separate texture, so each frame is zero-aligned
		DesiredFrameTargetRect = FIntRect(FIntPoint(0, 0), DesiredFrameTargetRect.Size());
	}

	// Special case mono->stereo
	const uint32 ViewportContextAmount = RenderSettings.bForceMono ? 1 : FrameTargetsAmount;

	// Freeze the image in the viewport only after the frame has been rendered
	const bool bViewportRendered = Contexts.Num() > 0 && Contexts.Num() == Resources[EDisplayClusterViewportResource::InputShaderResources].Num();
	if (bViewportRendered && RenderSettings.bEnable && ShouldFreezeRender())
	{
		// Raise freeze flag for this viewport logic
		RenderSettings.bFreezeRendering = true;

		// Block image resources from being re-allocated
		Resources.FreezeRendering(EDisplayClusterViewportResource::InputShaderResources);
		Resources.FreezeRendering(EDisplayClusterViewportResource::AdditionalTargetableResources);
		Resources.FreezeRendering(EDisplayClusterViewportResource::MipsShaderResources);

		// Release ViewState resources if they are not used for rendering.
		ViewStates.Empty();

		// Update context links for freezed viewport
		for (FDisplayClusterViewport_Context& ContextIt : Contexts)
		{
			ContextIt.StereoscopicPass = FDisplayClusterViewportStereoscopicPass::EncodeStereoscopicPass(ContextIt.ContextNum, ViewportContextAmount, InFrameSettings);
			ContextIt.StereoViewIndex = (int32)(InStereoViewIndex + ContextIt.ContextNum);
			ContextIt.bDisableRender = true;
			ContextIt.FrameTargetRect = FDisplayClusterViewportHelpers::GetValidViewportRect(DesiredFrameTargetRect, GetId(), TEXT("Context Frame"));

			// Force context to be recalculated.
			ContextIt.bCalculated = false;
			ContextIt.bValidData = false;
		}

		// Release only part of the resources, leaving resources that can be used by other viewports (viewport override feature)
		Resources.ReleaseNotSharedResources();

		return true;
	}

	// Snapshot the current contexts as the previous frame's contexts once per frame.
	// The frame number guard prevents redundant copies when this function is called multiple times per frame.
	const uint32 CurrentFrameNumber = Configuration->GetRenderFrameSettings().GetFrameNumber();
	if (CurrentFrameData.FrameNumber != CurrentFrameNumber)
	{
		// Cache previous frame data.
		PreviousFrameData = CurrentFrameData;
	}

	// Initialize new frame data
	CurrentFrameData.Reset();
	CurrentFrameData.FrameNumber = CurrentFrameNumber;

	// Free all resources
	Resources.ReleaseAllResources();

	if (RenderSettings.bEnable == false)
	{
		// Exclude this viewport from render and logic, but object still exist
		return false;
	}

	if (!VisibilitySettings.IsVisible())
	{
		// Exclude viewports that are empty from rendering.
		return false;
	}

	if (PostRenderSettings.GenerateMips.IsEnabled())
	{
		//Check if current projection policy supports this feature
		if (!ProjectionPolicy.IsValid() || !ProjectionPolicy->ShouldUseSourceTextureWithMips(this))
		{
			// Don't create unused mips texture
			PostRenderSettings.GenerateMips.Reset();
		}
	}

	// Make sure the frame target rect doesn't exceed the maximum resolution, and preserve its aspect ratio if it needs to be clamped
	FIntRect FrameTargetRect = FDisplayClusterViewportHelpers::GetValidViewportRect(DesiredFrameTargetRect, GetId(), TEXT("Context Frame"));

	// Exclude zero-size viewports from render
	if (FrameTargetRect.Size().GetMin() <= 0)
	{
		if (CanShowLogMsgOnce(EDisplayClusterViewportShowLogMsgOnce::UpdateFrameContexts_FrameTargetRectHasZeroSize))
		{
			UE_LOGF(LogDisplayClusterViewport, Error, "The viewport '%ls' FrameTarget rect has zero size %dx%d: Disabled", *GetId(), FrameTargetRect.Size().X, FrameTargetRect.Size().Y);
		}

		return false;
	}

	// When external overscan is enabled, bypass nDisplay's own overscan configuration
	// and apply a symmetric overscan sourced from either the explicit override value or the camera view info.
	if (Configuration->GetRenderFrameSettings().bUseExternalOverscan)
	{
		// External overscan applies to Outer and InnerFrustum viewports only.
		// ChromaKey, LightCard, and UVLightCard are excluded.
		const bool bViewportCanUseExternalOverscan = !EnumHasAnyFlags(GetViewportFlags(),
			EDisplayClusterViewportFlags::ICVFX_ChromaKey |
			EDisplayClusterViewportFlags::ICVFX_LightCard  |
			EDisplayClusterViewportFlags::ICVFX_UVLightCard);

		if (bViewportCanUseExternalOverscan)
		{
			float CameraOverscan = 0.f;
			if (Configuration->GetRenderFrameSettings().ExternalOverscanOverride.IsSet())
			{
				CameraOverscan = Configuration->GetRenderFrameSettings().ExternalOverscanOverride.GetValue();
			}
			else
			{
				FMinimalViewInfo CameraViewInfo;
				if (GetCameraViewPoint(CameraViewInfo))
				{
					CameraOverscan = CameraViewInfo.GetOverscan();
				}
			}

			// CameraOverscan is a per-side fraction of the half-frustum (UE camera convention).
			// nDisplay margin values are fractions of the full frustum extent, so divide by 2 to convert.
			const float FrustumMargin = CameraOverscan * 0.5f;

			// Apply overscan and enlarge the render target to match.
			RenderSettings.OverscanSettings.SetSymmetricOverscan(FrustumMargin, Configuration->GetRenderFrameSettings().bExternalOverscanOversize);
		}
	}

	// Scale context for rendering
	FIntPoint DesiredContextSize = GetDesiredContextSize(FrameTargetRect.Size(), InFrameSettings);

	// Tile rendering use custom size
	bool bUseTileRendering = false;
	FIntPoint TileContextSize = DesiredContextSize;
	FIntRect TileDestRect;
	FVector4 TileFrustumRegion(0,1,0,1);
	if (RenderSettings.TileSettings.GetType() == EDisplayClusterViewportTileType::Tile)
	{
		if (FDisplayClusterViewportManager* ViewportManager = Configuration->GetViewportManagerImpl())
		{
			// Source viewport shold be updated before tile.
			// The function GetPriority()
			TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe> SourceViewport = ViewportManager->ImplFindViewport(RenderSettings.TileSettings.GetSourceViewportId());
			if (SourceViewport.IsValid())
			{
				const TArray<FDisplayClusterViewport_Context>& SourceContexts = SourceViewport->GetContexts();
				if (!SourceContexts.IsEmpty())
				{
					// Currently Context[0] is always used to get the RenderTargetRect value.
					// But this will only work if the RenderTargetRect values for both contexts are the same, which is true when using a separate RTT for each context.
					// In the future we may set a goal to optimize stereo rendering within one RTT and one ViewFamily, then we will need to update this code.
					// Currently we always use a separate RTT for each viewport context to be able to use the highest possible texture resolution.
					// This is important when we use buffer ratio multiplier, overscan rendering function, etc.
					const FIntRect SrcRect = SourceContexts[0].RenderTargetRect;

					// Get the target rectangle for the tile in the original RTT viewport.
					TileDestRect = FDisplayClusterViewportConfigurationHelpers_Tile::GetDestRect(RenderSettings.TileSettings, SrcRect);

					// Use a custom tile size for rendering.
					TileContextSize = DesiredContextSize = TileDestRect.Size();

					bUseTileRendering = true;
				}
			}
		}

		if (!bUseTileRendering)
		{
			// don't use this tile
			return false;
		}
	}

	// Exclude zero-size viewports from render
	if (DesiredContextSize.GetMin() <= 0)
	{
		if (CanShowLogMsgOnce(EDisplayClusterViewportShowLogMsgOnce::UpdateFrameContexts_RenderTargetRectHasZeroSize))
		{
			UE_LOGF(LogDisplayClusterViewport, Error, "The viewport '%ls' RenderTarget rect has zero size %dx%d: Disabled", *GetId(), DesiredContextSize.X, DesiredContextSize.Y);
		}

		return false;
	}

	// Build RTT rect
	FIntRect RenderTargetRect = FIntRect(FIntPoint(0, 0), DesiredContextSize);

	// Support custom frustum rendering feature
	if (!RenderSettings.bDisableCustomFrustumFeature)
	{
		// Creates unique name "DCRA.Viewport"
		const FString UniqueViewportName = FString::Printf(TEXT("%s.%s"), *Configuration->GetRootActorName(), *GetId());
		FDisplayClusterViewport_CustomFrustumRuntimeSettings::UpdateCustomFrustumSettings(UniqueViewportName, RenderSettings.CustomFrustumSettings, CustomFrustumRuntimeSettings, RenderTargetRect);
	}

	FIntPoint ContextSize = RenderTargetRect.Size();

	if (bUseTileRendering && ContextSize != TileContextSize)
	{
		if (CanShowLogMsgOnce(EDisplayClusterViewportShowLogMsgOnce::UpdateFrameContexts_TileSizeNotEqualContextSize))
		{
			UE_LOGF(LogDisplayClusterViewport, Error, "The viewport '%ls' context size [%dx%d] should be equal with tile size [%dx%d]: Disabled", *GetId(), ContextSize.X, ContextSize.Y, TileContextSize.X, TileContextSize.Y);
		}

		return false;
	}

	// Support overscan rendering feature
	if (!RenderSettings.bDisableFrustumOverscanFeature)
	{
		FDisplayClusterViewport_OverscanRuntimeSettings::UpdateOverscanSettings(GetId(), RenderSettings.OverscanSettings, OverscanRuntimeSettings, RenderTargetRect);
	}

	// UV LightCard viewport use unique whole-cluster texture from LC manager
	if (EnumHasAllFlags(RenderSettingsICVFX.RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::UVLightcard))
	{
		// Use the UVLightCard viewport only when this type of lightcards has been defined
		bool bUseUVLightCardViewport = false;

		if (FDisplayClusterViewportManager* ViewportManager = Configuration->GetViewportManagerImpl())
		{
			const EDisplayClusterUVLightCardType UVLightCardType =
				EnumHasAllFlags(RenderSettingsICVFX.RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::OverInFrustum)
				? EDisplayClusterUVLightCardType::Over : EDisplayClusterUVLightCardType::Under;
			if (ViewportManager->LightCardManager->IsUVLightCardEnabled(UVLightCardType))
			{
				// Custom viewport size from LC Manager
				ContextSize = ViewportManager->LightCardManager->GetUVLightCardResourceSize(UVLightCardType);

				// Size must be not null
				if (ContextSize.GetMin() > 1)
				{
					FrameTargetRect = RenderTargetRect = FIntRect(FIntPoint(0, 0), ContextSize);

					// Allow to use this viewport
					bUseUVLightCardViewport = true;
				}
			}
		}

		if (!bUseUVLightCardViewport)
		{
			// do not use UV LightCard viewport
			return false;
		}
	}

	// Get the BufferRatio value so that the texture size does not exceed the maximum value.
	const float CustomBufferRatio = FDisplayClusterViewportHelpers::GetValidSizeMultiplier(RenderTargetRect.Size(), GetCustomBufferRatio(InFrameSettings), 1.f);

	// Is this viewport can be rendered.
	const bool bEnableRender = IsRenderEnabled();

	//Add new contexts
	for (uint32 ContextIt = 0; ContextIt < ViewportContextAmount; ++ContextIt)
	{
		const EStereoscopicPass StereoscopicPass = FDisplayClusterViewportStereoscopicPass::EncodeStereoscopicPass(ContextIt, ViewportContextAmount, InFrameSettings);
		const int32 StereoViewIndex = (int32)(InStereoViewIndex + ContextIt);

		FDisplayClusterViewport_Context Context(ContextIt, StereoscopicPass, StereoViewIndex);

		Context.GPUIndex = INDEX_NONE;

		// The nDisplay can use its own cross-GPU transfer
		if (InFrameSettings.CrossGPUTransfer.bEnable)
		{
			Context.bOverrideCrossGPUTransfer = true;
		}

		const int32 MaxExplicitGPUIndex = GDisplayClusterMultiGPUEnable ? GNumExplicitGPUsForRendering - 1 : 0;
		if (MaxExplicitGPUIndex > 0 && bEnableRender)
		{
			// Experimental: allow mGPU for preview rendering:
			if (const FIntPoint* GPURange = InFrameSettings.GetPreviewMultiGPURendering())
			{
				int32 MinGPUIndex = FMath::Min(GPURange->X, MaxExplicitGPUIndex);
				int32 MaxGPUIndex = FMath::Min(GPURange->Y, MaxExplicitGPUIndex);

				static int32 PreviewGPUIndex = MinGPUIndex;
				if (PreviewGPUIndex > MaxGPUIndex)
				{
					PreviewGPUIndex = MinGPUIndex;
				}

				Context.GPUIndex = PreviewGPUIndex++;
			}
			else
			{
				// Set custom GPU index for this view
				const int32 CustomMultiGPUIndex = (ContextIt > 0 && RenderSettings.StereoGPUIndex >= 0) ? RenderSettings.StereoGPUIndex : RenderSettings.GPUIndex;
				Context.GPUIndex = FMath::Min(CustomMultiGPUIndex, MaxExplicitGPUIndex);
			}
		}

		Context.FrameTargetRect = FrameTargetRect;
		Context.RenderTargetRect = RenderTargetRect;
		Context.TileDestRect = TileDestRect;
		Context.ContextSize = ContextSize;

		if (OverscanRuntimeSettings.bIsEnabled)
		{
			Context.OverscanInnerRenderTargetRect = OverscanRuntimeSettings.OverscanPixels.GetInnerRect(RenderTargetRect);
		}
		else
		{
			Context.OverscanInnerRenderTargetRect = RenderTargetRect;
		}

		// r.ScreenPercentage
		switch (RenderSettings.CaptureMode)
		{
		case EDisplayClusterViewportCaptureMode::Chromakey:
		case EDisplayClusterViewportCaptureMode::Lightcard:
			// we should not change the size of the Chromakey\Lighcards due to the way copy\resolve works for RT's
			// if the viewfamily resolves to RenderTarget it will remove alpha channel
			// if the viewfamily copying to RenderTarget, the texture would not match the size of RTT (when ScreenPercentage applied)
			break;
		default:
			Context.CustomBufferRatio = CustomBufferRatio;
		}

		Context.bDisableRender = !bEnableRender;

		Contexts.Add(Context);

		// Update internal caches for current frame
		FMinimalViewInfo DummyViewInfo;
		SetupViewPoint(ContextIt, DummyViewInfo);
	}

	// Reserve for resources
	if (IsResourceUsed(EDisplayClusterViewportResource::RenderTargets))
	{
		Resources[EDisplayClusterViewportResource::RenderTargets].AddZeroed(FrameTargetsAmount);
	}

	if (IsResourceUsed(EDisplayClusterViewportResource::InputShaderResources))
	{
		Resources[EDisplayClusterViewportResource::InputShaderResources].AddZeroed(FrameTargetsAmount);
	}

	if (IsResourceUsed(EDisplayClusterViewportResource::MipsShaderResources))
	{
		// Setup Mips resources:
		for (FDisplayClusterViewport_Context& ContextIt : Contexts)
		{
			ContextIt.NumMips = FDisplayClusterViewportHelpers::GetMaxTextureNumMips(InFrameSettings, PostRenderSettings.GenerateMips.GetRequiredNumMips(ContextIt.ContextSize));
			if (ContextIt.NumMips > 1)
			{
				Resources[EDisplayClusterViewportResource::MipsShaderResources].AddZeroed(FrameTargetsAmount);
				break;
			}
		}
	}

	// The AdditionalTargetableResource is used as a warpblend output
	if (IsResourceUsed(EDisplayClusterViewportResource::AdditionalTargetableResources))
	{
		Resources[EDisplayClusterViewportResource::AdditionalTargetableResources].AddZeroed(FrameTargetsAmount);
	}

	if (InFrameSettings.IsPreviewRendering() && IsResourceUsed(EDisplayClusterViewportResource::OutputPreviewTargetableResources))
	{
		// reserve preview texture resource for all visible viewports
		Resources[EDisplayClusterViewportResource::OutputPreviewTargetableResources].AddZeroed(FrameTargetsAmount);
	}

	ResetShowLogMsgOnce(EDisplayClusterViewportShowLogMsgOnce::UpdateFrameContexts);

	return true;
}

bool FDisplayClusterViewport::GetOCIOConversionSettings(FOpenColorIOColorConversionSettings& OutOCIOConversionSettings) const
{
	// Return OCIO conversion settings if available
	if (OpenColorIO)
	{
		OutOCIOConversionSettings = OpenColorIO->GetConversionSettings();
		return true;
	}

	return false;
}
