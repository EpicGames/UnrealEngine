// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Render/Viewport/IDisplayClusterViewport.h"

#include "Render/Viewport/Configuration/DisplayClusterViewportConfiguration.h"

#include "Render/Viewport/Containers/DisplayClusterViewport_CameraDepthOfField.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_CustomFrustumRuntimeSettings.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_Enums.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_OverscanRuntimeSettings.h"
#include "Render/Viewport/Containers/DisplayClusterViewportRemap.h"
#include "Render/Viewport/Containers/ImplDisplayClusterViewport_CameraMotionBlur.h"

#include "Render/Viewport/DisplayClusterViewport_CustomPostProcessSettings.h"
#include "Render/Viewport/DisplayClusterViewport_OpenColorIO.h"
#include "Render/Viewport/DisplayClusterViewport_VisibilitySettings.h"
#include "Render/Viewport/DisplayClusterViewportResources.h"

#include "SceneViewExtension.h"
#include "SceneViewExtensionContext.h"
#include "Templates/SharedPointer.h"

class UDisplayClusterConfigurationViewport;

class FDisplayClusterViewportManager;
class FDisplayClusterViewportManagerProxy;
class FDisplayClusterViewportPreview;
class FDisplayClusterViewportProxy;

struct FDisplayClusterRenderFrameSettings;


/**
 * Rendering viewport (sub-region of the main viewport)
 */
class FDisplayClusterViewport
	: public IDisplayClusterViewport
	, public TSharedFromThis<FDisplayClusterViewport, ESPMode::ThreadSafe>
{
public:
	/**
	* Constructs a new nDisplay viewport.
	*
	* @param InConfiguration      Viewport configuration object (thread-safe).
	* @param InViewportBaseId     The base viewport identifier (before ICVFX unique naming).
	* @param InViewportFlags      Usage type of the viewport (e.g. InCamera, Chromakey, LightCard).
	* @param InProjectionPolicy   Projection policy assigned to the viewport
	*                             (defines how the scene is projected/rendered).
	* @param InOptTilePos         Optional parameter for tile viewport position
	*/
	FDisplayClusterViewport(
		const TSharedRef<FDisplayClusterViewportConfiguration, ESPMode::ThreadSafe>& InConfiguration,
		const FString& InViewportBaseId,
		const EDisplayClusterViewportFlags InViewportFlags,
		const TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe>& InProjectionPolicy,
		const FIntPoint* InOptTilePos = nullptr);

	virtual ~FDisplayClusterViewport();

public:
	//////////////////////////////////////////////////////
	/// IDisplayClusterViewport
	//////////////////////////////////////////////////////
	virtual TSharedPtr<IDisplayClusterViewport, ESPMode::ThreadSafe> ToSharedPtr() override
	{
		return AsShared();
	}

	virtual TSharedPtr<const IDisplayClusterViewport, ESPMode::ThreadSafe> ToSharedPtr() const override
	{
		return AsShared();
	}

	virtual TSharedRef<class FDisplayClusterViewport, ESPMode::ThreadSafe> ToSharedRef() override
	{
		return AsShared();
	}

	virtual TSharedRef<const class FDisplayClusterViewport, ESPMode::ThreadSafe> ToSharedRef() const override
	{
		return AsShared();
	}

	virtual IDisplayClusterViewportConfiguration& GetConfiguration() override
	{
		return Configuration.Get();
	}

	virtual const IDisplayClusterViewportConfiguration& GetConfiguration() const override
	{
		return Configuration.Get();
	}

	virtual const TSharedRef<IDisplayClusterViewportConfiguration, ESPMode::ThreadSafe> GetConfigurationRef() override
	{
		return Configuration;
	}
	virtual const TSharedRef<const IDisplayClusterViewportConfiguration, ESPMode::ThreadSafe> GetConfigurationRef() const override
	{
		return Configuration;
	}

	/** Get viewport preview API */
	virtual IDisplayClusterViewportPreview& GetViewportPreview() const override;
	virtual TSharedRef<IDisplayClusterViewportProxy, ESPMode::ThreadSafe> GetViewportProxy() const override;

	virtual EDisplayClusterViewportFlags GetViewportFlags() const override
	{
		return ViewportFlags;
	}

	virtual FString GetBaseId() const override
	{
		return ViewportBaseId;
	}

	virtual FString GetId() const override
	{ 
		return ViewportId; 
	}

	virtual FString GetClusterNodeId() const override
	{
		return ClusterNodeId;
	}

	virtual const FDisplayClusterViewport_RenderSettings& GetRenderSettings() const override
	{
		check(IsInGameThread());
		return RenderSettings;
	}

	virtual void SetRenderSettings(const FDisplayClusterViewport_RenderSettings& InRenderSettings) override
	{
		check(IsInGameThread());
		RenderSettings = InRenderSettings;
	}

	virtual void SetContexts(TArray<FDisplayClusterViewport_Context>& InContexts) override
	{
		check(IsInGameThread());
		Contexts.Empty();
		Contexts.Append(InContexts);
	}

	virtual FVector2D GetClippingPlanes() const override;

	/** Provides current OCIO settings */
	virtual bool GetOCIOConversionSettings(FOpenColorIOColorConversionSettings& OutOCIOConversionSettings) const override;

	virtual void CalculateProjectionMatrix(const uint32 InContextNum, float Left, float Right, float Top, float Bottom, float ZNear, float ZFar, bool bIsAnglesInput) override;

	virtual bool    CalculateView(const uint32 InContextNum, FVector& InOutViewLocation, FRotator& InOutViewRotation, const float WorldToMeters) override;
	virtual bool    GetProjectionMatrix(const uint32 InContextNum, FMatrix& OutPrjMatrix)  override;

	virtual bool CalculateViewData(const uint32 InContextNum, const FVector& InViewLocation, const FRotator& InViewRotation, const float WorldToMeters) override;

	virtual bool SetupViewPoint(const uint32 InContextNum, FMinimalViewInfo& InOutViewInfo) override;
	virtual float GetStereoEyeOffsetDistance(const uint32 InContextNum) override;
	virtual class UDisplayClusterCameraComponent* GetViewPointCameraComponent(const EDisplayClusterRootActorType InRootActorType) const override;
	virtual class UDisplayClusterDisplayDeviceBaseComponent* GetDisplayDeviceComponent(const EDisplayClusterRootActorType InRootActorType) const override;
	virtual bool GetViewPointCameraEye(const uint32 InContextNum, FVector& OutViewLocation, FRotator& OutViewRotation, FVector& OutViewOffset) override;

	virtual const FDisplayClusterViewport_RenderSettingsICVFX& GetRenderSettingsICVFX() const override
	{
		check(IsInGameThread());
		return RenderSettingsICVFX;
	}

	virtual const FDisplayClusterViewport_PostRenderSettings& GetPostRenderSettings() const override
	{
		check(IsInGameThread());
		return PostRenderSettings;
	}

	virtual const TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe>& GetProjectionPolicy() const override
	{
		check(IsInGameThread());
		return ProjectionPolicy;
	}

	virtual const TArray<FDisplayClusterViewport_Context>& GetContexts() const override
	{
		check(IsInGameThread());
		return Contexts;
	}

	virtual const TArray<FDisplayClusterViewport_Context>& GetPrevFrameContexts() const override
	{
		check(IsInGameThread());
		return PreviousFrameData.Contexts;
	}

	virtual const FDisplayClusterViewportFrameData& GetFrameData() const override
	{
		check(IsInGameThread());
		return CurrentFrameData;
	}

	virtual const FDisplayClusterViewportFrameData& GetPrevFrameData() const override
	{
		check(IsInGameThread());
		return PreviousFrameData;
	}

	virtual TSharedPtr<FSceneViewStateReference, ESPMode::ThreadSafe> GetOrCreateViewState(uint32 InContextNum) override;

	virtual const IDisplayClusterViewport_CustomPostProcessSettings& GetViewport_CustomPostProcessSettings() const override
	{
		check(IsInGameThread());
		return CustomPostProcessSettings;
	}

	virtual IDisplayClusterViewport_CustomPostProcessSettings& GetViewport_CustomPostProcessSettings() override
	{
		check(IsInGameThread());
		return CustomPostProcessSettings;
	}

	virtual bool UseSameOCIO(const IDisplayClusterViewport* InViewportPtr) const override;

	// Setup scene view for rendering specified Context
	virtual void SetupSceneView(uint32 ContextNum, class UWorld* World, class FSceneViewFamily& InViewFamily, FSceneView& InView) const override;

	//////////////////////////////////////////////////////
	/// ~IDisplayClusterViewport
	//////////////////////////////////////////////////////

	void Initialize();

	/** Release all viewport resources: textures and ViewState. */
	void ReleaseAllResources();


	// Get from logic request for additional targetable resource
	bool ShouldUseFullSizeFrameTargetableResource() const;

	void SetViewportBufferRatio(const float InBufferRatio);

	inline bool FindContext(const int32 ViewIndex, uint32* OutContextNum)
	{
		check(IsInGameThread());

		for (int32 ContextNum = 0; ContextNum < Contexts.Num(); ContextNum++)
		{
			if (ViewIndex == Contexts[ContextNum].StereoViewIndex)
			{
				if (OutContextNum != nullptr)
				{
					*OutContextNum = ContextNum;
				}

				return true;
			}
		}

		return false;
	}
	
	/** Returns the raw ViewInfo from the camera, before stereo or warp-blend modification. */
	bool GetCameraViewPoint(FMinimalViewInfo& InOutViewInfo);

	void OnHandleStartScene();
	void OnHandleEndScene();

	void AddReferencedObjects(FReferenceCollector& Collector);

	/** This function MUST always be called before configuring the viewport at the beginning of each frame. */
	void ResetRuntimeParameters(const UDisplayClusterConfigurationViewport* InConfigurationViewport = nullptr);

	/** Collect the view extension active for the desired context of this viewport
	* @param ViewIndex - view index
	* @param InViewport - the viewport display
	*/
	const TArray<FSceneViewExtensionRef> GatherActiveExtensions(const int32 ViewIndex, FViewport* InViewport) const;

	/** Initialize viewport contexts and resources for new frame
	 *
	 * @param InStereoViewIndex - initial StereoViewIndex for this viewport
	 *
	 * @return - true, if success
	 */
	bool UpdateFrameContexts(const uint32 InStereoViewIndex);

	/** Reset viewport contexts and resources. */
	void ResetFrameContexts();

	/** Compare OCIO with another viewport, return true if they are equal. */
	bool IsOpenColorIOEquals(const FDisplayClusterViewport& InViewport) const;

	/** Get viewport OCIO instance. */
	const TSharedPtr<FDisplayClusterViewport_OpenColorIO, ESPMode::ThreadSafe>& GetOpenColorIO() const
	{
		check(IsInGameThread());
		return OpenColorIO;
	}

	/** Set viewport OCIO instance. */
	void SetOpenColorIO(const TSharedPtr<FDisplayClusterViewport_OpenColorIO, ESPMode::ThreadSafe>& InOpenColorIO)
	{
		check(IsInGameThread());

		OpenColorIO = InOpenColorIO;
	}

	/** Get viewport const resources for all contexts by type. */
	const TArray<TSharedPtr<FDisplayClusterViewportResource, ESPMode::ThreadSafe>>& GetViewportResources(const EDisplayClusterViewportResource InResourceType) const
	{
		check(IsInGameThread());
		return Resources[InResourceType];
	}

	/** Get viewport const resources for all contexts by type. */
	TArray<TSharedPtr<FDisplayClusterViewportResource, ESPMode::ThreadSafe>>& GetViewportResourcesImpl(const EDisplayClusterViewportResource InResourceType)
	{
		check(IsInGameThread());
		return Resources[InResourceType];
	}

	/** Create proxy data from this viewport internals.*/
	class FDisplayClusterViewportProxyData* CreateViewportProxyData();


	/** Gain direct access to internal data of the viewport. */
	FDisplayClusterViewport_RenderSettings& GetRenderSettingsImpl()
	{
		check(IsInGameThread());
		return RenderSettings;
	}

	/** Gain direct access to internal data of the viewport. */
	FDisplayClusterViewport_RenderSettingsICVFX& GetRenderSettingsICVFXImpl()
	{
		check(IsInGameThread());
		return RenderSettingsICVFX;
	}

	/** Gain direct access to internal data of the viewport. */
	FDisplayClusterViewport_PostRenderSettings& GetPostRenderSettingsImpl()
	{
		check(IsInGameThread());
		return PostRenderSettings;
	}

	/** Gain direct access to internal visibility data of the viewport. */
	FDisplayClusterViewport_VisibilitySettings& GetVisibilitySettingsImpl()
	{
		check(IsInGameThread());
		return VisibilitySettings;
	}

	/** Gain direct access to internal camera motion blur data of the viewport. */
	FImplDisplayClusterViewport_CameraMotionBlur& GetCameraMotionBlurImpl()
	{
		check(IsInGameThread());
		return CameraMotionBlur;
	}

	/** Gain direct access to internal depth of field data of the viewport. */
	FDisplayClusterViewport_CameraDepthOfField& GetCameraDepthOfFieldImpl()
	{
		check(IsInGameThread());
		return CameraDepthOfField;
	}

	/** Gain direct access to internal PostProcess data of the viewport. */
	FDisplayClusterViewport_CustomPostProcessSettings& GetCustomPostProcessSettings()
	{
		check(IsInGameThread());
		return CustomPostProcessSettings;
	}

	/** Some viewports are used as internal and skip some logic steps.
	* These viewports are handled separately from regular viewports.
	* Context: icvfx, tile
	*/
	bool IsInternalViewport() const;

	/** Return true if this Outer viewport.
	*/
	bool IsOuterViewport() const;

	/** Returns true if the RTT of this viewport is changed externally.
	* This means that this viewport will not be rendered.
	* This function is used as the basis for many others. Be careful when making changes to it.
	*/
	bool IsExternalRendering() const;

	/** Returns true if this viewport should be rendered. */
	bool IsRenderEnabled() const;

	/** Returns true if the rendering of this viewport is allowed by external media objects. */
	bool IsRenderEnabledByMedia() const;

	/**
	* Returns true if this viewport should not allocate resources.
	*
	* Note: This function also disables rendering, since rendering requires resources.
	*/
	bool ShouldDisableViewportResources() const;

	/** Returns true if this viewport used by external media objects. */
	bool IsUsedByMedia() const;

	/** Returns true if this viewport is to be used as a tile source. */
	bool CanSplitIntoTiles() const;

	/** Returns true if per-view Lumen scene is enabled. */
	bool ShouldUseLumenPerView() const;

	/** Return true if this viewport should freeze render. */
	bool ShouldFreezeRender() const;

	/** Return true if ViewState should be used by this viewport.*/
	bool ShouldUseViewStates() const;

	/** Returns true if textures in this viewport with an area larger than the MaxTextureArea limit
	*  must be reduced to stay within that limit with the same aspect ratio.
	*/
	bool ShouldApplyMaxTextureConstraints() const;

	/** Returns true if this viewport should use this type of resource. */
	bool IsResourceUsed(const EDisplayClusterViewportResource InResourceType) const;

	/** Start a new frame with the specified size. The associated objects and geometries will be updated accordingly. */
	void BeginNewFrame(const FIntPoint& InRenderFrameSize);

	/** Finalize new frame . */
	void FinalizeNewFrame();

	/** Release the projection policy assigned to this viewport. */
	void ReleaseProjectionPolicy()
	{
		ProjectionPolicy.Reset();
		UninitializedProjectionPolicy.Reset();
	}

	/** Update projection policy from configuration. */
	void UpdateConfiguration_ProjectionPolicy(const struct FDisplayClusterConfigurationProjection* InConfigurationProjectionPolicy = nullptr);

	/** setup overlay configuration for this viewport. */
	void UpdateConfiguration_OverlayRenderSettings(const struct FDisplayClusterConfigurationICVFX_OverlayAdvancedRenderSettings& InOverlaySettings);

	/** setup overscan configuration for this viewport. */
	void UpdateConfiguration_Overscan(const struct FDisplayClusterViewport_OverscanSettings& InOverscanSettings);

	/** setup CameraMotionBlur configuration for this viewport. */
	void UpdateConfiguration_CameraMotionBlur(const struct FDisplayClusterViewport_CameraMotionBlur& InCameraMotionBlur);

	/** setup camera depth of field configuration for this viewport. */
	void UpdateConfiguration_CameraDepthOfField(const class FDisplayClusterViewport_CameraDepthOfField& InCameraDepthOfField);

	/** setup PostRender mips configuration for this viewport. */
	void UpdateConfiguration_PostRenderGenerateMips(const struct FDisplayClusterConfigurationPostRender_GenerateMips& InGenerateMips);

	/** setup PostRender override configuration for this viewport. */
	void UpdateConfiguration_PostRenderOverride(const struct FDisplayClusterConfigurationPostRender_Override& InOverride);

	/** setup PostRender blur configuration for this viewport. */
	void UpdateConfiguration_PostRenderBlur(const struct FDisplayClusterConfigurationPostRender_BlurPostprocess& InBlurPostprocess);

	/** setup viewport remap configuration for this viewport. */
	bool UpdateConfiguration_ViewportRemap(const struct FDisplayClusterConfigurationViewport_Remap& InRemapConfiguration);

	/** Returns viewport configuration data. */
	const UDisplayClusterConfigurationViewport* GetViewportConfigurationData() const;

	/** Sets viewport configuration data. */
	void SetViewportConfigurationData(const UDisplayClusterConfigurationViewport* InConfigurationData);

	/** Cleanup view states. */
	void CleanupViewState();

	/** Returns true once if this type of log message can be displayed for the first time.*/
	bool CanShowLogMsgOnce(const EDisplayClusterViewportShowLogMsgOnce& InLogState) const
	{
		if (!EnumHasAnyFlags(ShowLogMsgOnceFlags, InLogState))
		{
			EnumAddFlags(ShowLogMsgOnceFlags, InLogState);

			return true;
		}

		return false;
	}

	/** Reset viewport log states. */
	void ResetShowLogMsgOnce(const EDisplayClusterViewportShowLogMsgOnce& InLogState) const
	{
		EnumRemoveFlags(ShowLogMsgOnceFlags, InLogState);
	}

	/** Viewports should be processed in the appropriate order.
	* Viewports with lower priority values will be processed earlier.
	*/
	uint8 GetPriority() const;

private:
	float GetClusterRenderTargetRatioMult(const FDisplayClusterRenderFrameSettings& InFrameSettings) const;
	FIntPoint GetDesiredContextSize(const FIntPoint& InSize, const FDisplayClusterRenderFrameSettings& InFrameSettings) const;
	float GetCustomBufferRatio(const FDisplayClusterRenderFrameSettings& InFrameSettings) const;

	/** Returns true if this viewport should use this type of resource.
	* This is function can be used only by the IsResourceUsed();
	*/
	bool IsResourceUsedImpl(const EDisplayClusterViewportResource InResourceType) const;

	/** Collect the view extension active for the desired context of this viewport
	* @param ViewIndex     - view index
	* @param InViewport    - the viewport display
	* @param OutExtensions - (out) Extension that can be used by this viewport context
	*/
	void ImplGatherActiveExtensions(const int32 ViewIndex, FViewport* InViewport, TArray<FSceneViewExtensionRef>& OutExtensions) const;

	/** Initialize View data
	 *
	 * @param InContextNum      - (in) viewport eye context index
	 * @param InOutViewLocation - (in) View location
	 * @param InOutViewRotation - (in) View rotator
	 * @param WorldToMeters     - UE world scale
	 *
	 * @return true on success
	*/
	bool PreCalculateViewData(const uint32 InContextNum, const FVector& InViewLocation, const FRotator& InViewRotation, const float WorldToMeters);

public:
	// Configuration of the current cluster node
	const TSharedRef<FDisplayClusterViewportConfiguration, ESPMode::ThreadSafe> Configuration;

	// Base viewport identifier. 
	// May not be unique across ICVFX viewports (multiple viewports can share the same base ID).
	const FString ViewportBaseId;

	// Usage flags of the viewport (e.g. InCamera, Chromakey, LightCard, Tile).
	const EDisplayClusterViewportFlags ViewportFlags;

	// Final unique viewport identifier. 
	// Guaranteed to be unique across the entire cluster (includes node ID and usage type if needed).
	const FString ViewportId;

	// Owner cluster node name
	const FString ClusterNodeId;

	// viewport proxy (render thread data)
	const TSharedRef<FDisplayClusterViewportProxy, ESPMode::ThreadSafe> ViewportProxy;

	// Viewport preview
	const TSharedRef<FDisplayClusterViewportPreview, ESPMode::ThreadSafe> ViewportPreview;

private:
	// Unified repository of viewport resources
	FDisplayClusterViewportResources Resources;

	/** nDisplay OpenColorIO object. */
	TSharedPtr<FDisplayClusterViewport_OpenColorIO, ESPMode::ThreadSafe> OpenColorIO;

	// Projection policy instance that serves this viewport
	TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe> ProjectionPolicy;
	TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe> UninitializedProjectionPolicy;

	// Postprocess
	FDisplayClusterViewport_CustomPostProcessSettings CustomPostProcessSettings;

	// Visibility settings
	FDisplayClusterViewport_VisibilitySettings VisibilitySettings;

	// Additional features
	FImplDisplayClusterViewport_CameraMotionBlur CameraMotionBlur;

	// Depth of field settings
	FDisplayClusterViewport_CameraDepthOfField CameraDepthOfField;

	// Overscan rendering feature
	FDisplayClusterViewport_OverscanRuntimeSettings      OverscanRuntimeSettings;

	// Custom frustum rendering feature
	FDisplayClusterViewport_CustomFrustumRuntimeSettings CustomFrustumRuntimeSettings;

	// viewport OutputRemap feature
	FDisplayClusterViewportRemap ViewportRemap;

	// Viewport render params
	FDisplayClusterViewport_RenderSettings       RenderSettings;
	FDisplayClusterViewport_RenderSettingsICVFX  RenderSettingsICVFX;
	FDisplayClusterViewport_PostRenderSettings   PostRenderSettings;

	/** Frame data snapshot for the current frame (contexts, cached view info). */
	FDisplayClusterViewportFrameData CurrentFrameData;

	/** Legacy reference alias to the current frame's viewport contexts stored in CurrentFrameData.Contexts. */
	TArray<FDisplayClusterViewport_Context>& Contexts;

	/** Frame data snapshot from the previous frame; used for motion blur and temporal effects. */
	FDisplayClusterViewportFrameData PreviousFrameData;

	// View states (preview only)
	TArray<TSharedPtr<FSceneViewStateReference, ESPMode::ThreadSafe>> ViewStates;

	// The soft ptr to the configuration data of this viewport. Updated every frame.
	TWeakObjectPtr<const UDisplayClusterConfigurationViewport> ConfigurationData;

	// A recurring message in the log will be shown only once
	mutable EDisplayClusterViewportShowLogMsgOnce ShowLogMsgOnceFlags = EDisplayClusterViewportShowLogMsgOnce::None;

	// Near clipping plane value (obtained from the GetDesiredView() functions).
	// If the value is less than zero, it does not apply to this viewport.
	// This value is changed in the SetupViewPoint() function, called at the beginning from LocalPlayer.
	float CustomNearClippingPlane = -1;
};
