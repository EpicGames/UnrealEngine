// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterViewportFrameStatsViewExtension.h"
#include "DisplayClusterSceneViewExtensions.h"

#include "Render/Viewport/DisplayClusterViewportManager.h"
#include "Render/Viewport/DisplayClusterViewport.h"
#include "Render/Viewport/DisplayClusterViewportManagerProxy.h"

#include "GlobalShader.h"
#include "Misc/App.h"
#include "Misc/Timecode.h"
#include "SceneView.h"
#include "ScreenPass.h"
#include "ShaderParameters.h"
#include "ShaderParameterStruct.h"
#include "SystemTextures.h"
#include "PostProcess/PostProcessMaterialInputs.h"


int32 GDisplayClusterShowFrameStats = 0;
static FAutoConsoleVariableRef CVarDisplayClusterShowFrameStats(
	TEXT("DC.Stats.Frame"),
	GDisplayClusterShowFrameStats,
	TEXT("Show frame stats (timecode, frame number) in a repeating pattern."),
	ECVF_RenderThreadSafe
);

BEGIN_SHADER_PARAMETER_STRUCT(FDisplayClusterFrameStatsShaderParameters, )
	SHADER_PARAMETER_TEXTURE(Texture2D, MiniFontTexture)
	SHADER_PARAMETER(uint32, FrameCount)
	SHADER_PARAMETER(uint32, Timecode)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

class FDisplayClusterFrameStatsShader : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FDisplayClusterFrameStatsShader);
	SHADER_USE_PARAMETER_STRUCT(FDisplayClusterFrameStatsShader, FGlobalShader);

	using FParameters = FDisplayClusterFrameStatsShaderParameters;
};

IMPLEMENT_SHADER_TYPE(, FDisplayClusterFrameStatsShader, TEXT("/Plugin/nDisplay/Private/FrameStats.usf"), TEXT("MainPS"), SF_Pixel);


BEGIN_SHADER_PARAMETER_STRUCT(FDisplayClusterFrameStatsOutputShaderParameters, )
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, InputTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, InputTextureSampler)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, FrameStatsTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, FrameStatsTextureSampler)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

class FDisplayClusterFrameStatsOutputShader : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FDisplayClusterFrameStatsOutputShader);
	SHADER_USE_PARAMETER_STRUCT(FDisplayClusterFrameStatsOutputShader, FGlobalShader);

	using FParameters = FDisplayClusterFrameStatsOutputShaderParameters;
};

IMPLEMENT_SHADER_TYPE(, FDisplayClusterFrameStatsOutputShader, TEXT("/Plugin/nDisplay/Private/FrameStats.usf"), TEXT("MainOutputPS"), SF_Pixel);

namespace UE::DisplayClusterViewExtension
{
	static FRHITexture* GetMiniFontTexture()
	{
		return GSystemTextures.AsciiTexture ? GSystemTextures.AsciiTexture->GetRHI() : GSystemTextures.WhiteDummy->GetRHI();
	}
}


///////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterViewportFrameStatsViewExtension
///////////////////////////////////////////////////////////////////////////////////////
FDisplayClusterViewportFrameStatsViewExtension::FDisplayClusterViewportFrameStatsViewExtension(const FAutoRegister& AutoRegister, const TSharedRef<FDisplayClusterViewportConfiguration, ESPMode::ThreadSafe>& InConfiguration)
	: FSceneViewExtensionBase(AutoRegister)
	, Configuration(InConfiguration)
{ }

void FDisplayClusterViewportFrameStatsViewExtension::SubscribeToPostProcessingPass(EPostProcessingPass PassId, const FSceneView& View, FAfterPassCallbackDelegateArray& InOutPassCallbacks, bool bIsPassEnabled)
{
	if (IsActive() && PassId == EPostProcessingPass::Tonemap)
	{
		InOutPassCallbacks.Add(FAfterPassCallbackDelegate::CreateRaw(this, &FDisplayClusterViewportFrameStatsViewExtension::PostProcessPassAfterTonemap_RenderThread));
	}
}

void FDisplayClusterViewportFrameStatsViewExtension::BeginRenderViewFamily(FSceneViewFamily& InViewFamily)
{
	if (IsActive())
	{
		const FTimecode GameThreadTimecode = FApp::GetTimecode();

		// Local lambda copy prevents race condition from further game thread updates before rendering.
		ENQUEUE_RENDER_COMMAND(DCFrameTimeCode)(
		[CurrentTimecode = GameThreadTimecode, WeakThis = AsWeak()](FRHICommandListImmediate& RHICmdList)
		{
			const TSharedPtr<FDisplayClusterViewportFrameStatsViewExtension> DCSVE = StaticCastSharedPtr<FDisplayClusterViewportFrameStatsViewExtension>(WeakThis.Pin());
			if (DCSVE.IsValid())
			{
				DCSVE->EncodedTimecode_RenderThread = (static_cast<uint8>(CurrentTimecode.Hours) << 24u) | (static_cast<uint8>(CurrentTimecode.Minutes) << 16u) | (static_cast<uint8>(CurrentTimecode.Seconds) << 8u) | static_cast<uint8>(CurrentTimecode.Frames);
			}
		});
	}
}

FScreenPassTexture FDisplayClusterViewportFrameStatsViewExtension::PostProcessPassAfterTonemap_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& View, const FPostProcessMaterialInputs& Inputs)
{
	const FScreenPassTexture& SceneColor = FScreenPassTexture::CopyFromSlice(GraphBuilder, Inputs.GetInput(EPostProcessMaterialInput::SceneColor));
	check(SceneColor.IsValid());

	FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(View.GetFeatureLevel());
	TShaderMapRef<FScreenPassVS> VertexShader(ShaderMap);

	FRDGTextureDesc TextureFrameStatsDesc = FRDGTextureDesc::Create2D(FIntPoint(128, 64), PF_R8G8B8A8, FClearValueBinding::Black, TexCreate_ShaderResource | TexCreate_RenderTargetable);
	FRDGTexture* TextureFrameStats = GraphBuilder.CreateTexture(TextureFrameStatsDesc, TEXT("DisplayCluster.FrameStatsRT"));

	// Render frame stats to a small render target to minimize shader costs
	// Note: As a further optimization, this could be renderered once per view family.
	if(VertexShader.IsValid())
	{
		// Always check if shaders are available on the current platform and hardware

		const FScreenPassTextureViewport Viewport(TextureFrameStats);
		FRenderTargetBinding FrameStatsRenderTargetBinding(TextureFrameStats, ERenderTargetLoadAction::EClear);

		TShaderMapRef<FDisplayClusterFrameStatsShader> FrameStatsPixelShader(ShaderMap);
		FDisplayClusterFrameStatsShaderParameters* Parameters = GraphBuilder.AllocParameters<FDisplayClusterFrameStatsShaderParameters>();
		Parameters->MiniFontTexture = UE::DisplayClusterViewExtension::GetMiniFontTexture();
		Parameters->RenderTargets[0] = FrameStatsRenderTargetBinding;
		Parameters->FrameCount = View.Family->FrameCounter; // Copy of GFrameCounter
		Parameters->Timecode = EncodedTimecode_RenderThread;

		AddDrawScreenPass(GraphBuilder, RDG_EVENT_NAME("DisplayClusterViewportFrameStatsPass"), View, Viewport, Viewport, VertexShader, FrameStatsPixelShader, Parameters);
	}

	FScreenPassRenderTarget Output = Inputs.OverrideOutput;
	if (!Output.IsValid())
	{
		Output = FScreenPassRenderTarget::CreateFromInput(GraphBuilder, SceneColor, View.GetOverwriteLoadAction(), TEXT("DisplayCluster.FrameStatsOutput"));
	}

	TShaderMapRef<FDisplayClusterFrameStatsOutputShader> OutputPixelShader(ShaderMap);
	if (!VertexShader.IsValid() || !OutputPixelShader.IsValid())
	{
		// Always check if shaders are available on the current platform and hardware
		return Output;
	}

	// Render output composited with frame stats texture
	{
		const FScreenPassTextureViewport InputViewport(SceneColor);
		const FScreenPassTextureViewport OutputViewport(Output);

		FDisplayClusterFrameStatsOutputShaderParameters* Parameters = GraphBuilder.AllocParameters<FDisplayClusterFrameStatsOutputShaderParameters>();
		Parameters->InputTexture = SceneColor.Texture;
		Parameters->InputTextureSampler = TStaticSamplerState<>::GetRHI();
		Parameters->FrameStatsTexture = TextureFrameStats;
		Parameters->FrameStatsTextureSampler = TStaticSamplerState<SF_Point, AM_Wrap, AM_Wrap>::GetRHI();
		Parameters->RenderTargets[0] = Output.GetRenderTargetBinding();

		AddDrawScreenPass(GraphBuilder, RDG_EVENT_NAME("DisplayClusterViewportFrameStatsOutputPass"), View, OutputViewport, InputViewport, VertexShader, OutputPixelShader, Parameters);
	}

	return Output;
}

bool FDisplayClusterViewportFrameStatsViewExtension::IsActiveThisFrame_Internal(const FSceneViewExtensionContext& Context) const
{
	if (IsActive())
	{
		static const FDisplayClusterSceneViewExtensionContext DCViewExtensionContext;
		if (Context.IsA(MoveTempIfPossible(DCViewExtensionContext)))
		{
			const FDisplayClusterSceneViewExtensionContext& DisplayContext = static_cast<const FDisplayClusterSceneViewExtensionContext&>(Context);
			if (DisplayContext.IsSameDisplayClusterViewportConfiguration(Configuration))
			{
				// Apply only for DC viewports
				return true;
			}
		}
	}

	return false;
}

bool FDisplayClusterViewportFrameStatsViewExtension::IsActive() const
{
	// VE is active as long as the viewport manager proxy still exists.
	return Configuration->Proxy->GetViewportManagerProxyImpl() != nullptr && GDisplayClusterShowFrameStats > 0;
}
