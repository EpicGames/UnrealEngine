// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "SceneViewExtension.h"
#include "Render/Viewport/Configuration/DisplayClusterViewportConfiguration.h"

#define DISPLAYCLUSTER_SCENE_DEBUG_VIEW_EXTENSION_PRIORITY 999

/**
 * DC-specific view extension to display frame stats
 */
class FDisplayClusterViewportFrameStatsViewExtension : public FSceneViewExtensionBase
{
public:
	FDisplayClusterViewportFrameStatsViewExtension(const FAutoRegister& AutoRegister, const TSharedRef<FDisplayClusterViewportConfiguration, ESPMode::ThreadSafe>& InConfiguration);
	virtual ~FDisplayClusterViewportFrameStatsViewExtension() = default;

public:
	//~ Begin ISceneViewExtension interface
	virtual void BeginRenderViewFamily(FSceneViewFamily& InViewFamily) override;

	void SubscribeToPostProcessingPass(EPostProcessingPass PassId, const FSceneView& View, FAfterPassCallbackDelegateArray& InOutPassCallbacks, bool bIsPassEnabled);
	FScreenPassTexture PostProcessPassAfterTonemap_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& View, const FPostProcessMaterialInputs& InOutInputs);

	virtual int32 GetPriority() const override { return DISPLAYCLUSTER_SCENE_DEBUG_VIEW_EXTENSION_PRIORITY; }

protected:
	virtual bool IsActiveThisFrame_Internal(const FSceneViewExtensionContext& Context) const override;

public:
	// Configuration of the current cluster node
	const TSharedRef<FDisplayClusterViewportConfiguration, ESPMode::ThreadSafe> Configuration;

private:
	/** True, if VE can be used at the moment. */
	bool IsActive() const;

private:

	// Encoded timecode
	uint32 EncodedTimecode_RenderThread;
};
