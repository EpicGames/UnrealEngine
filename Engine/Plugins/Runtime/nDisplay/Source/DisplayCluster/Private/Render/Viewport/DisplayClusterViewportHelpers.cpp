// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Viewport/DisplayClusterViewportHelpers.h"
#include "Render/Viewport/RenderTarget/DisplayClusterRenderTargetResource.h"
#include "Render/Viewport/RenderFrame/DisplayClusterRenderFrameSettings.h"

#include "Misc/DisplayClusterLog.h"

#include "Engine/TextureRenderTarget2D.h"

////////////////////////////////////////////////////////////////////////////////
int32 GDisplayClusterMaxTextureAreaRoot = 8192;
static FAutoConsoleVariableRef CVarDisplayClusterMaxTextureAreaRoot(
	TEXT("nDisplay.render.MaxTextureAreaRoot"),
	GDisplayClusterMaxTextureAreaRoot,
	TEXT("The root value of the maximum texture area. (The default value is 8192, which means Area=8192*8192.\n")
	TEXT("Textures with an area larger than this limit will be reduced to stay within this limit with the same aspect ratio.\n")
	TEXT("Set a negative or zero value to disable this constraint.\n"),
	ECVF_RenderThreadSafe
);

static FAutoConsoleVariableDeprecated CVarDisplayClusterOverrideMaxTextureDimension_Dep(TEXT("DC.OverrideMaxTextureDimension"), TEXT(""), TEXT("5.5"));
static FAutoConsoleVariableDeprecated CVarDisplayClusterOverrideMinTextureDimension_Dep(TEXT("DC.OverrideMinTextureDimension"), TEXT(""), TEXT("5.5"));
static FAutoConsoleVariableDeprecated CVarDisplayClusterMaxNumMips_Dep(TEXT("DC.TextureMaxNumMips"), TEXT(""), TEXT("5.5"));

// Default pixel format for preview rendering
int32 GDisplayClusterPreviewDefaultPixelFormat = 1;
static FAutoConsoleVariableRef CVarDisplayClusterPreviewDefaultPixelFormat(
	TEXT("nDisplay.preview.DefaultPixelFormat"),
	GDisplayClusterPreviewDefaultPixelFormat,
	TEXT("Defines the default preview RTT pixel format.\n")
	TEXT(" 0: 8bit fixed point RGBA\n")
	TEXT(" 1: 16bit Float RGBA\n")
	TEXT(" 2: 10bit fixed point RGB and 2bit Alpha\n"),
	ECVF_RenderThreadSafe
);

////////////////////////////////////////////////////////////////////////////////
namespace UE::DisplayCluster::ViewportHelpers
{
	static inline ETextureRenderTargetFormat ImplGetRenderTargetFormatFromInt(const int32 InDefaultPreviewPixelFormat)
	{
		const int32 ValidIndex = FMath::Clamp(InDefaultPreviewPixelFormat, 0, 2);
		static const ETextureRenderTargetFormat SPixelFormat[] = { RTF_RGBA8, RTF_RGBA16f, RTF_RGB10A2 };

		return SPixelFormat[ValidIndex];
	}
};

////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterViewportHelpers
////////////////////////////////////////////////////////////////////////////////
int32 FDisplayClusterViewportHelpers::GetMaxTextureNumMips(const FDisplayClusterRenderFrameSettings& InRenderFrameSettings, const int32 InNumMips)
{
	int32 NumMips = InNumMips;

	if(GDisplayClusterPreviewDefaultPixelFormat != 0 && InRenderFrameSettings.IsPreviewRendering())
	{
		//@todo: now UE support mips generation only for fixed point textures (8bit RGBA)
		// Remove this hack latter
		// Disable preview mips generation in case of unsupported RTT texture format.
		NumMips = 0;
	}

	return NumMips;
}

int32 FDisplayClusterViewportHelpers::GetMaxTextureArea()
{
	if (GDisplayClusterMaxTextureAreaRoot > 0)
	{
		// make sure this value is within hardware limits.
		const int32 MaxTextureAreaRoot = FMath::Clamp(
			GDisplayClusterMaxTextureAreaRoot,
			GetMinTextureDimension(),
			GetMaxTextureDimension());

		// Returns the area calculated from the root.
		return MaxTextureAreaRoot * MaxTextureAreaRoot;
	}

	return -1;
}

int32 FDisplayClusterViewportHelpers::GetMaxTextureDimension()
{
	static int32 NativeMaxTextureDimension = 1 << (GMaxTextureMipCount - 1);

	return NativeMaxTextureDimension;
}

int32 FDisplayClusterViewportHelpers::GetMinTextureDimension()
{
	return 1;
}

FIntRect FDisplayClusterViewportHelpers::GetValidViewportRect(const FIntRect& InRect, const FString& InViewportId, const TCHAR* InResourceName)
{
	// The target always needs be within GMaxTextureDimensions, larger dimensions are not supported by the engine
	const int32 MaxTextureSize = FDisplayClusterViewportHelpers::GetMaxTextureDimension();
	const int32 MinTextureSize = FDisplayClusterViewportHelpers::GetMinTextureDimension();

	int32 Width = FMath::Max(MinTextureSize, InRect.Width());
	int32 Height = FMath::Max(MinTextureSize, InRect.Height());

	FIntRect OutRect(InRect.Min, InRect.Min + FIntPoint(Width, Height));


	// Make sure the rect doesn't exceed the maximum resolution, and preserve its aspect ratio if it needs to be clamped
	const int32 RectMaxSize = OutRect.Max.GetMax();
	const float RectScale = (RectMaxSize > MaxTextureSize)
		? float(MaxTextureSize) / RectMaxSize
		: 1.f;

	OutRect.Min.X = FMath::Min(OutRect.Min.X, MaxTextureSize);
	OutRect.Min.Y = FMath::Min(OutRect.Min.Y, MaxTextureSize);

	const FIntPoint ScaledRectMax = FDisplayClusterViewportHelpers::ScaleTextureSize(OutRect.Max, RectScale);

	OutRect.Max.X = FMath::Clamp(ScaledRectMax.X, OutRect.Min.X, MaxTextureSize);
	OutRect.Max.Y = FMath::Clamp(ScaledRectMax.Y, OutRect.Min.Y, MaxTextureSize);

	// It's a temporary solution.
	// It does not work properly for multiple DCRAs with the same viewport names.
	static TMap<FString, bool> CachedLogMessages;
	const FString UniqueLogMessageId = FString::Printf(TEXT("%s.%s"), *InViewportId, (InResourceName == nullptr) ? TEXT("none") : InResourceName);
	bool* const LogMessageStatePtr = CachedLogMessages.Find(UniqueLogMessageId);

	if (InRect != OutRect)
	{
		if (LogMessageStatePtr == nullptr || *LogMessageStatePtr == false)
		{
			UE_LOGF(LogDisplayClusterViewport, Error,
				"The '%ls' %ls has been clamped %dx%d->%dx%d (max texture dimensions is %d)",
				*InViewportId,
				(InResourceName == nullptr) ? TEXT("none") : InResourceName,
				InRect.Width(), InRect.Height(),
				OutRect.Width(), OutRect.Height(),
				MaxTextureSize);

			// Raise the flag to show the log message once
			if (LogMessageStatePtr)
			{
				*LogMessageStatePtr = true;
			}
			else
			{
				CachedLogMessages.Emplace(UniqueLogMessageId, true);
			}
		}
	}
	else if (LogMessageStatePtr)
	{
		// Reset flag
		*LogMessageStatePtr = false;
	}


	return OutRect;
}

bool FDisplayClusterViewportHelpers::IsValidTextureSize(const FIntPoint& InSize)
{
	return InSize.GetMin() >= GetMinTextureDimension() && InSize.GetMax() <= GetMaxTextureDimension();
}

FIntPoint FDisplayClusterViewportHelpers::ScaleTextureSize(const FIntPoint& InSize, float InMult)
{
	// Floor-scale each axis and clamp to at least 1x1.
	return FIntPoint(
		FMath::Max(1, FMath::FloorToInt(InSize.X * InMult)),
		FMath::Max(1, FMath::FloorToInt(InSize.Y * InMult)));
}

FIntRect FDisplayClusterViewportHelpers::ScaleViewportRect(const FIntRect& InRect, const FVector2D& InPerAxisMult)
{
	FIntRect OutRect(
		FIntPoint(FMath::FloorToInt(InRect.Min.X * InPerAxisMult.X),
				  FMath::FloorToInt(InRect.Min.Y * InPerAxisMult.Y)),
		FIntPoint(FMath::FloorToInt(InRect.Max.X * InPerAxisMult.X),
				  FMath::FloorToInt(InRect.Max.Y * InPerAxisMult.Y))
		);
	
	// Ensure the rect is at least 1x1 after scaling.
	OutRect.Max.X = FMath::Max(OutRect.Max.X, OutRect.Min.X + 1);
	OutRect.Max.Y = FMath::Max(OutRect.Max.Y, OutRect.Min.Y + 1);

	return OutRect;
}

float FDisplayClusterViewportHelpers::GetValidSizeMultiplier(const FIntPoint& InSize, const float InSizeMult, const float InBaseSizeMult)
{
	// find best possible size mult in range 1..InSizeMult
	if (InSizeMult > 1.f)
	{
		const FIntPoint ScaledSize = ScaleTextureSize(InSize, FMath::Max(InSizeMult * InBaseSizeMult, 0.f));
		if (FDisplayClusterViewportHelpers::IsValidTextureSize(ScaledSize) == false)
		{
			// Try change 'RenderTargetAdaptRatio' to min possible value
			const float BaseMult = FMath::Max(InBaseSizeMult, 0.f);
			const FIntPoint MinScaledSize = ScaleTextureSize(InSize, BaseMult);

			if (FDisplayClusterViewportHelpers::IsValidTextureSize(MinScaledSize) == false)
			{
				// BaseSizeMult to big. Disable size mult
				return 1.f;
			}
			else
			{
				const int32 MinDimension = MinScaledSize.GetMax();
				const int32 MaxDimension = GetMaxTextureDimension();

				// Get the maximum allowed multiplier value
				const float OutMult = float(double(MaxDimension) / double(MinDimension));

#if UE_BUILD_DEBUG
				// debug purpose
				const FIntPoint FinalSize = ScaleTextureSize(InSize, FMath::Max(OutMult * InBaseSizeMult, 0.f));
				check(FDisplayClusterViewportHelpers::IsValidTextureSize(FinalSize));
#endif

				return OutMult;
			}
		}
	}

	return InSizeMult;
}

EPixelFormat FDisplayClusterViewportHelpers::GetPreviewDefaultPixelFormat()
{
	using namespace UE::DisplayCluster::ViewportHelpers;

	const ETextureRenderTargetFormat RenderTargetFormat = ImplGetRenderTargetFormatFromInt(GDisplayClusterPreviewDefaultPixelFormat);

	return GetPixelFormatFromRenderTargetFormat(RenderTargetFormat);
}

EPixelFormat FDisplayClusterViewportHelpers::GetDefaultPixelFormat()
{
	return EPixelFormat::PF_FloatRGBA;
}

bool FDisplayClusterViewportHelpers::GetValidResourceRectsForResolve(
	const FIntPoint& InputResourceSize,
	const FIntPoint& OutputResourceSize,
	FIntRect& InOutSourceRect,
	FIntRect& InOutDestRect)
{
	FIntRect SrcRect(InOutSourceRect);
	FIntRect DestRect(InOutDestRect);

	// If SrcRect or DestRect exceeds the texture size, RHI will crash. Let's adjust it to the texture size.
	{
		SrcRect.Min.X = FMath::Clamp(SrcRect.Min.X, 0, InputResourceSize.X);
		SrcRect.Min.Y = FMath::Clamp(SrcRect.Min.Y, 0, InputResourceSize.Y);
		SrcRect.Max.X = FMath::Clamp(SrcRect.Max.X, 0, InputResourceSize.X);
		SrcRect.Max.Y = FMath::Clamp(SrcRect.Max.Y, 0, InputResourceSize.Y);

		DestRect.Min.X = FMath::Clamp(DestRect.Min.X, 0, OutputResourceSize.X);
		DestRect.Min.Y = FMath::Clamp(DestRect.Min.Y, 0, OutputResourceSize.Y);
		DestRect.Max.X = FMath::Clamp(DestRect.Max.X, 0, OutputResourceSize.X);
		DestRect.Max.Y = FMath::Clamp(DestRect.Max.Y, 0, OutputResourceSize.Y);
	}

	// if InputRect.Min<0, also adjust the DestRect.Min
	DestRect.Min += FIntPoint(
		FMath::Max(0, -InOutSourceRect.Min.X),
		FMath::Max(0, -InOutSourceRect.Min.Y));

	// if OutputRect.Min<0, also adjust the SrcRect.Min
	SrcRect.Min += FIntPoint(
		FMath::Max(0, -InOutDestRect.Min.X),
		FMath::Max(0, -InOutDestRect.Min.Y));

	// Check the SrcRect and DestRect
	// SrcRect.Min and DestRect.Min should be always > 0
	if (SrcRect.Size().GetMin() <= 0 || DestRect.Size().GetMin() <= 0)
	{
		// The SrcRect or DestRect is invalid.
		return false;
	}

	// if the SrcRect.Max>DestRect.Max, truncate
	if (SrcRect.Width() > DestRect.Width())
	{
		SrcRect.Max.X = SrcRect.Min.X + DestRect.Width();
	}
	if (SrcRect.Height() > DestRect.Height())
	{
		SrcRect.Max.Y = SrcRect.Min.Y + DestRect.Height();
	}

	// Can be resolved
	InOutSourceRect = SrcRect;
	InOutDestRect = DestRect;

	return true;
}
