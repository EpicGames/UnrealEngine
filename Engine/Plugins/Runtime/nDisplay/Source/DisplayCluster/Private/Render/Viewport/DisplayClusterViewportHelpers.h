// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "PixelFormat.h"

/**
 * Helper class for DC viewports
 */
class FDisplayClusterViewportHelpers
{
public:
	/** Get the maximum allowable texture size used for the nDisplay viewport. */
	static int32 GetMinTextureDimension();

	/** Get the minimum allowable texture size used for the nDisplay viewport. */
	static int32 GetMaxTextureDimension();

	/** Get the maximum allowable texture area that can be used for the nDisplay viewport. */
	static int32 GetMaxTextureArea();

	/** Get the valid viewport size.
	* 
	* @param InRect         - source region
	* @param InViewportId   - The viewport name (debug purpose)
	* @param InResourceName - The resource name (debug purpose)
	* 
	* @return the region of the viewport that can be used
	*/
	static FIntRect GetValidViewportRect(const FIntRect& InRect, const FString& InViewportId, const TCHAR* InResourceName = nullptr);

	/** Return true, if size is valid.
	* 
	* @param InSize - the size to be checked.
	* 
	* @return true, if the size is within the minimum and maximum dimensions
	*/
	static bool IsValidTextureSize(const FIntPoint& InSize);

	/** Scaling texture size with a multiplier
	* 
	* @param InSize - the input texture size
	* @param InMult - texture size multiplier
	* 
	* @return - scaled texture size
	*/
	static FIntPoint ScaleTextureSize(const FIntPoint& InSize, float InMult);

	/** Scale a viewport rect by a per-axis multiplier.
	*
	* @param InRect        - the input rect
	* @param InPerAxisMult - per-axis size multiplier
	*
	* @return - scaled viewport rect
	*/
	static FIntRect ScaleViewportRect(const FIntRect& InRect, const FVector2D& InPerAxisMult);

	/** Find an acceptable multiplier for the texture size
	* 
	* @param InSize - in texture size
	* @param InSizeMult - the target multiplier
	* @param InBaseSizeMult - the base multiplier
	* 
	* @return - a multiplier that gives the valid size of the texture
	*/
	static float GetValidSizeMultiplier(const FIntPoint& InSize, const float InSizeMult, const float InBaseSizeMult);

	/** Getting the maximum allowable mips value for the rendering frame settings
	* 
	* @param InRenderFrameSettings  -the current render frame settings
	* @param InNumMips - texture mips value from the settings
	* 
	* @return - the mips value to be used.
	*/
	static int32 GetMaxTextureNumMips(const struct FDisplayClusterRenderFrameSettings& InRenderFrameSettings, const int32 InNumMips);

	/**
	 * Getting the default pixel format for preview rendering
	 */
	static EPixelFormat GetPreviewDefaultPixelFormat();

	/**
	 * Getting the default pixel format
	 */
	static EPixelFormat GetDefaultPixelFormat();

	/**
	* Check if resources with the specified regions can be resolved.
	* If any rect exceeds the texture size, RHI will crash.
	* This function adjusts the rects to the size of the textures.
	* 
	* @param InSourceTextureExtent - (in) Source RHI texture size
	* @param InDestTextureExtent   - (in) Destination RHI texture size
	* @param InOutSourceRect - (in, out) src rect
	* @param InOutDestRect   - (in, out) dest rect
	* 
	* @return false, if it isn't possible
	*/
	static bool GetValidResourceRectsForResolve(
		const FIntPoint& InSourceTextureExtent,
		const FIntPoint& InDestTextureExtent,
		FIntRect& InOutSourceRect,
		FIntRect& InOutDestRect);
};
