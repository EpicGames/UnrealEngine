// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Viewport/DisplayClusterViewportManager.h"
#include "Render/Viewport/DisplayClusterViewportManagerProxy.h"
#include "Render/Viewport/DisplayClusterViewportManagerViewExtension.h"
#include "Render/Viewport/DisplayClusterViewportManagerViewPointExtension.h"
#include "Render/Viewport/DisplayClusterViewportFrameStatsViewExtension.h"
#include "Render/Viewport/DisplayClusterViewportStereoscopicPass.h"

#include "Render/Viewport/Preview/DisplayClusterViewportManagerPreview.h"
#include "Render/Viewport/Preview/DisplayClusterViewportPreview.h"

#include "IDisplayCluster.h"
#include "Render/IDisplayClusterRenderManager.h"
#include "Render/Projection/IDisplayClusterProjectionPolicyFactory.h"
#include "Render/Projection/IDisplayClusterProjectionPolicy.h"
#include "Render/Warp/IDisplayClusterWarpPolicy.h"

#include "Render/Viewport/DisplayClusterViewport.h"
#include "Render/Viewport/DisplayClusterViewportProxy.h"
#include "Render/Viewport/DisplayClusterViewport_CustomPostProcessSettings.h"
#include "Render/Viewport/LightCard/DisplayClusterViewportLightCardManager.h"
#include "Render/Viewport/RenderFrame/DisplayClusterRenderFrameManager.h"
#include "Render/Viewport/RenderTarget/DisplayClusterRenderTargetManager.h"
#include "Render/Viewport/RenderTarget/DisplayClusterRenderTargetResource.h"
#include "Render/Viewport/Postprocess/DisplayClusterViewportPostProcessManager.h"
#include "Render/Viewport/Postprocess/DisplayClusterViewportPostProcessOutputRemap.h"
#include "Render/Viewport/Configuration/DisplayClusterViewportConfiguration.h"
#include "Render/Viewport/Configuration/DisplayClusterViewportConfigurationProxy.h"
#include "Render/Viewport/Configuration/DisplayClusterViewportConfiguration_Viewport.h"

#include "Render/Features/AutoExposure/DisplayClusterAutoExposure.h"
#include "Render/Features/DisplayClusterUpscaler.h"

#include "IDisplayClusterWarpBlend.h"

#include "SceneViewExtension.h"

#include "LegacyScreenPercentageDriver.h"

#include "DisplayClusterRootActor.h"

#include "Misc/DisplayClusterLog.h"
#include "Engine/LocalPlayer.h"
#include "Engine/Console.h"

int32 GDisplayClusterLightcardsAllowNanite = 0;
static FAutoConsoleVariableRef CVarDisplayClusterLightcardsAllowNanite(
	TEXT("DC.Lightcards.AllowNanite"),
	GDisplayClusterLightcardsAllowNanite,
	TEXT("0 disables Nanite when rendering lightcards. Otherwise uses default showflag."),
	ECVF_RenderThreadSafe
);

int32 GDisplayClusterChromaKeyAllowNanite = 0;
static FAutoConsoleVariableRef CVarDisplayClusterChromaKeyAllowNanite(
	TEXT("DC.ChromaKey.AllowNanite"),
	GDisplayClusterChromaKeyAllowNanite,
	TEXT("0 disables Nanite when rendering custom chroma keys. Otherwise uses default showflag."),
	ECVF_RenderThreadSafe
);

namespace UE::DisplayCluster::ViewportManager
{
	/**
	 * Container for the warp policy with its associated group of viewports.
	 */
	struct FDisplayClusterWarpPolicyViewportsGroup
	{
		/** HandleNewFrame() call for the warp policy interface. */
		inline void HandleNewFrame() const
		{
			if (WarpPolicy.IsValid())
			{
				WarpPolicy->HandleNewFrame(Viewports);
			}
		}

		// Warp polic instance
		TSharedPtr<IDisplayClusterWarpPolicy, ESPMode::ThreadSafe> WarpPolicy;

		// A group of viewports using the same warp policy
		TArray<TSharedPtr<IDisplayClusterViewport, ESPMode::ThreadSafe>> Viewports;
	};

	/**
	 * A helper class for handling the warp policies.
	 */
	struct FDisplayClusterWarpPolicyManager
	{
		/** Handle new frame for warp policies. */
		inline void HandleNewFrame(FDisplayClusterViewportManager& ViewportManager)
		{
			// Collect warp policies for the viewports of the entire cluster.
			// this requires that all viewports in the cluster exist at the moment.
			for (const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& Viewport : ViewportManager.ImplGetEntireClusterViewports())
			{
				if (IDisplayClusterWarpPolicy* ViewportWarpPolicyPtr = (Viewport.IsValid() && Viewport->GetProjectionPolicy().IsValid()) ? Viewport->GetProjectionPolicy()->GetWarpPolicy() : nullptr)
				{
					AddViewportToWarpPolicy(ViewportWarpPolicyPtr->ToSharedPtr(), Viewport);
				}
			}

			// Processing a new frame event for all warp policies
			for (const FDisplayClusterWarpPolicyViewportsGroup& ViewportsGroup : WarpPolicies)
			{
				ViewportsGroup.HandleNewFrame();
			}
		}

	protected:
		/** Adds a viewport to containers with warp policies and associated viewport groups. */
		inline void AddViewportToWarpPolicy(const TSharedPtr<IDisplayClusterWarpPolicy, ESPMode::ThreadSafe>& InWarpPolicy, const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& InViewport)
		{
			if (FDisplayClusterWarpPolicyViewportsGroup* ViewportsGroupPtr = WarpPolicies.FindByPredicate([InWarpPolicy](const FDisplayClusterWarpPolicyViewportsGroup& ViewportsGroupIt)
				{
					return ViewportsGroupIt.WarpPolicy == InWarpPolicy;
				}))
			{
				ViewportsGroupPtr->Viewports.Add(InViewport);
			}
			else
			{
				FDisplayClusterWarpPolicyViewportsGroup ViewportsGroup;
				ViewportsGroup.WarpPolicy = InWarpPolicy;
				ViewportsGroup.Viewports.Add(InViewport);

				WarpPolicies.Add(ViewportsGroup);
			}
		}

	private:
		// Warp policies viewports groups
		TArray<FDisplayClusterWarpPolicyViewportsGroup> WarpPolicies;
	};
};
using namespace UE::DisplayCluster::ViewportManager;

///////////////////////////////////////////////////////////////////////////////////////
//          FDisplayClusterViewportManager
///////////////////////////////////////////////////////////////////////////////////////
TSharedRef<IDisplayClusterViewportManager, ESPMode::ThreadSafe> IDisplayClusterViewportManager::CreateViewportManager()
{
	TSharedRef<FDisplayClusterViewportManager, ESPMode::ThreadSafe> ViewportManager = MakeShared<FDisplayClusterViewportManager, ESPMode::ThreadSafe>();

	// After the constructor, we should always call this function to initialize internal references.
	ViewportManager->Initialize();

	return ViewportManager;
}

void IDisplayClusterViewportManager::SetupSceneView(const EDisplayClusterViewportCaptureMode InMode, FSceneViewFamily& InOutViewFamily, FSceneView& InOutView)
{
	if (InMode == EDisplayClusterViewportCaptureMode::Lightcard)
	{
		// The UVLC texture is shared and can be used for any kind of viewport (with or without OCIO).
		// Because of UVLC texture sharing, always render any kind of LightCard viewports as for OCIO.
		// This will result in the same rendering result, and as a result the LC and UVLC colors will match.
		FOpenColorIORendering::PrepareView(InOutViewFamily, InOutView);
	}

	// Disable raytracing for lightcard and chromakey
	switch (InMode)
	{
	case EDisplayClusterViewportCaptureMode::Chromakey:
	case EDisplayClusterViewportCaptureMode::Lightcard:

		InOutView.bAllowRayTracing = false;
		break;

	default:
		break;
	}
}

void IDisplayClusterViewportManager::SetupEngineShowFlags(const EDisplayClusterViewportCaptureMode InMode, FEngineShowFlags& InOutEngineShowFlags)
{
	// Control NaniteMeshes for ChromaKey and Lightcards:
	switch (InMode)
	{
	case EDisplayClusterViewportCaptureMode::Chromakey:
		if (!GDisplayClusterChromaKeyAllowNanite)
		{
			InOutEngineShowFlags.SetNaniteMeshes(0);
		}

		break;
	case EDisplayClusterViewportCaptureMode::Lightcard:
		if (!GDisplayClusterLightcardsAllowNanite)
		{
			InOutEngineShowFlags.SetNaniteMeshes(0);
		}
		break;
	default:
		break;
	}

	switch (InMode)
	{
	case EDisplayClusterViewportCaptureMode::Chromakey:
	case EDisplayClusterViewportCaptureMode::Lightcard:
		// Disable postprocess for LC\CK
		InOutEngineShowFlags.SetPostProcessing(0);

		InOutEngineShowFlags.SetAtmosphere(0);
		InOutEngineShowFlags.SetFog(0);
		InOutEngineShowFlags.SetVolumetricFog(0);
		InOutEngineShowFlags.SetMotionBlur(0); // motion blur doesn't work correctly with scene captures.
		InOutEngineShowFlags.SetSeparateTranslucency(0);
		InOutEngineShowFlags.SetHMDDistortion(0);
		InOutEngineShowFlags.SetOnScreenDebug(0);
		InOutEngineShowFlags.SetHair(0);

		InOutEngineShowFlags.SetLumenReflections(0);
		InOutEngineShowFlags.SetLumenGlobalIllumination(0);
		InOutEngineShowFlags.SetGlobalIllumination(0);

		InOutEngineShowFlags.SetScreenSpaceAO(0);
		InOutEngineShowFlags.SetAmbientOcclusion(0);
		InOutEngineShowFlags.SetDeferredLighting(0);
		InOutEngineShowFlags.SetVirtualTexturePrimitives(0);
		InOutEngineShowFlags.SetRectLights(0);

		// Finish making it essentially Unlit
		InOutEngineShowFlags.SetLightFunctions(0);
		InOutEngineShowFlags.SetDynamicShadows(0);
		InOutEngineShowFlags.SetLighting(0);

		break;

	default:
		break;
	}
}

FDisplayClusterViewportManager::FDisplayClusterViewportManager()
	: Configuration(MakeShared<FDisplayClusterViewportConfiguration, ESPMode::ThreadSafe>())
	, ViewportManagerPreview(MakeShared<FDisplayClusterViewportManagerPreview, ESPMode::ThreadSafe>(Configuration))
	, RenderTargetManager(MakeShared<FDisplayClusterRenderTargetManager, ESPMode::ThreadSafe>(Configuration))
	, PostProcessManager(MakeShared<FDisplayClusterViewportPostProcessManager, ESPMode::ThreadSafe>(Configuration))
	, LightCardManager(MakeShared<FDisplayClusterViewportLightCardManager, ESPMode::ThreadSafe>(Configuration))
	, RenderFrameManager(MakeShared<FDisplayClusterRenderFrameManager>(Configuration))
	, AutoExposure(MakeShared<FDisplayClusterAutoExposure>(Configuration))
{
	// Always reset RTT when root actor re-created
	ResetSceneRenderTargetSize();

	RegisterCallbacks();
}

void FDisplayClusterViewportManager::Initialize()
{
	// Create ViewportManager proxy object
	ViewportManagerProxy = MakeShared<FDisplayClusterViewportManagerProxy, ESPMode::ThreadSafe>(*this);

	// Initialize configuration for exists viewport managers/proxy objects
	Configuration->Initialize(*this);

	// Register this viewport manager with the preview rendering pipeline.
	ViewportManagerPreview->RegisterPreviewRendering();
}

void FDisplayClusterViewportManager::Release()
{
	check(IsInGameThread());

	// Remove exclusive locks on release.
	Configuration->SetExclusiveLockOnRootActors(false);

	// Calls OnSceneEnd() for all subsystems. They will release their own resources.
	HandleEndScene();

	// Frees GPU memory as fast as possible.
	ReleaseAllResources();

	// This is a flag to check in the destructor that Release() has been called before.
	bReleaseFunctionCalledBeforeDestructor = true;
}

FDisplayClusterViewportManager::~FDisplayClusterViewportManager()
{
	if (!bReleaseFunctionCalledBeforeDestructor)
	{
		// This is the wrong way to call the destructor.
		// Lets show a message in the log
		UE_LOGF(LogDisplayClusterViewport, Error,
			"~FDisplayClusterViewportManager() is called at an inappropriate time. The Release() function must be called before this.");
	}

	UnregisterCallbacks();

	// Remove this viewport manager from the preview rendering pipeline.
	ViewportManagerPreview->UnregisterPreviewRendering();
	
	// Remove viewports
	EntireClusterViewports.Reset();
	CurrentRenderFrameViewports.Reset();

	// Release all DC VE
	ViewportManagerViewExtension.Reset();
	ViewportManagerViewPointExtension.Reset();
	FrameStatsViewExtension.Reset();

	LightCardManager->Release();
	RenderTargetManager->Release();
	PostProcessManager->Release_GameThread();

	if (ViewportManagerProxy.IsValid())
	{
		// Remove viewport manager proxy on render_thread
		ENQUEUE_RENDER_COMMAND(DeleteDisplayClusterViewportManagerProxy_Release)(
			[ViewportManagerProxy = ViewportManagerProxy](FRHICommandListImmediate& RHICmdList)
			{
				ViewportManagerProxy->Release_RenderThread();
			});

		ViewportManagerProxy.Reset();
	}
}

void FDisplayClusterViewportManager::ReleaseAllResources()
{
	for (const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& ViewportIt : ImplGetEntireClusterViewports())
	{
		if (ViewportIt.IsValid())
		{
			ViewportIt->ReleaseAllResources();
		}
	}

	if (ViewportManagerProxy.IsValid())
	{
		// Remove viewport manager proxy on render_thread
		ENQUEUE_RENDER_COMMAND(DisplayClusterViewportManagerProxy_ReleaseAllResources)(
			[ViewportManagerProxy = ViewportManagerProxy](FRHICommandListImmediate& RHICmdList)
			{
				ViewportManagerProxy->ReleaseAllResources_RenderThread();
			});
	}

	// Release RTT manager caches
	RenderTargetManager->Release();

	// Release AutoExposure inner data
	AutoExposure->Release();
}

IDisplayClusterViewportManagerPreview& FDisplayClusterViewportManager::GetViewportManagerPreview()
{
	return ViewportManagerPreview.Get();
}

const IDisplayClusterViewportManagerPreview& FDisplayClusterViewportManager::GetViewportManagerPreview() const
{
	return ViewportManagerPreview.Get();
}

const IDisplayClusterViewportManagerProxy* FDisplayClusterViewportManager::GetProxy() const
{
	return ViewportManagerProxy.Get();
}

IDisplayClusterViewportManagerProxy* FDisplayClusterViewportManager::GetProxy()
{
	return ViewportManagerProxy.Get();
}

void FDisplayClusterViewportManager::HandleStartScene()
{
	check(IsInGameThread());

	if (Configuration->IsSceneOpened())
	{
		// Do not start scene, if world not defined
		return;
	}

	UE_LOGF(LogDisplayClusterViewport, Log, "DCRA '%ls': Handling StartScene().", *Configuration->GetRootActorName());

	for (const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& ViewportIt : ImplGetEntireClusterViewports())
	{
		if (ViewportIt.IsValid())
		{
			ViewportIt->OnHandleStartScene();
		}
	}

	PostProcessManager->OnHandleStartScene();
	LightCardManager->OnHandleStartScene();
	Configuration->OnHandleStartScene();
}

void FDisplayClusterViewportManager::HandleEndScene()
{
	check(IsInGameThread());

	UE_LOGF(LogDisplayClusterViewport, Log, "DCRA '%ls': Handling EndScene().", *Configuration->GetRootActorName());

	// Release preview from prev scene
	ViewportManagerPreview->Release();

	for (const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& ViewportIt : ImplGetEntireClusterViewports())
	{
		if (ViewportIt.IsValid())
		{
			ViewportIt->OnHandleEndScene();
		}
	}

	PostProcessManager->OnHandleEndScene();
	LightCardManager->OnHandleEndScene();

	Configuration->OnHandleEndScene();
}

void FDisplayClusterViewportManager::HandleViewportRTTChanges(const TArray<FDisplayClusterViewport_Context>& InPrevContexts, const TArray<FDisplayClusterViewport_Context>& InContexts)
{
	// Support for resetting RTT size when viewport size is changed
	if (InPrevContexts.Num() != InContexts.Num())
	{
		// Reset scene RTT size when viewport disabled
		ResetSceneRenderTargetSize();
	}
	else
	{
		for (int32 ContextIt = 0; ContextIt < InContexts.Num(); ContextIt++)
		{
			if (InContexts[ContextIt].RenderTargetRect.Size() != InPrevContexts[ContextIt].RenderTargetRect.Size())
			{
				ResetSceneRenderTargetSize();
				break;
			}
		}
	}
}

void FDisplayClusterViewportManager::ResetSceneRenderTargetSize()
{
	SceneRenderTargetResizeMethod = ESceneRenderTargetResizeMethod::Reset;
}

void FDisplayClusterViewportManager::UpdateSceneRenderTargetSize()
{
	if (SceneRenderTargetResizeMethod != ESceneRenderTargetResizeMethod::None)
	{
		IConsoleVariable* const RTResizeCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.SceneRenderTargetResizeMethod"));
		if (RTResizeCVar)
		{
			switch (SceneRenderTargetResizeMethod)
			{
			case ESceneRenderTargetResizeMethod::Reset:
				// Resize to match requested render size (Default) (Least memory use, can cause stalls when size changes e.g. ScreenPercentage)
				RTResizeCVar->Set(0);

				// Wait for frame history is done
				// static const uint32 FSceneRenderTargets::FrameSizeHistoryCount = 3;
				FrameHistoryCounter = 3;
				SceneRenderTargetResizeMethod = ESceneRenderTargetResizeMethod::WaitFrameSizeHistory;
				break;

			case ESceneRenderTargetResizeMethod::WaitFrameSizeHistory:
				if (FrameHistoryCounter-- < 0)
				{
					SceneRenderTargetResizeMethod = ESceneRenderTargetResizeMethod::Restore;
				}
				break;

			case ESceneRenderTargetResizeMethod::Restore:
				// Expands to encompass the largest requested render dimension. (Most memory use, least prone to allocation stalls.)
				RTResizeCVar->Set(2);
				SceneRenderTargetResizeMethod = ESceneRenderTargetResizeMethod::None;
				break;

			default:
				break;
			}
		}
	}
}

bool FDisplayClusterViewportManager::ShouldUseFullSizeFrameTargetableResource() const
{
	check(IsInGameThread());

	// Postprocess requests an full-size FrameTargetable resource via
	// FDisplayClusterRenderFrameSettings::bUseFullSizeFrameTargetableForPostprocess
	// Note: FDisplayClusterViewport::ShouldUseFullSizeFrameTargetableResource() relies on this flag.

	for (const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& Viewport : ImplGetCurrentRenderFrameViewports())
	{
		if (Viewport.IsValid() && Viewport->ShouldUseFullSizeFrameTargetableResource())
		{
			return true;
		}
	}

	return false;
}

bool FDisplayClusterViewportManager::ShouldUseOutputFrameTargetableResources() const
{
	check(IsInGameThread());

	switch (Configuration->GetRenderFrameSettings().RenderMode)
	{
	case EDisplayClusterRenderFrameMode::PreviewInScene:
	case EDisplayClusterRenderFrameMode::PreviewProxyHitInScene:
		// Preview uses its own RTTs for each viewport.
		return false;

	default:
		break;
	}

	for (const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& Viewport : ImplGetCurrentRenderFrameViewports())
	{
		if (Viewport.IsValid() && Viewport->IsResourceUsed(EDisplayClusterViewportResource::OutputFrameTargetableResources))
		{
			return true;
		}
	}

	return false;
}

bool FDisplayClusterViewportManager::ShouldUseAdditionalFrameTargetableResource() const
{
	check(IsInGameThread());

	// AdditionalFrameTargetable is a subresource of FrameTargetable.
	if(!ShouldUseOutputFrameTargetableResources())
	{
		return false;
	}

	// Postprocess requests an AdditionalFrameTargetable resource via
	// FDisplayClusterRenderFrameSettings::bUseAdditionalFrameTargetableForPostprocess
	// Note: FDisplayClusterViewport::IsResourceUsedImpl() relies on this flag.

	for (const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& Viewport : ImplGetCurrentRenderFrameViewports())
	{
		if (Viewport.IsValid() && Viewport->IsResourceUsed(EDisplayClusterViewportResource::AdditionalFrameTargetableResources))
		{
			return true;
		}
	}

	return false;
}

void FDisplayClusterViewportManager::UpdateCurrentRenderFrameViewports() const
{
	TArray<TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>>& OutViewports = (TArray<TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>>&)CurrentRenderFrameViewports;

	// Get the viewports from the current cluster node.
	OutViewports = ImplGetEntireClusterViewports().FilterByPredicate([InClusterNodeId = Configuration->GetClusterNodeId()](const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& InViewport)
		{
			return InViewport.IsValid() && (InClusterNodeId.IsEmpty() || InViewport->GetClusterNodeId() == InClusterNodeId);
		});

	// Sort viewports by priority
	OutViewports.Sort([](const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& InViewport1, const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& InViewport2)
		{
			return InViewport1->GetPriority() < InViewport2->GetPriority();
		});

	// Clears the stale flag, marking the current frame’s viewport list as up to date.
	Configuration->ClearCurrentRenderFrameViewportsOutOfDate();
}

void FDisplayClusterViewportManager::RegisterCallbacks()
{
#if WITH_EDITOR
	PreGarbageCollectHandle = FCoreUObjectDelegates::GetPreGarbageCollectDelegate().AddRaw(this, &FDisplayClusterViewportManager::OnPreGarbageCollect);
#endif
}

void FDisplayClusterViewportManager::UnregisterCallbacks()
{
#if WITH_EDITOR
	FCoreUObjectDelegates::GetPreGarbageCollectDelegate().Remove(PreGarbageCollectHandle);
#endif
}

int32 FDisplayClusterViewportManager::GetViewportStereoViewIndexBase(const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& InViewport) const
{
	const int32 ViewPerViewportAmount = Configuration->GetRenderFrameSettings().GetViewPerViewportAmount();
	if (InViewport.IsValid() && ViewPerViewportAmount > 0)
	{
		const int32 ViewportIndex = ImplGetEntireClusterViewports().Find(InViewport);
		if (ViewportIndex != INDEX_NONE)
		{
			// Calculate the unique StereoViewIndex base for the viewport.
			const int32 ViewportStereoViewIndexBase =
				// Each StereoViewIndexBase has a distance to others equal to ViewPerViewportAmount.
				(ViewportIndex * ViewPerViewportAmount)
				//  Shifts to the nDisplay range; engine indices are below this.
				+ FDisplayClusterViewportStereoscopicPass::StereoViewIndexBase;

			return ViewportStereoViewIndexBase;
		}
	}

	return INDEX_NONE;
}

bool FDisplayClusterViewportManager::BeginNewFrame(FViewport* InViewport, FDisplayClusterRenderFrame& OutRenderFrame)
{
	check(IsInGameThread());
	if(!Configuration->IsSceneOpened())
	{
		// no world for render
		return false;
	}

	OutRenderFrame.ViewportManagerWeakPtr = this->AsShared();

	// Create DC ViewExtension to handle special features
	if (!ViewportManagerViewExtension.IsValid())
	{
		ViewportManagerViewExtension = FSceneViewExtensions::NewExtension<FDisplayClusterViewportManagerViewExtension>(Configuration);
	}

	// Create DC ViewPointExtension to handle special features
	if (!ViewportManagerViewPointExtension.IsValid())
	{
		ViewportManagerViewPointExtension = FSceneViewExtensions::NewExtension<FDisplayClusterViewportManagerViewPointExtension>(Configuration);
	}

	// Create DC FrameStatsViewExtension to handle special features
	if (!FrameStatsViewExtension.IsValid())
	{
		FrameStatsViewExtension = FSceneViewExtensions::NewExtension<FDisplayClusterViewportFrameStatsViewExtension>(Configuration);
	}

	// Before new frame
	PostProcessManager->HandleSetupNewFrame();

	// Initialize viewports from new render settings, and create new contexts, reset prev frame resources
	for (const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& ViewportIt : ImplGetCurrentRenderFrameViewports())
	{
		if (!ViewportIt.IsValid())
		{
			continue;
		}

		const int32 ViewportStereoViewIndexBase = GetViewportStereoViewIndexBase(ViewportIt);
		if (ViewportStereoViewIndexBase != INDEX_NONE)
		{
			// Save orig viewport contexts
			TArray<FDisplayClusterViewport_Context> PrevContexts;
			PrevContexts.Append(ViewportIt->GetContexts());

			if (ViewportIt->GetProjectionPolicy().IsValid())
			{
				ViewportIt->GetProjectionPolicy()->BeginUpdateFrameContexts(ViewportIt.Get());
			}

			ViewportIt->UpdateFrameContexts(ViewportStereoViewIndexBase);

			if (ViewportIt->GetProjectionPolicy().IsValid())
			{
				ViewportIt->GetProjectionPolicy()->EndUpdateFrameContexts(ViewportIt.Get());
			}

			HandleViewportRTTChanges(PrevContexts, ViewportIt->GetContexts());
		}
		else
		{
			// do not render this viewport
			ViewportIt->ResetFrameContexts();
		}
	}

	// Preliminary initialization of rendering thread data for viewport contexts
	// Currently, some data like EngineShowFlags and EngineGamma get updated on PostRenderViewFamily.
	// Since the viewports that have media input assigned never gets rendered in normal way, the PostRenderViewFamily
	// callback never gets called, therefore the data mentioned above never gets updated. This workaround initializes
	// those settings for all viewports. The viewports with no media input assigned will override the data
	// in PostRenderViewFamily like it was previously so nothing should be broken.
	const float DefaultDisplayGamma = GEngine ? GEngine->DisplayGamma : 2.2f;
	const FEngineShowFlags* const DefaultEngineShowFlags = (InViewport && InViewport->GetClient()) ? InViewport->GetClient()->GetEngineShowFlags() : nullptr;
	for (const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& ViewportIt : ImplGetCurrentRenderFrameViewports())
	{
		if (ViewportIt.IsValid())
		{
			TArray<FDisplayClusterViewport_Context> ViewportContexts = ViewportIt->GetContexts();

			for (FDisplayClusterViewport_Context& ContextIt : ViewportContexts)
			{
				// Get Context Display gamma
				const TSharedPtr<FDisplayClusterViewportResource, ESPMode::ThreadSafe> ContextRTT = ViewportIt->GetViewportResources(EDisplayClusterViewportResource::RenderTargets).IsValidIndex(ContextIt.ContextNum)
					? ViewportIt->GetViewportResources(EDisplayClusterViewportResource::RenderTargets)[ContextIt.ContextNum] : nullptr;
				const float ViewportDisplayGamma = ContextRTT.IsValid() ? ContextRTT->GetResourceSettings().GetDisplayGamma() : DefaultDisplayGamma;

				ContextIt.RenderThreadData.EngineDisplayGamma = ViewportDisplayGamma;


				if (DefaultEngineShowFlags && !EnumHasAnyFlags(ViewportIt->GetRenderSettingsICVFX().RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::UVLightcard))
				{
					// Todo: DefaultEngineShowFlags is not the correct value, since each viewport uses its own Engine flags.
					// UV Light cards should not be set to the default engine flags, since they are rendered using a custom render pass that does not utilize the flags
					ContextIt.RenderThreadData.EngineShowFlags = *DefaultEngineShowFlags;
				}
			}

			// Save changes
			ViewportIt->SetContexts(ViewportContexts);
		}
	}

	// Handle scene RTT resize
	UpdateSceneRenderTargetSize();

	// Build new frame structure
	if (!RenderFrameManager->BuildRenderFrame(InViewport, ImplGetCurrentRenderFrameViewports(), OutRenderFrame))
	{
		return false;
	}

	// Allocate resources for frame
	if (!RenderTargetManager->AllocateRenderFrameResources(InViewport, ImplGetCurrentRenderFrameViewports(), OutRenderFrame))
	{
		return false;
	}

	const FIntPoint RenderFrameSize = OutRenderFrame.FrameRect.Size();
	for (const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& Viewport : ImplGetCurrentRenderFrameViewports())
	{
		if (Viewport.IsValid())
		{
			Viewport->BeginNewFrame(RenderFrameSize);
		}
	}

	// Update desired views number
	OutRenderFrame.DesiredNumberOfViews = (ImplGetEntireClusterViewports().Num() * Configuration->GetRenderFrameSettings().GetViewPerViewportAmount()) + 1;

	PostProcessManager->HandleBeginNewFrame(OutRenderFrame);

	return true;
}

void FDisplayClusterViewportManager::InitializeNewFrame()
{
	check(IsInGameThread());

	// Prepares all registered upscaler modular features for the new frame.
	FDisplayClusterUpscaler::InitializeNewFrame();

	// Handle new frame for warp policies
	FDisplayClusterWarpPolicyManager WarpPolicyManager;
	WarpPolicyManager.HandleNewFrame(*this);

	// Send viewport manager data to rendering thread
	ViewportManagerProxy->ImplUpdateViewportManagerProxy_GameThread(*this);

	/**
	 * [1] Send new viewport render resources to proxy:
	 * At the moment, there are new resources in the viewport, but the math is not initialized.The math will be updated later in the game thread.
	 * But these new resources were needed for future rendering on the render thread.
	 * So we are sending viewports from the game stream data to the proxy.
	 * Lastly, after the viewport math data has been updated, we need to send it again.
	 */
	ViewportManagerProxy->ImplUpdateViewportProxies_GameThread(ImplGetCurrentRenderFrameViewports());

	// Send postprocess data to render thread
	// Update postprocess data from game thread
	PostProcessManager->Tick();

	// Send updated postprocess data to rendering thread
	PostProcessManager->FinalizeNewFrame();

	// Update AutoExposure data.
	AutoExposure->SyncAndComputeAutoExposure();
}

void FDisplayClusterViewportManager::FinalizeNewFrame()
{
	check(IsInGameThread());

	// [2] The viewport math has been updated, we need to send it to the proxy again.
	ViewportManagerProxy->ImplUpdateViewportProxies_GameThread(ImplGetCurrentRenderFrameViewports());

	for (const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& Viewport : ImplGetCurrentRenderFrameViewports())
	{
		if (Viewport.IsValid())
		{
			Viewport->FinalizeNewFrame();
		}
	}
}

void FDisplayClusterViewportManager::OnEntireClusterPreviewRendered()
{
	// Forward the "entire cluster preview rendered" event so AutoExposure can refresh its data.
	AutoExposure->OnEntireClusterPreviewRendered();
}

FSceneViewFamily::ConstructionValues FDisplayClusterViewportManager::CreateViewFamilyConstructionValues(
	const FDisplayClusterRenderFrameTarget& InFrameTarget,
	FSceneInterface* InScene,
	FEngineShowFlags InEngineShowFlags,
	const bool bInAdditionalViewFamily) const
{

	bool bResolveScene = true;

	// Sets the engine flags corresponding to the capture mode.
	IDisplayClusterViewportManager::SetupEngineShowFlags(InFrameTarget.CaptureMode, InEngineShowFlags);

	const FDisplayClusterRenderFrameSettings& RenderFrameSettings = Configuration->GetRenderFrameSettings();

	// A special case for DCRA previewing in a scene to avoid double use of PP.
	if(RenderFrameSettings.IsPostProcessDisabled())
	{
		// Disable postprocess for preview
		InEngineShowFlags.PostProcessing = 0;
	}

	// A special use case is setting up alpha channel capture:
	// (When using the DC viewport rendering pipeline).
	switch (InFrameTarget.CaptureMode)
	{
	case EDisplayClusterViewportCaptureMode::Chromakey:
	case EDisplayClusterViewportCaptureMode::Lightcard:

		// Disable postprocess for LightCard & CK
		InEngineShowFlags.PostProcessing = 0;

		switch (RenderFrameSettings.AlphaChannelCaptureMode)
		{
		case EDisplayClusterRenderFrameAlphaChannelCaptureMode::Copy:
			// Disable AA
			InEngineShowFlags.SetAntiAliasing(0);
			InEngineShowFlags.SetTemporalAA(0);
			break;

		case EDisplayClusterRenderFrameAlphaChannelCaptureMode::CopyAA:
			// Use AA
			InEngineShowFlags.SetAntiAliasing(1);
			InEngineShowFlags.SetTemporalAA(0);
			break;

		case EDisplayClusterRenderFrameAlphaChannelCaptureMode::FXAA:
			// Alpha captured without AA, own FXAA used
			InEngineShowFlags.SetAntiAliasing(0);
			InEngineShowFlags.SetTemporalAA(0);
			break;

		default:
			break;
		}
	default:
		break;
	}

	FRenderTarget* RenderTarget = InFrameTarget.RenderTargetResource.IsValid() ? InFrameTarget.RenderTargetResource->GetViewportResourceRenderTarget() : nullptr;
	return FSceneViewFamily::ConstructionValues(RenderTarget, InScene, InEngineShowFlags)
		.SetResolveScene(bResolveScene)
		.SetRealtimeUpdate(true)
		.SetAdditionalViewFamily(bInAdditionalViewFamily);
}

bool FDisplayClusterViewportManager::ShouldRenderFinalColor() const
{
	// Do not render final color, if postprocess is disabled
	return !Configuration->GetRenderFrameSettings().IsPostProcessDisabled();
}

void FDisplayClusterViewportManager::ConfigureViewFamily(
	const FDisplayClusterRenderFrameTarget& InFrameTarget,
	const FDisplayClusterRenderFrameTargetViewFamily& InFrameViewFamily,
	FSceneViewFamilyContext& ViewFamily)
{
	// Note: EngineShowFlags should have already been configured in CreateViewFamilyConstructionValues.
	ViewFamily.SceneCaptureCompositeMode = ESceneCaptureCompositeMode::SCCM_Overwrite;
	if (!ShouldRenderFinalColor() || InFrameTarget.CaptureMode == EDisplayClusterViewportCaptureMode::Lightcard)
	{
		ViewFamily.SceneCaptureSource = ESceneCaptureSource::SCS_SceneColorHDR;
	}

	// Gather Scene View Extensions
	// Note: Chromakey and Light maps use their own view extensions, collected in FDisplayClusterViewport::GatherActiveExtensions().
	ViewFamily.ViewExtensions = InFrameViewFamily.ViewExtensions;

	// Scene View Extension activation with ViewportId granularity only works if you have one ViewFamily per ViewportId
	for (FSceneViewExtensionRef& ViewExt : ViewFamily.ViewExtensions)
	{
		ViewExt->SetupViewFamily(ViewFamily);
	}
}

void FDisplayClusterViewportManager::PostConfigureViewFamily(
	const FDisplayClusterRenderFrameTarget& InFrameTarget,
	const FDisplayClusterRenderFrameTargetViewFamily& InFrameViewFamily,
	FSceneViewFamilyContext& ViewFamily,
	const TArray<FSceneView*>& Views,
	const EDisplayClusterViewportRenderingFlags InRenderingFlags,
	const float InDPIScale)
{
	// Configure upscaler
	const FName CustomUpscalerName = FDisplayClusterUpscaler::PostConfigureViewFamily(InFrameViewFamily.UpscalerSettings, InFrameViewFamily.CustomBufferRatio, InDPIScale, ViewFamily, Views);

	// Add ViewExtensions from the Custom Upscaler to this view family
	if (!CustomUpscalerName.IsNone() && InFrameViewFamily.UpscalerSettings.CustomUpscalerName == CustomUpscalerName)
	{
		// Iterate over all viewports, and collect all used ViewExtensions
		for (const FDisplayClusterRenderFrameTargetView& FrameTargetViewIt : InFrameViewFamily.Views)
		{
			if (FrameTargetViewIt.Viewport.IsValid())
			{
				TArray<FSceneViewExtensionRef> ViewExtensions = FrameTargetViewIt.Viewport->ToSharedRef()->GatherActiveExtensions(FrameTargetViewIt.ContextNum, InFrameTarget.Viewport);

				// Lookup for the new ViewExtensions, and register it for that ViewFamily
				for (FSceneViewExtensionRef& ViewExtensionIt : ViewExtensions)
				{
					if (!ViewFamily.ViewExtensions.Contains(ViewExtensionIt))
					{
						ViewFamily.ViewExtensions.Add(ViewExtensionIt);
						ViewExtensionIt->SetupViewFamily(ViewFamily);
					}
				}
			}
		}
	}
}

void FDisplayClusterViewportManager::RenderFrame(FViewport* InViewport)
{
	LightCardManager->RenderFrame();

	ViewportManagerProxy->ImplRenderFrame_GameThread(InViewport);
}

FDisplayClusterViewport* FDisplayClusterViewportManager::CreateViewport(const FString& InViewportBaseId, const class UDisplayClusterConfigurationViewport& ConfigurationViewport)
{
	check(IsInGameThread());

	// Check viewport ID
	if (InViewportBaseId.IsEmpty())
	{
		UE_LOGF(LogDisplayClusterViewport, Warning, "Wrong viewport ID");
		return nullptr;
	}

	// ID must be unique
	if (FindViewport(InViewportBaseId) != nullptr)
	{
		UE_LOGF(LogDisplayClusterViewport, Warning, "Viewport '%ls' already exists", *InViewportBaseId);

		return nullptr;
	}

	// Create projection policy for viewport
	const TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe> NewProjectionPolicy = CreateProjectionPolicy(InViewportBaseId, &ConfigurationViewport.ProjectionPolicy);
	if (NewProjectionPolicy.IsValid())
	{
		// Create viewport for new projection policy
		TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe> NewViewport = ImplCreateViewport(InViewportBaseId, NewProjectionPolicy);
		if (NewViewport.IsValid())
		{
			FDisplayClusterViewportConfiguration_Viewport::UpdateViewportConfiguration(*NewViewport , ConfigurationViewport);

			return NewViewport.Get();
		}
	}

	UE_LOGF(LogDisplayClusterViewport, Error, "Viewports '%ls' not created.", *InViewportBaseId);

	return nullptr;
}

IDisplayClusterViewport* FDisplayClusterViewportManager::FindViewport(const FString& InViewportId) const
{
	TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe> ExistViewport = ImplFindViewport(InViewportId);
	return ExistViewport.IsValid() ? ExistViewport.Get() : nullptr;
}

TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe> FDisplayClusterViewportManager::ImplFindViewport(
	const FString& InViewportBaseId,
	const EDisplayClusterViewportFlags InViewportFlags,
	const FIntPoint* InOptTilePos) const
{
	check(IsInGameThread());

	// Generate unique viewport name
	const FString ViewportId = Configuration->BuildViewportName(InViewportBaseId, InViewportFlags, InOptTilePos);

	// Ok, we have a request for a particular viewport. Let's find it.
	TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe> const* DesiredViewport = ImplGetEntireClusterViewports().FindByPredicate(
		[ViewportId](const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& ItemViewport)
	{
		return ItemViewport.IsValid() && ViewportId.Equals(ItemViewport->ViewportId, ESearchCase::IgnoreCase);
	});

	return (DesiredViewport && DesiredViewport->IsValid()) ? *DesiredViewport : nullptr;
}

FDisplayClusterViewport* FDisplayClusterViewportManager::CreateViewport(const FString& InViewportBaseId, const TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe>& InProjectionPolicy)
{
	check(IsInGameThread());

	TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe> ExistViewport = ImplFindViewport(InViewportBaseId);
	if (ExistViewport.IsValid())
	{
		//add error log: Viewport with name '%s' already exist
		return nullptr;
	}

	TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe> NewViewport = ImplCreateViewport(InViewportBaseId, InProjectionPolicy);

	return NewViewport.IsValid() ? NewViewport.Get() : nullptr;
}

bool FDisplayClusterViewportManager::DeleteViewport(const FString& ViewportId)
{
	check(IsInGameThread());

	TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe> ExistViewport = ImplFindViewport(ViewportId);
	if (ExistViewport.IsValid())
	{
		ImplDeleteViewport(ExistViewport);

		return true;
	}

	return false;
}

TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe> FDisplayClusterViewportManager::ImplCreateViewport(
	const FString& InViewportBaseId,
	const TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe>& InProjectionPolicy,
	const EDisplayClusterViewportFlags InViewportFlags,
	const FIntPoint* InOptTilePos)
{
	check(IsInGameThread());
	check(InProjectionPolicy.IsValid());

	// Create a new viewport for cluster node used for rendering
	TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe> NewViewport = MakeShared<FDisplayClusterViewport>(
		Configuration, InViewportBaseId, InViewportFlags, InProjectionPolicy, InOptTilePos);
	if (NewViewport.IsValid())
	{
		NewViewport->Initialize();

		// Add viewport on gamethread
		EntireClusterViewports.Add(NewViewport);
		
		// Raises the stale flag, marking the current frame's viewport list as needing an update.
		Configuration->MarkCurrentRenderFrameViewportsOutOfDate();

		if (Configuration->IsSceneOpened())
		{
			// Handle start scene for viewport
			NewViewport->OnHandleStartScene();
		}
	}

	return NewViewport;
}

void FDisplayClusterViewportManager::ImplDeleteViewport(const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& ExistViewport)
{
	check(ExistViewport.IsValid());

	// Handle projection policy event
	ExistViewport->ReleaseProjectionPolicy();

	// Remove viewport from the entire viewports list
	int32 ViewportIndex = EntireClusterViewports.Find(ExistViewport);
	if (ViewportIndex != INDEX_NONE)
	{
		EntireClusterViewports[ViewportIndex] = nullptr;
		EntireClusterViewports.RemoveAt(ViewportIndex);
	}

	// Raises the stale flag, marking the current frame's viewport list as needing an update.
	Configuration->MarkCurrentRenderFrameViewportsOutOfDate();

	// Reset RTT size after viewport delete
	ResetSceneRenderTargetSize();
}

FDisplayClusterViewport* FDisplayClusterViewportManager::ImplFindViewport(const int32 StereoViewIndex, uint32* OutContextNum) const
{
	check(IsInGameThread());

	if (!FDisplayClusterViewportStereoscopicPass::IsValidStereoViewIndex(StereoViewIndex))
	{
		return nullptr;
	}

	for (const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& ViewportIt : ImplGetEntireClusterViewports())
	{
		if (ViewportIt.IsValid() && ViewportIt->FindContext(StereoViewIndex, OutContextNum))
		{
			return ViewportIt.Get();
		}
	}

	// Viewport not found
	return nullptr;
}

TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe> FDisplayClusterViewportManager::CreateProjectionPolicy(const FString& InViewportId, const FDisplayClusterConfigurationProjection* InConfigurationProjectionPolicy)
{
	check(IsInGameThread());
	check(InConfigurationProjectionPolicy != nullptr);

	// Generate unique projection policy id from viewport name
	const FString ProjectionPolicyId = FString::Printf(TEXT("prj_%s"), *InViewportId);

	IDisplayClusterRenderManager* const DCRenderManager = IDisplayCluster::Get().GetRenderMgr();
	check(DCRenderManager);

	TSharedPtr<IDisplayClusterProjectionPolicyFactory> ProjPolicyFactory = DCRenderManager->GetProjectionPolicyFactory(InConfigurationProjectionPolicy->Type);
	if (ProjPolicyFactory.IsValid())
	{
		TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe> ProjPolicy = ProjPolicyFactory->Create(ProjectionPolicyId, InConfigurationProjectionPolicy);
		if (ProjPolicy.IsValid())
		{
			return ProjPolicy;
		}
		else
		{
			UE_LOGF(LogDisplayClusterViewport, Warning, "Invalid projection policy: type '%ls', RHI '%ls', viewport '%ls'", *InConfigurationProjectionPolicy->Type, GDynamicRHI->GetName(), *ProjectionPolicyId);
		}
	}
	else
	{
		UE_LOGF(LogDisplayClusterViewport, Warning, "No projection factory found for projection type '%ls'", *InConfigurationProjectionPolicy->Type);
	}

	return nullptr;
}

TArray<TSharedPtr<IDisplayClusterViewport, ESPMode::ThreadSafe>> FDisplayClusterViewportManager::GetEntireClusterViewportsForWarpPolicy(const TSharedPtr<IDisplayClusterWarpPolicy>& InWarpPolicy) const
{
	TArray<TSharedPtr<IDisplayClusterViewport, ESPMode::ThreadSafe>> WarpPolicyViewports;

	for (const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& Viewport : ImplGetEntireClusterViewports())
	{
		if (Viewport.IsValid() && Viewport->GetProjectionPolicy().IsValid() && Viewport->GetProjectionPolicy()->GetWarpPolicy() == InWarpPolicy.Get())
		{
			WarpPolicyViewports.Add(Viewport);
		}
	}

	return WarpPolicyViewports;
}

void FDisplayClusterViewportManager::MarkComponentGeometryDirty(const FName InComponentName)
{
	check(IsInGameThread());

	// 1. Update all ProceduralMeshComponent references for projection policies
	// (We need to update for all cluster viewports)
	for (const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& ViewportIt : ImplGetEntireClusterViewports())
	{
		if (ViewportIt.IsValid())
		{
			const TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe>& ProjectionPolicy = ViewportIt->GetProjectionPolicy();
			if (ProjectionPolicy.IsValid())
			{
				TSharedPtr<IDisplayClusterWarpBlend, ESPMode::ThreadSafe> WarpBlendInterface;
				if (ProjectionPolicy->GetWarpBlendInterface(WarpBlendInterface) && WarpBlendInterface.IsValid())
				{
					// Update only interfaces with ProceduralMesh as geometry source
					if (WarpBlendInterface->GetWarpGeometryType() == EDisplayClusterWarpGeometryType::WarpProceduralMesh)
					{
						// Set the ProceduralMeshComponent geometry dirty for all valid WarpBlendInterface
						WarpBlendInterface->MarkWarpGeometryComponentDirty(InComponentName);
					}
				}
			}
		}
	}

	// 2. Update all ProceduralMeshComponent references for OutputRemap
	PostProcessManager->GetOutputRemap()->MarkProceduralMeshComponentGeometryDirty(InComponentName);
}

void FDisplayClusterViewportManager::AddReferencedObjects(FReferenceCollector& Collector)
{
	for (const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& ViewportIt : ImplGetEntireClusterViewports())
	{
		if (ViewportIt.IsValid())
		{
			ViewportIt->AddReferencedObjects(Collector);
		}
	}
}

void FDisplayClusterViewportManager::SetCurrentStereoViewIndexForViewPointExtension(const int32 InStereoViewIndex) const
{
	if (ViewportManagerViewPointExtension.IsValid())
	{
		ViewportManagerViewPointExtension->SetCurrentStereoViewIndex(InStereoViewIndex);
	}
}

FSceneView* FDisplayClusterViewportManager::CalcSceneView(ULocalPlayer* LocalPlayer, FSceneViewFamily* ViewFamily, FVector& OutViewLocation, FRotator& OutViewRotation, FViewport* InViewport, FViewElementDrawer* InViewDrawer, int32 InStereoViewIndex)
{
	check(ViewportManagerViewPointExtension.IsValid());
	check(LocalPlayer);

	// Set current view index for this VE:
	// This function must be used because LocalPlayer::GetViewPoint() calls the ISceneViewExtension::SetupViewPoint() function from this VE.
	// And at this point the VE must know the current StereoViewIndex value in order to understand which viewport will be used for this SetupViewPoint() call.
	SetCurrentStereoViewIndexForViewPointExtension(InStereoViewIndex);

	// Calculate view inside LocalPlayer: VE used to override view point for this StereoViewIndex
	FSceneView* View = LocalPlayer->CalcSceneView(ViewFamily, OutViewLocation, OutViewRotation, InViewport, InViewDrawer, InStereoViewIndex);

	// Set INDEX_NONE when this VE no longer needs to be used.
	SetCurrentStereoViewIndexForViewPointExtension(INDEX_NONE);

	return View;
}

void FDisplayClusterViewportManager::OnPreGarbageCollect()
{
	// The view state can reference materials from the world being cleaned up. (Example post process materials)
	for (const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& ViewportIt : ImplGetEntireClusterViewports())
	{
		if (ViewportIt.IsValid())
		{
			ViewportIt->CleanupViewState();
		}
	}
}