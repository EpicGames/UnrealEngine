// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "SceneTypes.h"
#include "Templates/SharedPointer.h"

#include "Misc/DisplayClusterObjectRef.h"

#include "Render/Viewport/IDisplayClusterViewportManager.h"
#include "Render/Viewport/DisplayClusterViewport.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_Enums.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_InternalEnums.h"

class FDisplayClusterViewportManagerProxy;
class IDisplayClusterProjectionPolicy;
class  UDisplayClusterConfigurationViewport;
struct FDisplayClusterConfigurationProjection;
class FViewport;

/**
 * Implementation of the nDisplay IDisplayClusterViewportManager
 */
class FDisplayClusterViewportManager
	: public IDisplayClusterViewportManager
	, public TSharedFromThis<FDisplayClusterViewportManager, ESPMode::ThreadSafe>
{
public:
	FDisplayClusterViewportManager();
	virtual ~FDisplayClusterViewportManager();

public:
	/** Game thread funcs */
	void HandleStartScene();
	void HandleEndScene();

	void Initialize();

	/** Release all resources.
	* After that, the BeginNewFrame() function should be called for a new resource allocation.
	*
	* Note: This function has been renamed; its previous name was 'ReleaseTextures'.
	*/
	void ReleaseAllResources();

	//~ Begin IDisplayClusterViewportManager

	virtual void Release() override;

	virtual TSharedPtr<IDisplayClusterViewportManager, ESPMode::ThreadSafe> ToSharedPtr() override
	{
		return AsShared();
	}

	virtual TSharedPtr<const IDisplayClusterViewportManager, ESPMode::ThreadSafe> ToSharedPtr() const override
	{
		return AsShared();
	}

	virtual TSharedRef<FDisplayClusterViewportManager, ESPMode::ThreadSafe> ToSharedRef() override
	{
		return AsShared();
	}

	virtual TSharedRef<const FDisplayClusterViewportManager, ESPMode::ThreadSafe> ToSharedRef() const override
	{
		return AsShared();
	}

	/** Get viewport manager preview API */
	virtual IDisplayClusterViewportManagerPreview& GetViewportManagerPreview() override;
	virtual const IDisplayClusterViewportManagerPreview& GetViewportManagerPreview() const override;


	virtual const IDisplayClusterViewportManagerProxy* GetProxy() const override;
	virtual       IDisplayClusterViewportManagerProxy* GetProxy() override;

	virtual IDisplayClusterViewportConfiguration& GetConfiguration() override
	{
		return Configuration.Get();
	}
	virtual const IDisplayClusterViewportConfiguration& GetConfiguration() const override
	{
		return Configuration.Get();
	}

	virtual const TSharedRef<IDisplayClusterViewportConfiguration, ESPMode::ThreadSafe> GetConfigurationRef() override
	{
		return Configuration;
	}
	virtual const TSharedRef<const IDisplayClusterViewportConfiguration, ESPMode::ThreadSafe> GetConfigurationRef() const override
	{
		return Configuration;
	}

	virtual bool BeginNewFrame(FViewport* InViewport, FDisplayClusterRenderFrame& OutRenderFrame) override;
	virtual void InitializeNewFrame() override;
	virtual void FinalizeNewFrame() override;

	virtual FSceneViewFamily::ConstructionValues CreateViewFamilyConstructionValues(
		const FDisplayClusterRenderFrameTarget& InFrameTarget,
		FSceneInterface* InScene,
		FEngineShowFlags InEngineShowFlags,
		const bool bInAdditionalViewFamily
	) const override;

	virtual void ConfigureViewFamily(
		const FDisplayClusterRenderFrameTarget& InFrameTarget,
		const FDisplayClusterRenderFrameTargetViewFamily& InFrameViewFamily,
		FSceneViewFamilyContext& InOutViewFamily
	) override;

	virtual void PostConfigureViewFamily(
		const FDisplayClusterRenderFrameTarget& InFrameTarget,
		const FDisplayClusterRenderFrameTargetViewFamily& InFrameViewFamily,
		FSceneViewFamilyContext& InOutViewFamily,
		const TArray<FSceneView*>& InOutViews,
		const EDisplayClusterViewportRenderingFlags InRenderingFlags = EDisplayClusterViewportRenderingFlags::None,
		const float InDPIScale = 1.f
	) override;

	virtual void RenderFrame(FViewport* InViewport) override;

private:
	/** Called before garbage collection is run */
	void OnPreGarbageCollect();

public:
	virtual IDisplayClusterViewport* FindViewport(const FString& InViewportId) const override;
	virtual IDisplayClusterViewport* FindViewport(const int32 ViewIndex, uint32* OutContextNum = nullptr) const override
	{
		return ImplFindViewport(ViewIndex, OutContextNum);
	}

	virtual const TArrayView<TSharedPtr<IDisplayClusterViewport, ESPMode::ThreadSafe>> GetCurrentRenderFrameViewports() const override
	{
		const TArray<TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>>& OutViewports = ImplGetCurrentRenderFrameViewports();
		return TArrayView<TSharedPtr<IDisplayClusterViewport, ESPMode::ThreadSafe>>((TSharedPtr<IDisplayClusterViewport, ESPMode::ThreadSafe>*)(OutViewports.GetData()), OutViewports.Num());
	}

	virtual const TArrayView<TSharedPtr<IDisplayClusterViewport, ESPMode::ThreadSafe>> GetEntireClusterViewports() const override
	{
		return TArrayView<TSharedPtr<IDisplayClusterViewport, ESPMode::ThreadSafe>>((TSharedPtr<IDisplayClusterViewport, ESPMode::ThreadSafe>*)(EntireClusterViewports.GetData()), EntireClusterViewports.Num());
	}

	virtual TArray<TSharedPtr<IDisplayClusterViewport, ESPMode::ThreadSafe>> GetEntireClusterViewportsForWarpPolicy(const TSharedPtr<IDisplayClusterWarpPolicy>& InWarpPolicy) const override;

	virtual void MarkComponentGeometryDirty(const FName InComponentName = NAME_None) override;
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;

	virtual class FSceneView* CalcSceneView(class ULocalPlayer* LocalPlayer, class FSceneViewFamily* ViewFamily, FVector& OutViewLocation, FRotator& OutViewRotation, class FViewport* Viewport, class FViewElementDrawer* ViewDrawer, int32 StereoViewIndex) override;

	//~~ End IDisplayClusterViewportManager

	/** Getting the viewports of the current rendering frame (viewports from the current node or from a special named list). */
	inline const TArray<TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>>& ImplGetCurrentRenderFrameViewports() const
	{
		// Updates the current frame’s viewport list if it is out of date and requires refreshing.
		if (Configuration->IsCurrentRenderFrameViewportsOutOfDate())
		{
			UpdateCurrentRenderFrameViewports();
		}

		return CurrentRenderFrameViewports;
	}

	inline const TArray<TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>>& ImplGetEntireClusterViewports() const
	{
		return EntireClusterViewports;
	}

	// internal use only
	FDisplayClusterViewport* CreateViewport(const FString& InViewportBaseId, const class UDisplayClusterConfigurationViewport& ConfigurationViewport);
	FDisplayClusterViewport* CreateViewport(const FString& InViewportBaseId, const TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe>& InProjectionPolicy);
	bool                     DeleteViewport(const FString& ViewportId);

	FDisplayClusterViewport* ImplFindViewport(const int32 ViewIndex, uint32* OutContextNum = nullptr) const;
	/**
	* Creates and initializes a new nDisplay viewport instance.
	*
	* @param InViewportBaseId    The base viewport identifier (before ICVFX unique naming).
	* @param InProjectionPolicy  Projection policy assigned to the viewport
	*                            (defines how the scene is projected/rendered).
	* @param InViewportFlags     Usage flags of the viewport (e.g. InCamera/LightCard/Tile, etc.).
	* @param InOptTilePos        Optional parameter for tile viewport position
	*
	* @return A thread-safe shared pointer to the created FDisplayClusterViewport.
	*/
	TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe> ImplCreateViewport(
		const FString& InViewportBaseId,
		const TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe>& InProjectionPolicy,
		const EDisplayClusterViewportFlags InViewportFlags = EDisplayClusterViewportFlags::None,
		const FIntPoint* InOptTilePos = nullptr);

	/**
	* Finds a viewport instance by identifier and usage type.
	*
	* @param InViewportBaseId The base viewport identifier (before ICVFX unique naming).
	* @param InViewportFlags  Usage flags of the viewport (e.g. InCamera/LightCard/Tile, etc.).
	* @param InOptTilePos     Optional parameter for tile viewport position
	*
	* @return A thread-safe shared pointer to the matching FDisplayClusterViewport,
	*         or nullptr if no viewport matches the given criteria.
	*/
	TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe> ImplFindViewport(
		const FString& InViewportBaseId,
		const EDisplayClusterViewportFlags InViewportFlags = EDisplayClusterViewportFlags::None,
		const FIntPoint* InOptTilePos = nullptr) const;

	void ImplDeleteViewport(const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& InExistViewport);

	static TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe> CreateProjectionPolicy(
		const FString& InViewportId,
		const FDisplayClusterConfigurationProjection* InConfigurationProjectionPolicy);

	/** Get DC viewport manager proxy. */
	TSharedPtr<FDisplayClusterViewportManagerProxy, ESPMode::ThreadSafe> GetViewportManagerProxy() const
	{
		return ViewportManagerProxy;
	}

	/** 
	* Gets the unique StereoViewIndex base for the viewport.
	* Each StereoViewIndexBase has a distance to others equal to ViewPerViewportAmount.
	* 
	* Note: This function has been renamed. Its previous name was FindFirstViewportStereoViewIndex().
	*
	* @param InViewport - viewport asked for a StereoViewIndex base
	* @return StereoViewIndex base or INDEX_NONE, if the viewport cannot be rendered.
	*/
	int32 GetViewportStereoViewIndexBase(const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& InViewport) const;

	/** Returns true if the final color is renderable (to support preview in scene). */
	bool ShouldRenderFinalColor() const;

	/** Return nDisplay VE object. */
	const TSharedPtr<class FDisplayClusterViewportManagerViewExtension, ESPMode::ThreadSafe>& GetViewportManagerViewExtension() const
	{ return ViewportManagerViewExtension; }

	/** Return ViewPoint VE instance. */
	const TSharedPtr<class FDisplayClusterViewportManagerViewPointExtension, ESPMode::ThreadSafe>& GetViewportManagerViewPointExtension() const
	{ return ViewportManagerViewPointExtension; }

	/** Set current view index for FDisplayClusterViewportManagerViewPointExtension */
	void SetCurrentStereoViewIndexForViewPointExtension(const int32 InStereoViewIndex) const;

	/** Returns true if at least one of the viewports are visible on the frame RTT. */
	bool ShouldUseOutputFrameTargetableResources() const;
	bool ShouldUseAdditionalFrameTargetableResource() const;
	bool ShouldUseFullSizeFrameTargetableResource() const;

	void ResetSceneRenderTargetSize();

	/** Called every time previews for all cluster nodes are rendered. */
	void OnEntireClusterPreviewRendered();

private:
	/** This function updates CurrentRenderFrameViewports according to the current rendering settings. */
	void UpdateCurrentRenderFrameViewports() const;

	void UpdateSceneRenderTargetSize();
	void HandleViewportRTTChanges(const TArray<FDisplayClusterViewport_Context>& PrevContexts, const TArray<FDisplayClusterViewport_Context>& Contexts);

	/** Register any callbacks */
	void RegisterCallbacks();
	
	/** Unregister any callbacks used */
	void UnregisterCallbacks();

public:
	// Configuration of the current cluster node
	const TSharedRef<class FDisplayClusterViewportConfiguration, ESPMode::ThreadSafe> Configuration;

	// Viewport preview
	const TSharedRef<class FDisplayClusterViewportManagerPreview, ESPMode::ThreadSafe> ViewportManagerPreview;

	// Resource manager
	const TSharedRef<class FDisplayClusterRenderTargetManager, ESPMode::ThreadSafe> RenderTargetManager;

	// Postprocess manager
	const TSharedRef<class FDisplayClusterViewportPostProcessManager, ESPMode::ThreadSafe> PostProcessManager;

	// LC manager
	const TSharedRef<class FDisplayClusterViewportLightCardManager, ESPMode::ThreadSafe> LightCardManager;

	// Manager for creating the render frame stucture
	const TSharedRef<class FDisplayClusterRenderFrameManager> RenderFrameManager;

	// nDisplay multi-viewport AutoExposure feature.
	const TSharedRef<class FDisplayClusterAutoExposure> AutoExposure;

private:
	// Render thread proxy manager. Deleted on render thread
	TSharedPtr<FDisplayClusterViewportManagerProxy, ESPMode::ThreadSafe> ViewportManagerProxy;

	// Viewports of the entire cluster
	TArray<TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>> EntireClusterViewports;

	// Viewports of the current render frame (viewports from the current node or from a special named list)
	TArray<TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>> CurrentRenderFrameViewports;

	/** Scene RTT resize method.*/
	enum class ESceneRenderTargetResizeMethod : uint8
	{
		None = 0,
		Reset,
		WaitFrameSizeHistory,
		Restore
	};

	// Support for resetting RTT size (GROW method always grows and does not recover FPS when the viewport size or buffer ratio is changed)
	ESceneRenderTargetResizeMethod SceneRenderTargetResizeMethod = ESceneRenderTargetResizeMethod::None;
	int32 FrameHistoryCounter = 0;

	// Handle special features
	TSharedPtr<class FDisplayClusterViewportManagerViewExtension, ESPMode::ThreadSafe> ViewportManagerViewExtension;

	// This VE is used for LocalPlayer::GetViewPoint(). It calls the ISceneViewExtension::SetupViewPoint() function of this VE.
	TSharedPtr<class FDisplayClusterViewportManagerViewPointExtension, ESPMode::ThreadSafe> ViewportManagerViewPointExtension;

	// This VE is used to display frame information such as timecode and frame number.
	TSharedPtr<class FDisplayClusterViewportFrameStatsViewExtension, ESPMode::ThreadSafe> FrameStatsViewExtension;

	// This is a flag to check in the destructor that Release() has been called before.
	bool bReleaseFunctionCalledBeforeDestructor = false;

#if WITH_EDITOR
	/** The handle for FCoreUObjectDelegates::GetPreGarbageCollectDelegate */
	FDelegateHandle PreGarbageCollectHandle;
#endif
};
