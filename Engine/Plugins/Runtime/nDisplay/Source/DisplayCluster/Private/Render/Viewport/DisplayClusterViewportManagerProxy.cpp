// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Viewport/DisplayClusterViewportManagerProxy.h"

#include "Render/Viewport/DisplayClusterViewportManager.h"
#include "Render/Viewport/DisplayClusterViewportHelpers.h"
#include "Render/Viewport/Configuration/DisplayClusterViewportConfiguration.h"
#include "Render/Viewport/Configuration/DisplayClusterViewportConfigurationProxy.h"

#include "Misc/DisplayClusterLog.h"

#include "IDisplayCluster.h"
#include "IDisplayClusterCallbacks.h"
#include "Render/IDisplayClusterRenderManager.h"

#include "Render/Features/AutoExposure/DisplayClusterAutoExposure.h"

#include "Render/Projection/IDisplayClusterProjectionPolicyFactory.h"
#include "Render/Projection/IDisplayClusterProjectionPolicy.h"

#include "Render/Viewport/DisplayClusterViewportManager.h"
#include "Render/Viewport/DisplayClusterViewportManagerViewExtension.h"
#include "Render/Viewport/DisplayClusterViewportProxy.h"

#include "Render/Viewport/Containers/DisplayClusterViewportProxyData.h"

#include "Render/Viewport/LightCard/DisplayClusterViewportLightCardManager.h"
#include "Render/Viewport/LightCard/DisplayClusterViewportLightCardManagerProxy.h"

#include "Render/Viewport/Postprocess/DisplayClusterViewportPostProcessManager.h"

#include "Render/Viewport/RenderTarget/DisplayClusterRenderTargetManager.h"
#include "Render/Viewport/RenderTarget/DisplayClusterRenderTargetResource.h"
#include "Render/Viewport/DisplayClusterViewportStereoscopicPass.h"

#include "Misc/DisplayClusterLog.h"

#include "RHIContext.h"

// Enable/Disable ClearTexture for Frame RTT
static TAutoConsoleVariable<int32> CVarClearFrameRTTEnabled(
	TEXT("nDisplay.render.ClearFrameRTTEnabled"),
	1,
	TEXT("Enables FrameRTT clearing before viewport resolving.\n")
	TEXT("0 : disabled\n")
	TEXT("1 : enabled\n")
	,
	ECVF_RenderThreadSafe
);

///////////////////////////////////////////////////////////////////////////////////////
//          FDisplayClusterViewportManagerProxy
///////////////////////////////////////////////////////////////////////////////////////
FDisplayClusterViewportManagerProxy::FDisplayClusterViewportManagerProxy(const FDisplayClusterViewportManager& InViewportManager)
	: ConfigurationProxy(InViewportManager.Configuration->Proxy)
	, RenderTargetManager(InViewportManager.RenderTargetManager)
	, PostProcessManager(InViewportManager.PostProcessManager)
	, LightCardManagerProxy(InViewportManager.LightCardManager->LightCardManagerProxy)
	, AutoExposure(InViewportManager.AutoExposure)
	, ViewportManagerViewExtension(InViewportManager.GetViewportManagerViewExtension())
{ }

FDisplayClusterViewportManagerProxy::~FDisplayClusterViewportManagerProxy()
{ }

void FDisplayClusterViewportManagerProxy::Release_RenderThread()
{
	check(IsInRenderingThread());

	// Delete viewport proxy objects
	EntireClusterViewportProxies.Empty();
	CurrentRenderFrameViewportProxies.Empty();

	if (ViewportManagerViewExtension.IsValid())
	{
		// Force release of VE data since this TSharedPtr<> may also be held by other resources at this time.
		// So Reset() doesn't actually release it right away in some situations.
		ViewportManagerViewExtension->Release_RenderThread();
		ViewportManagerViewExtension.Reset();
	}
}

void FDisplayClusterViewportManagerProxy::ImplUpdateClusterNodeViewportProxies_RenderThread()
{
	TArray<TSharedPtr<FDisplayClusterViewportProxy, ESPMode::ThreadSafe>>& OutViewports = (TArray<TSharedPtr<FDisplayClusterViewportProxy, ESPMode::ThreadSafe>>&)CurrentRenderFrameViewportProxies;

	if (ConfigurationProxy->GetClusterNodeId_RenderThread().IsEmpty())
	{
		// When a cluster node name is empty, we render without the cluster nodes.
		OutViewports = ImplGetEntireClusterViewportProxies_RenderThread();
	}
	else
	{
		// Get viewports for current cluster nodes.
		OutViewports.Reset();
		for (const TSharedPtr<FDisplayClusterViewportProxy, ESPMode::ThreadSafe>& ViewportProxyIt : ImplGetEntireClusterViewportProxies_RenderThread())
		{
			if (ViewportProxyIt.IsValid() && (ViewportProxyIt->GetClusterNodeId() == ConfigurationProxy->GetClusterNodeId_RenderThread()))
			{
				OutViewports.Add(ViewportProxyIt);
			}
		}
	}

	// Sort viewports by priority.
	OutViewports.Sort([](const TSharedPtr<FDisplayClusterViewportProxy, ESPMode::ThreadSafe>& InViewportProxy1, const TSharedPtr<FDisplayClusterViewportProxy, ESPMode::ThreadSafe>& InViewportProxy2)
		{
			return InViewportProxy1->GetPriority_RenderThread() < InViewportProxy2->GetPriority_RenderThread();
		});
}

void FDisplayClusterViewportManagerProxy::CreateViewport_RenderThread(const TSharedPtr<FDisplayClusterViewportProxy, ESPMode::ThreadSafe>& InViewportProxy)
{
	check(IsInRenderingThread());
	
	EntireClusterViewportProxies.Add(InViewportProxy);

	ImplUpdateClusterNodeViewportProxies_RenderThread();
}

void FDisplayClusterViewportManagerProxy::DeleteViewport_RenderThread(const TSharedPtr<FDisplayClusterViewportProxy, ESPMode::ThreadSafe>& InViewportProxy)
{
	check(IsInRenderingThread());

	// Remove viewport obj from manager
	const int32 ViewportProxyIndex = EntireClusterViewportProxies.Find(InViewportProxy);
	if (ViewportProxyIndex != INDEX_NONE)
	{
		EntireClusterViewportProxies.RemoveAt(ViewportProxyIndex);
	}

	const int32 ClusterViewportProxyIndex = CurrentRenderFrameViewportProxies.Find(InViewportProxy);
	if (ClusterViewportProxyIndex != INDEX_NONE)
	{
		CurrentRenderFrameViewportProxies.RemoveAt(ClusterViewportProxyIndex);
	}
}

DECLARE_GPU_STAT_NAMED(nDisplay_ViewportManager_UpdateViewportManagerProxy, TEXT("nDisplay ViewportManager::UpdateProxy"));

void FDisplayClusterViewportManagerProxy::ImplUpdateViewportManagerProxy_GameThread(const FDisplayClusterViewportManager& InViewportManager)
{
	check(IsInGameThread());

	// Update configuration proxy:
	ConfigurationProxy->UpdateConfigurationProxy_GameThread(*InViewportManager.Configuration);

	// Send viewport manager data to proxy
	ENQUEUE_RENDER_COMMAND(DisplayClusterUpdateViewportManagerProxy)(
		[ ViewportManagerProxy = SharedThis(this)
		, ViewExtension = InViewportManager.GetViewportManagerViewExtension()
		](FRHICommandListImmediate& RHICmdList)
	{
		RHI_BREADCRUMB_EVENT_STAT(RHICmdList, nDisplay_ViewportManager_UpdateViewportManagerProxy, "nDisplay_ViewportManager_UpdateViewportManagerProxy");

		ViewportManagerProxy->ViewportManagerViewExtension = ViewExtension;

		// After updated settings we need update cluster node viewports
		ViewportManagerProxy->ImplUpdateClusterNodeViewportProxies_RenderThread();
	});
}

DECLARE_GPU_STAT_NAMED(nDisplay_ViewportManager_UpdateViewports, TEXT("nDisplay ViewportManager::UpdateViewports"));

void FDisplayClusterViewportManagerProxy::ImplUpdateViewportProxies_GameThread(const TArray<TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>>& InCurrentRenderFrameViewports)
{
	check(IsInGameThread());

	TArray<FDisplayClusterViewportProxyData*> ViewportProxiesData;
	for (const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& ViewportIt : InCurrentRenderFrameViewports)
	{
		if(FDisplayClusterViewportProxyData* NewData = ViewportIt.IsValid() ? ViewportIt->CreateViewportProxyData() : nullptr)
		{
			ViewportProxiesData.Add(NewData);
		}
	}

	// Send viewports settings to renderthread
	ENQUEUE_RENDER_COMMAND(DisplayClusterUpdateViewports)(
		[ProxiesData = std::move(ViewportProxiesData)](FRHICommandListImmediate& RHICmdList)
	{
		RHI_BREADCRUMB_EVENT_STAT(RHICmdList, nDisplay_ViewportManager_UpdateViewports, "nDisplay_ViewportManager_UpdateViewports");

		// Update game on rendering thread:
		for (FDisplayClusterViewportProxyData* ProxyDataIt : ProxiesData)
		{
			ProxyDataIt->DestViewportProxy->UpdateViewportProxyData_RenderThread(*ProxyDataIt);
			delete ProxyDataIt;
		}
	});
}

DECLARE_GPU_STAT_NAMED(nDisplay_ViewportManager_RenderFrame, TEXT("nDisplay ViewportManager::RenderFrame"));
DECLARE_GPU_STAT_NAMED(nDisplay_ViewportManager_CrossGPUTransfer, TEXT("nDisplay ViewportManager::CrossGPUTransfer"));
DECLARE_GPU_STAT_NAMED(nDisplay_ViewportManager_UpdateDeferredResources, TEXT("nDisplay ViewportManager::UpdateDeferredResources"));
DECLARE_GPU_STAT_NAMED(nDisplay_ViewportManager_WarpBlend, TEXT("nDisplay ViewportManager::WarpBlend"));

void FDisplayClusterViewportManagerProxy::ImplRenderFrame_GameThread(FViewport* InViewport)
{
	ENQUEUE_RENDER_COMMAND(DisplayClusterRenderFrame_Setup)(
		[InViewportManagerProxy = SharedThis(this)](FRHICommandListImmediate& RHICmdList)
	{
		RHI_BREADCRUMB_EVENT_STAT(RHICmdList, nDisplay_ViewportManager_RenderFrame, "nDisplay_ViewportManager_RenderFrame");

		const FDisplayClusterViewportManagerProxy* ViewportManagerProxy = &InViewportManagerProxy.Get();

		// Handle render setup
		ViewportManagerProxy->PostProcessManager->HandleRenderFrameSetup_RenderThread(RHICmdList, ViewportManagerProxy);

		// Collects AutoExposure (Eye Adaptation) data from the viewports
		ViewportManagerProxy->AutoExposure->GatherMeterViewportsAutoExposureDataAndSync_RenderThread(
			ViewportManagerProxy->ImplGetCurrentRenderFrameViewportProxies_RenderThread());
	});

	ENQUEUE_RENDER_COMMAND(DisplayClusterRenderFrame_CrossGPUTransfer)(
		[InViewportManagerProxy = SharedThis(this), OutputViewport = InViewport](FRHICommandListImmediate& RHICmdList)
	{
		RHI_BREADCRUMB_EVENT_STAT(RHICmdList, nDisplay_ViewportManager_CrossGPUTransfer, "nDisplay_ViewportManager_CrossGPUTransfer");

		const FDisplayClusterViewportManagerProxy* ViewportManagerProxy = &InViewportManagerProxy.Get();

		// mGPU not used for in-editor rendering
		if(ViewportManagerProxy->ConfigurationProxy->GetRenderFrameSettings().ShouldUseCrossGPUTransfers())
		{
			// Move all render target cross gpu
			ViewportManagerProxy->DoCrossGPUTransfers_RenderThread(RHICmdList);
			// Now all resources on GPU#0
		}

		// PostCrossGpuTransfer notification
		IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPostCrossGpuTransfer_RenderThread().Broadcast(RHICmdList, ViewportManagerProxy, OutputViewport);

		// Another capture+playback round for media passthrough
		IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPassthroughMediaCapture_RenderThread().Broadcast(RHICmdList, ViewportManagerProxy);
		IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPassthroughMediaInput_RenderThread().Broadcast(RHICmdList, ViewportManagerProxy);

		// Latency processing
		IDisplayCluster::Get().GetCallbacks().OnDisplayClusterProcessLatency_RenderThread().Broadcast(RHICmdList, ViewportManagerProxy, OutputViewport);
	});

	ENQUEUE_RENDER_COMMAND(DisplayClusterRenderFrame_UpdateDeferredResources)(
		[ViewportManagerProxy = SharedThis(this)](FRHICommandListImmediate& RHICmdList)
	{
		RHI_BREADCRUMB_EVENT_STAT(RHICmdList, nDisplay_ViewportManager_UpdateDeferredResources, "nDisplay_ViewportManager_UpdateDeferredResources");

		// Update viewports resources: vp/texture overlay, OCIO, blur, nummips, etc
		ViewportManagerProxy->UpdateDeferredResources_RenderThread(RHICmdList);
	});

	ENQUEUE_RENDER_COMMAND(DisplayClusterRenderFrame_WarpBlend)(
		[InViewportManagerProxy = SharedThis(this), OutputViewport = InViewport](FRHICommandListImmediate& RHICmdList)
	{
		RHI_BREADCRUMB_EVENT_STAT(RHICmdList, nDisplay_ViewportManager_WarpBlend, "nDisplay_ViewportManager_WarpBlend");

		const FDisplayClusterViewportManagerProxy* ViewportManagerProxy = &InViewportManagerProxy.Get();

		ViewportManagerProxy->PostProcessManager->HandleBeginUpdateFrameResources_RenderThread(RHICmdList, ViewportManagerProxy);

		// Update the frame resources: post-processing, warping, and finally resolving everything to the frame resource
		ViewportManagerProxy->UpdateFrameResources_RenderThread(RHICmdList);

		ViewportManagerProxy->PostProcessManager->HandleEndUpdateFrameResources_RenderThread(RHICmdList, ViewportManagerProxy);

		// Postrender notification before copying final image to the backbuffer
		IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPostFrameRender_RenderThread().Broadcast(RHICmdList, ViewportManagerProxy, OutputViewport);

		if (OutputViewport)
		{
			if (FRHITexture* FrameOutputRTT = OutputViewport->GetRenderTargetTexture())
			{
				// For quadbuf stereo copy only left eye, right copy from OutputFrameTarget
				//@todo Copy QuadBuf_LeftEye/(mono,sbs,tp) to separate rtt, before UI and debug rendering
				//@todo QuadBuf_LeftEye copied latter, before present
				if(ViewportManagerProxy->ConfigurationProxy->GetRenderFrameSettings().ShouldUseStereoRenderingOnMonoscopicDisplay())
				{
					ViewportManagerProxy->ResolveFrameTargetToBackBuffer_RenderThread(RHICmdList, 1, 0, FrameOutputRTT, FrameOutputRTT->GetSizeXY());
				}

				ViewportManagerProxy->ResolveFrameTargetToBackBuffer_RenderThread(RHICmdList, 0, 0, FrameOutputRTT, FrameOutputRTT->GetSizeXY());

				// Finally, notify about backbuffer update
				IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPostBackbufferUpdated_RenderThread().Broadcast(RHICmdList, OutputViewport);
PRAGMA_DISABLE_DEPRECATION_WARNINGS
				IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPostBackbufferUpdate_RenderThread().Broadcast(RHICmdList, ViewportManagerProxy, OutputViewport);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}
		}

		// At the end, some resources may be filled with black, etc.
		// This is useful because the resources are reused and the image from the previous frame goes into the new one.
		ViewportManagerProxy->CleanupResources_RenderThread(RHICmdList);
	});
}

void FDisplayClusterViewportManagerProxy::UpdateDeferredResources_RenderThread(FRHICommandListImmediate& RHICmdList) const
{
	check(IsInRenderingThread());

	// Viewports in the CurrentRenderFrameViewportProxies list are already sorted using GetPriority_RenderThread().
	for (const TSharedPtr<FDisplayClusterViewportProxy, ESPMode::ThreadSafe>& ViewportProxy : ImplGetCurrentRenderFrameViewportProxies_RenderThread())
	{
		if (ViewportProxy.IsValid())
		{
			ViewportProxy->UpdateDeferredResources(RHICmdList);
		}
	}
}

void FDisplayClusterViewportManagerProxy::ImplClearFrameTargets_RenderThread(FRHICommandListImmediate& RHICmdList) const
{
	TArray<FRHITexture*> FrameResources;
	TArray<FRHITexture*> AdditionalFrameResources;
	TArray<FIntPoint> TargetOffset;
	if (GetFrameTargets_RenderThread(FrameResources, TargetOffset, &AdditionalFrameResources))
	{
		for (FRHITexture* FrameResourceIt : FrameResources)
		{
			FDisplayClusterViewportProxy::FillTextureWithColor_RenderThread(RHICmdList, FrameResourceIt, FLinearColor::Black);
		}
	}
}

/**
 * WarpBlend render pass. (internal)
 */
enum class EWarpPass : uint8
{
	Begin = 0,
	Render,
	End,
	COUNT
};

void FDisplayClusterViewportManagerProxy::UpdateFrameResources_RenderThread(FRHICommandListImmediate& RHICmdList) const
{
	check(IsInRenderingThread());

	// Do postprocess before warp&blend
	PostProcessManager->PerformPostProcessViewBeforeWarpBlend_RenderThread(RHICmdList, this);

	// Support viewport overlap order sorting:
	TArray<TSharedPtr<FDisplayClusterViewportProxy, ESPMode::ThreadSafe>> SortedViewportProxy = ImplGetCurrentRenderFrameViewportProxies_RenderThread();
	SortedViewportProxy.Sort(
		[](const TSharedPtr<FDisplayClusterViewportProxy, ESPMode::ThreadSafe>& VP1, const TSharedPtr<FDisplayClusterViewportProxy, ESPMode::ThreadSafe>& VP2)
		{
			return  VP1->GetRenderSettings_RenderThread().OverlapOrder < VP2->GetRenderSettings_RenderThread().OverlapOrder;
		}
	);

	// Clear Frame RTT resources before viewport resolving
	const bool bClearFrameRTTEnabled = CVarClearFrameRTTEnabled.GetValueOnRenderThread() != 0;
	if (bClearFrameRTTEnabled)
	{
		ImplClearFrameTargets_RenderThread(RHICmdList);
	}

	IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPreWarp_RenderThread().Broadcast(RHICmdList, this);

	// Handle warped viewport projection policy logic:
	for (uint8 WarpPass = 0; WarpPass < (uint8)EWarpPass::COUNT; WarpPass++)
	{
		// Update deferred resources for viewports
		for (TSharedPtr<FDisplayClusterViewportProxy, ESPMode::ThreadSafe>& ViewportProxyIt : SortedViewportProxy)
		{
			// Iterate over visible viewports:
			if (ViewportProxyIt.IsValid() && ViewportProxyIt->GetRenderSettings_RenderThread().bVisible)
			{
				if (ViewportProxyIt->ShouldApplyWarpBlend_RenderThread())
				{
					const TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe>& PrjPolicy = ViewportProxyIt->GetProjectionPolicy_RenderThread();
					switch ((EWarpPass)WarpPass)
					{
					case EWarpPass::Begin:
						IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPreWarpViewport_RenderThread().Broadcast(RHICmdList, ViewportProxyIt.Get());
						PrjPolicy->BeginWarpBlend_RenderThread(RHICmdList, ViewportProxyIt.Get());
						break;

					case EWarpPass::Render:
						PrjPolicy->ApplyWarpBlend_RenderThread(RHICmdList, ViewportProxyIt.Get());
						break;

					case EWarpPass::End:
						PrjPolicy->EndWarpBlend_RenderThread(RHICmdList, ViewportProxyIt.Get());
						IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPostWarpViewport_RenderThread().Broadcast(RHICmdList, ViewportProxyIt.Get());
						break;

					default:
						break;
					}
				}
			}
		}
	}

	// per-frame handle
	PostProcessManager->HandleUpdateFrameResourcesAfterWarpBlend_RenderThread(RHICmdList, this);

	// Per-view postprocess
	PostProcessManager->PerformPostProcessViewAfterWarpBlend_RenderThread(RHICmdList, this);

	// Post resolve to Frame RTT
	// All warp&blend results are now inside AdditionalTargetableResource. Viewport images of other projection policies are still stored in the InputShaderResource.
	for (TSharedPtr<FDisplayClusterViewportProxy, ESPMode::ThreadSafe>& ViewportProxyIt : SortedViewportProxy)
	{
		// Iterate over visible viewports:
		if (ViewportProxyIt.IsValid() && ViewportProxyIt->GetRenderSettings_RenderThread().bVisible)
		{
			ViewportProxyIt->PostResolveViewport_RenderThread(RHICmdList);

			// It's possible that some viewports don't need to request scene rendering, but get their final textures
			// from another viewports. In such cases, no rendering callbacks like PostRenderViewFamily get called,
			// therefore we can't media-capture such viewports from normal pipeline. To allow them to be captured,
			// we explicitly trigger another callback for all viewports that are overridden.
			const bool bOverridden = ViewportProxyIt->GetRenderSettings_RenderThread().IsViewportOverridden();
			if (bOverridden)
			{
				IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPostResolveOverridden_RenderThread().Broadcast(RHICmdList, ViewportProxyIt.Get());
			}
		}
	}

	IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPostWarp_RenderThread().Broadcast(RHICmdList, this);

	PostProcessManager->PerformPostProcessFrameAfterWarpBlend_RenderThread(RHICmdList, this);
}

void FDisplayClusterViewportManagerProxy::CleanupResources_RenderThread(FRHICommandListImmediate& RHICmdList) const
{
	check(IsInRenderingThread());

	// Viewports in the CurrentRenderFrameViewportProxies list are already sorted using GetPriority_RenderThread().
	for (const TSharedPtr<FDisplayClusterViewportProxy, ESPMode::ThreadSafe>& ViewportProxy : ImplGetCurrentRenderFrameViewportProxies_RenderThread())
	{
		if (ViewportProxy.IsValid())
		{
			ViewportProxy->CleanupResources_RenderThread(RHICmdList);
		}
	}
}

void FDisplayClusterViewportManagerProxy::DoCrossGPUTransfers_RenderThread(FRHICommandListImmediate& RHICmdList) const
{
	check(IsInRenderingThread());

#if WITH_MGPU
	if (GNumExplicitGPUsForRendering < 2)
	{
		// for mGPU we need at least 2 GPUs
		return;
	}

	// Copy the view render results to all GPUs that are native to the viewport.
	TArray<FTransferResourceParams> TransferResources;

	for (const TSharedPtr<FDisplayClusterViewportProxy, ESPMode::ThreadSafe>& ViewportProxyIt : ImplGetCurrentRenderFrameViewportProxies_RenderThread())
	{
		if (!ViewportProxyIt.IsValid())
		{
			continue;
		}

		if (!ViewportProxyIt->GetRenderSettings_RenderThread().bEnableCrossGPUTransfer || ViewportProxyIt->GetResources_RenderThread()[EDisplayClusterViewportResource::RenderTargets].IsEmpty())
		{
			// Skip a frozen viewport that has already been transferred between GPUs
			// The first time freezing should do the transfer (RenderTargets must be assigned on the first pass)
			continue;
		}

		// Do cross GPU trasfers for all viewport contexts
		for (const FDisplayClusterViewport_Context& ViewportContext : ViewportProxyIt->GetContexts_RenderThread())
		{
			if (!ViewportContext.bOverrideCrossGPUTransfer)
			{
				// Skip this context because it uses the default way
				continue;
			}

			const int32 SrcGPUIndex = ViewportContext.RenderThreadData.GPUIndex;
			if (SrcGPUIndex < 0)
			{
				// This viewport context does not use the custom GPU index for rendering
				continue;
			}

			if (!ViewportProxyIt->GetResources_RenderThread()[EDisplayClusterViewportResource::RenderTargets].IsValidIndex(ViewportContext.ContextNum))
			{
				// RTT does not exist for this context
				continue;
			}

			// Clamp the view rect by the rendertarget rect to prevent issues when resizing the viewport.
			const FIntRect TransferRect = ViewportContext.RenderTargetRect;
			if (TransferRect.Width() <= 0 || TransferRect.Height() <= 0)
			{
				// Broken rect, skip
				continue;
			}

			const FDisplayClusterRenderFrameSettings& RenderFrameSettings = ConfigurationProxy->GetRenderFrameSettings();

			// Perform an MGPU transfer for a specific viewport context:
			TSharedPtr<FDisplayClusterViewportResource, ESPMode::ThreadSafe> ViewportRenderTargetResource = ViewportProxyIt->GetResources_RenderThread()[EDisplayClusterViewportResource::RenderTargets][ViewportContext.ContextNum];
			{
				if (FRenderTarget* RenderTarget = ViewportRenderTargetResource.IsValid() ? ViewportRenderTargetResource->GetViewportResourceRenderTarget() : nullptr)
				{
					if (FRHITexture* TextureRHI = ViewportRenderTargetResource->GetViewportResourceRHI_RenderThread())
					{
						const FRHIGPUMask RenderTargetGPUMask = RenderTarget->GetGPUMask(RHICmdList);

						for (uint32 DestGPUIndex : RenderTargetGPUMask)
						{
							if (DestGPUIndex != SrcGPUIndex)
							{
								TransferResources.Add(FTransferResourceParams(
									TextureRHI, TransferRect,
									SrcGPUIndex, DestGPUIndex,
									RenderFrameSettings.CrossGPUTransfer.bPullData,
									RenderFrameSettings.CrossGPUTransfer.bLockSteps
								));
							}
						}
					}
				}
			}
		}
	}

	if (TransferResources.Num() > 0)
	{
		RHICmdList.TransferResources(TransferResources);
	}

#endif // WITH_MGPU
}

bool FDisplayClusterViewportManagerProxy::GetFrameTargets_RenderThread(TArray<FRHITexture*>& OutFrameResources, TArray<FIntPoint>& OutTargetOffsets, TArray<FRHITexture*>* OutAdditionalFrameResources) const
{
	check(IsInRenderingThread());

	// Get any defined frame targets from first visible viewport
	for (const TSharedPtr<FDisplayClusterViewportProxy, ESPMode::ThreadSafe>& ViewportProxyIt : ImplGetCurrentRenderFrameViewportProxies_RenderThread())
	{
		// Process only valid viewports with output frame resources
		if (ViewportProxyIt.IsValid() && !ViewportProxyIt->GetResources_RenderThread()[EDisplayClusterViewportResource::OutputFrameTargetableResources].IsEmpty())
		{
			const TArray<TSharedPtr<FDisplayClusterViewportResource, ESPMode::ThreadSafe>>& Frames = ViewportProxyIt->GetResources_RenderThread()[EDisplayClusterViewportResource::OutputFrameTargetableResources];
			const TArray<TSharedPtr<FDisplayClusterViewportResource, ESPMode::ThreadSafe>>& AdditionalFrames = ViewportProxyIt->GetResources_RenderThread()[EDisplayClusterViewportResource::AdditionalFrameTargetableResources];

			for (int32 FrameIt = 0; FrameIt < Frames.Num(); FrameIt++)
			{
				if (FRHITexture* FrameTexture = Frames[FrameIt].IsValid() ? Frames[FrameIt]->GetViewportResourceRHI_RenderThread() : nullptr)
				{
					OutFrameResources.Add(FrameTexture);
					OutTargetOffsets.Add(Frames[FrameIt]->GetBackbufferFrameOffset());

					if (OutAdditionalFrameResources && AdditionalFrames.IsValidIndex(FrameIt))
					{
						if (FRHITexture* AdditionalFrameTexture = AdditionalFrames[FrameIt].IsValid() ? AdditionalFrames[FrameIt]->GetViewportResourceRHI_RenderThread() : nullptr)
						{
							OutAdditionalFrameResources->Add(AdditionalFrameTexture);
						}
					}
				}
			}

			// if all resources are received in the same amount
			const bool bValidAdditionalFrameResources = !OutAdditionalFrameResources || AdditionalFrames.IsEmpty() || (OutAdditionalFrameResources && OutAdditionalFrameResources->Num() == Frames.Num());
			if (OutFrameResources.Num() == Frames.Num() && bValidAdditionalFrameResources)
			{
				return true;
			}
		}

		// Resetting output values at the end of each cycle
		OutFrameResources.Reset();
		OutTargetOffsets.Reset();
		if (OutAdditionalFrameResources)
		{
			OutAdditionalFrameResources->Reset();
		}
	}

	// no visible viewports
	return false;
}

bool FDisplayClusterViewportManagerProxy::ImplResolveFrameTargetToBackBuffer_RenderThread(FRDGBuilder& GraphBuilder, const uint32 InContextNum, const int32 DestArrayIndex, FRDGTextureRef DestTexture, FVector2f WindowSize) const
{
	check(IsInRenderingThread());

	if (!DestTexture)
	{
		return false;
	}

	// Output texture must supports quadbuffer stereo.
	if (DestArrayIndex > 0)
	{
		// Check if the backbuffer texture has a second slice for stereo.
		const bool bStereoTexture = DestTexture->Desc.Dimension == ETextureDimension::Texture2DArray && DestTexture->Desc.ArraySize > 1;
		if (!bStereoTexture)
		{
			if (!bHasLoggedBackbufferStereoUnsupported)
			{
				UE_LOGF(LogDisplayClusterRender, Error, "Backbuffer does not support stereo rendering.");
				bHasLoggedBackbufferStereoUnsupported = 1;
			}

			return false;
		}
	}

	TArray<FRHITexture*>   FrameResources;
	TArray<FIntPoint>      TargetOffsets;
	if (GetFrameTargets_RenderThread(FrameResources, TargetOffsets) && FrameResources.IsValidIndex(InContextNum))
	{
		// Use internal frame textures as source
		if (FRHITexture* FrameTexture = FrameResources[InContextNum])
		{
			FIntRect SrcRect(FIntPoint::ZeroValue, FrameTexture->GetDesc().Extent);

			const FIntPoint& DestOffset = TargetOffsets[InContextNum];
			FIntRect DestRect(DestOffset, DestOffset + FrameTexture->GetDesc().Extent);

			// Check if resources with the specified regions can be resolved.
			if (!FDisplayClusterViewportHelpers::GetValidResourceRectsForResolve(FrameTexture->GetDesc().Extent, DestTexture->Desc.Extent, SrcRect, DestRect))
			{
				// The SrcRect or DestRect is invalid.
				return false;
			}

			FRHICopyTextureInfo CopyInfo;

			CopyInfo.SourceSliceIndex = 0;
			CopyInfo.DestSliceIndex = DestArrayIndex;

			CopyInfo.SourcePosition = FIntVector(SrcRect.Min.X, SrcRect.Min.Y, 0);
			CopyInfo.DestPosition = FIntVector(DestRect.Min.X, DestRect.Min.Y, 0);

			CopyInfo.Size = FIntVector(DestRect.Width(), DestRect.Height(), 1);

			FRDGTextureRef SrcTexture = RegisterExternalTexture(GraphBuilder, FrameTexture->GetTexture2D(), TEXT("DisplayClusterStereoTexture"));

			AddCopyTexturePass(GraphBuilder, SrcTexture, DestTexture, CopyInfo);

			return true;
		}
	}

	return false;
}

bool FDisplayClusterViewportManagerProxy::ResolveFrameTargetToBackBuffer_RenderThread(FRHICommandListImmediate& RHICmdList, const uint32 InContextNum, const int32 DestArrayIndex, FRHITexture* DestTexture, FVector2D WindowSize) const
{
	check(IsInRenderingThread());

	if (!DestTexture)
	{
		return false;
	}

	// Output texture must supports quadbuffer stereo.
	if (DestArrayIndex > 0)
	{
		// Check if the backbuffer texture has a second slice for stereo.
		const bool bStereoTexture = DestTexture->GetDesc().Dimension == ETextureDimension::Texture2DArray && DestTexture->GetDesc().ArraySize > 1;
		if (!bStereoTexture)
		{
			return false;
		}
	}

	TArray<FRHITexture*>   FrameResources;
	TArray<FIntPoint>        TargetOffsets;
	if (GetFrameTargets_RenderThread(FrameResources, TargetOffsets) && FrameResources.IsValidIndex(InContextNum))
	{
		// Use internal frame textures as source
		if (FRHITexture* FrameTexture = FrameResources[InContextNum])
		{
			FIntRect SrcRect(FIntPoint::ZeroValue, FrameTexture->GetDesc().Extent);

			const FIntPoint& DestOffset = TargetOffsets[InContextNum];
			FIntRect DestRect(DestOffset, DestOffset + FrameTexture->GetDesc().Extent);

			// Check if resources with the specified regions can be resolved.
			if (!FDisplayClusterViewportHelpers::GetValidResourceRectsForResolve(FrameTexture->GetDesc().Extent, DestTexture->GetDesc().Extent, SrcRect, DestRect))
			{
				// The SrcRect or DestRect is invalid.
				return false;
			}

			FRHICopyTextureInfo CopyInfo;

			CopyInfo.SourceSliceIndex = 0;
			CopyInfo.DestSliceIndex = DestArrayIndex;

			CopyInfo.SourcePosition = FIntVector(SrcRect.Min.X, SrcRect.Min.Y, 0);
			CopyInfo.DestPosition = FIntVector(DestRect.Min.X, DestRect.Min.Y, 0);

			CopyInfo.Size = FIntVector(DestRect.Width(), DestRect.Height(), 1);

			TransitionAndCopyTexture(RHICmdList, FrameTexture, DestTexture, CopyInfo);

			return true;
		}
	}

	return false;
}

void FDisplayClusterViewportManagerProxy::ReleaseAllResources_RenderThread()
{
	for (const TSharedPtr<FDisplayClusterViewportProxy, ESPMode::ThreadSafe>& ViewportProxyIt : ImplGetEntireClusterViewportProxies_RenderThread())
	{
		if (ViewportProxyIt.IsValid())
		{
			ViewportProxyIt->ReleaseAllResources_RenderThread();
		}
	}
}

FDisplayClusterViewportProxy* FDisplayClusterViewportManagerProxy::ImplFindViewportProxy_RenderThread(const FString& ViewportId) const
{
	check(IsInRenderingThread());

	// Ok, we have a request for a particular viewport. Let's find it.
	TSharedPtr<FDisplayClusterViewportProxy, ESPMode::ThreadSafe> const* DesiredViewport = ImplGetEntireClusterViewportProxies_RenderThread().FindByPredicate([ViewportId](const TSharedPtr<FDisplayClusterViewportProxy, ESPMode::ThreadSafe>& ItemViewport)
	{
		return ViewportId.Equals(ItemViewport->GetId(), ESearchCase::IgnoreCase);
	});

	return (DesiredViewport != nullptr) ? DesiredViewport->Get() : nullptr;
}

FDisplayClusterViewportProxy* FDisplayClusterViewportManagerProxy::ImplFindViewportProxy_RenderThread(const int32 StereoViewIndex, uint32* OutContextNum) const
{
	check(IsInRenderingThread());
	
	if (!FDisplayClusterViewportStereoscopicPass::IsValidStereoViewIndex(StereoViewIndex))
	{
		return nullptr;
	}

	for (const TSharedPtr<FDisplayClusterViewportProxy, ESPMode::ThreadSafe>&  ViewportProxyIt : ImplGetEntireClusterViewportProxies_RenderThread())
	{
		if (ViewportProxyIt.IsValid() && ViewportProxyIt->FindContext_RenderThread(StereoViewIndex, OutContextNum))
		{
			return ViewportProxyIt.Get();
		}
	}

	// Viewport proxy not found
	return nullptr;
}
