// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Render/Viewport/IDisplayClusterViewportManagerProxy.h"
#include "Render/Viewport/DisplayClusterViewportProxy.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_Enums.h"
#include "Render/Viewport/RenderFrame/DisplayClusterRenderFrameSettings.h"

#include "RenderGraphUtils.h"
#include "RenderGraphBuilder.h"

#include "Templates/SharedPointer.h"

class FDisplayClusterRenderTargetManager;
class FDisplayClusterViewportPostProcessManager;
class FDisplayClusterViewportManager;
class FDisplayClusterViewportManagerViewExtension;
class FDisplayClusterViewportResource;
class FDisplayClusterViewportLightCardManagerProxy;
class FDisplayClusterAutoExposure;
class IDisplayClusterProjectionPolicy;
class FViewport;

/**
 * ViewportManagerProxy implementation.
 * This is a proxy object for the rendering thread. The data is copied every frame from the ViewportManager on the game thread.
 */
class FDisplayClusterViewportManagerProxy
	: public IDisplayClusterViewportManagerProxy
	, public TSharedFromThis<FDisplayClusterViewportManagerProxy, ESPMode::ThreadSafe>
{
public:
	FDisplayClusterViewportManagerProxy(const FDisplayClusterViewportManager& InViewportManager);
	virtual ~FDisplayClusterViewportManagerProxy();

public:
	//~IDisplayClusterViewportManagerProxy
	virtual TSharedPtr<IDisplayClusterViewportManagerProxy, ESPMode::ThreadSafe> ToSharedPtr() override
	{
		return AsShared();
	}

	virtual TSharedPtr<const IDisplayClusterViewportManagerProxy, ESPMode::ThreadSafe> ToSharedPtr() const override
	{
		return AsShared();
	}

	/** Get viewport manager proxy configuration interface. */
	virtual const IDisplayClusterViewportConfigurationProxy& GetConfigurationProxy() const override
	{
		return *ConfigurationProxy;
	}

	virtual IDisplayClusterViewportProxy* FindViewport_RenderThread(const FString& InViewportId) const override
	{
		return ImplFindViewportProxy_RenderThread(InViewportId);
	}

	virtual IDisplayClusterViewportProxy* FindViewport_RenderThread(const int32 StereoViewIndex, uint32* OutContextNum = nullptr) const override
	{
		return ImplFindViewportProxy_RenderThread(StereoViewIndex, OutContextNum);
	}

	virtual const TArrayView<TSharedPtr<IDisplayClusterViewportProxy, ESPMode::ThreadSafe>> GetViewports_RenderThread() const override
	{
		return TArrayView<TSharedPtr<IDisplayClusterViewportProxy, ESPMode::ThreadSafe>>((TSharedPtr<IDisplayClusterViewportProxy, ESPMode::ThreadSafe>*)(CurrentRenderFrameViewportProxies.GetData()), CurrentRenderFrameViewportProxies.Num());
	}

	virtual bool GetFrameTargets_RenderThread(TArray<FRHITexture*>& OutFrameResources, TArray<FIntPoint>& OutTargetOffsets, TArray<FRHITexture*>* OutAdditionalFrameResources = nullptr) const override;
	virtual bool ResolveFrameTargetToBackBuffer_RenderThread(FRHICommandListImmediate& RHICmdList, const uint32 InContextNum, const int32 DestArrayIndex, FRHITexture* DestTexture, FVector2D WindowSize) const override;
	//~~IDisplayClusterViewportManagerProxy

	/**
	* Resolve to backbuffer
	* [RDG version]
	* 
	* @param GraphBuilder   - Render Graph interface
	* @param InContextNum   - renderframe source context num
	* @param DestArrayIndex - dest array index on backbuffer
	* @param DestTexture    - dest RDG texture
	* @param WindowSize     - dest backbuffer window size
	*
	* @return - true if success
	*/
	bool ImplResolveFrameTargetToBackBuffer_RenderThread(FRDGBuilder& GraphBuilder, const uint32 InContextNum, const int32 DestArrayIndex, FRDGTextureRef DestTexture, FVector2f WindowSize) const;

	/** Release all resources on render thread.
	*
	* Note: This function has been renamed; its previous name was 'ReleaseTextures_RenderThread()'
	*/
	void ReleaseAllResources_RenderThread();

	/** Custom cross-GPU implementation for mGPU. Applyed for viewports with bOverrideCrossGPUTransfer=true. */
	void DoCrossGPUTransfers_RenderThread(FRHICommandListImmediate& RHICmdList) const;

	/** UpdateDeferredResources for all viewports (OCIO, PP, mips, etc). */
	void UpdateDeferredResources_RenderThread(FRHICommandListImmediate& RHICmdList) const;

	/** Apply WarpBlend and resolve to frame resources. */
	void UpdateFrameResources_RenderThread(FRHICommandListImmediate& RHICmdList) const;

	/** Called at the end of the frame, after all callbacks.
	* At the end, some resources may be filled with black, etc.
	* This is useful because the resources are reused and the image from the previous frame goes into the new one.
	*/
	void CleanupResources_RenderThread(FRHICommandListImmediate& RHICmdList) const;

	/** Release all referenced objects and resources. */
	void Release_RenderThread();

	/** Register new viewport proxy. */
	void CreateViewport_RenderThread(const TSharedPtr<FDisplayClusterViewportProxy, ESPMode::ThreadSafe>& InViewportProxy);

	/** Unregister exist viewport proxy. */
	void DeleteViewport_RenderThread(const TSharedPtr<FDisplayClusterViewportProxy, ESPMode::ThreadSafe>& InViewportProxy);

	/** Get LightCardManager proxy object. */
	inline TSharedPtr<FDisplayClusterViewportLightCardManagerProxy, ESPMode::ThreadSafe> GetLightCardManagerProxy_RenderThread() const
	{ return LightCardManagerProxy; }

	/** Geting the viewport proxies of the current rendering frame. (viewports from the current node or from a special named list) */
	inline const TArray<TSharedPtr<FDisplayClusterViewportProxy, ESPMode::ThreadSafe>>& ImplGetCurrentRenderFrameViewportProxies_RenderThread() const
	{ return CurrentRenderFrameViewportProxies; }

	/** Get the viewport proxies  of the whole cluster on the render thread. */
	inline const TArray<TSharedPtr<FDisplayClusterViewportProxy, ESPMode::ThreadSafe>>& ImplGetEntireClusterViewportProxies_RenderThread() const
	{ return EntireClusterViewportProxies; }

	/** Find viewport by name. */
	FDisplayClusterViewportProxy* ImplFindViewportProxy_RenderThread(const FString& InViewportId) const;

	/** Find viewport and context index by StereoViewIndex. */
	FDisplayClusterViewportProxy* ImplFindViewportProxy_RenderThread(const int32 StereoViewIndex, uint32* OutContextNum = nullptr) const;

	/** Sending viewport data from the game stream to the viewport proxy that is in the rendering stream. */
	void ImplUpdateViewportProxies_GameThread(const TArray<TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>>& InCurrentRenderFrameViewports);

	/** Sending viewport manager data from the game stream to the viewport manager proxy that is in the rendering stream. */
	void ImplUpdateViewportManagerProxy_GameThread(const FDisplayClusterViewportManager& InViewportManager);

	/** Rendering the final frame of nDisplay. Called after all viewports have been rendered in RTT. */
	void ImplRenderFrame_GameThread(FViewport* InViewport);

	/** Rendering black color in all frame textures. */
	void ImplClearFrameTargets_RenderThread(FRHICommandListImmediate& RHICmdList) const;

private:
	/** Update ClusterNodeViewportProxies from ViewportProxies for the current cluster node. */
	void ImplUpdateClusterNodeViewportProxies_RenderThread();

public:
	// Configuration for proxy
	const TSharedRef<FDisplayClusterViewportConfigurationProxy, ESPMode::ThreadSafe> ConfigurationProxy;

	// RenderTarget manager object. Used to remove related objects in order.
	const TSharedRef<FDisplayClusterRenderTargetManager, ESPMode::ThreadSafe> RenderTargetManager;

	// PostProcess manager object. Used to remove related objects in order.
	const TSharedRef<FDisplayClusterViewportPostProcessManager, ESPMode::ThreadSafe> PostProcessManager;

	// LightCard manager proxy object.
	const TSharedRef<FDisplayClusterViewportLightCardManagerProxy, ESPMode::ThreadSafe> LightCardManagerProxy;

	// nDisplay multi-viewport AutoExposure feature.
	const TSharedRef<FDisplayClusterAutoExposure> AutoExposure;

private:
	// Viewport proxies of the entire cluster.
	TArray<TSharedPtr<FDisplayClusterViewportProxy, ESPMode::ThreadSafe>> EntireClusterViewportProxies;

	// Viewport proxies of the current rendering frame (viewports from the current node or from a special named list).
	TArray<TSharedPtr<FDisplayClusterViewportProxy, ESPMode::ThreadSafe>> CurrentRenderFrameViewportProxies;

	// nDisplay VE object. Used to remove related objects in order.
	TSharedPtr<class FDisplayClusterViewportManagerViewExtension, ESPMode::ThreadSafe> ViewportManagerViewExtension;

private:
	// True if the "Backbuffer does not support stereo" error has already been logged.
	// Used to ensure the error is only written to the log once.
	mutable uint8 bHasLoggedBackbufferStereoUnsupported : 1 = 0;

};
