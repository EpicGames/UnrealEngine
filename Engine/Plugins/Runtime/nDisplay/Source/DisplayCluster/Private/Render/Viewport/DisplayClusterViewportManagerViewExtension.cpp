// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterViewportManagerViewExtension.h"
#include "DisplayClusterSceneViewExtensions.h"

#include "Render/Viewport/DisplayClusterViewportManager.h"
#include "Render/Viewport/DisplayClusterViewportManagerProxy.h"

#include "IDisplayCluster.h"
#include "IDisplayClusterCallbacks.h"

#include "ScreenPass.h"
#include "CommonRenderResources.h"
#include "PostProcess/PostProcessMaterialInputs.h"
#include "SceneTextures.h"

///////////////////////////////////////////////////////////////////////////////////////
namespace UE::DisplayCluster::ViewportManagerViewExtension
{
	static const FName RendererModuleName(TEXT("Renderer"));
};
using namespace UE::DisplayCluster::ViewportManagerViewExtension;

///////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterViewportManagerViewExtension
///////////////////////////////////////////////////////////////////////////////////////
FDisplayClusterViewportManagerViewExtension::FDisplayClusterViewportManagerViewExtension(const FAutoRegister& AutoRegister, const TSharedRef<FDisplayClusterViewportConfiguration, ESPMode::ThreadSafe>& InConfiguration)
	: FSceneViewExtensionBase(AutoRegister)
	, Configuration(InConfiguration)
{
	RegisterCallbacks();
}

FDisplayClusterViewportManagerViewExtension::~FDisplayClusterViewportManagerViewExtension()
{
	UnregisterCallbacks();
}

bool FDisplayClusterViewportManagerViewExtension::IsActive() const
{
	return !bReleased && (Configuration->Proxy->GetViewportManagerProxyImpl() != nullptr);
}

void FDisplayClusterViewportManagerViewExtension::Release_RenderThread()
{
	check(IsInRenderingThread());

	bReleased = true;

	ViewportProxies.Empty();
	UnregisterCallbacks();
}

void FDisplayClusterViewportManagerViewExtension::PreRenderView_RenderThread(FRDGBuilder& GraphBuilder, FSceneView& InView)
{
	if (InView.bIsViewInfo)
	{
		// UE-145088: VR HMD device post-processing cannot be applied to nDisplay rendering
		InView.bHMDHiddenAreaMaskActive = false;
	}
}

void FDisplayClusterViewportManagerViewExtension::SubscribeToPostProcessingPass(EPostProcessingPass PassId, const FSceneView& View, FAfterPassCallbackDelegateArray& InOutPassCallbacks, bool bIsPassEnabled)
{
	if (!IsActive())
	{
		return;
	}

	switch (PassId)
	{
	case EPostProcessingPass::SSRInput:
		for (const FViewportProxy& ViewportIt : ViewportProxies)
		{
			if (FDisplayClusterViewportProxy* ViewportProxy = ViewportIt.GetViewportProxy())
			{
				if (ViewportProxy->ShouldUsePostProcessPassAfterSSRInput())
				{
					InOutPassCallbacks.Add(FAfterPassCallbackDelegate::CreateRaw(this, &FDisplayClusterViewportManagerViewExtension::PostProcessPassAfterSSRInput_RenderThread));
					break;
				}
			}
		}
		break;

	case EPostProcessingPass::FXAA:
		for (const FViewportProxy& ViewportIt : ViewportProxies)
		{
			if (FDisplayClusterViewportProxy* ViewportProxy = ViewportIt.GetViewportProxy())
			{
				if (ViewportProxy->ShouldUsePostProcessPassAfterFXAA())
				{
					InOutPassCallbacks.Add(FAfterPassCallbackDelegate::CreateRaw(this, &FDisplayClusterViewportManagerViewExtension::PostProcessPassAfterFXAA_RenderThread));
					break;
				}
			}
		}
		break;

	case EPostProcessingPass::Tonemap:
		for (const FViewportProxy& ViewportIt : ViewportProxies)
		{
			if (FDisplayClusterViewportProxy* ViewportProxy = ViewportIt.GetViewportProxy())
			{
				if (ViewportProxy->ShouldUsePostProcessPassTonemap())
				{
					InOutPassCallbacks.Add(FAfterPassCallbackDelegate::CreateRaw(this, &FDisplayClusterViewportManagerViewExtension::PostProcessPassAfterTonemap_RenderThread));
					break;
				}
			}
		}
		break;

	default:
		break;
	}
}

FScreenPassTexture FDisplayClusterViewportManagerViewExtension::PostProcessPassAfterFXAA_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& View, const FPostProcessMaterialInputs& Inputs)
{
	if (!IsActive())
	{
		return Inputs.ReturnUntouchedSceneColorForPostProcessing(GraphBuilder);
	}

	if (const FDisplayClusterViewportManagerProxy* ViewportManagerProxy = Configuration->Proxy->GetViewportManagerProxyImpl())
	{
		uint32 ContextNum = 0;
		if (FDisplayClusterViewportProxy* ViewportProxyPtr = ViewportManagerProxy->ImplFindViewportProxy_RenderThread(View.StereoViewIndex, &ContextNum))
		{
			if (ViewportProxyPtr->ShouldUsePostProcessPassAfterFXAA())
			{
				return ViewportProxyPtr->OnPostProcessPassAfterFXAA_RenderThread(GraphBuilder, View, Inputs, ContextNum);
			}
		}
	}

	return Inputs.ReturnUntouchedSceneColorForPostProcessing(GraphBuilder);
}

FScreenPassTexture FDisplayClusterViewportManagerViewExtension::PostProcessPassAfterSSRInput_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& View, const FPostProcessMaterialInputs& Inputs)
{
	if (!IsActive())
	{
		return Inputs.ReturnUntouchedSceneColorForPostProcessing(GraphBuilder);
	}

	if (const FDisplayClusterViewportManagerProxy* ViewportManagerProxy = Configuration->Proxy->GetViewportManagerProxyImpl())
	{
		uint32 ContextNum = 0;
		if (FDisplayClusterViewportProxy* ViewportProxyPtr = ViewportManagerProxy->ImplFindViewportProxy_RenderThread(View.StereoViewIndex, &ContextNum))
		{
			if (ViewportProxyPtr->ShouldUsePostProcessPassAfterSSRInput())
			{
				return ViewportProxyPtr->OnPostProcessPassAfterSSRInput_RenderThread(GraphBuilder, View, Inputs, ContextNum);
			}
		}
	}

	return Inputs.ReturnUntouchedSceneColorForPostProcessing(GraphBuilder);
}

FScreenPassTexture FDisplayClusterViewportManagerViewExtension::PostProcessPassAfterTonemap_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& View, const FPostProcessMaterialInputs& Inputs)
{
	if (!IsActive())
	{
		return Inputs.ReturnUntouchedSceneColorForPostProcessing(GraphBuilder);
	}

	if (const FDisplayClusterViewportManagerProxy* ViewportManagerProxy = Configuration->Proxy->GetViewportManagerProxyImpl())
	{
		uint32 ContextNum = 0;
		if (FDisplayClusterViewportProxy* ViewportProxyPtr = ViewportManagerProxy->ImplFindViewportProxy_RenderThread(View.StereoViewIndex, &ContextNum))
		{
			return ViewportProxyPtr->OnPostProcessPassAfterTonemap_RenderThread(GraphBuilder, View, Inputs, ContextNum);
		}
	}

	return Inputs.ReturnUntouchedSceneColorForPostProcessing(GraphBuilder);
}

bool FDisplayClusterViewportManagerViewExtension::IsActiveThisFrame_Internal(const FSceneViewExtensionContext& Context) const
{
	if (!IsActive())
	{
		return false;
	}

	static const FDisplayClusterSceneViewExtensionContext DCViewExtensionContext;
	if (Context.IsA(MoveTempIfPossible(DCViewExtensionContext)))
	{
		const FDisplayClusterSceneViewExtensionContext& DisplayContext = static_cast<const FDisplayClusterSceneViewExtensionContext&>(Context);
		if (DisplayContext.IsSameDisplayClusterViewportConfiguration(Configuration))
		{
			// Apply only for DC viewports
			return true;
		}
	}

	return false;
}

void FDisplayClusterViewportManagerViewExtension::PreRenderViewFamily_RenderThread(FRDGBuilder& GraphBuilder, FSceneViewFamily& InViewFamily)
{
	if (!IsActive())
	{
		return;
	}

	ViewportProxies.Empty();

	// Get all viewport proxies that are uses in this view family
	for (int32 ViewIndex = 0; ViewIndex < InViewFamily.Views.Num(); ViewIndex++)
	{
		if (const FSceneView* SceneView = InViewFamily.Views[ViewIndex])
		{
			FViewportProxy NewViewportProxy;

			if (const FDisplayClusterViewportManagerProxy* ViewportManagerProxy = Configuration->Proxy->GetViewportManagerProxyImpl())
			{
				if (FDisplayClusterViewportProxy* ViewportProxyPtr = static_cast<FDisplayClusterViewportProxy*>(ViewportManagerProxy->FindViewport_RenderThread(SceneView->StereoViewIndex, &NewViewportProxy.ViewportProxyContext.ContextNum)))
				{
					NewViewportProxy.ViewportProxyContext.ViewFamilyProfileDescription = InViewFamily.ProfileDescription;

					NewViewportProxy.ViewportProxyWeakPtr = ViewportProxyPtr->AsWeak();
					NewViewportProxy.ViewIndex = ViewIndex;

					ViewportProxies.Add(NewViewportProxy);
				}
			}
		}
	}
}

void FDisplayClusterViewportManagerViewExtension::PostRenderViewFamily_RenderThread(FRDGBuilder& GraphBuilder, FSceneViewFamily& InViewFamily)
{
	if (!IsActive())
	{
		return;
	}

	for (FViewportProxy& ViewportIt : ViewportProxies)
	{
		if (const FSceneView* SceneView = InViewFamily.Views[ViewportIt.ViewIndex])
		{
			if (FDisplayClusterViewportProxy* ViewportProxy = ViewportIt.GetViewportProxy())
			{
				ViewportProxy->OnPostRenderViewFamily_RenderThread(GraphBuilder, InViewFamily, *SceneView, ViewportIt.ViewportProxyContext);
			}
		}
	}

	ViewportProxies.Empty();
}

//////////////////////////////////////////////////////////////////////////////////////////////
void FDisplayClusterViewportManagerViewExtension::OnResolvedSceneColor_RenderThread(FRDGBuilder& GraphBuilder, const FSceneTextures& SceneTextures)
{
	if (!IsActive())
	{
		return;
	}

	for (FViewportProxy& ViewportIt : ViewportProxies)
	{
		if (FDisplayClusterViewportProxy* ViewportProxy = ViewportIt.GetViewportProxy())
		{
			ViewportProxy->OnResolvedSceneColor_RenderThread(GraphBuilder, SceneTextures, ViewportIt.ViewportProxyContext);
		}
	}
}

void FDisplayClusterViewportManagerViewExtension::RegisterCallbacks()
{
	if (!ResolvedSceneColorCallbackHandle.IsValid())
	{
		if (IRendererModule* RendererModule = FModuleManager::GetModulePtr<IRendererModule>(RendererModuleName))
		{
			ResolvedSceneColorCallbackHandle = RendererModule->GetResolvedSceneColorCallbacks().AddRaw(this, &FDisplayClusterViewportManagerViewExtension::OnResolvedSceneColor_RenderThread);
		}
	}
}

void FDisplayClusterViewportManagerViewExtension::UnregisterCallbacks()
{
	if (ResolvedSceneColorCallbackHandle.IsValid())
	{
		if (IRendererModule* RendererModule = FModuleManager::GetModulePtr<IRendererModule>(RendererModuleName))
		{
			RendererModule->GetResolvedSceneColorCallbacks().Remove(ResolvedSceneColorCallbackHandle);
		}

		ResolvedSceneColorCallbackHandle.Reset();
	}
}
