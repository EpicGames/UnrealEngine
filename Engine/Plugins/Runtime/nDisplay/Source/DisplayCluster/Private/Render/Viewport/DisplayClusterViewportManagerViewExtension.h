// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "SceneViewExtension.h"
#include "Render/Viewport/Configuration/DisplayClusterViewportConfiguration.h"
#include "Render/Viewport/Containers/DisplayClusterViewportProxy_Context.h"

class FDisplayClusterViewportProxy;
struct FSceneTextures;

#define DISPLAYCLUSTER_SCENE_VIEW_EXTENSION_PRIORITY -1

/**
 * View extension applying an DC Viewport features
 */
class FDisplayClusterViewportManagerViewExtension : public FSceneViewExtensionBase
{
public:
	FDisplayClusterViewportManagerViewExtension(const FAutoRegister& AutoRegister, const TSharedRef<FDisplayClusterViewportConfiguration, ESPMode::ThreadSafe>& InConfiguration);
	virtual ~FDisplayClusterViewportManagerViewExtension();

public:
	//~ Begin ISceneViewExtension interface
	virtual int32 GetPriority() const override
	{
		return DISPLAYCLUSTER_SCENE_VIEW_EXTENSION_PRIORITY;
	}

	virtual void PreRenderView_RenderThread(FRDGBuilder& GraphBuilder, FSceneView& InView) override;

	virtual void SubscribeToPostProcessingPass(EPostProcessingPass PassId, const FSceneView& View, FAfterPassCallbackDelegateArray& InOutPassCallbacks, bool bIsPassEnabled) override;

	virtual bool IsActiveThisFrame_Internal(const FSceneViewExtensionContext& Context) const override;

	virtual void PreRenderViewFamily_RenderThread(FRDGBuilder& GraphBuilder, FSceneViewFamily& InViewFamily) override;
	virtual void PostRenderViewFamily_RenderThread(FRDGBuilder& GraphBuilder, FSceneViewFamily& InViewFamily) override;
	//~End ISceneVIewExtension interface

	/** Release from render thread.*/
	void Release_RenderThread();

private:
	/** nDisplay VE Callback [subscribed to Renderer:ResolvedSceneColorCallbacks].
	 *
	 * @param GraphBuilder   - RDG interface
	 * @param SceneTextures  - Scene textures (SceneColor, Depth, etc)
	 *
	 * @return - none
	 */
	void OnResolvedSceneColor_RenderThread(FRDGBuilder& GraphBuilder, const FSceneTextures& SceneTextures);

	/** Callback OnPostProcessPassAfterFXAA.
	 *
	 * @param GraphBuilder - RDG interface
	 * @param View         - Scene View
	 * @param Inputs       - PP Input resources
	 *
	 * @return - Screen pass texture
	 */
	FScreenPassTexture PostProcessPassAfterFXAA_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& View, const FPostProcessMaterialInputs& Inputs);

	/** Callback OnPostProcessPassAfterSSRInput.
	 *
	 * @param GraphBuilder - RDG interface
	 * @param View         - Scene View
	 * @param Inputs       - PP Input resources
	 *
	 * @return - Screen pass texture
	 */
	FScreenPassTexture PostProcessPassAfterSSRInput_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& View, const FPostProcessMaterialInputs& Inputs);

	/** Callback OnPostProcessPassAfterTonemap.
	 *
	 * @param GraphBuilder - RDG interface
	 * @param View         - Scene View
	 * @param Inputs       - PP Input resources
	 *
	 * @return - Screen pass texture
	 */
	FScreenPassTexture PostProcessPassAfterTonemap_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& View, const FPostProcessMaterialInputs& Inputs);

	/** Register callbacks to external modules (Renderer:ResolvedSceneColor). */
	void RegisterCallbacks();

	/** UnRegister callbacks to external modules (Renderer:ResolvedSceneColor). */
	void UnregisterCallbacks();

	/** Return true if the VE is safe to use. */
	bool IsActive() const;

public:
	// Configuration of the current cluster node
	const TSharedRef<FDisplayClusterViewportConfiguration, ESPMode::ThreadSafe> Configuration;

private:
	FDelegateHandle ResolvedSceneColorCallbackHandle;

	struct FViewportProxy
	{
		inline FDisplayClusterViewportProxy* GetViewportProxy() const
		{
			return ViewportProxyWeakPtr.IsValid() ? ViewportProxyWeakPtr.Pin().Get() : nullptr;
		}

		// The ViewportProxy must exist until this object is removed.
		TWeakPtr<FDisplayClusterViewportProxy, ESPMode::ThreadSafe> ViewportProxyWeakPtr;

		// The index of view in viewfamily collection
		int32 ViewIndex = 0;

		// Context for viewport proxy object
		FDisplayClusterViewportProxy_Context ViewportProxyContext;
	};

	/** RenderThreadData:. Viewport proxies from rendered view family. */
	TArray<FViewportProxy> ViewportProxies;

	// Is this VE released
	bool bReleased = false;
};
