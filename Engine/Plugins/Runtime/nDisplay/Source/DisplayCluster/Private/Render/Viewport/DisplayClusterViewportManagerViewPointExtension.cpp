// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterViewportManagerViewPointExtension.h"

#include "Render/Viewport/DisplayClusterViewportManager.h"
#include "Render/Viewport/DisplayClusterViewport.h"

///////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterViewportManagerViewPointExtension
///////////////////////////////////////////////////////////////////////////////////////
FDisplayClusterViewportManagerViewPointExtension::FDisplayClusterViewportManagerViewPointExtension(const FAutoRegister& AutoRegister, const TSharedRef<FDisplayClusterViewportConfiguration, ESPMode::ThreadSafe>& InConfiguration)
	: FSceneViewExtensionBase(AutoRegister)
	, Configuration(InConfiguration)
{ }

bool FDisplayClusterViewportManagerViewPointExtension::IsActiveThisFrame_Internal(const FSceneViewExtensionContext& Context) const
{
	// This VE can be used in the editor to preview in the scene
	return IsActive();
}

bool FDisplayClusterViewportManagerViewPointExtension::IsActive() const
{
	return Configuration->GetViewportManager() != nullptr && CurrentStereoViewIndex != INDEX_NONE;
}

void FDisplayClusterViewportManagerViewPointExtension::SetupViewPoint(APlayerController* Player, FMinimalViewInfo& InOutViewInfo)
{
	uint32 ContextNum = 0;
	if (IDisplayClusterViewport* DCViewport = IsActive() ? Configuration->GetViewportManager()->FindViewport(CurrentStereoViewIndex, &ContextNum) : nullptr)
	{
		DCViewport->SetupViewPoint(ContextNum, InOutViewInfo);
	}
}

void FDisplayClusterViewportManagerViewPointExtension::SetupView(FSceneViewFamily& InViewFamily, FSceneView& InView)
{
	uint32 ContextNum = 0;
	if (IDisplayClusterViewport* DCViewport = IsActive() ? Configuration->GetViewportManager()->FindViewport(CurrentStereoViewIndex, &ContextNum) : nullptr)
	{
		UWorld* CurrentWorld = InViewFamily.Scene ? InViewFamily.Scene->GetWorld() : nullptr;
		if (!CurrentWorld)
		{
			CurrentWorld = Configuration->GetCurrentWorld();
		}
		if (CurrentWorld)
		{
			// Apply viewport context settings to view (crossGPU, visibility, etc)
			DCViewport->SetupSceneView(ContextNum, CurrentWorld, InViewFamily, InView);
		}
	}
}
