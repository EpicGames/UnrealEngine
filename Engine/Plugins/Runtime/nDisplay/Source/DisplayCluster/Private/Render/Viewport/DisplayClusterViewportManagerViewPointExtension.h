// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "SceneViewExtension.h"
#include "Render/Viewport/Configuration/DisplayClusterViewportConfiguration.h"

// This VE should be used before all others.
#define DISPLAYCLUSTER_SCENE_VIEWPOINT_EXTENSION_PRIORITY 1000

/**
 * View extension applying an DC Viewport features
 */
class FDisplayClusterViewportManagerViewPointExtension : public FSceneViewExtensionBase
{
public:
	FDisplayClusterViewportManagerViewPointExtension(const FAutoRegister& AutoRegister, const TSharedRef<FDisplayClusterViewportConfiguration, ESPMode::ThreadSafe>& InConfiguration);
	virtual ~FDisplayClusterViewportManagerViewPointExtension() = default;

public:
	//~ Begin ISceneViewExtension interface
	virtual void SetupViewPoint(APlayerController* Player, FMinimalViewInfo& InOutViewInfo) override;
	virtual void SetupView(FSceneViewFamily& InViewFamily, FSceneView& InView) override;

	virtual int32 GetPriority() const override { return DISPLAYCLUSTER_SCENE_VIEWPOINT_EXTENSION_PRIORITY; }

	/** Set current view index for this VE:
	 * This function must be used because LocalPlayer::GetViewPoint() calls the ISceneViewExtension::SetupViewPoint() function from this VE.
	 * And at this point the VE must know the current StereoViewIndex value in order to understand which viewport will be used for this SetupViewPoint() call.
	 * Set INDEX_NONE when this VE no longer needs to be used.
	 */
	void SetCurrentStereoViewIndex(const int32 InStereoViewIndex)
	{
		CurrentStereoViewIndex = InStereoViewIndex;
	}

protected:
	virtual bool IsActiveThisFrame_Internal(const FSceneViewExtensionContext& Context) const override;

private:
	/** True, if VE can be used at the moment. */
	bool IsActive() const;

public:
	// Configuration of the current cluster node
	const TSharedRef<FDisplayClusterViewportConfiguration, ESPMode::ThreadSafe> Configuration;

private:
	// Current StereoViewIndex for rendered viewport
	int32 CurrentStereoViewIndex = INDEX_NONE;
};
