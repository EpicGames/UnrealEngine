// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Viewport/DisplayClusterViewportProxy.h"

#include "CommonRenderResources.h"
#include "RenderGraphResources.h"
#include "RenderGraphUtils.h"
#include "SceneTextures.h"

#include "Engine/TextureRenderTarget.h"

#include "IDisplayCluster.h"
#include "IDisplayClusterCallbacks.h"
#include "IDisplayClusterShaders.h"
#include "IDisplayClusterShadersTextureUtils.h"

#include "Render/Containers/IDisplayClusterRender_MeshComponentProxy.h"
#include "Render/GUILayer/DisplayClusterGuiLayerController.h"
#include "Render/Projection/IDisplayClusterProjectionPolicy.h"
#include "Render/Viewport/DisplayClusterViewport.h"
#include "Render/Viewport/DisplayClusterViewportManagerProxy.h"
#include "Render/Viewport/Containers/DisplayClusterViewportProxyData.h"
#include "Render/Viewport/Resource/DisplayClusterViewportResource.h"

#include "ScreenPass.h"
#include "SceneTextures.h"
#include "PostProcess/PostProcessMaterialInputs.h"

namespace UE::DisplayCluster::ViewportProxy
{
	/**
	 * Collects the current eye adaptation data from the given scene view.
	 * 
	 * @param InSceneView  The scene view to query.
	 * @return A populated FDisplayClusterViewport_EyeAdaptationData struct with the latest auto-exposure data.
	 */
	static inline FDisplayClusterViewport_EyeAdaptationData GetEyeAdaptationData(const FSceneView& InSceneView)
	{
		// Get eye adaptation data from the FSceneView.
		FDisplayClusterViewport_EyeAdaptationData OutEyeAdaptationData;
		if (InSceneView.HasEyeAdaptationViewState())
		{
			OutEyeAdaptationData.LastAverageSceneLuminance = InSceneView.GetLastAverageSceneLuminance();
		}

		return OutEyeAdaptationData;
	}
};

///////////////////////////////////////////////////////////////////////////////////////
IDisplayClusterShaders& FDisplayClusterViewportProxy::GetShadersAPI()
{
	static IDisplayClusterShaders& ShadersAPISingleton = IDisplayClusterShaders::Get();

	return ShadersAPISingleton;
}

///////////////////////////////////////////////////////////////////////////////////////
FDisplayClusterViewportProxy::FDisplayClusterViewportProxy(
	const TSharedRef<FDisplayClusterViewportConfiguration, ESPMode::ThreadSafe>& InConfiguration,
	const FString& InViewportBaseId,
	const EDisplayClusterViewportFlags InViewportFlags,
	const FString& InViewportId,
	const TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe>& InProjectionPolicy)
	: ConfigurationProxy(InConfiguration->Proxy)
	, ViewportBaseId(InViewportBaseId)
	, ViewportFlags(InViewportFlags)
	, ViewportId(InViewportId)
	, ClusterNodeId(InConfiguration->GetClusterNodeId())
	, ProjectionPolicy(InProjectionPolicy)
	, Contexts(CurrentFrameData.Contexts)
{
	check(ProjectionPolicy.IsValid());
}

FDisplayClusterViewportProxy::~FDisplayClusterViewportProxy()
{ }

void FDisplayClusterViewportProxy::UpdateViewportProxyData_RenderThread(const FDisplayClusterViewportProxyData& InViewportProxyData)
{
	OpenColorIO = InViewportProxyData.OpenColorIO;

	DisplayDeviceProxy = InViewportProxyData.DisplayDeviceProxy;

	OverscanRuntimeSettings = InViewportProxyData.OverscanRuntimeSettings;

	RemapMesh = InViewportProxyData.RemapMesh;

	RenderSettings = InViewportProxyData.RenderSettings;

	RenderSettingsICVFX.SetParameters(InViewportProxyData.RenderSettingsICVFX);
	PostRenderSettings.SetParameters(InViewportProxyData.PostRenderSettings);

	ProjectionPolicy = InViewportProxyData.ProjectionPolicy;

	// Update frame data
	{
		// Update previous frame data
		uint32 CurrentFrameNumber = InViewportProxyData.CurrentFrameData.FrameNumber;
		if (CurrentFrameData.FrameNumber != CurrentFrameNumber)
		{
			PreviousFrameData = CurrentFrameData;
		}

		// Update current frame data
		CurrentFrameData = InViewportProxyData.CurrentFrameData;

		// Merge render-thread state (e.g. eye adaptation) captured in OnPostRenderViewFamily_RenderThread
		// into the game-thread copy. The game thread cannot observe these values directly.
		if (!CapturedRenderThreadData.IsEmpty())
		{
			for (FDisplayClusterViewport_Context& ContextIt : Contexts)
			{
				if (const FDisplayClusterViewport_Context::FRenderThreadData* CaptureData = CapturedRenderThreadData.Find(ContextIt.ContextNum))
				{
					ContextIt.RenderThreadData = *CaptureData;
				}
			}

			CapturedRenderThreadData.Reset();
		}
	}

	// Update viewport proxy resources from container
	Resources = InViewportProxyData.Resources;
	ViewStates = InViewportProxyData.ViewStates;

	// Discard any per-frame RHI overrides so stale external textures are never used in the next frame.
	TransientRHIOverrides.Reset();
}

//  Return viewport scene proxy resources by type
bool FDisplayClusterViewportProxy::GetResources_RenderThread(const EDisplayClusterViewportResourceType InExtResourceType, TArray<FRHITexture*>& OutResources) const
{
	// A per-frame override set via SetResourcesOverride_RenderThread takes precedence over the normal resource-fetch path.
	if(GetResourcesOverride_RenderThread(InExtResourceType, OutResources))
	{
		return true;
	}

	return ImplGetResources_RenderThread(InExtResourceType, OutResources, false);
}

const FDisplayClusterViewportProxy& FDisplayClusterViewportProxy::GetRenderingViewportProxy() const
{
	if(RenderSettings.GetViewportOverrideMode() != EDisplayClusterViewportOverrideMode::None)
	{
		if (FDisplayClusterViewportManagerProxy* ViewportManagerProxy = ConfigurationProxy->GetViewportManagerProxyImpl())
		{
			if (FDisplayClusterViewportProxy const* OverrideViewportProxy = ViewportManagerProxy->ImplFindViewportProxy_RenderThread(RenderSettings.GetViewportOverrideId()))
			{
				return *OverrideViewportProxy;
			}
		}
	}
	return *this;
}

bool FDisplayClusterViewportProxy::IsPostProcessDisabled() const
{
	if (ConfigurationProxy->GetRenderFrameSettings().IsPostProcessDisabled())
	{
		return true;
	}

	if (EnumHasAnyFlags(RenderSettingsICVFX.RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::UVLightcard | EDisplayClusterViewportRuntimeICVFXFlags::Lightcard | EDisplayClusterViewportRuntimeICVFXFlags::Chromakey))
	{
		return true;
	}

	return false;
}

EDisplayClusterViewportOpenColorIOMode FDisplayClusterViewportProxy::GetOpenColorIOMode() const
{
	if (OpenColorIO.IsValid() && OpenColorIO->IsValid_RenderThread())
	{
		if (IsPostProcessDisabled())
		{
			// Rendering without post-processing, OCIO is applied last, to the RTT texture of the viewport
			return EDisplayClusterViewportOpenColorIOMode::Resolved;
		}

		// By default, viewports render with a postprocess, OCIO must be done in between.
		return EDisplayClusterViewportOpenColorIOMode::PostProcess;
	}

	return EDisplayClusterViewportOpenColorIOMode::None;
}

void FDisplayClusterViewportProxy::PostResolveViewport_RenderThread(FRHICommandListImmediate& RHICmdList) const
{
	// resolve warped viewport resource to the output texture
	ResolveResources_RenderThread(RHICmdList, EDisplayClusterViewportResourceType::AfterWarpBlendTargetableResource, EDisplayClusterViewportResourceType::OutputTargetableResource);

	// Implement ViewportRemap feature
	ImplViewportRemap_RenderThread(RHICmdList);
}

void FDisplayClusterViewportProxy::ImplViewportRemap_RenderThread(FRHICommandListImmediate& RHICmdList) const
{
	// Preview in editor not support this feature
	if (ConfigurationProxy->IsPreviewRendering_RenderThread())
	{
		return;
	}

	if (RemapMesh.IsValid())
	{
		const IDisplayClusterRender_MeshComponentProxy* MeshProxy = RemapMesh->GetMeshComponentProxy_RenderThread();
		if (MeshProxy!=nullptr && MeshProxy->IsEnabled_RenderThread())
		{
			if (Resources[EDisplayClusterViewportResource::AdditionalFrameTargetableResources].Num() != Resources[EDisplayClusterViewportResource::OutputFrameTargetableResources].Num())
			{
				// error
				return;
			}

			for (int32 ContextIt = 0; ContextIt < Resources[EDisplayClusterViewportResource::AdditionalFrameTargetableResources].Num(); ContextIt++)
			{
				const TSharedPtr<FDisplayClusterViewportResource, ESPMode::ThreadSafe>& Src = Resources[EDisplayClusterViewportResource::AdditionalFrameTargetableResources][ContextIt];
				const TSharedPtr<FDisplayClusterViewportResource, ESPMode::ThreadSafe>& Dst = Resources[EDisplayClusterViewportResource::OutputFrameTargetableResources][ContextIt];

				FRHITexture* Input = Src.IsValid() ? Src->GetViewportResourceRHI_RenderThread() : nullptr;
				FRHITexture* Output = Dst.IsValid() ? Dst->GetViewportResourceRHI_RenderThread() : nullptr;

				if (Input && Output)
				{
					GetShadersAPI().RenderPostprocess_OutputRemap(RHICmdList, Input, Output, *MeshProxy);
				}
			}
		}
	}
}

bool FDisplayClusterViewportProxy::SetResourcesOverride_RenderThread(const EDisplayClusterViewportResourceType InResourceType, const TArray<FRHITexture*>& InResources, const TArray<FIntRect>* InOptRects) const
{
	// Empty array means "clear the override" — the normal resource-fetch path resumes next access.
	if(InResources.IsEmpty())
	{
		TransientRHIOverrides.Remove(InResourceType);

		return true;
	}

	TArray<FResourceWithRect> OverridenResourcesWithRects;

	// Build one entry per context. ContextNum is the caller-facing index into InResources/InRects.
	// Every context must have a valid texture — a partial override would leave nullptr entries that
	// crash downstream RHI calls, so we reject the whole batch if any context texture is missing.
	for(const FDisplayClusterViewport_Context& ContextIt : Contexts)
	{
		const uint32 ContextNum = ContextIt.ContextNum;

		if(!InResources.IsValidIndex(ContextNum) || !InResources[ContextNum])
		{
			return false;
		}

		FResourceWithRect ContextResource;
		ContextResource.Texture = InResources[ContextNum];

		// A zero rect signals "use the full texture" to the resource-fetch path.
		ContextResource.Rect = InOptRects && InOptRects->IsValidIndex(ContextNum) && !(*InOptRects)[ContextNum].IsEmpty()
		 	? (*InOptRects)[ContextNum]
			: FIntRect(FIntPoint::ZeroValue, ContextResource.Texture->GetDesc().Extent);

		OverridenResourcesWithRects.Add(ContextResource);
	}

	// Replace any previous override for this type; only one override per type is valid at a time.
	TransientRHIOverrides.Emplace(InResourceType, OverridenResourcesWithRects);

	return true;
}

bool FDisplayClusterViewportProxy::GetResourcesOverride_RenderThread(const EDisplayClusterViewportResourceType InResourceType, TArray<FRHITexture*>& OutResources, TArray<FIntRect>* OutOptRects) const
{
	const TArray<FResourceWithRect>* Overrides = TransientRHIOverrides.Find(InResourceType);
	if(!Overrides || Overrides->IsEmpty())
	{
		return false;
	}

	// Reset outputs before populating — callers must not assume prior contents are preserved.
	OutResources.Empty();

	if(OutOptRects)
	{
		OutOptRects->Empty();
	}

	for(const FResourceWithRect& ResourceIt : *Overrides)
	{
		// SetResourcesOverride_RenderThread rejects nulls, but guard here against stale data.
		if(!ResourceIt.Texture)
		{
			OutResources.Empty();
			if(OutOptRects)
			{
				OutOptRects->Empty();
			}
			return false;
		}

		OutResources.Add(ResourceIt.Texture);
		if(OutOptRects)
		{
			OutOptRects->Add(ResourceIt.Rect);
		}
	}

	return true;
}

bool FDisplayClusterViewportProxy::GetResourcesWithRects_RenderThread(const EDisplayClusterViewportResourceType InExtResourceType, TArray<FRHITexture*>& OutResources, TArray<FIntRect>& OutResourceRects) const
{
	// A per-frame override set via SetResourcesOverride_RenderThread takes precedence over the normal resource-fetch path.
	if(GetResourcesOverride_RenderThread(InExtResourceType, OutResources, &OutResourceRects))
	{
		return true;
	}

	return ImplGetResourcesWithRects_RenderThread(InExtResourceType, OutResources, OutResourceRects, false);
}

void FDisplayClusterViewportProxy::UpdateDeferredResources(FRHICommandListImmediate& RHICmdList) const
{
	check(IsInRenderingThread());

	if (RenderSettings.bFreezeRendering || RenderSettings.bSkipRendering)
	{
		// Disable deferred update
		return;
	}

	// Tiled viewports simply copy their RTT to the RTT of the source viewport.
	if (RenderSettings.TileSettings.GetType() == EDisplayClusterViewportTileType::Tile)
	{
		if (FDisplayClusterViewportManagerProxy* ViewportManagerProxy = ConfigurationProxy->GetViewportManagerProxyImpl())
		{
			if (FDisplayClusterViewportProxy const* SourceViewportProxy = ViewportManagerProxy->ImplFindViewportProxy_RenderThread(RenderSettings.TileSettings.GetSourceViewportId()))
			{
				// Copy tile to the source
				ImplResolveTileResource_RenderThread(RHICmdList, SourceViewportProxy);
			}
		}

		// The tile has been copied. This viewport is no longer needed.
		// All of the following logic is applied later, in the tile source viewport.
		return;
	}

	switch (RenderSettings.GetViewportOverrideMode())
	{
		case EDisplayClusterViewportOverrideMode::All:
		case EDisplayClusterViewportOverrideMode::InternalViewportResources:
		// Disable deferred update for clone viewports
		return;

		default:
			break;
	}

	const FDisplayClusterViewportProxy& SourceViewportProxy = GetRenderingViewportProxy();
	if(!SourceViewportProxy.IsInputRenderTargetResourceExists())
	{
		// No input RTT resource for deferred update
		return;
	}

	EDisplayClusterViewportResourceType SrcResourceType = EDisplayClusterViewportResourceType::InternalRenderTargetResource;

	// pre-Pass 0 (Projection policy):The projection policy can use its own method to resolve 'InternalRenderTargetResource' to 'InputShaderResource'
	if (ProjectionPolicy.IsValid() && ProjectionPolicy->ResolveInternalRenderTargetResource_RenderThread(RHICmdList, this, &SourceViewportProxy))
	{
		SrcResourceType = EDisplayClusterViewportResourceType::InputShaderResource;
	}

	if (GetOpenColorIOMode() == EDisplayClusterViewportOpenColorIOMode::Resolved)
		{
		// Pass 0:  OCIO + Linear gamma
		// At this point Resolver go to use RDG
		OpenColorIO->AddPass_RenderThread(
			FDisplayClusterShadersTextureUtilsSettings(),
			GetShadersAPI().CreateTextureUtils_RenderThread(RHICmdList)
				->SetInput(&SourceViewportProxy, SrcResourceType)
				->SetOutput(this, EDisplayClusterViewportResourceType::InputShaderResource));
		}
	else
	{
		// Pass 0: Linear gamma
		GetShadersAPI().CreateTextureUtils_RenderThread(RHICmdList)
			->SetInput(&SourceViewportProxy, SrcResourceType)
			->SetOutput(this, EDisplayClusterViewportResourceType::InputShaderResource)
			->Resolve();
	}

	// (Opt) Pass 1: Generate blur postprocess effect for render target texture rect for all contexts
	if (PostRenderSettings.PostprocessBlur.IsEnabled())
	{
		GetShadersAPI().CreateTextureUtils_RenderThread(RHICmdList)
			->SetOutput(this, EDisplayClusterViewportResourceType::InputShaderResource)
			->ForEachContextByPredicate(EDisplayClusterShaderTextureUtilsFlags::UseOutputTextureAsInput,
				[&](
					const FDisplayClusterShadersTextureViewportContext& Input,
					const FDisplayClusterShadersTextureViewportContext& Output)
				{
					GetShadersAPI().RenderPostprocess_Blur(RHICmdList, Input.TextureRHI, Output.TextureRHI, PostRenderSettings.PostprocessBlur);
				});
	}

	// Pass 2: Create mips texture and generate mips from render target rect for all contexts
	if (PostRenderSettings.GenerateMips.IsEnabled())
	{
		GetShadersAPI().CreateTextureUtils_RenderThread(RHICmdList)
			->SetInput(&SourceViewportProxy, EDisplayClusterViewportResourceType::InputShaderResource)
			->SetOutput(this, EDisplayClusterViewportResourceType::MipsShaderResource)
			->Resolve() // Copy `Input`->`Mips`
			->ForEachContextByPredicate([&](
				const FDisplayClusterShadersTextureViewportContext& Input,
				const FDisplayClusterShadersTextureViewportContext& Output)
			{
				// Generate mips
				GetShadersAPI().GenerateMips(RHICmdList, Output.TextureRHI, PostRenderSettings.GenerateMips);
			});
	}
}

// Resolve resource contexts
bool FDisplayClusterViewportProxy::ResolveResources_RenderThread(FRHICommandListImmediate& RHICmdList, const EDisplayClusterViewportResourceType InExtResourceType, const EDisplayClusterViewportResourceType OutExtResourceType, const int32 InContextNum) const
{
	return ImplResolveResources_RenderThread(RHICmdList, this, InExtResourceType, OutExtResourceType, InContextNum);
}

bool FDisplayClusterViewportProxy::ResolveResources_RenderThread(FRHICommandListImmediate& RHICmdList, IDisplayClusterViewportProxy* InputResourceViewportProxy, const EDisplayClusterViewportResourceType InExtResourceType, const EDisplayClusterViewportResourceType OutExtResourceType, const int32 InContextNum) const
{
	const FDisplayClusterViewportProxy* SourceProxy = static_cast<FDisplayClusterViewportProxy*>(InputResourceViewportProxy);

	return ImplResolveResources_RenderThread(RHICmdList, SourceProxy , InExtResourceType, OutExtResourceType, InContextNum);
}

void FDisplayClusterViewportProxy::OnResolvedSceneColor_RenderThread(FRDGBuilder& GraphBuilder, const FSceneTextures& SceneTextures, const FDisplayClusterViewportProxy_Context& InProxyContext)
{
	const uint32 InContextNum = InProxyContext.ContextNum;
	if (ShouldUseAlphaChannel_RenderThread())
	{
		switch (ConfigurationProxy->GetRenderFrameSettings().AlphaChannelCaptureMode)
		{
		case EDisplayClusterRenderFrameAlphaChannelCaptureMode::FXAA:
		case EDisplayClusterRenderFrameAlphaChannelCaptureMode::Copy:
		case EDisplayClusterRenderFrameAlphaChannelCaptureMode::CopyAA:
		{
			const FIntRect SrcRect = GetResourceRect_RenderThread(EDisplayClusterViewportResourceType::InternalRenderTargetResource, InContextNum);
			// Copy alpha channel from 'SceneTextures.Color.Resolve' to 'InputShaderResource'
			GetShadersAPI().CreateTextureUtils_RenderThread(GraphBuilder)
				->SetInput({ SceneTextures.Color.Resolve, SrcRect }, InContextNum)
				->SetOutput(this, EDisplayClusterViewportResourceType::InputShaderResource, InContextNum)
				->Resolve(EColorWriteMask::CW_ALPHA);
		}
		break;

		default:
			break;
		}
	}
}

FScreenPassTexture FDisplayClusterViewportProxy::OnPostProcessPassAfterSSRInput_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& View, const FPostProcessMaterialInputs& Inputs, const uint32 ContextNum)
{
	FScreenPassTexture OutScreenPassTexture = Inputs.ReturnUntouchedSceneColorForPostProcessing(GraphBuilder);
	if (OutScreenPassTexture.IsValid())
	{
		// Copy alpha channel to 'InputShaderResource'
		const FIntRect SrcRect = GetResourceRect_RenderThread(EDisplayClusterViewportResourceType::InternalRenderTargetResource, ContextNum);
		GetShadersAPI().CreateTextureUtils_RenderThread(GraphBuilder)
			->SetInput({ OutScreenPassTexture.Texture, SrcRect }, ContextNum)
			->SetOutput(this, EDisplayClusterViewportResourceType::InputShaderResource, ContextNum)
			->Resolve(EColorWriteMask::CW_ALPHA);
	}

	return OutScreenPassTexture;
}

FScreenPassTexture FDisplayClusterViewportProxy::OnPostProcessPassAfterFXAA_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& View, const FPostProcessMaterialInputs& Inputs, const uint32 ContextNum)
{
	FScreenPassTexture OutScreenPassTexture = Inputs.ReturnUntouchedSceneColorForPostProcessing(GraphBuilder);
	if (OutScreenPassTexture.IsValid())
	{
		// Restore alpha channel after OCIO
		// Copy alpha channel from 'InputShaderResource'
		const FIntRect DestRect = GetResourceRect_RenderThread(EDisplayClusterViewportResourceType::InternalRenderTargetResource, ContextNum);
		GetShadersAPI().CreateTextureUtils_RenderThread(GraphBuilder)
			->SetInput(this, EDisplayClusterViewportResourceType::InputShaderResource, ContextNum)
			->SetOutput({ OutScreenPassTexture.Texture, DestRect }, ContextNum)
			->Resolve(EColorWriteMask::CW_ALPHA);
	}

	return OutScreenPassTexture;
}

FScreenPassTexture FDisplayClusterViewportProxy::OnPostProcessPassAfterTonemap_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& View, const FPostProcessMaterialInputs& Inputs, const uint32 ContextNum)
{
	// Broadcast PassTonemap event
	IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPostTonemapPass_RenderThread().Broadcast(GraphBuilder, this, View, Inputs, ContextNum);

	// Perform OCIO rendering after the tonemapper
	if (GetOpenColorIOMode() == EDisplayClusterViewportOpenColorIOMode::PostProcess)
	{
		// Add OCIO pass
		return OpenColorIO->PostProcessPassAfterTonemap_RenderThread(GraphBuilder, GetResourceColorEncoding_RenderThread(EDisplayClusterViewportResourceType::InternalRenderTargetResource), View, Inputs);
	}

	return Inputs.ReturnUntouchedSceneColorForPostProcessing(GraphBuilder);
}

const FDisplayClusterViewport_Context::FRenderThreadData& FDisplayClusterViewportProxy::GetEffectiveRenderThreadData(const int32 InContextNum) const
{
	// Prefer data captured this frame in OnPostRenderViewFamily_RenderThread — it reflects
	// render-thread state (e.g. eye adaptation) that the game-thread copy doesn't have yet.
	if (CapturedRenderThreadData.Contains(InContextNum))
	{
		return CapturedRenderThreadData[InContextNum];
	}

	// Fall back to the game-thread copy synced at the start of the frame.
	if (Contexts.IsValidIndex(InContextNum))
	{
		return Contexts[InContextNum].RenderThreadData;
	}

	// Neither source is available (invalid context index or before first render).
	static const FDisplayClusterViewport_Context::FRenderThreadData UndefinedRenderThreadData;

	return UndefinedRenderThreadData;
}

void FDisplayClusterViewportProxy::OnPostRenderViewFamily_RenderThread(FRDGBuilder& GraphBuilder, FSceneViewFamily& InViewFamily, const FSceneView& InSceneView, const FDisplayClusterViewportProxy_Context& InProxyContext)
{
	const uint32 InContextNum = InProxyContext.ContextNum;

	// Capture Render thread data
	FDisplayClusterViewport_Context::FRenderThreadData ContextData;
	{
#if WITH_MGPU
		// Get the GPUIndex used to render this viewport
		const uint32 GPUIndex = InSceneView.GPUMask.GetFirstIndex();
		ContextData.GPUIndex = (GPUIndex < GNumExplicitGPUsForRendering) ? GPUIndex : -1;
#endif

		ContextData.EngineDisplayGamma = InSceneView.Family->RenderTarget->GetDisplayGamma();
		ContextData.EngineShowFlags = InSceneView.Family->EngineShowFlags;
		ContextData.FrameNumber = InSceneView.Family->FrameNumber;

		// Collect EyeAdaptation data from FSceneView.
		ContextData.EyeAdaptationData = UE::DisplayCluster::ViewportProxy::GetEyeAdaptationData(InSceneView);
	}
	CapturedRenderThreadData.Emplace(InContextNum, ContextData);

	// Allow the GUI controller to process output textures before PostRenderViewFamily is broadcasted. A dedicated
	// call is required to avoid any potential issues caused by the event call order.
	FDisplayClusterGuiLayerController::Get().ProcessPostRenderViewFamily_RenderThread(GraphBuilder, InViewFamily, this);

	if (!InProxyContext.ViewFamilyProfileDescription.IsEmpty())
	{
		static IDisplayClusterCallbacks& DCCallbacksAPI = IDisplayCluster::Get().GetCallbacks();
		if (DCCallbacksAPI.OnDisplayClusterPostRenderViewFamily_RenderThread().IsBound())
		{
			// Now we can perform viewport notification
			DCCallbacksAPI.OnDisplayClusterPostRenderViewFamily_RenderThread().Broadcast(GraphBuilder, InViewFamily, this);
		}
	}

	if (ShouldUseAlphaChannel_RenderThread())
	{
		const EDisplayClusterRenderFrameAlphaChannelCaptureMode AlphaChannelCaptureMode = ConfigurationProxy->GetRenderFrameSettings().AlphaChannelCaptureMode;
		switch (AlphaChannelCaptureMode)
		{
		case EDisplayClusterRenderFrameAlphaChannelCaptureMode::Copy:
		case EDisplayClusterRenderFrameAlphaChannelCaptureMode::CopyAA:
		case EDisplayClusterRenderFrameAlphaChannelCaptureMode::FXAA:
		{
			// RenderPass 1: Copy Alpha channels back from 'InputShaderResource' to 'InternalRenderTargetResource'
			TSharedRef<IDisplayClusterShadersTextureUtils> Resolver = GetShadersAPI().CreateTextureUtils_RenderThread(GraphBuilder)
				->SetInput(this, EDisplayClusterViewportResourceType::InputShaderResource, InContextNum)
				->SetOutput(this, EDisplayClusterViewportResourceType::InternalRenderTargetResource, InContextNum)
				->Resolve(EColorWriteMask::CW_ALPHA);

			EFXAAQuality FXAAQuality = EFXAAQuality::Q0;
			if (AlphaChannelCaptureMode == EDisplayClusterRenderFrameAlphaChannelCaptureMode::FXAA &&
				ShouldApplyFXAA_RenderThread(FXAAQuality))
			{
				Resolver
					->ForEachContextByPredicate([&](
						const FDisplayClusterShadersTextureViewportContext& Input,
						const FDisplayClusterShadersTextureViewportContext& Output)
						{
							// RenderPass 2: Do FXAA with 'InternalRenderTargetResource' as input
							FFXAAInputs PassInputs;
							PassInputs.SceneColor = Output.ToScreenPassTexture();
							PassInputs.Quality = FXAAQuality;
							FScreenPassTexture FXAAColorTexture = AddFXAAPass(GraphBuilder, InSceneView, PassInputs);

							// RenderPass 3: Copy FXAA result(RGB) back to the 'InternalRenderTargetResource'
							Resolver->ResolveTextureContext(
								EColorWriteMask::CW_RGB,
								{ {FXAAColorTexture}, Output.ColorEncoding },
								Output);
						});
			}
		}
		break;

		default:
			break;
		}
	}
}

void FDisplayClusterViewportProxy::ReleaseAllResources_RenderThread()
{
	// Release textures
	Resources.ReleaseAllResources();

	// Release references to the ViewState data
	ViewStates.Reset();
}

FDisplayClusterColorEncoding FDisplayClusterViewportProxy::GetResourceColorEncoding_RenderThread(const EDisplayClusterViewportResourceType InResourceType) const
{
	const EDisplayClusterViewportResourceType ResourceType = GetResourceType_RenderThread(InResourceType);
	switch (ResourceType)
	{
	case EDisplayClusterViewportResourceType::InternalRenderTargetEntireRectResource:
	case EDisplayClusterViewportResourceType::InternalRenderTargetResource:

		if (EnumHasAnyFlags(RenderSettingsICVFX.RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::UVLightcard | EDisplayClusterViewportRuntimeICVFXFlags::Lightcard))
		{
			// The LightCard viewport should always be rendered in linear color space with inverted alpha.
			// ICVFX expects all lightcards in linear color space (blending purpose)
			return { EDisplayClusterColorEncoding::Linear, EDisplayClusterColorPremultiply::InvertPremultiply };
		}

		if (!Contexts.IsEmpty())
		{
			// There is a special case where post processing and tonemapper are disabled. In this case tonemapper applies a static display Inverse of Gamma which defaults to 2.2.
			const FEngineShowFlags& EngineShowFlags = Contexts[0].RenderThreadData.EngineShowFlags;
			if (EngineShowFlags.Tonemapper == 0 || EngineShowFlags.PostProcessing == 0)
			{
				return { EDisplayClusterColorEncoding::Linear };
			}

			const float DefaultDisplayGamma = UTextureRenderTarget::GetDefaultDisplayGamma();
			const float DisplayGamma = GetEffectiveRenderThreadData(0).EngineDisplayGamma;
			if (FMath::IsNearlyEqual(DisplayGamma, DefaultDisplayGamma))
			{
				return { EDisplayClusterColorEncoding::Gamma };
			}

			// Custom gamma value is different from default
			return { DisplayGamma };
		}

		break;

	// Preview Output
	case EDisplayClusterViewportResourceType::OutputPreviewTargetableResource:
		if (ConfigurationProxy->GetRenderFrameSettings().ShouldUseHoldout())
		{
			// The HoldoutComposite plugin expects the input in linear gamma.
			return { EDisplayClusterColorEncoding::Linear };
		}
		break;

	default:
		break;
	}

	if (EnumHasAnyFlags(RenderSettingsICVFX.RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::UVLightcard | EDisplayClusterViewportRuntimeICVFXFlags::Lightcard))
	{
		// after the OCIO color space isn't linear anymore for LightCards.
		return { EDisplayClusterColorEncoding::Gamma, EDisplayClusterColorPremultiply::InvertPremultiply };
	}

	if (IsPostProcessDisabled())
	{
		return { EDisplayClusterColorEncoding::Linear };
	}

	if (!Contexts.IsEmpty())
	{
		const float DefaultDisplayGamma = UTextureRenderTarget::GetDefaultDisplayGamma();
		const float DisplayGamma = GetEffectiveRenderThreadData(0).EngineDisplayGamma;
		if (!FMath::IsNearlyEqual(DisplayGamma, DefaultDisplayGamma))
		{
			return { DisplayGamma };
		}
	}

	return { EDisplayClusterColorEncoding::Gamma };
}