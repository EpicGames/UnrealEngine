// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Render/Viewport/IDisplayClusterViewportProxy.h"
#include "Render/Viewport/DisplayClusterViewportResources.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_OverscanRuntimeSettings.h"
#include "Render/Viewport/Configuration/DisplayClusterViewportConfigurationProxy.h"

#include "Render/DisplayDevice/IDisplayClusterDisplayDeviceProxy.h"

#include "PostProcess/PostProcessAA.h"

#include "EngineUtils.h"
#include "ScreenRendering.h"
#include "SceneView.h"
#include "Templates/SharedPointer.h"

class FDisplayClusterViewport;
class IDisplayClusterProjectionPolicy;
class IDisplayClusterRender_MeshComponent;
class FDisplayClusterViewport_OpenColorIO;

class FRDGBuilder;
class FSceneView;
struct FPostProcessMaterialInputs;
struct FScreenPassTexture;
struct FSceneTextures;

/**
 * OCIO is applied in different ways. It depends on the rendering workflow
 */
enum class EDisplayClusterViewportOpenColorIOMode : uint8
{
	None = 0,

	// When the viewport renders with a postprocess, OCIO must be done in between
	PostProcess,

	// When the viewport is rendered without postprocessing, OCIO is applied last, to the RTT texture of the viewport
	Resolved
};

/**
 * nDisplay viewport proxy implementation.
 */
class FDisplayClusterViewportProxy
	: public IDisplayClusterViewportProxy
	, public TSharedFromThis<FDisplayClusterViewportProxy, ESPMode::ThreadSafe>
{
public:
	/**
	* Creates a viewport proxy instance.
	* 
	* @param InConfiguration    Viewport configuration object (thread-safe).
	* @param InViewportBaseId   The base viewport identifier (before ICVFX unique naming).
	* @param InViewportFlags    Usage type of the viewport (e.g. InCamera, Chromakey, LightCard).
	* @param InViewportId       A viewport name string unique within the cluster.
	* @param InProjectionPolicy Projection policy assigned to the viewport
	*                           (defines how the scene is projected/rendered).
	*/
	FDisplayClusterViewportProxy(
		const TSharedRef<FDisplayClusterViewportConfiguration, ESPMode::ThreadSafe>& InConfiguration,
		const FString& InViewportBaseId,
		const EDisplayClusterViewportFlags InViewportFlags,
		const FString& InViewportId,
		const TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe>& InProjectionPolicy);

	virtual ~FDisplayClusterViewportProxy();

public:
	//~~ BEGIN IDisplayClusterViewportProxy
	virtual TSharedPtr<IDisplayClusterViewportProxy, ESPMode::ThreadSafe> ToSharedPtr() override
	{
		return AsShared();
	}

	virtual TSharedPtr<const IDisplayClusterViewportProxy, ESPMode::ThreadSafe> ToSharedPtr() const override
	{
		return AsShared();
	}

	virtual TSharedRef<IDisplayClusterViewportProxy, ESPMode::ThreadSafe> ToSharedRef() override
	{
		return AsShared();
	}

	virtual TSharedRef<const IDisplayClusterViewportProxy, ESPMode::ThreadSafe> ToSharedRef() const override
	{
		return AsShared();
	}

	virtual const IDisplayClusterViewportConfigurationProxy& GetConfigurationProxy() const override
	{
		return *ConfigurationProxy;
	}

	virtual EDisplayClusterViewportFlags GetViewportFlags() const override
	{
		return ViewportFlags;
	}

	virtual FString GetBaseId() const override
	{
		return ViewportBaseId;
	}

	virtual FString GetId() const override
	{
		check(IsInRenderingThread());
		return ViewportId;
	}

	virtual FString GetClusterNodeId() const override
	{
		check(IsInRenderingThread());
		return ClusterNodeId;
	}

	virtual const FDisplayClusterViewport_RenderSettings& GetRenderSettings_RenderThread() const override
	{
		check(IsInRenderingThread());
		return RenderSettings;
	}

	virtual const FDisplayClusterViewport_RenderSettingsICVFX& GetRenderSettingsICVFX_RenderThread() const override
	{
		check(IsInRenderingThread());
		return RenderSettingsICVFX;
	}

	virtual const FDisplayClusterViewport_PostRenderSettings& GetPostRenderSettings_RenderThread() const override
	{
		check(IsInRenderingThread());
		return PostRenderSettings;
	}

	virtual const TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe>& GetProjectionPolicy_RenderThread() const override
	{
		check(IsInRenderingThread());
		return ProjectionPolicy;
	}
	
	virtual const TArray<FDisplayClusterViewport_Context>& GetContexts_RenderThread() const override
	{
		check(IsInRenderingThread());
		return Contexts;
	}

	virtual const FDisplayClusterViewportFrameData& GetFrameData_RenderThread() const override
	{
		check(IsInRenderingThread());
		return CurrentFrameData;
	}

	virtual const FDisplayClusterViewportFrameData& GetPrevFrameData_RenderThread() const override
	{
		check(IsInRenderingThread());
		return PreviousFrameData;
	}

	virtual void SetRenderSettings_RenderThread(const FDisplayClusterViewport_RenderSettings& InRenderSettings) const override
	{
		check(IsInRenderingThread());
		RenderSettings = InRenderSettings;
	}

	virtual void SetContexts_RenderThread(const TArray<FDisplayClusterViewport_Context>& InContexts) const override
	{
		check(IsInRenderingThread());
		Contexts.Empty();
		Contexts.Append(InContexts);
	}

	virtual FDisplayClusterColorEncoding GetResourceColorEncoding_RenderThread(const EDisplayClusterViewportResourceType InResourceType) const override;

	//  Return viewport scene proxy resources by type
	virtual bool GetResources_RenderThread(const EDisplayClusterViewportResourceType InResourceType, TArray<FRHITexture*>& OutResources) const override;
	virtual bool GetResourcesWithRects_RenderThread(const EDisplayClusterViewportResourceType InResourceType, TArray<FRHITexture*>& OutResources, TArray<FIntRect>& OutRects) const override;

	// Per-frame resource overrides; shadow the normal resource-fetch path and are cleared each frame.
	virtual bool SetResourcesOverride_RenderThread(const EDisplayClusterViewportResourceType InResourceType, const TArray<FRHITexture*>& InResources, const TArray<FIntRect>* InOptRects = nullptr) const override;
	virtual bool GetResourcesOverride_RenderThread(const EDisplayClusterViewportResourceType InResourceType, TArray<FRHITexture*>& OutResources, TArray<FIntRect>* OutOptRects = nullptr) const override;

	// Resolve resource contexts
	virtual bool ResolveResources_RenderThread(FRHICommandListImmediate& RHICmdList, const EDisplayClusterViewportResourceType InputResourceType, const EDisplayClusterViewportResourceType OutputResourceType, const int32 InContextNum = INDEX_NONE) const override;
	virtual bool ResolveResources_RenderThread(FRHICommandListImmediate& RHICmdList, IDisplayClusterViewportProxy* InputResourceViewportProxy, const EDisplayClusterViewportResourceType InputResourceType, const EDisplayClusterViewportResourceType OutputResourceType, const int32 InContextNum = INDEX_NONE) const override;

	//~~ END IDisplayClusterViewportProxy

public:
	/**
	* Fill the entire texture with the specified color.
	* 
	* @param InRenderTargetTexture - texture,  must be RTT.
	* @param InColor - the texture will be filled with this color.
	*/
	static void FillTextureWithColor_RenderThread(FRHICommandListImmediate& RHICmdList, FRHITexture* InRenderTargetTexture, const FLinearColor& InColor);

	/** Get valid resource type
	 * 
	 * @param InResourceType - the requested resource type from the entire namespace
	 * 
	 * @return - the type of resource actually used, depending on the current configuration of the rendering pipeline
	 */
	EDisplayClusterViewportResourceType GetResourceType_RenderThread(const EDisplayClusterViewportResourceType& InResourceType) const;

	/** Get actual region for viewport context
	 * 
	 * @param InResourceType - the requested resource type from the entire namespace
	 * @param InContextNum - context num (eye)
	 * 
	 * @return - InRect with applied overscan for RTT
	 */
	FIntRect GetResourceRect_RenderThread(const EDisplayClusterViewportResourceType InResourceType, const uint32 InContextNum) const;

	/** Resolve viewport RTT: render OCIO, PP, generate MIPS, etc.
	 *
	 * @param RHICmdList - RHI interface
	 *
	 * @return - none
	 */
	void UpdateDeferredResources(FRHICommandListImmediate& RHICmdList) const;

	/** Called at the end of the frame, after all callbacks.
	* At the end, some resources may be filled with black, etc.
	* This is useful because the resources are reused and the image from the previous frame goes into the new one.
	*/
	void CleanupResources_RenderThread(FRHICommandListImmediate& RHICmdList) const;

	/** nDisplay VE Callback [subscribed to Renderer:ResolvedSceneColorCallbacks].
	 *
	 * @param GraphBuilder   - RDG interface
	 * @param SceneTextures  - Scene textures (SceneColor, Depth, etc)
	 * @param InProxyContext - Saved from game thread context for this proxy
	 *
	 * @return - none
	 */
	void OnResolvedSceneColor_RenderThread(FRDGBuilder& GraphBuilder, const FSceneTextures& SceneTextures, const FDisplayClusterViewportProxy_Context& InProxyContext);

	/** nDisplay VE Callback [PostRenderViewFamily_RenderThread()].
	 *
	 * @param GraphBuilder   - RDG interface
	 * @param InViewFamily   - Scene View Family
	 * @param InSceneView    - Scene View
	 * @param InProxyContext - Saved from game thread context for this proxy
	 *
	 * @return - none
	 */
	void OnPostRenderViewFamily_RenderThread(FRDGBuilder& InGraphBuilder, FSceneViewFamily& InViewFamily, const FSceneView& InSceneView, const FDisplayClusterViewportProxy_Context& InProxyContext);

	/** Allow callback OnPostProcessPassAfterFXAA.  */
	bool ShouldUsePostProcessPassAfterFXAA() const;

	/** Callback OnPostProcessPassAfterFXAA.
	 *
	 * @param GraphBuilder - RDG interface
	 * @param View         - Scene View
	 * @param Inputs       - PP Input resources
	 * @param ContextNum   - viewport context index
	 *
	 * @return - Screen pass texture
	 */
	FScreenPassTexture OnPostProcessPassAfterFXAA_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& View, const FPostProcessMaterialInputs& Inputs, const uint32 ContextNum);

	/** Allow callback OnPostProcessPassAfterSSRInput. */
	bool ShouldUsePostProcessPassAfterSSRInput() const;

	/** Callback OnPostProcessPassAfterSSRInput.
	 *
	 * @param GraphBuilder - RDG interface
	 * @param View         - Scene View
	 * @param Inputs       - PP Input resources
	 * @param ContextNum   - viewport context index
	 *
	 * @return - Screen pass texture
	 */
	FScreenPassTexture OnPostProcessPassAfterSSRInput_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& View, const FPostProcessMaterialInputs& Inputs, const uint32 ContextNum);

	/** Allow callback OnPostProcessPassAfterTonemap. */
	bool ShouldUsePostProcessPassTonemap() const;

	/** Callback OnPostProcessPassAfterTonemap.
	 *
	 * @param GraphBuilder - RDG interface
	 * @param View         - Scene View
	 * @param Inputs       - PP Input resources
	 * @param ContextNum   - viewport context index
	 *
	 * @return - Screen pass texture
	 */
	FScreenPassTexture OnPostProcessPassAfterTonemap_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& View, const FPostProcessMaterialInputs& Inputs, const uint32 ContextNum);

	/** Enable alpha channel for this viewport (useful for overlays with alpha channel: ChromaKey, LightCard). */
	bool ShouldUseAlphaChannel_RenderThread() const;

	/** Finally, resolve the viewport to the output RTT and apply the last PPs (ViewportRemap, etc.)
	 *
	 * @param RHICmdList - RHI interface
	 *
	 * @return - none
	 */
	void PostResolveViewport_RenderThread(FRHICommandListImmediate& RHICmdList) const;

	/** Release all resources on the rendering thread. */
	void ReleaseAllResources_RenderThread();

	inline bool FindContext_RenderThread(const int32 ViewIndex, uint32* OutContextNum)
	{
		check(IsInRenderingThread());

		for (const FDisplayClusterViewport_Context& ContextIt : Contexts)
		{
			if (ViewIndex == ContextIt.StereoViewIndex)
			{
				if (OutContextNum != nullptr)
				{
					*OutContextNum = ContextIt.ContextNum;
				}

				return true;
			}
		}		

		return false;
	}

public:
	/* Returns true if the warp can be applied to this viewport. */
	bool ShouldApplyWarpBlend_RenderThread() const;

	const FDisplayClusterViewportResources& GetResources_RenderThread() const
	{
		return Resources;
	}

	void UpdateViewportProxyData_RenderThread(const class FDisplayClusterViewportProxyData& InViewportProxyData);

	/** Viewports should be processed in the appropriate order.
	* Viewports with lower priority values will be processed earlier.
	*/
	uint8 GetPriority_RenderThread() const;

	/** Returns the OCIO rendering type for the given viewport. */
	EDisplayClusterViewportOpenColorIOMode GetOpenColorIOMode() const;

	/** Return shaders API. */
	static class IDisplayClusterShaders& GetShadersAPI();

private:
	bool ImplGetResourcesWithRects_RenderThread(const EDisplayClusterViewportResourceType InResourceType, TArray<FRHITexture*>& OutResources, TArray<FIntRect>& OutResourceRects, const int32 InRecursionDepth) const;
	bool ImplGetResources_RenderThread(const EDisplayClusterViewportResourceType InResourceType, TArray<FRHITexture*>& OutResources, const int32 InRecursionDepth) const;

	void ImplViewportRemap_RenderThread(FRHICommandListImmediate& RHICmdList) const;

	bool ImplResolveResources_RenderThread(FRHICommandListImmediate& RHICmdList, FDisplayClusterViewportProxy const* SourceProxy, const EDisplayClusterViewportResourceType InputResourceType, const EDisplayClusterViewportResourceType OutputResourceType, const int32 InContextNum = INDEX_NONE) const;

	/** Copy the tile to the target viewport. */
	void ImplResolveTileResource_RenderThread(FRHICommandListImmediate& RHICmdList, FDisplayClusterViewportProxy const* InDestViewportProxy) const;

	/** Returns true if this viewport requires FXAA to be applied.
	* 
	* @param OutFXAAQuality - (out) FXAA shader parameter that define quality level.
	*/
	bool ShouldApplyFXAA_RenderThread(EFXAAQuality& OutFXAAQuality) const;

	/** When a resource by type can be overridden from another viewport, true is returned. */
	bool ShouldOverrideViewportResource(const EDisplayClusterViewportResourceType InResourceType) const;

	/** Return viewport used to render RTT (support ViewportOverride)
	 *
	 * @return - viewport proxy with valid render resources
	 */
	const FDisplayClusterViewportProxy& GetRenderingViewportProxy() const;

	/** Check if there is an RTT source (internal or external) in this viewport proxy. */
	bool IsInputRenderTargetResourceExists() const;
	
	/** Return true if postprocess is disbled for this viewport. */
	bool IsPostProcessDisabled() const;

	/** Returns the render thread data for the given context, preferring captured data over current frame data. */
	const FDisplayClusterViewport_Context::FRenderThreadData& GetEffectiveRenderThreadData(const int32 InContextNum) const;

public:
	// Configuration for proxy
	const TSharedRef<FDisplayClusterViewportConfigurationProxy, ESPMode::ThreadSafe> ConfigurationProxy;

	// Base viewport identifier. 
	// May not be unique across ICVFX viewports (multiple viewports can share the same base ID).
	const FString ViewportBaseId;

	// Usage type of the viewport (e.g. InCamera, Chromakey, LightCard).
	const EDisplayClusterViewportFlags ViewportFlags;

	// Final unique viewport identifier. 
	// Guaranteed to be unique across the entire cluster (includes node ID and usage type if needed).
	const FString ViewportId;

	/** Cluster node name. */
	const FString ClusterNodeId;

protected:
	/** OpenColorIO nDisplay interface ref. */
	TSharedPtr<FDisplayClusterViewport_OpenColorIO, ESPMode::ThreadSafe> OpenColorIO;

	/** Display Device Proxy. */
	TSharedPtr<IDisplayClusterDisplayDeviceProxy, ESPMode::ThreadSafe> DisplayDeviceProxy;

	// Viewport render params
	mutable FDisplayClusterViewport_RenderSettings RenderSettings;

	FDisplayClusterViewport_RenderSettingsICVFX  RenderSettingsICVFX;
	FDisplayClusterViewport_PostRenderSettings   PostRenderSettings;

	// Additional parameters
	FDisplayClusterViewport_OverscanRuntimeSettings OverscanRuntimeSettings;

	TSharedPtr<IDisplayClusterRender_MeshComponent, ESPMode::ThreadSafe> RemapMesh;

	// Projection policy instance that serves this viewport
	TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe> ProjectionPolicy;

	/** Frame data snapshot for the current frame (contexts, cached view info). */
	FDisplayClusterViewportFrameData CurrentFrameData;

	/** Legacy reference alias to the current frame's viewport contexts stored in CurrentFrameData.Contexts. */
	TArray<FDisplayClusterViewport_Context>& Contexts;

	/** Frame data snapshot from the previous frame; used for motion blur and temporal effects. */
	FDisplayClusterViewportFrameData PreviousFrameData;

	/** Per-context render-thread data (eye adaptation, etc.) captured in OnPostRenderViewFamily_RenderThread(). */
	TMap<int32, FDisplayClusterViewport_Context::FRenderThreadData> CapturedRenderThreadData;

	// Unified repository of viewport resources
	FDisplayClusterViewportResources Resources;

	// Used ViewStates
	TArray<TSharedPtr<FSceneViewStateReference, ESPMode::ThreadSafe>> ViewStates;

private:
	/** A raw RHI texture paired with the subrect it occupies within that texture. */
	struct FResourceWithRect
	{
		FRHITexture* Texture = nullptr;
		FIntRect     Rect;
	};

	/** Per-frame external resource overrides set via SetResourcesOverride_RenderThread().
	 *  Entries shadow the normal resource-fetch path for the current frame only and are cleared at the start of the next frame. */
	mutable TMap<EDisplayClusterViewportResourceType, TArray<FResourceWithRect>> TransientRHIOverrides;
};
