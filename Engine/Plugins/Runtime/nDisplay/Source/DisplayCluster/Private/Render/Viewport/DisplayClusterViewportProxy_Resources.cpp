// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Viewport/DisplayClusterViewportProxy.h"
#include "Render/Viewport/DisplayClusterViewportManagerProxy.h"
#include "Render/Viewport/DisplayClusterViewport.h"
#include "Render/Viewport/DisplayClusterViewportManager.h"
#include "Render/Viewport/DisplayClusterViewportManagerViewExtension.h"
#include "Render/Viewport/DisplayClusterViewportHelpers.h"

#include "Render/Viewport/Containers/DisplayClusterViewport_PostRenderSettings.h"
#include "Render/Viewport/Containers/DisplayClusterViewportProxyData.h"

#include "Render/Viewport/RenderTarget/DisplayClusterRenderTargetResource.h"
#include "Render/Viewport/RenderFrame/DisplayClusterRenderFrameSettings.h"

#include "Render/Viewport/LightCard/DisplayClusterViewportLightCardManagerProxy.h"

#include "Render/Projection/IDisplayClusterProjectionPolicy.h"

#include "Render/Containers/IDisplayClusterRender_MeshComponent.h"
#include "Render/Containers/IDisplayClusterRender_MeshComponentProxy.h"

#include "IDisplayCluster.h"
#include "IDisplayClusterCallbacks.h"
#include "IDisplayClusterShaders.h"
#include "IDisplayClusterShadersTextureUtils.h"
#include "TextureResource.h"

#include "RHIStaticStates.h"

#include "RenderResource.h"
#include "RenderingThread.h"
#include "CommonRenderResources.h"
#include "PixelShaderUtils.h"

#include "ClearQuad.h"

#include "GlobalShader.h"
#include "ShaderParameters.h"
#include "ShaderParameterUtils.h"

#include "ScreenRendering.h"
#include "RenderGraphUtils.h"
#include "RenderGraphResources.h"

// Tile rect border width
int32 GDisplayClusterRenderTileBorder = 0;
static FAutoConsoleVariableRef CVarDisplayClusterRenderTileBorder(
	TEXT("nDisplay.render.TileBorder"),
	GDisplayClusterRenderTileBorder,
	TEXT("Tile border width in pixels (default 0).\n"),
	ECVF_RenderThreadSafe
);

/** Specifies the blending algorithm for tile edges. */
enum class EDisplayClusterRenderTileEdgeBlendMode : uint8
{
	/** No blending is applied to tile edges. */
	None = 0,

	/** Linear fade between tiles — fast but can appear harsh. */
	Linear = 1,

	/** Smooth (non-linear) fade for softer transitions between tiles. */
	Smooth = 2
};

/** CVar that specifies the blending algorithm for tile edges. */
int32 GDisplayClusterRenderTileEdgeBlendMode = 2;
static FAutoConsoleVariableRef CVarDisplayClusterRenderTileBlendMode(
	TEXT("nDisplay.render.tile.BlendMode"),
	GDisplayClusterRenderTileEdgeBlendMode,
	TEXT("Tile blend mode (default = 2).\n")
	TEXT(" 0 - none (Disable tile blending).\n")
	TEXT(" 1 - linear.\n")
	TEXT(" 2 - smooth (non-linear, s-shape).\n"),
	ECVF_Default
);

///////////////////////////////////////////////////////////////////////////////////////
namespace UE::DisplayCluster::ViewportProxy
{
	// The viewport override has the maximum depth. This protects against a link cycle
	static const int32 DisplayClusterViewportProxyResourcesOverrideRecursionDepthMax = 4;

	/**
	 * Get current tile blending mode.
	 * Falls back to EDisplayClusterRenderTileEdgeBlendMode::None if the value is not a valid mode.
	 */
	static inline EDisplayClusterRenderTileEdgeBlendMode GetTileEdgeBlendMode()
	{
		switch (GDisplayClusterRenderTileEdgeBlendMode)
		{
		case static_cast<int32>(EDisplayClusterRenderTileEdgeBlendMode::Linear):
			return EDisplayClusterRenderTileEdgeBlendMode::Linear;

		case static_cast<int32>(EDisplayClusterRenderTileEdgeBlendMode::Smooth):
			return EDisplayClusterRenderTileEdgeBlendMode::Smooth;

		default:
			break;
		}

		return EDisplayClusterRenderTileEdgeBlendMode::None;
	}
};

void FDisplayClusterViewportProxy::FillTextureWithColor_RenderThread(FRHICommandListImmediate& RHICmdList, FRHITexture* InRenderTargetTexture, const FLinearColor& InColor)
{
	if (InRenderTargetTexture)
	{
		FRHIRenderPassInfo RPInfo(InRenderTargetTexture, ERenderTargetActions::DontLoad_Store);
		RHICmdList.Transition(FRHITransitionInfo(InRenderTargetTexture, ERHIAccess::Unknown, ERHIAccess::RTV));
		RHICmdList.BeginRenderPass(RPInfo, TEXT("nDisplay_FillTextureWithColor"));
		{
			const FIntPoint Size = InRenderTargetTexture->GetSizeXY();
			RHICmdList.SetViewport(0, 0, 0.0f, Size.X, Size.Y, 1.0f);
			DrawClearQuad(RHICmdList, FLinearColor::Black);
		}
		RHICmdList.EndRenderPass();
		RHICmdList.Transition(FRHITransitionInfo(InRenderTargetTexture, ERHIAccess::Unknown, ERHIAccess::SRVMask));
	}
}

bool FDisplayClusterViewportProxy::ImplGetResources_RenderThread(const EDisplayClusterViewportResourceType InExtResourceType, TArray<FRHITexture*>& OutResources, const int32 InRecursionDepth) const
{
	using namespace UE::DisplayCluster::ViewportProxy;
	check(IsInRenderingThread());

	const EDisplayClusterViewportResourceType InResourceType = GetResourceType_RenderThread(InExtResourceType);

	// Override resources from other viewport
	if (ShouldOverrideViewportResource(InResourceType))
	{
		if (InRecursionDepth < DisplayClusterViewportProxyResourcesOverrideRecursionDepthMax)
		{
			return GetRenderingViewportProxy().ImplGetResources_RenderThread(InExtResourceType, OutResources, InRecursionDepth + 1);
		}

		return false;
	}

	OutResources.Empty();

	switch (InResourceType)
	{
	case EDisplayClusterViewportResourceType::InternalRenderTargetEntireRectResource:
	case EDisplayClusterViewportResourceType::InternalRenderTargetResource:
	{
		bool bResult = false;

		if (Contexts.Num() > 0)
		{

			// 1. Replace RTT from configuration
			if (!bResult && PostRenderSettings.Replace.IsEnabled())
			{
				bResult = true;

				// Support texture replace:
				if (FRHITexture* ReplaceTextureRHI = PostRenderSettings.Replace.TextureRHI->GetTexture2D())
				{
					for (int32 ContextIndex = 0; ContextIndex < Contexts.Num(); ContextIndex++)
					{
						OutResources.Add(ReplaceTextureRHI);
					}
				}
			}

			// 2. Replace RTT from UVLightCard:
			if (!bResult && EnumHasAnyFlags(RenderSettingsICVFX.RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::UVLightcard))
			{
				bResult = true;

				// Get resources from external UV LightCard manager
				if (FDisplayClusterViewportManagerProxy* ViewportManagerProxy = ConfigurationProxy->GetViewportManagerProxyImpl())
				{
					TSharedPtr<FDisplayClusterViewportLightCardManagerProxy, ESPMode::ThreadSafe> LightCardManager = ViewportManagerProxy->GetLightCardManagerProxy_RenderThread();
					if (LightCardManager.IsValid())
					{
						const EDisplayClusterUVLightCardType UVLightCardType =
							EnumHasAllFlags(RenderSettingsICVFX.RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::OverInFrustum)
							? EDisplayClusterUVLightCardType::Over : EDisplayClusterUVLightCardType::Under;

						if (FRHITexture* UVLightCardRHIResource = LightCardManager->GetUVLightCardRHIResource_RenderThread(UVLightCardType))
						{
							for (int32 ContextIndex = 0; ContextIndex < Contexts.Num(); ContextIndex++)
							{
								OutResources.Add(UVLightCardRHIResource);
							}
						}
					}
				}
			}

			// 3. Finally Use InternalRTT
			if (!bResult)
			{
				bResult = Resources.GetRHIResources_RenderThread(EDisplayClusterViewportResource::RenderTargets, OutResources);
			}
		}

		if (!bResult || Contexts.Num() != OutResources.Num())
		{
			OutResources.Empty();
		}

		return OutResources.Num() > 0;
	}

	case EDisplayClusterViewportResourceType::InputShaderResource:
		return Resources.GetRHIResources_RenderThread(EDisplayClusterViewportResource::InputShaderResources, OutResources);

	case EDisplayClusterViewportResourceType::AdditionalTargetableResource:
		return Resources.GetRHIResources_RenderThread(EDisplayClusterViewportResource::AdditionalTargetableResources, OutResources);

	case EDisplayClusterViewportResourceType::MipsShaderResource:
		return Resources.GetRHIResources_RenderThread(EDisplayClusterViewportResource::MipsShaderResources, OutResources);

	case EDisplayClusterViewportResourceType::OutputFrameTargetableResource:
		return Resources.GetRHIResources_RenderThread(EDisplayClusterViewportResource::OutputFrameTargetableResources, OutResources);

	case EDisplayClusterViewportResourceType::AdditionalFrameTargetableResource:
		return Resources.GetRHIResources_RenderThread(EDisplayClusterViewportResource::AdditionalFrameTargetableResources, OutResources);

	case EDisplayClusterViewportResourceType::OutputPreviewTargetableResource:
		return Resources.GetRHIResources_RenderThread(EDisplayClusterViewportResource::OutputPreviewTargetableResources, OutResources);

	default:
		break;
	}

	return false;
}

bool FDisplayClusterViewportProxy::ImplGetResourcesWithRects_RenderThread(const EDisplayClusterViewportResourceType InExtResourceType, TArray<FRHITexture*>& OutResources, TArray<FIntRect>& OutResourceRects, const int32 InRecursionDepth) const
{
	using namespace UE::DisplayCluster::ViewportProxy;
	check(IsInRenderingThread());

	// Override resources from other viewport
	if (ShouldOverrideViewportResource(InExtResourceType))
	{
		if (InRecursionDepth < DisplayClusterViewportProxyResourcesOverrideRecursionDepthMax)
		{
			return GetRenderingViewportProxy().ImplGetResourcesWithRects_RenderThread(InExtResourceType, OutResources, OutResourceRects, InRecursionDepth + 1);
		}

		return false;
	}

	const EDisplayClusterViewportResourceType InResourceType = GetResourceType_RenderThread(InExtResourceType);
	if (!GetResources_RenderThread(InResourceType, OutResources))
	{
		return false;
	}

	// Collect all resource rects:
	for (int32 ContextIt = 0; ContextIt < OutResources.Num(); ContextIt++)
	{
		FIntRect ResourceRect = GetResourceRect_RenderThread(InResourceType, ContextIt);

		// Rect({0,0}, {0,0} means we are using the entire texture.
		if (ResourceRect.IsEmpty() && OutResources[ContextIt])
		{
			ResourceRect.Max = OutResources[ContextIt]->GetDesc().Extent;
		}

		OutResourceRects.Add(ResourceRect);
	}

	return true;
}

bool FDisplayClusterViewportProxy::ImplResolveResources_RenderThread(FRHICommandListImmediate& RHICmdList, FDisplayClusterViewportProxy const* SourceProxy, const EDisplayClusterViewportResourceType InExtResourceType, const EDisplayClusterViewportResourceType OutExtResourceType, const int32 InContextNum) const
{
	using namespace UE::DisplayCluster::ViewportProxy;

	check(IsInRenderingThread());
	check(SourceProxy);

	const EDisplayClusterViewportResourceType InResourceType = SourceProxy->GetResourceType_RenderThread(InExtResourceType);
	const EDisplayClusterViewportResourceType OutResourceType = GetResourceType_RenderThread(OutExtResourceType);

	if (InResourceType == EDisplayClusterViewportResourceType::MipsShaderResource)
	{
		// RenderTargetMips not allowved for resolve op
		return false;
	}

	FDisplayClusterShadersTextureUtilsSettings TextureUtilsSettings;
	// The mode used to blend textures
	if (OutResourceType == EDisplayClusterViewportResourceType::OutputPreviewTargetableResource)
	{
		// The preview texture should use only RGB colors and ignore the alpha channel.
		// The alpha channel may or may not be inverted in third-party libraries.
		TextureUtilsSettings.OverrideAlpha = EDisplayClusterShaderTextureUtilsOverrideAlpha::Set_Alpha_One;
	}

	TSharedRef<IDisplayClusterShadersTextureUtils> TextureUtils =
		GetShadersAPI().CreateTextureUtils_RenderThread(RHICmdList)
			->SetInput(SourceProxy, InExtResourceType)
			->SetOutput(this, OutExtResourceType);

	if (InExtResourceType == EDisplayClusterViewportResourceType::AfterWarpBlendTargetableResource
		&& OutExtResourceType == EDisplayClusterViewportResourceType::OutputTargetableResource
		&& DisplayDeviceProxy.IsValid()
		&& DisplayDeviceProxy->HasFinalPass_RenderThread())
	{
		// Custom resolve at external Display Device
		DisplayDeviceProxy->AddFinalPass_RenderThread(TextureUtilsSettings, TextureUtils);
	}
	else
	{
		// Standard resolve:
		TextureUtils->Resolve(TextureUtilsSettings);
	}

	return true;
}

void FDisplayClusterViewportProxy::ImplResolveTileResource_RenderThread(FRHICommandListImmediate& RHICmdList, FDisplayClusterViewportProxy const* InDestViewportProxy) const
{
	check(InDestViewportProxy);

	using namespace UE::DisplayCluster::ViewportProxy;

	// When AdaptResolution is disabled, the input RTT texture may be smaller than the output rect.
	// Use separate fields to store the input and output rect dimensions in this case.
	// 
	// Note: The AdaptResolution option in overscan settings controls whether the RTT size 
	// is increased to match the overscan size. If disabled, only the inner rectangle 
	// matching the original viewport size is used as the input.

	// Get tile position ans size
	const FIntPoint InTilePos = RenderSettings.TileSettings.GetPos();
	const FIntPoint InTileSize = RenderSettings.TileSettings.GetSize();

	// Get actual tile edge blending mode
	const EDisplayClusterRenderTileEdgeBlendMode TileEdgeBlendMode = GetTileEdgeBlendMode();

	// Since we are blending into the destination viewport, 
	// we should use its settings for blending.
	const FDisplayClusterViewport_OverscanSettings& DestOverscanSettings = InDestViewportProxy->GetRenderSettings_RenderThread().TileSettings.GetOverscanSettings();

	// Check if we can apply tile edge blending for dest viewport
	bool bApplyTileEdgeBlend = DestOverscanSettings.bApplyTileEdgeBlend
		&& TileEdgeBlendMode != EDisplayClusterRenderTileEdgeBlendMode::None;
	
	// If so, get input and output margins.
	FIntMarginSet InputRectEdgeBlendMargins;
	FIntMarginSet OutputRectEdgeBlendMargins;
	if (bApplyTileEdgeBlend)
	{
		// Clamp to 0%..100%: [0..1]
		const float OverscanPercentage = FMath::Clamp(DestOverscanSettings.TileEdgeBlendPercentage, 0.0f, 1.0f);

		// Use percentage of overscan
		InputRectEdgeBlendMargins = (FMarginSet(static_cast<FIntMarginSet>(OverscanRuntimeSettings.OverscanPixels)) * OverscanPercentage).GetCeilToInt();
		OutputRectEdgeBlendMargins = (FMarginSet(static_cast<FIntMarginSet>(OverscanRuntimeSettings.BaseOverscanPixels)) * OverscanPercentage).GetCeilToInt();

		// Per-tile adjustments: Optimize overscan values for edge tiles.
		{
			using EMarginSide = UE::Math::EMarginSetSideIndex;
			auto OptimizeSize = [&](const EMarginSide InSideIndex)
				{
					InputRectEdgeBlendMargins[InSideIndex] = 0;
					OutputRectEdgeBlendMargins[InSideIndex] = 0;
				};

			if (InTilePos.X == 0)
			{
				OptimizeSize(EMarginSide::Left);
			}
			if (InTilePos.Y == 0)
			{
				OptimizeSize(EMarginSide::Top);
			}
			if (InTilePos.X == (InTileSize.X - 1))
			{
				OptimizeSize(EMarginSide::Right);
			}
			if (InTilePos.Y == (InTileSize.Y - 1))
			{
				OptimizeSize(EMarginSide::Bottom);
			}
		}

		// check if margins not empty
		if (InputRectEdgeBlendMargins.IsZero() || OutputRectEdgeBlendMargins.IsZero())
		{
			// clear margins and avoid edge blending
			InputRectEdgeBlendMargins = 0;
			OutputRectEdgeBlendMargins = 0;
			bApplyTileEdgeBlend = false;
		}
	}

	const TArray<FDisplayClusterViewport_Context>& DestContexts = InDestViewportProxy->GetContexts_RenderThread();
	const TArray<FDisplayClusterViewport_Context>& SrcContexts = Contexts;
	TSharedRef<IDisplayClusterShadersTextureUtils> TextureUtils = GetShadersAPI().CreateTextureUtils_RenderThread(RHICmdList);

	TextureUtils
		->SetInput(this, EDisplayClusterViewportResourceType::InternalRenderTargetResource)
		->SetOutput(InDestViewportProxy, EDisplayClusterViewportResourceType::InternalRenderTargetResource)
		->ForEachContextByPredicate(
			EDisplayClusterShaderTextureUtilsFlags::DisableUpdateResourcesRectsForResolve, [&](
			const FDisplayClusterShadersTextureViewportContext& InputContext,
			const FDisplayClusterShadersTextureViewportContext& OutputContext)
			{
				const uint32 ContextNum = InputContext.ContextNum;
				if(!SrcContexts.IsValidIndex(ContextNum) || !DestContexts.IsValidIndex(ContextNum) || OutputContext.ContextNum != ContextNum)
				{
					return;
				}

				FDisplayClusterShadersTextureViewport Input(InputContext);
				FDisplayClusterShadersTextureViewport Output(OutputContext);

				Output.Rect = SrcContexts[ContextNum].TileDestRect;

				// By default, we just copy part of the tile texture into dest.
				EDisplayClusterShaderTextureUtilsFlags Flags = EDisplayClusterShaderTextureUtilsFlags::None;

				if (GDisplayClusterRenderTileBorder > 0)
				{
					// The maximum border is 1/4 of the minimum side of the rectangle.
					const int32 MaxBorderSize = FMath::Min(Input.Rect.Size().GetMin(), Output.Rect.Size().GetMin()) / 4;
					const int32 TileBorderSize = FMath::Min(GDisplayClusterRenderTileBorder, MaxBorderSize);

					FIntMarginSet InputBorderMargins = 0;
					if (InTilePos.X > 0)
					{
						InputBorderMargins.Left = TileBorderSize;
					}
					if (InTilePos.Y > 0)
					{
						InputBorderMargins.Top = TileBorderSize;
					}
					if ((InTilePos.X + 1) < InTileSize.X)
					{
						InputBorderMargins.Right = TileBorderSize;
					}
					if ((InTilePos.Y + 1) < InTileSize.Y)
					{
						InputBorderMargins.Bottom = TileBorderSize;
					}

					// Scale border margins from input rect space to output rect space.
					// Falls back to (1,1) if input size is zero (avoid division by zero).
					const FVector2D Scale =
						(Input.Rect.Size().GetMin() > 0)
						? FVector2D(Output.Rect.Size()) / Input.Rect.Size()
						: FVector2D(1.f, 1.f);

					const FIntMarginSet OutputBorderMargins = (FMarginSet(InputBorderMargins) * Scale).GetRoundToInt();

					// Set rect smaller to show gaps between tiles:
					InputBorderMargins.ApplyClampedInsetToRect(Input.Rect);
					OutputBorderMargins.ApplyClampedInsetToRect(Output.Rect);
				}
				// When we don't need to show tile borders, we can blend them together
				else if (bApplyTileEdgeBlend)
				{
					// Configure rendering flags that define how tile edge blending is applied
					switch (TileEdgeBlendMode)
					{
					case EDisplayClusterRenderTileEdgeBlendMode::Smooth:
						// Enables smooth (cubic Hermite) alpha feathering(fade - out) on all sides of the rectangle.
						// When set, alpha transitions smoothly from opaque to transparent along each edge using a smoothstep(S - curve) function.
						Flags |= EDisplayClusterShaderTextureUtilsFlags::EnableSmoothAlphaFeather;
						break;

					case EDisplayClusterRenderTileEdgeBlendMode::Linear:
						// Applies a linear alpha fade(feathering) to all edges of the rectangle.
						Flags |= EDisplayClusterShaderTextureUtilsFlags::EnableLinearAlphaFeather;
						break;

					default:
						check(false); // This case indicates an invalid value and should not occur if bApplyTileEdgeBlend was validated beforehand.
						break;
					};

					// Add margins to input and output rects
					InputRectEdgeBlendMargins.ApplyExpandToRect(Input.Rect);
					OutputRectEdgeBlendMargins.ApplyExpandToRect(Output.Rect);
				}

				// Copy texture region
				TextureUtils->ResolveTextureContext(
					{ Flags,  InputRectEdgeBlendMargins * 2},
					Input, Output);
			});
}

void FDisplayClusterViewportProxy::CleanupResources_RenderThread(FRHICommandListImmediate& RHICmdList) const
{
	bool bShouldCleanupRTT = false;

	/**
	 * Tile viewport rules:
	 * The [EDisplayClusterViewportTileType::Source] tile viewport represents the destination surface where all tiles
	 * are composed and blended together into the final frame.
	 */
	if (RenderSettings.TileSettings.GetType() == EDisplayClusterViewportTileType::Source)
	{
		if (GDisplayClusterRenderTileBorder > 0)
		{
			// Since the RTT is reused through frames, in case we need to show a black border between viewport tiles,
			// we must fill the original viewport with this colour.
			bShouldCleanupRTT = true;
		}

		// Check if this viewport applies blending between tiles.
		if (RenderSettings.TileSettings.GetOverscanSettings().bApplyTileEdgeBlend)
		{
			// Blending assumes the underlying surface is black.
			// Tiles are composited on top of each other across multiple render passes.
			bShouldCleanupRTT = true;
		}
	}

	if(bShouldCleanupRTT)
	{
		TArray<FRHITexture*> RenderTargets;
		if (GetResources_RenderThread(EDisplayClusterViewportResourceType::InternalRenderTargetEntireRectResource, RenderTargets))
		{
			for (FRHITexture* TextureIt : RenderTargets)
			{
				// Note: It may make sense to move the CVar and border color to the StageSettings.
				FDisplayClusterViewportProxy::FillTextureWithColor_RenderThread(RHICmdList, TextureIt, FLinearColor::Black);
			}
		}
	}
}
