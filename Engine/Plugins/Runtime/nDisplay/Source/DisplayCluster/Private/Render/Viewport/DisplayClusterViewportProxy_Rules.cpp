// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Viewport/DisplayClusterViewportProxy.h"
#include "Render/Viewport/DisplayClusterViewportManagerProxy.h"
#include "Render/Viewport/DisplayClusterViewport.h"
#include "Render/Viewport/DisplayClusterViewportManager.h"
#include "Render/Viewport/DisplayClusterViewportManagerViewExtension.h"

#include "Render/Viewport/Containers/DisplayClusterViewport_PostRenderSettings.h"
#include "Render/Viewport/Containers/DisplayClusterViewportProxyData.h"

#include "Render/Viewport/RenderTarget/DisplayClusterRenderTargetResource.h"
#include "Render/Viewport/RenderFrame/DisplayClusterRenderFrameSettings.h"

#include "Render/Viewport/LightCard/DisplayClusterViewportLightCardManagerProxy.h"

#include "Render/Projection/IDisplayClusterProjectionPolicy.h"

#include "Render/Containers/IDisplayClusterRender_MeshComponent.h"
#include "Render/Containers/IDisplayClusterRender_MeshComponentProxy.h"

#include "IDisplayCluster.h"
#include "IDisplayClusterCallbacks.h"
#include "IDisplayClusterShaders.h"

#include "TextureResource.h"

static TAutoConsoleVariable<int32> CVarDisplayClusterRenderOverscanResolve(
	TEXT("nDisplay.render.overscan.resolve"),
	1,
	TEXT("Allow resolve overscan internal rect to output backbuffer.\n")
	TEXT(" 0 - to disable.\n"),
	ECVF_RenderThreadSafe
);

// Enable/disable warp&blend
int32 GDisplayClusterRenderWarpBlendEnabled = 1;
static FAutoConsoleVariableRef CVarDisplayClusterRenderWarpBlendEnabled(
	TEXT("nDisplay.render.WarpBlendEnabled"),
	GDisplayClusterRenderWarpBlendEnabled,
	TEXT("Warp & Blend status\n")
	TEXT("0 : disabled\n")
	TEXT("1 : enabled\n"),
	ECVF_RenderThreadSafe
);

int32 GDisplayClusterShadersICVFXFXAALightCard = 2;
static FAutoConsoleVariableRef CVarDisplayClusterShadersICVFXFXAALightCard(
	TEXT("nDisplay.render.icvfx.fxaa.lightcard"),
	GDisplayClusterShadersICVFXFXAALightCard,
	TEXT("FXAA quality for lightcard (0 - disable).\n")
	TEXT("1..6 - FXAA quality from Lowest Quality(Fastest) to Highest Quality(Slowest).\n"),
	ECVF_RenderThreadSafe
);

int32 GDisplayClusterShadersICVFXFXAAChromakey = 2;
static FAutoConsoleVariableRef CVarDisplayClusterShadersICVFXFXAAChromakey(
	TEXT("nDisplay.render.icvfx.fxaa.chromakey"),
	GDisplayClusterShadersICVFXFXAAChromakey,
	TEXT("FXAA quality for chromakey (0 - disable).\n")
	TEXT("1..6 - FXAA quality from Lowest Quality(Fastest) to Highest Quality(Slowest).\n"),
	ECVF_RenderThreadSafe
);

///////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterViewportProxy
///////////////////////////////////////////////////////////////////////////////////////
bool FDisplayClusterViewportProxy::ShouldApplyFXAA_RenderThread(EFXAAQuality& OutFXAAQuality) const
{
	// Get FXAA quality for current viewport
	EFXAAQuality FXAAQuality = EFXAAQuality::MAX;

	switch (RenderSettings.CaptureMode)
	{
	case EDisplayClusterViewportCaptureMode::Chromakey:
		if (GDisplayClusterShadersICVFXFXAAChromakey > 0)
		{
			OutFXAAQuality = (EFXAAQuality)FMath::Clamp(GDisplayClusterShadersICVFXFXAAChromakey - 1, 0, (int32)EFXAAQuality::MAX - 1);
			return true;
		}
		break;

	case EDisplayClusterViewportCaptureMode::Lightcard:
		if (GDisplayClusterShadersICVFXFXAALightCard > 0)
		{
			OutFXAAQuality = (EFXAAQuality)FMath::Clamp(GDisplayClusterShadersICVFXFXAALightCard - 1, 0, (int32)EFXAAQuality::MAX - 1);
			return true;
		}
		break;

	default:
		break;
	}

	// No FXAA
	return false;
}

EDisplayClusterViewportResourceType FDisplayClusterViewportProxy::GetResourceType_RenderThread(const EDisplayClusterViewportResourceType& InResourceType) const
{
	check(IsInRenderingThread());

	switch (InResourceType)
	{
	case EDisplayClusterViewportResourceType::BeforeWarpBlendTargetableResource:
		return EDisplayClusterViewportResourceType::InputShaderResource;

	case EDisplayClusterViewportResourceType::AfterWarpBlendTargetableResource:
		return ShouldApplyWarpBlend_RenderThread() ? EDisplayClusterViewportResourceType::AdditionalTargetableResource : EDisplayClusterViewportResourceType::InputShaderResource;

	case EDisplayClusterViewportResourceType::OutputTargetableResource:

		/**
		* Output textures for preview rendering.
		*/
		if (ConfigurationProxy->IsPreviewRendering_RenderThread())
		{
			// [warp] -> OutputPreviewTargetableResource
			return EDisplayClusterViewportResourceType::OutputPreviewTargetableResource;
		}

		/**
		* Output 'Frame' textures for cluster rendering
		*/
		if (RemapMesh.IsValid())
		{
			const IDisplayClusterRender_MeshComponentProxy* MeshProxy = RemapMesh->GetMeshComponentProxy_RenderThread();
			if (MeshProxy != nullptr && MeshProxy->IsEnabled_RenderThread())
			{
				// In this case render to additional frame targetable
				// to minimize the rendering steps:
				// [warp] -> AdditionalFrameTargetableResource -> [OutputRemap] -> OutputFrameTargetableResource
				return EDisplayClusterViewportResourceType::AdditionalFrameTargetableResource;
			}
		}

		// [warp] -> OutputFrameTargetableResource
		return EDisplayClusterViewportResourceType::OutputFrameTargetableResource;


	default:
		// return the same value
		break;
	}

	return InResourceType;
}

bool FDisplayClusterViewportProxy::ShouldApplyWarpBlend_RenderThread() const
{
	if (GDisplayClusterRenderWarpBlendEnabled == 0)
	{
		return false;
	}

	if (RenderSettings.TileSettings.GetType() == EDisplayClusterViewportTileType::Tile)
	{
		// Ignore tile viewports
		return false;
	}

	if (GetPostRenderSettings_RenderThread().Replace.IsEnabled())
	{
		// When used override texture, disable warp blend
		return false;
	}

	if (!ConfigurationProxy->GetRenderFrameSettings().bAllowWarpBlend)
	{
		return false;
	}

	// Ask current projection policy if it's warp&blend compatible
	return ProjectionPolicy.IsValid() && ProjectionPolicy->IsWarpBlendSupported_RenderThread(this);
}

bool FDisplayClusterViewportProxy::ShouldOverrideViewportResource(const EDisplayClusterViewportResourceType InExtResourceType) const
{
	// Override resources from other viewport
	if (RenderSettings.IsViewportOverridden())
	{
		switch (RenderSettings.GetViewportOverrideMode())
		{
		case EDisplayClusterViewportOverrideMode::All:
		{
			switch (GetResourceType_RenderThread(InExtResourceType))
			{
			case EDisplayClusterViewportResourceType::InternalRenderTargetEntireRectResource:
			case EDisplayClusterViewportResourceType::InternalRenderTargetResource:
			case EDisplayClusterViewportResourceType::InputShaderResource:
			case EDisplayClusterViewportResourceType::MipsShaderResource:
			case EDisplayClusterViewportResourceType::AdditionalTargetableResource:
				return true;

			default:
				break;
			}
		}
		break;

		case EDisplayClusterViewportOverrideMode::InternalViewportResources:
		{
			switch (GetResourceType_RenderThread(InExtResourceType))
			{
			case EDisplayClusterViewportResourceType::InternalRenderTargetEntireRectResource:
			case EDisplayClusterViewportResourceType::InternalRenderTargetResource:
			case EDisplayClusterViewportResourceType::InputShaderResource:
			case EDisplayClusterViewportResourceType::MipsShaderResource:
				return true;

			default:
				break;
			}
		}
		break;

		case EDisplayClusterViewportOverrideMode::InternalRTT:
		{
			switch (GetResourceType_RenderThread(InExtResourceType))
			{
			case EDisplayClusterViewportResourceType::InternalRenderTargetEntireRectResource:
			case EDisplayClusterViewportResourceType::InternalRenderTargetResource:
				return true;

			default:
				break;
			}
		}
		break;

		default:
			break;
		}
	}

	return false;
}

bool FDisplayClusterViewportProxy::ShouldUseAlphaChannel_RenderThread() const
{
	// Chromakey and Light Cards use alpha channel
	if (EnumHasAnyFlags(RenderSettingsICVFX.RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::Lightcard | EDisplayClusterViewportRuntimeICVFXFlags::Chromakey))
	{
		return true;
	}

	return false;
}

bool FDisplayClusterViewportProxy::ShouldUsePostProcessPassAfterSSRInput() const
{
	if (ShouldUseAlphaChannel_RenderThread())
	{
		return ConfigurationProxy->GetRenderFrameSettings().AlphaChannelCaptureMode == EDisplayClusterRenderFrameAlphaChannelCaptureMode::ThroughTonemapper;
	}

	return false;
}

bool FDisplayClusterViewportProxy::ShouldUsePostProcessPassAfterFXAA() const
{
	if (ShouldUseAlphaChannel_RenderThread())
	{
		return ConfigurationProxy->GetRenderFrameSettings().AlphaChannelCaptureMode == EDisplayClusterRenderFrameAlphaChannelCaptureMode::ThroughTonemapper;
	}

	return false;
}

bool FDisplayClusterViewportProxy::ShouldUsePostProcessPassTonemap() const
{
	if (GetOpenColorIOMode() == EDisplayClusterViewportOpenColorIOMode::PostProcess)
	{
		return true;
	}
	// When capturing with late OCIO pass, PostTonemap is required
	else if (RenderSettings.HasAnyMediaStates(EDisplayClusterViewportMediaState::CaptureLateOCIO))
	{
		return true;
	}

	return false;
}

bool FDisplayClusterViewportProxy::IsInputRenderTargetResourceExists() const
{
	if (PostRenderSettings.Replace.IsEnabled())
	{
		// Use external texture
		return true;
	}

	if (EnumHasAnyFlags(RenderSettingsICVFX.RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::UVLightcard))
	{
		// Use external UVLightCard Resource
		return true;
	}

	return !Resources[EDisplayClusterViewportResource::RenderTargets].IsEmpty();
}

/**
* The viewport proxy priority values
* A lower priority value for a viewport means that this viewport will be the first in the list of viewports.
* The order in this list is used to process viewports one after the other.
* This is important when the viewports are linked to each other.
*/enum class EDisplayClusterViewportProxyPriority : uint8
{
	None = 0,

	// The tile viewport must be processed at the beginning to clump the tiles together.
	Tile = (1 << 0),

	// This tile source viewport
	TileSource = ( 1 << 1),

	// This viewport does not use tile rendering.
	TileDisable = (1 << 1),

	// The linked viewport should be right after the parent viewports because it uses data from them.
	Linked = (1 << 2),

	// Overridden viewports are not rendered, they just use the image from another viewport, so they should be processed last.
	Overriden = (1 << 3)
};

uint8 FDisplayClusterViewportProxy::GetPriority_RenderThread() const
{
	uint8 OutOrder = 0;

	// Tile rendering requires a special viewport processing order for the rendering thread.
	switch (RenderSettings.TileSettings.GetType())
	{
	case EDisplayClusterViewportTileType::Source:
		OutOrder += (uint8)EDisplayClusterViewportProxyPriority::TileSource;
		break;

	case EDisplayClusterViewportTileType::Tile:
	case EDisplayClusterViewportTileType::UnusedTile:
		OutOrder += (uint8)EDisplayClusterViewportProxyPriority::Tile;
		break;

	case EDisplayClusterViewportTileType::None:
		OutOrder += (uint8)EDisplayClusterViewportProxyPriority::TileDisable;
		break;

	default:
		break;
	}

	// Internal tile viewports must be processed first.
	// Additional rules does not apply to them.
	if (RenderSettings.TileSettings.IsInternalViewport())
	{
		return OutOrder;
	}

	if (RenderSettings.IsViewportHasParent())
	{
		OutOrder += (uint8)EDisplayClusterViewportProxyPriority::Linked;
	}

	if (RenderSettings.IsViewportOverridden())
	{
		OutOrder += (uint8)EDisplayClusterViewportProxyPriority::Overriden;
	}

	return OutOrder;
}

FIntRect FDisplayClusterViewportProxy::GetResourceRect_RenderThread(const EDisplayClusterViewportResourceType InResourceType, const uint32 InContextNum) const
{
	// Rect({0,0}, {0,0} means we are using the entire texture.
	FIntRect OutResourceRect = FIntRect(FIntPoint::ZeroValue, FIntPoint::ZeroValue);

	if (Contexts.IsValidIndex(InContextNum))
	{
		switch (InResourceType)
		{
		case EDisplayClusterViewportResourceType::InternalRenderTargetEntireRectResource:
		case EDisplayClusterViewportResourceType::InternalRenderTargetResource:
			if (PostRenderSettings.Replace.IsEnabled())
			{
				// Get external texture rect.
				OutResourceRect = PostRenderSettings.Replace.Rect;
			}
			else
			{
				// Get RTT rect
				OutResourceRect = Contexts[InContextNum].RenderTargetRect;

				// Get final rect
				if (InResourceType == EDisplayClusterViewportResourceType::InternalRenderTargetResource)
				{
					// Get the inner region from the overscan texture.
					if (OverscanRuntimeSettings.bIsEnabled && CVarDisplayClusterRenderOverscanResolve.GetValueOnRenderThread() != 0)
					{
						// Apply overscan crop
						OutResourceRect = OverscanRuntimeSettings.OverscanPixels.GetInnerRect(OutResourceRect);
					}
				}
			}
			break;

		case EDisplayClusterViewportResourceType::OutputFrameTargetableResource:
		case EDisplayClusterViewportResourceType::AdditionalFrameTargetableResource:
			OutResourceRect = Contexts[InContextNum].FrameTargetRect;
			break;

		default:
			// All other resources use the same context size
			OutResourceRect.Max = Contexts[InContextNum].ContextSize;
			break;
		}
	}

	return OutResourceRect;
}