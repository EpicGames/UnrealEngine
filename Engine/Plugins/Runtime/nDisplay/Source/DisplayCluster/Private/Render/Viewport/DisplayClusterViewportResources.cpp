// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Viewport/DisplayClusterViewportResources.h"
#include "Render/Viewport/Resource/DisplayClusterViewportResource.h"

/////////////////////////////////////////////////////////////////////
// FDisplayClusterViewportResources
/////////////////////////////////////////////////////////////////////
const TArray<TSharedPtr<FDisplayClusterViewportResource, ESPMode::ThreadSafe>> FDisplayClusterViewportResources::EmptyResources;

void FDisplayClusterViewportResources::FreezeRendering(const EDisplayClusterViewportResource InResourceType)
{
	for (const TSharedPtr<FDisplayClusterViewportResource, ESPMode::ThreadSafe>& TextureResource : operator[](InResourceType))
	{
		if (TextureResource.IsValid())
		{
			EnumAddFlags(TextureResource->GetResourceState(), EDisplayClusterViewportResourceState::DisableReallocate);
		}
	}
}

bool FDisplayClusterViewportResources::GetRHIResources_RenderThread(const EDisplayClusterViewportResource InResourceType, TArray<FRHITexture*>& OutResources) const
{
	OutResources.Reset();

	if (const TArray<TSharedPtr<FDisplayClusterViewportResource, ESPMode::ThreadSafe>>* ExistResources = ViewportResources.Find(InResourceType))
	{
		for (const TSharedPtr<FDisplayClusterViewportResource, ESPMode::ThreadSafe>& ViewportResourceIt : *ExistResources)
		{
			if (ViewportResourceIt.IsValid())
			{
				// When resource accessed on rendering thread, update this flag
				EnumAddFlags(ViewportResourceIt->GetResourceState(), EDisplayClusterViewportResourceState::UpdatedOnRenderingThread);

				if (FRHITexture* RHITexture2D = ViewportResourceIt->GetViewportResourceRHI_RenderThread())
				{
					// Collects only valid resources.
					OutResources.Add(RHITexture2D);
				}
			}
		}

		if (OutResources.Num() != ExistResources->Num())
		{
			// Some resources lost
			OutResources.Reset();
		}
	}

	// returns success if the number of output resources is equal to the input and is not empty
	return !OutResources.IsEmpty();
}
