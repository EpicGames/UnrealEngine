// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "RHI.h"
#include "RHIResources.h"
#include "RenderResource.h"

class FDisplayClusterViewportResource;

/**
 * Internal types of DC viewport resources
 * These types are used to point to actual instances of the resources in the viewport.
 */
enum class EDisplayClusterViewportResource : uint8
{
	// View family render to this resources
	// This is the resource that can be pointed to using the enum values:
	// - 'EDisplayClusterViewportResourceType::InternalRenderTargetEntireRectResource'
	// - 'EDisplayClusterViewportResourceType::InternalRenderTargetResource'
	RenderTargets,

	// unique viewport resources
	// This is the resource that can be pointed to using the enum 'EDisplayClusterViewportResourceType::InputShaderResource' value.
	InputShaderResources,

	// unique viewport resources
	// This is the resource that can be pointed to using the enum 'EDisplayClusterViewportResourceType::AdditionalTargetableResource' value.
	AdditionalTargetableResources,

	// unique viewport resources
	// This is the resource that can be pointed to using the enum 'EDisplayClusterViewportResourceType::MipsShaderResource' value.
	MipsShaderResources,

	// Projection policy output resources
	// This is the resource that can be pointed to using the enum 'EDisplayClusterViewportResourceType::OutputPreviewTargetableResource' value.
	OutputPreviewTargetableResources,

	// Projection policy output resources
	// This is the resource that can be pointed to using the enum 'EDisplayClusterViewportResourceType::OutputFrameTargetableResource' value.
	OutputFrameTargetableResources,

	// Projection policy output resources
	// This is the resource that can be pointed to using the enum 'EDisplayClusterViewportResourceType::AdditionalFrameTargetableResource' value.
	AdditionalFrameTargetableResources,
};

/** Helper class for unified viewport resource repository. */
class FDisplayClusterViewportResources
{
public:
	FDisplayClusterViewportResources() = default;
	~FDisplayClusterViewportResources() = default;

public:
	/** Get viewport resources for all contexts by type. */
	TArray<TSharedPtr<FDisplayClusterViewportResource, ESPMode::ThreadSafe>>& operator[](const EDisplayClusterViewportResource InResourceType)
	{
		if (TArray<TSharedPtr<FDisplayClusterViewportResource, ESPMode::ThreadSafe>>* ExistResources = ViewportResources.Find(InResourceType))
		{
			return *ExistResources;
		}

		return ViewportResources.Emplace(InResourceType);
	}

	/** Get viewport const resources for all contexts by type. */
	const TArray<TSharedPtr<FDisplayClusterViewportResource, ESPMode::ThreadSafe>>& operator[](const EDisplayClusterViewportResource InResourceType) const
	{
		if (const TArray<TSharedPtr<FDisplayClusterViewportResource, ESPMode::ThreadSafe>>* ExistResources = ViewportResources.Find(InResourceType))
		{
			return *ExistResources;
		}

		// This value is used for const-functions when resources are not defined
		return EmptyResources;
	}

	/** Release refs to all viewport resources. */
	void ReleaseAllResources()
	{
		ViewportResources.Empty();
	}

	/**
	* Release only part of the resources, leaving resources that can be used by other viewports(viewport override feature)
	* Discard resources that are not used in frame composition
	*/
	void ReleaseNotSharedResources()
	{
		ImplRelease(EDisplayClusterViewportResource::RenderTargets);
		ImplRelease(EDisplayClusterViewportResource::OutputFrameTargetableResources);
		ImplRelease(EDisplayClusterViewportResource::AdditionalFrameTargetableResources);
	}

	/** Raise DisableReallocate flag for all viewport resources of specified type.
	* Support render freeze feature
	*/
	void FreezeRendering(const EDisplayClusterViewportResource InResourceType);

	/**
	* Get viewport RHI resources
	*
	* @param InResourceType - The type of resources you want to retrieve.
	* @param OutResources - Array with RHI resources
	*
	* @return true, if all resources are valid
	*/
	bool GetRHIResources_RenderThread(const EDisplayClusterViewportResource InResourceType, TArray<FRHITexture*>& OutResources) const;

protected:
	/** Release specified type of resources. */
	inline void ImplRelease(const EDisplayClusterViewportResource InResourceType)
	{
		if (TArray<TSharedPtr<FDisplayClusterViewportResource, ESPMode::ThreadSafe>>* ExistResources = ViewportResources.Find(InResourceType))
		{
			ExistResources->Empty();
		}
	}

private:
	// Unified repository of viewport resources
	TMap<EDisplayClusterViewportResource, TArray<TSharedPtr<FDisplayClusterViewportResource, ESPMode::ThreadSafe>>> ViewportResources;

	// This value is used for const-functions when resources are not defined
	const static TArray<TSharedPtr<FDisplayClusterViewportResource, ESPMode::ThreadSafe>> EmptyResources;
};
