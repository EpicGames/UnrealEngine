// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "StereoRendering.h"
#include "Render/Viewport/RenderFrame/DisplayClusterRenderFrameSettings.h"

struct FDisplayClusterViewportStereoscopicPass
{
	// Base of the reserved StereoViewIndex range for nDisplay; engine indices are below this.
	static constexpr int32 StereoViewIndexBase = 100;

	/**
	* True if this view index falls in the nDisplay-reserved range.
	*/
	inline static bool IsValidStereoViewIndex(const int32 ViewIndex)
	{
		return ViewIndex >= StereoViewIndexBase;
	}

	// 'EStereoscopicPass': Stereoscopic rendering passes.
	// FULL implies stereoscopic rendering isn't enabled for this pass
	// PRIMARY implies the view needs its own pass, while SECONDARY implies the view can be instanced

	inline static EStereoscopicPass EncodeStereoscopicPass(const uint32 ContextNum, const uint32 ContextAmount, const FDisplayClusterRenderFrameSettings& InFrameSettings)
	{
		// Use special rendering mode (DCRA preview)
		if (!InFrameSettings.bUseDisplayClusterRenderDevice)
		{
			return EStereoscopicPass::eSSP_FULL;
		}

		// 'bEnableStereoscopicRenderingOptimization'
		// When stereoscopic rendering optimization is enabled, one image family and one RTT for both eyes is used. 
		// The result both eyes are rendered in the same family. It makes sense to use PRIMARY/SECONDARY.
		// Otherwise, we render each view into a unique family.As a result, only PRIMARY is used.
		if (InFrameSettings.bEnableStereoscopicRenderingOptimization && ContextAmount > 1)
		{
			switch (ContextNum)
			{
			case 0:
				// Left eye
				// PRIMARY implies the view needs its own pass
				return EStereoscopicPass::eSSP_PRIMARY;
			case 1:
				// Right eye
				// SECONDARY implies the view can be instanced
				return EStereoscopicPass::eSSP_SECONDARY;
			default:
				// now stereo only with 2 context
				check(false);
				break;
			}
		}

		// Render as primary
		return EStereoscopicPass::eSSP_PRIMARY;
	}
};

