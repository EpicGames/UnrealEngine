// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Viewport/DisplayClusterViewport.h"
#include "Render/Viewport/DisplayClusterViewportManager.h"
#include "Render/Viewport/DisplayClusterViewportHelpers.h"

#include "Render/Viewport/Configuration/DisplayClusterViewportConfigurationHelpers.h"

#include "Render/Projection/IDisplayClusterProjectionPolicy.h"

#include "DisplayClusterConfigurationTypes_Base.h"
#include "DisplayClusterConfigurationTypes_ICVFX.h"
#include "DisplayClusterConfigurationTypes_PostRender.h"
#include "DisplayClusterConfigurationTypes_Viewport.h"
#include "DisplayClusterConfigurationTypes_ViewportRemap.h"

#include "Misc/DisplayClusterLog.h"

///////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterViewport
///////////////////////////////////////////////////////////////////////////////////////
const UDisplayClusterConfigurationViewport* FDisplayClusterViewport::GetViewportConfigurationData() const
{
	return ConfigurationData.Get();
}

/** Sets viewport configuration data. */
void FDisplayClusterViewport::SetViewportConfigurationData(const UDisplayClusterConfigurationViewport* InConfigurationData)
{
	ConfigurationData = InConfigurationData;
}

void FDisplayClusterViewport::UpdateConfiguration_ProjectionPolicy(const FDisplayClusterConfigurationProjection* InConfigurationProjectionPolicy)
{
	// Runtime update projection policy
	const bool bNeedUpdateProjectionPolicy = InConfigurationProjectionPolicy && (
		(ProjectionPolicy.IsValid() && ProjectionPolicy->IsConfigurationChanged(InConfigurationProjectionPolicy))
		|| (UninitializedProjectionPolicy.IsValid() && UninitializedProjectionPolicy->IsConfigurationChanged(InConfigurationProjectionPolicy))
		);

	if (bNeedUpdateProjectionPolicy)
	{
		UE_LOGF(LogDisplayClusterViewport, Verbose, "Update projection policy for viewport '%ls'.", *GetId());

		// Release current projection
		OnHandleEndScene();
		UninitializedProjectionPolicy.Reset();

		// Create new projection type interface
		UninitializedProjectionPolicy = FDisplayClusterViewportManager::CreateProjectionPolicy(GetId(), InConfigurationProjectionPolicy);
	}

	// If the scene is open, immediately initialize the viewport's scene resources
	if (Configuration->IsSceneOpened())
	{
		OnHandleStartScene();
	}
}

void FDisplayClusterViewport::UpdateConfiguration_OverlayRenderSettings(const FDisplayClusterConfigurationICVFX_OverlayAdvancedRenderSettings& InOverlaySettings)
{
	SetViewportBufferRatio(InOverlaySettings.BufferRatio);
	RenderSettings.RenderTargetRatio = InOverlaySettings.RenderTargetRatio;

	RenderSettings.GPUIndex = InOverlaySettings.GPUIndex;
	RenderSettings.StereoGPUIndex = InOverlaySettings.StereoGPUIndex;

	RenderSettings.bForceMono = FDisplayClusterViewportConfigurationHelpers::IsForceMonoscopicRendering(InOverlaySettings.StereoMode);
}

void FDisplayClusterViewport::UpdateConfiguration_Overscan(const FDisplayClusterViewport_OverscanSettings& InOverscanSettings)
{
	RenderSettings.OverscanSettings = InOverscanSettings;
}

void FDisplayClusterViewport::UpdateConfiguration_CameraMotionBlur(const FDisplayClusterViewport_CameraMotionBlur& InCameraMotionBlur)
{
	CameraMotionBlur.BlurSetup = InCameraMotionBlur;
}

void FDisplayClusterViewport::UpdateConfiguration_CameraDepthOfField(const FDisplayClusterViewport_CameraDepthOfField& InCameraDepthOfField)
{
	CameraDepthOfField = InCameraDepthOfField;
}

void FDisplayClusterViewport::UpdateConfiguration_PostRenderOverride(const FDisplayClusterConfigurationPostRender_Override& InOverride)
{
	PostRenderSettings.Replace.TextureRHI.SafeRelease();
	if (InOverride.bAllowReplace && InOverride.SourceTexture != nullptr)
	{
		FTextureResource* TextureResource = InOverride.SourceTexture->GetResource();
		if (TextureResource)
		{
			FTextureRHIRef& TextureRHI = TextureResource->TextureRHI;

			if (TextureRHI.IsValid())
			{
				PostRenderSettings.Replace.TextureRHI = TextureRHI;
				FIntVector Size = TextureRHI->GetSizeXYZ();

				PostRenderSettings.Replace.Rect = FDisplayClusterViewportHelpers::GetValidViewportRect((InOverride.bShouldUseTextureRegion) ? InOverride.TextureRegion.ToRect() : FIntRect(FIntPoint(0, 0), FIntPoint(Size.X, Size.Y)), GetId(), TEXT("Configuration Override"));
			}
		}
	}
}

void FDisplayClusterViewport::UpdateConfiguration_PostRenderBlur(const FDisplayClusterConfigurationPostRender_BlurPostprocess& InBlurPostprocess)
{
	switch (InBlurPostprocess.Mode)
	{
	case EDisplayClusterConfiguration_PostRenderBlur::Gaussian:
		PostRenderSettings.PostprocessBlur.Mode = EDisplayClusterShaderParameters_PostprocessBlur::Gaussian;
		break;
	case EDisplayClusterConfiguration_PostRenderBlur::Dilate:
		PostRenderSettings.PostprocessBlur.Mode = EDisplayClusterShaderParameters_PostprocessBlur::Dilate;
		break;
	default:
		PostRenderSettings.PostprocessBlur.Mode = EDisplayClusterShaderParameters_PostprocessBlur::None;
		break;
	}

	PostRenderSettings.PostprocessBlur.KernelRadius = InBlurPostprocess.KernelRadius;
	PostRenderSettings.PostprocessBlur.KernelScale = InBlurPostprocess.KernelScale;
}

void FDisplayClusterViewport::UpdateConfiguration_PostRenderGenerateMips(const FDisplayClusterConfigurationPostRender_GenerateMips& InGenerateMips)
{
	if (InGenerateMips.bAutoGenerateMips)
	{
		PostRenderSettings.GenerateMips.bAutoGenerateMips = true;

		PostRenderSettings.GenerateMips.MipsSamplerFilter = InGenerateMips.MipsSamplerFilter;

		PostRenderSettings.GenerateMips.MipsAddressU = InGenerateMips.MipsAddressU;
		PostRenderSettings.GenerateMips.MipsAddressV = InGenerateMips.MipsAddressV;

		PostRenderSettings.GenerateMips.MaxNumMipsLimit = (InGenerateMips.bEnabledMaxNumMips) ? InGenerateMips.MaxNumMips : -1;
	}
	else
	{
		// Disable mips
		PostRenderSettings.GenerateMips.bAutoGenerateMips = false;
	}
}

bool FDisplayClusterViewport::UpdateConfiguration_ViewportRemap(const FDisplayClusterConfigurationViewport_Remap& InRemapConfiguration)
{
	return ViewportRemap.UpdateViewportRemap(*this, InRemapConfiguration);
}
