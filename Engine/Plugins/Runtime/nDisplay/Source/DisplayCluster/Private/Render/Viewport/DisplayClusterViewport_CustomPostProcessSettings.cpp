// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Viewport/DisplayClusterViewport_CustomPostProcessSettings.h"
#include "Render/Viewport/DisplayClusterViewport.h"
#include "Render/Viewport/DisplayClusterViewportManager.h"

#include "Render/Viewport/Configuration/DisplayClusterViewportConfigurationHelpers_Postprocess.h"
#include "DisplayClusterConfigurationTypes_Postprocess.h"

// Override post-processing for nDisplay is allowed by default.
int32 GDisplayClusterPostProcessOverrideEnable = 1;
static FAutoConsoleVariableRef CVarDisplayClusterPostProcessOverrideEnable(
	TEXT("nDisplay.render.postprocess.override.enable"),
	GDisplayClusterPostProcessOverrideEnable,
	TEXT("Enable postprocess overrides for nDisplay (0 to disable).\n"),
	ECVF_RenderThreadSafe
);

// Override post-processing for InCamera viewports is allowed by default.
int32 GDisplayClusterPostProcessOverrideInCameraVFX = 1;
static FAutoConsoleVariableRef CVarDisplayClusterPostProcessOverrideInCameraVFX(
	TEXT("nDisplay.render.postprocess.override.InCameraVFX"),
	GDisplayClusterPostProcessOverrideInCameraVFX,
	TEXT("Enable post-processing override for ICVFX Camera viewport (0 to disable).\n"),
	ECVF_RenderThreadSafe
);

// By default, post-processing for Outers viewports is disabled.
// Because of some issues with the depth of field effect.
int32 GDisplayClusterPostProcessOverrideOutersVFX = 0;
static FAutoConsoleVariableRef CVarDisplayClusterPostProcessOverrideOutersVFX(
	TEXT("nDisplay.render.postprocess.override.OutersVFX"),
	GDisplayClusterPostProcessOverrideOutersVFX,
	TEXT("Enable postprocess override for ICVFX Outer viewports (0 to disable).\n"),
	ECVF_RenderThreadSafe
);

/**
* Auxiliary functions for post-processing.
*/
namespace UE::DisplayClusterViewport::CustomPostProcess
{
	/** Overrides the DoF post-processing parameters for the nDisplay viewport. */
	static inline bool OverrideDepthOfFieldPostProcessSettings(IDisplayClusterViewport* InViewport, const uint32 InContextNum, FPostProcessSettings& InOutPostProcessSettings)
	{
		check(InViewport);

		// This math only works for the CineCamera DoF because it provides a valid 'DepthOfFieldSensorFocalLength' value.
		if (InOutPostProcessSettings.DepthOfFieldFocalDistance <= 0.0f)
		{
			return false;
		}

		const TArray<FDisplayClusterViewport_Context>& InViewportContexts = InViewport->GetContexts();
		if (!InViewportContexts.IsValidIndex(InContextNum))
		{
			return false;
		}

		const FDisplayClusterViewport_Context& ViewportContext = InViewportContexts[InContextNum];
		if (!(ViewportContext.DepthOfField.SensorFocalLength > 0.f))
		{
			return false;
		}

		const FMatrix& ProjectionMatrix = ViewportContext.GetRenderProjectionMatrix();

		// Ignore if the projection matrix is invalid.
		if (ProjectionMatrix.M[0][0] == 0.0f || ProjectionMatrix.M[1][1] == 0.0f)
		{
			return false;
		}

		// M00 = 2n/(r-l)
		// M11 = 2n/(t-b)
		// => (r-l)/(t-b) = M11/M00 (= "SensorAspectRatio")
		const double SensorAspectRatio = ProjectionMatrix.M[1][1] / ProjectionMatrix.M[0][0];
		const double RenderingAspectRatio = double(ViewportContext.RenderTargetRect.Width()) / double(ViewportContext.RenderTargetRect.Height());
		const double SensorToRenderAspectRatio = SensorAspectRatio / RenderingAspectRatio;

		// Override sensor width such that DoF recovers our desired focal length
		// 
		// FocalLength = SensorWidth * M00 / 2
		// => SensorWidth = 2 * FocalLength / M00
		//
		InOutPostProcessSettings.bOverride_DepthOfFieldSensorWidth = true;
		InOutPostProcessSettings.DepthOfFieldSensorWidth = 2.0 * ViewportContext.DepthOfField.SensorFocalLength / ProjectionMatrix.M[0][0] / FMath::Pow(SensorToRenderAspectRatio, 2);

		// Compensate with squeeze factor the effect of non-square pixels onto bokeh squeeze.
		InOutPostProcessSettings.bOverride_DepthOfFieldSqueezeFactor = true;
		InOutPostProcessSettings.DepthOfFieldSqueezeFactor = ViewportContext.DepthOfField.SqueezeFactor * SensorToRenderAspectRatio;

		return true;
	}
};

///////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterViewport_CustomPostProcessSettings
///////////////////////////////////////////////////////////////////////////////////////
void FDisplayClusterViewport_CustomPostProcessSettings::AddCustomPostProcess(const ERenderPass InRenderPass, const FPostProcessSettings& InSettings, float BlendWeight, bool bSingleFrame)
{
	if (BlendWeight > 0.f) // Ignore PP with zero weights
	{
		PostprocessAsset.Emplace(InRenderPass, FPostprocessData(InSettings, BlendWeight, bSingleFrame));
	}
}

void FDisplayClusterViewport_CustomPostProcessSettings::RemoveCustomPostProcess(const ERenderPass InRenderPass)
{
	if (PostprocessAsset.Contains(InRenderPass))
	{
		PostprocessAsset.Remove(InRenderPass);
	}
}

bool FDisplayClusterViewport_CustomPostProcessSettings::GetCustomPostProcess(const ERenderPass InRenderPass, FPostProcessSettings& OutSettings, float* OutBlendWeight) const
{
	const FPostprocessData* ExistSettings = PostprocessAsset.Find(InRenderPass);
	if (ExistSettings && ExistSettings->bIsEnabled)
	{
		OutSettings = ExistSettings->Settings;

		// Returns the weight value, if appropriate.
		if (OutBlendWeight != nullptr)
		{
			*OutBlendWeight = ExistSettings->BlendWeight;
		}

		return true;
	}

	return false;
}

void FDisplayClusterViewport_CustomPostProcessSettings::FinalizeFrame()
{
	// Safe remove items out of iterator
	for (TPair<ERenderPass, FPostprocessData>& It: PostprocessAsset)
	{
		if (It.Value.bIsSingleFrame)
		{
			It.Value.bIsEnabled = false;
		}
	}
}

bool FDisplayClusterViewport_CustomPostProcessSettings::ApplyCustomPostProcess(IDisplayClusterViewport* InViewport, const uint32 InContextNum, const ERenderPass InRenderPass, FPostProcessSettings& InOutPPSettings, float* InOutBlendWeight) const
{
	bool bDidOverride = false;

	switch (InRenderPass)
	{
	case IDisplayClusterViewport_CustomPostProcessSettings::ERenderPass::Start:
	case IDisplayClusterViewport_CustomPostProcessSettings::ERenderPass::Override:
		bDidOverride = GetCustomPostProcess(InRenderPass, InOutPPSettings, InOutBlendWeight);
		break;

	case IDisplayClusterViewport_CustomPostProcessSettings::ERenderPass::Final:
	{
		// Obtaining custom 'Final' PostProcess settings.
		bDidOverride = GetCustomPostProcess(InRenderPass, InOutPPSettings, InOutBlendWeight);

		float PerViewportPPWeight = 0;
		FPostProcessSettings PerViewportPPSettings;

		// The `Final` and `FinalPerViewport` are always applied together.
		// If 'FinalPerViewport' is also used, apply nDisplay ColorGrading as well.
		if (GetCustomPostProcess(IDisplayClusterViewport_CustomPostProcessSettings::ERenderPass::FinalPerViewport, PerViewportPPSettings, &PerViewportPPWeight))
		{
			bDidOverride = true;

			// Extract nDisplay ColorGrading data from PostProcessSettings.
			FDisplayClusterConfigurationViewport_ColorGradingRenderingSettings FinalColorGrading, PerViewportColorGrading;
			FDisplayClusterViewportConfigurationHelpers_Postprocess::CopyPPSStruct(&FinalColorGrading, &InOutPPSettings);
			FDisplayClusterViewportConfigurationHelpers_Postprocess::CopyPPSStructConditional(&PerViewportColorGrading, &PerViewportPPSettings);

			// Blending both using our custom math instead of standard PPS blending
			FDisplayClusterViewportConfigurationHelpers_Postprocess::BlendPostProcessSettings(InOutPPSettings, FinalColorGrading, PerViewportColorGrading);
		}
	}
	break;

	default:
		break;
	}

	// Update post-processing settings for the viewport (DoF, Blur, etc.).
	if (ConfigurePostProcessSettingsForViewport(InViewport, InContextNum, InRenderPass, InOutPPSettings))
	{
		bDidOverride = true;
	}

	return bDidOverride;
}

bool FDisplayClusterViewport_CustomPostProcessSettings::ConfigurePostProcessSettingsForViewport(IDisplayClusterViewport* InViewport, const uint32 InContextNum, const ERenderPass InRenderPass, FPostProcessSettings& InOutPostProcessSettings) const
{
	using namespace UE::DisplayClusterViewport::CustomPostProcess;

	if (!InViewport || !GDisplayClusterPostProcessOverrideEnable)
	{
		return false;
	}

	// These flags are used to define the purpose of the viewport.
	EDisplayClusterViewportRuntimeICVFXFlags ICVFXRuntimeFlags = InViewport->GetRenderSettingsICVFX().RuntimeFlags;

	// The tile viewport does not contain ICVFX flags, and they must be obtained from a reference to the original viewport.
	const FDisplayClusterViewport_RenderSettings& RenderSettings = InViewport->GetRenderSettings();
	if (RenderSettings.TileSettings.IsInternalViewport())
	{
		if (IDisplayClusterViewportManager* ViewportManager = InViewport->GetConfiguration().GetViewportManager())
		{
			// Find source viewport.
			const FString& SourceViewportId = RenderSettings.TileSettings.GetSourceViewportId();
			if (IDisplayClusterViewport* SourceViewport = ViewportManager->FindViewport(SourceViewportId))
			{
				// Use the ICVFX flags from the source viewport.
				ICVFXRuntimeFlags = SourceViewport->GetRenderSettingsICVFX().RuntimeFlags;
			}
		}
	}

	// Ignore ICVFX cameras
	if (!GDisplayClusterPostProcessOverrideInCameraVFX && EnumHasAnyFlags(ICVFXRuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::InCamera))
	{
		return false;
	}

	// Ignore Outers for ICVFX
	if (!GDisplayClusterPostProcessOverrideOutersVFX && EnumHasAnyFlags(ICVFXRuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::Target))
	{
		return false;
	}

	// If there are any changes to the PP settings, return true.
	bool bPostProcessSettingsHaveChanged = false;

	if (InRenderPass == IDisplayClusterViewport_CustomPostProcessSettings::ERenderPass::Override)
	{
		// Updates the CineCamera DoF PP settings for the current viewport.
		bPostProcessSettingsHaveChanged |= OverrideDepthOfFieldPostProcessSettings(InViewport, InContextNum, InOutPostProcessSettings);
	}

	return bPostProcessSettingsHaveChanged;
}
