// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Engine/Scene.h"
#include "Render/Viewport/IDisplayClusterViewport_CustomPostProcessSettings.h"


class FDisplayClusterViewport_CustomPostProcessSettings
	: public IDisplayClusterViewport_CustomPostProcessSettings
{
public:
	FDisplayClusterViewport_CustomPostProcessSettings() = default;
	virtual ~FDisplayClusterViewport_CustomPostProcessSettings() = default;

public:
	//~ Begin IDisplayClusterViewport_CustomPostProcessSettings
	virtual void AddCustomPostProcess(const ERenderPass InRenderPass, const FPostProcessSettings& InSettings, float BlendWeight, bool bSingleFrame) override;
	virtual void RemoveCustomPostProcess(const ERenderPass InRenderPass) override;

	virtual bool ApplyCustomPostProcess(IDisplayClusterViewport* InViewport, const uint32 InContextNum, const ERenderPass InRenderPass, FPostProcessSettings& InOutPPSettings, float* InOutBlendWeight = nullptr) const override;
	//~End IDisplayClusterViewport_CustomPostProcessSettings

	void FinalizeFrame();

private:
	/** Get the custom postprocess settings for the specified rendering pass.
	* Returns false if no custom post process exists for the specified InRenderPass.
	*/
	bool GetCustomPostProcess(const ERenderPass InRenderPass, FPostProcessSettings& OutSettings, float* OutBlendWeight) const;

	/** Applies changes to some postprocessing parameters depending on the viewport context. (DoF, etc.)
	*
	* @param InViewport       - DC viewport that is rendered with these settings.
	* @param InContextNum     - Index of view that is being processed for this viewport
	* @param InRenderPass     - PP rendering pass to be used. (The value of 'FinalPerViewport' is ignored.)
	* @param InOutPPSettings  - (in,out) PostProcess settings that need to be changed.
	*
	* @return true, if the postprocess settings have been modified.
	*/
	bool ConfigurePostProcessSettingsForViewport(IDisplayClusterViewport* InViewport, const uint32 InContextNum, const ERenderPass InRenderPass, FPostProcessSettings& InOutPostProcessSettings) const;

private:
	struct FPostprocessData
	{
		FPostProcessSettings Settings;

		float BlendWeight = 1.f;

		bool bIsEnabled = true;
		bool bIsSingleFrame = false;

		FPostprocessData(const FPostProcessSettings& InSettings, float InBlendWeight, bool bInSingleFrame)
			: Settings(InSettings)
			, BlendWeight(InBlendWeight)
			, bIsSingleFrame(bInSingleFrame)
		{ }
	};

	// Custom post processing settings
	TMap<ERenderPass, FPostprocessData> PostprocessAsset;
};
