// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Viewport/DisplayClusterViewport.h"
#include "Render/Viewport/DisplayClusterViewportProxy.h"
#include "Render/Projection/IDisplayClusterProjectionPolicy.h"

#include "Misc/DisplayClusterLog.h"

#include "DisplayClusterRootActor.h"
#include "Components/DisplayClusterScreenComponent.h"
#include "Render/DisplayDevice/Components/DisplayClusterDisplayDeviceBaseComponent.h"

#include "SceneManagement.h"

///////////////////////////////////////////////////////////////////////////////////////
//          FDisplayClusterViewport
///////////////////////////////////////////////////////////////////////////////////////
UDisplayClusterDisplayDeviceBaseComponent* FDisplayClusterViewport::GetDisplayDeviceComponent(const EDisplayClusterRootActorType InRootActorType) const
{
	if (!ProjectionPolicy.IsValid())
	{
		// ignore viewports with uninitialized prj policy
		return nullptr;
	}

	if (!Configuration->IsSceneOpened())
	{
		return nullptr;
	}

	if (ADisplayClusterRootActor* RootActor = Configuration->GetRootActor(InRootActorType))
	{
		// Get display device ID assigned to the viewport
		const FString& DisplayDeviceId = GetRenderSettings().DisplayDeviceId;

		// Default Display Device
		if (DisplayDeviceId.IsEmpty())
		{
			return RootActor->GetDefaultDisplayDevice();
		}

		// Manual Display Device
		return RootActor->GetComponentByName<UDisplayClusterDisplayDeviceBaseComponent>(DisplayDeviceId);
	}

	return nullptr;
}

void FDisplayClusterViewport::CleanupViewState()
{
	for (TSharedPtr<FSceneViewStateReference, ESPMode::ThreadSafe>& ViewState : ViewStates)
	{
		if (ViewState.IsValid())
		{
			if (FSceneViewStateInterface* Ref = ViewState->GetReference())
			{
				Ref->ClearMIDPool();
			}
		}
	}
}

TSharedPtr<FSceneViewStateReference, ESPMode::ThreadSafe> FDisplayClusterViewport::GetOrCreateViewState(uint32 ViewIndex)
{
	if(!ShouldUseViewStates())
	{
		// Release ViewState resources if they are not used for rendering.
		ViewStates.Empty();

		return nullptr;
	}

	// Do not create or access ViewState for ViewIndex outside the configured range.
	const uint32 MaxViewIndex = FMath::Max(0, Configuration->GetRenderFrameSettings().GetViewPerViewportAmount());
	if (ViewIndex >= MaxViewIndex)
	{
		return nullptr;
	}

	int32 RequiredAmount = (int32)ViewIndex - ViewStates.Num() + 1;
	if (RequiredAmount > 0)
	{
		ViewStates.AddDefaulted(RequiredAmount);
	}

	if (!ViewStates[ViewIndex].IsValid())
	{
		ViewStates[ViewIndex] = MakeShared<FSceneViewStateReference>();
	}

	if (ViewStates[ViewIndex]->GetReference() == NULL)
	{
		const UWorld* CurrentWorld = Configuration->GetCurrentWorld();
		const ERHIFeatureLevel::Type FeatureLevel = CurrentWorld ? CurrentWorld->GetFeatureLevel() : GMaxRHIFeatureLevel;

		ViewStates[ViewIndex]->Allocate(FeatureLevel);
	}

	return ViewStates[ViewIndex];
}