// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Viewport/DisplayClusterViewport.h"
#include "Render/Viewport/DisplayClusterViewportManager.h"
#include "Render/Viewport/DisplayClusterViewportStereoscopicPass.h"

#include "Render/Viewport/Configuration/DisplayClusterViewportConfiguration.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_PostRenderSettings.h"

#include "Render/Projection/IDisplayClusterProjectionPolicy.h"
#include "Math/DisplayClusterProjectionMath.h"

#include "Render/Viewport/RenderFrame/DisplayClusterRenderFrame.h"
#include "Render/Viewport/RenderFrame/DisplayClusterRenderFrameSettings.h"
#include "Render/Viewport/RenderTarget/DisplayClusterRenderTargetResource.h"

#include "DisplayClusterSceneViewExtensions.h"

#include "Components/DisplayClusterCameraComponent.h"
#include "Misc/DisplayClusterLog.h"

#include "EngineUtils.h"
#include "SceneView.h"


///////////////////////////////////////////////////////////////////////////////////////
//          FDisplayClusterViewport
///////////////////////////////////////////////////////////////////////////////////////
FVector2D FDisplayClusterViewport::GetClippingPlanes() const
{
	float NCP = (CustomNearClippingPlane >= 0)
		? CustomNearClippingPlane
		: GNearClippingPlane;

	// nDisplay does not use the far plane of the clipping
	float FCP = NCP;

	// Projection policies can change the clipping planes (domeprojection).
	if (ProjectionPolicy)
	{
		ProjectionPolicy->ApplyClippingPlanesOverrides(this, NCP, FCP);
	}

	return FVector2D(NCP, FCP);
}

bool FDisplayClusterViewport::GetViewPointCameraEye(const uint32 InContextNum, FVector& OutViewLocation, FRotator& OutViewRotation, FVector& OutViewOffset)
{
	// Here we use the ViewPoint component as the eye position
	if (UDisplayClusterCameraComponent* SceneCameraComponent = GetViewPointCameraComponent(EDisplayClusterRootActorType::Scene))
	{
		// Get eye position from the ViewPoint component.
		SceneCameraComponent->GetEyePosition(GetConfiguration(), OutViewLocation, OutViewRotation);

		// Calculate stereo ViewOffset:
		OutViewOffset = FDisplayClusterProjectionMath::GetStereoEyeOffset(OutViewRotation, GetStereoEyeOffsetDistance(InContextNum));

		// Add stereo eye offset to the output view location.
		OutViewLocation += OutViewOffset;

		return true;
	}

	return false;
}
bool FDisplayClusterViewport::CalculateView(const uint32 InContextNum, FVector& InOutViewLocation, FRotator& InOutViewRotation, const float WorldToMeters)
{
	if (Contexts.IsValidIndex(InContextNum))
	{
		// The function can be called several times per frame.
		// Each time it must return the same values. For optimization purposes, after the first call this function
		// stores the result in the context variables 'ViewLocation' and 'ViewRotation'.
		// Finally, raises this flag for subsequent calls in the current frame.

		// Calculate once per frame.
		if (!Contexts[InContextNum].bCalculated)
		{
			CalculateViewData(InContextNum, InOutViewLocation, InOutViewRotation, WorldToMeters);
		}

		if (Contexts[InContextNum].bValidData)
		{
			// Use calculated values
			// Since this function can be called several times from LocalPlayer.cpp, the cached values are used on repeated calls.
			// This should give a performance boost for 'mesh', 'mpcdi' projections with a large number of vertices in the geometry or large warp texture size.
			InOutViewLocation = Contexts[InContextNum].ViewData.RenderLocation;
			InOutViewRotation = Contexts[InContextNum].ViewData.RenderRotation;

			return true;
		}
	}

	return false;
}

bool FDisplayClusterViewport::CalculateViewData(const uint32 InContextNum, const FVector& InViewLocation, const FRotator& InViewRotation, const float InWorldToMeters)
{
	// Use the provided WorldToMeters value when valid; otherwise fall back to the configuration value.
	const float WorldToMeters = InWorldToMeters > 0 ? InWorldToMeters : Configuration->GetWorldToMeters();

	if (!PreCalculateViewData(InContextNum, InViewLocation, InViewRotation, WorldToMeters))
	{
		return false;
	}

	FDisplayClusterViewport_Context& OutContext = Contexts[InContextNum];

	// Gets the projection policy flags that control the logic below:
	const EDisplayClusterProjectionPolicyFlags ProjectionPolicyFlags = ProjectionPolicy->GetProjectionPolicyFlags(this, InContextNum);

	// Evaluate View
	FVector ViewLocation(InViewLocation);
	FRotator ViewRotation(InViewRotation);

	// Store Camera in world space
	OutContext.ViewData.CameraOrigin = ViewLocation;
	OutContext.ViewData.CameraRotation = ViewRotation;

	// Uses the eye position (ViewPoint) instead of the camera
	if (EnumHasAnyFlags(ProjectionPolicyFlags, EDisplayClusterProjectionPolicyFlags::UseEyeTracking))
	{
		// Here we use the ViewPoint component as the eye position
		if (UDisplayClusterCameraComponent* SceneCameraComponent = GetViewPointCameraComponent(EDisplayClusterRootActorType::Scene))
		{
			// Get eye position from the ViewPoint component.
			SceneCameraComponent->GetEyePosition(GetConfiguration(), ViewLocation, ViewRotation);
		}
	}

	// Store Eye in world space
	OutContext.ViewData.EyeOrigin = ViewLocation;
	OutContext.ViewData.EyeRotation = ViewRotation;

	// Stereo offset distance
	const float StereoEyeOffsetDistance = GetStereoEyeOffsetDistance(InContextNum);
	FVector ViewOffset = FDisplayClusterProjectionMath::GetStereoEyeOffset(ViewRotation, StereoEyeOffsetDistance);

	// Projection policy expects view location with stereo offset.
	ViewLocation += ViewOffset;

	// Convert eye to local space and to Geometry units
	if (EnumHasAnyFlags(ProjectionPolicyFlags, EDisplayClusterProjectionPolicyFlags::UseLocalSpace))
	{
		ViewRotation = OutContext.OriginToWorld.InverseTransformRotation(ViewRotation.Quaternion()).Rotator();

		// Scale units from Geometry to UE World
		const float ScaleWorldToGeometry = OutContext.GeometryToMeters / OutContext.WorldToMeters;

		ViewLocation = OutContext.OriginToWorld.InverseTransformPosition(ViewLocation) * ScaleWorldToGeometry;
		ViewOffset = OutContext.OriginToWorld.InverseTransformPosition(ViewOffset) * ScaleWorldToGeometry;
	}

	// Calculate View
	if (!ProjectionPolicy->CalculateView(this, InContextNum, ViewLocation, ViewRotation, ViewOffset, WorldToMeters, OutContext.ProjectionData.ZNear, OutContext.ProjectionData.ZFar))
	{
		// CalculateView() failed.
		return false;
	}

	// Convert eye to world space and units
	if (EnumHasAnyFlags(ProjectionPolicyFlags, EDisplayClusterProjectionPolicyFlags::UseLocalSpace))
	{
		ViewRotation = OutContext.OriginToWorld.TransformRotation(ViewRotation.Quaternion()).Rotator();

		// Scale units from Geometry to UE World
		const float ScaleGeometryToWorld = OutContext.WorldToMeters / OutContext.GeometryToMeters;

		ViewLocation = OutContext.OriginToWorld.TransformPosition(ViewLocation * ScaleGeometryToWorld);
	}

	// Save view: DCRA related view
	OutContext.ViewLocation = ViewLocation;
	OutContext.ViewRotation = ViewRotation;

	// Follow camera feature:
	// At this point, let's change the values for rendering only.
	if (EnumHasAnyFlags(ProjectionPolicyFlags, EDisplayClusterProjectionPolicyFlags::EnableFollowCameraFeature))
	{
		const UDisplayClusterCameraComponent* ConfigurationCameraComponent = GetViewPointCameraComponent(EDisplayClusterRootActorType::Configuration);
		const UDisplayClusterCameraComponent* SceneCameraComponent = GetViewPointCameraComponent(EDisplayClusterRootActorType::Scene);
		if (SceneCameraComponent && ConfigurationCameraComponent && ConfigurationCameraComponent->ShouldFollowCameraLocation())
		{
			FMinimalViewInfo CameraViewInfo;
			SceneCameraComponent->GetDesiredView(*Configuration, CameraViewInfo);

			const FTransform CameraToWorld(CameraViewInfo.Rotation, CameraViewInfo.Location);

			// Follow camera position: use camera position as a DCRA position without moving of an actor.
			// World -> [DCRA local space = CAMERA local space] -> World
			const FVector RootActorSpaceLocation = OutContext.RootActorToWorld.InverseTransformPosition(ViewLocation);
			const FVector NewViewLocation = CameraToWorld.TransformPosition(RootActorSpaceLocation);

			const FQuat    RootActorSpaceRotation = OutContext.RootActorToWorld.InverseTransformRotation(ViewRotation.Quaternion());
			const FRotator NewViewRotation = CameraToWorld.TransformRotation(RootActorSpaceRotation).Rotator();

			// Update calculated values and update the state of the context
			ViewLocation = NewViewLocation;
			ViewRotation = NewViewRotation;
		}
	}

	// save render view
	OutContext.ViewData.RenderLocation = ViewLocation;
	OutContext.ViewData.RenderRotation = ViewRotation;

	// calc final origin location
	const FVector RenderViewOffset = FDisplayClusterProjectionMath::GetStereoEyeOffset(ViewRotation, StereoEyeOffsetDistance);
	ViewLocation -= RenderViewOffset;
	OutContext.ViewData.RenderCameraLocation = ViewLocation;

	// Calculate Projection matrix
	FMatrix ProjectionMatrix;
	if (!ProjectionPolicy->GetProjectionMatrix(this, InContextNum, ProjectionMatrix))
	{
		// GetProjectionMatrix() failed.
		return false;
	}

	OutContext.ProjectionMatrix = ProjectionMatrix;

	// Set this context data as valid.
	OutContext.bValidData = true;

	return true;
}

bool FDisplayClusterViewport::GetProjectionMatrix(const uint32 InContextNum, FMatrix& OutPrjMatrix)
{
	if (Contexts.IsValidIndex(InContextNum))
	{
		if (Contexts[InContextNum].bValidData)
		{
			// Use calculated values
			OutPrjMatrix = Contexts[InContextNum].GetRenderProjectionMatrix();
			return true;
		}
	}

	return false;
}

void FDisplayClusterViewport::CalculateProjectionMatrix(const uint32 InContextNum, float Left, float Right, float Top, float Bottom, float ZNear, float ZFar, bool bIsAnglesInput)
{
	const double n = ZNear;
	const double f = ZFar;

	// limit max frustum to 89 degree
	const double MaxAngleDegree = 89;
	const double MaxValue = FDisplayClusterProjectionMath::DegreesToNearPlaneExtent(MaxAngleDegree, n);


	double t = bIsAnglesInput ? FDisplayClusterProjectionMath::DegreesToNearPlaneExtent(static_cast<double>(Top),    n) : Top;
	double b = bIsAnglesInput ? FDisplayClusterProjectionMath::DegreesToNearPlaneExtent(static_cast<double>(Bottom), n) : Bottom;
	double l = bIsAnglesInput ? FDisplayClusterProjectionMath::DegreesToNearPlaneExtent(static_cast<double>(Left),   n) : Left;
	double r = bIsAnglesInput ? FDisplayClusterProjectionMath::DegreesToNearPlaneExtent(static_cast<double>(Right),  n) : Right;

	// Protect PrjMatrix from bad input values, and fix\clamp FOV to limits
	{
		// Protect from broken input data, return valid matrix
		if (isnan(l) || isnan(r) || isnan(t) || isnan(b) || isnan(n) || isnan(f) || n <= 0)
		{
			return;
		}

		// Ignore inverted frustum
		if (l > r || b > t)
		{
			return;
		}

		// Clamp frustum values in range -89..89 degree
		l = FMath::Clamp(l, -MaxValue, MaxValue);
		r = FMath::Clamp(r, -MaxValue, MaxValue);
		t = FMath::Clamp(t, -MaxValue, MaxValue);
		b = FMath::Clamp(b, -MaxValue, MaxValue);
	}

	// Support custom frustum rendering
	const double OrigValues[] = {l, r, t, b};
	if (FDisplayClusterViewport_CustomFrustumRuntimeSettings::UpdateProjectionAngles(CustomFrustumRuntimeSettings, l, r, t, b))
	{
		const bool bIsValidLimits =  FMath::IsWithin(l, -MaxValue, MaxValue)
							&& FMath::IsWithin(r, -MaxValue, MaxValue)
							&& FMath::IsWithin(t, -MaxValue, MaxValue)
							&& FMath::IsWithin(b, -MaxValue, MaxValue);

		if (!bIsValidLimits)
		{
			// overscan out of frustum : disable
			CustomFrustumRuntimeSettings.bIsEnabled = false;

			// restore orig values
			l = OrigValues[0];
			r = OrigValues[1];
			t = OrigValues[2];
			b = OrigValues[3];
		}
	}

	FDisplayClusterProjectionMath::GetNonZeroFrustumRange(l, r, n);
	FDisplayClusterProjectionMath::GetNonZeroFrustumRange(b, t, n);

	Contexts[InContextNum].ProjectionMatrix = FDisplayClusterProjectionMath::MakeAsymmetricProjectionMatrix(l, r, t, b, n, f);

	// Update cached projection data:
	FDisplayClusterViewport_Context::FCachedProjectionData& CachedProjectionData = Contexts[InContextNum].ProjectionData;
	CachedProjectionData.bValid = true;

	// Store projection angles
	CachedProjectionData.ProjectionAngles = FVector4(l, r, t, b);

	// Update clipping planes
	CachedProjectionData.ZNear = n;
	CachedProjectionData.ZFar = f;

	if (FDisplayClusterViewport_OverscanRuntimeSettings::UpdateProjectionAngles(OverscanRuntimeSettings, l, r, t, b))
	{
		if (FMath::IsWithin(l, -MaxValue, MaxValue) &&
			FMath::IsWithin(r, -MaxValue, MaxValue) &&
			FMath::IsWithin(t, -MaxValue, MaxValue) &&
			FMath::IsWithin(b, -MaxValue, MaxValue)
			)
		{
			// Use overscan projection matrix
			Contexts[InContextNum].OverscanProjectionMatrix = FDisplayClusterProjectionMath::MakeAsymmetricProjectionMatrix(l, r, t, b, n, f);

			// Cache projection data for overscan
			CachedProjectionData.bUseOverscan = true;
			CachedProjectionData.OverscanProjectionAngles = FVector4(l, r, t, b);

			return;
		}
	}

	// overscan out of frustum: disable
	OverscanRuntimeSettings.bIsEnabled = false;
}

///////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterViewport
///////////////////////////////////////////////////////////////////////////////////////
FMatrix IDisplayClusterViewport::MakeProjectionMatrix(float l, float r, float t, float b, float n, float f)
{
	return FDisplayClusterProjectionMath::MakeAsymmetricProjectionMatrix(
		static_cast<double>(l), static_cast<double>(r),
		static_cast<double>(t), static_cast<double>(b),
		static_cast<double>(n), static_cast<double>(f));
}
