// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Viewport/DisplayClusterViewport_OpenColorIO.h"
#include "Render/Viewport/DisplayClusterViewport.h"

#include "Render/Viewport/Containers/DisplayClusterViewport_Context.h"
#include "Misc/DisplayClusterColorEncoding.h"

#include "OpenColorIOConfiguration.h"
#include "OpenColorIOColorTransform.h"
#include "OpenColorIODisplayManager.h"
#include "OpenColorIODisplayExtension.h"
#include "OpenColorIOShader.h"

#include "IDisplayClusterShadersTextureUtils.h"

#include "SceneView.h"
#include "ScreenPass.h"
#include "CommonRenderResources.h"
#include "PostProcess/PostProcessMaterialInputs.h"
#include "Engine/TextureRenderTarget.h"

//////////////////////////////////////////////////////////////////////////
// FDisplayClusterViewport_OpenColorIO
//////////////////////////////////////////////////////////////////////////
FDisplayClusterViewport_OpenColorIO::FDisplayClusterViewport_OpenColorIO(const FOpenColorIOColorConversionSettings& InDisplayConfiguration)
	: ConversionSettings(InDisplayConfiguration)
{ }

FDisplayClusterViewport_OpenColorIO::~FDisplayClusterViewport_OpenColorIO()
{ }

void FDisplayClusterViewport_OpenColorIO::SetupSceneView(FSceneViewFamily& InOutViewFamily, FSceneView& InOutView)
{
	FOpenColorIORenderPassResources PassResources = FOpenColorIORendering::GetRenderPassResources(ConversionSettings, InOutViewFamily.GetFeatureLevel());
	if (PassResources.IsValid())
	{
		FOpenColorIORendering::PrepareView(InOutViewFamily, InOutView);
	}

	// Update data on rendering thread
	ENQUEUE_RENDER_COMMAND(ProcessColorSpaceTransform)(
		[This = SharedThis(this), ResourcesRenderThread = MoveTemp(PassResources)](FRHICommandListImmediate& RHICmdList)
		{
			//Caches render thread resource to be used when applying configuration in PostRenderViewFamily_RenderThread
			This->CachedResourcesRenderThread = ResourcesRenderThread;
		}
	);
}

bool FDisplayClusterViewport_OpenColorIO::IsConversionSettingsEqual(const FOpenColorIOColorConversionSettings& InConversionSettings) const
{
	return ConversionSettings.ToString() == InConversionSettings.ToString();
}

float FDisplayClusterViewport_OpenColorIO::GetGammaCorrection(const struct FDisplayClusterColorEncoding& InColorEncoding)
{
	// OCIO shader applies gamma correction BEFORE unpremult
	// do not do gamma correction for pre-multiplied textures
	if (InColorEncoding.Premultiply == EDisplayClusterColorPremultiply::None)
	{
		const EDisplayClusterColorEncoding EqualEncoding = InColorEncoding.GetEqualEncoding();
		if (EqualEncoding == EDisplayClusterColorEncoding::Gamma)
		{
			// Get gamma correction value
			return UTextureRenderTarget::GetDefaultDisplayGamma() / InColorEncoding.GammaValue;
		}
	}

	// No gamma correction
	return 1.f;
}

EOpenColorIOTransformAlpha FDisplayClusterViewport_OpenColorIO::GetTransformAlpha(const FDisplayClusterColorEncoding& InColorEncoding)
{
	// Unpremultiply while gamma conversion
	switch (InColorEncoding.Premultiply)
	{
	case EDisplayClusterColorPremultiply::Premultiply:       return EOpenColorIOTransformAlpha::Unpremultiply;
	case EDisplayClusterColorPremultiply::InvertPremultiply: return EOpenColorIOTransformAlpha::InvertUnpremultiply;
	default:
		break;
	}

	return EOpenColorIOTransformAlpha::None;;
}

bool FDisplayClusterViewport_OpenColorIO::AddPass_RenderThread(
	const FDisplayClusterShadersTextureUtilsSettings& InTextureUtilsSettings,
	const TSharedRef<IDisplayClusterShadersTextureUtils>& InTextureUtils) const
{
	if (!CachedResourcesRenderThread.IsValid())
{
		// The OCIO shader is not ready at this point, use the default resolve method.
		InTextureUtils->Resolve(InTextureUtilsSettings);

		return true;
	}

	// When requesting RDG from the TextureUtils API, it switches from RHI to RDG.
	FRDGBuilder& GraphBuilder = InTextureUtils->GetOrCreateRDGBuilder();

	bool bFinalPassResult = false;
	InTextureUtils->ForEachContextByPredicate([&](
		const FDisplayClusterShadersTextureViewportContext& Input,
		const FDisplayClusterShadersTextureViewportContext& Output)
		{
			// returns true if OCIO is applied.
			bFinalPassResult = true;

			// The OCIO output is in sRGB
			FOpenColorIORendering::AddPass_RenderThread(
				GraphBuilder,
				FScreenPassViewInfo(),
				GMaxRHIFeatureLevel,
						Input.ToScreenPassTexture(),
						FScreenPassRenderTarget(Output.ToScreenPassTexture(), ERenderTargetLoadAction::EClear),
				CachedResourcesRenderThread,
				GetGammaCorrection(Input.ColorEncoding),
				GetTransformAlpha(Input.ColorEncoding)
			);

			// Convert OCIO output(sRGB) to output color space if necessary.
			if (Output.ColorEncoding.GetEqualEncoding() != EDisplayClusterColorEncoding::sRGB)
			{
				// Convert OutputTexture color encoding from sRGB to Linear(Holdout):
				InTextureUtils->ResolveTextureContext(
					// Customize the settings by requesting a temporary input texture cloned from the output texture.
					{ InTextureUtilsSettings , EDisplayClusterShaderTextureUtilsFlags::UseOutputTextureAsInput },
					// The input texture is empty, a temporary texture created from the output will be used instead.
					{ {}, {EDisplayClusterColorEncoding::Gamma, Input.ColorEncoding.Premultiply} },
					// The output texture already contains the OCIO result and will be used as input in this pass.
					Output
				);
			}
		});

	return true;
}

// This is a copy of FOpenColorIODisplayExtension::PostProcessPassAfterTonemap_RenderThread()
FScreenPassTexture FDisplayClusterViewport_OpenColorIO::PostProcessPassAfterTonemap_RenderThread(FRDGBuilder& GraphBuilder, const FDisplayClusterColorEncoding& InColorEncoding,
	const FSceneView& View, const FPostProcessMaterialInputs& InOutInputs)
{
	const FScreenPassTexture& SceneColor = FScreenPassTexture::CopyFromSlice(GraphBuilder, InOutInputs.GetInput(EPostProcessMaterialInput::SceneColor));
	check(SceneColor.IsValid());
	FScreenPassRenderTarget Output = InOutInputs.OverrideOutput;

	// If the override output is provided, it means that this is the last pass in post processing.
	if (!Output.IsValid())
	{
		Output = FScreenPassRenderTarget::CreateFromInput(GraphBuilder, SceneColor, View.GetOverwriteLoadAction(), TEXT("OCIORenderTarget"));
	}

	FOpenColorIORendering::AddPass_RenderThread(
		GraphBuilder,
		View,
		SceneColor,
		Output,
		CachedResourcesRenderThread,
		GetTransformAlpha(InColorEncoding)
	);

	return MoveTemp(Output);
}
