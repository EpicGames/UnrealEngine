// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "RHI.h"
#include "RHICommandList.h"
#include "RHIResources.h"
#include "TextureResource.h"
#include "Templates/SharedPointer.h"

#include "OpenColorIOColorSpace.h"
#include "OpenColorIORendering.h"

class FSceneView;
class FSceneViewFamily;
struct FScreenPassTexture;
struct FPostProcessMaterialInputs;
struct FDisplayClusterColorEncoding;
class FRDGBuilder;
class FDisplayClusterViewport_Context;

/**
 * nDisplay OCIO implementation.
 * 
 */
class FDisplayClusterViewport_OpenColorIO
	: public TSharedFromThis<FDisplayClusterViewport_OpenColorIO, ESPMode::ThreadSafe>
{
public:
	FDisplayClusterViewport_OpenColorIO(const FOpenColorIOColorConversionSettings& InDisplayConfiguration);
	virtual ~FDisplayClusterViewport_OpenColorIO();

public:
	/** Setup view for OCIO.
	 *
	 * @param InOutViewFamily - [In,Out] ViewFamily.
	 * @param InOutView       - [In,Out] View.
	 *
	 * @return - none.
	 */
	void SetupSceneView(FSceneViewFamily& InOutViewFamily, FSceneView& InOutView);

	/** Add OCIO render pass.
	 *
	 * @param TextureUtils   - resolving data
	 *
	 * @return - true if success.
	 */
	bool AddPass_RenderThread(
		const struct FDisplayClusterShadersTextureUtilsSettings& InTextureUtilsSettings,
		const TSharedRef<class IDisplayClusterShadersTextureUtils>& TextureUtils) const;

	/* This is a copy of FOpenColorIODisplayExtension::PostProcessPassAfterTonemap_RenderThread() */
	FScreenPassTexture PostProcessPassAfterTonemap_RenderThread(FRDGBuilder& GraphBuilder, const FDisplayClusterColorEncoding& InColorEncoding,
		const FSceneView& View, const FPostProcessMaterialInputs& InOutInputs);

	/** Compare two OCIO configurations.
	 *
	 * @param InConversionSettings - configuration to compare with.
	 *
	 * @return - true if equal.
	 */
	bool IsConversionSettingsEqual(const FOpenColorIOColorConversionSettings& InConversionSettings) const;

	/** Get current OCIO conversion settings. */
	const FOpenColorIOColorConversionSettings& GetConversionSettings() const
	{
		return ConversionSettings;
	}

	/** Returns true if OCIO can be used in the rendering thread. */
	bool IsValid_RenderThread() const
	{
		return CachedResourcesRenderThread.IsValid();
	}

	/** Gets the gamma correction for the OCIO shader from the InColorEncoding parameter. */
	static float GetGammaCorrection(const FDisplayClusterColorEncoding& InColorEncoding);

	/** Gets transform alpha value from the InColorEncoding parameter. */
	static EOpenColorIOTransformAlpha GetTransformAlpha(const FDisplayClusterColorEncoding& InColorEncoding);

private:
	/** Cached pass resources required to apply conversion for render thread. */
	FOpenColorIORenderPassResources CachedResourcesRenderThread;

	/** Configuration to apply during post render callback. */
	FOpenColorIOColorConversionSettings ConversionSettings;
};
