// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Viewport/DisplayClusterViewport.h"
#include "Render/Viewport/DisplayClusterViewportHelpers.h"
#include "Render/Viewport/DisplayClusterViewportManager.h"
#include "Render/Viewport/DisplayClusterViewportProxy.h"
#include "Render/Viewport/DisplayClusterViewportManagerProxy.h"
#include "Render/Viewport/DisplayClusterViewport_OpenColorIO.h"
#include "Render/Viewport/DisplayClusterViewportManagerViewExtension.h"

#include "Render/Viewport/Configuration/DisplayClusterViewportConfiguration.h"
#include "Render/Viewport/Configuration/DisplayClusterViewportConfigurationProxy.h"
#include "Render/Viewport/LightCard/DisplayClusterViewportLightCardManager.h"

#include "Render/Projection/IDisplayClusterProjectionPolicy.h"

#include "Render/Viewport/DisplayClusterViewportStereoscopicPass.h"
#include "Render/Viewport/RenderFrame/DisplayClusterRenderFrame.h"
#include "Render/Viewport/RenderFrame/DisplayClusterRenderFrameSettings.h"
#include "Render/Viewport/RenderTarget/DisplayClusterRenderTargetResource.h"

#include "Render/Viewport/Containers/DisplayClusterViewport_PostRenderSettings.h"
#include "Render/Viewport/Containers/DisplayClusterViewportProxyData.h"

#include "Render/Viewport/Preview/DisplayClusterViewportPreview.h"

#include "Render/DisplayDevice/Components/DisplayClusterDisplayDeviceBaseComponent.h"

#include "IDisplayCluster.h"
#include "IDisplayClusterCallbacks.h"

#include "EngineUtils.h"
#include "PrimitiveDrawingUtils.h"
#include "SceneView.h"
#include "UnrealClient.h"

#include "DisplayClusterSceneViewExtensions.h"
#include "DisplayClusterConfigurationTypes_Viewport.h"

#include "LegacyScreenPercentageDriver.h"

#include "Misc/CommandLine.h"
#include "Misc/DisplayClusterGlobals.h"
#include "Misc/DisplayClusterLog.h"
#include "Misc/Parse.h"

/** Auxiliary enumeration for CVar “DC.LumenPerView”. */
enum class EDisplayClusterLumenPerView : uint8
{
	// Disabled
	Disabled = 0,

	// Use only in Cluster.
	UseOnlyInCluster,

	// Use in both Cluster and Preview.
	UseEverywhere,

	// Use only in Preview.
	UseOnlyInPreview,

	// Use only for Inner Frustum in Cluster.
	UseOnlyForICVXFCameraInCluster,

	MAX = UseOnlyForICVXFCameraInCluster
};

int32 GDisplayClusterLumenPerView = (uint8)EDisplayClusterLumenPerView::UseOnlyInCluster;
static FAutoConsoleVariableRef CVarDisplayClusterLumenPerView(
	TEXT("DC.LumenPerView"),
	GDisplayClusterLumenPerView,
	TEXT("Separate Lumen scene cache allocated for each View. (Default = 1)\n")
	TEXT("Reduces artifacts where views affect one another, at a cost in GPU memory.\n")
	TEXT(" 0 - Disabled.\n")
	TEXT(" 1 - Use only in Cluster.\n")
	TEXT(" 2 - Use in both Cluster and Preview.\n")
	TEXT(" 3 - Use only in Preview.\n")
	TEXT(" 4 - Use only for Inner Frustum in Cluster.\n"),
	ECVF_RenderThreadSafe
);

int32 GDisplayClusterPreviewEnableViewState = 1;
static FAutoConsoleVariableRef CVarDisplayClusterPreviewEnableViewState(
	TEXT("nDisplay.preview.EnableViewState"),
	GDisplayClusterPreviewEnableViewState,
	TEXT("Enable view state for preview (0 - disable).\n"),
	ECVF_RenderThreadSafe
);

int32 GDisplayClusterPreviewEnableConfiguratorViewState = 0;
static FAutoConsoleVariableRef CVarDisplayClusterPreviewEnableConfiguratorViewState(
	TEXT("nDisplay.preview.EnableConfiguratorViewState"),
	GDisplayClusterPreviewEnableConfiguratorViewState,
	TEXT("Enable view state for preview in Configurator window (0 - disable).\n"),
	ECVF_RenderThreadSafe
);

int32 GDisplayClusterEnableViewStateForLightCard = 1;
static FAutoConsoleVariableRef CVarDisplayClusterEnableViewStateForLightCard(
	TEXT("nDisplay.render.ViewState.LightCard"),
	GDisplayClusterEnableViewStateForLightCard,
	TEXT("Enable view state for LightCard (0 - disable).\n"),
	ECVF_RenderThreadSafe
);

int32 GDisplayClusterEnableViewStateForChromakey = 0;
static FAutoConsoleVariableRef CVarDisplayClusterEnableViewStateForChromakey(
	TEXT("nDisplay.render.ViewState.Chromakey"),
	GDisplayClusterEnableViewStateForChromakey,
	TEXT("Enable view state for Chromakey (0 - disable).\n"),
	ECVF_RenderThreadSafe
);


///////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterViewport
///////////////////////////////////////////////////////////////////////////////////////
void FDisplayClusterViewport::ResetRuntimeParameters(const UDisplayClusterConfigurationViewport* InConfigurationViewport)
{
	// Reset runtim flags from prev frame:
	RenderSettings.BeginUpdateSettings();
	RenderSettingsICVFX.BeginUpdateSettings();
	PostRenderSettings.BeginUpdateSettings();
	VisibilitySettings.BeginUpdateSettings();
	CameraMotionBlur.BeginUpdateSettings();
	CameraDepthOfField.BeginUpdateSettings();

	OverscanRuntimeSettings = FDisplayClusterViewport_OverscanRuntimeSettings();
	CustomFrustumRuntimeSettings = FDisplayClusterViewport_CustomFrustumRuntimeSettings();

	// Obtain viewport media state from external multicast delegates (This viewport can be used by multiple media).
	EDisplayClusterViewportMediaState AllMediaStates = EDisplayClusterViewportMediaState::None;
	IDisplayCluster::Get().GetCallbacks().OnDisplayClusterUpdateViewportMediaState().Broadcast(this, AllMediaStates);

	// Update the media state for the new frame.
	RenderSettings.AssignMediaStates(AllMediaStates);

	// Read general settings from the configuration
	if (InConfigurationViewport && Configuration->IsICVFXEnabled())
	{
		if (const FDisplayClusterConfigurationICVFX_StageSettings* StageSettings = Configuration->GetStageSettings())
		{
			// Update base ICVFX settings of viewport.
			RenderSettingsICVFX.Flags = InConfigurationViewport->GetViewportICVFXFlags(*StageSettings);
		}
	}
}

bool FDisplayClusterViewport::IsInternalViewport() const
{
	// Ignore ICVFX internal resources.
	if (EnumHasAnyFlags(GetRenderSettingsICVFX().RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::InternalResource))
	{
		return true;
	}

	// Ignore internal tile viewports.
	switch (GetRenderSettings().TileSettings.GetType())
	{
	case EDisplayClusterViewportTileType::None:
	case EDisplayClusterViewportTileType::Source:
		break;

	default:
		return true;
	}

	return false;
}

bool FDisplayClusterViewport::IsOuterViewport() const
{
	// Ignore ICVFX internal resources.
	if (EnumHasAnyFlags(GetRenderSettingsICVFX().RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::InternalResource))
	{
		return false;
	}

	// Ignore internal tile viewports.
	switch (GetRenderSettings().TileSettings.GetType())
	{
	case EDisplayClusterViewportTileType::None:
	case EDisplayClusterViewportTileType::Source:
		break;

	default:
		return false;
	}

	return true;
}

bool FDisplayClusterViewport::IsExternalRendering() const
{
	if (PostRenderSettings.Replace.IsEnabled())
	{
		// The viewport is replaced by an external texture.
		return true;
	}

	if (RenderSettings.IsViewportOverridden())
	{
		// Viewport texture is overridden from another viewport.
		return true;
	}

	// UV LightCard viewport use unique whole-cluster texture from LC manager
	if (EnumHasAllFlags(RenderSettingsICVFX.RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::UVLightcard))
	{
		// Use external texture from LightCardManager instead of rendering
		return true;
	}

	return false;
}

bool FDisplayClusterViewport::ShouldDisableViewportResources() const
{
	// Viewport used by media.
	if (RenderSettings.HasAnyMediaStates(EDisplayClusterViewportMediaState::Capture | EDisplayClusterViewportMediaState::Input))
	{
		// Always allocate resources for viewports used by media (including offscreen nodes).
		// Note: this rule takes priority over the ones below.
		return false;
	}

	// Any other viewports on the headless node that do not use a backbuffer media output.
	if (Configuration->IsClusterNodeRenderingOffscreen()
		&& Configuration->ShouldOptimizeOutOffscreenNodeResources()
		&& !Configuration->GetRenderFrameSettings().CurrentNode.bHasBackbufferMediaOutput)
	{
		// Exception: Outer viewports on an Offscreen nodes.
		if (IsOuterViewport())
		{
			// Outer viewports are always used on offscreen nDisplay nodes.
			// They will allocate resources and be rendered even when otherwise unused (e.g. not consumed by media).
			return false;
		}

		// Resources should not be allocated for these viewports.
		return true;
	}

	// An ICVFX camera viewport with no outer viewports assigned is considered invisible.
	if (EnumHasAnyFlags(RenderSettingsICVFX.Flags, EDisplayClusterViewportICVFXFlags::CameraHasNoTargetViewports)
		&& EnumHasAnyFlags(RenderSettingsICVFX.RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::InCamera))
	{
		// Skip an invisible ICVFX Camera viewport.
		return true;
	}

	// Unbound tile viewports
	if (RenderSettings.TileSettings.GetType() == EDisplayClusterViewportTileType::Tile
		&& !RenderSettings.TileSettings.HasAnyTileFlags(EDisplayClusterViewportTileFlags::AllowUnboundRender))
	{
		// Skip unbound tile viewports if not in preview.
		if (!Configuration->IsPreviewRendering())
		{
			return true;
		}
	}

	return false;
}

bool FDisplayClusterViewport::IsRenderEnabled() const
{
	// Render requires resources.
	if (ShouldDisableViewportResources())
	{
		return false;
	}

	if (IsExternalRendering())
	{
		// The viewport uses an external rendering that overrides its RTT.
		// Disable rendering for this viewport.
		return false;
	}

	if (RenderSettings.bSkipRendering)
	{
		// Skip rendering.
		// For example this feature is used when the ICVFX camera uses full-frame chromakey colour,
		// thereby eliminating the ICVFX camera's viewport rendering for optimization.
		return false;
	}

	// Handle tile rendering rules.
	switch(RenderSettings.TileSettings.GetType())
	{
	case EDisplayClusterViewportTileType::Tile:
	case EDisplayClusterViewportTileType::None:
		// When tile rendering is used, only tiles are rendered.
		break;

	default:
		// When using tile rendering, other viewport types should never be rendered.
		return false;
	}

	// Handle media rendering rules.
	if (!IsRenderEnabledByMedia())
	{
		// rendering of this viewport is not allowed by media.
		return false;
	}

	return true;
}

bool FDisplayClusterViewport::IsRenderEnabledByMedia() const
{
	// Do not render the viewport if it is used as a media input.
	if (RenderSettings.HasAnyMediaStates(EDisplayClusterViewportMediaState::Input))
	{
		// This viewport is not rendered but gets the image from the media.
		// (media input replaces rendering.)
		return false;
	}

	return true;
}

bool FDisplayClusterViewport::IsUsedByMedia() const
{
	return RenderSettings.HasAnyMediaStates(EDisplayClusterViewportMediaState::Input | EDisplayClusterViewportMediaState::Capture);
}

bool FDisplayClusterViewport::ShouldApplyMaxTextureConstraints() const
{
	if (IsUsedByMedia())
	{
		// Do not apply this restriction to a viewport that uses media.
		return false;
	}

	return true;
}

bool FDisplayClusterViewport::CanSplitIntoTiles() const
{
	if (!RenderSettings.bEnable || RenderSettings.bSkipRendering || RenderSettings.bFreezeRendering)
	{
		// When this viewport is not rendering, ignore tile splitting.
		return false;
	}

	// Ignore internal tile viewports.
	switch(RenderSettings.TileSettings.GetType())
	{
		case EDisplayClusterViewportTileType::Tile:
		case EDisplayClusterViewportTileType::UnusedTile:
			return false;

		default:
			break;
	}

	// Ignore viewports that have a link to another.
	if (RenderSettings.IsViewportHasParent())
	{
		return false;
	}

	// Ignore viewports that use an external source instead of rendering.
	if (IsExternalRendering())
	{
		return false;
	}

	// Ignore viewports that used by media
	if (IsUsedByMedia())
	{
		return false;
	}

	// nDisplay tile splitting is ignored when MRP is active; MRP has its own tile rendering system.
	if (Configuration->IsMoviePipelineRendering())
	{
		return false;
	}

	return true;
}

bool FDisplayClusterViewport::IsResourceUsedImpl(const EDisplayClusterViewportResource InResourceType) const
{
	if (Configuration->IsMoviePipelineRendering())
	{
		// In MRP rendering mode, MRP manages all standard viewport resources itself.
		// The only resources allocated here are the two warp-blend temporaries:
		//   InputShaderResources          - warp shader input  (RTT copy-in)
		//   AdditionalTargetableResources - warp shader output (RTT copy-out)
		// All other resource types are suppressed.
		if (ProjectionPolicy.IsValid() && ProjectionPolicy->IsWarpBlendSupported(this))
		{
			switch (InResourceType)
			{
			case EDisplayClusterViewportResource::InputShaderResources:
				return true;

			case EDisplayClusterViewportResource::AdditionalTargetableResources:
				if (ProjectionPolicy->ShouldUseAdditionalTargetableResource(this))
				{
					return true;
				}
				break;

			default:
				break;
			};
		}

		return false;
	}

	if (RenderSettings.TileSettings.GetType() == EDisplayClusterViewportTileType::Tile)
	{
		// The tiled viewport does not use internal resources.
		// Since rendering results are copied between RTTs.
		if (InResourceType != EDisplayClusterViewportResource::RenderTargets)
		{
			return false;
		}
	}

	if (InResourceType == EDisplayClusterViewportResource::RenderTargets)
	{
		// When external rendering is used, this means that the RTT is a reference, not a texture.
		if (IsExternalRendering())
		{
			return false;
		}
	}

	if (RenderSettings.IsViewportOverridden())
	{
		switch (RenderSettings.GetViewportOverrideMode())
		{
		// Override all resources
		case EDisplayClusterViewportOverrideMode::All:
			switch (InResourceType)
			{
			case EDisplayClusterViewportResource::RenderTargets:
			case EDisplayClusterViewportResource::InputShaderResources:
			case EDisplayClusterViewportResource::MipsShaderResources:
			case EDisplayClusterViewportResource::AdditionalTargetableResources:
				return false;

			default:
				break;
			}
			break;

		// Override all internal resources except the texture for warpblend
		case EDisplayClusterViewportOverrideMode::InternalViewportResources:
			switch (InResourceType)
			{
			case EDisplayClusterViewportResource::RenderTargets:
			case EDisplayClusterViewportResource::InputShaderResources:
			case EDisplayClusterViewportResource::MipsShaderResources:
				return false;

			default:
				break;
			}
			break;

		case EDisplayClusterViewportOverrideMode::InternalRTT:
			// When all resources in a viewport are overridden from another viewport.
			if (InResourceType == EDisplayClusterViewportResource::RenderTargets)
			{
				return false;
			}
			break;

		default:
			break;
		}
	}

	// These are exceptions to the rules, for each case we must return true or false:
	switch (InResourceType)
	{
	case EDisplayClusterViewportResource::AdditionalTargetableResources:
		// Support additional resource demands from the projection policy.
		if (ProjectionPolicy.IsValid() && ProjectionPolicy->ShouldUseAdditionalTargetableResource(this))
		{
			return true;
		}

		return false;

		// Return true if this viewport requires to use any of output resources (OutputPreviewTargetableResources or OutputFrameTargetableResources)
	case EDisplayClusterViewportResource::OutputPreviewTargetableResources:
	case EDisplayClusterViewportResource::OutputFrameTargetableResources:
		// Only if this viewport is enabled and visible on the final frame.
		return RenderSettings.bEnable && RenderSettings.bVisible;

	case EDisplayClusterViewportResource::AdditionalFrameTargetableResources:
		// Only if this viewport is enabled and visible on the final frame.
		if (RenderSettings.bEnable && RenderSettings.bVisible)
		{
			// Viewport Remap requires AdditionalFrameTargetableResources.
			// [warp] -> AdditionalFrameTargetableResource -> [OutputRemap] -> OutputFrameTargetableResource
			if (ViewportRemap.IsUsed())
			{
				return true;
			}

			// Postprocess manager may request an AdditionalFrameTargetable resource
			if (Configuration->GetRenderFrameSettings().bUseAdditionalFrameTargetableForPostprocess)
			{
				return true;
			}
		}
		
		return false;

	default:
		break;
	}

	return true;
}

bool FDisplayClusterViewport::IsResourceUsed(const EDisplayClusterViewportResource InResourceType) const
{
	if (RenderSettings.bSkipRendering)
	{
		// When rendering is skipped, the resources isn't used
		// For example this feature is used when the ICVFX camera uses full-frame chromakey colour,
		// thereby eliminating the ICVFX camera's viewport rendering resources for optimization.
		return false;
	}

	// Resources may be disabled by other rules: media, offscreen nodes, etc.
	if (ShouldDisableViewportResources())
	{
		return false;
	}

	// Any other viewports on the headless node that do not use a backbuffer media output.
	if (Configuration->IsClusterNodeRenderingOffscreen()
		&& Configuration->ShouldOptimizeOutOffscreenNodeResources()
	&& !Configuration->GetRenderFrameSettings().CurrentNode.bHasBackbufferMediaOutput)
	{
		// Backbuffer not used: skip related resources.
		switch (InResourceType)
		{
		case EDisplayClusterViewportResource::OutputFrameTargetableResources:
		case EDisplayClusterViewportResource::AdditionalFrameTargetableResources:

			// Exception: Outer viewports on an Offscreen nodes.
			if (IsOuterViewport())
			{
				// Outer viewports are always used on offscreen nDisplay nodes.
				// They will allocate resources and be rendered even when otherwise unused (e.g. not consumed by media).
				break;
			}

			// Skip FrameTargetable RTTs (final composite copied to backbuffer).
			return false;

		default:
			break;
		}
	}

	if (!IsResourceUsedImpl(InResourceType))
	{
		return false;
	}

	return true;
};

bool FDisplayClusterViewport::ShouldUseFullSizeFrameTargetableResource() const
{
	if (Configuration->GetRenderFrameSettings().bUseFullSizeFrameTargetableForPostprocess)
	{
		return true;
	}

	if (ViewportRemap.IsUsed())
	{
		return true;
	}

	return false;
}

bool FDisplayClusterViewport::ShouldFreezeRender() const
{
	// Never freeze when the DCRA is exclusively locked.
	if (Configuration->IsRootActorExclusivelyLocked())
	{
		return false;
	}

	// Freeze preview rendering for all viewport
	if (Configuration->GetRenderFrameSettings().IsPreviewFreezeRender())
	{
		return true;
	}

	// ICVFX: Freeze only some viewports
	if (const FDisplayClusterConfigurationICVFX_StageSettings* StageSettings = Configuration->GetStageSettings())
	{	
		if (StageSettings->bFreezeRenderOuterViewports)
		{
			// Freeze rendering for outer viewports
			if (!IsInternalViewport())
			{
				return true;
			}

			// Enable\Disable freeze rendering for lightcards when outer viewports rendering also freezed. This will impact performance.
			if (!StageSettings->Lightcard.bIgnoreOuterViewportsFreezingForLightcards
				&& EnumHasAnyFlags(RenderSettingsICVFX.RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::Lightcard | EDisplayClusterViewportRuntimeICVFXFlags::UVLightcard))
			{
				return true;
			}
		}
	}
	
	return false;
}

bool FDisplayClusterViewport::ShouldUseViewStates() const
{
	// RenderMode rules
	switch (Configuration->GetRenderFrameSettings().RenderMode)
	{

	// MRQ has its own ViewState management.
	case EDisplayClusterRenderFrameMode::MRQ_Mono:
	case EDisplayClusterRenderFrameMode::MRQ_Stereo:
		return false;

	// MRG has its own ViewState management.
	case EDisplayClusterRenderFrameMode::MRG_Mono:
	case EDisplayClusterRenderFrameMode::MRG_Stereo:
		return false;

	// Preview-In-Scene
	case EDisplayClusterRenderFrameMode::PreviewProxyHitInScene:
	case EDisplayClusterRenderFrameMode::PreviewInScene:

		// CVar 'nDisplay.preview.EnableViewState' disables ViewState for preview in the scene.
		if (GDisplayClusterPreviewEnableViewState == 0)
		{
			return false;
		}

		// CVar 'nDisplay.preview.EnableConfiguratorViewState' disables ViewState for preview inside the Configurator.
		if (GDisplayClusterPreviewEnableConfiguratorViewState == 0 && Configuration->IsCurrentWorldHasAnyType(EWorldType::EditorPreview))
		{
			return false;
		}

		break;

	default:
		break;
	}

	// ICVFX Rules

	// CVar 'nDisplay.render.ViewState.Chromakey' disables ViewState for Chromakey.
	if (!GDisplayClusterEnableViewStateForChromakey
	&& EnumHasAnyFlags(RenderSettingsICVFX.RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::Chromakey))
	{
		return false;
	}

	// CVar 'nDisplay.render.ViewState.LightCard' disables ViewState for LightCard.
	if (!GDisplayClusterEnableViewStateForLightCard
		&& EnumHasAnyFlags(RenderSettingsICVFX.RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::Lightcard | EDisplayClusterViewportRuntimeICVFXFlags::UVLightcard))
	{
		return false;
	}

	return true;
}

bool FDisplayClusterViewport::IsOpenColorIOEquals(const FDisplayClusterViewport& InViewport) const
{
	bool bEnabledOCIO_1 = OpenColorIO.IsValid();
	bool bEnabledOCIO_2 = InViewport.OpenColorIO.IsValid();

	if (bEnabledOCIO_1 == bEnabledOCIO_2)
	{
		if (!bEnabledOCIO_1)
		{
			// Both OCIO disabled
			return true;
		}

		if (OpenColorIO->IsConversionSettingsEqual(InViewport.OpenColorIO->GetConversionSettings()))
		{
			return true;
		}
	}

	return false;
}

/**
* The viewport priority values
* A lower priority value for a viewport means that this viewport will be the first in the list of viewports.
* The order in this list is used to process viewports one after the other.
* This is important when the viewports are linked to each other.
*/
enum class EDisplayClusterViewportPriority : uint8
{
	None = 0,

	// This viewport does not use tile rendering.
	TileDisable = (1 << 0),

	// This tile source viewport should be configured before tiles.
	TileSource = (1 << 0),

	// The tile viewport setup after the tile.
	Tile = (1 << 1),

	// The linked viewport should be right after the parent viewports because it uses data from them.
	Linked = (1 << 2),

	// Overridden viewports are not rendered but depend on their source anyway, so they are processed at the end.
	Overriden = (1 << 3),
};

uint8 FDisplayClusterViewport::GetPriority() const
{
	uint8 OutOrder = 0;

	// Tile rendering requires a special viewport processing order for the game thread.
	switch (RenderSettings.TileSettings.GetType())
	{
	case EDisplayClusterViewportTileType::Source:
		OutOrder += (uint8)EDisplayClusterViewportPriority::TileSource;
		break;
	case EDisplayClusterViewportTileType::Tile:
	case EDisplayClusterViewportTileType::UnusedTile:
		OutOrder += (uint8)EDisplayClusterViewportPriority::Tile;
		break;

	case EDisplayClusterViewportTileType::None:
		OutOrder += (uint8)EDisplayClusterViewportPriority::TileDisable;
		break;

	default:
		break;
	}

	if (RenderSettings.IsViewportHasParent())
	{
		OutOrder += (uint8)EDisplayClusterViewportPriority::Linked;
	}

	if (RenderSettings.IsViewportOverridden())
	{
		OutOrder += (uint8)EDisplayClusterViewportPriority::Overriden;
	}

	return OutOrder;
}

bool FDisplayClusterViewport::ShouldUseLumenPerView() const
{
	switch (RenderSettings.CaptureMode)
	{
	case EDisplayClusterViewportCaptureMode::Chromakey:
	case EDisplayClusterViewportCaptureMode::Lightcard:
		// This type of viewport does not use Lumen
		return false;

	default:
		break;
	}

	const EDisplayClusterLumenPerView LumenPerViewMode = (EDisplayClusterLumenPerView)FMath::Clamp(
		GDisplayClusterLumenPerView, 0, (uint8)EDisplayClusterLumenPerView::MAX);

	switch (LumenPerViewMode)
	{
	// Use only in Cluster.
	case EDisplayClusterLumenPerView::UseOnlyInCluster:
		return !Configuration->IsPreviewRendering();

	// Use only in Preview.
	case EDisplayClusterLumenPerView::UseOnlyInPreview:
		return Configuration->IsPreviewRendering();

	// Use only for Inner Frustum in Cluster.
	case EDisplayClusterLumenPerView::UseOnlyForICVXFCameraInCluster:
		if (Configuration->IsPreviewRendering())
		{
			// Not a cluster
			return false;
		}

		// Full-frame In-Camera viewport
		if (EnumHasAnyFlags(RenderSettingsICVFX.RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::InCamera))
		{
			return true;
		}

		// Tile In-Camera viewport
		if (RenderSettings.TileSettings.GetType() == EDisplayClusterViewportTileType::Tile)
		{
			if (FDisplayClusterViewportManager* ViewportManager = Configuration->GetViewportManagerImpl())
			{
				TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe> SourceViewport =
					ViewportManager->ImplFindViewport(RenderSettings.TileSettings.GetSourceViewportId());

				if (SourceViewport.IsValid())
				{
					if (EnumHasAnyFlags(SourceViewport->RenderSettingsICVFX.RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::InCamera))
					{
						return true;
					}
				}
			}
		}
		
		// Other viewports
		return false;

	// Use in both Cluster and Preview.
	case EDisplayClusterLumenPerView::UseEverywhere:
		return true;

	default:
		break;
	}

	return false;
}
