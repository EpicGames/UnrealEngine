// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Viewport/DisplayClusterViewport.h"
#include "Render/Viewport/DisplayClusterViewportManager.h"

#include "Render/Projection/IDisplayClusterProjectionPolicy.h"
#include "Math/DisplayClusterProjectionMath.h"

#include "DisplayClusterRootActor.h"
#include "Components/DisplayClusterCameraComponent.h"
#include "Components/DisplayClusterICVFXCameraComponent.h"
#include "CineCameraComponent.h"

#include "SceneView.h"

#include "Misc/DisplayClusterLog.h"

#include "GameFramework/PlayerController.h"
#include "Camera/CameraComponent.h"

#define DISPLAYCLUSTER_VALIDATE_PROJECTION_ENCODING 0

/**
 * Controls two per-frame view caches to avoid redundant recalculation within a frame:
 *   FDisplayClusterViewport::CurrentFrameData.CachedCameraViewInfo  - camera view info resolved from the view point component
 *   FDisplayClusterViewport_Context::CachedViewPoint                 - nDisplay viewport calculated view (projection policy warpblend, etc)
 */
int32 GDisplayClusterEnableViewInfoCaches = 1;
static FAutoConsoleVariableRef CVarDisplayClusterEnableViewInfoCaches(
	TEXT("nDisplay.render.ViewPoint.EnableCache"),
	GDisplayClusterEnableViewInfoCaches,
	TEXT("Enable per-frame caching of view point and camera view info (0=disabled, 1=enabled).\n"),
	ECVF_RenderThreadSafe
);

namespace UE::DisplayClusterViewport::ViewPoint
{
	/** Optionally get custom NearClippingPlane from a camera component. Supports ICVFXCameraComponent by resolving
	 *  it to the underlying CineCameraComponent before reading the override value.
	 * @param InCameraComponent          - camera component
	 * @param OutCustomNearClippingPlane - receives the custom NCP value; set to -1 if not overridden; unchanged if nullptr
	 */
	static void GetCustomNCPFromCameraComponent(UCameraComponent* InCameraComponent, float* OutCustomNearClippingPlane)
	{
		// Get custom NCP from CineCamera component:
		if (OutCustomNearClippingPlane)
		{
			*OutCustomNearClippingPlane = -1;

			// Get settings from this cinecamera component
			if (UCineCameraComponent* CineCameraComponent = Cast<UCineCameraComponent>(InCameraComponent))
			{
				// Supports ICVFX camera component as input
				if (UDisplayClusterICVFXCameraComponent* ICVFXCameraComponent = Cast<UDisplayClusterICVFXCameraComponent>(CineCameraComponent))
				{
					// Getting settings from the actual CineCamera component
					CineCameraComponent = ICVFXCameraComponent->GetActualCineCameraComponent();
				}

				if (CineCameraComponent && CineCameraComponent->bOverride_CustomNearClippingPlane)
				{
					*OutCustomNearClippingPlane = CineCameraComponent->CustomNearClippingPlane;
				}
			}
		}
	}

	/** Optionally get custom NearClippingPlane from the first CameraComponent found on the actor.
	 * @param InActor                   - actor to search for a camera component
	 * @param OutCustomNearClippingPlane - receives the custom NCP value; set to -1 if not overridden; unchanged if nullptr
	 */
	static void GetCustomNCPFromActor(AActor* InActor, float* OutCustomNearClippingPlane)
	{
		if (OutCustomNearClippingPlane)
		{
			*OutCustomNearClippingPlane = -1.0f;

			TArray<UCameraComponent*> CameraComponents;
			InActor->GetComponents<UCameraComponent>(CameraComponents, true);

			if (CameraComponents.IsValidIndex(0))
			{
				GetCustomNCPFromCameraComponent(CameraComponents[0], OutCustomNearClippingPlane);
			}
		}
	}
};

///////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterViewport
///////////////////////////////////////////////////////////////////////////////////////
bool IDisplayClusterViewport::GetCameraComponentView(UCameraComponent* InCameraComponent, const float InDeltaTime, const bool bUseCameraPostprocess, FMinimalViewInfo& InOutViewInfo, float* OutCustomNearClippingPlane)
{
	if (!InCameraComponent)
	{
		// Required camera component
		return false;
	}

	InCameraComponent->GetCameraView(InDeltaTime, InOutViewInfo);

	if(!bUseCameraPostprocess)
	{
		InOutViewInfo.PostProcessSettings = FPostProcessSettings();
		InOutViewInfo.PostProcessBlendWeight = 0.0f;
	}

	// Optionally get custom NCP from camera component
	UE::DisplayClusterViewport::ViewPoint::GetCustomNCPFromCameraComponent(InCameraComponent, OutCustomNearClippingPlane);

	return true;
}

bool IDisplayClusterViewport::GetPlayerCameraView(UWorld* InWorld, const bool bUseCameraPostprocess, FMinimalViewInfo& InOutViewInfo, float* OutCustomNearClippingPlane)
{
	if (!InWorld)
	{
		return false;
	}

	APlayerController* const LocalPlayerController = InWorld->GetFirstPlayerController();
	if (!LocalPlayerController)
	{
		return false;
	}

	const TObjectPtr<APlayerCameraManager> LocalPlayerCameraManager = LocalPlayerController->PlayerCameraManager;
	if (LocalPlayerCameraManager == nullptr)
	{
		return false;
	}

	// Start from the cached view to pick up postprocess settings, aspect ratio, etc.
	InOutViewInfo = LocalPlayerCameraManager->GetCameraCacheView();

	if (!bUseCameraPostprocess)
	{
		InOutViewInfo.PostProcessSettings = FPostProcessSettings();
		InOutViewInfo.PostProcessBlendWeight = 0.0f;
	}

	// Overwrite FOV and transform with live values.
	InOutViewInfo.FOV = LocalPlayerCameraManager->GetFOVAngle();
	LocalPlayerCameraManager->GetCameraViewPoint(/*out*/ InOutViewInfo.Location, /*out*/ InOutViewInfo.Rotation);

	if (OutCustomNearClippingPlane)
	{
		// -1.0f means "no custom NCP"; overwritten below if the view actor provides one
		*OutCustomNearClippingPlane = -1.0f;

		if (AActor* ViewActor = LocalPlayerController->GetViewTarget())
		{
			UE::DisplayClusterViewport::ViewPoint::GetCustomNCPFromActor(ViewActor, OutCustomNearClippingPlane);
		}
	}

	return true;
}

///////////////////////////////////////////////////////////////////////////////////////
//          FDisplayClusterViewport
///////////////////////////////////////////////////////////////////////////////////////
UDisplayClusterCameraComponent* FDisplayClusterViewport::GetViewPointCameraComponent(const EDisplayClusterRootActorType InRootActorType) const
{
	ADisplayClusterRootActor* RootActor = Configuration->GetRootActor(InRootActorType);
	if (!RootActor)
	{
		if (CanShowLogMsgOnce(EDisplayClusterViewportShowLogMsgOnce::GetViewPointCameraComponent_NoRootActorFound))
		{
			UE_LOGF(LogDisplayClusterViewport, Warning, "Viewport '%ls' has no root actor found", *GetId());
		}

		return nullptr;
	}

	ResetShowLogMsgOnce(EDisplayClusterViewportShowLogMsgOnce::GetViewPointCameraComponent_NoRootActorFound);

	if (!ProjectionPolicy.IsValid())
	{
		// ignore viewports with uninitialized prj policy
		return nullptr;
	}

	if (!Configuration->IsSceneOpened())
	{
		return nullptr;
	}

	// Get camera ID assigned to the viewport
	const FString& CameraId = GetRenderSettings().CameraId;
	if (CameraId.Len() > 0)
	{
		if (CanShowLogMsgOnce(EDisplayClusterViewportShowLogMsgOnce::GetViewPointCameraComponent_HasAssignedViewPoint))
		{
			UE_LOGF(LogDisplayClusterViewport, Verbose, "Viewport '%ls' has assigned ViewPoint '%ls'", *GetId(), *CameraId);
		}
	}
	else
	{
		ResetShowLogMsgOnce(EDisplayClusterViewportShowLogMsgOnce::GetViewPointCameraComponent_HasAssignedViewPoint);
	}

	// Get camera component assigned to the viewport (or default camera if nothing assigned)
	if (UDisplayClusterCameraComponent* const ViewCamera = (CameraId.IsEmpty() ?
		RootActor->GetDefaultCamera() :
		RootActor->GetComponentByName<UDisplayClusterCameraComponent>(CameraId)))
	{
		ResetShowLogMsgOnce(EDisplayClusterViewportShowLogMsgOnce::GetViewPointCameraComponent);

		return ViewCamera;
	}

	if (CanShowLogMsgOnce(EDisplayClusterViewportShowLogMsgOnce::GetViewPointCameraComponent_NotFound))
	{
		UE_LOGF(LogDisplayClusterViewport, Warning, "ViewPoint '%ls' is not found for viewport '%ls'. The default viewpoint will be used.", *CameraId, *GetId());
	}

	return CameraId.IsEmpty() ? nullptr : RootActor->GetDefaultCamera();
}

bool FDisplayClusterViewport::GetCameraViewPoint(FMinimalViewInfo& InOutViewInfo)
{
	// Use cached value if possible
	if (GDisplayClusterEnableViewInfoCaches && CurrentFrameData.CachedCameraViewInfo.IsSet())
	{
		InOutViewInfo = CurrentFrameData.CachedCameraViewInfo.GetValue();
		return true;
	}

	// Get DCRA ViewPoint component (required).
	UDisplayClusterCameraComponent* ViewPointComponent = GetViewPointCameraComponent(EDisplayClusterRootActorType::Scene);
	if (!ViewPointComponent)
	{
		return false;
	}

	// Get ViewInfo from the ViewPoint component
	ViewPointComponent->GetDesiredView(*Configuration, InOutViewInfo, &CustomNearClippingPlane);

	// Apply projection policy overrides (policy type 'camera', etc)
	if (ProjectionPolicy.IsValid())
	{
		ProjectionPolicy->SetupProjectionViewPoint(this, Configuration->GetRootActorWorldDeltaSeconds(), InOutViewInfo, &CustomNearClippingPlane);
	}

	// Cache new value
	CurrentFrameData.CachedCameraViewInfo = InOutViewInfo;

	return true;
}

bool FDisplayClusterViewport::SetupViewPoint(const uint32 InContextNum, FMinimalViewInfo& InOutViewInfo)
{
	// Use cached value if possible
	if (GDisplayClusterEnableViewInfoCaches && Contexts.IsValidIndex(InContextNum) && Contexts[InContextNum].CachedViewPoint.IsSet())
	{
		InOutViewInfo = Contexts[InContextNum].CachedViewPoint.GetValue();

		return true;
	}

	if (!GetCameraViewPoint(InOutViewInfo))
	{
		return false;
	}

	// nDisplay always uses its own overscan - ignore camera overscan settings.
	InOutViewInfo.ClearOverscan();

	// Override near clipping plane from nDisplay viewport.
	InOutViewInfo.PerspectiveNearClipPlane = GetClippingPlanes().X;

	if (!Contexts.IsValidIndex(InContextNum))
	{
		return false;
	}

	const FDisplayClusterViewport_Context& SrcContext = Contexts[InContextNum];

	// Calculate nDisplay ViewInfo
	{
		// Calculate View data for the current viewport context.
		if (!CalculateView(InContextNum, InOutViewInfo.Location, InOutViewInfo.Rotation, 0.f))
		{
			return false;
		}

		// SetupViewPoint() should return Origin for ULocalPlayer::GetProjectionData()
		InOutViewInfo.Location = SrcContext.ViewData.RenderCameraLocation;
	}

	// Override FMinimalViewInfo fields with nDisplay-specific values.
	{
		// Encode the nDisplay projection matrix into FMinimalViewInfo so that
		// CalculateProjectionMatrix() reproduces it exactly.
		// Note: FMinimalViewInfo carries no far-plane distance,
		//       so the recovered matrix always uses an infinite far plane.
		{
			const FMatrix& P = SrcContext.GetRenderProjectionMatrix();
			const float mx = P.M[0][0];   // horizontal scale
			const float my = P.M[1][1];   // vertical scale

			// Recover horizontal FOV and aspect ratio from the projection scale factors.
			InOutViewInfo.FOV         = 2.f * FMath::RadiansToDegrees(FMath::Atan(1.f / mx));
			InOutViewInfo.AspectRatio = my / mx;

			// Force CalculateProjectionMatrix() which uses FOV/AspectRatio directly,
			// bypassing the viewport pixel dimensions used by the unconstrained path.
			InOutViewInfo.bConstrainAspectRatio = true;
			InOutViewInfo.AspectRatioAxisConstraint.Reset();

			// Recover the off-center shift from the projection matrix.
			InOutViewInfo.OffCenterProjectionOffset.X = -P.M[2][0];
			InOutViewInfo.OffCenterProjectionOffset.Y = -P.M[2][1];
		}

		// Temporary validation: encoding the nDisplay projection matrix into FMinimalViewInfo parameters
		// (FOV, AspectRatio, OffCenterProjectionOffset) and recovering it via CalculateProjectionMatrix()
		// is a new feature. This check verifies the round-trip math is correct and should be removed
		// once the encoding is well-tested.
#if DISPLAYCLUSTER_VALIDATE_PROJECTION_ENCODING
		FMatrix ProjectionMatrix = InOutViewInfo.CalculateProjectionMatrix();
		if (!FDisplayClusterProjectionMath::MatricesNearlyEqual(
			ProjectionMatrix, SrcContext.GetRenderProjectionMatrix()))
		{
			UE_LOGF(LogDisplayClusterViewport, Warning,
				"nDisplay projection matrix encoding mismatch for viewport '%ls'.\n  Computed: %ls\n  Expected: %ls",
				*GetId(),
				*ProjectionMatrix.ToString(),
				*SrcContext.GetRenderProjectionMatrix().ToString()
			);
		}
#endif /** DISPLAYCLUSTER_VALIDATE_PROJECTION_ENCODING */

		// Optionally set the previous-frame view transform for motion blur.
		if (PreviousFrameData.Contexts.IsValidIndex(InContextNum))
		{
			const FDisplayClusterViewport_Context& PrevFrameContext = PreviousFrameData.Contexts[InContextNum];
			InOutViewInfo.PreviousViewTransform = FTransform(
				PrevFrameContext.ViewData.RenderRotation,
				PrevFrameContext.ViewData.RenderCameraLocation);
		}
	}

	// Save additional data at this point: DoF FocalLength
	if (InOutViewInfo.PostProcessSettings.DepthOfFieldFocalDistance > 0.0f)
	{
		// Convert FOV to focal length,
		// 
		// fov = 2 * atan(d/(2*f))
		// where,
		//   d = sensor dimension (APS-C 24.576 mm)
		//   f = focal length
		// 
		// f = 0.5 * d * (1/tan(fov/2))
		const FMatrix ProjectionMatrix = InOutViewInfo.CalculateProjectionMatrix();

		FDisplayClusterViewport_Context& DestContext = Contexts[InContextNum];

		DestContext.DepthOfField.SensorFocalLength = 0.5f * InOutViewInfo.PostProcessSettings.DepthOfFieldSensorWidth * ProjectionMatrix.M[0][0];

		// Save actual squeeze factor value
		DestContext.DepthOfField.SqueezeFactor = FMath::Clamp(InOutViewInfo.PostProcessSettings.DepthOfFieldSqueezeFactor, 1.0f, 2.0f);
	}

	// Update cache
	Contexts[InContextNum].CachedViewPoint = InOutViewInfo;

	return true;
}

bool FDisplayClusterViewport::PreCalculateViewData(const uint32 InContextNum, const FVector& InViewLocation, const FRotator& InViewRotation, const float WorldToMeters)
{
	check(Contexts.IsValidIndex(InContextNum));

	FDisplayClusterViewport_Context& OutContext = Contexts[InContextNum];

	// Mark the data as computed to make sure we only compute once per frame.
	OutContext.bValidData = false;
	OutContext.bCalculated = true;

	// RootActor2World transform
	if (const ADisplayClusterRootActor* SceneRootActor = GetConfiguration().GetRootActor(EDisplayClusterRootActorType::Scene))
	{
		OutContext.RootActorToWorld = SceneRootActor->GetTransform();
	}
	else
	{
		// Exit: missed required actor.
		return false;
	}

	if (!ProjectionPolicy)
	{
		return false;
	}

	// Origin2World transform
	if (USceneComponent* const OriginComponent = ProjectionPolicy->GetOriginComponent())
	{
		// Get Origin component transform
		OutContext.OriginToWorld = OriginComponent->GetComponentTransform();
	}

	// Geometry units
	OutContext.WorldToMeters = WorldToMeters;
	OutContext.GeometryToMeters = ProjectionPolicy->GetGeometryToMeters(this, WorldToMeters);

	// Clipping planes
	const FVector2D ClipingPlanes = GetClippingPlanes();
	OutContext.ProjectionData.ZNear = ClipingPlanes.X;
	OutContext.ProjectionData.ZFar  = ClipingPlanes.Y;

	// Scale units from Geometry to UE World
	const float ScaleGeometryToWorld = OutContext.WorldToMeters / OutContext.GeometryToMeters;

	// Also add origin location offset
	const FVector OriginLocationOffsetInGeometryUnits = ProjectionPolicy->GetOriginLocationOffsetInGeometryUnits(this);
	OutContext.OriginToWorld.AddToTranslation(OriginLocationOffsetInGeometryUnits * ScaleGeometryToWorld);

	return true;
}

float FDisplayClusterViewport::GetStereoEyeOffsetDistance(const uint32 InContextNum)
{
	float StereoEyeOffsetDistance = 0.f;

	if (UDisplayClusterCameraComponent* ConfigurationCameraComponent = GetViewPointCameraComponent(EDisplayClusterRootActorType::Configuration))
	{
		enum EDisplayClusterEyeType : int32
		{
			StereoLeft = 0,
			Mono = 1,
			StereoRight = 2,
			COUNT
		};

		// Calculate eye offset considering the world scale
		const float CfgEyeDist = ConfigurationCameraComponent->GetInterpupillaryDistance();
		const float EyeOffset = CfgEyeDist / 2.f;
		const float EyeOffsetValues[] = { -EyeOffset, 0.f, EyeOffset };

		// Decode current eye type
		// This function should work correctly even if the viewport context data is not currently initialized.
		const int32 ViewPerViewportAmount = Configuration->GetRenderFrameSettings().GetViewPerViewportAmount();
		const EDisplayClusterEyeType EyeType = (ViewPerViewportAmount < 2)
			? EDisplayClusterEyeType::Mono
			: (InContextNum == 0) ? EDisplayClusterEyeType::StereoLeft : EDisplayClusterEyeType::StereoRight;

		float PassOffset = 0.f;

		if (EyeType == EDisplayClusterEyeType::Mono)
		{
			// For monoscopic camera let's check if the "force offset" feature is used
			// * Force left (-1) ==> 0 left eye
			// * Force right (1) ==> 2 right eye
			// * Default (0) ==> 1 mono
			const EDisplayClusterEyeStereoOffset CfgEyeOffset = ConfigurationCameraComponent->GetStereoOffset();
			const int32 EyeOffsetIdx =
				(CfgEyeOffset == EDisplayClusterEyeStereoOffset::None ? 0 :
					(CfgEyeOffset == EDisplayClusterEyeStereoOffset::Left ? -1 : 1));

			PassOffset = EyeOffsetValues[EyeOffsetIdx + 1];
			// Eye swap is not available for monoscopic so just save the value
			StereoEyeOffsetDistance = PassOffset;
		}
		else
		{
			// For stereo camera we can only swap eyes if required (no "force offset" allowed)
			PassOffset = EyeOffsetValues[EyeType];

			// Apply eye swap
			const bool  CfgEyeSwap = ConfigurationCameraComponent->GetSwapEyes();
			StereoEyeOffsetDistance = (CfgEyeSwap ? -PassOffset : PassOffset);
		}
	}

	return StereoEyeOffsetDistance;
}