// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Viewport/DisplayClusterViewport_VisibilitySettings.h"
#include "Render/Viewport/DisplayClusterViewportManager.h"
#include "Render/Viewport/DisplayClusterViewport.h"
#include "Render/Viewport/Configuration/DisplayClusterViewportConfigurationHelpers_Visibility.h"

#include "EngineUtils.h"
#include "SceneView.h"

void FDisplayClusterViewport_VisibilitySettings::SetupSceneView(const FDisplayClusterViewport& Owner, FSceneView& InOutView) const
{
	const TSet<FPrimitiveComponentId>& RootActorHidePrimitivesList = Owner.Configuration->GetRootActorHidePrimitivesList();

	switch (VisibilityMode)
	{
	case EDisplayClusterViewport_VisibilityMode::ShowOnly:
	{
		InOutView.ShowOnlyPrimitives.Emplace();
		for (const FPrimitiveComponentId& ComponentId : ComponentsList)
		{
			// Except hidden components
			if (RootActorHidePrimitivesList.Contains(ComponentId))
			{
				continue;
			}

			// Except components from the exclude list
			if (ExcludedComponents.Contains(ComponentId))
			{
				continue;
			}

			InOutView.ShowOnlyPrimitives->Add(ComponentId);
		}

		return;
	}
	case EDisplayClusterViewport_VisibilityMode::Hide:
	{
		// If ExcludedComponents is used, filter ComponentsList.
		if (!ExcludedComponents.IsEmpty())
		{
			for (const FPrimitiveComponentId& ComponentId : ComponentsList)
			{
				// Except components from the exclude list
				if (ExcludedComponents.Contains(ComponentId))
				{
					continue;
				}

				InOutView.HiddenPrimitives.Add(ComponentId);
			}

			break;
		}

		InOutView.HiddenPrimitives.Append(ComponentsList);
		break;
	}

	default:
		break;
	}

	// Also hide components from root actor 
	InOutView.HiddenPrimitives.Append(RootActorHidePrimitivesList);
}
