// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "SceneTypes.h"
#include "ActorLayerUtilities.h"

class FDisplayClusterViewport;
class FSceneView;
class UWorld;

enum class EDisplayClusterViewport_VisibilityMode : uint8
{
	None,
	ShowOnly,
	Hide
};

// GameThread-only data
class FDisplayClusterViewport_VisibilitySettings
{
public:
	virtual ~FDisplayClusterViewport_VisibilitySettings() = default;

public:
	// Reset actor layers visibility rules
	void BeginUpdateSettings()
	{
		VisibilityMode = EDisplayClusterViewport_VisibilityMode::None;
		ComponentsList.Empty();
		ExcludedComponents.Empty();
	}

	/** Return components list. */
	const TSet<FPrimitiveComponentId>& GetComponentsList() const
	{
		return ComponentsList;
	}

	/** Sets the visibility mode and component list. */
	void SetVisibilityModeAndComponentsList(EDisplayClusterViewport_VisibilityMode InVisibilityMode, const TSet<FPrimitiveComponentId>& InComponentsList)
	{

		VisibilityMode = InVisibilityMode;
		ComponentsList = InComponentsList;
	}

	/** Adds components to the list. Must be called after SetVisibilityModeAndComponentsList(). */
	bool AppendVisibilityComponentsList(EDisplayClusterViewport_VisibilityMode InVisibilityMode, const TSet<FPrimitiveComponentId>& InComponentsList)
	{
		if (VisibilityMode == InVisibilityMode)
		{
			ComponentsList.Append(InComponentsList);

			return true;
		}

		return false;
	}

	/** Exclude these components from the output list. */
	void AddExcludedComponents(const TSet<FPrimitiveComponentId>& InComponentsList)
	{
		ExcludedComponents.Append(InComponentsList);
	}

	/** Returns true if this viewport contains any geometry and can be rendered. */
	bool IsVisible() const
	{
		if (VisibilityMode == EDisplayClusterViewport_VisibilityMode::ShowOnly && ComponentsList.IsEmpty())
		{
			return false;
		}
		
		return true;
	}

	/** Configure InOutView using the viewport settings. */
	void SetupSceneView(const FDisplayClusterViewport& Owner, FSceneView& InOutView) const;

private:
	EDisplayClusterViewport_VisibilityMode VisibilityMode = EDisplayClusterViewport_VisibilityMode::None;
	TSet<FPrimitiveComponentId> ComponentsList;

	// Exclude this components from ComponentsList
	TSet<FPrimitiveComponentId> ExcludedComponents;
};
