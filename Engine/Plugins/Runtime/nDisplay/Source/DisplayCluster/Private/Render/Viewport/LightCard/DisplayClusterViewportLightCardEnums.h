// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/* Consolidated UV light card rendering mode for the current cluster node. **/
enum class EDisplayClusterUVLightCardRenderMode : uint8
{
	// Use light card render mode from the StageSettings
	Default,

	// The lightcard will always be displayed only "Over the In-Camera".
	AlwaysOver,

	// The lightcard will always be displayed only "Under the In-Camera".
	AlwaysUnder,

	// Lighting maps are not used.
	Disabled
};

/** The UV lightcards have 2 type: 'Under' and 'Over'. */
enum class EDisplayClusterUVLightCardType: uint8
{
	// The UV lightcards that are displayed under inner frustum.
	Under,

	// The UV lightcards that are displayed over inner frustum.
	Over
};
