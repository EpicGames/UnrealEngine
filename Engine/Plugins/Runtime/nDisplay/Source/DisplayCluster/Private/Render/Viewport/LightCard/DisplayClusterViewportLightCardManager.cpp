// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterViewportLightCardManager.h"
#include "DisplayClusterViewportLightCardManagerProxy.h"
#include "DisplayClusterViewportLightCardResource.h"

#include "Render/Viewport/Configuration/DisplayClusterViewportConfiguration.h"

#include "ShaderParameters/DisplayClusterShaderParameters_UVLightCards.h"

#include "DisplayClusterLightCardActor.h"
#include "Blueprints/DisplayClusterBlueprintLib.h"

#include "IDisplayClusterShaders.h"

#include "Render/Viewport/DisplayClusterViewportManager.h"

#include "PreviewScene.h"

///////////////////////////////////////////////////////////////////////////////////////////////
/** Console variable used to control the size of the UV light card map texture */
static TAutoConsoleVariable<int32> CVarUVLightCardTextureSize(
	TEXT("nDisplay.render.uvlightcards.UVTextureSize"),
	4096,
	TEXT("The size of the texture UV light cards are rendered to.")
);

///////////////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterViewportLightCardManager
///////////////////////////////////////////////////////////////////////////////////////////////
FDisplayClusterViewportLightCardManager::FDisplayClusterViewportLightCardManager(const TSharedRef<const FDisplayClusterViewportConfiguration, ESPMode::ThreadSafe>& InConfiguration)
	: Configuration(InConfiguration)
	, LightCardManagerProxy(MakeShared<FDisplayClusterViewportLightCardManagerProxy, ESPMode::ThreadSafe>())
{ }

FDisplayClusterViewportLightCardManager::~FDisplayClusterViewportLightCardManager()
{
	Release();
}

void FDisplayClusterViewportLightCardManager::Release()
{
	// The destructor is usually called from the rendering thread, so Release() must be called first from the game thread.
	check(IsInGameThread());

	// Release UVLightCard
	ReleaseUVLightCardData(EDisplayClusterUVLightCardType::Under);
	ReleaseUVLightCardResource(EDisplayClusterUVLightCardType::Under);

	ReleaseUVLightCardData(EDisplayClusterUVLightCardType::Over);
	ReleaseUVLightCardResource(EDisplayClusterUVLightCardType::Over);
}

///////////////////////////////////////////////////////////////////////////////////////////////
void FDisplayClusterViewportLightCardManager::OnHandleStartScene()
{
}

void FDisplayClusterViewportLightCardManager::OnHandleEndScene()
{
	ReleaseUVLightCardData(EDisplayClusterUVLightCardType::Under);
	ReleaseUVLightCardData(EDisplayClusterUVLightCardType::Over);
}

void FDisplayClusterViewportLightCardManager::RenderFrame()
{
	UpdateUVLightCardData(EDisplayClusterUVLightCardType::Under);
	RenderUVLightCard(EDisplayClusterUVLightCardType::Under);

	UpdateUVLightCardData(EDisplayClusterUVLightCardType::Over);
	RenderUVLightCard(EDisplayClusterUVLightCardType::Over);
}

///////////////////////////////////////////////////////////////////////////////////////////////
FIntPoint FDisplayClusterViewportLightCardManager::GetUVLightCardResourceSize(const EDisplayClusterUVLightCardType InUVLightCardType) const
{
	const TSharedPtr<FDisplayClusterViewportLightCardResource, ESPMode::ThreadSafe>& UVLightCardResource = GetUVLightCardResource(InUVLightCardType);
	return UVLightCardResource.IsValid() ? UVLightCardResource->GetSizeXY() : FIntPoint(0, 0);
}

EDisplayClusterUVLightCardRenderMode FDisplayClusterViewportLightCardManager::GetUVLightCardRenderMode() const
{
	// Value should be updated.
	if (const FDisplayClusterViewportManager* ViewportManager = Configuration->GetViewportManagerImpl())
	{
		// Collect all ICVFX flags
		EDisplayClusterViewportICVFXFlags ICVFXFlags = EDisplayClusterViewportICVFXFlags::None;
		for (const TSharedPtr<IDisplayClusterViewport, ESPMode::ThreadSafe>& ViewportIt : ViewportManager->GetEntireClusterViewports())
		{
			if (ViewportIt.IsValid())
			{
				ICVFXFlags |= ViewportIt->GetRenderSettingsICVFX().Flags;
			}
		}

		if (EnumHasAnyFlags(ICVFXFlags, EDisplayClusterViewportICVFXFlags::DisableLightcard))
		{
			return EDisplayClusterUVLightCardRenderMode::Disabled;
		}
		else
		{
			// Returns the consolidated light card rendering mode for this cluster node:
			const EDisplayClusterViewportICVFXFlags LightcardRenderModeFlags = ICVFXFlags & EDisplayClusterViewportICVFXFlags::LightcardRenderModeMask;
			if (LightcardRenderModeFlags == EDisplayClusterViewportICVFXFlags::LightcardAlwaysUnder)
			{
				// The lightcard will always be displayed only "Under the In-Camera" for this cluster node.
				return EDisplayClusterUVLightCardRenderMode::AlwaysUnder;
			}
			else if (LightcardRenderModeFlags == EDisplayClusterViewportICVFXFlags::LightcardAlwaysOver)
			{
				// The lightcard will always be displayed only "Over the In-Camera" for this cluster node.
				return EDisplayClusterUVLightCardRenderMode::AlwaysOver;
			}
		}
	}
	
	return EDisplayClusterUVLightCardRenderMode::Default;
}

bool FDisplayClusterViewportLightCardManager::IsUVLightCardEnabled(const EDisplayClusterUVLightCardType InUVLightCardType) const
{
	switch (GetUVLightCardRenderMode())
	{
	case EDisplayClusterUVLightCardRenderMode::Disabled:
		return false;

	case EDisplayClusterUVLightCardRenderMode::AlwaysOver:
		if (InUVLightCardType != EDisplayClusterUVLightCardType::Over)
		{
			// Force to render only over
			return false;
		}
		break;

	case EDisplayClusterUVLightCardRenderMode::AlwaysUnder:
		if (InUVLightCardType != EDisplayClusterUVLightCardType::Under)
		{
			// Force to render only under
			return false;
		}
		break;

	default:
		break;
	}

	const TArray<UPrimitiveComponent*>& UVLightCardPrimitiveComponents = GetUVLightCardPrimitiveComponents(InUVLightCardType);
	return !UVLightCardPrimitiveComponents.IsEmpty();
}

void FDisplayClusterViewportLightCardManager::ReleaseUVLightCardData(const EDisplayClusterUVLightCardType InUVLightCardType)
{
	TArray<UPrimitiveComponent*>& UVLightCardPrimitiveComponents = GetUVLightCardPrimitiveComponents(InUVLightCardType);
	UVLightCardPrimitiveComponents.Empty();
}

void FDisplayClusterViewportLightCardManager::UpdateUVLightCardData(const EDisplayClusterUVLightCardType InUVLightCardType)
{
	ReleaseUVLightCardData(InUVLightCardType);

	// Special use-case - when all viewports force to use only over or under, we have to ignore per-light card mode.
	bool bEnablePerLightcardRenderMode = true;
	switch (GetUVLightCardRenderMode())
	{
	case EDisplayClusterUVLightCardRenderMode::Disabled:
		return;

	case EDisplayClusterUVLightCardRenderMode::AlwaysOver:
		bEnablePerLightcardRenderMode = false;
		if (InUVLightCardType != EDisplayClusterUVLightCardType::Over)
		{
			// Force to render only over
			return;
		}
		break;

	case EDisplayClusterUVLightCardRenderMode::AlwaysUnder:
		bEnablePerLightcardRenderMode = false;
		if (InUVLightCardType != EDisplayClusterUVLightCardType::Under)
		{
			// Force to render only under
			return;
		}
		break;

	default:
		break;
	}


	const FDisplayClusterConfigurationICVFX_StageSettings* StageSettings = Configuration->GetStageSettings();
	if (!StageSettings)
	{
		return;
	}

	/** The list of UV light card actors that are referenced by the root actor */
	TArray<ADisplayClusterLightCardActor*> UVLightCardActors;

	if (ADisplayClusterRootActor* SceneRootActorPtr = Configuration->GetRootActor(EDisplayClusterRootActorType::Scene))
	{
		TSet<ADisplayClusterLightCardActor*> LightCards;
		UDisplayClusterBlueprintLib::FindLightCardsForRootActor(SceneRootActorPtr, LightCards);

		for (ADisplayClusterLightCardActor* LightCard : LightCards)
		{
			if (LightCard->bIsUVLightCard)
			{
				if (bEnablePerLightcardRenderMode)
				{
					// Per-lightcard rules:
					const EDisplayClusterShaderParametersICVFX_LightCardRenderMode LightCardRenderMode = StageSettings->Lightcard.GetLightCardRenderMode(LightCard->PerLightcardRenderMode, nullptr);
					const bool bLightCardActorOver = (LightCardRenderMode == EDisplayClusterShaderParametersICVFX_LightCardRenderMode::Over);
					const bool bLightCardTypeOver = InUVLightCardType == EDisplayClusterUVLightCardType::Over;
					if (bLightCardTypeOver == bLightCardActorOver)
					{
						UVLightCardActors.Add(LightCard);
					}
				}
				else
				{
					// Render all UVLC to the one RTT
					UVLightCardActors.Add(LightCard);
				}
			}
		}
	}

	TArray<UMeshComponent*> LightCardMeshComponents;
	for (ADisplayClusterLightCardActor* LightCard : UVLightCardActors)
	{
		if (LightCard->IsHidden() || LightCard->IsActorBeingDestroyed() || LightCard->GetWorld() == nullptr)
		{
			continue;
		}

		LightCardMeshComponents.Empty(LightCardMeshComponents.Num());
		LightCard->GetLightCardMeshComponents(LightCardMeshComponents);

		TArray<UPrimitiveComponent*>& UVLightCardPrimitiveComponents = GetUVLightCardPrimitiveComponents(InUVLightCardType);
		for (UMeshComponent* LightCardMeshComp : LightCardMeshComponents)
		{
			if (LightCardMeshComp && LightCardMeshComp->SceneProxy == nullptr)
			{
				UVLightCardPrimitiveComponents.Add(LightCardMeshComp);
			}
		}
	}
}

void FDisplayClusterViewportLightCardManager::CreateUVLightCardResource(const FIntPoint& InResourceSize, const EDisplayClusterUVLightCardType InUVLightCardType)
{
	TSharedPtr<FDisplayClusterViewportLightCardResource, ESPMode::ThreadSafe>& UVLightCardResource = GetUVLightCardResource(InUVLightCardType);
	UVLightCardResource = MakeShared<FDisplayClusterViewportLightCardResource>(InResourceSize);
	LightCardManagerProxy->UpdateUVLightCardResource(UVLightCardResource, InUVLightCardType);
}

void FDisplayClusterViewportLightCardManager::ReleaseUVLightCardResource(const EDisplayClusterUVLightCardType InUVLightCardType)
{
	TSharedPtr<FDisplayClusterViewportLightCardResource, ESPMode::ThreadSafe>& UVLightCardResource = GetUVLightCardResource(InUVLightCardType);
	if (UVLightCardResource.IsValid())
	{
		LightCardManagerProxy->ReleaseUVLightCardResource(InUVLightCardType);
	}

	UVLightCardResource.Reset();
}

void FDisplayClusterViewportLightCardManager::UpdateUVLightCardResource(const EDisplayClusterUVLightCardType InUVLightCardType)
{
	const uint32 UVLightCardTextureSize = CVarUVLightCardTextureSize.GetValueOnGameThread();
	const FIntPoint UVLightCardResourceSize = FIntPoint(UVLightCardTextureSize, UVLightCardTextureSize);

	TSharedPtr<FDisplayClusterViewportLightCardResource, ESPMode::ThreadSafe>& UVLightCardResource = GetUVLightCardResource(InUVLightCardType);
	if (UVLightCardResource.IsValid())
	{
		if (UVLightCardResource->GetSizeXY() != UVLightCardResourceSize)
		{
			ReleaseUVLightCardResource(InUVLightCardType);
		}
	}

	if (!UVLightCardResource.IsValid())
	{
		CreateUVLightCardResource(UVLightCardResourceSize, InUVLightCardType);
	}
}

void FDisplayClusterViewportLightCardManager::RenderUVLightCard(const EDisplayClusterUVLightCardType InUVLightCardType)
{
	// Render UV LightCard:
	FDisplayClusterViewportManager* ViewportManager = Configuration->GetViewportManagerImpl();
	UWorld* CurrentWorld = Configuration->GetCurrentWorld();
	if (IsUVLightCardEnabled(InUVLightCardType) && CurrentWorld && ViewportManager)
	{
		UpdateUVLightCardResource(InUVLightCardType);

		TSharedPtr<FDisplayClusterViewportLightCardResource, ESPMode::ThreadSafe>& UVLightCardResource = GetUVLightCardResource(InUVLightCardType);
		if (UVLightCardResource.IsValid())
		{
			FDisplayClusterShaderParameters_UVLightCards UVLightCardParameters;
			UVLightCardParameters.ProjectionPlaneSize = ADisplayClusterLightCardActor::UVPlaneDefaultSize;

			// Store any components that were invisible but forced to be visible so they can be set back to invisible after the render
			TArray<UPrimitiveComponent*> ComponentsToUnload;
			const TArray<UPrimitiveComponent*>& UVLightCardPrimitiveComponents = GetUVLightCardPrimitiveComponents(InUVLightCardType);
			for (UPrimitiveComponent* PrimitiveComponent : UVLightCardPrimitiveComponents)
			{
				// Set the component's visibility to true and force it to generate its scene proxies
				if (!PrimitiveComponent->IsVisible())
				{
					PrimitiveComponent->SetVisibility(true);
					PrimitiveComponent->RecreateRenderState_Concurrent();
					ComponentsToUnload.Add(PrimitiveComponent);
				}

				if (PrimitiveComponent->SceneProxy)
				{
					UVLightCardParameters.PrimitivesToRender.Add(PrimitiveComponent->SceneProxy);
				}
			}

			LightCardManagerProxy->RenderUVLightCard(CurrentWorld->Scene, UVLightCardParameters, InUVLightCardType);

			for (UPrimitiveComponent* LoadedComponent : ComponentsToUnload)
			{
				LoadedComponent->SetVisibility(false);
				LoadedComponent->RecreateRenderState_Concurrent();
			}
		}
	}
	else
	{
		ReleaseUVLightCardResource(InUVLightCardType);
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////
void FDisplayClusterViewportLightCardManager::AddReferencedObjects(FReferenceCollector& Collector)
{
}
