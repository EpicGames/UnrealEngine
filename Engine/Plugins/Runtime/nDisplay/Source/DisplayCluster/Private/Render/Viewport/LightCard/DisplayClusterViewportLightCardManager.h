// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DisplayClusterViewportLightCardManagerProxy.h"
#include "Containers/DisplayClusterShader_Enums.h"

#include "UObject/GCObject.h"

class FDisplayClusterViewportConfiguration;
class ADisplayClusterLightCardActor;
class UWorld;

/**
 * Manages the rendering of UV light cards for the viewport manager (Game Thread object)
 */
class FDisplayClusterViewportLightCardManager
	: public FGCObject
{
public:
	FDisplayClusterViewportLightCardManager(const TSharedRef<const FDisplayClusterViewportConfiguration, ESPMode::ThreadSafe>& InConfiguration);
	virtual ~FDisplayClusterViewportLightCardManager();

	void Release();

	//~ Begin FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override { return TEXT("FDisplayClusterViewportLightCardManager"); }
	//~ End FGCObject interface

public:
	/** Return true if UV LightCard is used in this frame. */
	bool IsUVLightCardEnabled(const EDisplayClusterUVLightCardType InUVLightCardType) const;

	/** Get UV LightCard texture size. */
	FIntPoint GetUVLightCardResourceSize(const EDisplayClusterUVLightCardType InUVLightCardType) const;

public:
	/** Handle StartScene event: created and update internal resources. */
	void OnHandleStartScene();

	/** Handle EndScene event: release internal resources. */
	void OnHandleEndScene();

	/** Render internal resoures for current frame. */
	void RenderFrame();

private:
	/** Render UVLightCard */
	void RenderUVLightCard(const EDisplayClusterUVLightCardType InUVLightCardType);

	/** Update the UV light card map texture */
	void UpdateUVLightCardResource(const EDisplayClusterUVLightCardType InUVLightCardType);

	/** Releases the UV light card map texture */
	void ReleaseUVLightCardResource(const EDisplayClusterUVLightCardType InUVLightCardType);

	/** Create the UV light card map texture */
	void CreateUVLightCardResource(const FIntPoint& InResourceSize, const EDisplayClusterUVLightCardType InUVLightCardType);

	/** Update UVLightCard data game thread*/
	void UpdateUVLightCardData(const EDisplayClusterUVLightCardType InUVLightCardType);

	/** Release UVLightCard data game thread*/
	void ReleaseUVLightCardData(const EDisplayClusterUVLightCardType InUVLightCardType);

private:


	/** Returns the consolidated UV-light card rendering mode for the current cluster node.
	* If the light card override mode is the same for all viewports of the current cluster node, return this value.
	* Or returns 'None' if they do not.
	*/
	EDisplayClusterUVLightCardRenderMode GetUVLightCardRenderMode() const;

	TSharedPtr<FDisplayClusterViewportLightCardResource, ESPMode::ThreadSafe>& GetUVLightCardResource(const EDisplayClusterUVLightCardType InUVLightCardType)
	{
		return (InUVLightCardType == EDisplayClusterUVLightCardType::Over) ? UVLightCardOverResource : UVLightCardUnderResource;
	}

	const TSharedPtr<FDisplayClusterViewportLightCardResource, ESPMode::ThreadSafe>& GetUVLightCardResource(const EDisplayClusterUVLightCardType InUVLightCardType) const
	{
		return (InUVLightCardType == EDisplayClusterUVLightCardType::Over) ? UVLightCardOverResource : UVLightCardUnderResource;
	}

	TArray<UPrimitiveComponent*>& GetUVLightCardPrimitiveComponents(const EDisplayClusterUVLightCardType InUVLightCardType)
	{
		return (InUVLightCardType == EDisplayClusterUVLightCardType::Over) ? UVLightCardOverPrimitiveComponents : UVLightCardUnderPrimitiveComponents;
	}

	const TArray<UPrimitiveComponent*>& GetUVLightCardPrimitiveComponents(const EDisplayClusterUVLightCardType InUVLightCardType) const
	{
		return (InUVLightCardType == EDisplayClusterUVLightCardType::Over) ? UVLightCardOverPrimitiveComponents : UVLightCardUnderPrimitiveComponents;
	}

public:
	// Configuration of the current cluster node
	const TSharedRef<const FDisplayClusterViewportConfiguration, ESPMode::ThreadSafe> Configuration;

	/** RenderThread Proxy object*/
	const TSharedRef<FDisplayClusterViewportLightCardManagerProxy, ESPMode::ThreadSafe> LightCardManagerProxy;

private:
	/** A list of primitive components that have been added to the preview scene for rendering in the current frame */
	TArray<UPrimitiveComponent*> UVLightCardUnderPrimitiveComponents;

	/** A list of primitive components that have been added to the preview scene for rendering in the current frame */
	TArray<UPrimitiveComponent*> UVLightCardOverPrimitiveComponents;

	/** The render target to which the UV light card map is rendered */
	TSharedPtr<FDisplayClusterViewportLightCardResource, ESPMode::ThreadSafe> UVLightCardUnderResource;

	/** The render target to which the UV light card map is rendered */
	TSharedPtr<FDisplayClusterViewportLightCardResource, ESPMode::ThreadSafe> UVLightCardOverResource;
};
