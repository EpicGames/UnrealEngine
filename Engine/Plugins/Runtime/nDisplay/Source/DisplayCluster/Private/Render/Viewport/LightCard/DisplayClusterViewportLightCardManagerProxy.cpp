// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterViewportLightCardManagerProxy.h"

#include "IDisplayClusterShaders.h"
#include "ShaderParameters/DisplayClusterShaderParameters_UVLightCards.h"

#include "SceneInterface.h"
#include "RenderingThread.h"

///////////////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterViewportLightCardManagerProxy
///////////////////////////////////////////////////////////////////////////////////////////////
FDisplayClusterViewportLightCardManagerProxy::~FDisplayClusterViewportLightCardManagerProxy()
{
	ImplReleaseUVLightCardResource_RenderThread(EDisplayClusterUVLightCardType::Under);
	ImplReleaseUVLightCardResource_RenderThread(EDisplayClusterUVLightCardType::Over);
}

///////////////////////////////////////////////////////////////////////////////////////////////
FRHITexture* FDisplayClusterViewportLightCardManagerProxy::GetUVLightCardRHIResource_RenderThread(const EDisplayClusterUVLightCardType InUVLightCardType) const
{
	const TSharedPtr<FDisplayClusterViewportLightCardResource, ESPMode::ThreadSafe>& UVLightCardMapResource = GetUVLightCardMapResource(InUVLightCardType);
	return UVLightCardMapResource.IsValid() ? UVLightCardMapResource->GetTextureRHI().GetReference() : nullptr;
}

void FDisplayClusterViewportLightCardManagerProxy::UpdateUVLightCardResource(const TSharedPtr<FDisplayClusterViewportLightCardResource, ESPMode::ThreadSafe>& InUVLightCardMapResource, const EDisplayClusterUVLightCardType InUVLightCardType)
{
	ENQUEUE_RENDER_COMMAND(DisplayClusterViewportLightCardManagerProxy_UpdateUVLightCardResource)(
		[InProxyData = SharedThis(this), NewUVLightCardMapResource = InUVLightCardMapResource, InUVLightCardType](FRHICommandListImmediate& RHICmdList)
		{
			InProxyData->ImplUpdateUVLightCardResource_RenderThread(NewUVLightCardMapResource, InUVLightCardType);
		});
}

void FDisplayClusterViewportLightCardManagerProxy::ReleaseUVLightCardResource(const EDisplayClusterUVLightCardType InUVLightCardType)
{
	ENQUEUE_RENDER_COMMAND(DisplayClusterViewportLightCardManagerProxy_ReleaseUVLightCardResource)(
		[InProxyData = SharedThis(this), InUVLightCardType](FRHICommandListImmediate& RHICmdList)
		{
			InProxyData->ImplReleaseUVLightCardResource_RenderThread(InUVLightCardType);
		});
}

void FDisplayClusterViewportLightCardManagerProxy::RenderUVLightCard(FSceneInterface* InScene, const FDisplayClusterShaderParameters_UVLightCards& InParameters, const EDisplayClusterUVLightCardType InUVLightCardType) const
{
	UE::RenderCommandPipe::FSyncScope SyncScope;

	ENQUEUE_RENDER_COMMAND(DisplayClusterViewportLightCardManagerProxy_RenderUVLightCard)(
		[InProxyData = SharedThis(this), Scene = InScene, Parameters = InParameters, InUVLightCardType](FRHICommandListImmediate& RHICmdList)
		{
			InProxyData->ImplRenderUVLightCard_RenderThread(RHICmdList, Scene, Parameters, InUVLightCardType);
		});
}

void FDisplayClusterViewportLightCardManagerProxy::ImplUpdateUVLightCardResource_RenderThread(const TSharedPtr<FDisplayClusterViewportLightCardResource, ESPMode::ThreadSafe>& InUVLightCardMapResource, const EDisplayClusterUVLightCardType InUVLightCardType)
{
	TSharedPtr<FDisplayClusterViewportLightCardResource, ESPMode::ThreadSafe>& UVLightCardMapResource = GetUVLightCardMapResource(InUVLightCardType);
	if (UVLightCardMapResource != InUVLightCardMapResource)
	{
		ImplReleaseUVLightCardResource_RenderThread(InUVLightCardType);

		// Update resource ptr
		UVLightCardMapResource = InUVLightCardMapResource;
		if (UVLightCardMapResource.IsValid())
		{
			UVLightCardMapResource->InitResource(FRHICommandListImmediate::Get());
		}
	}
}

void FDisplayClusterViewportLightCardManagerProxy::ImplReleaseUVLightCardResource_RenderThread(const EDisplayClusterUVLightCardType InUVLightCardType)
{
	// Release the texture's resources and delete the texture object from the rendering thread
	TSharedPtr<FDisplayClusterViewportLightCardResource, ESPMode::ThreadSafe>& UVLightCardMapResource = GetUVLightCardMapResource(InUVLightCardType);
	if (UVLightCardMapResource.IsValid())
	{
		UVLightCardMapResource->ReleaseResource();
		UVLightCardMapResource.Reset();
	}
}

void FDisplayClusterViewportLightCardManagerProxy::ImplRenderUVLightCard_RenderThread(FRHICommandListImmediate& RHICmdList, FSceneInterface* InSceneInterface, const FDisplayClusterShaderParameters_UVLightCards& InParameters, const EDisplayClusterUVLightCardType InUVLightCardType) const
{
	const TSharedPtr<FDisplayClusterViewportLightCardResource, ESPMode::ThreadSafe>& UVLightCardMapResource = GetUVLightCardMapResource(InUVLightCardType);
	if (InParameters.PrimitivesToRender.Num() && UVLightCardMapResource.IsValid())
	{
		IDisplayClusterShaders& ShadersAPI = IDisplayClusterShaders::Get();
		ShadersAPI.RenderPreprocess_UVLightCards(RHICmdList, InSceneInterface, UVLightCardMapResource.Get(), InParameters);
	}
}
