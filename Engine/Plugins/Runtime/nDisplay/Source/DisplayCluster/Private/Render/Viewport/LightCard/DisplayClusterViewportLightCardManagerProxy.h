// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DisplayClusterViewportLightCardEnums.h"
#include "DisplayClusterViewportLightCardResource.h"

#include "RHIResources.h"
#include "Templates/SharedPointer.h"

struct FDisplayClusterShaderParameters_UVLightCards;
class FSceneInterface;

/*
 * Manages the rendering of UV light cards for the viewport manager (Render Thread proxy object)
 */
class FDisplayClusterViewportLightCardManagerProxy
	: public TSharedFromThis<FDisplayClusterViewportLightCardManagerProxy, ESPMode::ThreadSafe>
{
public:
	virtual ~FDisplayClusterViewportLightCardManagerProxy();

	/** Update UVLightCard resource. */
	void UpdateUVLightCardResource(const TSharedPtr<FDisplayClusterViewportLightCardResource, ESPMode::ThreadSafe>& InUVLightCardMapResource, const EDisplayClusterUVLightCardType InUVLightCardType);

	/** Release UVLightCard resource. */
	void ReleaseUVLightCardResource(const EDisplayClusterUVLightCardType InUVLightCardType);

	/** Render UVLightCard. */
	void RenderUVLightCard(FSceneInterface* InSceneInterface, const FDisplayClusterShaderParameters_UVLightCards& InParameters, const EDisplayClusterUVLightCardType InUVLightCardType) const;

	/** Get current UVLightCard RHI resource on rendering thread. */
	FRHITexture* GetUVLightCardRHIResource_RenderThread(const EDisplayClusterUVLightCardType InUVLightCardType) const;

protected:
	void ImplUpdateUVLightCardResource_RenderThread(const TSharedPtr<FDisplayClusterViewportLightCardResource, ESPMode::ThreadSafe>& InUVLightCardMapResource, const EDisplayClusterUVLightCardType InUVLightCardType);
	void ImplReleaseUVLightCardResource_RenderThread(const EDisplayClusterUVLightCardType InUVLightCardType);

	/** Render UV-lightcards to texture (the layer is defined by the bOverInFrustum parameter.). */
	void ImplRenderUVLightCard_RenderThread(FRHICommandListImmediate& RHICmdList, FSceneInterface* InScene, const FDisplayClusterShaderParameters_UVLightCards& InParameters, const EDisplayClusterUVLightCardType InUVLightCardType) const;

	TSharedPtr<FDisplayClusterViewportLightCardResource, ESPMode::ThreadSafe>& GetUVLightCardMapResource(const EDisplayClusterUVLightCardType InUVLightCardType)
	{
		return (InUVLightCardType == EDisplayClusterUVLightCardType::Over) ? UVLightCardOverMapResource: UVLightCardUnderMapResource;
	}

	const TSharedPtr<FDisplayClusterViewportLightCardResource, ESPMode::ThreadSafe>& GetUVLightCardMapResource(const EDisplayClusterUVLightCardType InUVLightCardType) const
	{
		return (InUVLightCardType == EDisplayClusterUVLightCardType::Over) ? UVLightCardOverMapResource : UVLightCardUnderMapResource;
	}

private:
	/** The render thread copy of the pointer to the UV light card map */
	TSharedPtr<FDisplayClusterViewportLightCardResource, ESPMode::ThreadSafe> UVLightCardUnderMapResource;

	/** The render thread copy of the pointer to the UV light card map */
	TSharedPtr<FDisplayClusterViewportLightCardResource, ESPMode::ThreadSafe> UVLightCardOverMapResource;
};
