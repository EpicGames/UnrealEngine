// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterViewportLightCardResource.h"

///////////////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterViewportLightCardResource
///////////////////////////////////////////////////////////////////////////////////////////////
void FDisplayClusterViewportLightCardResource::InitRHI(FRHICommandListBase& RHICmdList)
{
	ETextureCreateFlags CreateFlags = TexCreate_MultiGPUGraphIgnore;

	const FRHITextureCreateDesc Desc = FRHITextureCreateDesc::Create2D(TEXT("DisplayClusterViewportLightCardResource"))
		.SetExtent(GetSizeX(), GetSizeY())
		.SetFormat(PF_FloatRGBA)
		.SetFlags(CreateFlags | ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::ShaderResource)
		.SetInitialState(ERHIAccess::SRVMask)
		.SetClearValue(FClearValueBinding::Transparent);

	RenderTargetTextureRHI = TextureRHI = RHICmdList.CreateTexture(Desc);
}
