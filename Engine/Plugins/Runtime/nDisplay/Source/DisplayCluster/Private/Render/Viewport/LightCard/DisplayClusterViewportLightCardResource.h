// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"

#include "RenderResource.h"
#include "UnrealClient.h"

/**
 * A render targetable texture resource used to render the light cards to.
 */
class FDisplayClusterViewportLightCardResource
	: public FTexture
	, public FRenderTarget
{
public:
	FDisplayClusterViewportLightCardResource(const FIntPoint& InLightCardResourceSize)
		: LightCardResourceSize(InLightCardResourceSize)
	{ }

	virtual ~FDisplayClusterViewportLightCardResource() = default;

	virtual uint32 GetSizeX() const override { return LightCardResourceSize.X; }
	virtual uint32 GetSizeY() const override { return LightCardResourceSize.Y; }
	virtual FIntPoint GetSizeXY() const override { return LightCardResourceSize; }

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;

	virtual FString GetFriendlyName() const override { return TEXT("DisplayClusterViewportLightCardResource"); }

private:
	FIntPoint LightCardResourceSize;
};
