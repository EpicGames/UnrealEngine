// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Viewport/Postprocess/DisplayClusterViewportPostProcessManager.h"
#include "Render/Viewport/Postprocess/DisplayClusterViewportPostProcessOutputRemap.h"
#include "Render/Viewport/DisplayClusterViewportManager.h"
#include "Render/Viewport/DisplayClusterViewportManagerProxy.h"
#include "Render/Viewport/DisplayClusterViewportProxy.h"
#include "Render/Viewport/Configuration/DisplayClusterViewportConfiguration.h"
#include "Render/PostProcess/IDisplayClusterPostProcessFactory.h"
#include "Render/IPDisplayClusterRenderManager.h"
#include "Render/IDisplayClusterRenderManager.h"
#include "DisplayClusterConfigurationTypes.h"

#include "Misc/DisplayClusterHelpers.h"
#include "Misc/DisplayClusterLog.h"
#include "Misc/DisplayClusterGlobals.h"
#include "HAL/IConsoleManager.h"

//////////////////////////////////////////////////////////////////////////////////////////////
// Round 1: VIEW before warp&blend
//////////////////////////////////////////////////////////////////////////////////////////////

// Enable/disable nDisplay post-process
static TAutoConsoleVariable<int32> CVarCustomPPEnabled(
	TEXT("nDisplay.render.postprocess"),
	1,
	TEXT("Custom post-process (0 = disabled)\n"),
	ECVF_RenderThreadSafe
);

// Enable/disable PP round 1
static TAutoConsoleVariable<int32> CVarPostprocessViewBeforeWarpBlend(
	TEXT("nDisplay.render.postprocess.ViewBeforeWarpBlend"),
	1,
	TEXT("Enable PP per view before warp&blend (0 = disabled)\n"),
	ECVF_RenderThreadSafe
);

// Enable/disable PP round 4
static TAutoConsoleVariable<int32> CVarPostprocessViewAfterWarpBlend(
	TEXT("nDisplay.render.postprocess.ViewAfterWarpBlend"),
	1,
	TEXT("Enable PP per view after warp&blend (0 = disabled)\n"),
	ECVF_RenderThreadSafe
);

// Enable/disable PP round 5
int32 GDisplayClusterPostprocessFrameAfterWarpBlendEnable = 1;
static FAutoConsoleVariableRef CVarDisplayClusterPostprocessFrameAfterWarpBlendEnable(
	TEXT("nDisplay.render.postprocess.FrameAfterWarpBlend"),
	GDisplayClusterPostprocessFrameAfterWarpBlendEnable,
	TEXT("Enable PP per eye frame after warp&blend (0 = disabled)\n"),
	ECVF_RenderThreadSafe
);

// Enables RTT cleanup before each full-frame post processing.
int32 GDisplayClusterPostProcessFrameAfterWarpBlendAlwaysClearRTT = 1;
static FAutoConsoleVariableRef CVarDisplayClusterPostProcessFrameAfterWarpBlendAlwaysClearRTT(
	TEXT("nDisplay.render.postprocess.FrameAfterWarpBlend.AlwaysClearRTT"),
	GDisplayClusterPostProcessFrameAfterWarpBlendAlwaysClearRTT,
	TEXT("Enables RTT cleanup before each full-frame post processing. (0 to disable).\n"),
	ECVF_RenderThreadSafe
);


//----------------------------------------------------------------------------------------------------------
// FDisplayClusterViewportPostProcessManager
//----------------------------------------------------------------------------------------------------------
FDisplayClusterViewportPostProcessManager::FDisplayClusterViewportPostProcessManager(const TSharedRef<const FDisplayClusterViewportConfiguration, ESPMode::ThreadSafe>& InConfiguration)
	: Configuration(InConfiguration)
{
	OutputRemap = MakeShared<FDisplayClusterViewportPostProcessOutputRemap, ESPMode::ThreadSafe>();
}

FDisplayClusterViewportPostProcessManager::~FDisplayClusterViewportPostProcessManager()
{
	check(Postprocess.IsEmpty());
	check(PostprocessProxy.IsEmpty());
	check(!OutputRemap.IsValid());
}

void FDisplayClusterViewportPostProcessManager::Release_GameThread()
{
	check(IsInGameThread());

	Postprocess.Empty();
	OutputRemap.Reset();

	ENQUEUE_RENDER_COMMAND(DisplayClusterViewportPostProcessManager_Release)(
		[PostprocessManager = SharedThis(this)](FRHICommandListImmediate& RHICmdList)
		{
			PostprocessManager->PostprocessProxy.Empty();
		});
}

bool FDisplayClusterViewportPostProcessManager::OnHandleStartScene()
{
	check(IsInGameThread());

	bool bResult = true;

	IDisplayClusterViewportManager* ViewportManager = Configuration->GetViewportManager();
	if (ViewportManager && Configuration->IsSceneOpened())
	{
		for (const TSharedPtr<IDisplayClusterPostProcess, ESPMode::ThreadSafe>& It : Postprocess)
		{
			if (It.IsValid())
			{
				if (!It->HandleStartScene(ViewportManager))
				{
					bResult = false;
				}
			}
		}
	}

	return bResult;
}

void FDisplayClusterViewportPostProcessManager::OnHandleEndScene()
{
	check(IsInGameThread());

	if (IDisplayClusterViewportManager* ViewportManager = Configuration->GetViewportManager())
	{
		for (const TSharedPtr<IDisplayClusterPostProcess, ESPMode::ThreadSafe>& It : Postprocess)
		{
			if (It.IsValid())
			{
				It->HandleEndScene(ViewportManager);
			}
		}
	}
}

const TArray<FString> FDisplayClusterViewportPostProcessManager::GetPostprocess() const
{
	check(IsInGameThread());

	TArray<FString> ExistPostProcess;
	for (const TSharedPtr<IDisplayClusterPostProcess, ESPMode::ThreadSafe>& It : Postprocess)
	{
		if (It.IsValid())
		{
			ExistPostProcess.Add(It->GetId());
		}
	}

	return ExistPostProcess;
}

bool FDisplayClusterViewportPostProcessManager::CanBeCreated(const FDisplayClusterConfigurationPostprocess* InConfigurationPostprocess) const
{
	check(IsInGameThread());
	check(InConfigurationPostprocess != nullptr);

	IDisplayClusterRenderManager* const DCRenderManager = IDisplayCluster::Get().GetRenderMgr();
	check(DCRenderManager);

	const TSharedPtr<IDisplayClusterPostProcessFactory> PostProcessFactory = DCRenderManager->GetPostProcessFactory(InConfigurationPostprocess->Type);

	return PostProcessFactory.IsValid() && PostProcessFactory->CanBeCreated(InConfigurationPostprocess);
}

bool FDisplayClusterViewportPostProcessManager::CreatePostprocess(const FString& InPostprocessId, const FDisplayClusterConfigurationPostprocess* InConfigurationPostprocess)
{
	check(IsInGameThread());
	check(InConfigurationPostprocess != nullptr);

	const TSharedPtr<IDisplayClusterPostProcess, ESPMode::ThreadSafe> DesiredPP = ImplFindPostProcess(InPostprocessId);
	if (DesiredPP.IsValid())
	{
		UE_LOGF(LogDisplayClusterRender, Error, "PostProcess '%ls', type '%ls' : Already exist", *InConfigurationPostprocess->Type, *InPostprocessId);
		return false;
	}

	IDisplayClusterRenderManager* const DCRenderManager = IDisplayCluster::Get().GetRenderMgr();
	check(DCRenderManager);

	const TSharedPtr<IDisplayClusterPostProcessFactory> PostProcessFactory = DCRenderManager->GetPostProcessFactory(InConfigurationPostprocess->Type);
	if (PostProcessFactory.IsValid())
	{
		TSharedPtr<IDisplayClusterPostProcess, ESPMode::ThreadSafe> PostProcessInstance = PostProcessFactory->Create(InPostprocessId, InConfigurationPostprocess);
		if (PostProcessInstance.IsValid())
		{
			UE_LOGF(LogDisplayClusterRender, Log, "PostProcess '%ls', type '%ls' : Created", *InConfigurationPostprocess->Type, *InPostprocessId);

			IDisplayClusterViewportManager* ViewportManager = Configuration->GetViewportManager();
			if (ViewportManager && Configuration->IsSceneOpened())
			{
				PostProcessInstance->HandleStartScene(ViewportManager);
			}

			for (int32 Order = 0; Order < Postprocess.Num(); Order++)
			{
				if (Postprocess[Order].IsValid() && Postprocess[Order]->GetOrder() > PostProcessInstance->GetOrder())
				{
					// Add sorted
					Postprocess.Insert(PostProcessInstance, Order);

					return true;
				}
			}

			// Add last
			Postprocess.Add(PostProcessInstance);

			return true;
		}
		else
		{
			UE_LOGF(LogDisplayClusterRender, Warning, "Invalid PostProcess '%ls', type '%ls' : Can't create postprocess", *InConfigurationPostprocess->Type, *InPostprocessId);
		}
	}
	else
	{
		UE_LOGF(LogDisplayClusterRender, Warning, "No postprocess found for type '%ls' in '%ls'", *InConfigurationPostprocess->Type, *InPostprocessId);
	}

	return false;
}

bool FDisplayClusterViewportPostProcessManager::RemovePostprocess(const FString& InPostprocessId)
{
	check(IsInGameThread());

	TSharedPtr<IDisplayClusterPostProcess, ESPMode::ThreadSafe> DesiredPP = ImplFindPostProcess(InPostprocessId);
	if (DesiredPP.IsValid())
	{
		UE_LOGF(LogDisplayClusterRender, Log, "PostProcess '%ls', type '%ls' : Removed", *DesiredPP->GetType(), *DesiredPP->GetId());

		Postprocess.Remove(DesiredPP);

		return true;
	}

	return false;
}

bool FDisplayClusterViewportPostProcessManager::UpdatePostprocess(const FString& InPostprocessId, const FDisplayClusterConfigurationPostprocess* InConfigurationPostprocess)
{
	// Now update is remove+create
	RemovePostprocess(InPostprocessId);

	return CreatePostprocess(InPostprocessId, InConfigurationPostprocess);
}

TSharedPtr<IDisplayClusterPostProcess, ESPMode::ThreadSafe> FDisplayClusterViewportPostProcessManager::ImplFindPostProcess(const FString& InPostprocessId) const
{
	check(IsInGameThread());

	// Ok, we have a request for a particular viewport. Let's find it.
	TSharedPtr<IDisplayClusterPostProcess, ESPMode::ThreadSafe> const* DesiredPP = Postprocess.FindByPredicate([InPostprocessId](const TSharedPtr<IDisplayClusterPostProcess, ESPMode::ThreadSafe>& ItemPP)
	{
		return InPostprocessId.Equals(ItemPP->GetId(), ESearchCase::IgnoreCase);
	});

	return (DesiredPP != nullptr) ? *DesiredPP : nullptr;
}

bool FDisplayClusterViewportPostProcessManager::IsPostProcessViewBeforeWarpBlendRequired(const TSharedPtr<IDisplayClusterPostProcess, ESPMode::ThreadSafe>& PostprocessInstance) const
{
	if (PostprocessInstance.IsValid() && PostprocessInstance->IsPostProcessViewBeforeWarpBlendRequired())
	{
		return (CVarPostprocessViewBeforeWarpBlend.GetValueOnAnyThread() != 0);
	}

	return false;
}

bool FDisplayClusterViewportPostProcessManager::IsPostProcessViewAfterWarpBlendRequired(const TSharedPtr<IDisplayClusterPostProcess, ESPMode::ThreadSafe>& PostprocessInstance) const
{
	if (PostprocessInstance.IsValid() && PostprocessInstance->IsPostProcessViewAfterWarpBlendRequired())
	{
		return (CVarPostprocessViewAfterWarpBlend.GetValueOnAnyThread() != 0);
	}

	return false;
}

bool FDisplayClusterViewportPostProcessManager::IsPostProcessFrameAfterWarpBlendRequired(const TSharedPtr<IDisplayClusterPostProcess, ESPMode::ThreadSafe>& PostprocessInstance) const
{
	if (GDisplayClusterPostprocessFrameAfterWarpBlendEnable
		&& PostprocessInstance.IsValid() && PostprocessInstance->IsPostProcessFrameAfterWarpBlendRequired())
	{
		return true;
	}

	return false;
}

bool FDisplayClusterViewportPostProcessManager::IsAnyPostProcessRequired(const TSharedPtr<IDisplayClusterPostProcess, ESPMode::ThreadSafe>& PostprocessInstance) const
{
	if (IsPostProcessViewBeforeWarpBlendRequired(PostprocessInstance))
	{
		return true;
	}

	if (IsPostProcessViewAfterWarpBlendRequired(PostprocessInstance))
	{
		return true;
	}

	if (IsPostProcessFrameAfterWarpBlendRequired(PostprocessInstance))
	{
		return true;
	}

	return false;
}

bool FDisplayClusterViewportPostProcessManager::ShouldUseAdditionalFrameTargetableResource() const
{
	check(IsInGameThread());

	if (OutputRemap.IsValid() && OutputRemap->IsEnabled())
	{
		return true;
	}

	for (const TSharedPtr<IDisplayClusterPostProcess, ESPMode::ThreadSafe >& It : Postprocess)
	{
		if (It.IsValid() && It->ShouldUseAdditionalFrameTargetableResource())
		{
			return IsAnyPostProcessRequired(It);
		}
	}

	return false;
}

bool FDisplayClusterViewportPostProcessManager::ShouldUseFullSizeFrameTargetableResource() const
{
	// Viewport remap required full-size Frame texture
	if (OutputRemap.IsValid() && OutputRemap->IsEnabled())
	{
		return true;
	}

	// Frame postprocess required full size
	for (const TSharedPtr<IDisplayClusterPostProcess, ESPMode::ThreadSafe >& It : Postprocess)
	{
		if (It.IsValid() && It->IsPostProcessFrameAfterWarpBlendRequired())
		{
			return true;
		}
	}

	return false;
}

void FDisplayClusterViewportPostProcessManager::Tick()
{
	check(IsInGameThread());

	for (TSharedPtr<IDisplayClusterPostProcess, ESPMode::ThreadSafe >& It : Postprocess)
	{
		if(It.IsValid() && IsAnyPostProcessRequired(It))
		{
			It->Tick();
		}
	}
}

void FDisplayClusterViewportPostProcessManager::HandleSetupNewFrame()
{
	IDisplayClusterViewportManager* ViewportManager = Configuration->GetViewportManager();
	if (ViewportManager && CVarCustomPPEnabled.GetValueOnGameThread() != 0)
	{
		for (const TSharedPtr<IDisplayClusterPostProcess, ESPMode::ThreadSafe >& It : Postprocess)
		{
			It->HandleSetupNewFrame(ViewportManager);
		}
	}
}

void FDisplayClusterViewportPostProcessManager::HandleBeginNewFrame(FDisplayClusterRenderFrame& InOutRenderFrame)
{
	IDisplayClusterViewportManager* ViewportManager = Configuration->GetViewportManager();
	if (ViewportManager && CVarCustomPPEnabled.GetValueOnGameThread() != 0)
	{
		for (const TSharedPtr<IDisplayClusterPostProcess, ESPMode::ThreadSafe >& It : Postprocess)
		{
			It->HandleBeginNewFrame(ViewportManager, InOutRenderFrame);
		}
	}
}

void FDisplayClusterViewportPostProcessManager::FinalizeNewFrame()
{
	check(IsInGameThread());

	ENQUEUE_RENDER_COMMAND(DisplayClusterViewportPostProcessManager_FinalizeNewFrame)(
		[PostprocessManager = SharedThis(this), PostprocessData = Postprocess](FRHICommandListImmediate& RHICmdList)
	{
		PostprocessManager->PostprocessProxy.Empty();
		PostprocessManager->PostprocessProxy.Append(PostprocessData);
	});
}

void FDisplayClusterViewportPostProcessManager::HandleRenderFrameSetup_RenderThread(FRHICommandListImmediate& RHICmdList, const FDisplayClusterViewportManagerProxy* InViewportManagerProxy)
{
	if (CVarCustomPPEnabled.GetValueOnRenderThread() != 0)
	{
		for (const TSharedPtr<IDisplayClusterPostProcess, ESPMode::ThreadSafe >& It : PostprocessProxy)
		{
			It->HandleRenderFrameSetup_RenderThread(RHICmdList, InViewportManagerProxy);
		}
	}
}

void FDisplayClusterViewportPostProcessManager::HandleBeginUpdateFrameResources_RenderThread(FRHICommandListImmediate& RHICmdList, const FDisplayClusterViewportManagerProxy* InViewportManagerProxy)
{
	if (CVarCustomPPEnabled.GetValueOnRenderThread() != 0)
	{
		for (const TSharedPtr<IDisplayClusterPostProcess, ESPMode::ThreadSafe >& It : PostprocessProxy)
		{
			It->HandleBeginUpdateFrameResources_RenderThread(RHICmdList, InViewportManagerProxy);
		}
	}
}

void FDisplayClusterViewportPostProcessManager::HandleUpdateFrameResourcesAfterWarpBlend_RenderThread(FRHICommandListImmediate& RHICmdList, const IDisplayClusterViewportManagerProxy* InViewportManagerProxy)
{
	if (CVarCustomPPEnabled.GetValueOnRenderThread() != 0)
	{
		for (const TSharedPtr<IDisplayClusterPostProcess, ESPMode::ThreadSafe >& It : PostprocessProxy)
		{
			It->HandleUpdateFrameResourcesAfterWarpBlend_RenderThread(RHICmdList, InViewportManagerProxy);
		}
	}
}

void FDisplayClusterViewportPostProcessManager::HandleEndUpdateFrameResources_RenderThread(FRHICommandListImmediate& RHICmdList, const FDisplayClusterViewportManagerProxy* InViewportManagerProxy)
{
	if (CVarCustomPPEnabled.GetValueOnRenderThread() != 0)
	{
		for (const TSharedPtr<IDisplayClusterPostProcess, ESPMode::ThreadSafe >& It : PostprocessProxy)
		{
			It->HandleEndUpdateFrameResources_RenderThread(RHICmdList, InViewportManagerProxy);
		}
	}
}

void FDisplayClusterViewportPostProcessManager::PerformPostProcessViewBeforeWarpBlend_RenderThread(FRHICommandListImmediate& RHICmdList, const FDisplayClusterViewportManagerProxy* InViewportManagerProxy) const
{
	const bool bIsCustomPPEnabled = (CVarCustomPPEnabled.GetValueOnRenderThread() != 0);

	// Post-process before warp&blend
	if (bIsCustomPPEnabled)
	{
		// PP round 1: post-process for each view region before warp&blend
		ImplPerformPostProcessViewBeforeWarpBlend_RenderThread(RHICmdList, InViewportManagerProxy);
	}
}

void FDisplayClusterViewportPostProcessManager::PerformPostProcessViewAfterWarpBlend_RenderThread(FRHICommandListImmediate& RHICmdList, const FDisplayClusterViewportManagerProxy* InViewportManagerProxy) const
{
	const bool bIsCustomPPEnabled = (CVarCustomPPEnabled.GetValueOnRenderThread() != 0);

	// Post-process after warp&blend
	if (bIsCustomPPEnabled)
	{
		// PP round 4: post-process for each view region after warp&blend
		ImplPerformPostProcessViewAfterWarpBlend_RenderThread(RHICmdList, InViewportManagerProxy);
	}
}

void FDisplayClusterViewportPostProcessManager::PerformPostProcessFrameAfterWarpBlend_RenderThread(FRHICommandListImmediate& RHICmdList, const FDisplayClusterViewportManagerProxy* InViewportManagerProxy) const
{
	const bool bIsCustomPPEnabled = (CVarCustomPPEnabled.GetValueOnRenderThread() != 0);

	// Post-process after warp&blend
	if (bIsCustomPPEnabled)
	{
		// PP round 5: post-process for each eye frame after warp&blend
		ImplPerformPostProcessFrameAfterWarpBlend_RenderThread(RHICmdList, InViewportManagerProxy);
	}
}

//////////////////////////////////////////////////////////////////////////////////////////////
// Round 1: VIEW before warp&blend
//////////////////////////////////////////////////////////////////////////////////////////////
void FDisplayClusterViewportPostProcessManager::ImplPerformPostProcessViewBeforeWarpBlend_RenderThread(FRHICommandListImmediate& RHICmdList, const FDisplayClusterViewportManagerProxy* InViewportManagerProxy) const
{
	const bool bEnabled = (CVarPostprocessViewBeforeWarpBlend.GetValueOnRenderThread() != 0);
	UE_LOGF(LogDisplayClusterRender, VeryVerbose, "Postprocess VIEW before WarpBlend: %d", bEnabled ? 1 : 0);

	if (bEnabled && InViewportManagerProxy)
	{
		for (const TSharedPtr<IDisplayClusterPostProcess, ESPMode::ThreadSafe >& It : PostprocessProxy)
		{
			if (IsPostProcessViewBeforeWarpBlendRequired(It))
			{
				// send nullptr viewport first to handle begin
				It->PerformPostProcessViewBeforeWarpBlend_RenderThread(RHICmdList, nullptr);

				for (TSharedPtr<IDisplayClusterViewportProxy, ESPMode::ThreadSafe>& ViewportProxyIt : InViewportManagerProxy->GetViewports_RenderThread())
				{
					if (ViewportProxyIt.IsValid())
					{
						UE_LOGF(LogDisplayClusterRender, VeryVerbose, "Postprocess VIEW before WarpBlend - Viewport '%ls'", *ViewportProxyIt->GetId());
						It->PerformPostProcessViewBeforeWarpBlend_RenderThread(RHICmdList, ViewportProxyIt.Get());
					}
				}
			}
		}
	}
}

//////////////////////////////////////////////////////////////////////////////////////////////
// Round 2: VIEW after warp&blend
//////////////////////////////////////////////////////////////////////////////////////////////
void FDisplayClusterViewportPostProcessManager::ImplPerformPostProcessViewAfterWarpBlend_RenderThread(FRHICommandListImmediate& RHICmdList, const FDisplayClusterViewportManagerProxy* InViewportManagerProxy) const
{
	const bool bEnabled = (CVarPostprocessViewAfterWarpBlend.GetValueOnRenderThread() != 0);
	UE_LOGF(LogDisplayClusterRender, VeryVerbose, "Postprocess VIEW after WarpBlend: %d", bEnabled ? 1 : 0);

	if (bEnabled != 0 && InViewportManagerProxy)
	{
		for (const TSharedPtr<IDisplayClusterPostProcess, ESPMode::ThreadSafe >& It : PostprocessProxy)
		{
			if (IsPostProcessViewAfterWarpBlendRequired(It))
			{
				// send nullptr viewport first to handle begin
				It->PerformPostProcessViewAfterWarpBlend_RenderThread(RHICmdList, nullptr);

				for (const TSharedPtr<IDisplayClusterViewportProxy, ESPMode::ThreadSafe>& ViewportProxyIt : InViewportManagerProxy->GetViewports_RenderThread())
				{
					if (ViewportProxyIt.IsValid())
					{
						UE_LOGF(LogDisplayClusterRender, VeryVerbose, "Postprocess VIEW after WarpBlend - Viewport '%ls'", *ViewportProxyIt->GetId());
						It->PerformPostProcessViewAfterWarpBlend_RenderThread(RHICmdList, ViewportProxyIt.Get());
					}
				}
			}
		}
	}
}

//////////////////////////////////////////////////////////////////////////////////////////////
// Round 3: FRAME after warp&blend
//////////////////////////////////////////////////////////////////////////////////////////////
void FDisplayClusterViewportPostProcessManager::ImplPerformPostProcessFrameAfterWarpBlend_RenderThread(FRHICommandListImmediate& RHICmdList, const FDisplayClusterViewportManagerProxy* InViewportManagerProxy) const
{
	if (GDisplayClusterPostprocessFrameAfterWarpBlendEnable && InViewportManagerProxy)
	{
		TArray<FRHITexture*> FrameResources;
		TArray<FRHITexture*> TempFrameResources;
		TArray<FIntPoint> TargetOffset;
		if (InViewportManagerProxy->GetFrameTargets_RenderThread(FrameResources, TargetOffset, &TempFrameResources))
		{
			// This is expected to be the way the postprocess is implemented:
			// 1. The 'FrameResources' is used as input and output data.
			// 2. The 'AdditionalFrameResources' is a temporary RTT that can be provided by nDisplay to PP if requested by the PP party.
			// 3. The 'AdditionalFrameResources' RTT will be cleared before each PP (if the GDisplayClusterPostProcessFrameAfterWarpBlendAlwaysClearRTT is set)

			for (const TSharedPtr<IDisplayClusterPostProcess, ESPMode::ThreadSafe >& PostProcessProxyIt : PostprocessProxy)
			{
				if (IsPostProcessFrameAfterWarpBlendRequired(PostProcessProxyIt))
				{
					TArray<FRHITexture*>* TempFrameResourcesPtr = (!TempFrameResources.IsEmpty() && PostProcessProxyIt->ShouldUseAdditionalFrameTargetableResource()) ? &TempFrameResources : nullptr;

					UE_LOGF(LogDisplayClusterRender, VeryVerbose, "Perform PostProcessFrameAfterWarpBlend '%ls'", *PostProcessProxyIt->GetId());

					if (GDisplayClusterPostProcessFrameAfterWarpBlendAlwaysClearRTT && TempFrameResourcesPtr)
					{
						// Always perform RTT cleanup before each full-frame post-processing.
						for (FRHITexture* FrameResourceIt : *TempFrameResourcesPtr)
						{
							FDisplayClusterViewportProxy::FillTextureWithColor_RenderThread(RHICmdList, FrameResourceIt, FLinearColor::Black);
						}
					}

					PostProcessProxyIt->PerformPostProcessFrameAfterWarpBlend_RenderThread(RHICmdList, &FrameResources, TempFrameResourcesPtr);
				}
			}

			// Apply OutputRemap after all postprocess. The TempFrameResources is required.
			if (OutputRemap.IsValid() && OutputRemap->IsEnabled() && !TempFrameResources.IsEmpty())
			{
				if (GDisplayClusterPostProcessFrameAfterWarpBlendAlwaysClearRTT)
				{
					// Always perform RTT cleanup before each full-frame post-processing.
					for (FRHITexture* FrameResourceIt : TempFrameResources)
					{
						FDisplayClusterViewportProxy::FillTextureWithColor_RenderThread(RHICmdList, FrameResourceIt, FLinearColor::Black);
					}
				}

				OutputRemap->PerformPostProcessFrame_RenderThread(RHICmdList, &FrameResources, &TempFrameResources);
			}
		}
	}
}
