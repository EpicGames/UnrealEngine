// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Render/PostProcess/IDisplayClusterPostProcess.h"
#include "Render/IDisplayClusterRenderManager.h"
#include "Templates/SharedPointer.h"

class FDisplayClusterViewportManagerProxy;
class FDisplayClusterViewportPostProcessOutputRemap;
class FDisplayClusterViewportConfiguration;
struct FDisplayClusterConfigurationPostprocess;

/**
 * Helper class to collect post-process code and to easy the FDisplayClusterDeviceBase
 */

class FDisplayClusterViewportPostProcessManager
	: public TSharedFromThis<FDisplayClusterViewportPostProcessManager, ESPMode::ThreadSafe>
{
public:
	FDisplayClusterViewportPostProcessManager(const TSharedRef<const FDisplayClusterViewportConfiguration, ESPMode::ThreadSafe>& InConfiguration);
	virtual ~FDisplayClusterViewportPostProcessManager();

	void Release_GameThread();

public:
	bool IsPostProcessViewBeforeWarpBlendRequired(const TSharedPtr<IDisplayClusterPostProcess, ESPMode::ThreadSafe>& PostprocessInstance) const;
	bool IsPostProcessViewAfterWarpBlendRequired(const TSharedPtr<IDisplayClusterPostProcess, ESPMode::ThreadSafe>& PostprocessInstance) const;
	bool IsPostProcessFrameAfterWarpBlendRequired(const TSharedPtr<IDisplayClusterPostProcess, ESPMode::ThreadSafe>& PostprocessInstance) const;

	bool IsAnyPostProcessRequired(const TSharedPtr<IDisplayClusterPostProcess, ESPMode::ThreadSafe>& PostprocessInstance) const;

	bool ShouldUseAdditionalFrameTargetableResource() const;
	bool ShouldUseFullSizeFrameTargetableResource() const;

	void PerformPostProcessViewBeforeWarpBlend_RenderThread(FRHICommandListImmediate& RHICmdList, const FDisplayClusterViewportManagerProxy* InViewportManagerProxy) const;
	void PerformPostProcessViewAfterWarpBlend_RenderThread(FRHICommandListImmediate& RHICmdList, const FDisplayClusterViewportManagerProxy* InViewportManagerProxy) const;
	void PerformPostProcessFrameAfterWarpBlend_RenderThread(FRHICommandListImmediate& RHICmdList, const FDisplayClusterViewportManagerProxy* InViewportManagerProxy) const;

	void Tick();

	bool OnHandleStartScene();
	void OnHandleEndScene();

	void HandleSetupNewFrame();
	void HandleBeginNewFrame(FDisplayClusterRenderFrame& InOutRenderFrame);

	void HandleRenderFrameSetup_RenderThread(FRHICommandListImmediate& RHICmdList, const FDisplayClusterViewportManagerProxy* InViewportManagerProxy);

	void HandleBeginUpdateFrameResources_RenderThread(FRHICommandListImmediate& RHICmdList, const FDisplayClusterViewportManagerProxy* InViewportManagerProxy);
	void HandleUpdateFrameResourcesAfterWarpBlend_RenderThread(FRHICommandListImmediate& RHICmdList, const IDisplayClusterViewportManagerProxy* InViewportManagerProxy);
	void HandleEndUpdateFrameResources_RenderThread(FRHICommandListImmediate& RHICmdList, const FDisplayClusterViewportManagerProxy* InViewportManagerProxy);

	// Send data to render thread
	void FinalizeNewFrame();

	TSharedPtr<FDisplayClusterViewportPostProcessOutputRemap, ESPMode::ThreadSafe>& GetOutputRemap()
	{ return OutputRemap; }

public:
	/** Special rules for creating a PP instance.
	* Note: This function must be called before Create() on the nDisplay side.
	* (useful to avoid unnecessary nDisplay log messages).
	*
	* For example, some postprocess types may use an instance as a singleton.
	* In this case, Create() returns nullptr if the PP instance already exists,
	* and as a result nDisplay issues an error message to the log, and then avoids creating this PP in the future.
	*
	* @param InConfigurationPostProcess - postprocess configuration
	*
	* @return true if a postprocess with this type can be created at this time.
	*/
	bool CanBeCreated(const FDisplayClusterConfigurationPostprocess* InConfigurationPostprocess) const;

	bool CreatePostprocess(const FString& InPostprocessId, const FDisplayClusterConfigurationPostprocess* InConfigurationPostprocess);
	bool RemovePostprocess(const FString& InPostprocessId);
	bool UpdatePostprocess(const FString& InPostprocessId, const FDisplayClusterConfigurationPostprocess* InConfigurationPostprocess);

	const TArray<FString> GetPostprocess() const;
	TSharedPtr<IDisplayClusterPostProcess, ESPMode::ThreadSafe> FindPostProcess(const FString& InPostprocessId) const
	{
		return ImplFindPostProcess(InPostprocessId);
	}

private:
	TSharedPtr<IDisplayClusterPostProcess, ESPMode::ThreadSafe> ImplFindPostProcess(const FString& InPostprocessId) const;

protected:
	void ImplPerformPostProcessViewBeforeWarpBlend_RenderThread(FRHICommandListImmediate& RHICmdList, const FDisplayClusterViewportManagerProxy* InViewportManagerProxy) const;
	void ImplPerformPostProcessViewAfterWarpBlend_RenderThread(FRHICommandListImmediate& RHICmdList,  const FDisplayClusterViewportManagerProxy* InViewportManagerProxy) const;
	void ImplPerformPostProcessFrameAfterWarpBlend_RenderThread(FRHICommandListImmediate& RHICmdList, const FDisplayClusterViewportManagerProxy* InViewportManagerProxy) const;

public:
	// Configuration of the current cluster node
	const TSharedRef<const FDisplayClusterViewportConfiguration, ESPMode::ThreadSafe> Configuration;

private:
	// Game thread instances
	TArray<TSharedPtr<IDisplayClusterPostProcess, ESPMode::ThreadSafe>> Postprocess;

	// Render thread instances
	TArray<TSharedPtr<IDisplayClusterPostProcess, ESPMode::ThreadSafe>> PostprocessProxy;

	// Output remap implementation
	TSharedPtr<FDisplayClusterViewportPostProcessOutputRemap, ESPMode::ThreadSafe> OutputRemap;
};
