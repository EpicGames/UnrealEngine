// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Viewport/Postprocess/DisplayClusterViewportPostProcessOutputRemap.h"

#include "DisplayClusterConfigurationTypes_OutputRemap.h"

#include "Misc/DisplayClusterHelpers.h"
#include "Misc/DisplayClusterLog.h"
#include "Misc/DisplayClusterGlobals.h"
#include "Misc/Paths.h"

#include "DisplayClusterRootActor.h"

#include "Render/IDisplayClusterRenderManager.h"

#include "Render/Containers/IDisplayClusterRender_MeshComponent.h"
#include "Render/Containers/IDisplayClusterRender_MeshComponentProxy.h"
#include "Render/Containers/DisplayClusterRender_MeshGeometry.h"

#include "Render/Viewport/IDisplayClusterViewportManager.h"

#include "IDisplayClusterShaders.h"

#include "Engine/RendererSettings.h"
#include "Engine/StaticMesh.h"
#include "ProceduralMeshComponent.h"

//////////////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterViewportPostProcessOutputRemap
//////////////////////////////////////////////////////////////////////////////////////////////
bool FDisplayClusterViewportPostProcessOutputRemap::ImplInitializeOutputRemap()
{
	if (!OutputRemapMesh.IsValid())
	{
		OutputRemapMesh = IDisplayCluster::Get().GetRenderMgr()->CreateMeshComponent();
	}

	// Reset stored values
	ExternalFile.Empty();

	return OutputRemapMesh.IsValid();
}

bool FDisplayClusterViewportPostProcessOutputRemap::UpdateConfiguration_ExternalFile(const FString& InExternalFile)
{
	check(IsInGameThread());

	// Empty input filename -> disable output remap
	if (InExternalFile.IsEmpty())
	{
		UpdateConfiguration_Disabled();

		return false;
	}

	// Support related paths
	FString FullPathFileName = DisplayClusterHelpers::filesystem::GetFullPathForConfigResource(InExternalFile);

	// Don't update twice
	if (ExternalFile.Equals(FullPathFileName, ESearchCase::IgnoreCase))
	{
		return IsEnabled();
	}

	if (!FPaths::FileExists(FullPathFileName))
	{
		UE_LOGF(LogDisplayClusterViewport, Error, "OutputRemap - Failed to find file '%ls'", *FullPathFileName);
		UpdateConfiguration_Disabled();

		// Store input filename for next update
		ExternalFile = FullPathFileName;

		return false;
	}

	// Try load geometry from file
	FDisplayClusterRender_MeshGeometry MeshGeometry;
	if (!MeshGeometry.LoadFromFile(FullPathFileName))
	{
		UE_LOGF(LogDisplayClusterViewport, Error, "OutputRemap - Failed to load ext mesh from file '%ls'", *FullPathFileName);
		UpdateConfiguration_Disabled();

		// Store input filename for next update
		ExternalFile = FullPathFileName;
	}
	else if (ImplInitializeOutputRemap())
	{
		// Assign loaded geometry
		OutputRemapMesh->SetGeometryFunc(EDisplayClusterRender_MeshComponentProxyDataFunc::OutputRemapScreenSpace);
		OutputRemapMesh->AssignMeshGeometry(&MeshGeometry);

		// Store input filename for next update
		ExternalFile = FullPathFileName;

		return true;
	}

	return false;
}

bool FDisplayClusterViewportPostProcessOutputRemap::UpdateConfiguration_StaticMesh(UStaticMesh* InStaticMesh)
{
	check(IsInGameThread());

	// nullptr as input - disable
	if (InStaticMesh == nullptr)
	{
		UpdateConfiguration_Disabled();

		return false;
	}

	// don't update twice
	if (OutputRemapMesh.IsValid() && !OutputRemapMesh->IsMeshComponentRefGeometryDirty())
	{
		const UStaticMesh* AssignedStaticMesh = OutputRemapMesh->GetStaticMesh();
		if (AssignedStaticMesh == InStaticMesh)
		{
			return IsEnabled();
		}
	}

	if (ImplInitializeOutputRemap())
	{
		OutputRemapMesh->SetGeometryFunc(EDisplayClusterRender_MeshComponentProxyDataFunc::OutputRemapScreenSpace);
		OutputRemapMesh->AssignStaticMesh(InStaticMesh, FDisplayClusterMeshUVs());

		return true;
	}

	return false;
}

bool FDisplayClusterViewportPostProcessOutputRemap::UpdateConfiguration_StaticMeshComponent(UStaticMeshComponent* InStaticMeshComponent)
{
	check(IsInGameThread());

	// nullptr as input - disable
	if (InStaticMeshComponent == nullptr)
	{
		UpdateConfiguration_Disabled();

		return false;
	}

	// don't update twice
	if (OutputRemapMesh.IsValid() && !OutputRemapMesh->IsMeshComponentRefGeometryDirty())
	{
		const UStaticMeshComponent* AssignedStaticMeshComponent = OutputRemapMesh->GetStaticMeshComponent();
		if (AssignedStaticMeshComponent == InStaticMeshComponent)
		{
			return IsEnabled();
		}
	}

	// Begin new configuration:
	if (ImplInitializeOutputRemap())
	{
		OutputRemapMesh->SetGeometryFunc(EDisplayClusterRender_MeshComponentProxyDataFunc::OutputRemapScreenSpace);
		OutputRemapMesh->AssignStaticMeshComponentRefs(InStaticMeshComponent, FDisplayClusterMeshUVs());

		return true;
	}

	return false;
}

bool FDisplayClusterViewportPostProcessOutputRemap::UpdateConfiguration_ProceduralMeshComponent(UProceduralMeshComponent* InProceduralMeshComponent)
{
	check(IsInGameThread());

	// nullptr as input - disable
	if (InProceduralMeshComponent == nullptr)
	{
		UpdateConfiguration_Disabled();

		return false;
	}

	// don't update twice
	if (OutputRemapMesh.IsValid() && !OutputRemapMesh->IsMeshComponentRefGeometryDirty())
	{
		const UProceduralMeshComponent* AssignedProceduralMeshComponent = OutputRemapMesh->GetProceduralMeshComponent();
		if (AssignedProceduralMeshComponent == InProceduralMeshComponent)
		{
			return IsEnabled();
		}
	}

	// Begin new configuration:
	if (ImplInitializeOutputRemap())
	{
		OutputRemapMesh->SetGeometryFunc(EDisplayClusterRender_MeshComponentProxyDataFunc::OutputRemapScreenSpace);
		OutputRemapMesh->AssignProceduralMeshComponentRefs(InProceduralMeshComponent, FDisplayClusterMeshUVs());

		return true;
	}

	return false;
}

void FDisplayClusterViewportPostProcessOutputRemap::UpdateConfiguration_Disabled()
{
	check(IsInGameThread());

	ExternalFile.Empty();
	OutputRemapMesh.Reset();
}

bool FDisplayClusterViewportPostProcessOutputRemap::MarkProceduralMeshComponentGeometryDirty(const FName& InComponentName)
{
	// Mark the procedural mesh component as dirty
	if (OutputRemapMesh.IsValid() && OutputRemapMesh->GetGeometrySource() == EDisplayClusterRender_MeshComponentGeometrySource::ProceduralMeshComponentRef)
	{
		if (InComponentName == NAME_None || OutputRemapMesh->EqualsMeshComponentName(InComponentName))
		{
			OutputRemapMesh->MarkMeshComponentRefGeometryDirty();
			return true;
		}
	}

	return false;
}

void FDisplayClusterViewportPostProcessOutputRemap::PerformPostProcessFrame_RenderThread(FRHICommandListImmediate& RHICmdList, const TArray<FRHITexture*>* InFrameTargets, const TArray<FRHITexture*>* InAdditionalFrameTargets) const
{
	check(IsInRenderingThread());

	if (InFrameTargets && InAdditionalFrameTargets && OutputRemapMesh.IsValid())
	{
		const IDisplayClusterRender_MeshComponentProxy* MeshProxy = OutputRemapMesh->GetMeshComponentProxy_RenderThread();
		if (MeshProxy!=nullptr && MeshProxy->IsEnabled_RenderThread())
		{
			for (int32 Index = 0; Index < InFrameTargets->Num(); Index++)
			{
				FRHITexture* InOutTexture = (*InFrameTargets)[Index];
				FRHITexture* TempTargetableTexture = (*InAdditionalFrameTargets)[Index];

				static IDisplayClusterShaders& ShadersAPI = IDisplayClusterShaders::Get();
				if (ShadersAPI.RenderPostprocess_OutputRemap(RHICmdList, InOutTexture, TempTargetableTexture, *MeshProxy))
				{
					TransitionAndCopyTexture(RHICmdList, TempTargetableTexture, InOutTexture, {});
				}
			}
		}
	}
}
