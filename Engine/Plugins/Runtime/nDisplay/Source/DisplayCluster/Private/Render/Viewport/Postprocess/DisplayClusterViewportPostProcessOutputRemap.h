// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHI.h"
#include "RHICommandList.h"
#include "RHIResources.h"
#include "RHIUtilities.h"

class IDisplayClusterViewportManager;
class IDisplayClusterRender_MeshComponent;
class UStaticMesh;
class UStaticMeshComponent;
class UProceduralMeshComponent;

/**
 * Display Cluster Viewport PostProcess OutputRemap
 */
class FDisplayClusterViewportPostProcessOutputRemap
{
public:
	// Game thread update calls
	bool UpdateConfiguration_ExternalFile(const FString& InExternalFile);
	bool UpdateConfiguration_StaticMesh(UStaticMesh* InStaticMesh);
	bool UpdateConfiguration_StaticMeshComponent(UStaticMeshComponent* InStaticMeshComponent);
	bool UpdateConfiguration_ProceduralMeshComponent(UProceduralMeshComponent* InProceduralMeshComponent);
	void UpdateConfiguration_Disabled();

	bool MarkProceduralMeshComponentGeometryDirty(const FName& InComponentName);

	bool IsEnabled() const
	{
		return OutputRemapMesh.IsValid();
	}

public:
	void PerformPostProcessFrame_RenderThread(FRHICommandListImmediate& RHICmdList, const TArray<FRHITexture*>* InFrameTargets = nullptr, const TArray<FRHITexture*>* InAdditionalFrameTargets = nullptr) const;

private:
	bool ImplInitializeOutputRemap();

private:
	// Name of the external file used for remapping. Empty if not used.
	FString ExternalFile;

	// Reference to the mesh component that is used for remapping.
	TSharedPtr<IDisplayClusterRender_MeshComponent, ESPMode::ThreadSafe> OutputRemapMesh;
};
