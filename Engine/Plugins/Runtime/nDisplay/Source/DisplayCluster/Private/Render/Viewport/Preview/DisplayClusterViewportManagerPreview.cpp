// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Viewport/Preview/DisplayClusterViewportManagerPreview.h"
#include "Render/Viewport/Preview/DisplayClusterViewportPreview.h"
#include "Render/Viewport/Preview/DisplayClusterViewportManagerPreviewRendering.h"
#include "Render/Viewport/Configuration/DisplayClusterViewportConfiguration.h"

#include "Render/Viewport/DisplayClusterViewportManager.h"

#include "DisplayClusterRootActor.h"
#include "DisplayClusterConfigurationTypes.h"

#include "Misc/DisplayClusterLog.h"

////////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterViewportManagerPreview
////////////////////////////////////////////////////////////////////////////////////////
FDisplayClusterViewportManagerPreview::FDisplayClusterViewportManagerPreview(const TSharedRef<FDisplayClusterViewportConfiguration, ESPMode::ThreadSafe>& InConfiguration)
	: Configuration(InConfiguration)
{ }

FString FDisplayClusterViewportManagerPreview::GetClusterNodeId() const
{
	if (const UDisplayClusterConfigurationData* CurrentConfigData = Configuration->GetConfigurationData())
	{
		TArray<FString> ExistClusterNodesIDs;
		CurrentConfigData->Cluster->GetNodeIds(ExistClusterNodesIDs);
		if (!ExistClusterNodesIDs.IsEmpty())
		{
			// Iterate over cluster nodes:
			int32 NodeIndex = ExistClusterNodesIDs.Find(Configuration->GetClusterNodeId());
			if (NodeIndex == INDEX_NONE)
			{
				// Begin rendering cycle
				NodeIndex = 0;
			}
			else if (!PreviewRenderFrame.IsValid())
			{
				// When the PreviewRenderFrame variable is freed, it means that rendering of the current cluster node is complete.
				// Iterate cluster nodes
				NodeIndex++;
			}

			// next loop
			if (!ExistClusterNodesIDs.IsValidIndex(NodeIndex))
			{
				NodeIndex = 0;

				// When rendering of all nodes in the cluster is complete, notify parent.
				if (FDisplayClusterViewportManager* ViewportManager = Configuration->GetViewportManagerImpl())
				{
					ViewportManager->OnEntireClusterPreviewRendered();
				}

				// When rendering of all nodes in the cluster is complete, generate events for subscribers
				OnEntireClusterPreviewGenerated.ExecuteIfBound();
			}

			return ExistClusterNodesIDs[NodeIndex];
		}
	}

	return TEXT("");
}

void FDisplayClusterViewportManagerPreview::ResetEntireClusterPreviewRendering()
{
	PreviewRenderFrame.Reset();
	bEntireClusterRendered = false;
}

void FDisplayClusterViewportManagerPreview::OnPostRenderPreviewTick()
{
	// Render ICVFX frustum
	if (Configuration->GetPreviewSettings().bPreviewICVFXFrustums)
	{
		RenderPreviewFrustums();
	}
}

void FDisplayClusterViewportManagerPreview::OnPreviewRenderTick()
{
	ADisplayClusterRootActor* SceneRootActor = Configuration->GetRootActor(EDisplayClusterRootActorType::Scene);
	UWorld* CurrentWorld = SceneRootActor ? SceneRootActor->GetWorld() : nullptr;
	if (!CurrentWorld)
	{
		// Scene DCRA and World is required for rendering preview
		return;
	}

	// Special case for RootActorProxy object (proxy always refers to external DCRA in the scene)
	ADisplayClusterRootActor* RootActorProxy = Configuration->GetRootActor(EDisplayClusterRootActorType::Preview);
	if (RootActorProxy && RootActorProxy != SceneRootActor)
	{
		// Since this DCRA is a proxy and the DCRA on the scene is a separate object,
		// it does not pass component positions and properties from the scene to the proxy.
		// When the cluster preview was rendered for RootActorProxy, the component properties from
		// the scene were already retrieved via the EDisplayClusterRootActorType::Scene reference.
		// This was done via a link to the RootActor in the scene, which means the RootActorProxy components did not change.
		// But RootActorProxy is used in custom preview rendering, meaning it uses its own mesh components,
		// so their positions must be synchronized with the scene components.
		//
		// Move the RootActorProxy components to the same relative position as the components inside the RootActor in the scene.
		RootActorProxy->ForEachComponent<UPrimitiveComponent>(true, [&](UPrimitiveComponent* ProxyComponent)
			{
				if (!IsValid(ProxyComponent))
				{
					return;
				}

				SceneRootActor->ForEachComponent<UPrimitiveComponent>(true, [&](UPrimitiveComponent* SceneComponent)
					{
						if (!IsValid(SceneComponent))
						{
							return;
						}

						const FName SceneComponentName = SceneComponent->GetFName();
						const FName ProxyComponentName = ProxyComponent->GetFName();

						// Propagate the component transformation from the scene to the proxy.
						if (SceneComponentName == ProxyComponentName)
						{
							const FTransform NewRelativeTransform = SceneComponent->GetRelativeTransform();
							const FTransform OldRelativeTransform = ProxyComponent->GetRelativeTransform();
							if (!NewRelativeTransform.Equals(OldRelativeTransform, UE_KINDA_SMALL_NUMBER))
							{
								ProxyComponent->SetRelativeTransform(NewRelativeTransform);
							}
						}
					});
			});
	}

	const FDisplayClusterRenderFrameSettings& RenderFrameSetting = Configuration->GetRenderFrameSettings();

	// Update preview RTTs correspond to 'TickPerFrame' value
	if (++TickPerFrameCounter < RenderFrameSetting.PreviewSettings.TickPerFrame)
	{
		return;
	}
	TickPerFrameCounter = 0;

	int32 ViewportsAmmount = RenderFrameSetting.PreviewSettings.ViewportsPerFrame;
	int32 CycleDepth = 0;
	FString FirstClusterNodeId;

	while (ViewportsAmmount > 0)
	{
		// The function GetClusterNodeId() returns an empty string if no nodes are found to render.
		const FString CurrentClusterNodeId = GetClusterNodeId();
		if (CurrentClusterNodeId.IsEmpty())
		{
			// No cluster nodes found.
			Release();

			return;
		}

		if (FirstClusterNodeId.IsEmpty())
		{
			FirstClusterNodeId = CurrentClusterNodeId;
		}

		// The cluster node name has been changed to the first value in the loop
		if (CycleDepth++ && !PreviewRenderFrame.IsValid() && FirstClusterNodeId == CurrentClusterNodeId)
		{
			// protect from overrun, when user set ViewportsPerFrame to big value
			break;
		}

		// Experimental
		if (CycleDepth > 100)
		{
			// No more than 100 nodes per tick
			return;
		}

		const FString& CurrentClusterNodeIdCfg = Configuration->GetClusterNodeId();
		if (CurrentClusterNodeIdCfg != CurrentClusterNodeId || !PreviewRenderFrame.IsValid())
		{
			// Initialize new cluster node for render
			if (!InitializeClusterNodePreview(Configuration->GetRenderFrameSettings().PreviewSettings.EntireClusterPreviewRenderMode, CurrentWorld, CurrentClusterNodeId, nullptr))
			{
				// this cluster node cannot be initialized, skip and try the next node
				continue;
			}
		}

		// Render viewports
		ViewportsAmmount = RenderClusterNodePreview(ViewportsAmmount);

		// Call this function after rendering to immediately perform
		// OnEntireClusterPreviewGenerated() after rendering of all cluster nodes is complete.
		GetClusterNodeId();
	}
}

void FDisplayClusterViewportManagerPreview::Update()
{
	// This set is passed to the preview update of each viewport so that they can avoid fighting to control
	// the same mesh component by knowing if a previous viewport is already in control of it.
	TSet<const UMeshComponent*> MeshComponentsVisited;

	for (const TSharedPtr<FDisplayClusterViewportPreview, ESPMode::ThreadSafe>& ViewportPreviewIt : GetEntireClusterPreviewViewportsImpl())
	{
		if (ViewportPreviewIt.IsValid())
		{
			ViewportPreviewIt->Update(MeshComponentsVisited);
		}
	}
}

void FDisplayClusterViewportManagerPreview::Release()
{
	for (const TSharedPtr<FDisplayClusterViewportPreview, ESPMode::ThreadSafe>& ViewportPreviewIt : GetEntireClusterPreviewViewportsImpl())
	{
		if (ViewportPreviewIt.IsValid())
		{
			ViewportPreviewIt->Release();
		}
	}

	ResetEntireClusterPreviewRendering();
}

const TArray<TSharedPtr<IDisplayClusterViewportPreview, ESPMode::ThreadSafe>> FDisplayClusterViewportManagerPreview::GetEntireClusterPreviewViewports() const
{
	// Convert type from FDisplayClusterViewportPreview to IDisplayClusterViewportPreview:
	// Note: try to find existing macros for TArray to perform this type conversion operation in a single line
	// we cannot use TArrayView because a local variable was used as the result.
	TArray<TSharedPtr<FDisplayClusterViewportPreview, ESPMode::ThreadSafe>> InViewports = GetEntireClusterPreviewViewportsImpl();
	TArray<TSharedPtr<IDisplayClusterViewportPreview, ESPMode::ThreadSafe>> OutViewports;

	OutViewports.Reserve(InViewports.Num());
	for (TSharedPtr<FDisplayClusterViewportPreview, ESPMode::ThreadSafe>& InViewportIt : InViewports)
	{
		OutViewports.Add(InViewportIt);
	}

	return OutViewports;
}

const TArray<TSharedPtr<FDisplayClusterViewportPreview, ESPMode::ThreadSafe>> FDisplayClusterViewportManagerPreview::GetEntireClusterPreviewViewportsImpl() const
{
	TArray<TSharedPtr<FDisplayClusterViewportPreview, ESPMode::ThreadSafe>> OutViewports;

	if (FDisplayClusterViewportManager* ViewportManager = Configuration->GetViewportManagerImpl())
	{
		for (const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& ViewportIt : ViewportManager->ImplGetEntireClusterViewports())
		{
			if (ViewportIt.IsValid() && ViewportIt->IsResourceUsed(EDisplayClusterViewportResource::OutputPreviewTargetableResources))
			{
				OutViewports.Add(ViewportIt->ViewportPreview);
			}
		}
	}

	return OutViewports;
}

void FDisplayClusterViewportManagerPreview::RegisterPreviewRendering()
{
	// Preview rendering depends on the DC VM
	FDisplayClusterViewportManagerPreviewRenderingSingleton::HandleEvent(EDisplayClusteViewportManagerPreviewRenderingEvent::Create, this);
}

void FDisplayClusterViewportManagerPreview::UnregisterPreviewRendering()
{
	// Preview rendering depends on the  DC VM
	FDisplayClusterViewportManagerPreviewRenderingSingleton::HandleEvent(EDisplayClusteViewportManagerPreviewRenderingEvent::Remove, this);

}

void FDisplayClusterViewportManagerPreview::UpdateEntireClusterPreviewRender(bool bEnablePreviewRendering)
{
	if (bEnablePreviewRendering)
	{
		FDisplayClusterViewportManagerPreviewRenderingSingleton::HandleEvent(EDisplayClusteViewportManagerPreviewRenderingEvent::Render, this);
		bEntireClusterPreview = true;
	}
	else if (bEntireClusterPreview)
	{
		FDisplayClusterViewportManagerPreviewRenderingSingleton::HandleEvent(EDisplayClusteViewportManagerPreviewRenderingEvent::Stop, this);

		ResetEntireClusterPreviewRendering();

		// Release current configuration
		Configuration->ReleaseConfiguration();

		bEntireClusterPreview = false;
	}
}
