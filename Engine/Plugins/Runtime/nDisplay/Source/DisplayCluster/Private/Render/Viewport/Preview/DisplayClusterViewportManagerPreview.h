// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Render/Viewport/IDisplayClusterViewportManagerPreview.h"
#include "Render/Viewport/Configuration/DisplayClusterViewportConfiguration.h"

#include "SceneView.h"

#include "Templates/SharedPointer.h"

class FDisplayClusterViewportPreview;
class FDisplayClusterRenderFrame;

/**
* Store and manage preview resources of the viewport
*/
class FDisplayClusterViewportManagerPreview
	: public IDisplayClusterViewportManagerPreview
	, public TSharedFromThis<FDisplayClusterViewportManagerPreview, ESPMode::ThreadSafe>
{
public:
	FDisplayClusterViewportManagerPreview(const TSharedRef<FDisplayClusterViewportConfiguration, ESPMode::ThreadSafe>& InConfiguration);
	virtual ~FDisplayClusterViewportManagerPreview() = default;

public:
	//~BEGIN IDisplayClusterViewportManagerPreview
	virtual TSharedPtr<IDisplayClusterViewportManagerPreview, ESPMode::ThreadSafe> ToSharedPtr() override
	{
		return AsShared();
	}

	virtual TSharedPtr<const IDisplayClusterViewportManagerPreview, ESPMode::ThreadSafe> ToSharedPtr() const override
	{
		return AsShared();
	}

	virtual IDisplayClusterViewportConfiguration& GetConfiguration() override
	{
		return Configuration.Get();
	}

	virtual const IDisplayClusterViewportConfiguration& GetConfiguration() const override
	{
		return Configuration.Get();
	}

	virtual void UpdateEntireClusterPreviewRender(bool bEnablePreviewRendering) override;
	virtual void ResetEntireClusterPreviewRendering() override;

	virtual bool InitializeClusterNodePreview(const EDisplayClusterRenderFrameMode InRenderMode, UWorld* InWorld, const FString& InClusterNodeId, FViewport* InViewport) override;
	virtual int32 RenderClusterNodePreview(const int32 InViewportsAmmount, FViewport* InViewport = nullptr, FCanvas* SceneCanvas = nullptr) override;

	virtual const TArray<TSharedPtr<IDisplayClusterViewportPreview, ESPMode::ThreadSafe>> GetEntireClusterPreviewViewports() const override;

	virtual FOnOnClusterNodePreviewGenerated& GetOnClusterNodePreviewGenerated() override
	{
		return OnClusterNodePreviewGenerated;
	}

	virtual FOnOnEntireClusterPreviewGenerated& GetOnEntireClusterPreviewGenerated() override
	{
		return 	OnEntireClusterPreviewGenerated;
	}
	//~~END IDisplayClusterViewportManagerPreview

public:
	/** Register this class with the preview rendering pipeline. */
	void RegisterPreviewRendering();

	/** Remove this class from the preview rendering pipeline. */
	void UnregisterPreviewRendering();

	/** Rendering multiple viewports from the entire cluster. These numbers are in the preview settings. */
	void OnPreviewRenderTick();

	/** Rendering of special debug objects after preview (Frustum, etc.). */
	void OnPostRenderPreviewTick();

	/** Update viewports preview. */
	void Update();

	/** Release viewports preview. */
	void Release();

protected:
	/** Retrieve viewport preview instances for the entire cluster. */
	const TArray<TSharedPtr<FDisplayClusterViewportPreview, ESPMode::ThreadSafe>> GetEntireClusterPreviewViewportsImpl() const;

	/** Render frustum for viewport. */
	void RenderFrustumPreviewForViewport(IDisplayClusterViewport* InViewport);

	/** Getting the cluster node ID for preview. */
	FString GetClusterNodeId() const;

	/** Render preview frustums. */
	void RenderPreviewFrustums();

	/** Render ViewFamily with viewport.	*/
	bool RenderViewportPreview(FViewport* InViewport, FCanvas* InSceneCanvas);

public:
	// Configuration of the current cluster node
	const TSharedRef<FDisplayClusterViewportConfiguration, ESPMode::ThreadSafe> Configuration;

private:
	FOnOnEntireClusterPreviewGenerated OnEntireClusterPreviewGenerated;
	FOnOnClusterNodePreviewGenerated OnClusterNodePreviewGenerated;

	// Is entire cluster preview in scene is used
	bool bEntireClusterPreview = false;

	// true when entire cluster was rendered
	bool bEntireClusterRendered = false;

	// The OnPreviewRenderTick() function may be executed only once per multiple frames depending on the settings.
	int32 TickPerFrameCounter = 0;

	// This render frame created for a cluster node
	TUniquePtr<FDisplayClusterRenderFrame> PreviewRenderFrame;
};
