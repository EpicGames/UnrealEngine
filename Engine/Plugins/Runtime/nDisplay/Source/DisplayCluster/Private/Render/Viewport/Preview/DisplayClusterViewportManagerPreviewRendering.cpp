// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Viewport/Preview/DisplayClusterViewportManagerPreviewRendering.h"
#include "Render/Viewport/Preview/DisplayClusterViewportManagerPreview.h"

#include "Engine/World.h"
#include "Engine/GameViewportClient.h"
#include "HAL/IConsoleManager.h"

///////////////////////////////////////////////////////////////////
int32 GDisplayClusterViewportManagerPreviewRenderingEnable = 1;
static FAutoConsoleVariableRef CVarDisplayClusterRootActorPreviewRenderingEnable(
	TEXT("nDisplay.preview.render.enable"),
	GDisplayClusterViewportManagerPreviewRenderingEnable,
	TEXT("Globally enable DCRA preview rendering\n")
	TEXT("0 : Disable preview render \n"),
	ECVF_Default
);

int32 GDisplayClusterViewportManagerPreviewRenderingOptimization = 1;
static FAutoConsoleVariableRef CVarDisplayClusterRootActorPreviewRenderingOptimization(
	TEXT("nDisplay.preview.render.optimize"),
	GDisplayClusterViewportManagerPreviewRenderingOptimization,
	TEXT("Optimize DCRA preview rendering only 1 DCRA per frame\n")
	TEXT("0 : Disable optimization\n"),
	ECVF_Default
);

///////////////////////////////////////////////////////////////////
namespace UE::DisplayCluster::ViewportManagerPreview
{
	static FDisplayClusterViewportManagerPreviewRenderingSingleton& GetViewportManagerPreviewRenderingSingleton()
	{
		static FDisplayClusterViewportManagerPreviewRenderingSingleton ViewportManagerPreviewRenderingSingleton;

		return ViewportManagerPreviewRenderingSingleton;
	}
};
using namespace UE::DisplayCluster::ViewportManagerPreview;

void FDisplayClusterViewportManagerPreviewRenderingInstance::HandleRenderRequest()
{
	if (!GDisplayClusterViewportManagerPreviewRenderingEnable)
	{
		return;
	}

	if (GDisplayClusterViewportManagerPreviewRenderingOptimization)
	{
		// Request for preview render
		bRenderPreviewRequested = true;
	}
	else
	{
		RenderPreview();
	}

	PostRenderPreviewTick();
}

void FDisplayClusterViewportManagerPreviewRenderingInstance::RenderPreview() const
{
	if (FDisplayClusterViewportManagerPreview* ViewportManagerPreview = GetViewportManagerPreview())
	{
		ViewportManagerPreview->OnPreviewRenderTick();
	}
}

void FDisplayClusterViewportManagerPreviewRenderingInstance::PostRenderPreviewTick() const
{
	if (FDisplayClusterViewportManagerPreview* ViewportManagerPreview = GetViewportManagerPreview())
	{
		ViewportManagerPreview->OnPostRenderPreviewTick();
	}
}
//------------------------------------------------------------------
// FDisplayClusterViewportManagerPreviewRenderingSingleton
//------------------------------------------------------------------
void FDisplayClusterViewportManagerPreviewRenderingSingleton::HandleEvent(const EDisplayClusteViewportManagerPreviewRenderingEvent InPreviewEvent, FDisplayClusterViewportManagerPreview* InViewportManagerPreview)
{
	GetViewportManagerPreviewRenderingSingleton().HandleEventImpl(InPreviewEvent, InViewportManagerPreview);
}

void FDisplayClusterViewportManagerPreviewRenderingSingleton::HandleEventImpl(const EDisplayClusteViewportManagerPreviewRenderingEvent InPreviewEvent, FDisplayClusterViewportManagerPreview* InViewportManagerPreview)
{
	const int32 PreviewObjectIndex = PreviewObjects.IndexOfByPredicate([InViewportManagerPreview](const FDisplayClusterViewportManagerPreviewRenderingInstance& PreviewObjectIt)
		{
			return PreviewObjectIt.IsEqual(InViewportManagerPreview);
		});

	switch (InPreviewEvent)
	{
	case EDisplayClusteViewportManagerPreviewRenderingEvent::Create:
		if (PreviewObjectIndex == INDEX_NONE)
		{
			PreviewObjects.Add(FDisplayClusterViewportManagerPreviewRenderingInstance(InViewportManagerPreview->AsShared()));
		}
		break;

	case EDisplayClusteViewportManagerPreviewRenderingEvent::Remove:
		if (PreviewObjectIndex != INDEX_NONE)
		{
			PreviewObjects.RemoveAt(PreviewObjectIndex);
		}
		break;

	case EDisplayClusteViewportManagerPreviewRenderingEvent::Stop:
		if (PreviewObjectIndex != INDEX_NONE)
		{
			PreviewObjects[PreviewObjectIndex].HandleStopRenderRequest();
		}
		break;

	case EDisplayClusteViewportManagerPreviewRenderingEvent::Render:
		if (PreviewObjectIndex != INDEX_NONE)
		{
			PreviewObjects[PreviewObjectIndex].HandleRenderRequest();
		}
		else
		{
			// Create new ref by render request
			const int32 NewIndex = PreviewObjects.Add(FDisplayClusterViewportManagerPreviewRenderingInstance(InViewportManagerPreview->AsShared()));
			PreviewObjects[NewIndex].HandleRenderRequest();
		}
		break;

	default:
		break;
	}

	UpdateTickableGameObject();
}

void FDisplayClusterViewportManagerPreviewRenderingSingleton::UpdateTickableGameObject()
{
	if (PreviewObjects.IsEmpty())
	{
		TickableGameObject.Reset();
	}
	else
	{
		if (!TickableGameObject.IsValid())
		{
			TickableGameObject = MakeUnique<FDisplayClusterTickableGameObject >();
			TickableGameObject->OnTick().AddRaw(this, &FDisplayClusterViewportManagerPreviewRenderingSingleton::Tick);
		}
	}
}

void FDisplayClusterViewportManagerPreviewRenderingSingleton::Tick(float DeltaTime)
{
	if (!GDisplayClusterViewportManagerPreviewRenderingOptimization || !GDisplayClusterViewportManagerPreviewRenderingEnable)
	{
		return;
	}

	// Render one DCRA per frame:
	int32 NextPreviewObjectIndex = GetNextPreviewObject();
	if (NextPreviewObjectIndex == INDEX_NONE)
	{
		// Reset render cycle
		BeginNewRenderCycle();

		// start new cycle
		NextPreviewObjectIndex = GetNextPreviewObject();
	}

	if (NextPreviewObjectIndex != INDEX_NONE)
	{
		// Render this DCRA
		PreviewObjects[NextPreviewObjectIndex].RenderPreview();
	}
}

int32 FDisplayClusterViewportManagerPreviewRenderingSingleton::GetNextPreviewObject()
{
	int32 OutIndex = INDEX_NONE;

	// rebuild list - remove invalid DCRA's
	TArray<FDisplayClusterViewportManagerPreviewRenderingInstance> NewPreviewObjects;
	NewPreviewObjects.Reserve(PreviewObjects.Num());

	for (FDisplayClusterViewportManagerPreviewRenderingInstance& PreviewObjectIt : PreviewObjects)
	{
		if (PreviewObjectIt.GetViewportManagerPreview())
		{
			if (OutIndex == INDEX_NONE && PreviewObjectIt.ShouldRenderPreview())
			{
				// Mark this object as rendered
				PreviewObjectIt.MarkAsRendered();

				OutIndex = NewPreviewObjects.Num();
			}

			NewPreviewObjects.Add(PreviewObjectIt);
		}
	}

	PreviewObjects = NewPreviewObjects;

	return OutIndex;
}

void FDisplayClusterViewportManagerPreviewRenderingSingleton::BeginNewRenderCycle()
{
	for (FDisplayClusterViewportManagerPreviewRenderingInstance& PreviewObject : PreviewObjects)
	{
		if (PreviewObject.ShouldRenderPreview())
		{
			// This object was not render in the current loop
			return;
		}
	}

	for (FDisplayClusterViewportManagerPreviewRenderingInstance& PreviewObject : PreviewObjects)
	{
		// Allow to render again
		PreviewObject.MarkAsNotRendered();
	}
}
