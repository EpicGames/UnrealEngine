// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "Misc/DisplayClusterTickableGameObject.h"
#include "Engine/EngineTypes.h"

class FDisplayClusterViewportManagerPreview;

/**
 * Preview manager receive events from DCRAs
 */
enum class EDisplayClusteViewportManagerPreviewRenderingEvent : uint8
{
	// Viewport manager preview was created
	Create = 0,

	// Viewport manager preview was removed
	Remove,

	// Request from viewport manager for one time preview rendering.
	// This event must be called many times to render the entire cluster.
	Render,

	// Request from viewport manager for stop preview rendering.
	Stop
};

/**
 * Container for ViewportManagerPreview instances that are used for preview rendering.
 */
class FDisplayClusterViewportManagerPreviewRenderingInstance
{
public:
	FDisplayClusterViewportManagerPreviewRenderingInstance() = default;
	FDisplayClusterViewportManagerPreviewRenderingInstance(const TSharedRef<FDisplayClusterViewportManagerPreview, ESPMode::ThreadSafe>& InViewportManagerPreview)
		: ViewportManagerPreviewWeakPtr(InViewportManagerPreview)
	{ }

public:
	/** Returns true if the ViewportManager preview can be performed right now. */
	inline bool ShouldRenderPreview() const
	{
		return bRenderPreviewRequested && !bPreviewRenderingCompleted;
	}

	/** Changes the internal flags of this ViewportManager container to a state indicating that the preview rendering is complete. */
	inline void MarkAsRendered()
	{
		bRenderPreviewRequested = false;
		bPreviewRenderingCompleted = true;
	}

	/** Changes the internal flags of this ViewportManager container to a state that allows render previews. */
	inline void MarkAsNotRendered()
	{
		bPreviewRenderingCompleted = false;
	}

	/** Returns true if the InRootActor is the same as the one used inside this container. */
	inline bool IsEqual(const FDisplayClusterViewportManagerPreview* InViewportManagerPreview) const
	{
		return GetViewportManagerPreview() == InViewportManagerPreview && InViewportManagerPreview;
	}

	/** Return the pointer to the ViewportManager preview. */
	inline FDisplayClusterViewportManagerPreview* GetViewportManagerPreview() const
	{
		return ViewportManagerPreviewWeakPtr.IsValid() ? ViewportManagerPreviewWeakPtr.Pin().Get() : nullptr;
	}

	/** When ViewportManager tries to call the preview rendering, this function will be called. */
	void HandleRenderRequest();

	void HandleStopRenderRequest()
	{
		// Clear request for preview render
		bRenderPreviewRequested = false;
	}

	/** Performs a preview of the rendering for ViewportManager in this container. */
	void RenderPreview() const;

	/** This function is called every tick after rendering the preview.*/
	void PostRenderPreviewTick() const;

private:
	// Weak pointer to the ViewportManager
	TWeakPtr<FDisplayClusterViewportManagerPreview, ESPMode::ThreadSafe> ViewportManagerPreviewWeakPtr;

	// Preview is already rendered for this actor
	bool bPreviewRenderingCompleted = false;

	// Preview render request from DCRA
	bool bRenderPreviewRequested = false;
};

/**
 * This is a singleton class that controls the preview rendering of multiple ViewportManager preview instances.
 */
class FDisplayClusterViewportManagerPreviewRenderingSingleton
{
public:
	/** The entry point for ViewportManager, which is used to handle all workflow events related to preview rendering. */
	static void HandleEvent(const EDisplayClusteViewportManagerPreviewRenderingEvent InPreviewEvent, FDisplayClusterViewportManagerPreview* InViewportManagerPreview);

protected:
	friend class FDisplayClusterPreviewTickableGameObject;

	/** This function is used to perform delayed ViewportManager preview rendering requests. */
	void Tick(float DeltaTime);

	/** Creating or deleting a Tickable object depends on whether the ViewportManagers list is not empty. */
	void UpdateTickableGameObject();

	/** Implementation of the static HandleEvent() function. */
	void HandleEventImpl(const EDisplayClusteViewportManagerPreviewRenderingEvent InPreviewEvent, FDisplayClusterViewportManagerPreview* InViewportManagerPreview);

	/** Gets the index of the next ViewportManager container, which can be used for rendering the preview. */
	int32  GetNextPreviewObject();

	/** When all registered DCRA objects are drawn, you must call this function to reset the internal flags. */
	void BeginNewRenderCycle();

private:
	// List of existing ViewportManagers
	TArray<FDisplayClusterViewportManagerPreviewRenderingInstance> PreviewObjects;

	// When CachedObjects is not empty, this ticking object will be created.
	// Also, this object will be deleted when CachedObjects becomes empty.
	TUniquePtr<FDisplayClusterTickableGameObject> TickableGameObject;
};
