// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Viewport/Preview/DisplayClusterViewportManagerPreview.h"
#include "Render/Viewport/Preview/DisplayClusterViewportPreview.h"
#include "Render/Viewport/Preview/DisplayClusterViewportManagerPreviewRendering.h"

#include "Render/Viewport/Configuration/DisplayClusterViewportConfiguration.h"
#include "Render/Viewport/Configuration/DisplayClusterViewportConfigurationHelpers_ICVFX.h"

#include "Render/Viewport/DisplayClusterViewportManager.h"

#include "Components/LineBatchComponent.h"

#include "DisplayClusterRootActor.h"
#include "DisplayClusterConfigurationTypes.h"

#include "Render/Viewport/DisplayClusterViewportManager.h"
#include "Render/Viewport/DisplayClusterViewportManagerProxy.h"

#include "EngineModule.h"
#include "CanvasTypes.h"
#include "LegacyScreenPercentageDriver.h"

#include "SceneView.h"
#include "SceneViewExtension.h"

#include "Engine/Scene.h"
#include "GameFramework/WorldSettings.h"

#include "Render/Projection/IDisplayClusterProjectionPolicy.h"
#include "Render/Viewport/DisplayClusterViewport.h"
#include "Render/Viewport/DisplayClusterViewport_CustomPostProcessSettings.h"
#include "Render/Viewport/Resource/DisplayClusterViewportResource.h"

#include "Render/Viewport/RenderFrame/DisplayClusterRenderFrame.h"
#include "Render/Viewport/RenderFrame/DisplayClusterRenderFrameSettings.h"

#include "Render/Viewport/Configuration/DisplayClusterViewportConfiguration.h"

#include "Misc/DisplayClusterLog.h"
#include "DisplayClusterRootActor.h" 

#include "ClearQuad.h"

////////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterViewportManagerPreview
////////////////////////////////////////////////////////////////////////////////////////
void FDisplayClusterViewportManagerPreview::RenderPreviewFrustums()
{
	FDisplayClusterViewportManager* ViewportManager = Configuration->GetViewportManagerImpl();
	const UDisplayClusterConfigurationData* CurrentConfigData = Configuration->GetConfigurationData();

	if (CurrentConfigData == nullptr || !ViewportManager)
	{
		return;
	}

	for (const TPair<FString, TObjectPtr<UDisplayClusterConfigurationClusterNode>>& Node : CurrentConfigData->Cluster->Nodes)
	{
		if (Node.Value == nullptr)
		{
			continue;
		}

		// collect node viewports
		for (const TPair<FString, TObjectPtr<UDisplayClusterConfigurationViewport>>& ViewportConfig : Node.Value->Viewports)
		{
			if (ViewportConfig.Value->bAllowPreviewFrustumRendering == false
#if WITH_EDITOR
			|| ViewportConfig.Value->bIsVisible == false
#endif
			|| ViewportConfig.Value == nullptr)
			{
				continue;
			}

			RenderFrustumPreviewForViewport(ViewportManager->FindViewport(ViewportConfig.Key));
		}
	}

	// collect incameras
	if (Configuration->GetRenderFrameSettings().IsPreviewRendering() && Configuration->GetRenderFrameSettings().PreviewSettings.bPreviewICVFXFrustums)
	{
		// Iterate over rendered inner camera viewports (whole cluster)
		for (const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& InnerCameraViewportIt : FDisplayClusterViewportConfigurationHelpers_ICVFX::GetAllVisibleInnerCameraViewports(*Configuration))
		{
			RenderFrustumPreviewForViewport(InnerCameraViewportIt.Get());
		}
	}
}

void FDisplayClusterViewportManagerPreview::RenderFrustumPreviewForViewport(IDisplayClusterViewport* InViewport)
{
	if (InViewport == nullptr || InViewport->GetContexts().IsEmpty() || InViewport->GetRenderSettings().bEnable == false)
	{
		return;
	}

	// Preview rendered only in mono
	const TArray<FDisplayClusterViewport_Context>& Contexts = InViewport->GetContexts();
	if (!(Contexts.Num() == 1 && Contexts[0].bValidData))
	{
		return;
	}

	const FMatrix ProjectionMatrix = Contexts[0].ProjectionMatrix;

	FMatrix ViewRotationMatrix = FInverseRotationMatrix(Contexts[0].ViewRotation) * FMatrix(
		FPlane(0, 0, 1, 0),
		FPlane(1, 0, 0, 0),
		FPlane(0, 1, 0, 0),
		FPlane(0, 0, 0, 1));
	const FMatrix ViewMatrix = FTranslationMatrix(-Contexts[0].ViewLocation) * ViewRotationMatrix;


	const FVector ViewOrigin = Contexts[0].ViewLocation;

	const float FarPlane = Configuration->GetRenderFrameSettings().PreviewSettings.PreviewICVFXFrustumsFarDistance;
	const float NearPlane = GNearClippingPlane;
	const FColor Color = FColor::Green;
	const float Thickness = 1.0f;

	ADisplayClusterRootActor* SceneRootActor = Configuration->GetRootActor(EDisplayClusterRootActorType::Scene);
	ULineBatchComponent* LineBatcher = SceneRootActor ? SceneRootActor->GetLineBatchComponent() : nullptr;
	if (!LineBatcher)
	{
		return;
	}

	// Get FOV and AspectRatio from the view's projection matrix.
	const float AspectRatio = ProjectionMatrix.M[1][1] / ProjectionMatrix.M[0][0];
	const bool bIsPerspectiveProjection = true;

	// Build the camera frustum for this cascade
	const float HalfHorizontalFOV = bIsPerspectiveProjection ? FMath::Atan(1.0f / ProjectionMatrix.M[0][0]) : PI / 4.0f;
	const float HalfVerticalFOV = bIsPerspectiveProjection ? FMath::Atan(1.0f / ProjectionMatrix.M[1][1]) : FMath::Atan((FMath::Tan(PI / 4.0f) / AspectRatio));
	const float AsymmetricFOVScaleX = ProjectionMatrix.M[2][0];
	const float AsymmetricFOVScaleY = ProjectionMatrix.M[2][1];

	// Near plane
	const float StartHorizontalTotalLength = NearPlane * FMath::Tan(HalfHorizontalFOV);
	const float StartVerticalTotalLength = NearPlane * FMath::Tan(HalfVerticalFOV);
	const FVector StartCameraLeftOffset = ViewMatrix.GetColumn(0) * -StartHorizontalTotalLength * (1 + AsymmetricFOVScaleX);
	const FVector StartCameraRightOffset = ViewMatrix.GetColumn(0) * StartHorizontalTotalLength * (1 - AsymmetricFOVScaleX);
	const FVector StartCameraBottomOffset = ViewMatrix.GetColumn(1) * -StartVerticalTotalLength * (1 + AsymmetricFOVScaleY);
	const FVector StartCameraTopOffset = ViewMatrix.GetColumn(1) * StartVerticalTotalLength * (1 - AsymmetricFOVScaleY);

	// Far plane
	const float EndHorizontalTotalLength = FarPlane * FMath::Tan(HalfHorizontalFOV);
	const float EndVerticalTotalLength = FarPlane * FMath::Tan(HalfVerticalFOV);
	const FVector EndCameraLeftOffset = ViewMatrix.GetColumn(0) * -EndHorizontalTotalLength * (1 + AsymmetricFOVScaleX);
	const FVector EndCameraRightOffset = ViewMatrix.GetColumn(0) * EndHorizontalTotalLength * (1 - AsymmetricFOVScaleX);
	const FVector EndCameraBottomOffset = ViewMatrix.GetColumn(1) * -EndVerticalTotalLength * (1 + AsymmetricFOVScaleY);
	const FVector EndCameraTopOffset = ViewMatrix.GetColumn(1) * EndVerticalTotalLength * (1 - AsymmetricFOVScaleY);

	const FVector CameraDirection = ViewMatrix.GetColumn(2);

	// Preview frustum vertices
	FVector PreviewFrustumVerts[8];

	// Get the 4 points of the camera frustum near plane, in world space
	PreviewFrustumVerts[0] = ViewOrigin + CameraDirection * NearPlane + StartCameraRightOffset + StartCameraTopOffset;         // 0 Near  Top    Right
	PreviewFrustumVerts[1] = ViewOrigin + CameraDirection * NearPlane + StartCameraRightOffset + StartCameraBottomOffset;      // 1 Near  Bottom Right
	PreviewFrustumVerts[2] = ViewOrigin + CameraDirection * NearPlane + StartCameraLeftOffset + StartCameraTopOffset;          // 2 Near  Top    Left
	PreviewFrustumVerts[3] = ViewOrigin + CameraDirection * NearPlane + StartCameraLeftOffset + StartCameraBottomOffset;       // 3 Near  Bottom Left

	// Get the 4 points of the camera frustum far plane, in world space
	PreviewFrustumVerts[4] = ViewOrigin + CameraDirection * FarPlane + EndCameraRightOffset + EndCameraTopOffset;         // 4 Far  Top    Right
	PreviewFrustumVerts[5] = ViewOrigin + CameraDirection * FarPlane + EndCameraRightOffset + EndCameraBottomOffset;      // 5 Far  Bottom Right
	PreviewFrustumVerts[6] = ViewOrigin + CameraDirection * FarPlane + EndCameraLeftOffset + EndCameraTopOffset;          // 6 Far  Top    Left
	PreviewFrustumVerts[7] = ViewOrigin + CameraDirection * FarPlane + EndCameraLeftOffset + EndCameraBottomOffset;       // 7 Far  Bottom Left	

	// frustum lines
	LineBatcher->DrawLine(PreviewFrustumVerts[0], PreviewFrustumVerts[4], Color, SDPG_World, Thickness, 0.f); // right top
	LineBatcher->DrawLine(PreviewFrustumVerts[1], PreviewFrustumVerts[5], Color, SDPG_World, Thickness, 0.f); // right bottom
	LineBatcher->DrawLine(PreviewFrustumVerts[2], PreviewFrustumVerts[6], Color, SDPG_World, Thickness, 0.f); // left top
	LineBatcher->DrawLine(PreviewFrustumVerts[3], PreviewFrustumVerts[7], Color, SDPG_World, Thickness, 0.f); // left bottom

	// near plane square
	LineBatcher->DrawLine(PreviewFrustumVerts[0], PreviewFrustumVerts[1], Color, SDPG_World, Thickness, 0.f); // right top to right bottom
	LineBatcher->DrawLine(PreviewFrustumVerts[1], PreviewFrustumVerts[3], Color, SDPG_World, Thickness, 0.f); // right bottom to left bottom
	LineBatcher->DrawLine(PreviewFrustumVerts[3], PreviewFrustumVerts[2], Color, SDPG_World, Thickness, 0.f); // left bottom to left top
	LineBatcher->DrawLine(PreviewFrustumVerts[2], PreviewFrustumVerts[0], Color, SDPG_World, Thickness, 0.f); // left top to right top

	// far plane square
	LineBatcher->DrawLine(PreviewFrustumVerts[4], PreviewFrustumVerts[5], Color, SDPG_World, Thickness, 0.f); // right top to right bottom
	LineBatcher->DrawLine(PreviewFrustumVerts[5], PreviewFrustumVerts[7], Color, SDPG_World, Thickness, 0.f); // right bottom to left bottom
	LineBatcher->DrawLine(PreviewFrustumVerts[7], PreviewFrustumVerts[6], Color, SDPG_World, Thickness, 0.f); // left bottom to left top
	LineBatcher->DrawLine(PreviewFrustumVerts[6], PreviewFrustumVerts[4], Color, SDPG_World, Thickness, 0.f); // left top to right top
}
