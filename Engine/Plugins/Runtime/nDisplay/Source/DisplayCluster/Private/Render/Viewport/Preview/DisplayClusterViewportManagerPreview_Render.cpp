// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Viewport/Preview/DisplayClusterViewportManagerPreview.h"
#include "Render/Viewport/Preview/DisplayClusterViewportPreview.h"
#include "Render/Viewport/Preview/DisplayClusterViewportManagerPreviewRendering.h"
#include "Render/Viewport/Configuration/DisplayClusterViewportConfiguration.h"

#include "Render/Viewport/DisplayClusterViewportManager.h"

#include "DisplayClusterRootActor.h"
#include "DisplayClusterConfigurationTypes.h"

#include "Render/Viewport/DisplayClusterViewportManager.h"
#include "Render/Viewport/DisplayClusterViewportManagerProxy.h"

#include "EngineModule.h"
#include "CanvasTypes.h"
#include "LegacyScreenPercentageDriver.h"

#include "SceneView.h"
#include "SceneViewExtension.h"

#include "Engine/Scene.h"
#include "GameFramework/WorldSettings.h"

#include "Render/Projection/IDisplayClusterProjectionPolicy.h"
#include "Render/Viewport/DisplayClusterViewport.h"
#include "Render/Viewport/DisplayClusterViewport_CustomPostProcessSettings.h"
#include "Render/Viewport/Resource/DisplayClusterViewportResource.h"

#include "Render/Viewport/RenderFrame/DisplayClusterRenderFrame.h"
#include "Render/Viewport/RenderFrame/DisplayClusterRenderFrameSettings.h"

#include "Render/Viewport/Configuration/DisplayClusterViewportConfiguration.h"

#include "Misc/DisplayClusterLog.h"
#include "DisplayClusterRootActor.h" 

#include "ClearQuad.h"

////////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterViewportManagerPreview
////////////////////////////////////////////////////////////////////////////////////////
int32 FDisplayClusterViewportManagerPreview::RenderClusterNodePreview(const int32 InViewportsAmmount, FViewport* InViewport, FCanvas* InSceneCanvas)
{
	if (!InViewportsAmmount)
	{
		return 0;
	}

	if (!PreviewRenderFrame.IsValid())
	{
		return InViewportsAmmount;
	}

	// InViewportsAmmount - Total ammount of viewports that should be rendered on this frame.
	// INDEX_NONE means render all viewports of this node
	int32 RenderedViewportsAmount = 0;
	while (RenderViewportPreview(InViewport, InSceneCanvas))
	{
		RenderedViewportsAmount++;

		// Stop if all viewports rendered
		if (InViewportsAmmount >= 0 && RenderedViewportsAmount >= InViewportsAmmount)
		{
			return 0;
		}
	}

	if (!PreviewRenderFrame.IsValid())
	{
		// If error, ignore that render frame
		return InViewportsAmmount;
	}

	// Entire cluster node rendered
	if (PreviewRenderFrame->RenderTargets.IsEmpty())
	{
		// Current node render is completed
		PreviewRenderFrame.Reset();
		RenderedViewportsAmount++;

		if (FDisplayClusterViewportManager* ViewportManager = Configuration->GetViewportManagerImpl())
		{
			// Handle special viewports game-thread logic at frame end
			// custom postprocess single frame flag must be removed at frame end on game thread
			ViewportManager->FinalizeNewFrame();

			// After all render target rendered call nDisplay frame rendering:
			ViewportManager->RenderFrame(InViewport);
		}

		// Send event about cluster node rendering is finished
		OnClusterNodePreviewGenerated.ExecuteIfBound(Configuration->GetClusterNodeId());
	}

	// Returns the remaining number of viewports to render.
	if (InViewportsAmmount >= 0)
	{
		return FMath::Max(0, InViewportsAmmount - RenderedViewportsAmount);
	}

	return 0;
}

bool FDisplayClusterViewportManagerPreview::InitializeClusterNodePreview(const EDisplayClusterRenderFrameMode InRenderMode, UWorld* InWorld, const FString& InClusterNodeId, FViewport* InViewport)
{
	check(InWorld);

	PreviewRenderFrame.Reset();

	if (FDisplayClusterViewportManager* ViewportManager = Configuration->GetViewportManagerImpl())
	{
		// Update preview configuration for cluster node
		DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FDisplayClusterViewportManagerPreview::BeginClusterNodeRendering"), STAT_DisplayClusterViewportManagerPreview_BeginClusterNodeRendering, STATGROUP_NDisplay);

		// Update local node viewports (update\create\delete) and build new render frame
		if (Configuration->UpdateConfigurationForClusterNode(InRenderMode, InWorld, InClusterNodeId))
		{
			// Build cluster node render frame
			PreviewRenderFrame = MakeUnique<FDisplayClusterRenderFrame>();
			if (ViewportManager->BeginNewFrame(InViewport, *PreviewRenderFrame))
			{
				// Initialize frame for render
				ViewportManager->InitializeNewFrame();

				// Update viewport preview instances if preview is used and DCRA supports previews:
				Update();

				return true;
			}
		}
	}

	PreviewRenderFrame.Reset();

	return false;
}

bool FDisplayClusterViewportManagerPreview::RenderViewportPreview(FViewport* InViewport, FCanvas* InSceneCanvas)
{
	/**
	* Uses PreviewRenderFrame as a container with viewports for rendering.
	* During rendering, modifies the PreviewRenderFrame to remove the processed viewport data.
	* When all viewports are rendered (bShouldRenderViewport=true), the final composition logic is called.
	* Finally, returns true, which signals the external function to initialize the PreviewRenderFrame for the next cluster node.
	*/

	FDisplayClusterViewportManager* ViewportManager = Configuration->GetViewportManagerImpl();
	UWorld* PreviewWorld = Configuration->GetCurrentWorld();
	if (!ViewportManager || !PreviewWorld)
	{
		// Remove invalid render frame
		PreviewRenderFrame.Reset();

		return false;
	}

	FSceneInterface* PreviewScene = PreviewWorld->Scene;
	FEngineShowFlags EngineShowFlags = FEngineShowFlags(EShowFlagInitMode::ESFIM_Game);

	while(!PreviewRenderFrame->RenderTargets.IsEmpty())
	{
		if (PreviewRenderFrame->RenderTargets[0].ViewFamilies.IsEmpty())
		{
			PreviewRenderFrame->RenderTargets.RemoveAt(0);
			continue;
		}

		// Special flag, allow clear RTT surface only for first family
		bool bAdditionalViewFamily = false;
		while(!PreviewRenderFrame->RenderTargets[0].ViewFamilies.IsEmpty())
		{
			if (PreviewRenderFrame->RenderTargets[0].ViewFamilies[0].Views.IsEmpty())
			{
				PreviewRenderFrame->RenderTargets[0].ViewFamilies.RemoveAt(0);
				continue;
			}

			// Process ViewFamily
			const FDisplayClusterRenderFrameTarget& RenderTargetIt = PreviewRenderFrame->RenderTargets[0];
			const FDisplayClusterRenderFrameTargetViewFamily& ViewFamiliesIt = PreviewRenderFrame->RenderTargets[0].ViewFamilies[0];

			// Create the view family for rendering the world scene to the viewport's render target
			FSceneViewFamilyContext ViewFamily = ViewportManager->CreateViewFamilyConstructionValues(
				RenderTargetIt,
				PreviewScene,
				EngineShowFlags,
				bAdditionalViewFamily
			);

			ViewportManager->ConfigureViewFamily(RenderTargetIt, ViewFamiliesIt, ViewFamily);

			if (RenderTargetIt.CaptureMode == EDisplayClusterViewportCaptureMode::Default && Configuration->GetRenderFrameSettings().IsPostProcessDisabled())
			{
				if (ViewFamily.EngineShowFlags.TemporalAA)
				{
					ViewFamily.EngineShowFlags.SetTemporalAA(false);
					ViewFamily.EngineShowFlags.SetAntiAliasing(true);
				}
			}

			// ViewState references that must stay alive until rendering is complete.
			// They are passed to the render thread to prevent premature destruction.
			TArray<TSharedPtr<FSceneViewStateReference, ESPMode::ThreadSafe>> ViewStatesToKeepAlive;

			TArray<FSceneView*> Views;
			for(const FDisplayClusterRenderFrameTargetView& ViewIt : ViewFamiliesIt.Views)
			{
				FDisplayClusterViewport* ViewportPtr = static_cast<FDisplayClusterViewport*>(ViewIt.Viewport.Get());
				if (!ViewportPtr)
				{
					continue;
				}

				check(ViewportPtr->GetContexts().IsValidIndex(ViewIt.ContextNum));

				// Always call CalcSceneView() because this function also starts VE and after that calls FDisplayClusterViewport::SetupSceneView() -> OCIO
				// uvLC viewport is not rendered, but late OCIO is used in FDisplayClusterViewportProxy::ApplyOCIO_RenderThread()
				TSharedPtr<FSceneViewStateReference, ESPMode::ThreadSafe> ReferencedViewState;
				FSceneView* View = ViewportPtr->ViewportPreview->CalcSceneView(ViewFamily, ViewIt.ContextNum, ReferencedViewState);

				// Remove the view that is not used for rendering.
				if (View && (!ViewIt.IsViewportContextCanBeRendered() || ViewFamily.RenderTarget == nullptr))
				{
					ViewFamily.Views.Remove(View);

					delete View;
					View = nullptr;
				}

				if (View)
				{
					Views.Add(View);

					// Keep the ViewState alive until rendering is complete.
					if (ReferencedViewState.IsValid())
					{
						ViewStatesToKeepAlive.Add(ReferencedViewState);
					}
				}
			}

			const bool bShouldRenderViewport = !ViewFamily.Views.IsEmpty();
			if (bShouldRenderViewport)
			{
				ViewportManager->PostConfigureViewFamily(RenderTargetIt, ViewFamiliesIt, ViewFamily, Views);

				if (InSceneCanvas)
				{
					GetRendererModule().BeginRenderingViewFamily(InSceneCanvas, &ViewFamily);
				}
				else
				{
					const ERHIFeatureLevel::Type FeatureLevel = PreviewWorld ? PreviewWorld->GetFeatureLevel() : GMaxRHIFeatureLevel;
					FCanvas Canvas((FRenderTarget*)ViewFamily.RenderTarget, nullptr, PreviewWorld, FeatureLevel, FCanvas::CDM_DeferDrawing /*FCanvas::CDM_ImmediateDrawing*/, 1.0f);
					Canvas.Clear(FLinearColor::Black);

					GetRendererModule().BeginRenderingViewFamily(&Canvas, &ViewFamily);
				}

				// Pass ViewState references to the render thread to keep them alive until rendering is complete.
				ENQUEUE_RENDER_COMMAND(FDisplayClusterViewportManagerPreview_KeepViewStatesAlive)(
					[ViewStatesToKeepAlive = MoveTemp(ViewStatesToKeepAlive)](FRHICommandListImmediate& RHICmdList) { });

				if (GNumExplicitGPUsForRendering > 1)
				{
					const FRHIGPUMask SubmitGPUMask = ViewFamily.Views.Num() == 1 ? ViewFamily.Views[0]->GPUMask : FRHIGPUMask::All();
					ENQUEUE_RENDER_COMMAND(UDisplayClusterViewportClient_SubmitCommandList)(
						[SubmitGPUMask](FRHICommandListImmediate& RHICmdList)
						{
							SCOPED_GPU_MASK(RHICmdList, SubmitGPUMask);
							RHICmdList.ImmediateFlush(EImmediateFlushType::DispatchToRHIThread);
						});
				}
			}

			// Remove processed Views from the FrameData.
			PreviewRenderFrame->RenderTargets[0].ViewFamilies[0].Views.Reset();

			// Return true if a single viewport was drawn.
			if (bShouldRenderViewport)
			{
				return true;
			}
		}
	}

	return false;
}