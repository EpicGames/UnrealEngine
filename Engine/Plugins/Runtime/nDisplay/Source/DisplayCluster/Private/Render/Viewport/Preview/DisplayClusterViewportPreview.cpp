// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Viewport/Preview/DisplayClusterViewportPreview.h"
#include "Render/Viewport/Resource/DisplayClusterViewportResource.h"
#include "Render/Viewport/DisplayClusterViewport.h"
#include "Render/Projection/IDisplayClusterProjectionPolicy.h"
#include "Render/Warp/IDisplayClusterWarpPolicy.h"

#include "Render/DisplayDevice/Components/DisplayClusterDisplayDeviceBaseComponent.h"
#include "Components/DisplayClusterCameraComponent.h"

#include "Components/StaticMeshComponent.h"

////////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterViewportPreview
////////////////////////////////////////////////////////////////////////////////////////
FDisplayClusterViewportPreview::FDisplayClusterViewportPreview(const TSharedRef<FDisplayClusterViewportConfiguration, ESPMode::ThreadSafe>& InConfiguration, const FString& InViewportId)
	: Configuration(InConfiguration)
	, ViewportId(InViewportId)
	, ClusterNodeId(InConfiguration->GetClusterNodeId())
	, PreviewMesh(EDisplayClusterDisplayDeviceMeshType::PreviewMesh, InConfiguration)
	, PreviewEditableMesh(EDisplayClusterDisplayDeviceMeshType::PreviewEditableMesh, InConfiguration)
{ }

FDisplayClusterViewportPreview::~FDisplayClusterViewportPreview()
{
	Release();
}

TSharedPtr<IDisplayClusterViewport, ESPMode::ThreadSafe> FDisplayClusterViewportPreview::GetOwningViewport() const
{
	return OwningViewportWeakPtr.Pin();
}

void FDisplayClusterViewportPreview::Initialize(FDisplayClusterViewport& InViewport)
{
	OwningViewportWeakPtr = InViewport.AsShared();
}

void FDisplayClusterViewportPreview::Release()
{
	PreviewRTT.Reset();
	RuntimeFlags = EDisplayClusterViewportPreviewFlags::None;

	// Keeps a reference to the owning viewport so that it is not deleted within this logic.
	TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe> OwningViewport = OwningViewportWeakPtr.Pin();

	PreviewMesh.Release(OwningViewport.Get());
	PreviewEditableMesh.Release(OwningViewport.Get());
}

void FDisplayClusterViewportPreview::Update(TSet<const UMeshComponent*>& MeshComponentsVisited)
{
	RuntimeFlags = EDisplayClusterViewportPreviewFlags::None;

	// Update viewport output RTT
	TSharedPtr<FDisplayClusterViewportResource, ESPMode::ThreadSafe> NewPreviewRTT = GetOutputPreviewTargetableResources();
	if (NewPreviewRTT != PreviewRTT)
	{
		EnumAddFlags(RuntimeFlags, EDisplayClusterViewportPreviewFlags::HasChangedPreviewRTT);
		PreviewRTT = NewPreviewRTT;
	}

	if (PreviewRTT.IsValid() && EnumHasAnyFlags(PreviewRTT->GetResourceState(), EDisplayClusterViewportResourceState::UpdatedOnRenderingThread))
	{
		// Update flags when preview RTT is valid
		EnumAddFlags(RuntimeFlags, EDisplayClusterViewportPreviewFlags::HasValidPreviewRTT);
	}

	// Don't update the meshes if any of them are being controlled by other viewports
	if (MeshComponentsVisited.Contains(PreviewMesh.GetMeshComponent()) || MeshComponentsVisited.Contains(PreviewEditableMesh.GetMeshComponent()))
	{
		return;
	}

	// Keeps a reference to the owning viewport so that it is not deleted within this logic.
	TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe> OwningViewport = OwningViewportWeakPtr.Pin();

	UDisplayClusterCameraComponent* ViewPointComponent = nullptr;
	UDisplayClusterDisplayDeviceBaseComponent* DisplayDeviceComponent = nullptr;

	if (OwningViewport.IsValid())
	{
		ViewPointComponent = OwningViewport->GetViewPointCameraComponent(EDisplayClusterRootActorType::Configuration);
		DisplayDeviceComponent = OwningViewport->GetDisplayDeviceComponent(Configuration->GetPreviewSettings().DisplayDeviceRootActorType);
	}
	
	// Utility function to update both PreviewMesh and PreviewEditableMesh
	auto UpdatePreviewMeshAndMaterial = [&](
		FDisplayClusterViewportPreviewMesh& PreviewMeshObj,
		EDisplayClusterViewportPreviewFlags ChangedMaterialFlag,
		EDisplayClusterDisplayDeviceMeshType InMeshType)
		{
			EDisplayClusterDisplayDeviceMeshType MeshType = InMeshType;
			if (!EnumHasAnyFlags(RuntimeFlags, EDisplayClusterViewportPreviewFlags::HasValidPreviewRTT))
			{
				// Use mesh without preview.
				MeshType = EDisplayClusterDisplayDeviceMeshType::DefaultMesh;
			}

			PreviewMeshObj.Update(OwningViewport.Get(), DisplayDeviceComponent, ViewPointComponent);

			// Update Runtime Flags
			if (PreviewMeshObj.HasAnyFlag(EDisplayClusterViewportPreviewMeshFlags::HasDeletedMaterialInstance | EDisplayClusterViewportPreviewMeshFlags::HasChangedMaterialInstance))
			{
				EnumAddFlags(RuntimeFlags, ChangedMaterialFlag);
			}

			// Record that these meshes have been updated so that other viewports don't try to.
			if (const UMeshComponent* MeshComponent = PreviewMeshObj.GetMeshComponent())
			{
				MeshComponentsVisited.Add(MeshComponent);
			}

			// Update the preview mesh and materials in the DisplayDevice component
			if (DisplayDeviceComponent)
			{
				DisplayDeviceComponent->OnUpdateDisplayDeviceMeshAndMaterialInstance(
					*this,
					MeshType,
					PreviewMeshObj.GetCurrentMaterialType(),
					PreviewMeshObj.GetMeshComponent(),
					PreviewMeshObj.GetMaterialInstance()
				);
			}

			// Update the preview mesh and materials in the ViewPointComponent component
			if (ViewPointComponent)
			{
				ViewPointComponent->OnUpdateDisplayDeviceMeshAndMaterialInstance(
					*this,
					MeshType,
					PreviewMeshObj.GetCurrentMaterialType(),
					PreviewMeshObj.GetMeshComponent(),
					PreviewMeshObj.GetMaterialInstance()
				);
			}
		};

	// Update PreviewMesh and PreviewEditableMesh

	UpdatePreviewMeshAndMaterial(
		PreviewMesh,                                                                // PreviewMeshObj
		EDisplayClusterViewportPreviewFlags::HasChangedPreviewMeshMaterialInstance, // ChangedMaterialFlag
		EDisplayClusterDisplayDeviceMeshType::PreviewMesh                           // MeshType
	);

	UpdatePreviewMeshAndMaterial(
		PreviewEditableMesh,                                                                // PreviewMeshObj
		EDisplayClusterViewportPreviewFlags::HasChangedPreviewEditableMeshMaterialInstance, // ChangedMaterialFlag
		EDisplayClusterDisplayDeviceMeshType::PreviewEditableMesh                           // MeshType
	);
}

TSharedPtr<FDisplayClusterViewportResource, ESPMode::ThreadSafe> FDisplayClusterViewportPreview::GetOutputPreviewTargetableResources() const
{
	// Keeps a reference to the owning viewport so that it is not deleted within this logic.
	TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe> OwningViewport = OwningViewportWeakPtr.Pin();
	if (OwningViewport.IsValid())
	{
		const TArray<TSharedPtr<FDisplayClusterViewportResource, ESPMode::ThreadSafe>>& PreviewResources = OwningViewport->GetViewportResources(EDisplayClusterViewportResource::OutputPreviewTargetableResources);
		if (!PreviewResources.IsEmpty())
		{
			return PreviewResources[0];
		}
	}

	return nullptr;
}

UTextureRenderTarget2D* FDisplayClusterViewportPreview::GetPreviewTextureRenderTarget2D() const
{
	if (PreviewRTT.IsValid())
	{
		return PreviewRTT->GetTextureRenderTarget2D();
	}

	return nullptr;
}
