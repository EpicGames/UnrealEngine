// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Render/Viewport/IDisplayClusterViewportPreview.h"
#include "Render/Viewport/Configuration/DisplayClusterViewportConfiguration.h"
#include "Render/Viewport/Preview/DisplayClusterViewportPreviewMesh.h"
#include "Render/Viewport/Containers/DisplayClusterViewportPreview_InternalEnums.h"

#include "Templates/SharedPointer.h"

class FDisplayClusterViewport;
class FDisplayClusterViewportPreviewMesh;
class FDisplayClusterViewportResource;

/**
* Store and manage preview resources of the viewport
*/
class FDisplayClusterViewportPreview
	: public IDisplayClusterViewportPreview
	, public TSharedFromThis<FDisplayClusterViewportPreview, ESPMode::ThreadSafe>
{
public:
	FDisplayClusterViewportPreview(const TSharedRef<FDisplayClusterViewportConfiguration, ESPMode::ThreadSafe>& InConfiguration, const FString& ViewportId);
	virtual ~FDisplayClusterViewportPreview();

public:
	//~ BEGIN IDisplayClusterViewportPreview
	virtual TSharedPtr<IDisplayClusterViewportPreview, ESPMode::ThreadSafe> ToSharedPtr() override
	{
		return AsShared();
	}

	virtual TSharedPtr<const IDisplayClusterViewportPreview, ESPMode::ThreadSafe> ToSharedPtr() const override
	{
		return AsShared();
	}

	virtual IDisplayClusterViewportConfiguration& GetConfiguration() override
	{
		return Configuration.Get();
	}

	virtual const IDisplayClusterViewportConfiguration& GetConfiguration() const override
	{
		return Configuration.Get();
	}

	virtual FString GetId() const override
	{
		return ViewportId;
	}

	virtual FString GetClusterNodeId() const override
	{
		return ClusterNodeId;
	}

	virtual TSharedPtr<IDisplayClusterViewport, ESPMode::ThreadSafe> GetOwningViewport() const override;

	virtual UTextureRenderTarget2D* GetPreviewTextureRenderTarget2D() const override;

	virtual UMeshComponent* GetPreviewMeshComponent() const override
	{
		return PreviewMesh.GetMeshComponent();
	}
	virtual UMeshComponent* GetPreviewEditableMeshComponent()  const override
	{
		return PreviewEditableMesh.GetMeshComponent();
	}

	virtual bool HasAnyFlags(const EDisplayClusterViewportPreviewFlags InPreviewFlags) const override
	{
		return EnumHasAnyFlags(RuntimeFlags, InPreviewFlags);
	}

	//~~  END IDisplayClusterViewportPreview

public:
	/** Update preview resources for this viewport. */
	void Update(TSet<const UMeshComponent*>& MeshComponentsVisited);

	/** Release preview resources and materials for this viewport. */
	void Release();

	/** Initialize reference to viewport. */
	void Initialize(FDisplayClusterViewport& InViewport);

	/**
	 * Creates an FSceneView for the viewport.
	 * This function is based on similar LocalPlayer logic.
	 *
	 * @param InOutViewFamily View family used to render this viewport.
	 * @param ContextNum      Rendering context index.
	 * @param OutViewState    Receives the ViewState used by the SceneView.
	 *                        This must be passed to the render thread to ensure it is not destroyed before rendering is complete.
	 *
	 * @return A pointer to the created FSceneView, or nullptr if creation fails.
	 */	
	FSceneView* CalcSceneView(class FSceneViewFamilyContext& InOutViewFamily, uint32 ContextNum,
		TSharedPtr<FSceneViewStateReference, ESPMode::ThreadSafe>& OutViewState);

	/** Returns true once if this type of log message can be displayed for the first time.*/
	bool CanShowLogMsgOnce(const EDisplayClusterViewportPreviewShowLogMsgOnce& InLogState)
	{
		if (!EnumHasAnyFlags(ShowLogMsgOnceFlags, InLogState))
		{
			EnumAddFlags(ShowLogMsgOnceFlags, InLogState);

			return true;
		}

		return false;
	}

	/** Reset log states. */
	void ResetShowLogMsgOnce(const EDisplayClusterViewportPreviewShowLogMsgOnce& InLogState)
	{
		EnumRemoveFlags(ShowLogMsgOnceFlags, InLogState);
	}

protected:
	/** Return output preview resource. */
	TSharedPtr<FDisplayClusterViewportResource, ESPMode::ThreadSafe> GetOutputPreviewTargetableResources() const;

	/** Calculating stereoviewer offset for preview. */
	bool CalculateStereoViewOffset(
		const TSharedRef<FDisplayClusterViewport, ESPMode::ThreadSafe>& InViewport,
		const uint32 InContextNum,
		FRotator& ViewRotation,
		FVector& ViewLocation,
		FVector& StereoViewLocation);

	/** Get viewport context projection matrix. */
	FMatrix GetStereoProjectionMatrix(FDisplayClusterViewport& InViewport, const uint32 InContextNum);

public:
	// Configuration of the current cluster node
	const TSharedRef<FDisplayClusterViewportConfiguration, ESPMode::ThreadSafe> Configuration;

	// Unique viewport name
	const FString ViewportId;

	// Owner cluster node name
	const FString ClusterNodeId;

private:
	// Weak pointer to the owning DC viewport currently used for rendering.
	TWeakPtr<FDisplayClusterViewport, ESPMode::ThreadSafe> OwningViewportWeakPtr;

	// Preview RTT
	TSharedPtr<FDisplayClusterViewportResource, ESPMode::ThreadSafe> PreviewRTT;

	// Preview mesh
	FDisplayClusterViewportPreviewMesh PreviewMesh;

	// Preview editable mesh
	FDisplayClusterViewportPreviewMesh PreviewEditableMesh;

	// Runtime flags
	EDisplayClusterViewportPreviewFlags RuntimeFlags = EDisplayClusterViewportPreviewFlags::None;

	// A recurring message in the log will be shown only once
	EDisplayClusterViewportPreviewShowLogMsgOnce ShowLogMsgOnceFlags = EDisplayClusterViewportPreviewShowLogMsgOnce::None;
};
