// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Viewport/Preview/DisplayClusterViewportPreviewMesh.h"
#include "Render/Viewport/DisplayClusterViewport.h"
#include "Render/Viewport/Preview/DisplayClusterViewportPreview.h"

#include "Render/DisplayDevice/Components/DisplayClusterDisplayDeviceBaseComponent.h"
#include "Render/Projection/IDisplayClusterProjectionPolicy.h"
#include "Components/DisplayClusterCameraComponent.h"
#include "Components/StaticMeshComponent.h"

#include "CompositeCoreSubsystem.h"

namespace UE::DisplayCluster::ViewportPreviewMesh
{
	/** Reset the mesh material to default values from its archetype. */
	static inline bool RestoreMeshMaterialsFromArchetype(UMeshComponent* InMeshComponent)
	{
		if (IsValid(InMeshComponent))
		{
			if (const UMeshComponent* MeshArchetype = Cast<UMeshComponent>(InMeshComponent->GetArchetype()))
			{
				// Retrieve material on the preview mesh from an archetype or OrigMaterial.
				if (UMaterialInterface* OrigMaterial = MeshArchetype->OverrideMaterials.IsValidIndex(0) ? MeshArchetype->OverrideMaterials[0] : nullptr)
				{
					InMeshComponent->SetMaterial(0, OrigMaterial);

					return true;
				}
			}
		}

		return false;
	}

	/** Get material for preview mesh. */
	TObjectPtr<UMaterial> FindPreviewMeshMaterial(const EDisplayClusterDisplayDeviceMeshType InMeshType, const EDisplayClusterDisplayDeviceMaterialType InMaterialType, UDisplayClusterDisplayDeviceBaseComponent* InDisplayDeviceComponent, UDisplayClusterCameraComponent* ViewPointComponent)
	{
		TObjectPtr<UMaterial> OutMaterial = nullptr;

		// First get the material from the ViewPoint component (WarpPolicy)
		OutMaterial = IsValid(ViewPointComponent) ? ViewPointComponent->GetDisplayDeviceMaterial(InMeshType, InMaterialType) : nullptr;

		// Finally, get the material from the DisplayDevice
		if (!OutMaterial)
		{
			OutMaterial = IsValid(InDisplayDeviceComponent) ? InDisplayDeviceComponent->GetDisplayDeviceMaterial(InMeshType, InMaterialType) : nullptr;
		}

		// Ignore deleted materials
		return OutMaterial;
	}

	/** Configure Holdout for the preview mesh
	* 
	* @param Configuration   - current configuration
	* @param InMeshComponent - Mesh for to be configured for Holdout
	* @param InMeshType      - Mesh type
	* @param InMaterialType  - Type of material on mesh
	*/
	static void UpdateHoldoutForMeshComponent(
		FDisplayClusterViewportConfiguration& Configuration,
		UMeshComponent* InMeshComponent,
		const EDisplayClusterDisplayDeviceMeshType InMeshType,
		const EDisplayClusterDisplayDeviceMaterialType InMaterialType = EDisplayClusterDisplayDeviceMaterialType::PreviewMeshMaterial)
	{
		if (!IsValid(InMeshComponent) || !UCompositeCoreSubsystem::IsProjectSettingsValid())
		{
			return;
		}

		UCompositeCoreSubsystem* CompositeSubsystem = UWorld::GetSubsystem<UCompositeCoreSubsystem>(Configuration.GetCurrentWorld());
		if (!IsValid(CompositeSubsystem))
		{
			return;
		}

		//By default, all materials must support Holdout.
		bool bUseHoldout = true;
		if (InMeshType == EDisplayClusterDisplayDeviceMeshType::DefaultMesh)
		{
			bUseHoldout = false;
		}

		if (bUseHoldout && Configuration.GetRenderFrameSettings().ShouldUseHoldout())
		{
			// Render this mesh using into default (holdout) composite pipeline
			CompositeSubsystem->RegisterPrimitive(InMeshComponent);
		}
		else
		{
			// Use normal rendering
			CompositeSubsystem->UnregisterPrimitive(InMeshComponent);
		}
	}
};

////////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterViewportPreviewMesh
////////////////////////////////////////////////////////////////////////////////////////
EDisplayClusterDisplayDeviceMaterialType FDisplayClusterViewportPreviewMesh::GetCurrentMaterialType() const
{
	return Configuration->IsTechvisEnabled()
		? EDisplayClusterDisplayDeviceMaterialType::PreviewMeshTechvisMaterial
		: EDisplayClusterDisplayDeviceMaterialType::PreviewMeshMaterial;
}

void FDisplayClusterViewportPreviewMesh::Update(FDisplayClusterViewport* InViewport, UDisplayClusterDisplayDeviceBaseComponent* InDisplayDeviceComponent, UDisplayClusterCameraComponent* ViewPointComponent)
{
	using namespace UE::DisplayCluster::ViewportPreviewMesh;

	// Reset runtime flags before each update
	RuntimeFlags = EDisplayClusterViewportPreviewMeshFlags::None;

	if (!InViewport || !InViewport->ViewportPreview->HasAnyFlags(EDisplayClusterViewportPreviewFlags::HasValidPreviewRTT) || !ShouldUseMeshComponent(InViewport) || !InDisplayDeviceComponent)
	{
		// The mesh component and its resources are no longer used.
		Release(InViewport);

		return;
	}

	// Update default material
	DefaultMaterialPtr = FindPreviewMeshMaterial(EDisplayClusterDisplayDeviceMeshType::DefaultMesh, GetCurrentMaterialType(), InDisplayDeviceComponent, ViewPointComponent);

	// Get current preview material
	UMaterial* InMeshMaterial = FindPreviewMeshMaterial(MeshType, GetCurrentMaterialType(), InDisplayDeviceComponent, ViewPointComponent);
	if (!InMeshMaterial)
	{
		// Do not use preview if material is not defined.
		Release(InViewport);

		return;
	}

	// Get or create warp mesh:
	bool bNewIsRootActorPreviewMesh = false;
	UMeshComponent* NewMeshComponent = GetOrCreatePreviewMeshComponent(InViewport, bNewIsRootActorPreviewMesh);
	if (GetMeshComponent() != NewMeshComponent || bNewIsRootActorPreviewMesh != bIsRootActorMeshComponent)
	{
		// Release the reference to the old mesh component
		Release(InViewport);
	}
	else if (CurrentMaterialPtr != InMeshMaterial)
	{
		// Release a material instance when the material is changed
		ReleaseMaterialInstance();
	}

	if (NewMeshComponent != GetMeshComponent())
	{
		// The mesh component has been modified.
		EnumAddFlags(RuntimeFlags, EDisplayClusterViewportPreviewMeshFlags::HasChangedMeshComponent);

		// Update mesh component refs
		bIsRootActorMeshComponent = bNewIsRootActorPreviewMesh;
		MeshComponentPtr = NewMeshComponent;
		MeshComponentBinding.SetSceneComponent(NewMeshComponent);
	}

	// Update material instance and assign to the  mesh
	if (UMeshComponent* MeshComponent = GetMeshComponent())
	{
		// Update material instance and assign to the  mesh
		if (!MaterialInstancePtr.IsValid())
		{
			CurrentMaterialPtr = InMeshMaterial;

			MaterialInstancePtr = UMaterialInstanceDynamic::Create(InMeshMaterial, MeshComponent);
			EnumAddFlags(RuntimeFlags, EDisplayClusterViewportPreviewMeshFlags::HasChangedMaterialInstance);
		}

		// The material must be assigned to the mesh at each tick, without any conditions in case it can be changed externally.
		MeshComponent->SetMaterial(0, MaterialInstancePtr.Get());

		// Apply a holdout to this mesh component if possible.
		UpdateHoldoutForMeshComponent(*Configuration, MeshComponent, MeshType, GetCurrentMaterialType());
	}

	// Handling the material overlay logic for the preview mesh:
	UpdateOverlayMaterial(InViewport);
}

void FDisplayClusterViewportPreviewMesh::SetCustomOverlayMaterial(UMeshComponent* InMeshComponent, UMaterialInterface* InOverlayMaterial)
{
	if (InMeshComponent)
	{
		if (!OrigOverlayMaterialPtr.IsValid())
		{
			OrigOverlayMaterialPtr = InMeshComponent->GetOverlayMaterial();
		}

		InMeshComponent->SetOverlayMaterial(InOverlayMaterial);
	}
}

void FDisplayClusterViewportPreviewMesh::RestoreOverlayMaterial(UMeshComponent* InMeshComponent)
{
	if (InMeshComponent)
	{
		if (UMaterialInterface* const OrigOverlayMaterial = OrigOverlayMaterialPtr.Get())
		{
			InMeshComponent->SetOverlayMaterial(OrigOverlayMaterial);
		}
		else
		{
			if (const UMeshComponent* MeshArchetype = Cast<UMeshComponent>(InMeshComponent->GetArchetype()))
			{
				// Retrieve material on the preview mesh from an archetype or OrigMaterial.
				if (UMaterialInterface* ArchetypeOverlayMaterial = MeshArchetype->GetOverlayMaterial())
				{
					InMeshComponent->SetOverlayMaterial(ArchetypeOverlayMaterial);
				}
			}
		}
	}
}

void FDisplayClusterViewportPreviewMesh::UpdateOverlayMaterial(FDisplayClusterViewport* InViewport)
{
	// Update material instance and assign to the  mesh
	if (UMeshComponent* MeshComponent = GetMeshComponent())
	{
		if (Configuration->GetPreviewSettings().bPreviewEnableOverlayMaterial)
		{
			// Restore overlay material
			RestoreOverlayMaterial(MeshComponent);
		}
		else
		{
			// Disable overlay material when preview is used
			SetCustomOverlayMaterial(MeshComponent, nullptr);
		}
	}
}

void FDisplayClusterViewportPreviewMesh::ReleaseMeshComponent(FDisplayClusterViewport* InViewport)
{
	using namespace UE::DisplayCluster::ViewportPreviewMesh;

	// The weak ptr cache can go stale when the RootActor reruns its construction
	// scripts (mesh component destroyed + recreated). Re-resolve via the binding
	// so Release still operates on the live component.
	UMeshComponent* MeshComponent = GetMeshComponent();
	if (MeshComponent == nullptr)
	{
		if (USceneComponent* PreviewMeshComp = MeshComponentBinding.GetOrFindSceneComponent())
		{
			MeshComponent = Cast<UMeshComponent>(PreviewMeshComp);
		}
	}

	if (MeshComponent)
	{
		UpdateHoldoutForMeshComponent(*Configuration, MeshComponent, EDisplayClusterDisplayDeviceMeshType::DefaultMesh);

		EnumAddFlags(RuntimeFlags, EDisplayClusterViewportPreviewMeshFlags::HasDeletedMeshComponent);

		UMaterial* const DefaultMaterial = DefaultMaterialPtr.Get();

		// Restore materials on exists mesh
		RestoreOverlayMaterial(MeshComponent);

		const bool bRestored = RestoreMeshMaterialsFromArchetype(MeshComponent);
		if (!bRestored && DefaultMaterial)
		{
			MeshComponent->SetMaterial(0, DefaultMaterial);
		}

		if (!bIsRootActorMeshComponent)
		{
			// Release this mesh component from DCRA
			MeshComponent->UnregisterComponent();
			MeshComponent->DestroyComponent();
		}
		else
		{
			// Restore the default material for an existing mesh in DCRA
			EnumAddFlags(RuntimeFlags, EDisplayClusterViewportPreviewMeshFlags::HasRestoredDefaultMaterial);
		}
	}

	MeshComponentPtr.Reset();
	MeshComponentBinding.ResetSceneComponent();
}

void FDisplayClusterViewportPreviewMesh::ReleaseMaterialInstance()
{
	// The material instance references the mesh, so it must also be deleted
	if (UMaterialInstanceDynamic* MaterialInstance = MaterialInstancePtr.Get())
	{
		EnumAddFlags(RuntimeFlags, EDisplayClusterViewportPreviewMeshFlags::HasDeletedMaterialInstance);
		MaterialInstance->ClearParameterValues();
	}

	MaterialInstancePtr.Reset();
	CurrentMaterialPtr.Reset();
}

bool FDisplayClusterViewportPreviewMesh::ShouldUseMeshComponent(FDisplayClusterViewport* InViewport) const
{
	if (!InViewport || !InViewport->GetRenderSettings().bEnable)
	{
		// disable viewport
		return false;
	}

	const FDisplayClusterViewport_PreviewSettings PreviewSettings = Configuration->GetRenderFrameSettings().PreviewSettings;
	TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe> ProjectionPolicy = InViewport->GetProjectionPolicy();
	if (ProjectionPolicy.IsValid())
	{
		switch (MeshType)
		{
		case EDisplayClusterDisplayDeviceMeshType::PreviewMesh:
			return ProjectionPolicy->HasPreviewMesh(InViewport) && PreviewSettings.bEnablePreviewMesh;

		case EDisplayClusterDisplayDeviceMeshType::PreviewEditableMesh:
			return ProjectionPolicy->HasPreviewEditableMesh(InViewport) && PreviewSettings.bEnablePreviewEditableMesh;

		default:
			break;
		}
	}

	return false;
}

UMeshComponent* FDisplayClusterViewportPreviewMesh::GetOrCreatePreviewMeshComponent(FDisplayClusterViewport* InViewport, bool& bOutIsRootActorComponent) const
{
	if (!ShouldUseMeshComponent(InViewport))
	{
		return nullptr;
	}

	TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe> ProjectionPolicy = InViewport ? InViewport->GetProjectionPolicy() : nullptr;
	if (ProjectionPolicy.IsValid())
	{
		switch (MeshType)
		{
		case EDisplayClusterDisplayDeviceMeshType::PreviewMesh:
			return ProjectionPolicy->GetOrCreatePreviewMeshComponent(InViewport, bOutIsRootActorComponent);

		case EDisplayClusterDisplayDeviceMeshType::PreviewEditableMesh:
			bOutIsRootActorComponent = false;
			return ProjectionPolicy->GetOrCreatePreviewEditableMeshComponent(InViewport);

		default:
			break;
		}
	}

	return nullptr;
}

UMeshComponent* FDisplayClusterViewportPreviewMesh::GetMeshComponent() const
{
	return MeshComponentPtr.Get();
}

UMaterialInstanceDynamic* FDisplayClusterViewportPreviewMesh::GetMaterialInstance() const
{
	return MaterialInstancePtr.Get();
}
