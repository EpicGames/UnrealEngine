// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "Components/MeshComponent.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"

#include "Render/DisplayDevice/Containers/DisplayClusterDisplayDevice_Enums.h"
#include "Render/Viewport/Configuration/DisplayClusterViewportConfiguration.h"

#include "Misc/DisplayClusterObjectRef.h"


class UDisplayClusterDisplayDeviceBaseComponent;
class UDisplayClusterCameraComponent;
class FDisplayClusterViewport;

/**
 * Runtime configuration of preview mesh.
 */
enum class EDisplayClusterViewportPreviewMeshFlags : uint8
{
	None = 0,

	// The Mesh component has been removed
	HasDeletedMeshComponent = 1 << 0,

	// the Mesh component has been modified
	HasChangedMeshComponent = 1 << 1,

	// The MaterialInstance has been removed
	HasDeletedMaterialInstance = 1 << 2,

	// The MaterialInstance has been modified
	HasChangedMaterialInstance = 1 << 3,

	// Default Material is restored on the Mesh
	HasRestoredDefaultMaterial = 1 << 4,
};
ENUM_CLASS_FLAGS(EDisplayClusterViewportPreviewMeshFlags);

/**
* Manage preview mesh of the viewport
*/
class FDisplayClusterViewportPreviewMesh
{
public:
	FDisplayClusterViewportPreviewMesh(const EDisplayClusterDisplayDeviceMeshType InMeshType, const TSharedRef<FDisplayClusterViewportConfiguration, ESPMode::ThreadSafe> InConfiguration)
		: Configuration(InConfiguration), MeshType(InMeshType)
	{ }

	~FDisplayClusterViewportPreviewMesh() = default;

public:
	/** Get preview mesh component. */
	UMeshComponent* GetMeshComponent() const;

	/** Get MID used on preview mesh. */
	UMaterialInstanceDynamic* GetMaterialInstance() const;

	/** Return current material type that used on this mesh. */
	EDisplayClusterDisplayDeviceMaterialType GetCurrentMaterialType() const;

	/** Update mesh component and materials for viewport. */
	void Update(FDisplayClusterViewport* InViewport, UDisplayClusterDisplayDeviceBaseComponent* InDisplayDeviceComponent, UDisplayClusterCameraComponent* ViewPointComponent);

	/** Restore default material and release mesh component with materials for viewport. */
	void Release(FDisplayClusterViewport* InViewport)
	{
		ReleaseMeshComponent(InViewport);
		ReleaseMaterialInstance();
	}

	/** Returns true if the runtime flags have any of the input flags. */
	bool HasAnyFlag(const EDisplayClusterViewportPreviewMeshFlags InMeshFlags) const
	{
		return EnumHasAnyFlags(RuntimeFlags, InMeshFlags);
	}

private:
	/** Returns true if this mesh type is supported by the viewport projection policy and DCRA. */
	bool ShouldUseMeshComponent(FDisplayClusterViewport* InViewport) const;

	/** Release material instance */
	void ReleaseMaterialInstance();

	/** Release mesh component. */
	void ReleaseMeshComponent(FDisplayClusterViewport* InViewport);

	/** Update overlay materials on mesh. */
	void UpdateOverlayMaterial(FDisplayClusterViewport* InViewport);

	/** Set custom overlay materials on mesh. */
	void SetCustomOverlayMaterial(UMeshComponent* InMeshComponent, UMaterialInterface* InOverlayMaterial);

	/** Restore overlay material on mesh.*/
	void RestoreOverlayMaterial(UMeshComponent* InMeshComponent);

	/** Get mesh component. */
	UMeshComponent* GetOrCreatePreviewMeshComponent(FDisplayClusterViewport* InViewport, bool& bOutIsRootActorComponent) const;

public:
	// Configuration of the current cluster node
	const TSharedRef<FDisplayClusterViewportConfiguration, ESPMode::ThreadSafe> Configuration;

	// the type of mesh
	const EDisplayClusterDisplayDeviceMeshType MeshType;

private:
	// runtime flags
	EDisplayClusterViewportPreviewMeshFlags RuntimeFlags;

	// Persistent by-name binding to the mesh component used for preview.
	FDisplayClusterSceneComponentRef MeshComponentBinding;

	// Resolved weak handle from the binding above.
	TWeakObjectPtr<UMeshComponent> MeshComponentPtr;

	// This mesh component exists in DCRA and does not need to be deleted.
	bool bIsRootActorMeshComponent = false;

	// Overlay materials for the mesh can be customized when using preview rendering.
	TWeakObjectPtr<UMaterialInterface> OrigOverlayMaterialPtr;

	// Preview material used on the mesh
	TWeakObjectPtr<UMaterialInstanceDynamic> MaterialInstancePtr;

	// The current material assigned to the preview mesh
	TWeakObjectPtr<UMaterial> CurrentMaterialPtr;

	// The default material defined in the DisplayDevice
	TWeakObjectPtr<UMaterial> DefaultMaterialPtr;
};
