// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Viewport/Preview/DisplayClusterViewportPreview.h"
#include "Render/Viewport/Resource/DisplayClusterViewportResource.h"
#include "Render/Viewport/DisplayClusterViewport.h"

#include "EngineModule.h"
#include "CanvasTypes.h"
#include "LegacyScreenPercentageDriver.h"
#include "PrimitiveDrawingUtils.h"
#include "SceneView.h"
#include "SceneViewExtension.h"

#include "Engine/Scene.h"
#include "GameFramework/WorldSettings.h"

#include "Render/Viewport/DisplayClusterViewportManager.h"
#include "Render/Viewport/RenderFrame/DisplayClusterRenderFrameSettings.h"

#include "Misc/DisplayClusterLog.h"

////////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterViewportPreview
////////////////////////////////////////////////////////////////////////////////////////
FSceneView* FDisplayClusterViewportPreview::CalcSceneView(FSceneViewFamilyContext& InOutViewFamily, uint32 InContextNum,
	TSharedPtr<FSceneViewStateReference, ESPMode::ThreadSafe>& OutViewState)
{
	// Reset the output ViewState before attempting to create a new scene view.
	OutViewState.Reset();

	// Keeps a reference to the owning viewport so that it is not deleted within this logic.
	TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe> OwningViewport = OwningViewportWeakPtr.Pin();
	if (!OwningViewport.IsValid() || !OwningViewport->GetContexts().IsValidIndex(InContextNum))
	{
		return nullptr;
	}

	const FDisplayClusterViewport_Context& InViewportContext = OwningViewport->GetContexts()[InContextNum];

	const float MaxViewDistance = 1000000;
	const bool bUseSceneColorTexture = false;
	const float LODDistanceFactor = 1.0f;

	const AActor* ViewOwner = nullptr;

	FVector ViewLocation;
	FVector StereoViewLocation;
	FRotator ViewRotation;

	if (CalculateStereoViewOffset(OwningViewport.ToSharedRef(), InContextNum, ViewRotation, ViewLocation, StereoViewLocation))
	{
		FMatrix ProjectionMatrix = GetStereoProjectionMatrix(*OwningViewport, InContextNum);

		FMatrix ViewRotationMatrix = FInverseRotationMatrix(ViewRotation);
		ViewRotationMatrix = ViewRotationMatrix * FMatrix(
			FPlane(0, 0, 1, 0),
			FPlane(1, 0, 0, 0),
			FPlane(0, 1, 0, 0),
			FPlane(0, 0, 0, 1));


		FIntRect ViewRect = InViewportContext.RenderTargetRect;

		FSceneViewInitOptions ViewInitOptions;

		ViewInitOptions.SetViewRectangle(ViewRect);
		ViewInitOptions.ViewFamily = &InOutViewFamily;

		// Keeps a reference to the ViewState so that it is not destroyed before rendering is complete.
		OutViewState = OwningViewport->GetOrCreateViewState(InContextNum);
		ViewInitOptions.SceneViewStateInterface = OutViewState.IsValid() ? OutViewState->GetReference() : nullptr;

		ViewInitOptions.ViewActor = ViewOwner;

		ViewInitOptions.ViewOrigin = StereoViewLocation;
		ViewInitOptions.ViewLocation = ViewLocation;
		ViewInitOptions.ViewRotation = ViewRotation;

		ViewInitOptions.ViewRotationMatrix = ViewRotationMatrix;
		ViewInitOptions.ProjectionMatrix = ProjectionMatrix;

		ViewInitOptions.OverrideFarClippingPlaneDistance = MaxViewDistance;
		ViewInitOptions.StereoPass = InViewportContext.StereoscopicPass;
		ViewInitOptions.StereoViewIndex = InViewportContext.StereoViewIndex;

		ViewInitOptions.LODDistanceFactor = FMath::Clamp(LODDistanceFactor, 0.01f, 100.0f);
		ViewInitOptions.WorldToMetersScale = Configuration->GetWorldToMeters();
		ViewInitOptions.BackgroundColor = FLinearColor::Black;

		if (Configuration->GetRenderFrameSettings().IsPostProcessDisabled())
		{
			ViewInitOptions.OverlayColor = FLinearColor::Black;
		}

		ViewInitOptions.bSceneCaptureUsesRayTracing = false;
		ViewInitOptions.bIsPlanarReflection = false;

		FSceneView* View = new FSceneView(ViewInitOptions);

		InOutViewFamily.Views.Add(View);

		// Configure postprocesses for the current viewport.
		// The code below is based on the code from ULocalPlayer.
		{
			// ERenderPass::Start
			View->StartFinalPostprocessSettings(ViewLocation);
			OwningViewport->GetViewport_CustomPostProcessSettings().ApplyCustomPostProcess(OwningViewport.Get(), InContextNum, IDisplayClusterViewport_CustomPostProcessSettings::ERenderPass::Start, View->FinalPostProcessSettings);

			// ERenderPass::Override
			FPostProcessSettings OverridePostProcessingSettings;
			float OverridePostProcessBlendWeight = 1.0f;
			if (OwningViewport->GetViewport_CustomPostProcessSettings().ApplyCustomPostProcess(OwningViewport.Get(), InContextNum, IDisplayClusterViewport_CustomPostProcessSettings::ERenderPass::Override, OverridePostProcessingSettings, &OverridePostProcessBlendWeight))
			{
				View->OverridePostProcessSettings(OverridePostProcessingSettings, OverridePostProcessBlendWeight);
			}

			// ERenderPass::Final
			OwningViewport->GetViewport_CustomPostProcessSettings().ApplyCustomPostProcess(OwningViewport.Get(), InContextNum, IDisplayClusterViewport_CustomPostProcessSettings::ERenderPass::Final, View->FinalPostProcessSettings);
			View->EndFinalPostprocessSettings(ViewInitOptions);
		}

		FDisplayClusterViewportManager* ViewportManager = Configuration->GetViewportManagerImpl();
		if (ViewportManager && OwningViewport->GetContexts().IsValidIndex(InContextNum))
		{
			// Configure FDisplayClusterViewportManagerViewPointExtension for this viewport.
			ViewportManager->SetCurrentStereoViewIndexForViewPointExtension(OwningViewport->GetContexts()[InContextNum].StereoViewIndex);
		}

		// Setup view extension for this view
		for (int32 ViewExt = 0; ViewExt < InOutViewFamily.ViewExtensions.Num(); ViewExt++)
		{
			InOutViewFamily.ViewExtensions[ViewExt]->SetupView(InOutViewFamily, *View);
		}

		if (ViewportManager)
		{
			// Configure FDisplayClusterViewportManagerViewPointExtension for this viewport.
			ViewportManager->SetCurrentStereoViewIndexForViewPointExtension(INDEX_NONE);
		}

		return View;
	}

	return nullptr;
}

FMatrix FDisplayClusterViewportPreview::GetStereoProjectionMatrix(FDisplayClusterViewport& InViewport, const uint32 InContextNum)
{
	check(IsInGameThread());

	FMatrix PrjMatrix = FMatrix::Identity;
	if (Configuration->IsSceneOpened() && InViewport.GetProjectionMatrix(InContextNum, PrjMatrix) == false)
	{
		if (CanShowLogMsgOnce(EDisplayClusterViewportPreviewShowLogMsgOnce::StereoProjectionMatrixIsInvalid))
		{
			UE_LOGF(LogDisplayClusterViewport, Verbose, "Got invalid projection matrix: Viewport %ls, ViewIdx: %d", *InViewport.GetId(), InContextNum);
		}
	}

	ResetShowLogMsgOnce(EDisplayClusterViewportPreviewShowLogMsgOnce::StereoProjectionMatrixIsInvalid);

	return PrjMatrix;
}

bool FDisplayClusterViewportPreview::CalculateStereoViewOffset(
	const TSharedRef<FDisplayClusterViewport, ESPMode::ThreadSafe>& InViewport,
	const uint32 InContextNum,
	FRotator& ViewRotation,
	FVector& ViewLocation,
	FVector& StereoViewLocation)
{
	check(IsInGameThread());

	// Obtaining the internal viewpoint for a given viewport with stereo eye offset distance.
	FMinimalViewInfo ViewInfo;
	if (!InViewport->SetupViewPoint(InContextNum, ViewInfo))
	{
		return false;
	}

	ViewLocation = ViewInfo.Location;
	ViewRotation = ViewInfo.Rotation;

	// Obtaining the offset of the stereo eye and the values of the projection clipping plane for the given viewport was moved inside CalculateView().
	// Perform view calculations on a policy side
	StereoViewLocation = ViewInfo.Location;
	if (!InViewport->CalculateView(InContextNum, StereoViewLocation, ViewRotation, Configuration->GetWorldToMeters()))
	{
		if (CanShowLogMsgOnce(EDisplayClusterViewportPreviewShowLogMsgOnce::CalculateViewIsFailed))
		{
			UE_LOGF(LogDisplayClusterViewport, Verbose, "Couldn't compute preview parameters for Viewport %ls, ViewIdx: %d", *InViewport->GetId(), InContextNum);
		}

		return false;
	}

	ResetShowLogMsgOnce(EDisplayClusterViewportPreviewShowLogMsgOnce::CalculateViewIsFailed);

	return true;
}
