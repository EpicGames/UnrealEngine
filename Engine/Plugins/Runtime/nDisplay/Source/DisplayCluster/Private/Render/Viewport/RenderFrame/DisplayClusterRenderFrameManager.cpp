// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Viewport/RenderFrame/DisplayClusterRenderFrameManager.h"
#include "Render/Viewport/RenderFrame/DisplayClusterRenderFrame.h"
#include "Render/Viewport/RenderFrame/DisplayClusterRenderFrameSettings.h"
#include "Render/Viewport/Configuration/DisplayClusterViewportConfiguration.h"
#include "Render/Viewport/DisplayClusterViewport.h"
#include "Render/Viewport/DisplayClusterViewportManager.h"
#include "Render/Features/DisplayClusterUpscaler.h"

#include "Misc/Parse.h"

///////////////////////////////////////////////////////////////
// FDisplayClusterRenderTargetFrame
///////////////////////////////////////////////////////////////
bool FDisplayClusterRenderFrameManager::BuildRenderFrame(FViewport* InViewport, const TArray<TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>>& InViewports, FDisplayClusterRenderFrame& OutRenderFrame)
{
	const FDisplayClusterRenderFrameSettings& InRenderFrameSettings = Configuration->GetRenderFrameSettings();

	// Dont use render frame for preview
	if (!InRenderFrameSettings.IsPreviewRendering())
	{
		if (!FindFrameTargetRect(InViewport, InViewports, OutRenderFrame.FrameRect))
		{
			return false;
		}
	}

	// @todo add atlasing, merge multiple viewports in single viewfamily, etc
	// now prototype, just simple frame: use separate RTT for each viewport eye

	// Sort viewports, childs after parents
	//@todo save this order inside logic
	TArray<TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>> SortedViewports;
	TArray<TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>> ChildsViewports;

	// First add root viewports
	for (const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& Viewport : InViewports)
	{
		if (Viewport.IsValid())
		{
			if (Viewport->GetRenderSettings().GetParentViewportId().IsEmpty())
			{
				SortedViewports.Add(Viewport);
			}
			else
			{
				ChildsViewports.Add(Viewport);
			}
		}
	}

	SortedViewports.Append(ChildsViewports);
	
	// At this point, we have the list of the viewports sorted in a way that
	// child (dependent) viewports follow after the parent ones. Here we do
	// some re-ordering so all the viewports that have a media capture
	// device assigned, will be first in the list. Child viewports can't
	// have media capture assigned, so there won't be any problems.
	{
		// Find all viewports being captured
		TArray<TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>> ViewportsBeingCaptured = SortedViewports.FilterByPredicate([](const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& Viewport)
		{
			return Viewport.IsValid() && Viewport->GetRenderSettings().HasAnyMediaStates(EDisplayClusterViewportMediaState::Capture);
		});

		// Put them all in the beginning of the list
		const int32 CapturingViewportsAmount = ViewportsBeingCaptured.Num();
		if (CapturingViewportsAmount > 0 && CapturingViewportsAmount != SortedViewports.Num())
		{
			for (TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& ViewportBeingCaptured : ViewportsBeingCaptured)
			{
				SortedViewports.Remove(ViewportBeingCaptured);
				SortedViewports.Insert(ViewportBeingCaptured, 0);
			}
		}
	}

	return BuildSimpleFrame(InViewport, SortedViewports, OutRenderFrame);
}

bool FDisplayClusterRenderFrameManager::BuildSimpleFrame(FViewport* InViewport, const TArray<TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>>& InViewports, FDisplayClusterRenderFrame& OutRenderFrame)
{
	for (const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& ViewportIt : InViewports)
	{
		if (ViewportIt.IsValid())
		{
			// Exclude all ViewExtensions. They are added later, in FDisplayClusterViewportManager::PostConfigureViewFamily()
			ViewportIt->GetRenderSettingsImpl().UpscalerSettings.bIsActive = false;

			for (const FDisplayClusterViewport_Context& ContextIt : ViewportIt->GetContexts())
			{
				FDisplayClusterRenderFrameTargetView FrameView;
				{
					FrameView.ContextNum = ContextIt.ContextNum;
					FrameView.Viewport = ViewportIt.ToSharedRef();
					FrameView.bDisableRender = ContextIt.bDisableRender;
					FrameView.bFreezeRendering = ViewportIt->GetRenderSettings().bFreezeRendering;
				}

				FDisplayClusterRenderFrameTargetViewFamily FrameViewFamily;
				{
					FrameViewFamily.Views.Add(FrameView);
					FrameViewFamily.CustomBufferRatio = ContextIt.CustomBufferRatio;
					FrameViewFamily.UpscalerSettings = ViewportIt->GetRenderSettings().UpscalerSettings;

					// We need to determine if we can use upscalers in the future when AntiAliasing methods are configured in SceneViews.
					FrameViewFamily.ViewExtensions = ViewportIt->GatherActiveExtensions(ContextIt.ContextNum, InViewport);
				}

				FDisplayClusterRenderFrameTarget FrameRenderTarget;
				{
					FrameRenderTarget.Viewport = InViewport;

					// Simple frame use unique RTT  for each viewport, so disable RTT when viewport rendering disabled
					FrameRenderTarget.bShouldUseRenderTarget = ViewportIt->GetViewportResources(EDisplayClusterViewportResource::RenderTargets).Num() > 0;

					FrameRenderTarget.ViewFamilies.Add(FrameViewFamily);

					FrameRenderTarget.RenderTargetSize = ContextIt.RenderTargetRect.Max;
					FrameRenderTarget.CaptureMode = ViewportIt->GetRenderSettings().CaptureMode;
				}

				OutRenderFrame.RenderTargets.Add(FrameRenderTarget);
			}

			// After the GatherActiveExtensions() enable upscalers ViewExtensions back.
			ViewportIt->GetRenderSettingsImpl().UpscalerSettings.bIsActive = true;
		}
	}

	return true;
}

bool FDisplayClusterRenderFrameManager::FindFrameTargetRect(FViewport* InViewport, const TArray<TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>>& InOutViewports, FIntRect& OutFrameTargetRect) const
{
	// Calculate Backbuffer frame
	bool bIsUsed = false;

	if (InViewport && Configuration->GetRenderFrameSettings().bShouldUseFullSizeFrameTargetableResource)
	{
		// Use full-size frame RTT
		OutFrameTargetRect = FIntRect(FIntPoint(0, 0), InViewport->GetSizeXY());
		bIsUsed = true;
	}

	// Optimize frame target RTT
	for (const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& ViewportIt : InOutViewports)
	{
		if (ViewportIt.IsValid() && ViewportIt->GetRenderSettings().bEnable && ViewportIt->GetRenderSettings().bVisible)
		{
			for (const auto& ContextIt : ViewportIt->GetContexts())
			{
				if (!bIsUsed)
				{
					OutFrameTargetRect = ContextIt.FrameTargetRect;
					bIsUsed = true;
				}
				else
				{
					OutFrameTargetRect.Include(ContextIt.FrameTargetRect.Min);
					OutFrameTargetRect.Include(ContextIt.FrameTargetRect.Max);
				}
			}
		}
	}

	if (OutFrameTargetRect.Width() <= 0 || OutFrameTargetRect.Height() <= 0)
	{
		// We need to be able to render inner views (ICVFX camera view) without any outers. Current pipeline has
		// a bunch of optimizations that block inner rendering if no outers available. To avoid the limitation
		// we force inner data initialization if no outers found.
		if (Configuration->IsClusterNodeRenderingOffscreen())
		{
			bIsUsed = true;
		}
	}

	return bIsUsed;
}
