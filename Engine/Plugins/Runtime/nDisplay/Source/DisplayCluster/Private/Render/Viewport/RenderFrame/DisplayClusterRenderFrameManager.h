// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FDisplayClusterViewportConfiguration;
class FDisplayClusterViewport;
class FViewport;
class FDisplayClusterRenderFrame;

class FDisplayClusterRenderFrameManager
{
public:
	FDisplayClusterRenderFrameManager(const TSharedRef<const FDisplayClusterViewportConfiguration, ESPMode::ThreadSafe>& InConfiguration)
		: Configuration(InConfiguration)
	{ }

public:
	bool BuildRenderFrame(FViewport* InViewport, const TArray<TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>>& InViewports, FDisplayClusterRenderFrame& OutRenderFrame);

private:
	bool FindFrameTargetRect(FViewport* InViewport, const TArray<TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>>& InViewports, FIntRect& OutFrameTargetRect) const;
	bool BuildSimpleFrame(FViewport* InViewport, const TArray<TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>>& InViewports, FDisplayClusterRenderFrame& OutRenderFrame);

private:
	// Configuration of the current cluster node
	const TSharedRef<const FDisplayClusterViewportConfiguration, ESPMode::ThreadSafe> Configuration;
};
