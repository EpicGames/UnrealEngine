// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Viewport/RenderFrame/DisplayClusterRenderFrameSettings.h"
#include "Render/Viewport/DisplayClusterViewport.h"
#include "Render/Viewport/DisplayClusterViewportHelpers.h"
#include "HAL/IConsoleManager.h"

////////////////////////////////////////////////////////////////////////////////
// Experimental feature: to be approved after testing
int32 GDisplayClusterPreviewEnableReuseViewportInCluster = 1;
static FAutoConsoleVariableRef CVarDisplayClusterPreviewEnableReuseViewportInCluster(
	TEXT("DC.Preview.EnableReuseViewportInCluster"),
	GDisplayClusterPreviewEnableReuseViewportInCluster,
	TEXT("Experimental feature (0 == disabled, 1 == enabled)"),
	ECVF_RenderThreadSafe
);

////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterRenderFrameSettings
////////////////////////////////////////////////////////////////////////////////
bool FDisplayClusterRenderFrameSettings::IsPreviewFreezeRender() const
{
	return IsPreviewRendering() && PreviewSettings.bFreezePreviewRender;
}
const FIntPoint* FDisplayClusterRenderFrameSettings::GetPreviewMultiGPURendering() const
{
	if (IsPreviewRendering() && PreviewMultiGPURendering.IsSet())
	{
		const FIntPoint& GPURange = PreviewMultiGPURendering.GetValue();
		if (GPURange.X <= GPURange.Y)
		{
			return &GPURange;
		}
	}

	return nullptr;
}

FIntPoint FDisplayClusterRenderFrameSettings::GetDesiredRTTSize(const FIntPoint& InSize) const
{
	FVector2D NewSize = GetDesiredRTTSize(FVector2D(InSize.X, InSize.Y));

	return FIntPoint(NewSize.X, NewSize.Y);
}

FVector2D FDisplayClusterRenderFrameSettings::GetDesiredFrameMult() const
{
	const float BaseMult = IsPreviewRendering() ? FMath::Clamp(PreviewSettings.PreviewRenderTargetRatioMult, 0.f, 1.f) : 1.f;

	return GetDesiredRTTSize(FVector2D(BaseMult, BaseMult));
}

bool FDisplayClusterRenderFrameSettings::CanReuseViewportWithinClusterNodes() const
{
	if (GDisplayClusterPreviewEnableReuseViewportInCluster > 0)
	{
		if (IsPreviewRendering())
		{
			return true;
		}
	}

	return false;
}

FIntPoint FDisplayClusterRenderFrameSettings::ApplyViewportSizeConstraint(const FDisplayClusterViewport& InViewport, const FIntPoint& InSize) const
{
	// Adjust the size of the viewport size depending on the constraints
	FIntPoint OutSize = InSize;

	if (InViewport.ShouldApplyMaxTextureConstraints())
	{
		// Apply restrictions on the maximum texture area
		const int32 MaxTextureArea = FDisplayClusterViewportHelpers::GetMaxTextureArea();
		if (MaxTextureArea > 0)
		{
			const int32 OutArea = OutSize.X * OutSize.Y;
			if (OutArea > MaxTextureArea)
			{
				// downsize to the max area
				const double DownsizeMult = FMath::Sqrt(double(MaxTextureArea) / OutArea);
				OutSize = FDisplayClusterViewportHelpers::ScaleTextureSize(OutSize, DownsizeMult);
			}
		}
	}

	const int32 MinTextureDimension = FDisplayClusterViewportHelpers::GetMinTextureDimension();
	int32 MaxTextureDimension = FDisplayClusterViewportHelpers::GetMaxTextureDimension();

	// Preview rendering has its own maximum texture size, which overrides the default one
	if (IsPreviewRendering() && PreviewSettings.PreviewMaxTextureDimension > 0)
	{
		const int32 MaxPreviewTextureDimension = FMath::Clamp(PreviewSettings.PreviewMaxTextureDimension, MinTextureDimension, MaxTextureDimension);

		// Preview consume less GPU memory
		MaxTextureDimension = MaxPreviewTextureDimension;
	}

	// Downsize the size to the limits
	if (MaxTextureDimension < OutSize.GetMax())
	{
		// downsize to the max dimension
		const double DownsizeMult = double(MaxTextureDimension) / OutSize.GetMax();
		OutSize = FDisplayClusterViewportHelpers::ScaleTextureSize(OutSize, DownsizeMult);
	}

	// Final crop
	OutSize.X = FMath::Clamp(OutSize.X, MinTextureDimension, MaxTextureDimension);
	OutSize.Y = FMath::Clamp(OutSize.Y, MinTextureDimension, MaxTextureDimension);

	return OutSize;
}

bool FDisplayClusterRenderFrameSettings::ShouldUseLinearGamma() const
{
	if (IsPreviewRendering())
	{
		return !PreviewSettings.bPreviewEnablePostProcess;
	}

	return false;
}

bool FDisplayClusterRenderFrameSettings::IsEnabledOpenColorIO() const
{
	if (IsPreviewRendering() && !PreviewSettings.bPreviewEnableOCIO)
	{
		// Disable OCIO on preview
		return false;
	}

	// OCIO is enabled by default
	return true;
}

bool FDisplayClusterRenderFrameSettings::IsPostProcessDisabled() const
{
	if (IsPreviewRendering())
	{
		return !PreviewSettings.bPreviewEnablePostProcess;
	}

	return false;
}

bool FDisplayClusterRenderFrameSettings::ShouldUseHoldout() const
{
	if (IsPreviewRendering())
	{
		return PreviewSettings.bPreviewEnableHoldoutComposite;
	}

	return false;
}

bool FDisplayClusterRenderFrameSettings::IsICVFXEnabled() const
{
	// MoviePipeline instances can't use ICVFX yet.
	switch (RenderMode)
	{
	case EDisplayClusterRenderFrameMode::MRG_Mono:
	case EDisplayClusterRenderFrameMode::MRG_Stereo:
	case EDisplayClusterRenderFrameMode::MRQ_Mono:
	case EDisplayClusterRenderFrameMode::MRQ_Stereo:
		return false;
	}

	return true;
}

bool FDisplayClusterRenderFrameSettings::IsPreviewRendering() const
{
	if (RenderMode == EDisplayClusterRenderFrameMode::PreviewInScene
		|| RenderMode == EDisplayClusterRenderFrameMode::PreviewProxyHitInScene)
	{
		return PreviewSettings.bPreviewEnable;
	}

	return false;
}

/** Returns true if this DCRA is using cluster rendering. */
bool FDisplayClusterRenderFrameSettings::IsClusterRendering() const
{
	switch (RenderMode)
	{
	case EDisplayClusterRenderFrameMode::Mono:
	case EDisplayClusterRenderFrameMode::Stereo:
	case EDisplayClusterRenderFrameMode::SideBySide:
	case EDisplayClusterRenderFrameMode::TopBottom:
		return true;

	default:
		break;
	}

	return false;
}

bool FDisplayClusterRenderFrameSettings::IsTechvisEnabled() const
{
	// Don't use Techvis to render ProxyHit
	return RenderMode == EDisplayClusterRenderFrameMode::PreviewInScene
		&& PreviewSettings.bEnablePreviewTechvis;
}

bool FDisplayClusterRenderFrameSettings::IsPreviewInGameEnabled() const
{
	// Don't use Techvis to render ProxyHit
	return RenderMode == EDisplayClusterRenderFrameMode::PreviewInScene
		&& PreviewSettings.bPreviewInGameEnable;
}

FVector2D FDisplayClusterRenderFrameSettings::GetDesiredRTTSize(const FVector2D& InSize) const
{
	switch (RenderMode)
	{
	case EDisplayClusterRenderFrameMode::SideBySide:
	case EDisplayClusterRenderFrameMode::PIE_SideBySide:
		return FVector2D(InSize.X * 0.5f, InSize.Y);

	case EDisplayClusterRenderFrameMode::TopBottom:
	case EDisplayClusterRenderFrameMode::PIE_TopBottom:
		return FVector2D(InSize.X, InSize.Y * 0.5f);

	default:
		break;
	}

	return InSize;
}

int32 FDisplayClusterRenderFrameSettings::GetViewPerViewportAmount() const
{
	switch (RenderMode)
	{
	// Quadbuffer stereo
	case EDisplayClusterRenderFrameMode::Stereo:
		return 2;

	// Render stereo on monoscopic display
	case EDisplayClusterRenderFrameMode::SideBySide:
	case EDisplayClusterRenderFrameMode::TopBottom:
	case EDisplayClusterRenderFrameMode::PIE_SideBySide:
	case EDisplayClusterRenderFrameMode::PIE_TopBottom:
		return 2;

	// MRQ/MRG stereo
	case EDisplayClusterRenderFrameMode::MRQ_Stereo:
	case EDisplayClusterRenderFrameMode::MRG_Stereo:
		return 2;

	default:
		break;
	}

	return 1;
}

bool FDisplayClusterRenderFrameSettings::ShouldUseStereoRenderingOnMonoscopicDisplay() const
{
	switch (RenderMode)
	{
	case EDisplayClusterRenderFrameMode::SideBySide:
	case EDisplayClusterRenderFrameMode::TopBottom:
	case EDisplayClusterRenderFrameMode::PIE_SideBySide:
	case EDisplayClusterRenderFrameMode::PIE_TopBottom:
		return true;

	default:
		break;
	}

	return false;
}
