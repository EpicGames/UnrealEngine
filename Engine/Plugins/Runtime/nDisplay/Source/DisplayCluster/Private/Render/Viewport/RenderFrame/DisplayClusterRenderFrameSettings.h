// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Render/Viewport/RenderFrame/DisplayClusterRenderFrameEnums.h"
#include "Render/Viewport/RenderFrame/DisplayClusterRenderFrameSettingsEnums.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_PreviewSettings.h"

// Settings for render frame builder
struct FDisplayClusterRenderFrameSettings
{
	// If set, use this value instead of GFrameNumber.
	TOptional<uint32> FrameNumberOverride;

	// customize mono\stereo render modes
	EDisplayClusterRenderFrameMode RenderMode = EDisplayClusterRenderFrameMode::Unknown;

	// Alpha channel capture mode for viewports (Lightcard, chromakey)
	EDisplayClusterRenderFrameAlphaChannelCaptureMode AlphaChannelCaptureMode = EDisplayClusterRenderFrameAlphaChannelCaptureMode::None;

	// nDisplay has its own implementation of cross-GPU transfer.
	struct FCrossGPUTransfer
	{
		// Enable cross-GPU transfers using nDisplay
		// That replaces the default cross-GPU transfers using UE Core for the nDisplay viewports viewfamilies.
		uint8 bEnable : 1 = 0;

		// The bLockSteps parameter is simply passed to the FTransferResourceParams structure.
		// Whether the GPUs must handshake before and after the transfer. Required if the texture rect is being written to in several render passes.
		// Otherwise, minimal synchronization will be used.
		uint8 bLockSteps : 1 = 0;

		// The bPullData parameter is simply passed to the FTransferResourceParams structure.
		// Whether the data is read by the dest GPU, or written by the src GPU (not allowed if the texture is a backbuffer)
		uint8 bPullData : 1 = 1;

	} CrossGPUTransfer;

	// This rendering frame uses OutputFrameTargetable resources.
	uint8 bShouldUseOutputFrameTargetableResources : 1 = 0;

	// This rendering frame uses AdditionalFrameTargetable resources.
	uint8 bShouldUseAdditionalFrameTargetableResource : 1 = 0;

	// This rendering frame uses full size for the FrameTargetable resources.
	uint8 bShouldUseFullSizeFrameTargetableResource : 1 = 0;

	// Use an AdditionalFrameTargetable for postprocess in this frame.
	uint8 bUseAdditionalFrameTargetableForPostprocess : 1 = 0;

	// Use an full-size FrameTargetable resources for postprocess in this frame.
	uint8 bUseFullSizeFrameTargetableForPostprocess : 1 = 0;

	// Allow warpblend render
	uint8 bAllowWarpBlend : 1 = 1;

	// True if alpha channel should be written to all viewport outputs.
	// Note: Controlled by "DC.EnableAlphaChannelRendering" and "r.PostProcessing.PropagateAlpha".
	uint8 bEnableAlphaOutput : 1 = 0;

	// Use DC render device for rendering
	uint8 bUseDisplayClusterRenderDevice : 1 = 1;

	// Performance: Stereoscopic rendering uses a single viewfamily(RTT) for both eyes
	uint8 bEnableStereoscopicRenderingOptimization : 1 = 0;

	// When set, nDisplay's own overscan settings are ignored and this external value is used instead.
	uint8 bUseExternalOverscan : 1 = 0;

	// When external overscan is active, controls whether viewports render at the overscan resolution (true)
	// or at the configured resolution with post-render scaling (false).
	uint8 bExternalOverscanOversize : 1 = 1;

	// Overrides the camera overscan value when bUseExternalOverscan is true.
	// When not set, the overscan is read from FMinimalViewInfo.GetOverscan().
	TOptional<float> ExternalOverscanOverride;

	/** Settings for the current cluster node. */
	struct FCurrentClusterNodeSettings
	{
		// True if this cluster node is configured to render headless.
		// Mirrors UDisplayClusterConfigurationClusterNode::bRenderHeadless.
		uint8 bRenderHeadless : 1 = 0;

		// True if this cluster node has a backbuffer media output assigned.
		uint8 bHasBackbufferMediaOutput : 1 = 0;

		// Cluster node name for render
		FString Id;

	} CurrentNode;

public:
	// Scales the render resolution of all viewports uniformly.
	// Set via IDisplayClusterViewportConfiguration::SetRenderResolutionScale().
	float RenderResolutionScale = 1.f;

	// Multiply all viewports RTT size's for whole cluster by this value
	float ClusterRenderTargetRatioMult = 1.f;

	// Multiply inner frustum RTT size's for whole cluster by this value
	float ClusterICVFXInnerViewportRenderTargetRatioMult = 1.f;

	// Multiply outer viewports RTT size's for whole cluster by this value
	float ClusterICVFXOuterViewportRenderTargetRatioMult = 1.f;

	// Multiply all buffer ratios for whole cluster by this value
	float ClusterBufferRatioMult = 1.f;

	// Multiply inner frustums buffer ratios for whole cluster by this value
	float ClusterICVFXInnerFrustumBufferRatioMult = 1.f;

	// Multiply outer viewports buffer ratios for whole cluster by this value
	float ClusterICVFXOuterViewportBufferRatioMult = 1.f;

	// Settings for preview rendering
	FDisplayClusterViewport_PreviewSettings PreviewSettings;

	// [Experimental] Render preview in multi-GPU
	// Specifies the mGPU index range for rendering the DCRA preview.
	TOptional<FIntPoint> PreviewMultiGPURendering;

public:
	/** Returns true if ICVFX rendering can be used. */
	bool IsICVFXEnabled() const;

	/** Current frame is preview. */
	bool IsPreviewRendering() const;

	/** Returns true if this DCRA is using cluster rendering. */
	bool IsClusterRendering() const;

	/** Returns true, if Techvis is used. */
	bool IsTechvisEnabled() const;

	/** Returns true if the DCRA preview feature in Standalone/Package builds is used. */
	bool IsPreviewInGameEnabled() const;

	/** returns true if the preview rendering has been updated.If this function returns false, the DCRA preview image should be frozen. */
	bool IsPreviewFreezeRender() const;

	/** [Experimental] Get the GPU range used for rendering preview.
	* returns nullptr if this rendering function is disabled on mGPU
	*/
	const FIntPoint* GetPreviewMultiGPURendering() const;

	/** True if OCIO is enabled for nDisplay. */
	bool IsEnabledOpenColorIO() const;

	/** PostProcess should be disabled for the current frame. */
	bool IsPostProcessDisabled() const;

	/** Should use linear gamma. */
	bool ShouldUseLinearGamma() const;

	/** Should use Holdout. */
	bool ShouldUseHoldout() const;

	/** Is stereo rendering on monoscopic display (sbs, tb) . */
	bool ShouldUseStereoRenderingOnMonoscopicDisplay() const;

	/** Getting the desired frame size multipliers. */
	FVector2D GetDesiredFrameMult() const;

	/** Obtain the desired RTT size (for sbs and tb this is half the size in one of the dimensions). */
	FVector2D GetDesiredRTTSize(const FVector2D& InSize) const;
	FIntPoint GetDesiredRTTSize(const FIntPoint& InSize) const;

	/** 
	* Limit viewport size within:
	* 1. MaxArea - for main viewports only, controlled by ShouldApplyMaxTextureConstraints().
	* 2. Maximum texture size based on hardware limitations	* The output size will have the same aspect ratio as the input size.
	* 
	* @param InViewport - the nDisplay viewport for which the size is used.
	* @param InSize     - The base size of the viewport.
	*/
	FIntPoint ApplyViewportSizeConstraint(const class FDisplayClusterViewport& InViewport, const FIntPoint& InSize) const;

	/** Should use Cross-GPU transfers. */
	inline bool ShouldUseCrossGPUTransfers() const
	{
		return bUseDisplayClusterRenderDevice || GetPreviewMultiGPURendering() != nullptr;
	}

	/** Ability to reuse the viewport across all nodes in the cluster (DCRA Preview). */
	bool CanReuseViewportWithinClusterNodes() const;

	/** Return number of views per viewport. */
	int32 GetViewPerViewportAmount() const;

	/** Return current frame number. */
	inline uint32 GetFrameNumber() const
	{
		if (FrameNumberOverride.IsSet())
		{
			return *FrameNumberOverride;
		}

		return GFrameNumber;
	}
};
