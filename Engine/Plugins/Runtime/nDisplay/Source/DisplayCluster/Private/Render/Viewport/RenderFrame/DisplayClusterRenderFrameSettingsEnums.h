// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * Select the method to save the alpha channel for nDisplay rendering.
 */
enum class EDisplayClusterRenderFrameAlphaChannelCaptureMode: uint8
{
	/** [None]- Do not capture alpha. */
	None = 0,

	/** [ThroughTonemapper] - Render alpha Through Tonemapper, Copy alpha from PP input texture to final.
	* 
	 * When rendering with the PropagateAlpha experimental mode turned on, the alpha channel is forwarded to post-processes.
	 * In this case, the alpha channel is anti-aliased along with the color.
	 * Since some post-processing may change the alpha, it is copied at the beginning of the PP and restored after all post-processing is completed.
	 */
	ThroughTonemapper,

	/** [FXAA] - Disable AA and TAA, Use FXAA for RGB, Copy alpha from scenecolor texture to final.
	* 
	 * Otherwise, if the PropagateAlpha mode is disabled in the project settings, we need to save the alpha before it becomes invalid.
	 * The alpha is valid until the scene color is resolved (on the ResolvedSceneColor callback). Therefore, it is copied to a temporary resource on this cb.
	 * Since we need to remove AA jittering (because alpha copied before AA), anti-aliasing is turned off for this viewport.
	 * And finally the FXAA is used for smoothing.
	 */
	 FXAA,

	/** [CopyAA] - Use AA, disable TAA, Copy alpha from scenecolor texture to final
	* 
	 * These experimental (temporary) modes for the performance tests.
	 */
	CopyAA,

	/** [Copy] - Disable AA and TAA, Copy alpha from scenecolor texture to final
	* 
	 * These experimental (temporary) modes for the performance tests.
	 */
	Copy
};
