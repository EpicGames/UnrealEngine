// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Viewport/RenderTarget/DisplayClusterRenderTargetManager.h"
#include "Render/Viewport/RenderTarget/DisplayClusterRenderTargetResourcesPool.h"
#include "Render/Viewport/RenderTarget/DisplayClusterRenderTargetResource.h"

#include "Render/Viewport/DisplayClusterViewport.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_Enums.h"
#include "Render/Viewport/Configuration/DisplayClusterViewportConfiguration.h"

#include "Render/Viewport/RenderFrame/DisplayClusterRenderFrame.h"
#include "Render/Viewport/RenderFrame/DisplayClusterRenderFrameSettings.h"

#include "Misc/DisplayClusterLog.h"

////////////////////////////////////////////////////////////////////////////////////////////////////////////
int32 GDisplayClusterSceneColorFormat = 0;
static FAutoConsoleVariableRef CVarDisplayClusterSceneColorFormat(
	TEXT("nDisplay.render.SceneColorFormat"),
	GDisplayClusterSceneColorFormat,
	TEXT("Defines the memory layout (RGBA) used for the scene color\n"
		"(affects performance, mostly through bandwidth, quality especially with translucency).\n"
		" 0: Use the backbuffer format\n"
		" 1: PF_FloatRGBA 64Bit (default, might be overkill, especially if translucency is mostly using SeparateTranslucency)\n"
		" 2: PF_A32B32G32R32F 128Bit (unreasonable but good for testing)"),
	ECVF_Default
);

////////////////////////////////////////////////////////////////////////////////////////////////////////////
namespace UE::DisplayCluster::RenderTargetManager
{
	/** Return custom flags for capture mode. */
	static EDisplayClusterViewportResourceSettingsFlags ImplGetCustomFlags(EDisplayClusterViewportCaptureMode CaptureMode)
	{
		switch (CaptureMode)
		{
		case EDisplayClusterViewportCaptureMode::Chromakey:
		case EDisplayClusterViewportCaptureMode::Lightcard:

			// LightCards renders in linear color space
			return EDisplayClusterViewportResourceSettingsFlags::LinearColorSpace;

		default:
			break;
		}

		return EDisplayClusterViewportResourceSettingsFlags::None;
	}

	/** Return texture format for capture mode. */
	static EPixelFormat ImplGetCustomFormat(EDisplayClusterViewportCaptureMode CaptureMode)
	{
		EPixelFormat OutPixelFormat = EPixelFormat::PF_Unknown;

		// Pre-defined formats for targets:
		switch (CaptureMode)
		{
		case EDisplayClusterViewportCaptureMode::Chromakey:
		case EDisplayClusterViewportCaptureMode::Lightcard:
			// The Chromakey and LightCard always uses PF_FloatRGBA for OCIO support.
			OutPixelFormat = EPixelFormat::PF_FloatRGBA;
			break;

		case EDisplayClusterViewportCaptureMode::MoviePipeline:
			// Movie pipeline always use PF_FloatRGBA
			OutPixelFormat = EPixelFormat::PF_FloatRGBA;
			break;

		default:
			switch (GDisplayClusterSceneColorFormat)
			{
			case 1:
				OutPixelFormat = EPixelFormat::PF_FloatRGBA;
				break;
			case 2:
				OutPixelFormat = EPixelFormat::PF_A32B32G32R32F;
				break;

			default:
				// by default use the backbuffer format (when function returns PF_Unknown its mean use default scene format)
				break;
			}
			break;
		}

		if (OutPixelFormat != EPixelFormat::PF_Unknown)
		{
			// Fallback in case the scene color selected isn't supported.
			if (!GPixelFormats[OutPixelFormat].Supported)
			{
				OutPixelFormat = PF_FloatRGBA;
			}
		}

		return OutPixelFormat;
	}

	static void ImplViewportTextureResourceLogging(const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& InViewport, const uint32 ContextNum, const FString& ResourceId, const TSharedPtr<FDisplayClusterViewportResource, ESPMode::ThreadSafe>& InTextureResource)
	{
		if (InViewport.IsValid() && InTextureResource != nullptr && EnumHasAnyFlags(InTextureResource->GetResourceState(), EDisplayClusterViewportResourceState::Initialized) == false)
		{
			// Log: New resource created
			UE_LOGF(LogDisplayClusterViewport, Verbose, "Created new %ls resource (%dx%d) for viewport '%ls':%d", *ResourceId, InTextureResource->GetResourceSettings().GetSizeXY().X, InTextureResource->GetResourceSettings().GetSizeXY().Y, *InViewport->GetId(), ContextNum);
		}
	}

	static FString EmptyString;
};
using namespace UE::DisplayCluster::RenderTargetManager;

////////////////////////////////////////////////////////////////////////////////////////////////////////////
/// FDisplayClusterRenderTargetManager
////////////////////////////////////////////////////////////////////////////////////////////////////////////
FDisplayClusterRenderTargetManager::FDisplayClusterRenderTargetManager(const TSharedRef<FDisplayClusterViewportConfiguration, ESPMode::ThreadSafe>& InConfiguration)
	: Configuration(InConfiguration)
{
	ResourcesPool = MakeUnique<FDisplayClusterRenderTargetResourcesPool>();
}

FDisplayClusterRenderTargetManager::~FDisplayClusterRenderTargetManager()
{
	Release();
}

void FDisplayClusterRenderTargetManager::Release()
{
	ResourcesPool->Release();
}

bool FDisplayClusterRenderTargetManager::AllocateRenderFrameResources(FViewport* InViewport, const TArray<TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>>& InViewports, FDisplayClusterRenderFrame& InOutRenderFrame)
{
	bool bResult = true;

	if(ResourcesPool->BeginReallocateResources(InViewport, Configuration->GetRenderFrameSettings()))
	{
		// ReAllocate Render targets for all viewports
		for (FDisplayClusterRenderFrameTarget& FrameRenderTargetIt : InOutRenderFrame.RenderTargets)
		{
			if (FrameRenderTargetIt.bShouldUseRenderTarget)
			{
				const EDisplayClusterViewportResourceSettingsFlags CustomFlags = ImplGetCustomFlags(FrameRenderTargetIt.CaptureMode);
				const EPixelFormat CustomFormat = ImplGetCustomFormat(FrameRenderTargetIt.CaptureMode);

				// reallocate
				TSharedPtr<FDisplayClusterViewportResource, ESPMode::ThreadSafe> NewResource = ResourcesPool->AllocateResource(EmptyString, FrameRenderTargetIt.RenderTargetSize, CustomFormat, EDisplayClusterViewportResourceSettingsFlags::RenderTarget | CustomFlags);
				if (NewResource.IsValid())
				{
					// Set RenderFrame resource
					FrameRenderTargetIt.RenderTargetResource = NewResource;

					// Assign for all views in render target families
					for (FDisplayClusterRenderFrameTargetViewFamily& ViewFamily : FrameRenderTargetIt.ViewFamilies)
					{
						for (FDisplayClusterRenderFrameTargetView& ViewIt : ViewFamily.Views)
						{
							if (FDisplayClusterViewport* ViewportPtr = static_cast<FDisplayClusterViewport*>(ViewIt.Viewport.Get()))
							{
								// Array already resized in function FDisplayClusterViewport::UpdateFrameContexts() with RenderTargets.AddZeroed(ViewportContextAmount);
								check(ViewportPtr->GetViewportResources(EDisplayClusterViewportResource::RenderTargets).IsValidIndex(ViewIt.ContextNum));

								ViewportPtr->GetViewportResourcesImpl(EDisplayClusterViewportResource::RenderTargets)[ViewIt.ContextNum] = NewResource;

								if (EnumHasAnyFlags(NewResource->GetResourceState(), EDisplayClusterViewportResourceState::Initialized) == false)
								{
									// Log: New resource created
									UE_LOGF(LogDisplayClusterViewport, Verbose, "Created new ViewportRenderTarget resource %08llX (%dx%d) for viewport '%ls'", NewResource.Get(), NewResource->GetResourceSettings().GetSizeX(), NewResource->GetResourceSettings().GetSizeY(), *ViewportPtr->GetId());
								}
							}
						}
					}
				}
			}
		}

		// Allocate viewport internal resources
		for (const TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>& ViewportIt : InViewports)
		{
			if (ViewportIt.IsValid())
			{
				const EDisplayClusterViewportCaptureMode CaptureMode = ViewportIt->GetRenderSettings().CaptureMode;

				const EDisplayClusterViewportResourceSettingsFlags CustomFlags = ImplGetCustomFlags(CaptureMode);
				const EPixelFormat CustomFormat = ImplGetCustomFormat(CaptureMode);

				// Allocate all context resources:
				for (const FDisplayClusterViewport_Context& ContextIt : ViewportIt->GetContexts())
				{
					const FIntPoint& ContextSize = ContextIt.ContextSize;
					const uint32& ContextNum = ContextIt.ContextNum;

					// Allocate per-viewport preview output textures:
					if (ViewportIt->GetViewportResources(EDisplayClusterViewportResource::OutputPreviewTargetableResources).IsValidIndex(ContextNum))
					{
						TSharedPtr<FDisplayClusterViewportResource, ESPMode::ThreadSafe>& ReallocResource = ViewportIt->GetViewportResourcesImpl(EDisplayClusterViewportResource::OutputPreviewTargetableResources)[ContextNum];
						if (!ReallocResource.IsValid() || !EnumHasAnyFlags(ReallocResource->GetResourceState(), EDisplayClusterViewportResourceState::DisableReallocate))
						{
							ReallocResource = ResourcesPool->AllocateResource(ViewportIt->GetId(), ContextSize, CustomFormat, EDisplayClusterViewportResourceSettingsFlags::PreviewTargetableTexture | CustomFlags);
							ImplViewportTextureResourceLogging(ViewportIt, ContextNum, TEXT("InputShader"), ViewportIt->GetViewportResources(EDisplayClusterViewportResource::OutputPreviewTargetableResources)[ContextNum]);
						}
					}

					if (ViewportIt->GetViewportResources(EDisplayClusterViewportResource::InputShaderResources).IsValidIndex(ContextNum))
					{
						TSharedPtr<FDisplayClusterViewportResource, ESPMode::ThreadSafe>& ReallocResource = ViewportIt->GetViewportResourcesImpl(EDisplayClusterViewportResource::InputShaderResources)[ContextNum];
						if (!ReallocResource.IsValid() || !EnumHasAnyFlags(ReallocResource->GetResourceState(), EDisplayClusterViewportResourceState::DisableReallocate))
						{
							ReallocResource = ResourcesPool->AllocateResource(EmptyString, ContextSize, CustomFormat,
								EDisplayClusterViewportResourceSettingsFlags::RenderTargetableTexture // 'InputShader' resource can be used as RTT.
								| EDisplayClusterViewportResourceSettingsFlags::ResolveTargetableTexture | CustomFlags);
							ImplViewportTextureResourceLogging(ViewportIt, ContextNum, TEXT("InputShader"), ViewportIt->GetViewportResources(EDisplayClusterViewportResource::InputShaderResources)[ContextNum]);
						}
					}

					// Allocate custom resources:
					if (ViewportIt->GetViewportResources(EDisplayClusterViewportResource::AdditionalTargetableResources).IsValidIndex(ContextNum))
					{
						TSharedPtr<FDisplayClusterViewportResource, ESPMode::ThreadSafe>& ReallocResource = ViewportIt->GetViewportResourcesImpl(EDisplayClusterViewportResource::AdditionalTargetableResources)[ContextNum];
						if (!ReallocResource.IsValid() || !EnumHasAnyFlags(ReallocResource->GetResourceState(), EDisplayClusterViewportResourceState::DisableReallocate))
						{
							ReallocResource = ResourcesPool->AllocateResource(EmptyString, ContextSize, CustomFormat, EDisplayClusterViewportResourceSettingsFlags::RenderTargetableTexture | CustomFlags);
							ImplViewportTextureResourceLogging(ViewportIt, ContextNum, TEXT("AdditionalRTT"), ViewportIt->GetViewportResources(EDisplayClusterViewportResource::AdditionalTargetableResources)[ContextNum]);
						}
					}

					if (ViewportIt->GetViewportResources(EDisplayClusterViewportResource::MipsShaderResources).IsValidIndex(ContextNum))
					{
						TSharedPtr<FDisplayClusterViewportResource, ESPMode::ThreadSafe>& ReallocResource = ViewportIt->GetViewportResourcesImpl(EDisplayClusterViewportResource::MipsShaderResources)[ContextNum];
						if (!ReallocResource.IsValid() || !EnumHasAnyFlags(ReallocResource->GetResourceState(), EDisplayClusterViewportResourceState::DisableReallocate))
						{
							ReallocResource = ResourcesPool->AllocateResource(EmptyString, ContextSize, CustomFormat, EDisplayClusterViewportResourceSettingsFlags::ResolveTargetableTexture | CustomFlags, ContextIt.NumMips);
							ImplViewportTextureResourceLogging(ViewportIt, ContextNum, TEXT("Mips"), ViewportIt->GetViewportResources(EDisplayClusterViewportResource::MipsShaderResources)[ContextNum]);
						}
					}
				}
			}
		}

		// Allocate frame targets for all visible  on backbuffer viewports
		FIntPoint ViewportSize = InViewport ? InViewport->GetSizeXY() : FIntPoint(0, 0);

		if (!AllocateFrameTargets(ViewportSize, InOutRenderFrame))
		{
			UE_LOGF(LogDisplayClusterViewport, Error, "DisplayClusterRenderTargetManager: Can't allocate frame targets.");
			bResult = false;
		}

		ResourcesPool->EndReallocateResources();
	}

	return bResult;
}

bool FDisplayClusterRenderTargetManager::AllocateFrameTargets(const FIntPoint& InViewportSize, FDisplayClusterRenderFrame& InOutRenderFrame)
{
	// ICVFX internal viewports are invisible and do not use output frame resources.
	// In case of off-screen rendering this texture is not used.
	if (!Configuration->GetRenderFrameSettings().bShouldUseOutputFrameTargetableResources)
	{
		// Skip creating output resources if there are no visible viewports. This will save GPU memory.
		return true;
	}

	const uint32 FrameTargetsAmount = Configuration->GetRenderFrameSettings().GetViewPerViewportAmount();
	const FIntPoint DesiredRTTSize = Configuration->GetRenderFrameSettings().GetDesiredRTTSize(InViewportSize);
	FIntPoint TargetLocation(ForceInitToZero);

	// Offset for stereo rendering on a monoscopic display (side-by-side or top-bottom)
	const FIntPoint TargetOffset(
		DesiredRTTSize.X < InViewportSize.X ? DesiredRTTSize.X : 0,
		DesiredRTTSize.Y < InViewportSize.Y ? DesiredRTTSize.Y : 0
	);

	// Reallocate frame target resources
	TArray<TSharedPtr<FDisplayClusterViewportResource, ESPMode::ThreadSafe>> NewFrameTargetResources;
	TArray<TSharedPtr<FDisplayClusterViewportResource, ESPMode::ThreadSafe>> NewAdditionalFrameTargetableResources;

	for (uint32 FrameTargetsIt = 0; FrameTargetsIt < FrameTargetsAmount; FrameTargetsIt++)
	{
		TSharedPtr<FDisplayClusterViewportResource, ESPMode::ThreadSafe> NewResource = ResourcesPool->AllocateResource(EmptyString, InOutRenderFrame.FrameRect.Size(), PF_Unknown, EDisplayClusterViewportResourceSettingsFlags::RenderTargetableTexture);
		if (NewResource.IsValid())
		{
			if (EnumHasAnyFlags(NewResource->GetResourceState(), EDisplayClusterViewportResourceState::Initialized) == false)
			{
				// Log: New resource created
				UE_LOGF(LogDisplayClusterViewport, Verbose, "Created new RenderFrame resource (%dx%d)", NewResource->GetResourceSettings().GetSizeX(), NewResource->GetResourceSettings().GetSizeY());
			}

			// calc and assign backbuffer offset (side_by_side, top_bottom)
			NewResource->SetBackbufferFrameOffset(InOutRenderFrame.FrameRect.Min + TargetLocation);
			TargetLocation += TargetOffset;
			NewFrameTargetResources.Add(NewResource);
		}

		if (Configuration->GetRenderFrameSettings().bShouldUseAdditionalFrameTargetableResource)
		{
			TSharedPtr<FDisplayClusterViewportResource, ESPMode::ThreadSafe> NewAdditionalResource = ResourcesPool->AllocateResource(EmptyString, InOutRenderFrame.FrameRect.Size(), PF_Unknown, EDisplayClusterViewportResourceSettingsFlags::RenderTargetableTexture);
			if (NewAdditionalResource.IsValid())
			{
				if (EnumHasAnyFlags(NewAdditionalResource->GetResourceState(), EDisplayClusterViewportResourceState::Initialized) == false)
				{
					// Log: New resource created
					UE_LOGF(LogDisplayClusterViewport, Verbose, "Created new RenderFrame2 resource (%dx%d)", NewAdditionalResource->GetResourceSettings().GetSizeX(), NewAdditionalResource->GetResourceSettings().GetSizeY());
				}

				NewAdditionalFrameTargetableResources.Add(NewAdditionalResource);
			}
		}
	}

	// Assign frame resources for all visible viewports
	for (FDisplayClusterRenderFrameTarget& RenderTargetIt : InOutRenderFrame.RenderTargets)
	{
		for (FDisplayClusterRenderFrameTargetViewFamily& ViewFamilieIt : RenderTargetIt.ViewFamilies)
		{
			for (FDisplayClusterRenderFrameTargetView& ViewIt : ViewFamilieIt.Views)
			{
				FDisplayClusterViewport* ViewportPtr = static_cast<FDisplayClusterViewport*>(ViewIt.Viewport.Get());
				if (ViewportPtr && ViewportPtr->GetRenderSettings().bVisible)
				{
					ViewportPtr->GetViewportResourcesImpl(EDisplayClusterViewportResource::OutputFrameTargetableResources) = NewFrameTargetResources;
					ViewportPtr->GetViewportResourcesImpl(EDisplayClusterViewportResource::AdditionalFrameTargetableResources) = NewAdditionalFrameTargetableResources;

					// Adjust viewports frame rects. This offset saved in 'BackbufferFrameOffset'
					if (ViewportPtr->GetContexts().IsValidIndex(ViewIt.ContextNum))
					{
						TArray<FDisplayClusterViewport_Context> ViewportContexts = ViewportPtr->GetContexts();

						FDisplayClusterViewport_Context& Context = ViewportContexts[ViewIt.ContextNum];
						Context.FrameTargetRect.Min -= InOutRenderFrame.FrameRect.Min;
						Context.FrameTargetRect.Max -= InOutRenderFrame.FrameRect.Min;

						// Override data in viewport context:
						ViewportPtr->SetContexts(ViewportContexts);
					}
				}
			}
		}
	}

	return true;
}
