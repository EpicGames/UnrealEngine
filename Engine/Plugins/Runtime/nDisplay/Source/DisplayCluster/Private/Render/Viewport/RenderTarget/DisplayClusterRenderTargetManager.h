// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Render/Viewport/DisplayClusterViewport.h"

class FViewport;
class FDisplayClusterViewport;
class FDisplayClusterRenderTargetResourcesPool;
class FDisplayClusterViewportConfiguration;
class FDisplayClusterRenderFrame;

class FDisplayClusterRenderTargetManager
{
public:
	FDisplayClusterRenderTargetManager(const TSharedRef<FDisplayClusterViewportConfiguration, ESPMode::ThreadSafe>& InConfiguration);
	~FDisplayClusterRenderTargetManager();

	void Release();

public:
	bool AllocateRenderFrameResources(FViewport* InViewport, const TArray<TSharedPtr<FDisplayClusterViewport, ESPMode::ThreadSafe>>& InViewports, FDisplayClusterRenderFrame& InOutRenderFrame);

protected:
	bool AllocateFrameTargets(const FIntPoint& InViewportSize, FDisplayClusterRenderFrame& InOutRenderFrame);

private:
	TUniquePtr<FDisplayClusterRenderTargetResourcesPool> ResourcesPool;

	// Configuration of the current cluster node
	const TSharedRef<FDisplayClusterViewportConfiguration, ESPMode::ThreadSafe> Configuration;
};
