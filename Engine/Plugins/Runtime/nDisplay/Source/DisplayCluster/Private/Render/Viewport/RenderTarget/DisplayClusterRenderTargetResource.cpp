// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Viewport/RenderTarget/DisplayClusterRenderTargetResource.h"

#include "RHI.h"
#include "RenderResource.h"
#include "UnrealClient.h"

////////////////////////////////////////////////////////////////////////
// FDisplayClusterViewportTextureResource
////////////////////////////////////////////////////////////////////////
FDisplayClusterViewportTextureResource::FDisplayClusterViewportTextureResource(const FDisplayClusterViewportResourceSettings& InResourceSettings)
	: FDisplayClusterViewportResource(InResourceSettings)
{
	bSRGB = EnumHasAnyFlags(GetResourceSettings().GetResourceFlags(), EDisplayClusterViewportResourceSettingsFlags::ShouldUseSRGB);
	bGreyScaleFormat = false;
}

void FDisplayClusterViewportTextureResource::InitRHI(FRHICommandListBase& RHICmdList)
{
	FTextureRHIRef NewTextureRHI;

	if (GetResourceSettings().GetNumMips() > 1)
	{
		FTextureRHIRef DummyTextureRHI;
		ImplInitDynamicRHI_RenderTargetResource2D(RHICmdList, NewTextureRHI, DummyTextureRHI);
	}
	else
	{
		ImplInitDynamicRHI_TextureResource2D(RHICmdList, NewTextureRHI);
	}

	TextureRHI = (FTextureRHIRef&)NewTextureRHI;
}

////////////////////////////////////////////////////////////////////////
// FDisplayClusterViewportRenderTargetResource
////////////////////////////////////////////////////////////////////////
void FDisplayClusterViewportRenderTargetResource::InitRHI(FRHICommandListBase& RHICmdList)
{
	// Create RTT and shader resources
	FTextureRHIRef NewTextureRHI;
	ImplInitDynamicRHI_RenderTargetResource2D(RHICmdList, RenderTargetTextureRHI, NewTextureRHI);
	TextureRHI = (FTextureRHIRef&)NewTextureRHI;

	// Create the sampler state RHI resource.
	FSamplerStateInitializerRHI SamplerStateInitializer
	(
		SF_Bilinear,
		AM_Clamp,
		AM_Clamp,
		AM_Clamp
	);
	SamplerStateRHI = GetOrCreateSamplerState(SamplerStateInitializer);
}
