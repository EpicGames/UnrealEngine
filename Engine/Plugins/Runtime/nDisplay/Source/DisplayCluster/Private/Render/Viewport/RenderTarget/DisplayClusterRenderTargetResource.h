// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Render/Viewport/Resource/DisplayClusterViewportResource.h"

#include "RHI.h"
#include "RHIResources.h"
#include "RenderResource.h"
#include "UnrealClient.h"

/**
 * Viewport texture resource
 */
class FDisplayClusterViewportTextureResource
	: public FDisplayClusterViewportResource
	, public FTexture
{
public:
	FDisplayClusterViewportTextureResource(const FDisplayClusterViewportResourceSettings& InResourceSettings);

	virtual ~FDisplayClusterViewportTextureResource()
	{ }

public:
	//~ Begin FDisplayClusterViewportResource
	virtual FRHITexture* GetViewportResourceRHI_RenderThread() const override
	{
		check(IsInRenderingThread());

		return TextureRHI.IsValid() ? TextureRHI->GetTexture2D() : nullptr;
	}

	virtual FIntPoint GetBackbufferFrameOffset() const override
	{
		return BackbufferFrameOffset;
	}

	virtual void SetBackbufferFrameOffset(const FIntPoint& InBackbufferFrameOffset) override
	{
		BackbufferFrameOffset = InBackbufferFrameOffset;
	}

	virtual void InitializeViewportResource_RenderThread(FRHICommandListBase& RHICmdList) override
	{
		// Call FTexture::InitResource()
		InitResource(RHICmdList);
	}

	virtual void ReleaseViewportResource_RenderThread(FRHICommandListBase& RHICmdList) override
	{
		// Call FTexture::ReleaseResource()
		ReleaseResource();
	}
	//~~ End FDisplayClusterViewportResource

	//~ Begin FTexture
	virtual uint32 GetSizeX() const override
	{
		return GetResourceSettings().GetSizeXY().X;
	}

	virtual uint32 GetSizeY() const override
	{
		return GetResourceSettings().GetSizeXY().Y;
	}

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;

	virtual FString GetFriendlyName() const override
	{
		return TEXT("DisplayClusterViewportTextureResource");
	}
	//~~ End FTexture

public:
	// OutputFrameTargetableResources frame offset on backbuffer (special for 'side_by_side' and 'top_bottom' DCRenderDevices)
	FIntPoint BackbufferFrameOffset = FIntPoint::ZeroValue;
};

/**
 * Viewport RTT resource
 */
class FDisplayClusterViewportRenderTargetResource
	: public FDisplayClusterViewportTextureResource
	, public FRenderTarget
{
public:
	FDisplayClusterViewportRenderTargetResource(const FDisplayClusterViewportResourceSettings& InResourceSettings)
		: FDisplayClusterViewportTextureResource(InResourceSettings)
	{ }

	virtual ~FDisplayClusterViewportRenderTargetResource()
	{ }

public:
	//~ Begin FDisplayClusterViewportResource
	virtual FRenderTarget* GetViewportResourceRenderTarget() override
	{
		return this;
	}
	//~~ End FDisplayClusterViewportResource

	//~ Begin FRenderTarget
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;

	virtual FIntPoint GetSizeXY() const override
	{
		return GetResourceSettings().GetSizeXY();
	}

	virtual float GetDisplayGamma() const override
	{
		return GetResourceSettings().GetDisplayGamma();
	}

	virtual FString GetFriendlyName() const override 
	{
		return TEXT("DisplayClusterViewportRenderTargetResource");
	}
	//~~ End FRenderTarget
};
