// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterRenderTargetResourcesPool.h"
#include "DisplayClusterRenderTargetResource.h"

#include "Render/Viewport/Configuration/DisplayClusterViewportConfiguration.h"
#include "Render/Viewport/Resource/DisplayClusterViewportPreviewResource.h"
#include "Render/Viewport/RenderFrame/DisplayClusterRenderFrameSettings.h"
#include "Render/Viewport/DisplayClusterViewportHelpers.h"
#include "Render/Viewport/DisplayClusterViewportManagerProxy.h"

#include "RHICommandList.h"
#include "Engine/RendererSettings.h"
#include "RenderingThread.h"

////////////////////////////////////////////////////////////////////////////////////////////////////////////
/// FDisplayClusterRenderTargetResourcesPool
////////////////////////////////////////////////////////////////////////////////////////////////////////////
FDisplayClusterRenderTargetResourcesPool::~FDisplayClusterRenderTargetResourcesPool()
{
	Release();
}

void FDisplayClusterRenderTargetResourcesPool::Release()
{
	// Release all resources
	ImplUpdateResources(ViewportResources, EResourceUpdateMode::Release);
}

void FDisplayClusterRenderTargetResourcesPool::ImplUpdateResources(TArray<TSharedPtr<FDisplayClusterViewportResource, ESPMode::ThreadSafe>>& InOutViewportResources, const EResourceUpdateMode InUpdateMode)
{
	if (InOutViewportResources.IsEmpty())
	{
		return;
	}

	// Update 'Initialized' resource state
	for (const TSharedPtr<FDisplayClusterViewportResource, ESPMode::ThreadSafe>& ResourceIt : InOutViewportResources)
	{
		if (ResourceIt.IsValid())
		{
			switch (InUpdateMode)
			{
			case EResourceUpdateMode::Release:
				ResourceIt->ReleaseViewportResource();
				break;

			default:
			case EResourceUpdateMode::Initialize:
				ResourceIt->InitializeViewportResource();
				break;
			}
		}
	}

	// Update all viewports resources on rendering thread
	ENQUEUE_RENDER_COMMAND(DisplayCluster_UpdateViewportResources)
	([ViewportResources = InOutViewportResources, InUpdateMode](FRHICommandListImmediate& RHICmdList)
	{
		for (const TSharedPtr<FDisplayClusterViewportResource, ESPMode::ThreadSafe>& ResourceIt : ViewportResources)
		{
			if (ResourceIt.IsValid())
			{
				switch (InUpdateMode)
				{
				case EResourceUpdateMode::Release:
					ResourceIt->ReleaseViewportResource_RenderThread(RHICmdList);
					break;

				default:
				case EResourceUpdateMode::Initialize:
					ResourceIt->InitializeViewportResource_RenderThread(RHICmdList);
					break;
				}
			}
		}
	});

	InOutViewportResources.Empty();
}

bool FDisplayClusterRenderTargetResourcesPool::BeginReallocateResources(FViewport* InViewport, const FDisplayClusterRenderFrameSettings& InRenderFrameSettings)
{
	check(ResourceSettings == nullptr);

	// Initialize settings for new render frame
	ResourceSettings = new FDisplayClusterViewportResourceSettings(InRenderFrameSettings, InViewport);

	// Begin reallocate resources for current cluster node
	// Mark cluster node resources as unused
	for (TSharedPtr<FDisplayClusterViewportResource, ESPMode::ThreadSafe>& ViewportResourceIt : ViewportResources)
	{
		if (ViewportResourceIt.IsValid() && EnumHasAnyFlags(ViewportResourceIt->GetResourceState(), EDisplayClusterViewportResourceState::DisableReallocate) == false)
		{
			if (const bool bIsCurrentClusterNodeResource = ViewportResourceIt->GetResourceSettings().IsClusterNodeNameEqual(*ResourceSettings))
			{
				EnumAddFlags(ViewportResourceIt->GetResourceState(), EDisplayClusterViewportResourceState::Unused);
			}
			else
			{
				EnumAddFlags(ViewportResourceIt->GetResourceState(), EDisplayClusterViewportResourceState::DisableReallocate);
			}
		}
	}

	return true;
}

void FDisplayClusterRenderTargetResourcesPool::EndReallocateResources()
{
	// Finish reallocate resources for current cluster node
	if (ResourceSettings != nullptr)
	{
		// Collect unused resources for removal.
		TArray<TSharedPtr<FDisplayClusterViewportResource, ESPMode::ThreadSafe>> UnusedResources;

		// Collect new resources
		TArray<TSharedPtr<FDisplayClusterViewportResource, ESPMode::ThreadSafe>> NewResources;

		for (TSharedPtr<FDisplayClusterViewportResource, ESPMode::ThreadSafe>& ResourceIt : ViewportResources)
		{
			if (ResourceIt.IsValid())
			{
				if (EnumHasAnyFlags(ResourceIt->GetResourceState(), EDisplayClusterViewportResourceState::Unused))
				{
					// Collect unused resources
					UnusedResources.Add(ResourceIt);
				}
				else if (!EnumHasAnyFlags(ResourceIt->GetResourceState(), EDisplayClusterViewportResourceState::Initialized))
				{
					// Collect new resources
					NewResources.Add(ResourceIt);
				}

				// Clear 'DisableReallocate' at frame end
				EnumRemoveFlags(ResourceIt->GetResourceState(), EDisplayClusterViewportResourceState::DisableReallocate);
			}
		}

		// Remove GameThread resource references
		for (const TSharedPtr<FDisplayClusterViewportResource, ESPMode::ThreadSafe>& ResourceIt : UnusedResources)
		{
			int ResourceIndex = INDEX_NONE;
			while (ViewportResources.Find(ResourceIt, ResourceIndex))
			{
				ViewportResources.RemoveAt(ResourceIndex);
			}
		}

		// Initialize new resources on rendering thread:
		ImplUpdateResources(NewResources, EResourceUpdateMode::Initialize);

		// Remove unused resources
		ImplUpdateResources(UnusedResources, EResourceUpdateMode::Release);
	}

	delete ResourceSettings;
	ResourceSettings = nullptr;
}

TSharedPtr<FDisplayClusterViewportResource, ESPMode::ThreadSafe> FDisplayClusterRenderTargetResourcesPool::AllocateResource(const FString InViewportId, const FIntPoint& InSize, EPixelFormat CustomPixelFormat, const EDisplayClusterViewportResourceSettingsFlags InResourceFlags, int32 InNumMips)
{
	check(ResourceSettings != nullptr);

	FDisplayClusterViewportResourceSettings InSettings(*ResourceSettings, InViewportId, InSize, CustomPixelFormat, InResourceFlags, InNumMips);

	if (!FDisplayClusterViewportHelpers::IsValidTextureSize(InSettings.GetSizeXY()))
	{
		return nullptr;
	}

	// Unused resources marked for current cluster node
	auto ExistAndUnusedResourceIndex = ViewportResources.IndexOfByPredicate([InSettings](const TSharedPtr<FDisplayClusterViewportResource, ESPMode::ThreadSafe>& ResourceIt)
		{
			if (ResourceIt.IsValid() && EnumHasAnyFlags(ResourceIt->GetResourceState(), EDisplayClusterViewportResourceState::Unused) == true)
			{
				return ResourceIt->GetResourceSettings().IsResourceSettingsEqual(InSettings);
			}

			return false;
		});

	if (ExistAndUnusedResourceIndex != INDEX_NONE)
	{
		// Use exist resource again
		TSharedPtr<FDisplayClusterViewportResource, ESPMode::ThreadSafe> ExistResource = ViewportResources[ExistAndUnusedResourceIndex];

		if (ExistResource.IsValid())
		{
			// Clear unused state for re-used resource
			EnumRemoveFlags(ExistResource->GetResourceState(), EDisplayClusterViewportResourceState::Unused);

			return ExistResource;
		}
	}

	// Create new resource:
	TSharedPtr<FDisplayClusterViewportResource, ESPMode::ThreadSafe> NewResource;

	if (EnumHasAnyFlags(InSettings.GetResourceFlags(), EDisplayClusterViewportResourceSettingsFlags::RenderTarget))
	{
		// Render Target
		NewResource = MakeShared<FDisplayClusterViewportRenderTargetResource>(InSettings);
	}
	else if (EnumHasAnyFlags(InSettings.GetResourceFlags(), EDisplayClusterViewportResourceSettingsFlags::PreviewTargetableTexture))
	{
		// Preview texture
		NewResource = MakeShared<FDisplayClusterViewportPreviewResource>(InSettings);
	}
	else if (EnumHasAnyFlags(InSettings.GetResourceFlags(), EDisplayClusterViewportResourceSettingsFlags::RenderTargetableTexture | EDisplayClusterViewportResourceSettingsFlags::ResolveTargetableTexture))
	{
		// Texture
		NewResource = MakeShared<FDisplayClusterViewportTextureResource>(InSettings);
	}

	if (NewResource.IsValid())
	{
		ViewportResources.Add(NewResource);
	}

	return NewResource;
}
