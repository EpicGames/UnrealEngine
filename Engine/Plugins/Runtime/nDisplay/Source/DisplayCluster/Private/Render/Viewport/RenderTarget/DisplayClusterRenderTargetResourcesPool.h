// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Render/Viewport/Resource/DisplayClusterViewportResource.h"

/**
 * DC viewport resources pool
 */
class FDisplayClusterRenderTargetResourcesPool
{
public:
	~FDisplayClusterRenderTargetResourcesPool();

	void Release();

public:
	/** Begin reallocate resources for the specified cluster node.
	* 
	* @param InRenderFrameSettings - default resource settings
	* @param InViewport - the window that uses these resources
	* 
	* @return true, if success
	*/
	bool BeginReallocateResources(class FViewport* InViewport, const struct FDisplayClusterRenderFrameSettings& InRenderFrameSettings);
	
	/** Allocate a new resource or reuse exists
	* 
	* @param InViewportId      - This resource belongs to this viewport only. Empty string if not used
	* @param InSize            - resource dimensions
	* @param CustomPixelFormat - resource pixel format
	* @param InResourceFlags   - These flags define the behavior of the resource.
	* @param NumMips           - number of mips in the texture
	* 
	* @return ref to the resource instance
	*/
	TSharedPtr<FDisplayClusterViewportResource, ESPMode::ThreadSafe> AllocateResource(const FString InViewportId, const FIntPoint& InSize, EPixelFormat CustomPixelFormat, const EDisplayClusterViewportResourceSettingsFlags InResourceFlags, int32 NumMips = 1);

	/** End reallocate resources for the specified cluster node. */
	void EndReallocateResources();

private:
	/**
	 * Resource update mode
	 */
	enum class EResourceUpdateMode : uint8
	{
		Initialize = 0,
		Release
	};

	/** Update resources from array
	* 
	* @param InOutViewportResources - resources to update
	* @param InUpdateMode           - initialize or release
	*/
	void ImplUpdateResources(TArray<TSharedPtr<FDisplayClusterViewportResource, ESPMode::ThreadSafe>>& InOutViewportResources, const EResourceUpdateMode InUpdateMode);

private:
	// Current render resource settings
	FDisplayClusterViewportResourceSettings* ResourceSettings = nullptr;

	// Viewport render resources
	TArray<TSharedPtr<FDisplayClusterViewportResource, ESPMode::ThreadSafe>> ViewportResources;
};
