// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Viewport/Resource/DisplayClusterViewportPreviewResource.h"

#include "Engine/TextureRenderTarget2D.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Package.h"

#include "RHI.h"
#include "RenderResource.h"
#include "UnrealClient.h"
#include "TextureResource.h"

////////////////////////////////////////////////////////////////////////
// FDisplayClusterViewportPreviewResource
////////////////////////////////////////////////////////////////////////
FDisplayClusterViewportPreviewResource::FDisplayClusterViewportPreviewResource(const FDisplayClusterViewportResourceSettings& InResourceSettings)
	: FDisplayClusterViewportResource(InResourceSettings)
{ }

FDisplayClusterViewportPreviewResource::~FDisplayClusterViewportPreviewResource()
{ }

void FDisplayClusterViewportPreviewResource::InitializeViewportResource()
{
	check(IsInGameThread());

	if (EnumHasAnyFlags(GetResourceState(), EDisplayClusterViewportResourceState::Initialized))
	{
		// Already initialized
		return;
	}

	RenderTargetTexture = NewObject<UTextureRenderTarget2D>(GetTransientPackage());
	if (RenderTargetTexture)
	{
		RenderTargetTexture->AddToRoot();

		RenderTargetTexture->ClearColor = FLinearColor::Black;
		RenderTargetTexture->TargetGamma = ResourceSettings.GetDisplayGamma();
		RenderTargetTexture->SRGB = EnumHasAnyFlags(ResourceSettings.GetResourceFlags(), EDisplayClusterViewportResourceSettingsFlags::ShouldUseSRGB);
		RenderTargetTexture->InitCustomFormat(ResourceSettings.GetSizeX(), ResourceSettings.GetSizeY(), ResourceSettings.GetFormat(), false);

		EnumAddFlags(GetResourceState(), EDisplayClusterViewportResourceState::Initialized);
	}
}

void FDisplayClusterViewportPreviewResource::ReleaseViewportResource()
{
	check(IsInGameThread());

	if (RenderTargetTexture)
	{
		RenderTargetTexture->RemoveFromRoot();
		RenderTargetTexture = nullptr;
	}

	EnumRemoveFlags(GetResourceState(), EDisplayClusterViewportResourceState::Initialized);
}

FRHITexture* FDisplayClusterViewportPreviewResource::GetViewportResourceRHI_RenderThread() const
{
	check(IsInRenderingThread());

	if (RenderTargetTexture)
	{
		if (FTextureRenderTargetResource* RenderTargetResource = RenderTargetTexture->GetRenderTargetResource())
		{
			return RenderTargetResource->TextureRHI;
		}
	}

	return nullptr;
}
