// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Render/Viewport/Resource/DisplayClusterViewportResource.h"

#include "RHI.h"
#include "RHIResources.h"
#include "RenderResource.h"
#include "Engine/TextureRenderTarget2D.h"

/**
 * Viewport preview texture resource
 */
class FDisplayClusterViewportPreviewResource
	: public FDisplayClusterViewportResource
{
public:
	FDisplayClusterViewportPreviewResource(const FDisplayClusterViewportResourceSettings& InResourceSettings);
	virtual ~FDisplayClusterViewportPreviewResource();

public:
	//~ Begin FDisplayClusterViewportResource
	virtual void InitializeViewportResource() override;
	virtual void ReleaseViewportResource() override;

	virtual UTextureRenderTarget2D* GetTextureRenderTarget2D() const override
	{
		return RenderTargetTexture;
	}

	virtual FRHITexture* GetViewportResourceRHI_RenderThread() const override;
	//~~ End FDisplayClusterViewportResource

public:
	// Preview RTT resource
	TObjectPtr<UTextureRenderTarget2D> RenderTargetTexture = nullptr;
};
