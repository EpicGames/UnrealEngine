// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Viewport/Resource/DisplayClusterViewportResource.h"

#include "RHI.h"
#include "RenderResource.h"
#include "UnrealClient.h"

////////////////////////////////////////////////////////////////////////
// FDisplayClusterViewportResource
////////////////////////////////////////////////////////////////////////
FDisplayClusterViewportResource::~FDisplayClusterViewportResource()
{
	// resource must be released in the rendering thread before the destructor is called
	check(!EnumHasAnyFlags(ResourceState, EDisplayClusterViewportResourceState::Initialized));
}

void FDisplayClusterViewportResource::ImplInitDynamicRHI_RenderTargetResource2D(FRHICommandListBase& RHICmdList, FTextureRHIRef& OutRenderTargetTextureRHI, FTextureRHIRef& OutTextureRHI)
{
	// -- we will be manually copying this cross GPU, tell render graph not to
	ETextureCreateFlags CreateFlags = TexCreate_MultiGPUGraphIgnore;

	// reflect srgb from settings
	if (EnumHasAnyFlags(ResourceSettings.GetResourceFlags(), EDisplayClusterViewportResourceSettingsFlags::ShouldUseSRGB))
	{
		CreateFlags |= TexCreate_SRGB;
	}

	uint32 NumMips = 1;
	if (ResourceSettings.GetNumMips() > 1)
	{
		// Create nummips texture!
		CreateFlags |= TexCreate_UAV;
		NumMips = ResourceSettings.GetNumMips();
	}

	const FRHITextureCreateDesc Desc =
		FRHITextureCreateDesc::Create2D(TEXT("DisplayClusterViewportRenderTargetResource"))
		.SetExtent(ResourceSettings.GetSizeXY().X, ResourceSettings.GetSizeXY().Y)
		.SetFormat(ResourceSettings.GetFormat())
		.SetNumMips(NumMips)
		.SetFlags(CreateFlags | ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::ShaderResource)
		.SetInitialState(ERHIAccess::SRVMask)
		.SetClearValue(FClearValueBinding::Black);

	OutRenderTargetTextureRHI = OutTextureRHI = RHICmdList.CreateTexture(Desc);
}

void FDisplayClusterViewportResource::ImplInitDynamicRHI_TextureResource2D(FRHICommandListBase& RHICmdList, FTextureRHIRef& OutTextureRHI)
{
	FRHITextureCreateDesc Desc =
		FRHITextureCreateDesc::Create2D(TEXT("DisplayClusterViewportTextureResource"), ResourceSettings.GetSizeXY().X, ResourceSettings.GetSizeXY().Y, ResourceSettings.GetFormat())
		.SetClearValue(FClearValueBinding::Black)
		.SetFlags(ETextureCreateFlags::ShaderResource)
		// -- we will be manually copying this cross GPU, tell render graph not to
		.AddFlags(ETextureCreateFlags::MultiGPUGraphIgnore)
		.SetInitialState(ERHIAccess::SRVMask);

	// reflect srgb from settings
	if (EnumHasAnyFlags(ResourceSettings.GetResourceFlags(), EDisplayClusterViewportResourceSettingsFlags::ShouldUseSRGB))
	{
		Desc.AddFlags(ETextureCreateFlags::SRGB);
	}

	// reflect RenderTargetable flags
	if (EnumHasAnyFlags(ResourceSettings.GetResourceFlags(), EDisplayClusterViewportResourceSettingsFlags::RenderTargetableTexture))
	{
		Desc.AddFlags(ETextureCreateFlags::RenderTargetable);
	}
	else
	{
		Desc.AddFlags(ETextureCreateFlags::ResolveTargetable);
	}

	OutTextureRHI = RHICmdList.CreateTexture(Desc);
}
