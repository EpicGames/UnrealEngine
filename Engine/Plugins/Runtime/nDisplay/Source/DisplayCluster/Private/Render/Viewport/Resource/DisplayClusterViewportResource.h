// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "DisplayClusterViewportResourceSettings.h"

#include "RHI.h"
#include "RHIResources.h"
#include "RenderResource.h"

#include "Templates/SharedPointer.h"

class FRenderTarget;
class UTextureRenderTarget2D;

/**
 * Runtime state of the viewport resource
 */
enum class EDisplayClusterViewportResourceState : uint8
{
	// empty state, without flags
	None = 0,

	// This resource was initialized in the rendering thread
	Initialized = 1 << 0,

	// This resource is not used for the current frame
	Unused = 1 << 1,

	// Disable reallocation of this resource for the current frame
	DisableReallocate = 1 << 3,

	// This resource was updated in the rendering thread
	UpdatedOnRenderingThread = 1 << 4,
};
ENUM_CLASS_FLAGS(EDisplayClusterViewportResourceState);

/**
 * Viewport resource texture base class
 */
class FDisplayClusterViewportResource
	: public TSharedFromThis<FDisplayClusterViewportResource, ESPMode::ThreadSafe>
{
public:
	FDisplayClusterViewportResource(const FDisplayClusterViewportResourceSettings& InResourceSettings)
		: ResourceSettings(InResourceSettings)
	{ }

	virtual ~FDisplayClusterViewportResource();

public:
	/** Get the current state of the resource. */
	inline EDisplayClusterViewportResourceState& GetResourceState()
	{
		return ResourceState;
	}

	/** Get the current const state of the resource. */
	inline const EDisplayClusterViewportResourceState& GetResourceState() const
	{
		return ResourceState;
	}

	/** Get the current settings of the resource. */
	inline const FDisplayClusterViewportResourceSettings& GetResourceSettings() const
	{
		return ResourceSettings;
	}

public:
	/** Assign external RHI resource. */
	virtual void SetExternalViewportResourceRHI(FTextureRHIRef& InExternalViewportResourceRHI)
	{ }

	/** Get current RHI resource. */
	virtual FRHITexture* GetViewportResourceRHI_RenderThread() const
	{
		return nullptr;
	}

	/** Get RenderTarget resource. */
	virtual FRenderTarget* GetViewportResourceRenderTarget()
	{
		return nullptr;
	}

	virtual UTextureRenderTarget2D* GetTextureRenderTarget2D() const
	{
		return nullptr;
	}

	/** Getting the viewport rect offset in the backbuffer. */
	virtual FIntPoint GetBackbufferFrameOffset() const
	{
		return FIntPoint::ZeroValue;
	}

	/** Settings the viewport rect offset in the backbuffer. */
	virtual void SetBackbufferFrameOffset(const FIntPoint& InBackbufferFrameOffset)
	{ }

	/** Initialize this resource in the game thread. */
	virtual void InitializeViewportResource()
	{
		EnumAddFlags(ResourceState, EDisplayClusterViewportResourceState::Initialized);
	}

	/** Release this resource in the game thread*/
	virtual void ReleaseViewportResource()
	{
		EnumRemoveFlags(ResourceState, EDisplayClusterViewportResourceState::Initialized);
	}

	/** RHI initialization for this resource in the rendering thread. */
	virtual void InitializeViewportResource_RenderThread(FRHICommandListBase& RHICmdList)
	{ }

	/** RHI release for this resource in the rendering thread*/
	virtual void ReleaseViewportResource_RenderThread(FRHICommandListBase& RHICmdList)
	{ }

protected:
	/** Helper function: create RTT texture. */
	void ImplInitDynamicRHI_RenderTargetResource2D(FRHICommandListBase& RHICmdList, FTextureRHIRef& OutRenderTargetTextureRHI, FTextureRHIRef& OutTextureRHI);

	/** Helper function: create 2D texture. */
	void ImplInitDynamicRHI_TextureResource2D(FRHICommandListBase& RHICmdList, FTextureRHIRef& OutTextureRHI);

protected:
	// Settings used by this resource
	const FDisplayClusterViewportResourceSettings ResourceSettings;

	// Resource states
	EDisplayClusterViewportResourceState ResourceState = EDisplayClusterViewportResourceState::None;
};
