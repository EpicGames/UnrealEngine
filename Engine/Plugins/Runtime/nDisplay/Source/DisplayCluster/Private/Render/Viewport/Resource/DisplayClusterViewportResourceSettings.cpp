// Copyright Epic Games, Inc. All Rights Reserved.

#include "Render/Viewport/Resource/DisplayClusterViewportResourceSettings.h"
#include "Render/Viewport/RenderFrame/DisplayClusterRenderFrameSettings.h"
#include "Render/Viewport/DisplayClusterViewportHelpers.h"

#include "RHI.h"
#include "RenderResource.h"
#include "UnrealClient.h"

/////////////////////////////////////////////////////////////////////
// FDisplayClusterViewportResource
/////////////////////////////////////////////////////////////////////
FDisplayClusterViewportResourceSettings::FDisplayClusterViewportResourceSettings(const FDisplayClusterRenderFrameSettings& InRenderFrameSettings, FViewport* InViewport)
	: ClusterNodeId(InRenderFrameSettings.CurrentNode.Id)
{
	if (InViewport && InViewport->GetRenderTargetTexture())
	{
		FRHITexture* ViewportTexture = InViewport->GetRenderTargetTexture();
		Format = ViewportTexture->GetFormat();

		if (EnumHasAnyFlags(ViewportTexture->GetFlags(), TexCreate_SRGB))
		{
			EnumAddFlags(ResourceFlags, EDisplayClusterViewportResourceSettingsFlags::ShouldUseSRGB);
		}

		DisplayGamma = InViewport->GetDisplayGamma();
	}
	else
	{
		// Use default settings:
		Format = FDisplayClusterViewportHelpers::GetDefaultPixelFormat();
		DisplayGamma = 2.2f;

		// Always use srgb for preview rendering
		EnumAddFlags(ResourceFlags, EDisplayClusterViewportResourceSettingsFlags::ShouldUseSRGB);

		if (InRenderFrameSettings.ShouldUseLinearGamma())
		{
				DisplayGamma = 1.f;
		}

		if(InRenderFrameSettings.IsPreviewRendering())
		{
			// for preview rendering use custom pixel format
			Format = FDisplayClusterViewportHelpers::GetPreviewDefaultPixelFormat();
		}
	}
}

FDisplayClusterViewportResourceSettings::FDisplayClusterViewportResourceSettings(const FDisplayClusterViewportResourceSettings& InBaseSettings, const FString InViewportId, const FIntPoint& InSize, const EPixelFormat InFormat, const EDisplayClusterViewportResourceSettingsFlags InResourceFlags, const int32 InNumMips)
	: ClusterNodeId(InBaseSettings.ClusterNodeId)
	, ViewportId(InViewportId)
	, Size(InSize)
	, Format((InFormat == PF_Unknown) ? InBaseSettings.Format : InFormat)
	, DisplayGamma(InBaseSettings.DisplayGamma)
	, NumMips(InNumMips)
	, ResourceFlags(InBaseSettings.ResourceFlags)
{
	EnumAddFlags(ResourceFlags, InResourceFlags);

	// For a texture in linear color space, set Gamma to 1.0f
	if (EnumHasAnyFlags(InResourceFlags, EDisplayClusterViewportResourceSettingsFlags::LinearColorSpace))
	{
		DisplayGamma = 1.0f;
	}
}
