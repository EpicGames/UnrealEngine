// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "PixelFormat.h"

struct FDisplayClusterRenderFrameSettings;
class FViewport;

/**
 * Runtime state of the viewport resource
 * Depending on the flags used, different child classes will be used.
 */
enum class EDisplayClusterViewportResourceSettingsFlags : uint8
{
	// Default value, no flags
	None = 0,

	// The resource will use the 'ETextureCreateFlags::SRGB' flag.
	ShouldUseSRGB = 1 << 0,

	// This is the RenderTarget resource used in the viewfamily (not the usual texture).
	RenderTarget = 1 << 1,

	// This is the texture.
	// The render resource will be created with the 'ETextureCreateFlags::RenderTargetable' flag.
	RenderTargetableTexture = 1 << 3,

	// This is the texture.
	// The render resource will be created with the 'ETextureCreateFlags::ResolveTargetable' flag.
	ResolveTargetableTexture = 1 << 4,

	// This is the texture that is used to render the DCRA preview
	// (contains the UTexture that is used for UMaterialInstanceDynamic).
	PreviewTargetableTexture = 1 << 5,

	// This texture is in linear color space
	// DisplayGamma should be set to 1.0f
	LinearColorSpace = 1 << 6,
};
ENUM_CLASS_FLAGS(EDisplayClusterViewportResourceSettingsFlags);

/**
 * nDisplay viewport render resource settings
 */
struct FDisplayClusterViewportResourceSettings
{
	/** Create default resource settings
	* 
	* @param InRenderFrameSettings - render frame settings used (preview or normal rendering, etc.)
	* @param InViewport - the window viewport interface (used display gamma, format, etc.)
	*/
	FDisplayClusterViewportResourceSettings(const FDisplayClusterRenderFrameSettings& InRenderFrameSettings, class FViewport* InViewport);

	/** Create a custom resource setting based on another one.
	* 
	* @param InBaseSettings  -  basic settings, if no values are defined, the values from them will be used.
	* @param InSize          - resource size
	* @param InFormat        - resource format
	* @param InResourceFlags - these flags determine what type of resource will be created.
	* @param InNumMips       - the number of mips that will be created. This value is only supported by a certain type of resource.
	*/
	FDisplayClusterViewportResourceSettings(const FDisplayClusterViewportResourceSettings& InBaseSettings, const FString InViewportId, const FIntPoint& InSize, const EPixelFormat InFormat, const EDisplayClusterViewportResourceSettingsFlags InResourceFlags = EDisplayClusterViewportResourceSettingsFlags::None, const int32 InNumMips = 1);

public:
	/** Returns true if the input resource settings match the local values. */
	inline bool IsResourceSettingsEqual(const FDisplayClusterViewportResourceSettings& In) const
	{
		// Special rules for preview RTT resource
		if (EnumHasAnyFlags(ResourceFlags, EDisplayClusterViewportResourceSettingsFlags::PreviewTargetableTexture))
		{
			// A cluster node can have multiple viewports using RTT previews
			// But since these textures are linked to the preview mesh material, we will also add this rule to link to a specific viewport.
			if (In.ViewportId != ViewportId)
			{
				return false;
			}
		}

		return (In.Size == Size)
			&& (In.Format == Format)
			&& (In.ResourceFlags == ResourceFlags)
			&& (In.DisplayGamma == DisplayGamma)
			&& (In.NumMips == NumMips);
	}

	/** Returns true if the resource settings are used in the same cluster node. */
	inline bool IsClusterNodeNameEqual(const FDisplayClusterViewportResourceSettings& In) const
	{
		return ClusterNodeId == In.ClusterNodeId;
	}

	/** Get the X and Y dimensions of the resource. */
	inline const FIntPoint& GetSizeXY() const
	{
		return Size;
	}

	/** Get the width of the resource. */
	inline const int32 GetSizeX() const
	{
		return Size.X;
	}

	/** Get the height of the resource. */
	inline const int32 GetSizeY() const
	{
		return Size.Y;
	}

	/** Get the pixel format of the resource. */
	inline EPixelFormat GetFormat() const
	{
		return Format;
	}

	/** Get the display gamma used by this resource */
	inline float GetDisplayGamma() const
	{
		return DisplayGamma;
	}

	/** Get the number of resource mips. */
	inline int32 GetNumMips() const
	{
		return NumMips;
	}

	/** Retrieving resource flags. */
	inline EDisplayClusterViewportResourceSettingsFlags GetResourceFlags() const
	{
		return ResourceFlags;
	}

private:
	// This is a resource belonging to the cluster node
	const FString ClusterNodeId;

	// The preview resource is always used by the same Outer viewport
	const FString ViewportId;

	// Resource width and height
	FIntPoint    Size;

	// The pixel format of the resource.
	EPixelFormat Format = EPixelFormat::PF_Unknown;

	// The display gamma used by this resource
	float DisplayGamma = 2.2f;

	// Number of resource mips.
	int32 NumMips = 0;

	// Resource flags.
	EDisplayClusterViewportResourceSettingsFlags ResourceFlags = EDisplayClusterViewportResourceSettingsFlags::None;
};
