// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterICVFXCameraCustomVersion.h"
#include "Serialization/CustomVersion.h"

const FGuid FDisplayClusterICVFXCameraCustomVersion::GUID(0x02AA05D1, 0xAF1A4366, 0x92017933, 0x729C6419);
// Register the custom version with core
FCustomVersionRegistration GRegisterDisplayClusterConfigurationObjectVersion(FDisplayClusterICVFXCameraCustomVersion::GUID, FDisplayClusterICVFXCameraCustomVersion::LatestVersion, TEXT("DisplayClusterICVFXCameraVer"));