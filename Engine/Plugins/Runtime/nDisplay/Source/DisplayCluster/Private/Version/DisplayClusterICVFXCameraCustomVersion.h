// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"

// Custom serialization version for UDisplayClusterICVFXCameraComponent
struct FDisplayClusterICVFXCameraCustomVersion
{
	enum Type
	{
		// Before any version changes were made
		BeforeCustomVersionWasAdded = 0,
		UpdateChromakeyConfig = 1,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	const static FGuid GUID;

private:
	FDisplayClusterICVFXCameraCustomVersion() {}
};
