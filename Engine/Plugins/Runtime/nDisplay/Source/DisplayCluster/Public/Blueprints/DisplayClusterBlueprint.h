// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DisplayClusterConfigurationTypes.h"
#include "Engine/Blueprint.h"
#include "UObject/AssetRegistryTagsContext.h"

#include "DisplayClusterBlueprint.generated.h"

class USCS_Node;
class UActorComponent;

/**
 * Display Cluster Configuration Asset
 *
 * Asset containing the full nDisplay cluster setup and settings.
 * Can also contain any blueprint logic like any other actor blueprints.
 *
 * @see UBlueprint
 */
UCLASS(BlueprintType, DisplayName = "nDisplay Blueprint")
class DISPLAYCLUSTER_API UDisplayClusterBlueprint : public UBlueprint
{
	GENERATED_BODY()

public:
	UDisplayClusterBlueprint();

#if WITH_EDITORONLY_DATA
	/** Whether or not you want to continuously rerun the construction script while dragging a slider */
	UPROPERTY(EditAnywhere, Category=BlueprintOptions)
	uint8 bRunConstructionScriptOnInteractiveChange : 1;
#endif

#if WITH_EDITOR
	// UBlueprint
	virtual bool SupportedByDefaultBlueprintFactory() const override { return false; }
	virtual UClass* GetBlueprintClass() const override;
	virtual void GetReparentingRules(TSet<const UClass*>& AllowedChildrenOfClasses, TSet<const UClass*>& DisallowedChildrenOfClasses) const override;
	// ~UBlueprint
#endif // WITH_EDITOR
	
	//~ Begin UObject Interface
	virtual void PreSave(FObjectPreSaveContext SaveContext) override;
	virtual void PostLoad() override;
	virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	//~ End UObject Interface

	class UDisplayClusterBlueprintGeneratedClass* GetGeneratedClass() const;

	UDisplayClusterConfigurationData* GetOrLoadConfig();
	UDisplayClusterConfigurationData* GetConfig() const { return ConfigData; }

	/**
	 * Set the config data on the CDO and update the config file path.
	 * When bForceRecreate is false this only updates the config path after initial creation.
	 * 
	 * @param InConfigData New config data to set. This will be a template for the CDO to use if being created initially or force recreated.
	 * @param bForceRecreate Force recreate the config data on the CDO. This will break instance sync and only recommended for importing.
	 */
	void SetConfigData(UDisplayClusterConfigurationData* InConfigData, bool bForceRecreate = false);

	const FString& GetConfigPath() const;
	void SetConfigPath(const FString& InPath);

	/** Configure the config with needed blueprint data for an export. */
	void PrepareConfigForExport();
	
	//** Updates the ConfigExport property. Called when saving the asset.
	void UpdateConfigExportProperty();

	//** Updates the Summary property. Called when saving the asset.
	void UpdateSummaryProperty();

public:
	// Holds the last saved config export. Added to the AssetRegistry to allow parsing without loading.
	UPROPERTY()
	FString ConfigExport;

	// Cache of the summary of this asset. Will be used as asset registry tag and show up in the asset tooltip.
	UPROPERTY(AssetRegistrySearchable)
	FString Summary;

private:

	/** Returns normal component name. We get component name from a SCS component template which has _GEN_VARIABLE postfix. */
	FString GetObjectNameFromSCSNode(const UObject* const Object) const;

	/** Auxiliary recursive function that builds child-parentId components map */
	void GatherParentComponentsInfo(const USCS_Node* const InNode, TMap<UActorComponent*, FString>& OutParentsMap) const;

	/** Auxiliary function to remove invalid pairs from the TMap containers in the config*/
	void CleanupConfigMaps(UDisplayClusterConfigurationData* Data) const;

protected:
	UPROPERTY()
	TObjectPtr<UDisplayClusterConfigurationData> ConfigData;

private:
	friend class FDisplayClusterConfiguratorVersionUtils;
	
	UPROPERTY(AssetRegistrySearchable)
	int32 AssetVersion;
};
