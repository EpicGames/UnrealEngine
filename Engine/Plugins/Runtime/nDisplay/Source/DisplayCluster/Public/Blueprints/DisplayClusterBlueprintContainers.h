// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "StereoRendering.h"

#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"

#include "DisplayClusterBlueprintContainers.generated.h"

USTRUCT()
struct DISPLAYCLUSTER_API FDisplayClusterViewportContext
{
	GENERATED_BODY()

public:
	// Viewport Name
	UPROPERTY(EditAnywhere, Category = "NDisplay")
	FString ViewportID;

	// Location on a backbuffer.
	UPROPERTY(EditAnywhere, Category = "NDisplay")
	FIntPoint RectLocation = FIntPoint::ZeroValue;

	// Size on a backbuffer.
	UPROPERTY(EditAnywhere, Category = "NDisplay")
	FIntPoint RectSize = FIntPoint::ZeroValue;

	// Camera view location
	UPROPERTY(EditAnywhere, Category = "NDisplay")
	FVector  ViewLocation = FVector::ZeroVector;

	// Camera view rotation
	UPROPERTY(EditAnywhere, Category = "NDisplay")
	FRotator ViewRotation = FRotator::ZeroRotator;

	// Camera projection Matrix
	UPROPERTY(EditAnywhere, Category = "NDisplay")
	FMatrix ProjectionMatrix = FMatrix::Identity;

	// Rendering status for this viewport (Overlay and other configuration rules can disable rendering for this viewport.)
	UPROPERTY(EditAnywhere, Category = "NDisplay")
	bool bIsRendering = true;
};

USTRUCT()
struct DISPLAYCLUSTER_API FDisplayClusterViewportStereoContext
{
	GENERATED_BODY()

public:
	// Viewport Name
	UPROPERTY(EditAnywhere, Category = "NDisplay")
	FString ViewportID;

	// Location on a backbuffer.
	UPROPERTY(EditAnywhere, Category = "NDisplay")
	FIntPoint RectLocation = FIntPoint::ZeroValue;

	// Size on a backbuffer.
	UPROPERTY(EditAnywhere, Category = "NDisplay")
	FIntPoint RectSize = FIntPoint::ZeroValue;

	// Camera view location
	UPROPERTY(EditAnywhere, Category = "NDisplay")
	TArray<FVector> ViewLocation;

	// Camera view rotation
	UPROPERTY(EditAnywhere, Category = "NDisplay")
	TArray<FRotator> ViewRotation;

	// Camera projection Matrix
	UPROPERTY(EditAnywhere, Category = "NDisplay")
	TArray<FMatrix> ProjectionMatrix;

	// Rendering status for this viewport (Overlay and other configuration rules can disable rendering for this viewport.)
	UPROPERTY(EditAnywhere, Category = "NDisplay")
	bool bIsRendering = true;
};
