// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DisplayClusterEnums.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Blueprints/IDisplayClusterBlueprintAPI.h"
#include "DisplayClusterBlueprintLib.generated.h"

class ADisplayClusterChromakeyCardActor;
class ADisplayClusterLightCardActor;
class ADisplayClusterRootActor;
class IDisplayClusterClusterEventListener;
struct FDisplayClusterClusterEventBinary;
struct FDisplayClusterClusterEventJson;


/**
 * Blueprint API function library
 */
UCLASS()
class UDisplayClusterBlueprintLib
	: public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/** [DEPRECATED] Return Display Cluster API interface. */
	UE_DEPRECATED(5.4, "GetAPI function has been deprecated. All functions are now available in UDisplayClusterBlueprintLib.")
	UFUNCTION(BlueprintPure, meta = (DeprecatedFunction, DeprecationMessage = "GetAPI has been deprecated. All functions are now available in the main blueprint functions list under 'nDisplay' category."))
	static void GetAPI(TScriptInterface<IDisplayClusterBlueprintAPI>& OutAPI);

public:

	/** Returns current operation mode. */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Get Operation Mode"), Category = "NDisplay")
	static DISPLAYCLUSTER_API EDisplayClusterOperationMode GetOperationMode();

	/** Returns currently active root actor. */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Get Active Root Actor"), Category = "NDisplay")
	static DISPLAYCLUSTER_API ADisplayClusterRootActor* GetRootActor();

public:

	/** Returns Id of the current node in a cluster. */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Get Node ID"), Category = "NDisplay|Cluster")
	static DISPLAYCLUSTER_API FString GetNodeId();

	/** Returns List of the active nodes in the runtime cluster node in a cluster. */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Get Cluster Node IDs"), Category = "NDisplay|Cluster")
	static DISPLAYCLUSTER_API void GetActiveNodeIds(TArray<FString>& OutNodeIds);

	/** Returns amount of active nodes in a cluster. */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Get Cluster Nodes Amount"), Category = "NDisplay|Cluster")
	static DISPLAYCLUSTER_API int32 GetActiveNodesAmount();

	/** Returns true if current node is a primary node in a cluster. */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Is Primary Node"), Category = "NDisplay|Cluster")
	static DISPLAYCLUSTER_API bool IsPrimary();

	/** Returns true if current node is a secondary node in a cluster. */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Is Secondary Node"), Category = "NDisplay|Cluster")
	static DISPLAYCLUSTER_API bool IsSecondary();

	/** Returns true if current node is a backup node in a cluster. */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Is Backup Node"), Category = "NDisplay|Cluster")
	static DISPLAYCLUSTER_API bool IsBackup();

	/** Checks if the node has a specific role. */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Has a Role"), Category = "NDisplay|Cluster")
	static DISPLAYCLUSTER_API bool HasClusterRole(EDisplayClusterNodeRole Role);

	/** Returns the role of the current cluster node. */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Get Cluster Node Role"), Category = "NDisplay|Cluster")
	static DISPLAYCLUSTER_API EDisplayClusterNodeRole GetClusterRole();

	/** Adds cluster event listener. */
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Add Cluster Event Listener"), Category = "NDisplay|Cluster|Events")
	static DISPLAYCLUSTER_API void AddClusterEventListener(TScriptInterface<IDisplayClusterClusterEventListener> Listener);

	/** Removes cluster event listener. */
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Remove Cluster Event Listener"), Category = "NDisplay|Cluster|Events")
	static DISPLAYCLUSTER_API void RemoveClusterEventListener(TScriptInterface<IDisplayClusterClusterEventListener> Listener);

	/** Emits JSON cluster event. */
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Emit JSON Cluster Event"), Category = "NDisplay|Cluster|Events")
	static DISPLAYCLUSTER_API void EmitClusterEventJson(const FDisplayClusterClusterEventJson& Event, bool bPrimaryOnly);

	/** Emits binary cluster event. */
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Emit Binary Cluster Event"), Category = "NDisplay|Cluster|Events")
	static DISPLAYCLUSTER_API void EmitClusterEventBinary(const FDisplayClusterClusterEventBinary& Event, bool bPrimaryOnly);

	/** Sends JSON cluster event to a specific target (may be outside of the cluster). */
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Send JSON Event To Host"), Category = "NDisplay|Cluster|Events")
	static DISPLAYCLUSTER_API void SendClusterEventJsonTo(const FString& Address, const int32 Port, const FDisplayClusterClusterEventJson& Event, bool bPrimaryOnly);

	/** Sends binary cluster event to a specific target (may be outside of the cluster). */
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Send Binary Event To Host"), Category = "NDisplay|Cluster|Events")
	static DISPLAYCLUSTER_API void SendClusterEventBinaryTo(const FString& Address, const int32 Port, const FDisplayClusterClusterEventBinary& Event, bool bPrimaryOnly);

public:

	/** Create a new light card parented to the given nDisplay root actor. */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Create Light Card"), Category = "NDisplay|LC & CK")
	static ADisplayClusterLightCardActor* CreateLightCard(ADisplayClusterRootActor* RootActor);

	/** Create duplicates of a list of existing light cards. */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Duplicate Light Cards"), Category = "NDisplay|LC & CK")
	static void DuplicateLightCards(TArray<ADisplayClusterLightCardActor*> OriginalLightcards, TArray<ADisplayClusterLightCardActor*>& OutNewLightCards);

	/** Gets a list of all light card actors on the level linked to the specified root actor. */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Find Light Cards"), Category = "NDisplay|LC & CK")
	static DISPLAYCLUSTER_API void FindLightCardsForRootActor(const ADisplayClusterRootActor* RootActor, TSet<ADisplayClusterLightCardActor*>& OutLightCards);

	/** Gets a list of all chromakey card actors on the level linked to the specified root actor. */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Find Chromakey Cards"), Category = "NDisplay|LC & CK")
	static DISPLAYCLUSTER_API void FindChromakeyCardsForRootActor(const ADisplayClusterRootActor* RootActor, TSet<ADisplayClusterChromakeyCardActor*>& OutChromakeyCards);
};
