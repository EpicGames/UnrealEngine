// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"

#include "DisplayClusterEnums.h"
#include "Cluster/DisplayClusterClusterEvent.h"

#include "Engine/Scene.h"
#include "SceneViewExtension.h"

// Forward declarations for the following classes leads to a build error
#include "DisplayClusterRootActor.h"
#include "Components/DisplayClusterOriginComponent.h"
#include "Components/DisplayClusterCameraComponent.h"
#include "Components/DisplayClusterScreenComponent.h"
#include "DisplayClusterConfigurationTypes.h"

#include "IDisplayClusterBlueprintAPI.generated.h"


class IDisplayClusterClusterEventListener;
class UCineCameraComponent;
struct FPostProcessSettings;


UINTERFACE(meta = (CannotImplementInterfaceInBlueprint))
class DISPLAYCLUSTER_API UDisplayClusterBlueprintAPI : public UInterface
{
	GENERATED_BODY()
};

/**
 * Blueprint API interface
 */
class DISPLAYCLUSTER_API IDisplayClusterBlueprintAPI
{
	GENERATED_BODY()

public:
	/* Returns true if the module has been initialized. */
	UE_DEPRECATED(5.4, "This function has been deprecated and will be removed soon.")
	UFUNCTION(BlueprintCallable, meta = (DeprecatedFunction, DeprecationMessage = "This function has been deprecated and will be removed soon."))
	virtual bool IsModuleInitialized() const = 0;

	/** Returns current operation mode. */
	UE_DEPRECATED(5.4, "This function has been moved to UDisplayClusterBlueprintLib.")
	UFUNCTION(BlueprintCallable, meta = (DeprecatedFunction, DeprecationMessage = "This function is now available in the main blueprint functions list under 'nDisplay' section."))
	virtual EDisplayClusterOperationMode GetOperationMode() const = 0;

public:
	/** Returns DisplayCluster root actor. */
	UE_DEPRECATED(5.4, "This function has been moved to UDisplayClusterBlueprintLib.")
	UFUNCTION(BlueprintCallable, meta = (DeprecatedFunction, DeprecationMessage = "This function is now available in the main blueprint functions list under 'nDisplay' section."))
	virtual ADisplayClusterRootActor* GetRootActor() const = 0;

	// Runtime local node API

	/** Returns Id of the current node in a cluster. */
	UE_DEPRECATED(5.4, "This function has been moved to UDisplayClusterBlueprintLib.")
	UFUNCTION(BlueprintCallable, meta = (DeprecatedFunction, DeprecationMessage = "This function is now available in the main blueprint functions list under 'nDisplay' section."))
	virtual FString GetNodeId() const = 0;

	/** Returns List of the active nodes in the runtime cluster node in a cluster. */
	UE_DEPRECATED(5.4, "This function has been moved to UDisplayClusterBlueprintLib.")
	UFUNCTION(BlueprintCallable, meta = (DeprecatedFunction, DeprecationMessage = "This function is now available in the main blueprint functions list under 'nDisplay' section."))
	virtual void GetActiveNodeIds(TArray<FString>& OutNodeIds) const = 0;

	/** Returns amount of active nodes in a cluster. */
	UE_DEPRECATED(5.4, "This function has been moved to UDisplayClusterBlueprintLib.")
	UFUNCTION(BlueprintCallable, meta = (DeprecatedFunction, DeprecationMessage = "This function is now available in the main blueprint functions list under 'nDisplay' section."))
	virtual int32 GetActiveNodesAmount() const = 0;

	/** Returns true if current node is a primary node in a cluster. */
	UE_DEPRECATED(5.4, "This function has been moved to UDisplayClusterBlueprintLib.")
	UFUNCTION(BlueprintCallable, meta = (DeprecatedFunction, DeprecationMessage = "This function is now available in the main blueprint functions list under 'nDisplay' section."))
	virtual bool IsPrimary() const = 0;

	/** Returns true if current node is a secondary node in a cluster. */
	UE_DEPRECATED(5.4, "This function has been moved to UDisplayClusterBlueprintLib.")
	UFUNCTION(BlueprintCallable, meta = (DeprecatedFunction, DeprecationMessage = "This function is now available in the main blueprint functions list under 'nDisplay' section."))
	virtual bool IsSecondary() const = 0;

	/** Returns true if current node is a backup node in a cluster. */
	UE_DEPRECATED(5.4, "This function has been moved to UDisplayClusterBlueprintLib.")
	UFUNCTION(BlueprintCallable, meta = (DeprecatedFunction, DeprecationMessage = "This function is now available in the main blueprint functions list under 'nDisplay' section."))
	virtual bool IsBackup() const = 0;

	/** Returns the role of the current cluster node. */
	UE_DEPRECATED(5.4, "This function has been moved to UDisplayClusterBlueprintLib.")
	UFUNCTION(BlueprintCallable, meta = (DeprecatedFunction, DeprecationMessage = "This function is now available in the main blueprint functions list under 'nDisplay' section."))
	virtual EDisplayClusterNodeRole GetClusterRole() const = 0;

	// cluster events api

	/** Adds cluster event listener. */
	UE_DEPRECATED(5.4, "This function has been moved to UDisplayClusterBlueprintLib.")
	UFUNCTION(BlueprintCallable, meta = (DeprecatedFunction, DeprecationMessage = "This function is now available in the main blueprint functions list under 'nDisplay' section."))
	virtual void AddClusterEventListener(TScriptInterface<IDisplayClusterClusterEventListener> Listener) = 0;

	/** Removes cluster event listener. */
	UE_DEPRECATED(5.4, "This function has been moved to UDisplayClusterBlueprintLib.")
	UFUNCTION(BlueprintCallable, meta = (DeprecatedFunction, DeprecationMessage = "This function is now available in the main blueprint functions list under 'nDisplay' section."))
	virtual void RemoveClusterEventListener(TScriptInterface<IDisplayClusterClusterEventListener> Listener) = 0;

	/** Emits JSON cluster event. */
	UE_DEPRECATED(5.4, "This function has been moved to UDisplayClusterBlueprintLib.")
	UFUNCTION(BlueprintCallable, meta = (DeprecatedFunction, DeprecationMessage = "This function is now available in the main blueprint functions list under 'nDisplay' section."))
	virtual void EmitClusterEventJson(const FDisplayClusterClusterEventJson& Event, bool bPrimaryOnly) = 0;

	/** Emits binary cluster event. */
	UE_DEPRECATED(5.4, "This function has been moved to UDisplayClusterBlueprintLib.")
	UFUNCTION(BlueprintCallable, meta = (DeprecatedFunction, DeprecationMessage = "This function is now available in the main blueprint functions list under 'nDisplay' section."))
	virtual void EmitClusterEventBinary(const FDisplayClusterClusterEventBinary& Event, bool bPrimaryOnly) = 0;

	/** Sends JSON cluster event to a specific target (outside of the cluster). */
	UE_DEPRECATED(5.4, "This function has been moved to UDisplayClusterBlueprintLib.")
	UFUNCTION(BlueprintCallable, meta = (DeprecatedFunction, DeprecationMessage = "This function is now available in the main blueprint functions list under 'nDisplay' section."))
	virtual void SendClusterEventJsonTo(const FString& Address, const int32 Port, const FDisplayClusterClusterEventJson& Event, bool bPrimaryOnly) = 0;

	/** Sends binary cluster event to a specific target (outside of the cluster). */
	UE_DEPRECATED(5.4, "This function has been moved to UDisplayClusterBlueprintLib.")
	UFUNCTION(BlueprintCallable, meta = (DeprecatedFunction, DeprecationMessage = "This function is now available in the main blueprint functions list under 'nDisplay' section."))
	virtual void SendClusterEventBinaryTo(const FString& Address, const int32 Port, const FDisplayClusterClusterEventBinary& Event, bool bPrimaryOnly) = 0;
};
