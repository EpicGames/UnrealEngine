// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Cluster/CustomStates/IDisplayClusterCustomState.h"

#include "IDisplayCluster.h"
#include "Cluster/IDisplayClusterClusterManager.h"


namespace UE::nDisplay::Private
{
	/**
	 * Type-independent base class for custom states
	 */
	class DISPLAYCLUSTER_API FCustomStateBase
		: public IDisplayClusterCustomState
	{
	public:

		FCustomStateBase(const FName& UniqueName, const FName& ClusterNodeId);
		virtual ~FCustomStateBase() = default;

	public:

		//~ Begin IDisplayClusterCustomState
		virtual FName GetName() const override;
		virtual void Lock() const override;
		virtual void Unlock() const override;
		//~ End IDisplayClusterCustomState

	protected:

		/**
		 * Type-independent private implementation of render task scheduler.
		 * The main idea of this function is to keep ENQUEUE_RENDER_COMMAND in the private
		 * space, and therefore not introduce any new public dependencies on RenderCore.
		 */
		void ExecuteOnRenderThread(TUniqueFunction<void()> FuncRT);

	protected:

		/** Quick access for ClusterMgr cached pointer */
		IDisplayClusterClusterManager* GetClusterMgr() const
		{
			return ClusterMgr;
		}

		/** Quick access for this cluster node ID */
		const FName& GetNodeId() const
		{
			return ClusterNodeId;
		}

		/** Access to the main access guard of the state instance */
		FCriticalSection& GetCritSec() const
		{
			return *StateCS;
		}

		/** Access to the main access guard of the state instance */
		TSharedRef<FCriticalSection> GetCritSecRef() const
		{
			return StateCS;
		}

	private:

		/** Unique ID of this state */
		const FName UniqueName;

		/** Id of this cluster node to speed up internal operations */
		const FName ClusterNodeId;

		/** DC cluster manager to speed up internal operations */
		IDisplayClusterClusterManager* ClusterMgr = nullptr;

		/**
		 * Main access guard for this state instance, wrapped in
		 * a shared reference to ensure thread-safe sharing.
		 */
		TSharedRef<FCriticalSection> StateCS;
	};


	/**
	 * Custom state data class
	 * 
	 * Provides general control over local/cluster state data. Also manages game and render thread data.
	 */
	template <typename TDataType>
	class TCustomStateData
		: public FCustomStateBase
	{
	public:
	
		template <typename... TArgs>
		TCustomStateData(const FName& InUniqueName, const FName& InNodeId, TArgs&&... Args)
			: FCustomStateBase(InUniqueName, InNodeId)
			, ClusterStatesData(MakeShared<TMap<FName, FCustomStateDataHolder<TDataType>>>())
		{
			// Make sure local data of this node always exists
			ClusterStatesData->Emplace(GetNodeId(), Forward<TArgs>(Args)...);
		}
		
		virtual ~TCustomStateData() = default;

	public:

		/** Returns local state data bound to current thread and frame */
		const TDataType& GetData() const
		{
			FScopeLock Lock(&GetCritSec());
			return GetThreadData();
		}

		/** Sets new state data for the next frame. */
		template <typename TValue>
		requires std::convertible_to<TValue, TDataType>
		void SetData(TValue&& NewData)
		{
			FScopeLock Lock(&GetCritSec());
			if (IsSetDataAllowed())
			{
				NextFrameData = Forward<TValue>(NewData);
			}
		}

	protected:

		/** Ask children whether SetData is allowed for the instance. */
		virtual bool IsSetDataAllowed() const
		{
			return true;
		}

	protected:

		/** Internal thread sensitive getter of local state data */
		TDataType& GetThreadData()
		{
			return GetThreadDataImpl(ClusterStatesData->FindChecked(GetNodeId()));
		}

		/** Internal thread sensitive getter of local state data (const version) */
		const TDataType& GetThreadData() const
		{
			return GetThreadDataImpl(ClusterStatesData->FindChecked(GetNodeId()));
		}

		/** Internal thread sensitive getter of a specific node state data */
		TDataType& GetThreadData(const FName& NodeId, bool bCreateIfNotExists = false)
		{
			FCustomStateDataHolder<TDataType>* const Found = bCreateIfNotExists ? &ClusterStatesData->FindOrAdd(NodeId) : ClusterStatesData->Find(NodeId);
			return GetThreadDataImpl(Found ? *Found : ClusterStatesData->FindChecked(GetNodeId()));
		}

		/** Internal thread sensitive getter of a specific node state data (const version) */
		const TDataType& GetThreadData(const FName& NodeId) const
		{
			const FCustomStateDataHolder<TDataType>* const Found = ClusterStatesData->Find(NodeId);
			return GetThreadDataImpl(Found ? *Found : ClusterStatesData->FindChecked(GetNodeId()));
		}

		/** Returns node IDs available in the storage */
		TSet<FName> GetNodes() const
		{
			TSet<FName> NodeIds;
			ClusterStatesData->GetKeys(NodeIds);
			return NodeIds;
		}

	protected:

		/** Game thread data update */
		void AdvanceFrameData_GT()
		{
			checkSlow(IsInGameThread());

			// Update local value only
			ClusterStatesData->FindChecked(GetNodeId()).Data = NextFrameData;
		}

		/** Render thread data update */
		void AdvanceFrameData_RT()
		{
			checkSlow(IsInGameThread());

			// Update render data for every node
			for (TPair<FName, FCustomStateDataHolder<TDataType>>& NodeStateData : *ClusterStatesData)
			{
				// Current game thread values will replace the render thread ones on the rendering thread
				ExecuteOnRenderThread(
					[
						Data   = ClusterStatesData,
						NodeId = NodeStateData.Key,
						Value  = MoveTemp(NodeStateData.Value.Data),
						CritSecRef = GetCritSecRef()
					]() mutable
					{
						FScopeLock Lock(&CritSecRef.Get());
						Data->FindChecked(NodeId).DataRT = MoveTemp(Value);
					});
			}
		}

	private:

		/** Internal helper to get data based on the current thread (for const and non-const callers) */
		template <typename THolderType>
		static auto& GetThreadDataImpl(THolderType& Holder)
		{
			if (IsInRenderingThread())
			{
				return Holder.DataRT;
			}

			checkfSlow(IsInGameThread(), TEXT("Accessed state data from AnyThread"));
			return Holder.Data;
		}

	private:

		/**
		 * Auxiliary structure that holds all necessary data of a single state
		 */
		template <typename THolderDataType>
		struct FCustomStateDataHolder
		{
			/** Game thread data */
			THolderDataType Data { };

			/** Render thread data */
			THolderDataType DataRT { };
		};

		/**
		 * Holds data of this state on all nodes. Its content type depends on the concrete
		 * state implementation. Wrapped into a TSharedPtr to ensure safe access across threads.
		 */
		TSharedRef<TMap<FName, FCustomStateDataHolder<TDataType>>> ClusterStatesData;

		/** Data to use on the next frame */
		TDataType NextFrameData{ };
	};


	/**
	 * Factory method implementation for custom state templates
	 *
	 * Provides a simple way to instantiate custom states of any types. It also performs
	 * some additional checks, and automatically registers newly created states.
	 */
	template <typename TDataType>
	struct TCustomStateFactory
	{
		/**
		 * Custom states factory method
		 */
		template <typename... TArgs>
		requires std::derived_from<TDataType, FCustomStateBase>
		static TSharedPtr<TDataType> Create(const FName& InUniqueName, TArgs&&... Args)
		{
			// Invalid names not allowed
			if (InUniqueName.IsNone())
			{
				return nullptr;
			}

			// Make sure nDisplay is loaded
			if (!IDisplayCluster::IsAvailable())
			{
				return nullptr;
			}

			// Make sure cluster manager is available
			IDisplayClusterClusterManager* const ClusterMgr = IDisplayCluster::Get().GetClusterMgr();
			if (!ClusterMgr)
			{
				return nullptr;
			}

			// Make sure this name is not used already
			const bool bAlreadyExists = ClusterMgr->IsCustomStateRegistered(InUniqueName);
			if (bAlreadyExists)
			{
				return nullptr;
			}

			// Instantiate and register
			TDataType* const NewInstance = new TDataType(InUniqueName, *ClusterMgr->GetNodeId(), Forward<TArgs>(Args)...);
			TSharedPtr<TDataType> NewState = MakeShareable<TDataType>(NewInstance);
			TSharedPtr<IDisplayClusterCustomState> NewStatePtr = StaticCastSharedPtr<IDisplayClusterCustomState>(NewState);
			if (!ClusterMgr->RegisterCustomState(InUniqueName, NewStatePtr))
			{
				return nullptr;
			}

			return NewState;
		}
	};


	/**
	 * Type ID generator
	 * 
	 * It's an auxiliary class that provides a way to differentiate various state implementations
	 * (template specialization) in runtime without RTTI. It converts the text based function signatures
	 * into FName which is quite good for serialization and comparison.
	 */
	template<typename TType>
	class TCustomStateTypeId
	{
	public:

		/** Type getter */
		static const FName& GetTypeId()
		{
			static const FName TypeId = GenerateTypeId();
			return TypeId;
		}

	private:

		/**
		 * Internal type name generator. Extracts the actual TType from the full function signature.
		 * 
		 * For example, full signature might be like this:
		 * {Signature=L"class FName __cdecl UE::nDisplay::Private::TCustomStateTypeId<class TDistributedCustomState<struct FMyStruct> >::GenerateTypeId(void)" }
		 * 
		 * The type ID would be:
		 * class TDistributedCustomState<struct FMyStruct>
		 */
		static FName GenerateTypeId()
		{
#if defined(_MSC_VER)
			FString Signature(ANSI_TO_TCHAR(__FUNCSIG__));
#else
			FString Signature(ANSI_TO_TCHAR(__PRETTY_FUNCTION__));
#endif

			/** Auxiliary substring extractor */
			auto SubString = [&Signature](const FString& Left, const FString& Right)
				{
					const int32 FoundLeft  = Signature.Find(Left, ESearchCase::CaseSensitive, ESearchDir::FromStart);
					const int32 FoundRight = Signature.Find(Right, ESearchCase::CaseSensitive, ESearchDir::FromEnd);

					checkSlow(FoundLeft  != INDEX_NONE);
					checkSlow(FoundRight != INDEX_NONE);
					checkSlow(FoundRight > FoundLeft);

					if (FoundRight != INDEX_NONE)
					{
						Signature = Signature.Left(FoundRight);
					}

					if (FoundLeft != INDEX_NONE)
					{
						Signature = Signature.Right(Signature.Len() - (FoundLeft + Left.Len()));
					}

					Signature.TrimStartAndEndInline();
				};

			// Remove unnecessary pieces from the original signature
			SubString(TEXT("TCustomStateTypeId"), TEXT("::GenerateTypeId"));
			SubString(TEXT("<"), TEXT(">"));

			return *Signature;
		}
	};
}
