// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Cluster/CustomStates/DisplayClusterCustomStateBase.h"

#include "Misc/ScopeLock.h"
#include "Serialization/Archive.h"
#include "Serialization/MemoryReader.h"

#include "CoreGlobals.h"


/**
 * Distributed custom state
 *
 * This state enables data exchange in any direction within a cluster.
 * By default, any cluster node can access the values of any other node.
 * An additional API is provided to customize data propagation paths
 * in arbitrary ways.
 *
 * GetData() and GetData(NodeId) are thread-sensitive. The data is
 * maintained in separate copies for the game and render threads.
 * Access from other threads is not supported.
 *
 * Distributed states are automatically registered for replication
 * during creation (see the Create() call in the examples below).
 * Once registered, replication within the cluster begins immediately.
 * To stop replication, the state must be explicitly unregistered.
 *
 * This state type has no leader or explicit data source; it is a
 * generic mechanism for data exchange. Data propagation is performed
 * synchronously to preserve determinism, which may introduce update
 * latency.
 *
 * When the state is modified on the game thread, the expected latency
 * is one frame: a value set on frame N becomes visible via GetData()
 * on frame N + 1. If the value is updated before state synchronization,
 * no latency is introduced.
 *
 * When modified on the render thread, the latency may be 2–3 frames,
 * as the value is first propagated to the game thread and then passed
 * back to the render thread.
 *
 * This class uses internal data serialization. Any custom TDataType
 * must provide its own serialization functions. Refer to the examples
 * below for details.
 *
 *        +---------------------+       +---------------------+
 *        ¦   NodeA::Value=5    ¦       ¦   NodeB::Value=2    ¦
 *        +---------------------¦       +---------------------¦
 *  +---- ¦ NodeA: Value=5, R/W ¦ ----> ¦ NodeA: Value=5, R/O ¦
 *  ¦     ¦ NodeB: Value=2, R/O ¦ <---- ¦ NodeB: Value=2, R/W ¦ ----+
 *  ¦ +-> ¦ NodeC: Value=8, R/O ¦       ¦ NodeC: Value=8, R/O ¦ <-+ ¦
 *  ¦ ¦   +---------------------+       +---------------------+   ¦ ¦
 *  ¦ ¦                                                           ¦ ¦
 *  ¦ ¦                  +---------------------+                  ¦ ¦
 *  ¦ ¦                  ¦   NodeC::Value=8    ¦                  ¦ ¦
 *  ¦ ¦                  +---------------------¦                  ¦ ¦
 *  +-+----------------> ¦ NodeA: Value=5, R/O ¦                  ¦ ¦
 *    ¦                  ¦ NodeB: Value=2, R/O ¦ <----------------+-+
 *    +----------------- ¦ NodeC: Value=8, R/W ¦ -----------------+
 *                       +---------------------+
 *
 * EXAMPLE - Base type
 * ------------------------
 * using FMyState = TDistributedCustomState<float>;
 *
 * const FName MyStateName = TEXT("MyStateFloat");  // The name must be unique
 * TSharedPtr<FMyState> MyState = FMyState::Create(MyStateName);  // Don't forget to nullptr-check.
 * or
 * TSharedPtr<FMyState> MyState = FMyState::Create(MyStateName, 0.72f);  // With initialization value
 *
 * MyState->SetData(0.75f);  // Set new value
 * MyState->SetData(0.85f);  // It can be called multiple times, the last one will be used on the next frame
 *
 * const TSet<FName> NodeIds = MyState->GetAvailableNodes();
 * for (const FName& NodeId : NodeIds)
 * {
 *     const int32 Value = MyState->GetData(NodeId);
 *     DoSomething(NodeId, Value);
 * }
 *
 * IDisplayCluster::Get().GetClusterMgr()->UnregisterCustomState(MyStateName);  // Unregister it from replication on this cluster node
 * IDisplayCluster::Get().GetClusterMgr()->RegisterCustomState(MyStateName, MyState);  // Register again
 *
 * MyState->SetCustomUpstreamsEnabled(true); // Enable custom upstream configuration
 * MyState->SetUpstreams({ TEXT("Node_0"), TEXT("Node_1") }); // Receive updates from Node_0 and Node_1 only
 *
 *
 * EXAMPLE - Custom type
 * ------------------------
 * // Any custom types
 * struct FMyStruct
 * {
 *    int32 Items = 0;
 *    float Progress = 0.f;
 *    bool  bIsActive = false;
 * };
 *
 * // With custom serialization
 * inline FArchive& operator<<(FArchive& Ar, FMyStruct& MyStruct)
 * {
 *    Ar << MyStruct.Items;
 *    Ar << MyStruct.Progress;
 *    Ar << MyStruct.bIsActive;
 *    return Ar;
 * }
 *
 * using FMyState = TDistributedCustomState<FMyStruct>;
 *
 * const FName MyStateName = TEXT("MyStateStruct"); // The name must be unique
 * TSharedPtr<FMyState> MyState = FMyState::Create(MyStateName); // Don't forget to nullptr-check. It's also auto-registered
 * or
 * TSharedPtr<FMyState> MyState = FMyState::Create(MyStateName, FMyStruct{ 3, 2, 0.1f, true });
 *
 * MyState->SetData({ 2, 0.3f, true }); // Set new value
 * MyState->SetData({ 9, 0.5f, true }); // It can be called multiple times, the last one will be used on the next frame
 *
 * const TSet<FName> NodeIds = MyState->GetAvailableNodes();
 * for (const FName& NodeId : NodeIds)
 * {
 *     FMyStruct Value = MyState->GetData(NodeId);
 *     DoSomething(NodeId, Value);
 * }
 *
 * FMyStruct CurrentFrameState = MyState->GetData(); // Get data on current frame (game/render thread sensitive)
 * CurrentFrameState = { 10, 1.f, false };
 * MyState->SetData(MoveTemp(CurrentFrameState)); // Move semantics can be useful for complex types
 *
 */
template <typename TDataType>
class TDistributedCustomState
	: public UE::nDisplay::Private::TCustomStateData<TDataType>
	, public UE::nDisplay::Private::TCustomStateFactory<TDistributedCustomState<TDataType>>
{
	// Access to the private constructor
	friend UE::nDisplay::Private::TCustomStateFactory<TDistributedCustomState<TDataType>>;

	using ThisType = TDistributedCustomState<TDataType>;

protected:

	// Bring some parent names explicitly because of the CRTP
	using UE::nDisplay::Private::FCustomStateBase::GetCritSec;
	using UE::nDisplay::Private::FCustomStateBase::GetNodeId;
	using UE::nDisplay::Private::TCustomStateData<TDataType>::AdvanceFrameData_GT;
	using UE::nDisplay::Private::TCustomStateData<TDataType>::AdvanceFrameData_RT;
	using UE::nDisplay::Private::TCustomStateData<TDataType>::GetNodes;
	using UE::nDisplay::Private::TCustomStateData<TDataType>::GetThreadData;

public:

	// Expose some parent names explicitly because of name hiding
	using UE::nDisplay::Private::TCustomStateData<TDataType>::GetData;

private:

	/** Private constructor. Only factory method can instantiate this class. */
	template <typename... TArgs>
	TDistributedCustomState(const FName& InUniqueName, const FName& InNodeId, TArgs&&... Args)
		: UE::nDisplay::Private::TCustomStateData<TDataType>(InUniqueName, InNodeId, Forward<TArgs>(Args)...)
		, UE::nDisplay::Private::TCustomStateFactory<TDistributedCustomState<TDataType>>()
	{ }

public:

	virtual ~TDistributedCustomState() = default;

public:

	//~ Begin IDisplayClusterCustomState

	virtual FName GetType() const override
	{
		return UE::nDisplay::Private::TCustomStateTypeId<ThisType>::GetTypeId();
	}

	virtual void Serialize(FArchive& Ar) override
	{
		checkSlow(IsInGameThread());
		FScopeLock Lock(&GetCritSec());
		Ar << GetThreadData();
	}

	virtual void AdvanceFrame() override
	{
		checkSlow(IsInGameThread());
		FScopeLock Lock(&GetCritSec());

		// Update render thread data
		AdvanceFrameData_RT();

		// Distributed states always update their game thread data because
		// it's allowed for them to have individual values.
		AdvanceFrameData_GT();
	}

	virtual bool HasCustomUpstreamConfiguration() const override
	{
		FScopeLock Lock(&GetCritSec());
		return bUseCustomUpstreamConfiguration;
	}

	virtual TSet<FName> GetUpstreams() const
	{
		FScopeLock Lock(&GetCritSec());
		return UpstreamNodes;
	}

	virtual void Update(const TMap<FName, TArray<uint8>>& ClusterStates) override
	{
		checkSlow(IsInGameThread());

		// Ignore invalid input
		if (ClusterStates.Num() <= 0)
		{
			return;
		}

		FScopeLock Lock(&GetCritSec());

		// Deserialize and store new data
		for (const TPair<FName, TArray<uint8>>& NodeState : ClusterStates)
		{
			FMemoryReader DataReader(NodeState.Value);
			// Always create new space for the nodes that have just joined
			constexpr bool bShouldCreateIfNotExists = true;
			DataReader << GetThreadData(NodeState.Key, bShouldCreateIfNotExists);
		}
	}

	//~ End IDisplayClusterCustomState

public:
	
	/** Returns state data of a specific node. The data is bound to current thread and frame. */
	const TDataType& GetData(const FName& NodeId) const
	{
		FScopeLock Lock(&GetCritSec());
		return GetThreadData(NodeId);
	}

	/** Returns node IDs available in the storage  */
	TSet<FName> GetAvailableNodes() const
	{
		return GetNodes();
	}

public:

	/** Enable/disable custom upstream configuration */
	void SetCustomUpstreamsEnabled(bool bEnabled)
	{
		FScopeLock Lock(&GetCritSec());
		bUseCustomUpstreamConfiguration = bEnabled;
	}

	/**
	 * Configure upstream nodes that should propagate their updates to this state instance
	 * 
	 * @param NewUpstreamNodes - new set of cluster node IDs
	 */
	void SetUpstreams(const TSet<FName>& NewUpstreamNodes)
	{
		FScopeLock Lock(&GetCritSec());
		UpstreamNodes = NewUpstreamNodes;
	}

private:

	/** Whether custom upstream configuration is enabled */
	bool bUseCustomUpstreamConfiguration = false;

	/** Custom upstream cluster node IDs */
	TSet<FName> UpstreamNodes;
};
