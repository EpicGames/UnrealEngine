// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Cluster/CustomStates/DisplayClusterCustomStateBase.h"
#include "Cluster/IDisplayClusterClusterManager.h"
#include "Misc/ScopeLock.h"
#include "Serialization/Archive.h"
#include "Serialization/MemoryReader.h"

#include "CoreGlobals.h"


/**
 * Shared custom state.
 *
 * "Shared" means there is a single source of truth. This implementation
 * always propagates the state data from the primary node to all other
 * nodes. Regardless of which node GetData() is called on, the returned
 * data will always match the primary node's value. Consequently, SetData()
 * is ignored on all non-primary nodes.
 *
 * GetData() is thread-sensitive. The state maintains separate copies
 * for the game and render threads. Access from other threads is not
 * supported.
 *
 * Shared states are automatically registered for replication
 * during creation (see the Create() call in the examples below).
 * Once registered, replication within the cluster begins immediately.
 * To stop replication, the state must be explicitly unregistered.
 *
 * Data propagation is performed synchronously to preserve determinism,
 * which may introduce update latency. When the state is modified on
 * the game thread, the expected latency is one frame: a value set on
 * frame N becomes visible via GetData() on frame N + 1. If the value
 * is updated before state synchronization, no latency is introduced.
 *
 * When updated on the render thread, latency may be 2–3 frames, as the
 * value is first propagated to the game thread and then back to the
 * render thread.
 *
 * This class uses internal data serialization. Any custom TDataType
 * must provide its own serialization functions. Refer to the examples
 * below for details.
 *
 *    +----------------+                    +--------------------+
 *    ¦  Primary node  ¦                    ¦   Secondary node   ¦
 *    +----------------¦   Replication      +--------------------¦
 *    ¦  Value=5, R/W  ¦  --------------->  ¦ Value=5, Read only ¦
 *    +----------------+             ¦      +--------------------+
 *                                   ¦
 *                                   ¦
 *                                   ¦      +--------------------+
 *                                   ¦      ¦   Secondary node   ¦
 *                                   ¦      +--------------------¦
 *                                   +--->  ¦ Value=5, Read only ¦
 *                                          +--------------------+
 *
 * EXAMPLE - Base type
 * ------------------------
 * using FMyState = TSharedCustomState<float>;
 *
 * const FName MyStateName = TEXT("MyStateFloat");  // The name must be unique
 * TSharedPtr<FMyState> MyState = FMyState::Create(MyStateName);  // Don't forget to nullptr-check.
 * or
 * TSharedPtr<FMyState> MyState = FMyState::Create(MyStateName, 0.72f);  // With initialization value
 *
 * MyState->SetData(0.75f);  // Set new value
 * MyState->SetData(0.85f);  // It can be called multiple times, the last one will be used on the next frame
 *
 * IDisplayCluster::Get().GetClusterMgr()->UnregisterCustomState(MyStateName);  // Unregister it from replication on this cluster node
 * IDisplayCluster::Get().GetClusterMgr()->RegisterCustomState(MyStateName, MyState);  // Register again
 *
 *
 * EXAMPLE - Custom type
 * ------------------------
 * // Any custom types
 * struct FMyStruct
 * {
 *    int32 Items = 0;
 *    float Progress = 0.f;
 *    bool  bIsActive = false;
 * };
 *
 * // With custom serialization
 * inline FArchive& operator<<(FArchive& Ar, FMyStruct& MyStruct)
 * {
 *    Ar << MyStruct.Items;
 *    Ar << MyStruct.Progress;
 *    Ar << MyStruct.bIsActive;
 *    return Ar;
 * }
 *
 * using FMyState = TSharedCustomState<FMyStruct>;
 *
 * const FName MyStateName = TEXT("MyStateStruct"); // The name must be unique
 * TSharedPtr<FMyState> MyState = FMyState::Create(MyStateName); // Don't forget to nullptr-check. It's also auto-registered
 *
 * MyState->SetData({ 2, 0.3f, true }); // Set new value
 * MyState->SetData({ 9, 0.5f, true }); // It can be called multiple times, the last one will be used on the next frame
 *
 * FMyStruct CurrentFrameState = MyState->GetData(); // Get data on current frame (game/render thread sensitive)
 * CurrentFrameState = { 10, 1.f, false };
 * MyState->SetData(MoveTemp(CurrentFrameState)); // Move semantics can be useful for complex types
 *
 */
template <typename TDataType>
class TSharedCustomState
	: public UE::nDisplay::Private::TCustomStateData<TDataType>
	, public UE::nDisplay::Private::TCustomStateFactory<TSharedCustomState<TDataType>>
{
	// Access to the private constructor
	friend UE::nDisplay::Private::TCustomStateFactory<TSharedCustomState<TDataType>>;

	using ThisType = TSharedCustomState<TDataType>;

	// Bring some parent names explicitly because of the CRTP
	using UE::nDisplay::Private::FCustomStateBase::GetCritSec;
	using UE::nDisplay::Private::FCustomStateBase::GetClusterMgr;
	using UE::nDisplay::Private::FCustomStateBase::GetNodeId;
	using UE::nDisplay::Private::TCustomStateData<TDataType>::AdvanceFrameData_GT;
	using UE::nDisplay::Private::TCustomStateData<TDataType>::AdvanceFrameData_RT;
	using UE::nDisplay::Private::TCustomStateData<TDataType>::GetThreadData;

private:

	/** Private constructor. Only factory method can instantiate this class. */
	template <typename... TArgs>
	TSharedCustomState(const FName& InUniqueName, const FName& InNodeId, TArgs&&... Args)
		: UE::nDisplay::Private::TCustomStateData<TDataType>(InUniqueName, InNodeId, Forward<TArgs>(Args)...)
		, UE::nDisplay::Private::TCustomStateFactory<TSharedCustomState<TDataType>>()
	{ }

public:

	virtual ~TSharedCustomState() = default;

public:

	//~ Begin IDisplayClusterCustomState

	virtual FName GetType() const override
	{
		return UE::nDisplay::Private::TCustomStateTypeId<ThisType>::GetTypeId();
	}

	virtual bool ShouldPropagate() const override
	{
		// Propagate from primary nodes only
		return GetClusterMgr() ? GetClusterMgr()->IsPrimary() : false;
	}

	virtual void Serialize(FArchive& Ar) override
	{
		checkSlow(IsInGameThread());
		FScopeLock Lock(&GetCritSec());
		Ar << GetThreadData();
	}

	virtual void AdvanceFrame() override
	{
		checkSlow(IsInGameThread());
		FScopeLock Lock(&GetCritSec());

		// Update render thread data
		AdvanceFrameData_RT();

		// Update game thread data on primary nodes only. That's because shared custom
		// staes propagate update from p-nodes only.
		if (GetClusterMgr() && GetClusterMgr()->IsPrimary())
		{
			AdvanceFrameData_GT();
		}
	}

	virtual void Update(const TMap<FName, TArray<uint8>>& ClusterStates) override
	{
		FScopeLock Lock(&GetCritSec());

		// It's expected to have a single state from the primary node,
		// but still possible the input would be empty
		if (ClusterStates.Num() > 0)
		{
			auto It = ClusterStates.CreateConstIterator();
			FMemoryReader DataReader(It.Value());
			DataReader << GetThreadData();
		}
	}

	//~ End IDisplayClusterCustomState

protected:

	virtual bool IsSetDataAllowed() const override final
	{
		// Value updates are allowed only on primary nodes
		return GetClusterMgr() && GetClusterMgr()->IsPrimary();
	}
};
