// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FArchive;


/**
 * Custom state interface
 */
class IDisplayClusterCustomState
{
public:

	virtual ~IDisplayClusterCustomState() = default;

public:

	/**
	 * Returns custom state's unique name
	 * 
	 * @return Unique state name
	 */
	virtual FName GetName() const = 0;

	/**
	 * Returns type identificator
	 * 
	 * Custom states are implemented using C++ templates, which allows them to carry
	 * arbitrary payload types. Since RTTI is not used, it is not possible to determine
	 * at runtime whether two states with the same name also share the same payload type.
	 * If such a type mismatch occurs, serialization will fail and the application will
	 * likely crash. To prevent this, each state is associated with a unique type ID.
	 * 
	 * @return Type ID
	 */
	virtual FName GetType() const
	{
		return NAME_None;
	}

	/**
	 * Whether this state should propagate its data to any other cluster nodes
	 * 
	 * @return True if we need to propagate it within a cluster
	 */
	virtual bool ShouldPropagate() const
	{
		return true;
	}

	/**
	 * Serialization
	 * 
	 * It's used to serialize data of a custom state into an output buffer
	 * 
	 * @param Ar - An archive to write data in
	 */
	virtual void Serialize(FArchive& Ar) = 0;

	/**
	 * Whether this state requires some custom update path. If returns true,
	 * then GetUpstreams() will be called next in order to configure data senders.
	 * 
	 * @return - True if has its own list of upstream nodes
	 */
	virtual bool HasCustomUpstreamConfiguration() const
	{
		return false;
	}

	/**
	 * Returns set of node IDs that this custom state is expecting to get updates from.
	 * Based on the data propagation technique, this may disable any data senders, even all of them.
	 * It can also be used to reduce traffic while replicationg huge blobs.
	 * 
	 * @return - A set of cluster node IDs that should propagate their updates to this instance
	 */
	virtual TSet<FName> GetUpstreams() const
	{
		return { };
	}

	/**
	 * Progress to the next frame
	 */
	virtual void AdvanceFrame()
	{ }

	/**
	 * Deserialization
	 * 
	 * It's used to deserialize and process post-synchronization response data,
	 * and probably update internal data.
	 * 
	 * @param ClusterStates - ClusterNodeId-to-StateData blobs
	 */
	virtual void Update(const TMap<FName, TArray<uint8>>& ClusterStates) = 0;

	/**
	 * Lock this state to prevent any changes
	 */
	virtual void Lock() const = 0;

	/**
	 * Enable any changes after Lock() call
	 */
	virtual void Unlock() const = 0;
};
