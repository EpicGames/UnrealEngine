// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DisplayClusterClusterEvent.generated.h"

constexpr const auto DisplayClusterResetSyncType = TEXT("nDCReset");

class FArchive;


//////////////////////////////////////////////////////////////////////////////////////////////
// Common cluster event data
//////////////////////////////////////////////////////////////////////////////////////////////
USTRUCT(BlueprintType)
struct DISPLAYCLUSTER_API FDisplayClusterClusterEventBase
{
	GENERATED_BODY()

public:

	FDisplayClusterClusterEventBase()
		: bIsSystemEvent(false)
		, bShouldDiscardOnRepeat(true)
	{ }

	virtual ~FDisplayClusterClusterEventBase() = default;

public:

	/** Is nDisplay internal event(should never be true for end users) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (DisplayName = "Is Sytem Event. 'True' is reserved for nDisplay internals."), Category = "NDisplay")
	bool bIsSystemEvent = false;

	/** Should older events with the same Name / Type / Category(for JSON) or ID(for binary) be discarded if a new one received */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "NDisplay")
	bool bShouldDiscardOnRepeat = true;

public:

	/** Serialization */
	virtual void Serialize(FArchive& Ar);
};


//////////////////////////////////////////////////////////////////////////////////////////////
// Cluster event JSON
//////////////////////////////////////////////////////////////////////////////////////////////
USTRUCT(BlueprintType)
struct DISPLAYCLUSTER_API FDisplayClusterClusterEventJson
	: public FDisplayClusterClusterEventBase
{
	GENERATED_BODY()

public:

	/** Event name(used for discarding outdated events) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "NDisplay")
	FString Name;

	/** Event type(used for discarding outdated events) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "NDisplay")
	FString Type;

	/** Event category(used for discarding outdated events) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "NDisplay")
	FString Category;

public:

	/** Event parameters */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "NDisplay")
	TMap<FString, FString> Parameters;

public:

	/** Serialization */
	virtual void Serialize(FArchive& Ar) override;

	/** Serialization */
	friend FArchive& operator<<(FArchive& Ar, FDisplayClusterClusterEventJson& Instance)
	{
		Instance.Serialize(Ar);
		return Ar;
	}

	/** String based representation */
	FString ToString(bool bWithParams = false) const;

public:

	UE_DEPRECATED(5.6, "This function has been deprecated and will be removed soon. Please use 'Serialize' for JSON event serialization.")
	FString SerializeToString() const
	{
		return FString();
	}

	UE_DEPRECATED(5.6, "This function has been deprecated and will be removed soon. Please use 'Serialize' for JSON event deserialization.")
	bool DeserializeFromString(const FString& Arch)
	{
		return false;
	}
};


//////////////////////////////////////////////////////////////////////////////////////////////
// Cluster event BINARY
//////////////////////////////////////////////////////////////////////////////////////////////
USTRUCT(BlueprintType)
struct DISPLAYCLUSTER_API FDisplayClusterClusterEventBinary
	: public FDisplayClusterClusterEventBase
{
	GENERATED_BODY()

public:

	/** Event ID(used for discarding outdated events) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "NDisplay")
	int32 EventId = -1;

	/** Binary event data */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "NDisplay")
	TArray<uint8> EventData;

public:

	/** Raw serialization */
	void SerializeToByteArray(TArray<uint8>& Arch) const;

	/** Raw deserialization */
	bool DeserializeFromByteArray(const TArray<uint8>& Arch);

	/** Serialization */
	virtual void Serialize(FArchive& Ar) override;

	/** Serialization */
	friend FArchive& operator<<(FArchive& Ar, FDisplayClusterClusterEventBinary& Instance)
	{
		Instance.Serialize(Ar);
		return Ar;
	}
};
