// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Cluster/DisplayClusterClusterEvent.h"
#include "Engine/EngineBaseTypes.h"

#include "DisplayClusterNetDriverHelper.generated.h"

class FDisplayClusterClusterEventsBinaryClient;

/** Helper for DisplayCluster synchronous replication
 */

USTRUCT()
struct DISPLAYCLUSTER_API FDisplayClusterNetDriverHelper
{
	GENERATED_BODY()

public:
	/** Returns number of nodes in current cluster */
	int32 GetNumberOfClusterNodes() const;

	/** Returns primary node IP address of current cluster */
	FString GetPrimaryNodeAddress();

	/** Creates and stores nDisplay cluster events client using cluster id and primary node parameters
	 *  @param СlusterId unique identifier of cluster
	 *  @param ClientAddress primary node IP address
	 *  @param ClientPort primary node port
	 */
	bool RegisterClusterEventsBinaryClient(uint32 СlusterId, const FString& ClientAddress, uint16 ClientPort);

	/** Removes cluster event client for ClusterId
	 *  @param СlusterId unique identifier of cluster
	 */
	bool RemoveClusterEventsBinaryClient(uint32 СlusterId);

	/** Checks if nDisplay cluster events client registered for specified cluster
	 *  @param СlusterId unique identifier of cluster
	 */
	bool HasClient(uint32 ClusterId);

	/** Checks if URL has all required arguments for DisplayClusterNetDriver
	 *  @param URL Url to check
	 */
	static bool GetRequiredArguments(const FURL& URL, const TCHAR*& OutClusterId, const TCHAR*& OutPrimaryNodeId, const TCHAR*& OutPrimaryNodePort, const TCHAR*& OutClusterNodesNum);

	/** Sends command to nDisplay cluster specified by id
	 *  @param СlusterId identifier of cluster
	 *  @param NetworkDriverSyncEvent cluster event binary data
	 */
	bool SendCommandToCluster(uint32 СlusterId, const FDisplayClusterClusterEventBinary& NetworkDriverSyncEvent);

	/** Sends command to all nDisplay clusters stored by RegisterClusterEventsBinaryClient 
	 *  @param NetworkDriverSyncEvent cluster event binary data
	 */
	void SendCommandToAllClusters(const FDisplayClusterClusterEventBinary& NetworkDriverSyncEvent);

private:
	// nDisplay primary node events clients for sending control messages to clusters
	TMap<uint32, TSharedPtr<FDisplayClusterClusterEventsBinaryClient>> ClusterEventsBinaryClients;
};
