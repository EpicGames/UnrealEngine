// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Cluster/DisplayClusterClusterEvent.h"
#include "UObject/Interface.h"
#include "IDisplayClusterClusterEventListener.generated.h"



UINTERFACE()
class DISPLAYCLUSTER_API UDisplayClusterClusterEventListener
	: public UInterface
{
	GENERATED_BODY()

public:
	UDisplayClusterClusterEventListener(const FObjectInitializer& ObjectInitializer)
		: Super(ObjectInitializer)
	{ }
};

 
/**
 * Interface for cluster event listeners
 */
class DISPLAYCLUSTER_API IDisplayClusterClusterEventListener
{
	GENERATED_BODY()

public:
	// React on incoming JSON cluster events
	UFUNCTION(BlueprintImplementableEvent, Category = "NDisplay")
	void OnClusterEventJson(const FDisplayClusterClusterEventJson& Event);

	// React on incoming binary cluster events
	UFUNCTION(BlueprintImplementableEvent, Category = "NDisplay")
	void OnClusterEventBinary(const FDisplayClusterClusterEventBinary& Event);
};
