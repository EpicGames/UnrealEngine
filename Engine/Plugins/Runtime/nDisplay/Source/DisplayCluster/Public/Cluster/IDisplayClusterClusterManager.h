// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Cluster/IDisplayClusterGenericBarriersClient.h"
#include "Delegates/Delegate.h"
#include "UObject/ScriptInterface.h"
#include "DisplayClusterEnums.h"

struct FDisplayClusterClusterEventBinary;
struct FDisplayClusterClusterEventJson;
class IDisplayClusterClusterEventBinaryListener;
class IDisplayClusterClusterEventJsonListener;
class IDisplayClusterClusterEventListener;
class IDisplayClusterClusterSyncObject;
class IDisplayClusterCustomState;


DECLARE_MULTICAST_DELEGATE_OneParam(FOnClusterEventJson, const FDisplayClusterClusterEventJson& /* Event */);
typedef FOnClusterEventJson::FDelegate FOnClusterEventJsonListener;

DECLARE_MULTICAST_DELEGATE_OneParam(FOnClusterEventBinary, const FDisplayClusterClusterEventBinary& /* Event */);
typedef FOnClusterEventBinary::FDelegate FOnClusterEventBinaryListener;


/**
 * Public cluster manager interface
 */
class IDisplayClusterClusterManager
{
public:
	virtual ~IDisplayClusterClusterManager() = default;

public:
	/** Returns true if current node has primary role. */
	virtual bool IsPrimary() const = 0;
	/** Returns true if current node has secondary role. */
	virtual bool IsSecondary()  const = 0;
	/** Returns true if current node has backup role. */
	virtual bool IsBackup() const = 0;
	/** Returns cluster node role. */
	virtual EDisplayClusterNodeRole GetClusterRole() const = 0;
	/** Checks if the node has a specific role. */
	virtual bool HasClusterRole(EDisplayClusterNodeRole Role) const = 0;

	/** Returns current primary node ID. */
	virtual FString GetPrimaryNodeId() const = 0;

	/** Returns current cluster node ID. */
	virtual FString GetNodeId() const = 0;
	/** Returns amount of cluster nodes in the cluster. */
	virtual uint32 GetNodesAmount() const = 0;
	/** Returns IDs of available cluster nodes. */
	virtual void GetNodeIds(TArray<FString>& OutNodeIds) const = 0;
	/** Returns IDs of available cluster nodes. */
	virtual void GetNodeIds(TSet<FString>& OutNodeIds) const = 0;

	/** Drop specific cluster node */
	virtual bool DropClusterNode(const FString& NodeId) = 0;

	/** Registers object to synchronize. */
	virtual void RegisterSyncObject(IDisplayClusterClusterSyncObject* SyncObj, EDisplayClusterSyncGroup SyncGroup) = 0;
	/** Unregisters synchronization object. */
	virtual void UnregisterSyncObject(IDisplayClusterClusterSyncObject* SyncObj) = 0;

	/**
	 * Registers custom state for auto-replication
	 * 
	 * @param UniqueStateName - Unique name to distinguish different states
	 * @param StateVariable   - A variable that references the state
	 * 
	 * @return True if registered successfully, otherwise false (has been registered already)
	 */
	virtual bool RegisterCustomState(const FName& UniqueStateName, TSharedPtr<IDisplayClusterCustomState>& StateVariable) = 0;

	/**
	 * Unregisters specific custom state
	 *
	 * @param UniqueStateName - Unique name to distinguish different states
	 *
	 * @return True if unregistered successfully, otherwise false (not found)
	 */
	virtual bool UnregisterCustomState(const FName& UniqueStateName) = 0;

	/**
	 * Returns the names of all registered states
	 *
	 * @return Set of custom state names currently registered
	 */
	virtual TSet<FName> GetRegisteredCustomStateNames() const = 0;

	/**
	 * Checks whether a custom state with the specified name has already been registered
	 *
	 * @return True if such state exists
	 */
	virtual bool IsCustomStateRegistered(const FName& UniqueStateName) const = 0;


	/** Creates a general purpose barriers interface. */
	virtual TSharedRef<IDisplayClusterGenericBarriersClient> CreateGenericBarriersClient() = 0;


	//////////////////////////////////////////////////////////////////////////////////////////////
	// Cluster events
	//////////////////////////////////////////////////////////////////////////////////////////////

	/** Registers cluster event listener. */
	virtual void AddClusterEventListener(TScriptInterface<IDisplayClusterClusterEventListener> Listener) = 0;

	/** Unregisters cluster event listener. */
	virtual void RemoveClusterEventListener(TScriptInterface<IDisplayClusterClusterEventListener> Listener) = 0;

	/** Registers json cluster event listener. */
	virtual void AddClusterEventJsonListener(const FOnClusterEventJsonListener& Listener) = 0;

	/** Unregisters json cluster event listener. */
	virtual void RemoveClusterEventJsonListener(const FOnClusterEventJsonListener& Listener) = 0;

	/** Registers binary cluster event listener. */
	virtual void AddClusterEventBinaryListener(const FOnClusterEventBinaryListener& Listener) = 0;

	/** Unregisters binary cluster event listener. */
	virtual void RemoveClusterEventBinaryListener(const FOnClusterEventBinaryListener& Listener) = 0;

	/** Emits JSON cluster event. */
	virtual void EmitClusterEventJson(const FDisplayClusterClusterEventJson& Event, bool bPrimaryOnly) = 0;

	/** Emits binary cluster event. */
	virtual void EmitClusterEventBinary(const FDisplayClusterClusterEventBinary& Event, bool bPrimaryOnly) = 0;

	/** Sends JSON cluster event to a specific target (outside of the cluster). */
	virtual void SendClusterEventTo(const FString& Address, const uint16 Port, const FDisplayClusterClusterEventJson& Event, bool bPrimaryOnly) = 0;

	/** Sends binary cluster event to a specific target (outside of the cluster). */
	virtual void SendClusterEventTo(const FString& Address, const uint16 Port, const FDisplayClusterClusterEventBinary& Event, bool bPrimaryOnly) = 0;

public:
	
	UE_DEPRECATED(5.6, "This API method has been deprecated. Please use 'GetGenericBarriersClient'.")
	virtual TUniquePtr<IDisplayClusterGenericBarriersClient> CreateGenericBarriersClient(const FString& ClientName)
	{
		return nullptr;
	}
};
