// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDisplayClusterSerializable.h"
#include "IDisplayClusterStringSerializable.h"


/**
 * Synchronizable object interface
 */
class IDisplayClusterClusterSyncObject
	: public IDisplayClusterSerializable
	, public IDisplayClusterStringSerializable // Deprecated, don't override any of its functions
{
public:

	virtual ~IDisplayClusterClusterSyncObject() = default;

public:

	/** Whether an object needs to be syncrhonized on current frame */
	virtual bool IsActive() const = 0;

	/** Unique ID of the object being synchronized */
	virtual FString GetSyncId() const = 0;

	/** Whether the object's state has changed since last ClearDirty call */
	virtual bool IsDirty() const
	{
		return false;
	}

	/** Resets the dirty flag, marking the state as not yet changed */
	virtual void ClearDirty()
	{ }
};
