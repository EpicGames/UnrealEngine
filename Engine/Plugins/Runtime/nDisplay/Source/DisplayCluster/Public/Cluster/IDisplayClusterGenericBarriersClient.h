// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"


/**
 * Barrier synchonrization callback data
 */
struct FGenericBarrierSynchronizationDelegateData
{
	/** Barrier ID */
	const FString& BarrierId;

	/** Associates the caller IDs with the owning cluster nodes (Caller ID - to - cluster node) */
	const TMap<FString, FString>& ThreadToNodeMap;

	/** Binary data provided on sync request (Caller ID - to - data mapping) */
	const TMap<FString, TArray<uint8>>& RequestData;

	/** Binary data to respond (Caller ID - to - data mapping) */
	TMap<FString, TArray<uint8>>& ResponseData;
};


/**
 * Generic barriers client interface
 */
class IDisplayClusterGenericBarriersClient
{
public:

	virtual ~IDisplayClusterGenericBarriersClient() = default;

public:

	/** Synchronization delegate. It's called on the primary node only. */
	DECLARE_DELEGATE_OneParam(FOnGenericBarrierSynchronizationDelegate, FGenericBarrierSynchronizationDelegateData&);

	/** Barrier timeout delegate. It's called on the primary node only. */
	DECLARE_DELEGATE_OneParam(FOnGenericBarrierTimeoutDelegate, const TSet<FString>&);

public:

	/**
	 * Creates new barrier
	 * 
	 * @param BarrierId         - ID of the new barrier
	 * @param NodeToSyncCallers - NodeId-to-CallerId association map
	 * @param Timeout           - Barrier synchronization timeout
	 * 
	 * @return true if the barrier was created successfully, or already exists
	 */
	virtual bool CreateBarrier(const FString& BarrierId, const TMap<FString, TSet<FString>>& NodeToSyncCallers, uint32 Timeout) = 0;

	/**
	 * Wait until a barrier with specific ID is created and ready to go
	 * 
	 * @param BarrierId - ID of the barrier to wait
	 * 
	 * @return true if created or existed
	 */
	virtual bool WaitUntilBarrierIsCreated(const FString& BarrierId) = 0;

	/**
	 * Checks if a specific barrier exists
	 * 
	 * @param BarrierId - ID of the barrier to check
	 *
	 * @return true if the barrier exists
	 */
	virtual bool IsBarrierAvailable(const FString& BarrierId) = 0;

	/**
	 * Releases specific barrier
	 *
	 * @param BarrierId - ID of a barrier to release
	 *
	 * @return true if the barrier was successfully released
	 */
	virtual bool ReleaseBarrier(const FString& BarrierId) = 0;

	/**
	 * Synchronize the calling thread on a specific barrier
	 *
	 * @param BarrierId - ID of a barrier to use for synchronization
	 * @param CallerId  - ID of a synchronization caller (thread)
	 *
	 * @return true if synchronization succeeded
	 */
	virtual bool Synchronize(const FString& BarrierId, const FString& CallerId) = 0;

	/**
	 * Synchronize the calling thread on a specific barrier with custom data
	 *
	 * @param BarrierId   - ID of a barrier to use for synchronization
	 * @param CallerId    - ID of a synchronization caller (thread)
	 * @param RequestData - Synchronization request data of the calling thread (caller)
	 * @param OutResponseData - [out] Synchronization response data
	 *
	 * @return true if synchronization succeeded
	 */
	virtual bool Synchronize(const FString& BarrierId, const FString& CallerId, const TArray<uint8>& RequestData, TArray<uint8>& OutResponseData) = 0;

public:

	/**
	 * Returns synchronization delegate of a specific barrier. This delegate is called on the primary node only.
	 *
	 * @param BarrierId - ID of the barrier
	 *
	 * @return Synchronization delegate or nullptr if barrier is not available
	 */
	virtual FOnGenericBarrierSynchronizationDelegate* GetBarrierSyncDelegate(const FString& BarrierId) = 0;

	/**
	 * Returns timeout delegate of a specific barrier. This delegate is called on the primary node only.
	 *
	 * @param BarrierId - ID of the barrier
	 *
	 * @return Timeout delegate or nullptr if barrier is not available
	 */
	virtual FOnGenericBarrierTimeoutDelegate* GetBarrierTimeoutDelegate(const FString& BarrierId) = 0;

public:

	UE_DEPRECATED(5.6, "This method has been deprecated. There is no need to connect/disconnect anymore.")
	virtual bool Connect()
	{
		return true;
	}

	UE_DEPRECATED(5.6, "This method has been deprecated. There is no need to connect/disconnect anymore.")
	virtual void Disconnect()
	{
	}

	UE_DEPRECATED(5.6, "This method has been deprecated. There is no need to connect/disconnect anymore.")
	virtual bool IsConnected() const
	{
		return true;
	}

	UE_DEPRECATED(5.6, "This method has been deprecated.")
	virtual FString GetName() const
	{
		return FString();
	}

	UE_DEPRECATED(5.6, "This method has been deprecated. Please use a TMap based version of CreateBarrier.")
	virtual bool CreateBarrier(const FString& BarrierId, const TArray<FString>& UniqueThreadMarkers, uint32 Timeout)
	{
		return false;
	}
};


struct UE_DEPRECATED(5.6, "No longer used.") FDisplayClusterGenericBarriersClientDeleter
{
	void operator()(void*)
	{ }
};
