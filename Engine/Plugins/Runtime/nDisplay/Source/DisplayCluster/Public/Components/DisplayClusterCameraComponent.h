// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/SceneComponent.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Components/IDisplayClusterComponent.h"
#include "Render/DisplayDevice/Containers/DisplayClusterDisplayDevice_Enums.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_Enums.h"

#include "DisplayClusterCameraComponent.generated.h"

class UMaterial;
class UMaterialInstanceDynamic;
class UMeshComponent;
class UBillboardComponent;
class UTexture2D;
class IDisplayClusterViewportManager;
class IDisplayClusterWarpPolicy;
class IDisplayClusterViewportConfiguration;
class IDisplayClusterViewportPreview;
class IDisplayClusterViewport;
class UCameraComponent;
class ACineCameraActor;
struct FMinimalViewInfo;

UENUM()
enum class EDisplayClusterEyeStereoOffset : uint8
{
	None  UMETA(DisplayName = "Default"),
	Left  UMETA(DisplayName = "Left Eye"),
	Right UMETA(DisplayName = "Right Eye"),
};

/** The type of camera used. */
UENUM()
enum class EDisplayClusterTargetCameraType: int32
{
	// No camera used
	None  UMETA(DisplayName = "Disabled"),

	// Use active engine camera
	ActiveEngineCamera UMETA(DisplayName = "Active Engine Camera"),

	// Use camera component from the root actor
	ICVFXCameraComponent UMETA(DisplayName = "ICVFX Camera"),

	// Use the CineCamera actor reference.
	ExternalCineCameraActor UMETA(DisplayName = "Camera Actor"),
};


/**
 * 3D point in space used to render nDisplay viewports from
 */
UCLASS(ClassGroup = (DisplayCluster), HideCategories = (Navigation, AssetUserData, LOD, Physics, Cooking, Activation, Tags, Gizmo, Collision, ComponentReplication, Events, Sockets, ComponentTick, Rendering), meta = (BlueprintSpawnableComponent, DisplayName = "NDisplay View Point"))
class DISPLAYCLUSTER_API UDisplayClusterCameraComponent
	: public USceneComponent
	, public IDisplayClusterComponent
{
	GENERATED_BODY()

public:
	UDisplayClusterCameraComponent(const FObjectInitializer& ObjectInitializer);

	/** Return ViewPoint for this component
	 * If the component logic supports postprocess, it will also be in the ViewInfo structure.
	 *
	 * @param InOutViewInfo - ViewInfo data
	 * @param OutCustomNearClippingPlane - Custom NCP, or a value less than zero if not defined.
	 */
	UE_DEPRECATED(5.4, "This function has been deprecated. Please use 'GetDesiredView()'.")
	virtual void GetDesiredView(FMinimalViewInfo& InOutViewInfo, float* OutCustomNearClippingPlane = nullptr)
	{ }

	/** Return ViewPoint for this component
	 * If the component logic supports postprocess, it will also be in the ViewInfo structure.
	 *
	 * @param InOutViewInfo - ViewInfo data
	 * @param OutCustomNearClippingPlane - Custom NCP, or a value less than zero if not defined.
	 */
	virtual void GetDesiredView(const IDisplayClusterViewportConfiguration& InViewportConfiguration, FMinimalViewInfo& InOutViewInfo, float* OutCustomNearClippingPlane = nullptr) const;

	/** Returns the position of the observer's eyes in the Stage. */
	virtual void GetEyePosition(const IDisplayClusterViewportConfiguration& InViewportConfiguration, FVector& OutViewLocation, FRotator& OutViewRotation);

	/**
	 * All cluster viewports that reference this component will be created in the background on the current cluster node if the function returns true.
	 */
	virtual bool ShouldUseEntireClusterViewports(IDisplayClusterViewportManager* InViewportManager) const
	{
		return false;
	}

	/**
	 * Get the warp policy instance used by this compoenent.
	 * From the DC ViewportManager, these policies will be assigned to the viewports that use this viewpoint component.
	 */
	virtual IDisplayClusterWarpPolicy* GetWarpPolicy(IDisplayClusterViewportManager* InViewportManager)
	{
		return nullptr;
	}

	/** Override DisplayDevice material by type for 
	* The UDisplayClusterInFrustumFitCameraComponent uses its own material to display additional deformed preview meshes in front of the camera.
	*
	* @param InMeshType     - mesh type
	* @param InMaterialType - the type of material being requested
	* 
	* @return nullptr if DisplayDevice material is used.
	*/
	virtual TObjectPtr<UMaterial> GetDisplayDeviceMaterial(const EDisplayClusterDisplayDeviceMeshType InMeshType, const EDisplayClusterDisplayDeviceMaterialType InMaterialType) const
	{
		return nullptr;
	}

	/** Perform any operations on the mesh and material instance, such as setting parameter values.
	*
	* @param InViewport             - current viewport
	* @param InMeshType             - mesh type
	* @param InMaterialType         - type of material being requested
	* @param InMeshComponent        - mesh component to be updated
	* @param InMeshMaterialInstance - material instance that used on this mesh
	*/
	virtual void OnUpdateDisplayDeviceMeshAndMaterialInstance(IDisplayClusterViewportPreview& InViewportPreview, const EDisplayClusterDisplayDeviceMeshType InMeshType, const EDisplayClusterDisplayDeviceMaterialType InMaterialType, UMeshComponent* InMeshComponent, UMaterialInstanceDynamic* InMeshMaterialInstance) const
	{ }

	/** Apply the ViewPoint component's post-processes to the viewport.
	* (Outer viewport camera)
	*
	* @param InViewport - viewport to be configured.
	*/
	virtual void ApplyViewPointComponentPostProcessesToViewport(IDisplayClusterViewport* InViewport);

	/** Return a reference to the Camera component, which is used for Outer viewports.
	* 
	* return nullptr if the camera is not in use.
	*/
	virtual UCameraComponent* GetTargetCameraComponent(const IDisplayClusterViewportConfiguration& InViewportConfiguration) const;

	/** Returns true if the given viewpoint position should be used as a camera position. */
	virtual bool IsViewPointOverrideCameraPosition() const;

	/** Returns true if the eye position (ViewPoint) inside the DCRa can be overriden. */
	virtual bool CanOverrideEyePosition() const;

	/** Returns true if ViewPoint should use the camera position instead of the DCRA position when transforming. */
	virtual bool ShouldFollowCameraLocation() const;

protected:
	/** Get view from the referenced camera. */
	virtual bool GetTargetCameraDesiredViewInternal(const IDisplayClusterViewportConfiguration& InViewportConfiguration, FMinimalViewInfo& InOutViewInfo, float* OutCustomNearClippingPlane = nullptr) const;

#if WITH_EDITOR
	virtual bool GetEditorPreviewInfo(float DeltaTime, FMinimalViewInfo& ViewOut) override;
	virtual TSharedPtr<SWidget> GetCustomEditorPreviewWidget() override;
#endif

public:
	/**
	* Get interpupillary distance
	*
	* @return - Interpupillary distance
	*/
	UFUNCTION(BlueprintCallable, Category = "Stereo")
	float GetInterpupillaryDistance() const
	{
		return InterpupillaryDistance;
	}

	/**
	* Set interpupillary distance
	*
	* @param Distance - New interpupillary distance
	*/
	UFUNCTION(BlueprintCallable, Category = "Stereo")
	void SetInterpupillaryDistance(float Distance)
	{
		InterpupillaryDistance = Distance;
	}

	/**
	* Get swap eyes state
	*
	* @return - Eyes swap state. False - normal eyes left|right, true - swapped eyes right|left
	*/
	UFUNCTION(BlueprintCallable, Category = "Stereo")
	bool GetSwapEyes() const
	{
		return bSwapEyes;
	}

	/**
	* Set swap eyes state
	*
	* @param SwapEyes - New eyes swap state. False - normal eyes left|right, true - swapped eyes right|left
	*/
	UFUNCTION(BlueprintCallable, Category = "Stereo")
	void SetSwapEyes(bool SwapEyes)
	{
		bSwapEyes = SwapEyes;
	}

	/**
	* Toggles eyes swap state
	*
	* @return - New eyes swap state. False - normal eyes left|right, true - swapped eyes right|left
	*/
	UFUNCTION(BlueprintCallable, Category = "Stereo")
	bool ToggleSwapEyes()
	{
		return (bSwapEyes = !bSwapEyes);
	}

	/**
	* Get stereo offset type
	*
	* @return - Current forced stereo offset type
	*/
	UFUNCTION(BlueprintCallable, Category = "Stereo")
	EDisplayClusterEyeStereoOffset GetStereoOffset() const
	{
		return StereoOffset;
	}

	/**
	* Set stereo offset type
	*
	* @param StereoOffset - New forced stereo offset type
	*/
	UFUNCTION(BlueprintCallable, Category = "Stereo")
	void SetStereoOffset(EDisplayClusterEyeStereoOffset InStereoOffset)
	{
		StereoOffset = InStereoOffset;
	}

public:
	/** Decodes parameters into flags. */
	EDisplayClusterViewportCameraPostProcessFlags GetCameraPostProcessFlags() const;

	/** Returns the camera component from the 'ExternalCameraActor' parameter. */
	virtual UCameraComponent* GetExternalCineCameraActorComponent() const;

protected:
	/** Returns true if active engine camera is used. */
	virtual bool IsActiveEngineCameraBeingUsed() const;

	/** Returns true if the `ICVFXCameraComponentName` parameter is used. */
	virtual bool IsICVFXCameraBeingUsed() const;

	/** Returns true if the `ExternalCineCameraActor` parameter is used. */
	virtual bool IsExternalCameraBeingUsed() const;

#if WITH_EDITOR
	/** Return used camera component. */
	virtual UCameraComponent* GetEditorPreviewCameraComponent();
#endif

public:
#if WITH_EDITOR
	// Begin IDisplayClusterComponent
	virtual void SetVisualizationScale(float Scale) override;
	virtual void SetVisualizationEnabled(bool bEnabled) override;
	// End IDisplayClusterComponent
#endif

	// Begin UActorComponent
	virtual void OnRegister() override;
	// End UActorComponent

	// Begin UObject
#if WITH_EDITOR
	virtual bool CanEditChange(const FProperty* InProperty) const override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	// End UObject

protected:
#if WITH_EDITOR
	/** Refreshes the visual components to match the component state */
	virtual void RefreshVisualRepresentation();
#endif

#if WITH_EDITORONLY_DATA
protected:
	/** Gizmo visibility */
	UPROPERTY(EditAnywhere, Category = "Gizmo")
	uint8 bEnableGizmo : 1;

	/** Base gizmo scale */
	UPROPERTY(EditAnywhere, Category = "Gizmo")
	FVector BaseGizmoScale;

	/** Gizmo scale multiplier */
	UPROPERTY(EditAnywhere, Category = "Gizmo", meta = (UIMin = "0", UIMax = "2.0", ClampMin = "0.01", ClampMax = "10.0"))
	float GizmoScaleMultiplier;

	UPROPERTY(Transient)
	TObjectPtr<UBillboardComponent> SpriteComponent;

	UPROPERTY()
	TObjectPtr<UTexture2D> SpriteTexture;
#endif

public:
	/** Type of source camera used. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Camera Settings", meta = (DisplayName = "Source Camera"))
	EDisplayClusterTargetCameraType TargetCameraType = EDisplayClusterTargetCameraType::None;

	/** The name of the camera component that is used as the PP source. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Camera Settings", meta = (DisplayName = "ICVFX Camera", EditCondition = "TargetCameraType==EDisplayClusterTargetCameraType::ICVFXCameraComponent", EditConditionHides))
	FString ICVFXCameraComponentName;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Camera Settings", meta = (DisplayName = "Follow ICVFX Camera", EditCondition = "TargetCameraType==EDisplayClusterTargetCameraType::ICVFXCameraComponent", EditConditionHides))
	uint8 bUseICVFXCameraComponentTracking: 1 = 1;

	/** Use camera position for rendering instead of RootActor. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Camera Settings", meta = (DisplayName = "Follow Camera", EditCondition = "TargetCameraType!=EDisplayClusterTargetCameraType::ICVFXCameraComponent && TargetCameraType!=EDisplayClusterTargetCameraType::None", EditConditionHides))
	uint8 bFollowCameraPosition : 1 = 0;

	/** Use a specific actor camera instead of a game camera. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Camera Settings", meta = (DisplayName = "Camera Actor", EditCondition = "TargetCameraType==EDisplayClusterTargetCameraType::ExternalCineCameraActor", EditConditionHides))
	TSoftObjectPtr<ACineCameraActor> ExternalCineCameraActor;

public:

	/** Use the PP settings from the specified camera. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Camera Settings", meta = (DisplayName = "Include Post Process Settings", EditCondition = "TargetCameraType!=EDisplayClusterTargetCameraType::None", EditConditionHides))
	uint8 bEnablePostProcess : 1 = 1;

	/** Use the NearClippingPlane value from the specified cine camera. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Camera Settings", meta = (DisplayName = "Include Custom Near Clipping Plane", EditCondition = "TargetCameraType!=EDisplayClusterTargetCameraType::None && TargetCameraType!=EDisplayClusterTargetCameraType::ICVFXCameraComponent", EditConditionHides))
	uint8 bEnableNearClippingPlane : 1 = 1;

	/** Enable the DoF PP settings from the specified camera. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Camera Settings", meta = (DisplayName = "Include Depth Of Field", EditCondition = "TargetCameraType!=EDisplayClusterTargetCameraType::None && TargetCameraType!=EDisplayClusterTargetCameraType::ICVFXCameraComponent", EditConditionHides))
	uint8 bEnableDepthOfField : 1 = 1;

	/** Use the DC Depth-Of-Field settings from the specified ICVFX camera. */
	UPROPERTY(BlueprintReadWrite, Category = "Camera Settings", meta = (DisplayName = "Include Inner Frustum Depth Of Field Compensation", EditCondition = "TargetCameraType==EDisplayClusterTargetCameraType::ICVFXCameraComponent", EditConditionHides))
	uint8 bEnableICVFXDepthOfFieldCompensation : 1 = 1;

	/** Use the DC ColorGrading from the specified ICVFX camera. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Camera Settings", meta = (DisplayName = "Include Inner Frustum Color Grading", EditCondition = "TargetCameraType==EDisplayClusterTargetCameraType::ICVFXCameraComponent", EditConditionHides))
	uint8 bEnableICVFXColorGrading : 1 = 1;

	/** Use the DC Motion Blur settings from the specified ICVFX camera. */
	UPROPERTY(BlueprintReadWrite, Category = "Camera Settings", meta = (DisplayName = "Include Inner Frustum Motion Blur", EditCondition = "TargetCameraType==EDisplayClusterTargetCameraType::ICVFXCameraComponent", EditConditionHides))
	uint8 bEnableICVFXMotionBlur : 1 = 1;

private:
	UPROPERTY(EditAnywhere, Category = "Stereo")
	float InterpupillaryDistance;
	
	UPROPERTY(EditAnywhere, Category = "Stereo")
	bool bSwapEyes;
	
	UPROPERTY(EditAnywhere, Category = "Stereo")
	EDisplayClusterEyeStereoOffset StereoOffset;
};
