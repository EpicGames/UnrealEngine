// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DisplayClusterStageActorComponent.h"

#include "DisplayClusterChromakeyCardStageActorComponent.generated.h"

class ADisplayClusterRootActor;
class UDisplayClusterICVFXCameraComponent;

/**
 * Stage Actor Component to be placed in chromakey card actors
 */
UCLASS(MinimalAPI, ClassGroup = (DisplayCluster), meta = (DisplayName = "Chromakey Card Stage Actor"), HideCategories=(Physics, Collision, Lighting, Navigation, Cooking, LOD, MaterialParameters, HLOD, RayTracing, TextureStreaming, Mobile))
class UDisplayClusterChromakeyCardStageActorComponent final : public UDisplayClusterStageActorComponent
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	DISPLAYCLUSTER_API virtual void SetRootActor(ADisplayClusterRootActor* InRootActor) override;

	/** Retrieve current ICVFX cameras associated with this stage actor */
	const TArray<FSoftComponentReference>& GetICVFXCameras() const { return ICVFXCameras; }

	/**
	* Checks if the given ICVFX camera has chroma key settings supporting this actor.
	* 
	* @param InCamera The ICVFX camera component to compare against.
	*
	* @return true if the ICVFX camera component is referenced by this component.
	*/
	DISPLAYCLUSTER_API bool IsReferencedByICVFXCamera(const UDisplayClusterICVFXCameraComponent* InCamera) const;
	
protected:
	/** Update linked ICVFX cameras based on current settings */
	void PopulateICVFXOwners();
	
protected:
	/** Associate this Chromakey Card with the nDisplay configuration actor and ICVFX Camera component(s) specified here.
	 * The content will only appear in the ICVFX Editor when this nDisplay configuration is selected. */
	UPROPERTY(EditInstanceOnly, Category = "NDisplay Root Actor", DisplayName = "ICVFX Cameras", meta = (UseComponentPicker, AllowedClasses = "/Script/DisplayCluster.DisplayClusterICVFXCameraComponent", AllowAnyActor))
	TArray<FSoftComponentReference> ICVFXCameras;
};