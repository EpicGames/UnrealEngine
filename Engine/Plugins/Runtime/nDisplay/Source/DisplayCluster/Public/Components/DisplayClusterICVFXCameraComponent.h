// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "CineCameraActor.h"
#include "CineCameraComponent.h"

#include "DisplayClusterConfigurationTypes_ICVFX.h"
#include "DisplayClusterEditorPropertyReference.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_Enums.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_CameraMotionBlur.h"
#include "ShaderParameters/DisplayClusterShaderParameters_ICVFX.h"

#include "DisplayClusterICVFXCameraComponent.generated.h"

struct FMinimalViewInfo;
class SWidget;
class UCameraComponent;
class IDisplayClusterViewport;


/**
 * nDisplay in-camera VFX camera representation
 */
UCLASS(ClassGroup = (DisplayCluster), HideCategories = (AssetUserData, Collision, Cooking, ComponentReplication, Events, Physics, Sockets, Activation, Tags, ComponentTick), meta = (BlueprintSpawnableComponent, DisplayName="ICVFX Camera"))
class DISPLAYCLUSTER_API UDisplayClusterICVFXCameraComponent
	: public UCineCameraComponent
{
	GENERATED_BODY()

public:
	UDisplayClusterICVFXCameraComponent(const FObjectInitializer& ObjectInitializer);

	virtual void Serialize(FArchive& Ar) override;
	virtual void PostLoad() override;
	virtual void PostApplyToComponent() override;

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = NDisplay)
	FDisplayClusterConfigurationICVFX_CameraSettings CameraSettings;

#if WITH_EDITOR
	virtual bool GetEditorPreviewInfo(float DeltaTime, FMinimalViewInfo& ViewOut) override;
	virtual TSharedPtr<SWidget> GetCustomEditorPreviewWidget() override;
#endif

	virtual void TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;

	/** The ICVFXCamera component uses its own PostProcess and applies it to the viewport.
	*
	* @param InViewport - a viewport that need to be configured with a PP settings.
	* @param InPostProcessFlags - Define the PostProcess set that will be used
	*/
	virtual void ApplyICVFXCameraPostProcessesToViewport(IDisplayClusterViewport* InDestViewport, const EDisplayClusterViewportCameraPostProcessFlags InPostProcessingFlags);

public:
	UE_DEPRECATED(5.5, "This function has been deprecated. Please use 'FDisplayClusterViewportConfigurationHelpers_Postprocess::GetICVFXCameraMotionBlurParameters()'.")
	FDisplayClusterViewport_CameraMotionBlur GetMotionBlurParameters()
	{
		return FDisplayClusterViewport_CameraMotionBlur();
	}

	/**
	 * Return the actual source camera, e.g. the camera component of the referenced cine camera.
	 * Use GetCameraView() function to get viewinfo with actual camera position, postprocess and ICVFX postprocess
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "ICVFX Camera")
	UCineCameraComponent* GetActualCineCameraComponent();

	/** Returns true if this camera is active in the current node. */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "ICVFX Camera")
	bool IsICVFXEnabled() const;

	// Return unique camera name
	FString GetCameraUniqueId() const;

	const FDisplayClusterConfigurationICVFX_CameraSettings& GetCameraSettingsICVFX() const;

	/** Obtaining view information for the actual camera, such as the camera component to which the cine-camera is referencing.
	 * The data from the CameraSettings variable is used in postprocess settings (EnableCameraPP, OverrideMotionBlur, etc.).
	 */
	virtual void GetCameraView(float DeltaTime, FMinimalViewInfo& InOutViewInfo) override;

	// UActorComponent interface
	virtual void OnRegister() override;

	/** Sets new depth of field parameters and updates the dynamically generated compensation LUT if needed */
	UFUNCTION(BlueprintCallable, Category = "ICVFX Camera")
	void SetDepthOfFieldParameters(const FDisplayClusterConfigurationICVFX_CameraDepthOfField& NewDepthOfFieldParams);

	/** Get ICVFX camera frame resolution. */
	FIntPoint GetICVFXCameraFrameSize(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings, const FDisplayClusterConfigurationICVFX_CameraSettings& InCameraSettings);

	/** Get ICVFX camera shader parameters. */
	FDisplayClusterShaderParameters_ICVFX::FCameraSettings GetICVFXCameraShaderParameters(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings, const FDisplayClusterConfigurationICVFX_CameraSettings& InCameraSettings);

private:
	void UpdateOverscanEstimatedFrameSize();

//////////////////////////////////////////////////////////////////////////////////////////////
// Details Panel Property Referencers
//////////////////////////////////////////////////////////////////////////////////////////////
#if WITH_EDITORONLY_DATA
public:
	// update the status of the Editor's ICVFX preview elements
	void UpdateICVFXPreviewState();

	// saves the value of external camera reference
	TSoftObjectPtr<ACineCameraActor> ExternalCameraCachedValue;

	virtual void PreEditChange(FProperty* PropertyThatWillChange) override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

private:
	friend class FDisplayClusterICVFXCameraComponentDetailsCustomization;
	
	UPROPERTY(EditAnywhere, Transient, Category = "Inner Frustum", meta = (PropertyPath = "CameraSettings.bEnable"))
	FDisplayClusterEditorPropertyReference IsEnabledRef;

	UPROPERTY(EditAnywhere, Transient, Category = "Inner Frustum", meta = (PropertyPath = "CameraSettings.HiddenICVFXViewports"))
	FDisplayClusterEditorPropertyReference HiddenICVFXViewportsRef;

	UPROPERTY(EditAnywhere, Transient, Category = "Inner Frustum", meta = (PropertyPath = "CameraSettings.UpscalerSettings"))
	FDisplayClusterEditorPropertyReference UpscalerSettingsRef;

	UPROPERTY(EditAnywhere, Transient, Category = "Inner Frustum", meta = (PropertyPath = "CameraSettings.BufferRatio"))
	FDisplayClusterEditorPropertyReference BufferRatioRef;

	UPROPERTY(EditAnywhere, Transient, Category = "Inner Frustum", meta = (DisplayName = "Inner Frustum Overscan", PropertyPath = "CameraSettings.CustomFrustum"))
	FDisplayClusterEditorPropertyReference CustomFrustumRef;

	UPROPERTY(EditAnywhere, Transient, Category = "Inner Frustum", meta = (PropertyPath = "CameraSettings.SoftEdge"))
	FDisplayClusterEditorPropertyReference SoftEdgeRef;

	UPROPERTY(EditAnywhere, Transient, Category = "Inner Frustum", meta = (PropertyPath = "CameraSettings.Border"))
	FDisplayClusterEditorPropertyReference BorderRef;

	UPROPERTY(EditAnywhere, Transient, Category = "Inner Frustum", meta = (PropertyPath = "CameraSettings.FrustumRotation"))
	FDisplayClusterEditorPropertyReference FrustumRotationRef;

	UPROPERTY(EditAnywhere, Transient, Category = "Inner Frustum", meta = (PropertyPath = "CameraSettings.FrustumOffset"))
	FDisplayClusterEditorPropertyReference FrustumOffsetRef;

	UPROPERTY(EditAnywhere, Transient, Category = "Inner Frustum", meta = (PropertyPath = "CameraSettings.OffCenterprojectionoffset"))
	FDisplayClusterEditorPropertyReference OffCenterProjectionOffsetRef;

	UPROPERTY(EditAnywhere, Transient, Category = "Inner Frustum", meta = (PropertyPath = "CameraSettings.RenderSettings.GenerateMips"))
	FDisplayClusterEditorPropertyReference GenerateMipsRef;

	UPROPERTY(EditAnywhere, Transient, Category = "Camera ICVFX", meta = (PropertyPath = "CameraSettings.ExternalCameraActor"))
	FDisplayClusterEditorPropertyReference ExternalCameraActorRef;

	/** Exposed reference to the camera's inner depth of field settings */
	UPROPERTY(EditAnywhere, Transient, Category = "Camera ICVFX", meta = (PropertyPath = "CameraSettings.CameraDepthOfField", DisplayName = "ICVFX Depth of Field"))
	FDisplayClusterEditorPropertyReference CameraDepthOfFieldRef;

	UPROPERTY(EditAnywhere, Transient, Category = "Camera ICVFX", meta = (PropertyPath = "CameraSettings.CameraMotionBlur", DisplayName = "ICVFX Camera Motion Blur"))
	FDisplayClusterEditorPropertyReference CameraMotionBlurRef;

	UPROPERTY(EditAnywhere, Transient, Category = "Inner Frustum", meta = (PropertyPath = "CameraSettings.CameraHideList"))
	FDisplayClusterEditorPropertyReference CameraHideListRef;

	UPROPERTY(EditAnywhere, Transient, Category = Chromakey, meta = (PropertyPath = "CameraSettings.Chromakey.bEnable"))
	FDisplayClusterEditorPropertyReference ChromaKeyEnabledRef;

	UPROPERTY(EditAnywhere, Transient, Category = Chromakey, meta = (PropertyPath = "CameraSettings.Chromakey.ChromakeyType"))
	FDisplayClusterEditorPropertyReference ChromakeyTypeRef;

	UPROPERTY(EditAnywhere, Transient, Category = Chromakey, meta = (PropertyPath = "CameraSettings.Chromakey.ChromakeySettingsSource"))
	FDisplayClusterEditorPropertyReference ChromakeySettingsSourceRef;

	UPROPERTY(EditAnywhere, Transient, Category = Chromakey, meta = (PropertyPath = "CameraSettings.Chromakey.ChromakeyColor"))
	FDisplayClusterEditorPropertyReference ChromakeyColorRef;

	UPROPERTY(EditAnywhere, Transient, Category = Chromakey, meta = (PropertyPath = "CameraSettings.Chromakey.ChromakeyMarkers"))
	FDisplayClusterEditorPropertyReference ChromakeyMarkersRef;

	UPROPERTY(EditAnywhere, Transient, Category = Chromakey, meta = (PropertyPath = "CameraSettings.Chromakey.ChromakeyRenderTexture"))
	FDisplayClusterEditorPropertyReference ChromakeyRenderTextureRef;

	UPROPERTY(EditAnywhere, Transient, Category = OCIO, meta = (PropertyPath = "CameraSettings.CameraOCIO.AllNodesOCIOConfiguration.bIsEnabled", DisplayName = "Enable Inner Frustum OCIO"))
	FDisplayClusterEditorPropertyReference EnableInnerFrustuOCIORef;

	UPROPERTY(EditAnywhere, Transient, Category = OCIO, meta = (PropertyPath = "CameraSettings.CameraOCIO.AllNodesOCIOConfiguration.ColorConfiguration", DisplayName = "All Nodes Color Configuration"))
	FDisplayClusterEditorPropertyReference AllNodesOCIOConfigurationRef;

	UPROPERTY(EditAnywhere, Transient, Category = OCIO, meta = (PropertyPath = "CameraSettings.CameraOCIO.PerNodeOCIOProfiles", DisplayName = "Per-Node OCIO Overrides"))
	FDisplayClusterEditorPropertyReference PerNodeOCIOProfilesRef;

	UPROPERTY(EditAnywhere, Transient, Category = "Inner Frustum Color Grading", meta = (PropertyPath = "CameraSettings.EnableInnerFrustumColorGrading", DisplayName = "Enable Inner Frustum Color Grading"))
	FDisplayClusterEditorPropertyReference EnableInnerFrustumColorGrading;

	UPROPERTY(EditAnywhere, Transient, Category = "Inner Frustum Color Grading", meta = (PropertyPath = "CameraSettings.AllNodesColorGrading", DisplayName = "All Nodes"))
	FDisplayClusterEditorPropertyReference AllNodesColorGradingRef;

	UPROPERTY(EditAnywhere, Transient, Category = "Inner Frustum Color Grading", meta = (PropertyPath = "CameraSettings.PerNodeColorGrading"))
	FDisplayClusterEditorPropertyReference PerNodeColorGradingRef;

	UPROPERTY(EditAnywhere, Transient, Category = "Texture Replacement", meta = (PropertyPath = "CameraSettings.RenderSettings.Replace.bAllowReplace", DisplayName = "Enable Inner Frustum Texture Replacement", ToolTip = "Set to True to replace the entire inner frustum with the specified texture."))
	FDisplayClusterEditorPropertyReference TextureReplacementEnabledRef;

	UPROPERTY(EditAnywhere, Transient, Category = "Texture Replacement", meta = (PropertyPath = "CameraSettings.RenderSettings.Replace.SourceTexture"))
	FDisplayClusterEditorPropertyReference SourceTextureRef;

	UPROPERTY(EditAnywhere, Transient, Category = "Texture Replacement", meta = (PropertyPath = "CameraSettings.RenderSettings.Replace.bShouldUseTextureRegion"))
	FDisplayClusterEditorPropertyReference ShouldUseTextureRegionRef;

	UPROPERTY(EditAnywhere, Transient, Category = "Texture Replacement", meta = (PropertyPath = "CameraSettings.RenderSettings.Replace.TextureRegion"))
	FDisplayClusterEditorPropertyReference TextureRegionRef;

	UPROPERTY(EditDefaultsOnly, Transient, Category = "Media", meta = (PropertyPath = "CameraSettings.RenderSettings.Media"))
	FDisplayClusterEditorPropertyReference MediaRef;

	UPROPERTY(EditAnywhere, Transient, Category = Configuration, meta = (PropertyPath = "CameraSettings.RenderSettings.RenderOrder"))
	FDisplayClusterEditorPropertyReference RenderOrderRef;

	UPROPERTY(EditAnywhere, Transient, Category = Configuration, meta = (PropertyPath = "CameraSettings.RenderSettings.CustomFrameSize"))
	FDisplayClusterEditorPropertyReference CustomFrameSizeRef;

	UPROPERTY(EditAnywhere, Transient, Category = Configuration, meta = (PropertyPath = "CameraSettings.RenderSettings.AdvancedRenderSettings.RenderTargetRatio"))
	FDisplayClusterEditorPropertyReference RenderTargetRatioRef;

	UPROPERTY(EditAnywhere, Transient, Category = Configuration, meta = (PropertyPath = "CameraSettings.RenderSettings.AdvancedRenderSettings.GPUIndex"))
	FDisplayClusterEditorPropertyReference GPUIndexRef;

	UPROPERTY(EditAnywhere, Transient, Category = Configuration, meta = (PropertyPath = "CameraSettings.RenderSettings.AdvancedRenderSettings.StereoGPUIndex"))
	FDisplayClusterEditorPropertyReference StereoGPUIndexRef;

	UPROPERTY(EditAnywhere, Transient, Category = Configuration, meta = (PropertyPath = "CameraSettings.RenderSettings.AdvancedRenderSettings.StereoMode"))
	FDisplayClusterEditorPropertyReference StereoModeRef;

#endif // WITH_EDITORONLY_DATA
};
