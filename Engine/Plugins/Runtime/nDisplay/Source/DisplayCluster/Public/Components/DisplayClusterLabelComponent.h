// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/SceneComponent.h"
#include "Components/DisplayClusterLabelConfiguration.h"

#include "DisplayClusterLabelComponent.generated.h"

class ADisplayClusterRootActor;
class UDisplayClusterLabelWidget;
class UDisplayClusterWidgetComponent;
class UTextRenderComponent;
class UUserWidget;
class UWidgetComponent;

/**
 * A label component specific to nDisplay. Displays a widget with a consistent scale facing the root actor view point.
 * Visible only in scene capture when in the editor or in 3d space on the wall.
 *
 * The component needs to be transient so its settings aren't saved, but should still transact over multi-user.
 */
UCLASS(transient, meta=(BlueprintSpawnableComponent, DisplayClusterMultiUserInclude))
class DISPLAYCLUSTER_API UDisplayClusterLabelComponent : public USceneComponent
{
	GENERATED_BODY()

public:
	
	UDisplayClusterLabelComponent();

	/** Set the configuration for this label updating multiple properties at once */
	void SetLabelConfiguration(const FDisplayClusterLabelConfiguration& InConfiguration);
	
	/** Set the root actor this label should use when determining transform */
	void SetRootActor(ADisplayClusterRootActor* InActor);

	/** Set the widget scale */
	void SetWidgetScale(float NewValue);

	/** Return the widget scale */
	float GetWidgetScale() const;

	/** Updates flags for this label */
	void SetLabelFlags(EDisplayClusterLabelFlags InFlags);
	
	/** Clears all label flags */
	void ClearLabelFlags(EDisplayClusterLabelFlags InFlags);
	
	/** Retrieve label flags */
	EDisplayClusterLabelFlags GetLabelFlags() const { return LabelFlags; }
	
	/** Checks for existence of a flag on this label */
	bool HasAnyLabelFlags(EDisplayClusterLabelFlags InFlags) const;
	
	virtual void TickComponent(float DeltaTime, ELevelTick TickType,
	                           FActorComponentTickFunction* ThisTickFunction) override;

	virtual void OnRegister() override;

	/** Returns the widget component */
	UWidgetComponent* GetWidgetComponent() const;

#if WITH_EDITOR
public:
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	
protected:
	/** Updates the widget component text and transform */
	void UpdateWidgetComponent();

	/** Verify the widget visibility matches this component's visibility */
	void CheckForVisibilityChange();
	
private:
	/** The root actor this label needs to know about */
	UPROPERTY()
	TWeakObjectPtr<ADisplayClusterRootActor> RootActor;
	
	/** The widget component to display for this label */
	UPROPERTY(VisibleAnywhere, Category=LabelText)
	TObjectPtr<UDisplayClusterWidgetComponent> WidgetComponent;

	/** The widget class to apply to the widget component */
	UPROPERTY(EditAnywhere, Category=LabelText)
	TSoftClassPtr<UDisplayClusterLabelWidget> WidgetClass;

	/** A uniform scale to apply to the text which will keep consistency across distance from the label to the root actor view */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Label, meta = (AllowPrivateAccess))
	float WidgetScale = 1.f;

	/** Current flags set for this label component */
	UPROPERTY()
	EDisplayClusterLabelFlags LabelFlags;
};
