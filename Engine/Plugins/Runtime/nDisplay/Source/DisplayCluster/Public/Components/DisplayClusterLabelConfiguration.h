// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectPtr.h"

#include "DisplayClusterLabelConfiguration.generated.h"

class ADisplayClusterRootActor;

UENUM()
enum class EDisplayClusterLabelFlags : uint8
{
	/** No label flags */
	None              = 0,
	/** Label allowed to be displayed in -game */
	DisplayInGame     = 1 << 0,
	/** Label allowed to be displayed in editor */
	DisplayInEditor   = 1 << 1
};
ENUM_CLASS_FLAGS(EDisplayClusterLabelFlags)

/** Generic args a label can recognize */
struct FDisplayClusterLabelConfiguration
{
	/** The root actor */
	TObjectPtr<ADisplayClusterRootActor> RootActor;
		
	/**
	 * The scale to apply to the label
	 */
	float Scale = 1.f;

	/**
	 * Should the label be visible
	 */
	bool bVisible = false;

	/**
	 * Flags for the label. Can be used to fine-tune visibility settings
	 */
	EDisplayClusterLabelFlags LabelFlags;
};