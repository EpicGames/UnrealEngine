// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DisplayClusterStageActorComponent.h"

#include "DisplayClusterLightCardStageActorComponent.generated.h"

/**
 * Stage Actor Component to be placed in light card actors
 */
UCLASS(MinimalAPI, ClassGroup = (DisplayCluster), meta = (DisplayName = "Light Card Stage Actor"), HideCategories=(Physics, Collision, Lighting, Navigation, Cooking, LOD, MaterialParameters, HLOD, RayTracing, TextureStreaming, Mobile))
class UDisplayClusterLightCardStageActorComponent final : public UDisplayClusterStageActorComponent
{
	GENERATED_BODY()
};