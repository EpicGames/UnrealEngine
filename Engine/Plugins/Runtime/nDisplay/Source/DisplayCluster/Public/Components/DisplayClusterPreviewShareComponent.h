// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ActorComponent.h"
#include "Containers/Map.h"

#if WITH_EDITOR
	#include "UnrealEdMisc.h"
#endif //WITH_EDITOR

#include "DisplayClusterPreviewShareComponent.generated.h"


class ADisplayClusterRootActor;
class UMediaCapture;
class UMediaOutput;
class UMediaPlayer;
class UMediaSource;
class UMediaTexture;
class UTexture;


/** Available sharing modes */
UENUM()
enum class EDisplayClusterPreviewShareMode
{
	/** Preview share disabled. */
	None,

	/** Pull from Source nDisplay Actor (if set) */
	PullActor,

	/** Sends the viewport textures for sharing */
	Send,

	/** Receives textures to replace the viewport textures with */
	Receive,
};

/** Available Icvfx camera sync types */
UENUM()
enum class EDisplayClusterPreviewShareIcvfxSync
{
	/** Icvfx cameras will not be synced */
	None,

	/** Pull from Source nDisplay Actor */
	PullActor,

	/** Pushes to source display actor */
	PushActor,
};

class IDisplayClusterViewportManager;

/**
 * nDisplay Viewport preview share component
 * 
 * It shares using Shared Memory Media the viewport textures of the parent nDisplay Actor.
 * It should only be added to DisplayClusterRootActor instances, and only one component per instance.
 * The way it works is that the sender generates a unique name for each viewport and captures its texture
 * by getting a pointer to it from the corresponding Preview Component.
 * The receiver will read it using the corresponding media source, and use the Texture Replace functionality
 * in the nDisplay actor viewports to have them used and displayed.
 */
UCLASS(ClassGroup = (DisplayCluster), meta = (BlueprintSpawnableComponent), HideCategories = (Activation, Collision, Cooking))
class DISPLAYCLUSTER_API UDisplayClusterPreviewShareComponent
	: public UActorComponent
{
	GENERATED_BODY()

public:

	/** Constructor */
	UDisplayClusterPreviewShareComponent(const FObjectInitializer& ObjectInitializer);

	//~ UActorComponent interface begin
	virtual void TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	virtual void OnComponentDestroyed(bool bDestroyingHierarchy) override;
	//~ UActorComponent interface end

	//~ UObject interface begin

	virtual void Serialize(FArchive& Ar) override;
	virtual void PostLoad() override;

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR

	//~ UObject interface end

	/** Sets the sharing mode */
	UFUNCTION(BlueprintCallable, Category = "Sharing")
	void SetMode(EDisplayClusterPreviewShareMode NewMode);

	/** Sets the unique name, which should match between sender and receiver of viewport textures */
	UFUNCTION(BlueprintCallable, Category = "Sharing")
	void SetUniqueName(const FString& NewUniqueName);


protected:
	/**
	* Creates or updates a custom rendering for the specified RootActor.
	* Returns nullptr if rendering is not possible, or a reference to a custom viewport manager storing rendered resources.
	*/
	bool UpdateCustomViewportManager(const ADisplayClusterRootActor* InSrcRootActor);

	//~ UActorComponent interface begin
	virtual void OnRegister() override;
	//~ UActorComponent interface end

public:
	/** Current sharing mode of this component */
	UPROPERTY(EditAnywhere, Setter=SetMode, BlueprintSetter=SetMode, Category=Sharing)
	EDisplayClusterPreviewShareMode Mode = EDisplayClusterPreviewShareMode::None;

	/** Current unique name of this component, which should match between sender and receiver of viewport textures */
	UPROPERTY(EditAnywhere, Setter = SetUniqueName, BlueprintSetter = SetUniqueName, Category = Sharing, meta = (EditCondition = "Mode == EDisplayClusterPreviewShareMode::Send || Mode == EDisplayClusterPreviewShareMode::Receive"))
	FString UniqueName;

	/** The source nDisplay actor to pull the preview from */
	UPROPERTY(EditAnywhere, meta = (DisplayName = "Source nDisplay Actor"), Category = Sharing, meta = (EditCondition = "Mode == EDisplayClusterPreviewShareMode::PullActor"))
	TSoftObjectPtr<ADisplayClusterRootActor> SourceNDisplayActor;

	/** Type of Icvfx camera sync to be performed between the Source nDisplay actor and the owner of this component */
	UPROPERTY(EditAnywhere, Category = Sharing, meta = (EditCondition = "Mode == EDisplayClusterPreviewShareMode::PullActor"))
	EDisplayClusterPreviewShareIcvfxSync IcvfxCamerasSyncType = EDisplayClusterPreviewShareIcvfxSync::PullActor;

private:
	// DC ViewportManager instance for this component
	// Used to perform rendering with custom settings from an external DCRA
	TSharedPtr<IDisplayClusterViewportManager, ESPMode::ThreadSafe> CustomViewportManager;

	/** Disabling all special preview renderers and reverting to the default view. */
	void RestoreDefaultPreviewSettings();

	/** Closes all media related objects (i.e. media captures and media players) */
	void CloseAllMedia();

	/** True if this component is valid to be actively used for sharing. E.g. CDOs are not considered active. */
	bool AllowedToShare() const;

	/** Called when the sharing mode was changed, so that it can update its internal state accordingly */
	void ModeChanged();

	/** Generates a string id for the viewport, used as a key to store data about it */
	FString GenerateViewportKey(const FString& ActorName, const FString& UniqueViewportName) const;

	/** Generates a Unique Name for the media being shared */
	FString GenerateMediaUniqueName(const FString& NodeName, const FString& ViewportName) const;

	/** Logic that should run every tick when in Send mode */
	void TickSend();

	/** Logic that should run every tick when in Receive mode */
	void TickReceive();

	/** Logic that should run every tick when in Pull Actor mode
	* 
	* @param bUseSourceActorSettings - Get preview settings from Source nDisplay Actor
	*/
	void TickPullActor(const bool bUseSourceActorSettings);

	/** Syncs the Icvfx cameras from the given source nDisplay actor to the given destination nDisplay actor */
	void SyncIcvxCamerasFromSourceActor(const ADisplayClusterRootActor* SrcRootActor, const ADisplayClusterRootActor* DstRootActor);

	/** Restores settings in the nDisplay actor that were altered by this component to achieve its intended purpose */
	void RestoreRootActorOriginalSettings();

	/** Enables/Disables component ticking */
	void SetTickEnable(const bool bEnable);

#if WITH_EDITOR
	/** Called when the editor map is changed. Used to remove unwanted references to external maps. */
	void HandleMapChanged_Editor(UWorld* InWorld, EMapChangeType InMapChangeType);
#endif

	/** Unsubcribe from preview of all root actors */
	UE_DEPRECATED(5.4, "This function has been deprecated.")
	void UnsubscribeFromAllPreviews() { };

	/** Subscribe for preview of given root actor */
	UE_DEPRECATED(5.4, "This function has been deprecated.")
	void SubscribeToPreview(ADisplayClusterRootActor* RootActor) { };

	/** Unsubscribe from preview of given root actor */
	UE_DEPRECATED(5.4, "This function has been deprecated.")
	void UnsubscribeFromPreview(ADisplayClusterRootActor* RootActor) { };

private:
	/** Media Outputs associated with the given viewport unique names */
	UPROPERTY(Transient)
	TMap<FString, TObjectPtr<UMediaOutput>> MediaOutputs;

	/** Media Captures associated with the given viewport unique names */
	UPROPERTY(Transient)
	TMap<FString, TObjectPtr<UMediaCapture>> MediaCaptures;

	/** Media Sources associated with the given viewport unique names */
	UPROPERTY(Transient)
	TMap<FString, TObjectPtr<UMediaSource>> MediaSources;

	/** Media Players associated with the given viewport unique names */
	UPROPERTY(Transient)
	TMap<FString, TObjectPtr<UMediaPlayer>> MediaPlayers;

	/** Media Textures associated with the given viewport unique names */
	UPROPERTY(Transient)
	TMap<FString, TObjectPtr<UMediaTexture>> MediaTextures;

	/** Cache of original Texture Replace Source Textures associated with the given viewport unique names. Used when restoring the original state */
	UPROPERTY(Transient)
	TMap<FString, TObjectPtr<UTexture>> OriginalSourceTextures;

	/** Cache of original Texture Replace enable boolean associated with the given viewport unique names. Used when restoring the original state */
	UPROPERTY(Transient)
	TMap<FString, bool> OriginalTextureReplaces;
};
