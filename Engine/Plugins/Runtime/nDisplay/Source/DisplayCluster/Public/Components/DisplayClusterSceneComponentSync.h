// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/SceneComponent.h"
#include "Cluster/IDisplayClusterClusterSyncObject.h"
#include "DisplayClusterSceneComponentSync.generated.h"


/**
 * Abstract synchronization component
 */
UCLASS(Abstract)
class DISPLAYCLUSTER_API UDisplayClusterSceneComponentSync
	: public USceneComponent
	, public IDisplayClusterClusterSyncObject
{
	GENERATED_BODY()

public:

	//~ Begin IDisplayClusterClusterSyncObject

	virtual bool IsActive() const override;
	
	virtual FString GetSyncId() const override
	{
		return SyncId;
	}
	
	virtual bool IsDirty() const override
	{
		return true;
	}

	//~ End IDisplayClusterClusterSyncObject

public:

	//~ Begin IDisplayClusterSerializable
	virtual void SerializeDC(FArchive& Ar) override;
	//~ End IDisplayClusterSerializable

public:

	//~ Begin USceneComponent
	virtual void BeginPlay() override;
	virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
	//~ End USceneComponent

protected:

	// Generates sync ID of this instance
	virtual FString GenerateSyncId();

	// Lets children provide their transforms for synchronization
	virtual FTransform GetSyncTransform() const
	{
		return FTransform();
	}

	// Lets children update thir transforms with new data
	virtual void SetSyncTransform(const FTransform& t)
	{ }

protected:

	// Caching state
	FVector  LastSyncLoc;
	FRotator LastSyncRot;
	FVector  LastSyncScale;

private:

	/** Unique synchronization ID of this instance */
	FString SyncId;
};
