// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DisplayClusterSceneComponentSync.h"
#include "DisplayClusterSceneComponentSyncParent.generated.h"


/**
 * Synchronization component. Synchronizes parent scene component.
 */
UCLASS(ClassGroup = (DisplayCluster), Blueprintable, meta = (BlueprintSpawnableComponent, DisplayName = "NDisplay Sync Parent"))
class DISPLAYCLUSTER_API UDisplayClusterSceneComponentSyncParent
	: public UDisplayClusterSceneComponentSync
{
	GENERATED_BODY()

public:
	UDisplayClusterSceneComponentSyncParent(const FObjectInitializer& ObjectInitializer);

public:
	//////////////////////////////////////////////////////////////////////////////////////////////
	// IDisplayClusterClusterSyncObject
	//////////////////////////////////////////////////////////////////////////////////////////////
	virtual bool IsDirty() const override;
	virtual void ClearDirty() override;

protected:
	//////////////////////////////////////////////////////////////////////////////////////////////
	// UDisplayClusterSceneComponentSync
	//////////////////////////////////////////////////////////////////////////////////////////////
	virtual FString GenerateSyncId() override;
	virtual FTransform GetSyncTransform() const override;
	virtual void SetSyncTransform(const FTransform& t) override;
};
