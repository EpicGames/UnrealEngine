// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DisplayClusterSceneComponentSync.h"
#include "DisplayClusterSceneComponentSyncThis.generated.h"


/**
 * Synchronization component. Synchronizes itself
 */
UCLASS(ClassGroup = (DisplayCluster), Blueprintable, meta = (BlueprintSpawnableComponent, DisplayName = "NDisplay Sync This"))
class DISPLAYCLUSTER_API UDisplayClusterSceneComponentSyncThis
	: public UDisplayClusterSceneComponentSync
{
	GENERATED_BODY()

public:
	UDisplayClusterSceneComponentSyncThis(const FObjectInitializer& ObjectInitializer);

public:
	//////////////////////////////////////////////////////////////////////////////////////////////
	// IDisplayClusterClusterSyncObject
	//////////////////////////////////////////////////////////////////////////////////////////////
	virtual bool IsDirty() const override;
	virtual void ClearDirty() override;

protected:
	//////////////////////////////////////////////////////////////////////////////////////////////
	// UDisplayClusterSceneComponentSync
	//////////////////////////////////////////////////////////////////////////////////////////////
	virtual FString GenerateSyncId() override;
	virtual FTransform GetSyncTransform() const override;
	virtual void SetSyncTransform(const FTransform& t) override;
};
