// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/StaticMeshComponent.h"

#include "DisplayClusterScreenComponent.generated.h"


/**
 * Simple projection policy screen component
 */
UCLASS(ClassGroup = (DisplayCluster), meta = (BlueprintSpawnableComponent, DisplayName = "NDisplay Screen"), HideCategories=(StaticMesh, Materials, ComponentTick, Physics, Collision, Lighting, Navigation, VirtualTexture, ComponentReplication, Cooking, LOD, MaterialParameters, HLOD, RayTracing, TextureStreaming, Mobile))
class DISPLAYCLUSTER_API UDisplayClusterScreenComponent
	: public UStaticMeshComponent
{
	GENERATED_BODY()

public:
	UDisplayClusterScreenComponent(const FObjectInitializer& ObjectInitializer);

public:
	/** Return the screen size adjusted by its transform scale. */
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Get screen size"), Category = "NDisplay|Components")
	FVector2D GetScreenSize() const;

	/** Set screen size (update transform scale). */
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Set screen size"), Category = "NDisplay|Components")
	void SetScreenSize(const FVector2D& Size);

	virtual void Serialize(FArchive& Ar) override;
	
#if WITH_EDITOR
protected:
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	/** Updates Size vector when scale is changed explicitly */
	void UpdateScreenSizeFromScale();
#endif

#if WITH_EDITORONLY_DATA
protected:
	friend class FDisplayClusterConfiguratorScreenDetailsCustomization;

	/** Adjust the size of the screen. */
	UPROPERTY(EditDefaultsOnly, Category = "Screen Size", meta = (DisplayName = "Size", AllowPreserveRatio))
	FVector2D Size;
#endif
};
