// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ActorComponent.h"

#include "DisplayClusterStageActorComponent.generated.h"

class ADisplayClusterRootActor;

/**
 * Stage Actor Component used to determine root actor ownership
 */
UCLASS(Abstract, ClassGroup = (DisplayCluster), meta = (DisplayName = "Stage Actor"), HideCategories=(Physics, Collision, Lighting, Navigation, Cooking, LOD, MaterialParameters, HLOD, RayTracing, TextureStreaming, Mobile))
class UDisplayClusterStageActorComponent : public UActorComponent
{
	GENERATED_BODY()

public:
	/** Set the Display Cluster Root Actor owner */
	DISPLAYCLUSTER_API virtual void SetRootActor(ADisplayClusterRootActor* InRootActor);

	/** Return the root actor owner of the light card */
	DISPLAYCLUSTER_API const TSoftObjectPtr<ADisplayClusterRootActor>& GetRootActor() const;
	
protected:
	/** Associate this content with the nDisplay configuration actor specified here.
	 * The content will only appear in the ICVFX Editor when this nDisplay configuration is selected. */
	UPROPERTY(EditInstanceOnly, DisplayName = "nDisplay Root Actor", Category = "NDisplay Root Actor")
	TSoftObjectPtr<ADisplayClusterRootActor> RootActor;
};