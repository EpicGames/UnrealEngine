// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SceneView.h"
#include "Components/ActorComponent.h"

#include "DisplayClusterStageGeometryComponent.generated.h"

class UProceduralMeshComponent;
class UTexture2D;
class UTextureRenderTarget2D;
class FDisplayClusterMeshProjectionRenderer;

/** A geometry map for the stage, which contains a generated texture of the stage geometry's depth and normal vectors */
USTRUCT()
struct FDisplayClusterStageGeometryMap
{
	GENERATED_BODY()

	/** The render target of the geometry map */
	UPROPERTY()
	TObjectPtr<UTextureRenderTarget2D> RenderTarget;

	/** The raw texture data of the geometry map, which stores the normal vector in the RGB components and the depth in the A component */
	TArray<FFloat16Color> GeometryData;

	/** The view matrices used to render the geometry map */
	FViewMatrices ViewMatrices;
};

/** A component that stores the generated geometry map of the stage actor, which is used for placing stage actors (light cards, CCWs, etc) flush to the stage's walls and ceiling */
UCLASS()
class DISPLAYCLUSTER_API UDisplayClusterStageGeometryComponent : public UActorComponent
{
	GENERATED_BODY()

private:
	/** The default size of the geometry map texture */
	static const uint32 GeometryMapSize;

	/** The field of view to render the geometry map with */
	static const float GeometryMapFOV;

public:
	UDisplayClusterStageGeometryComponent();

	//~ Begin UObject interface
	virtual void TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	//~ End UObject interface

	/** Gets whether the stage geometry map is valid and usable */
	UFUNCTION(BlueprintCallable, Category = "NDisplay")
	bool IsGeometryMapValid() const { return bGeometryMapLoaded; }

	/**
	 * Invalidates and regenerates the cached stage geometry map
	 * @param bForceImmediateRedraw Indicates whether the geometry map is regenerated immediately or is queued to redraw on the next tick
	 */
	UFUNCTION(BlueprintCallable, Category = "NDisplay")
	void Invalidate(bool bForceImmediateRedraw = false);

	/**
	 * Gets the distance and normal vector (in radial space) of the stage's geometry in the specified world direction
	 * @param InDirection The direction in world coordinates to query the stage geometry map
	 * @param OutDistance The distance from the stage's default view point to the nearest stage geometry in the specified direction
	 * @param OutNormal The normal vector in radial space (with x axis pointing in the direction of the stage's default view point) of the nearest stage geometry in the specified direction
	 * @return true if the stage geometry map was successfully queried, otherwise false
	 */
	UFUNCTION(BlueprintCallable, Category = "NDisplay")
	bool GetStageDistanceAndNormal(const FVector& InDirection, float& OutDistance, FVector& OutNormal);

	/**
	 * Gets the bounding radius of the stage's geometry
	 */
	UFUNCTION(BlueprintGetter, Category = "NDisplay")
	float GetStageBoundingRadius() const { return StageBoundingRadius; }

	/**
	 * Morphs the specified procedural mesh to match the stage's geometry map
	 * @param InProceduralMeshComponent - The mesh component to morph
	 * @param bSyncMeshLocation - When true, the procedural mesh's world location will be synced to the view point's world location
	 */
	bool MorphProceduralMesh(UProceduralMeshComponent* InProceduralMeshComponent, bool bSyncMeshLocation = false);

private:
	/** Gets the parent stage's common view point, or null if none were found */
	USceneComponent* GetCommonViewPoint() const;

	/** Creates a new render target that can be used to render the stage's geometry map */
	UTextureRenderTarget2D* CreateRenderTarget();

	/** Redraws the stage's geometry map */
	void RedrawGeometryMap();

	/** Updates the stage's current geometry with the renderer */
	void UpdateStageGeometry();

	/** Updates any isosphere that is found on the stage to match the geometry map */
	void UpdateStageIsosphere();

	/** Renders the stage's geometry to the specified geometry map */
	void GenerateGeometryMap(bool bIsNorthMap);

	/** Computes the scene view init options necessary to render the geometry map in the specified view direction */
	void GetSceneViewInitOptions(const FVector& ViewDirection, FSceneViewInitOptions& OutViewInitOptions);

private:
	/** The cached geometry map for the stage's northern hemisphere */
	UPROPERTY(Transient)
	FDisplayClusterStageGeometryMap NorthGeometryMap;

	/** The cached geometry map for the stage's southern hemisphere */
	UPROPERTY(Transient)
	FDisplayClusterStageGeometryMap SouthGeometryMap;

	/** The cached bounding radius of the stage's geometry */
	UPROPERTY(Transient)
	float StageBoundingRadius = 0.0f;

	/** The renderer used to generate the stage's geometry map */
	TSharedPtr<FDisplayClusterMeshProjectionRenderer> Renderer;

	/** Indicates that the geometry map should be updated on the next tick */
	bool bUpdateGeometryMap = false;

	/** Indicates that the resources for the geometry map have been loaded */
	bool bGeometryMapLoaded = false;
};