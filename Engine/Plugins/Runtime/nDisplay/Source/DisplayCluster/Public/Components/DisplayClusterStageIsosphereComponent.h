// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ProceduralMeshComponent.h"

#include "DisplayClusterStageIsosphereComponent.generated.h"

/** A procedural mesh component generated by morphing an isosphere to match the stage's geometry map, providing a 3D version of the map that can be ray traced against */
UCLASS()
class DISPLAYCLUSTER_API UDisplayClusterStageIsosphereComponent : public UProceduralMeshComponent
{
	GENERATED_BODY()

public:
	UDisplayClusterStageIsosphereComponent(const FObjectInitializer& ObjectInitializer);

	virtual void OnComponentCreated() override;

	/**
	 * Resets the vertices of the mesh component to be an isosphere
	 */
	void ResetIsosphere();

private:
	/** Checks that the isosphere static mesh used when generating the morphed geometry is loaded and has bAllowCPUAccess set to true */
	bool IsIsosphereMeshValid() const;

private:
	/** A static mesh containing an isosphere that can be morphed */
	UPROPERTY(Transient)
	TObjectPtr<UStaticMesh> IsosphereMesh;
};