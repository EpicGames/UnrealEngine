// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ActorComponent.h"
#include "DisplayClusterSyncTickComponent.generated.h"


/**
 * Helper component to trigger nDisplay sync for Tick sync group
 */
UCLASS(ClassGroup = (DisplayCluster), meta = (BlueprintSpawnableComponent, DisplayName = "NDisplay Sync Tick"))
class DISPLAYCLUSTER_API UDisplayClusterSyncTickComponent
	: public UActorComponent
{
	GENERATED_BODY()

public:
	UDisplayClusterSyncTickComponent(const FObjectInitializer& ObjectInitializer);

protected:
	virtual void TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
};
