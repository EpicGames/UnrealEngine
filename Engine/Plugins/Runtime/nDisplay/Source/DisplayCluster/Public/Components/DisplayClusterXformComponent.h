// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/SceneComponent.h"
#include "Components/IDisplayClusterComponent.h"

#include "DisplayClusterXformComponent.generated.h"

class UStaticMesh;
class UStaticMeshComponent;


/**
 * nDisplay Transform component
 */
UCLASS(ClassGroup = (DisplayCluster), meta = (BlueprintSpawnableComponent, DisplayName = "NDisplay Transform"))
class DISPLAYCLUSTER_API UDisplayClusterXformComponent
	: public USceneComponent
	, public IDisplayClusterComponent
{
	GENERATED_BODY()

public:
	UDisplayClusterXformComponent(const FObjectInitializer& ObjectInitializer);

public:
#if WITH_EDITOR
	// Begin IDisplayClusterComponent
	virtual void SetVisualizationScale(float Scale) override;
	virtual void SetVisualizationEnabled(bool bEnabled) override;
	// End IDisplayClusterComponent
#endif

	// Begin UActorComponent
	virtual void OnRegister() override;
	// End UActorComponent

	// Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	// End UObject

protected:
#if WITH_EDITOR
	/** Refreshes the visual components to match the component state */
	virtual void RefreshVisualRepresentation();
#endif

#if WITH_EDITORONLY_DATA
protected:
	/** Gizmo visibility */
	UPROPERTY(EditAnywhere, Category = "Gizmo")
	uint8 bEnableGizmo : 1;

	/** Base gizmo scale */
	UPROPERTY(EditAnywhere, Category = "Gizmo")
	FVector BaseGizmoScale;

	/** Gizmo scale multiplier */
	UPROPERTY(EditAnywhere, Category = "Gizmo", meta = (UIMin = "0", UIMax = "2.0", ClampMin = "0.01", ClampMax = "10.0"))
	float GizmoScaleMultiplier;

	/** Proxy mesh to render */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Gizmo")
	TObjectPtr<UStaticMesh> ProxyMesh;

	/** Proxy mesh component */
	UPROPERTY(Transient)
	TObjectPtr<UStaticMeshComponent> ProxyMeshComponent;
#endif
};
