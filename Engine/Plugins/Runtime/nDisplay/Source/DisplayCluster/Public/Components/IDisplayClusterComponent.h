// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"

#include "IDisplayClusterComponent.generated.h"


UINTERFACE(MinimalAPI, meta = (CannotImplementInterfaceInBlueprint))
class UDisplayClusterComponent : public UInterface
{
	GENERATED_BODY()
};


class IDisplayClusterComponent
{
	GENERATED_BODY()

#if WITH_EDITOR
public:
	/** Changes scale of visualization data (custom viz components, gizmo, etc.) */
	virtual void SetVisualizationScale(float Scale)
	{ }

	/** Activates or deactivates component visualization */
	virtual void SetVisualizationEnabled(bool bEnabled)
	{ }
#endif
};
