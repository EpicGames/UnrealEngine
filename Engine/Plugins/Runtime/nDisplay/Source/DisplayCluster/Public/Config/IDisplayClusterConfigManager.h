// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class  UDisplayClusterConfigurationData;
class  UDisplayClusterConfigurationClusterNode;
class  UDisplayClusterConfigurationViewport;
class  FString;
struct FDisplayClusterConfigurationPostprocess;
struct FDisplayClusterConfigurationProjection;

/**
 * Public config manager interface
 */
class IDisplayClusterConfigManager
{
public:
	virtual ~IDisplayClusterConfigManager() = default;

public:
	// Returns current config data
	virtual UDisplayClusterConfigurationData* GetConfig() const = 0;

public:
	// Returns path of the config file that is currently used
	virtual FString GetConfigPath() const = 0;

	// Returns ID of cluster node that is assigned to this application instance
	virtual FString GetLocalNodeId() const = 0;
	// Returns primary node ID
	virtual FString GetPrimaryNodeId() const = 0;

	// Returns primary node configuration data
	virtual const UDisplayClusterConfigurationClusterNode* GetPrimaryNode() const = 0;
	// Returns configuration data for cluster node that is assigned to this application instance
	virtual const UDisplayClusterConfigurationClusterNode* GetLocalNode() const = 0;
	// Returns configuration data for a specified local viewport 
	virtual const UDisplayClusterConfigurationViewport*    GetLocalViewport(const FString& ViewportId) const = 0;
	// Returns configuration data for a specified local postprocess operation
	virtual bool  GetLocalPostprocess(const FString& PostprocessId, FDisplayClusterConfigurationPostprocess& OutPostprocess) const = 0;
	// Returns configuration data for local projection policy
	virtual bool  GetLocalProjection(const FString& ViewportId, FDisplayClusterConfigurationProjection& OutProjection) const = 0;
};
