// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DisplayClusterLightCardActor.h"

#include "DisplayClusterChromakeyCardActor.generated.h"

class UDisplayClusterICVFXCameraComponent;

UCLASS(Blueprintable, NotPlaceable, DisplayName = "Chromakey Card", HideCategories = (Tick, Physics, Collision, Networking, Replication, Cooking, Input, Actor, HLOD))
class DISPLAYCLUSTER_API ADisplayClusterChromakeyCardActor : public ADisplayClusterLightCardActor
{
	GENERATED_BODY()
public:
	ADisplayClusterChromakeyCardActor(const FObjectInitializer& ObjectInitializer);
	virtual ~ADisplayClusterChromakeyCardActor() override;

	virtual void Tick(float DeltaSeconds) override;

#if WITH_EDITOR
	// ~Begin UObject interface
	virtual void PostEditUndo() override;
	// ~End UObject interface
#endif

	// ~Begin ADisplayClusterLightCardActor interface
	virtual void AddToRootActor(ADisplayClusterRootActor* InRootActor) override;
	virtual void RemoveFromRootActor() override;
	// ~End ADisplayClusterLightCardActor interface
	
	/**
	 * Checks if the given ICVFX camera has chroma key settings supporting this actor.
	 * 
	 * @param InCamera The ICVFX camera component to compare against.
	 *
	 * @return true if the camera component references this actor.
	 */
	bool IsReferencedByICVFXCamera(const UDisplayClusterICVFXCameraComponent* InCamera) const;
protected:
	void UpdateChromakeySettings();

#if WITH_EDITOR
private:
	/** Called when any UObject has its property changed. */
	void OnObjectPropertyChanged(UObject* InObject, FPropertyChangedEvent& InPropertyChangedEvent);

private:
	/** Handle for UObject property change. */
	FDelegateHandle PropertyChangedHandle;
#endif
};
