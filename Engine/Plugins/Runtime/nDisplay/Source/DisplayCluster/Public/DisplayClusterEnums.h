// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"

#include "DisplayClusterEnums.generated.h"


/**
 * Display cluster operation mode
 */
UENUM(BlueprintType)
enum class EDisplayClusterOperationMode : uint8
{
	Cluster = 0,
	Editor,
	Disabled
};

/**
 * Display cluster node role
 */
UENUM(BlueprintType)
enum class EDisplayClusterNodeRole : uint8
{
	None = 0,
	Primary,
	Secondary,
	Backup,
};


/**
 * Display cluster synchronization groups
 */
UENUM(BlueprintType)
enum class EDisplayClusterSyncGroup : uint8
{
	PreTick = 0,
	Tick,
	PostTick
};

/**
 * UDisplayClusterGameEngine running state.
 */
enum class EDisplayClusterRunningMode : uint8
{
	Startup = 0,
	Synced,
	WaitingForSync,
};
