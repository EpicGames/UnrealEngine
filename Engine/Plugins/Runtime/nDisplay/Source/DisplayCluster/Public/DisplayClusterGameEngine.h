// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/GameEngine.h"

#include "DisplayClusterEnums.h"
#include "DisplayClusterConfigurationTypes.h"

#include "Cluster/DisplayClusterClusterEvent.h"
#include "DisplayClusterGameEngine.generated.h"


class IDisplayClusterClusterNodeController;
class IDisplayClusterClusterSyncObject;
class UDisplayClusterConfigurationData;

DECLARE_DELEGATE_RetVal_ThreeParams(EBrowseReturnVal::Type, FOnBrowseLoadMap, FWorldContext& WorldContext, FURL URL, FString& Error);
DECLARE_DELEGATE_TwoParams(FOnPengineLevelUpdate, FWorldContext& Context, float DeltaSeconds);

/**
 * Extended game engine
 */
UCLASS()
class DISPLAYCLUSTER_API UDisplayClusterGameEngine
	: public UGameEngine
{
	GENERATED_BODY()

	TObjectPtr<UObject> WorldContextObject = nullptr;

public:
	virtual void Init(class IEngineLoop* InEngineLoop) override;
	virtual void PreExit() override;
	virtual void Tick(float DeltaSeconds, bool bIdleMode) override;
	virtual bool LoadMap(FWorldContext& WorldContext, FURL URL, class UPendingNetGame* Pending, FString& Error) override;

public:
	EDisplayClusterOperationMode GetOperationMode() const
	{
		return OperationMode;
	}

	/** Resets forced idle mode flag needed for multiplayer connections to tick the engine side to process handshake
	 * Used by DisplayClusterNetDriver when all nodes are connected
	 */
	void ResetForcedIdleMode()
	{
		bForcedTickIdleMode = false;
	}

	/** This override control either we want to trigger the rendering or not on the frame
	 * By modifying variable state in BrowseLoadMap when multiplayer connections are being established to prevent Redraw on the client side
	 */
	virtual bool IsRenderingSuspended() const override
	{
		return bIsRenderingSuspended;
	}

protected:
	virtual void UpdateTimeAndHandleMaxTickRate() override;

protected:
	virtual bool InitializeInternals();
	EDisplayClusterOperationMode DetectOperationMode() const;
	bool GetResolvedNodeId(const UDisplayClusterConfigurationData* ConfigData, FString& NodeId) const;
	bool ValidateConfigFile(const FString& FilePath);
	void PreExitImpl();

private:
	bool OutOfSync() const;
	void ReceivedSync(const FString &Level, const FString &NodeId);

	void CheckGameStartBarrier();
	bool CanTick() const;

	bool BarrierAvoidanceOn() const;

	void GameSyncChange(const FDisplayClusterClusterEventJson& InEvent);
	
	/** Implements custom loading logic to support listening server
	 */
	EBrowseReturnVal::Type BrowseLoadMap(FWorldContext& WorldContext, FURL URL, FString& Error);

	/** Custom logic to update PendingLevel in order to support listening server
	 */
	void PendingLevelUpdate(FWorldContext& Context, float DeltaSeconds);

	/** Sets up CVars */
	void ModifyCVars();

private:

	FDisplayClusterConfigurationDiagnostics Diagnostics;

	EDisplayClusterOperationMode OperationMode = EDisplayClusterOperationMode::Disabled;
	EDisplayClusterRunningMode   RunningMode = EDisplayClusterRunningMode::Startup;

	TMap<FString, TSet<FString>> SyncMap;

	// Flag used to force nodes to skip frame rendering
	// This is needed when multiplayer connections being established in the cluster,
	// system have to wait for all nodes to be connected before proceeding with rendering otherwise cluster will deadlock itself between GT/RT barriers
	// Enforced automatically into true on BrowseLoadMap for multiplayer connections
	bool bForcedTickIdleMode = false;

	// Flag used to suspend rendering and avoid Redraw method that leads to barrier lock among multiplayer client nodes, used in MP mode only
	bool bIsRenderingSuspended = false;
};
