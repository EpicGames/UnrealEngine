// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DisplayClusterLabelConfiguration.h"
#include "StageActor/IDisplayClusterStageActor.h"
#include "DisplayClusterConfigurationTypes_Enums.h"

#include "GameFramework/Actor.h"

#include "DisplayClusterLightCardActor.generated.h"

enum class EDisplayClusterLabelFlags : uint8;
class ADisplayClusterRootActor;
class UActorComponent;
class UDisplayClusterLabelComponent;
class UDisplayClusterStageActorComponent;
class UMeshComponent;
class USceneComponent;
class USpringArmComponent;
class UStaticMeshComponent;
class UStaticMesh;
class UTexture;

UENUM(BlueprintType)
enum class EDisplayClusterLightCardMask : uint8
{
	Circle,
	Square,
	UseTextureAlpha,
	Polygon,
	MAX	UMETA(Hidden)
};

USTRUCT(Blueprintable)
struct FLightCardAlphaGradientSettings
{
	GENERATED_BODY()

	/** Enables/disables alpha gradient effect */
	UPROPERTY(EditAnywhere, Interp, BlueprintReadWrite, Category = "Appearance")
	bool bEnableAlphaGradient = false;

	/** Starting alpha value in the gradient */
	UPROPERTY(EditAnywhere, Interp, BlueprintReadWrite, Category = "Appearance")
	float StartingAlpha = 0;

	/** Ending alpha value in the gradient */
	UPROPERTY(EditAnywhere, Interp, BlueprintReadWrite, Category = "Appearance")
	float EndingAlpha = 1;

	/** The angle (degrees) determines the gradient direction. */
	UPROPERTY(EditAnywhere, Interp, BlueprintReadWrite, Category = "Appearance")
	float Angle = 0;
};

UCLASS(Blueprintable, DisplayName = "Light Card", HideCategories = (Tick, Physics, Collision, Networking, Replication, Cooking, Input, Actor, HLOD))
class DISPLAYCLUSTER_API ADisplayClusterLightCardActor : public AActor, public IDisplayClusterStageActor
{
	GENERATED_BODY()

public:
	/** The default size of the projection plane UV light cards are rendered to */
	static const float UVPlaneDefaultSize;

	/** The default distance from the view of the projection plane UV light cards are rendered to */
	static const float UVPlaneDefaultDistance;

	/** The name used for the stage actor component */
	static const FName LightCardStageActorComponentName;
	
public:
	ADisplayClusterLightCardActor(const FObjectInitializer& ObjectInitializer);
	virtual ~ADisplayClusterLightCardActor() override;

	virtual void PostLoad() override;
	virtual void OnConstruction(const FTransform& Transform) override;
	virtual void Tick(float DeltaSeconds) override;
	virtual void Destroyed() override;

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual bool ShouldTickIfViewportsOnly() const override { return true; }
	virtual FName GetCustomIconName() const override;
#endif

	/** Gets the light card mesh components */
	void GetLightCardMeshComponents(TArray<UMeshComponent*>& MeshComponents) const;

	/** Returns the current static mesh used by this light card */
	UStaticMesh* GetStaticMesh() const;

	/** Return the label component */
	UDisplayClusterLabelComponent* GetLabelComponent() const { return LabelComponent; }
	
	/** Sets a new static mesh for the light card */
	void SetStaticMesh(UStaticMesh* InStaticMesh);

	/** Updates the card's material instance parameters */
	void UpdateLightCardMaterialInstance();

	/** Updates the polygon texture from the polygon points */
	void UpdatePolygonTexture();

	/** Updates the light card actor visibility */
	void UpdateLightCardVisibility();

	/** Updates the UV Indicator */
	void UpdateUVIndicator();

	/** Sync the position to the root actor */
	void UpdateLightCardPositionToRootActor();

	/** Makes the light card flush to the wall */
	void MakeFlushToWall();

	/** Configures this light card as a flag */
	void SetIsLightCardFlag(bool bNewFlagValue);

	/** If this light card is considered a flag */
	bool IsLightCardFlag() const { return bIsLightCardFlag; }

	/** Configures this light card as a UV actor */
	void SetIsUVActor(bool bNewUVValue);

	/** Configure light card labels */
	void ShowLightCardLabel(const FDisplayClusterLabelConfiguration& InLabelConfiguration);
	
	/** Show or hide the light card label  */
	UE_DEPRECATED(5.3, "Use the ShowLightCardLabel function which accepts FDisplayClusterLabelConfiguration arguments.")
	void ShowLightCardLabel(bool bValue, float ScaleValue, ADisplayClusterRootActor* InRootActor);

	/** Set a weak root actor owner. This should be used on legacy light cards that don't have StageActorComponent->RootActor set */
	void SetWeakRootActorOwner(ADisplayClusterRootActor* InRootActor);

	/** Set the root actor of the stage actor component */
	void SetRootActorOwner(ADisplayClusterRootActor* InRootActor);
	
	/** Return the root actor owner of the light card */
	ADisplayClusterRootActor* GetRootActorOwner() const;

	/** Retrieve the stage actor component */
	UDisplayClusterStageActorComponent* GetStageActorComponent() const;

	/** Add this light card to a light card layer on the given root actor */
	UE_DEPRECATED(5.2, "Layers are no longer used for light cards by default, use 'AddToRootActor' instead")
	void AddToLightCardLayer(ADisplayClusterRootActor* InRootActor);

	/** Add this light card to a root actor */
	virtual void AddToRootActor(ADisplayClusterRootActor* InRootActor);

	/** Remove this light card from the root actor */
	virtual void RemoveFromRootActor();
	
	// ~Begin IDisplayClusterStageActor interface
	/** Updates the Light Card transform based on its positional properties (Lat, Long, etc.) */
	virtual void UpdateStageActorTransform() override;

	/**
	 * Gets the transform in world space of the light card component
	 */
	virtual FTransform GetStageActorTransform(bool bRemoveOrigin = false) const override;

	/** Gets the object oriented bounding box of the light card component */
	virtual FBox GetBoxBounds(bool bLocalSpace = false) const override;
	
	virtual void SetLongitude(double InValue) override;
	virtual double GetLongitude() const override;

	virtual void SetLatitude(double InValue) override;
	virtual double GetLatitude() const override;

	virtual void SetDistanceFromCenter(double InValue) override;
	virtual double GetDistanceFromCenter() const override;

	virtual void SetSpin(double InValue) override;
	virtual double GetSpin() const override;

	virtual void SetPitch(double InValue) override;
	virtual double GetPitch() const override;

	virtual void SetYaw(double InValue) override;
	virtual double GetYaw() const override;

	virtual void SetRadialOffset(double InValue) override;
	virtual double GetRadialOffset() const override;

	virtual bool IsUVActor() const override;

	virtual void SetOrigin(const FTransform& InOrigin) override;
	virtual FTransform GetOrigin() const override;

	virtual void SetScale(const FVector2D& InScale) override;
	virtual FVector2D GetScale() const override;

	virtual void SetAlwaysFlushToWall(bool bInAlwaysFlushToWall) override { bAlwaysFlushToWall = bInAlwaysFlushToWall; }
	virtual bool IsAlwaysFlushToWall() const override { return bAlwaysFlushToWall; }
	virtual void SetUVCoordinates(const FVector2D& InUVCoordinates) override;
	virtual FVector2D GetUVCoordinates() const override;

	virtual void GetPositionalProperties(FPositionalPropertyArray& OutPropertyPairs) const override;
	// ~End IDisplayClusterStageActor interface
	
public:

	/** Radius of light card polar coordinates. Does not include the effect of RadialOffset */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Orientation", meta = (EditCondition = "!bIsUVLightCard", HideEditConditionToggle, EditConditionHides))
	double DistanceFromCenter;

	/** Related to the Azimuth of light card polar coordinates */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Orientation", meta = (UIMin = 0, ClampMin = 0, UIMax = 360, ClampMax = 360, EditCondition = "!bIsUVLightCard", HideEditConditionToggle, EditConditionHides))
	double Longitude;

	/** Related to the Elevation of light card polar coordinates */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Orientation", meta = (UIMin = -90, ClampMin = -90, UIMax = 90, ClampMax = 90, EditCondition = "!bIsUVLightCard", HideEditConditionToggle, EditConditionHides))
	double Latitude;

	/** The UV coordinates of the light card, if it is in UV space */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Orientation", meta = (DisplayName = "UV Coodinates", EditCondition = "bIsUVLightCard", HideEditConditionToggle, EditConditionHides))
	FVector2D UVCoordinates = FVector2D(0.5, 0.5);

	/** Roll rotation of light card around its plane axis */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Orientation", meta = (UIMin = -360, ClampMin = -360, UIMax = 360, ClampMax = 360))
	double Spin;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Orientation", meta = (UIMin = -360, ClampMin = -360, UIMax = 360, ClampMax = 360, EditCondition = "!bIsUVLightCard", HideEditConditionToggle, EditConditionHides))
	double Pitch;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Orientation", meta = (UIMin = -360, ClampMin = -360, UIMax = 360, ClampMax = 360, EditCondition = "!bIsUVLightCard", HideEditConditionToggle, EditConditionHides))
	double Yaw;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Orientation")
	FVector2D Scale;

	/** Used by the flush constraint to offset the location of the light card form the wall */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Orientation", meta = (EditCondition = "!bIsUVLightCard", HideEditConditionToggle, EditConditionHides))
	double RadialOffset;

	/** Indicates whether the light card is always made to be flush to a stage wall or not */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Orientation", meta = (EditCondition = "!bIsUVLightCard", HideEditConditionToggle, EditConditionHides))
	bool bAlwaysFlushToWall;

	/** Specify how to render this Light Card Actor in relation to the inner frustum. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Appearance", meta = (DisplayName = "Blending Mode"))
	EDisplayClusterConfigurationICVFX_PerLightcardRenderMode PerLightcardRenderMode;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Appearance")
	EDisplayClusterLightCardMask Mask;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Appearance")
	TObjectPtr<UTexture> Texture;

	/** Light card color, before any modifier is applied */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Appearance")
	FLinearColor Color;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Appearance", meta = (UIMin = 0, ClampMin = 0, UIMax = 10000, ClampMax = 10000))
	float Temperature;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Appearance", meta = (UIMin = -1, ClampMin = -1, UIMax = 1, ClampMax = 1))
	float Tint;

	/** 2^Exposure color value multiplier */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Appearance", meta = (UIMin = -100, ClampMin = -100, UIMax = 100, ClampMax = 100))
	float Exposure;

	/** Linear color value multiplier */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Appearance", meta = (UIMin = 0, ClampMin = 0))
	float Gain;

	/** Linear alpha multiplier */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Appearance", meta = (UIMin = 0, ClampMin = 0, UIMax = 1, ClampMax = 1))
	float Opacity;

	/** Feathers in the alpha from the edges */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Appearance", meta = (UIMin = 0, ClampMin = 0))
	float Feathering;

	/** Settings related to an alpha gradient effect */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Appearance")
	FLightCardAlphaGradientSettings AlphaGradient;

	/** A flag that controls whether the light card's location and rotation are locked to its "owning" root actor */
	UPROPERTY()
	bool bLockToOwningRootActor = true;

	/** Indicates if the light card exists in 3D space or in UV space */
	UPROPERTY(Getter = IsUVActor, Setter = SetIsUVActor)
	bool bIsUVLightCard = false;

	/** Polygon points when using this type of mask */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Interp, Category = "Appearance")
	TArray<FVector2D> Polygon;

	/** Used to flag this light card as a proxy of a "real" light card. Used by the LightCard Editor */
	UPROPERTY(Transient)
	TObjectPtr<UTexture> PolygonMask = nullptr;

	static FName GetExtenderNameToComponentMapMemberName()
	{
		return GET_MEMBER_NAME_CHECKED(ADisplayClusterLightCardActor, ExtenderNameToComponentMap);
	}

protected:
	/** Creates components for IDisplayClusterLightCardActorExtender */
	void CreateComponentsForExtenders();

	/** Removes components that were added by IDisplayClusterLightCardActorExtender */
	void CleanUpComponentsForExtenders();

	/** Removes this actor from a root actor's ShowOnlyList.Actors, if possible. */
	void RemoveFromRootActorList(ADisplayClusterRootActor* RootActor);

#if WITH_EDITOR
	/** Called when a level actor is deleted */
	void OnLevelActorDeleted(AActor* DeletedActor);
#endif
	
protected:
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Default")
	TObjectPtr<USceneComponent> DefaultSceneRootComponent;

	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Default")
	TObjectPtr<USpringArmComponent> MainSpringArmComponent;

	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Default")
	TObjectPtr<USceneComponent> LightCardTransformerComponent;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Default")
	TObjectPtr<UStaticMeshComponent> LightCardComponent;

	/** Components added by the IDisplayLightCardActorExtender */
	UPROPERTY(VisibleAnywhere, Category = "Default")
	TMap<FName, TObjectPtr<UActorComponent>> ExtenderNameToComponentMap;

	UPROPERTY(Transient, BlueprintReadOnly, Category = "Default")
	TObjectPtr<UDisplayClusterLabelComponent> LabelComponent;

	/** Manages stage actor properties */
	UPROPERTY(VisibleAnywhere, Category = "Default")
	TObjectPtr<UDisplayClusterStageActorComponent> StageActorComponent;

#if WITH_EDITORONLY_DATA
	UPROPERTY(Transient, BlueprintReadOnly, Category = "Default")
	TObjectPtr<UStaticMeshComponent> UVIndicatorComponent;
#endif

	/** Indicates this light card should be considered a flag */
	UPROPERTY(Getter = IsLightCardFlag, Setter = SetIsLightCardFlag, meta = (AllowPrivateAccess = "true"))
	bool bIsLightCardFlag = false;

private:
	/** Set by DCRA during tick. Meant for legacy light cards that don't have a SoftObjectPtr set */
	TWeakObjectPtr<ADisplayClusterRootActor> WeakRootActorOwner;

	/** Stores the location relative to the root actor's origin that stage actors like light cards can orbit around */
	FVector LastOrbitLocation = FVector::ZeroVector;

#if WITH_EDITOR
	/** Set if there was conflicting root actor ownership */
	bool bHadRootActorMismatch = false;
#endif
};
