// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EnhancedPlayerInput.h"
#include "DisplayClusterPlayerInput.generated.h"


/**
 * An object within PlayerController that processes player input.
 *
 * This one is nDisplay specific implementation. It's responsible for replication
 * of UE native input within a cluster to support simulation determinism. Various
 * input sync policies might be implemented here.
 */
UCLASS(Within=PlayerController, config=Input, transient)
class DISPLAYCLUSTER_API UDisplayClusterPlayerInput : public UEnhancedPlayerInput
{
	GENERATED_BODY()

public:

	UDisplayClusterPlayerInput();

public:

	/** Process the frame's input events given the current input component stack. */
	virtual void ProcessInputStack(const TArray<UInputComponent*>& InputComponentStack, const float DeltaTime, const bool bGamePaused) override;

protected:

	/** Performs sync logic where all input data from the primary node is replicated to all other nodes */
	virtual void ProcessPolicy_ReplicatePrimary(const TArray<UInputComponent*>& InputComponentStack, const float DeltaTime, const bool bGamePaused);

protected:

	/** Auxiliary function that serializes whole input data into a byte stream */
	bool SerializeKeyStateMap(TMap<FString, TArray<uint8>>& OutKeyStateMap);

	/** Aux function that deserializes whole input data from a map */
	bool DeserializeKeyStateMap(const TMap<FString, TArray<uint8>>& InKeyStateMap);

private:

	// Currently we have this bool only wich provides 2 cases: sync or not sync. It covers all
	// functionality currently available . However, this might be refactored one day in order
	// to support multiple input sync policies including custom ones.
	bool bReplicatePrimary;
};
