// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "SceneTypes.h"

#include "GameFramework/Actor.h"
#include "Camera/PlayerCameraManager.h"

#include "Misc/DisplayClusterObjectRef.h"
#include "DisplayClusterEnums.h"

#include "SceneInterface.h"

#include "DisplayClusterConfigurationStrings.h"
#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterConfigurationTypes_Viewport.h"
#include "DisplayClusterConfigurationTypes_ICVFX.h"
#include "DisplayClusterConfigurationTypes_OCIO.h"
#include "DisplayClusterEditorPropertyReference.h"

#include "Render/Viewport/IDisplayClusterViewportManager.h"
#include "Render/Viewport/RenderFrame/DisplayClusterRenderFrame.h"

#include "DisplayClusterRootActor.generated.h"

#if WITH_EDITOR
class IDisplayClusterConfiguratorBlueprintEditor;
class FTransactionObjectEvent;
#endif

class FDisplayClusterViewportManager;
class IDisplayClusterStageActor;
class USceneComponent;
class UDisplayClusterDisplayDeviceBaseComponent;
class ULineBatchComponent;
class UDisplayClusterConfigurationData;
class UDisplayClusterCameraComponent;
class UDisplayClusterOriginComponent;
class UDisplayClusterStageGeometryComponent;
class UDisplayClusterStageIsosphereComponent;
class UDisplayClusterSyncTickComponent;
class UProceduralMeshComponent;
struct FDisplayClusterRootActorPropertyOverrides;

/**
 * VR root. This contains nDisplay VR hierarchy in the game.
 */
UCLASS(notplaceable, HideCategories=(Replication, Collision, Input, Actor, HLOD, Cooking, Physics, Activation, AssetUserData, ActorTick, Advanced, DataLayers, Events), meta = (DisplayName = "nDisplay Root Actor"))
class DISPLAYCLUSTER_API ADisplayClusterRootActor
	: public AActor
{
	friend class FDisplayClusterRootActorDetailsCustomization;

	GENERATED_BODY()

public:
	ADisplayClusterRootActor(const FObjectInitializer& ObjectInitializer);
	~ADisplayClusterRootActor();

public:
	/**
	 * Initialializes the instance with specified config data
	 *
	 * @param ConfigData - Configuration data
	 */
	void InitializeFromConfig(UDisplayClusterConfigurationData* ConfigData);

	/**
	 * Cherry picking settings from a specified config data
	 *
	 * @param ConfigData - Configuration data
	 */
	void OverrideFromConfig(UDisplayClusterConfigurationData* ConfigData);

	/**
	 * Update or create the config data object. The config sub object is only instantiated once.
	 * Subsequent calls will only update ConfigDataName unless bForceRecreate is true.
	 *
	 * @param ConfigDataTemplate The config template to use for this actors' config data object.
	 * @param bForceRecreate Deep copies properties from the config data template to this actors' config data object.
	 */
	void UpdateConfigDataInstance(UDisplayClusterConfigurationData* ConfigDataTemplate, bool bForceRecreate = false);

	/** Returns true if this RootActor is used as the primary RootActor in the DC GameManager. */
	bool IsPrimaryRootActor() const;

	/** Returns true if this RootActor is primary and should be displayed in PIE mode. */
	bool IsPrimaryRootActorForPIE() const;

	/** Returns true if this RootActor is running in PIE mode. */
	bool IsRunningPIE() const;

	/** Returns true if this RootActor is running in game or in PIE mode. */
	bool IsRunningGameOrPIE() const;

	/** Returns true if this RootActor is running in DC mode. */
	bool IsRunningDisplayCluster() const;

	/** Override properties of this root actor. */
	void OverrideRootActorProperties(const FDisplayClusterRootActorPropertyOverrides& InPropertyOverrides);

	UDisplayClusterConfigurationData* GetDefaultConfigDataFromAsset() const;
	UDisplayClusterConfigurationData* GetConfigData() const;

	// Return hidden in game privitives set
	bool GetHiddenInGamePrimitives(TSet<FPrimitiveComponentId>& OutPrimitives);

	/** Collect UActorComponent primitives by names.
	* Note: Components must have bHiddenInGame set to true.
	* 
	* @param InNames       - The names of the primitives to be searched for.
	* @param OutPrimitives - (out) An array with the primitives that were found.
	* @param bForceHide    - (opt) if true, collects components with any value of the bHiddenInGame property.
	* 
	* @return true if any primitives were found.
	*/
	bool FindPrimitivesByName(const TArray<FString>& InNames, TSet<FPrimitiveComponentId>& OutPrimitives, bool bForceHide = false);

	bool IsBlueprint() const;

	UDisplayClusterSyncTickComponent* GetSyncTickComponent() const
	{
		return SyncTickComponent;
	}

	const FDisplayClusterConfigurationICVFX_StageSettings& GetStageSettings() const;
	const FDisplayClusterConfigurationRenderFrame& GetRenderFrameSettings() const;

	/** The Tick() function that calls the preview renderer for this root actor. */
	void TickPreviewRenderer();

	/** Returns the current rendering mode of this DCRA (not a value from configuration).
	 * This value can be overridden from DCRenderDevice or other rendering subsystems (e.g. Preview).
	 */
	UE_DEPRECATED(5.4, "This function has been deprecated.")
	EDisplayClusterRenderFrameMode GetRenderMode() const
	{
		return EDisplayClusterRenderFrameMode::Unknown;
	}

	/** Returns the preview rendering mode of this DCRA. */
	UE_DEPRECATED(5.4, "This function has been deprecated.")
	EDisplayClusterRenderFrameMode GetPreviewRenderMode() const
	{
		return EDisplayClusterRenderFrameMode::Unknown;
	}

protected:
	//////////////////////////////////////////////////////////////////////////////////////////////
	// AActor
	//////////////////////////////////////////////////////////////////////////////////////////////
	virtual void BeginPlay() override;
	virtual void Tick(float DeltaSeconds) override;
	virtual void PostLoad() override;
	virtual void PostActorCreated() override;
	virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
	virtual void BeginDestroy() override;
	virtual void Destroyed() override;
#if WITH_EDITOR
	virtual void RerunConstructionScripts() override;
#endif

	// Initializes the actor on spawn and load
	void InitializeRootActor();

	// Creates all hierarchy objects declared in a config file
	bool BuildHierarchy();

	/** Determine which light card actors are owned by this root actor */
	void SetLightCardOwnership();

public:
	UFUNCTION(BlueprintCallable, Category = "NDisplay|DCRA")
	bool GetFlushPositionAndNormal(const FVector& WorldPosition, FVector& OutPosition, FVector& OutNormal);

	UFUNCTION(BlueprintCallable, Category = "NDisplay|DCRA")
	bool MakeStageActorFlushToWall(const TScriptInterface<IDisplayClusterStageActor>& StageActor, double DesiredOffsetFromFlush = 0.0f);

	/**
	 * Gets the distance from a world position to the stage's geometry along the specified direction, if there is an intersection
	 * @param WorldPosition - The world position to measure the distance from
	 * @param WorldDirection - The direction to find the distance to the geometry along
	 * @param OutDistance - The distance to the stage geometry from the specified point
	 * @return True if an intersection point from WorldPosition along WorldDirection was found, false if not
	 */
	UFUNCTION(BlueprintCallable, Category = "NDisplay|Stage")
	bool GetDistanceToStageGeometry(const FVector& WorldPosition, const FVector& WorldDirection, float& OutDistance) const;

	UFUNCTION(BlueprintGetter)
	UDisplayClusterStageGeometryComponent* GetStageGeometryComponent() const { return StageGeometryComponent; }

	UFUNCTION(BlueprintGetter)
	UDisplayClusterCameraComponent* GetDefaultCamera() const;

	/** Retrieve the default display device, creating it if it doesn't exist */
	UFUNCTION(BlueprintGetter)
	UDisplayClusterDisplayDeviceBaseComponent* GetDefaultDisplayDevice() const;

	/** Retrieve the line batch component. */
	ULineBatchComponent* GetLineBatchComponent() const;

	/**
	 * Get the view point most commonly used by viewports in this cluster.
	 * If no viewports override the camera, this returns the default camera, or if there isn't one, the actor's root component.
	 */
	UFUNCTION(BlueprintCallable, Category = "NDisplay|DCRA")
	USceneComponent* GetCommonViewPoint() const;

	UFUNCTION(BlueprintCallable, Category = "NDisplay|Render")
	bool SetReplaceTextureFlagForAllViewports(bool bReplace);

	UFUNCTION(BlueprintCallable, Category = "NDisplay|Render")
	bool SetFreezeOuterViewports(bool bEnable);

	template <typename TComp>
	TComp* GetComponentByName(const FString& ComponentName) const
	{
		static_assert(std::is_base_of<UActorComponent, TComp>::value, "TComp is not derived from UActorComponent");

		TArray<TComp*> FoundComponents;
		this->GetComponents<TComp>(FoundComponents, false);

		for (TComp* Component : FoundComponents)
		{
			// Search for the one that has a specified name
			if (Component->GetName().Equals(ComponentName, ESearchCase::IgnoreCase))
			{
				return Component;
			}
		}

		return nullptr;
	}

	/**
	* Update the geometry of the procedural mesh component(s) referenced inside nDisplay
	*
	* @param InProceduralMeshComponent - (optional) Mark the specified procedural mesh component, not all
	*/
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Update ProceduralMeshComponent data"), Category = "NDisplay|Components")
	void UpdateProceduralMeshComponentData(const UProceduralMeshComponent* InProceduralMeshComponent = nullptr);


	/**
	* Blueprint setter for the PreviewEnablePostProcess property. Makes sure preview pipeline is updated properly.
	*
	* @param bNewPreviewEnablePostProcess - Desired new property value.
	*/
	UFUNCTION(BlueprintSetter)
	void SetPreviewEnablePostProcess(const bool bNewPreviewEnablePostProcess);

	/**
	* Blueprint setter for the PreviewEnableHoldoutComposite property. Makes sure preview pipeline is updated properly.
	*
	* @param bNewbPreviewEnableHoldoutComposite - Desired new property value.
	*/
	UFUNCTION(BlueprintSetter)
	void SetPreviewEnableHoldoutComposite(const bool bNewPreviewEnableHoldoutComposite);

public:
	/** Get ViewportManager API. */
	IDisplayClusterViewportManager* GetViewportManager() const;

	/** Get or Create ViewportManager API. */
	IDisplayClusterViewportManager* GetOrCreateViewportManager();

	/** Release the viewport manager instance, if it exists. */
	void RemoveViewportManager();

	/** Get ViewportConfiguration API.*/
	IDisplayClusterViewportConfiguration* GetViewportConfiguration() const;
	
	static FName GetCurrentConfigDataMemberName()
	{
		return GET_MEMBER_NAME_CHECKED(ADisplayClusterRootActor, CurrentConfigData);
	}

private:
	/** Get ViewportManager API. */
	FDisplayClusterViewportManager* GetViewportManagerImpl() const;

	/** Reset preview rendering. */
	void ResetEntireClusterPreviewRendering();

private:
	// DC ViewportManager instance for this DCRA
	TSharedPtr<FDisplayClusterViewportManager, ESPMode::ThreadSafe> ViewportManagerPtr;

//////////////////////////////////////////////////////////////////////////////////////////////
// Details Panel Property Referencers
// Placed here to ensure layout builders process referencers first
//////////////////////////////////////////////////////////////////////////////////////////////
#if WITH_EDITORONLY_DATA
private:
	UPROPERTY(EditAnywhere, Transient, Category = Viewports, meta = (PropertyPath = "CurrentConfigData.StageSettings.OuterViewportUpscalerSettings", DisplayName = "Default Upscaler Settings"))
	FDisplayClusterEditorPropertyReference OuterViewportUpscalerSettingsRef;

	UPROPERTY(EditInstanceOnly, Transient, Category = Viewports, meta = (DisplayName = "Per-Viewport Upscaling Setting Overrides", PropertyPath = "CurrentConfigData.Cluster.Nodes.Viewports.RenderSettings.UpscalerSettings", ToolTip = "Adjust upscaler settings for an individual viewport."))
	FDisplayClusterEditorPropertyReference ViewportUpscalerSettingsRef;

	UPROPERTY(EditAnywhere, Transient, Category = Viewports, meta = (PropertyPath = "CurrentConfigData.RenderFrameSettings.ClusterICVFXOuterViewportBufferRatioMult"))
	FDisplayClusterEditorPropertyReference ViewportScreenPercentageMultiplierRef;

	UPROPERTY(EditInstanceOnly, Transient, Category = Viewports, meta = (DisplayName = "Viewport Screen Percentage", PropertyPath = "CurrentConfigData.Cluster.Nodes.Viewports.RenderSettings.BufferRatio", ToolTip = "Adjust resolution scaling for an individual viewport.  Viewport Screen Percentage Multiplier is applied to this value."))
	FDisplayClusterEditorPropertyReference ViewportScreenPercentageRef;

	UPROPERTY(EditInstanceOnly, Transient, Category = Viewports, meta = (DisplayName = "Viewport Overscan", PropertyPath = "CurrentConfigData.Cluster.Nodes.Viewports.RenderSettings.Overscan", ToolTip = "Render a larger frame than specified in the configuration to achieve continuity across displays when using post-processing effects."))
	FDisplayClusterEditorPropertyReference ViewportOverscanRef;

	UPROPERTY(EditAnywhere, Transient, Category = Viewports, meta = (PropertyPath = "CurrentConfigData.StageSettings.bFreezeRenderOuterViewports"))
	FDisplayClusterEditorPropertyReference FreezeRenderOuterViewportsRef;

	UPROPERTY(EditAnywhere, Transient, Category = Viewports, meta = (PropertyPath = "CurrentConfigData.StageSettings.HideList"))
	FDisplayClusterEditorPropertyReference ClusterHideListRef;

	UPROPERTY(EditAnywhere, Transient, Category = Viewports, meta = (PropertyPath = "CurrentConfigData.StageSettings.OuterViewportHideList"))
	FDisplayClusterEditorPropertyReference OuterHideListRef;

	UPROPERTY(EditInstanceOnly, Transient, Category = Viewports, meta = (DisplayName = "Content Hidden Per-Viewport", PropertyPath = "CurrentConfigData.Cluster.Nodes.Viewports.RenderSettings.HiddenContent", ToolTip = "Hide actors from a specific viewport."))
	FDisplayClusterEditorPropertyReference PerOuterHideListRef;
	
	UPROPERTY(EditAnywhere, Transient, Category = "In Camera VFX", meta = (PropertyPath = "CurrentConfigData.StageSettings.GlobalInnerFrustumUpscalerSettings", DisplayName = "Default Upscaler Settings"))
	FDisplayClusterEditorPropertyReference GlobalInnerFrustumUpscalerSettingsRef;

	UPROPERTY(EditAnywhere, Transient, Category = "In Camera VFX", meta = (PropertyPath = "CurrentConfigData.StageSettings.bEnableInnerFrustums"))
	FDisplayClusterEditorPropertyReference EnableInnerFrustumsRef;

	UPROPERTY(EditAnywhere, Transient, Category = "In Camera VFX", meta = (PropertyPath = "CurrentConfigData.StageSettings.bEnableInnerFrustumChromakeyOverlap", DisplayName = "Enable Chromakey Overlap"))
	FDisplayClusterEditorPropertyReference ShowInnerFrustumOverlapsRef;

	UPROPERTY(EditAnywhere, Transient, Category = "Color Grading", meta = (PropertyPath = "CurrentConfigData.StageSettings.EnableColorGrading"))
	FDisplayClusterEditorPropertyReference EnableInnerFrustumChromakeyOverlapRef;

	UPROPERTY(EditAnywhere, Transient, Category = "Color Grading", meta = (PropertyPath = "CurrentConfigData.StageSettings.EntireClusterColorGrading"))
	FDisplayClusterEditorPropertyReference ClusterColorGradingRef;

	UPROPERTY(EditAnywhere, Transient, Category = "Color Grading", meta = (PropertyPath = "CurrentConfigData.StageSettings.PerViewportColorGrading"))
	FDisplayClusterEditorPropertyReference PerViewportColorGradingRef;

	UPROPERTY(EditAnywhere, Transient, Category = OCIO, meta = (PropertyPath = "CurrentConfigData.StageSettings.ViewportOCIO.AllViewportsOCIOConfiguration.bIsEnabled", DisplayName = "Enable Viewport OCIO"))
	FDisplayClusterEditorPropertyReference EnableViewportOCIORef;

	UPROPERTY(EditAnywhere, Transient, Category = OCIO, meta = (PropertyPath = "CurrentConfigData.StageSettings.ViewportOCIO.AllViewportsOCIOConfiguration.ColorConfiguration", DisplayName = "All Viewports Color Configuration"))
	FDisplayClusterEditorPropertyReference AllViewportColorConfigurationRef;

	UPROPERTY(EditAnywhere, Transient, Category = OCIO, meta = (PropertyPath = "CurrentConfigData.StageSettings.ViewportOCIO.PerViewportOCIOProfiles", DisplayName = "Per-Viewport OCIO Overrides"))
	FDisplayClusterEditorPropertyReference PerViewportOCIOProfilesRef;

	UPROPERTY(EditAnywhere, Transient, Category = "Exposure", meta = (PropertyPath = "CurrentConfigData.StageSettings.AutoExposureSettings.bEnabled"))
	FDisplayClusterEditorPropertyReference AutoExposureEnabledRef;

	UPROPERTY(EditAnywhere, Transient, Category = "Exposure", meta = (PropertyPath = "CurrentConfigData.StageSettings.AutoExposureSettings.MeteringSettings"))
	FDisplayClusterEditorPropertyReference AutoExposureMeteringSettingsRef;

	UPROPERTY(EditAnywhere, Transient, Category = "Exposure", meta = (PropertyPath = "CurrentConfigData.StageSettings.AutoExposureSettings.ExcludeSettings"))
	FDisplayClusterEditorPropertyReference AutoExposureExcludeSettingsRef;

	UPROPERTY(EditAnywhere, Transient, Category = Chromakey, meta = (PropertyPath = "CurrentConfigData.StageSettings.GlobalChromakey.ChromakeyColor"))
	FDisplayClusterEditorPropertyReference GlobalChromakeyColorRef;

	UPROPERTY(EditAnywhere, Transient, Category = Chromakey, meta = (PropertyPath = "CurrentConfigData.StageSettings.GlobalChromakey.ChromakeyMarkers"))
	FDisplayClusterEditorPropertyReference GlobalChromakeyMarkersRef;

	UPROPERTY(EditAnywhere, Transient, Category = "Light Cards", meta = (PropertyPath = "CurrentConfigData.StageSettings.Lightcard.bEnable"))
	FDisplayClusterEditorPropertyReference EnableLightcardsRef;

	UPROPERTY(EditAnywhere, Transient, Category = "Light Cards", meta = (PropertyPath = "CurrentConfigData.StageSettings.Lightcard.Blendingmode", EditConditionPath = "CurrentConfigData.StageSettings.Lightcard.bEnable"))
	FDisplayClusterEditorPropertyReference LightCardBlendingModeRef;

	UPROPERTY(EditAnywhere, Transient, Category = "Light Cards", meta = (PropertyPath = "CurrentConfigData.StageSettings.Lightcard.ShowOnlyList", EditConditionPath = "CurrentConfigData.StageSettings.Lightcard.bEnable"))
	FDisplayClusterEditorPropertyReference LightCardContentRef;

	UPROPERTY(EditAnywhere, Transient, Category = "Light Cards", meta = (PropertyPath = "CurrentConfigData.StageSettings.Lightcard.LightcardOCIO.LightcardOCIOMode", DisplayName = "Light Cards OCIO"))
	FDisplayClusterEditorPropertyReference LightcardOCIOModeRef;

	UPROPERTY(EditAnywhere, Transient, Category = "Light Cards", meta = (PropertyPath = "CurrentConfigData.StageSettings.Lightcard.LightcardOCIO.CustomOCIO.AllViewportsOCIOConfiguration.ColorConfiguration", DisplayName = "All Viewports Color Configuration"))
	FDisplayClusterEditorPropertyReference LightcardAllViewportColorConfigurationRef;

	UPROPERTY(EditAnywhere, Transient, Category = "Light Cards", meta = (PropertyPath = "CurrentConfigData.StageSettings.Lightcard.LightcardOCIO.CustomOCIO.PerViewportOCIOProfiles", DisplayName = "Per-Viewport OCIO Overrides"))
	FDisplayClusterEditorPropertyReference LightcardPerViewportOCIOProfilesRef;

	// Media
	UPROPERTY(EditDefaultsOnly, Transient, Category = "Media", meta = (PropertyPath = "CurrentConfigData.MediaSettings"))
	FDisplayClusterEditorPropertyReference MediaSettingsRef;

#endif // WITH_EDITORONLY_DATA

private:
	/**
	 * Name of the CurrentConfigData asset. Only required if this is a parent of a DisplayClusterBlueprint.
	 * The name is used to lookup the config data as a default sub-object, specifically in packaged builds.
	 */
	UPROPERTY()
	FName ConfigDataName;

	/**
	 * The root component for our hierarchy.
	 * Must have CPF_Edit(such as VisibleDefaultsOnly) on property for Live Link.
	 * nDisplay details panel will hide this from actually being visible.
	 */
	UPROPERTY(EditAnywhere, Category = "NDisplay", meta = (HideProperty))
	TObjectPtr<USceneComponent> DisplayClusterRootComponent;

	/**
	 * Default camera component. It's an outer camera in VP/ICVFX terminology. Always exists on a DCRA instance.
	 */
	UPROPERTY(VisibleAnywhere, BlueprintGetter=GetDefaultCamera, Category = "NDisplay|Components")
	TObjectPtr<UDisplayClusterCameraComponent> DefaultViewPoint;

	/**
	 * Helper sync component. Performs sync procedure during Tick phase.
	 */
	UPROPERTY()
	TObjectPtr<UDisplayClusterSyncTickComponent> SyncTickComponent;

	/** Component that stores the stage's geometry map, which is used to make objects flush with the stage's walls and ceilings */
	UPROPERTY()
	TObjectPtr<UDisplayClusterStageGeometryComponent> StageGeometryComponent;

	/** Component that stores a 3D representation of the stage's geometry map, which can be used to perform ray traces against the processed stage geometry */
	UPROPERTY()
	TObjectPtr<UDisplayClusterStageIsosphereComponent> StageIsosphereComponent;

private:
	// Current operation mode
	EDisplayClusterOperationMode OperationMode;

	float LastDeltaSecondsValue = 0.f;

private:
	template <typename TComp>
	void GetTypedPrimitives(TSet<FPrimitiveComponentId>& OutPrimitives, const TArray<FString>* InCompNames = nullptr, bool bForceHide = false, bool bCollectChildrenVisualizationComponent = true) const;

public:
	/** Set the priority for inner frustum rendering if there is any overlap when enabling multiple ICVFX cameras. */
	UPROPERTY(EditInstanceOnly, EditFixedSize, Category = "In Camera VFX", meta = (TitleProperty = "Name", DisplayAfter = "ShowInnerFrustumOverlapsRef"))
	TArray<FDisplayClusterComponentRef> InnerFrustumPriority;

	/**
	 * If set from the DisplayCluster BP Compiler it will be loaded from the class default subobjects in run-time.
	 */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Instanced, Category = "NDisplay", meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UDisplayClusterConfigurationData> CurrentConfigData;

public:
	// UObject interface
	static void AddReferencedObjects(class UObject* InThis, class FReferenceCollector& Collector);
	// End of UObject interface

	bool IsInnerFrustumEnabled(const FString& InnerFrustumID) const;

	// Return inner frustum priority by InnerFrustum name (from InnerFrustumPriority property)
	// return -1, if not defined
	int GetInnerFrustumPriority(const FString& InnerFrustumID) const;

	float GetWorldDeltaSeconds() const
	{
		return LastDeltaSecondsValue;
	}

	/** Get current settings for preview rendering.
	* 
	* @param bIgnorePreviewSetttingsSource - if true, the PreviewSettingsSource is ignored
	*/
	FDisplayClusterViewport_PreviewSettings GetPreviewSettings(bool bIgnorePreviewSetttingsSource = false) const;

private:
	/**
	* Prepares new preview settings.
	* (Access preview settings as read-only data; does not alter DCRA)
	*
	* @param OutPreviewSettings  Filled with the prepared settings.
	* @return True if preview rendering should proceed, false otherwise.
	*/
	bool PreparePreviewRendererSettings(FDisplayClusterViewport_PreviewSettings& OutPreviewSettings) const;

//////////////////////////////////////////////////////////////////////////////////////////////
// EDITOR RELATED SETTINGS
//////////////////////////////////////////////////////////////////////////////////////////////
public:

	/** Render the scene and display it as a preview on the nDisplay root actor in the editor.  This will impact editor performance. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Preview", meta = (DisplayName = "Enable Editor Preview"))
	bool bPreviewEnable = true;
	
	/** Render this DCRA in game for Standalone/Package builds. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Preview", AdvancedDisplay, meta = (DisplayName = "Enable Preview in Game"))
	bool bPreviewInGameEnable = false;

	/** Adjust resolution scaling for the editor preview. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Preview", meta = (DisplayName = "Preview Screen Percentage", ClampMin = "0.05", UIMin = "0.05", ClampMax = "1", UIMax = "1"))
	float PreviewRenderTargetRatioMult = 0.25;

	/** Enable PostProcess for preview. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Preview", meta = (DisplayName = "Enable Post Process"), BlueprintSetter = SetPreviewEnablePostProcess)
	bool bPreviewEnablePostProcess = false;

	/**
	* Render a preview with postprocessing in the scene using the HoldoutComposite plugin.
	* 
	* The `Preview`>`Enable Post Process` option must be set
	* 
	* These project settings also need to be set:
	* 1. "Alpha Output"
	* 2. "Support Primitive Alpha Holdout (Deferred)"
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Preview", meta = (DisplayName = "Enable Preview Holdout", EditCondition = "bPreviewEnablePostProcess"), BlueprintSetter = SetPreviewEnableHoldoutComposite)
	bool bPreviewEnableHoldoutComposite = true;

	/** Enable TemporalAA/TSR in preview.
	* Increases memory consumption. This will impact editor performance. 
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Preview", meta = (DisplayName = "Allow TemporalAA/TSR in Preview"))
	bool bPreviewEnableTSR = false;

	/** Enable OCIO on preview. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Preview", meta = (DisplayName = "Enable OCIO on Preview"))
	bool bPreviewEnableOCIO = false;

	/** Show overlay material on the preview mesh when preview rendering is enabled (UMeshComponent::OverlayMaterial). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Preview", AdvancedDisplay, meta = (DisplayName = "Enable Preview Overlay"))
	bool bPreviewEnableOverlayMaterial = true;

	/** Configure the root actor for Techvis rendering with preview components. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Preview", AdvancedDisplay)
	bool bEnablePreviewTechvis = false;

	/** Enable the use of a preview mesh for the preview for this DCRA. */
	UPROPERTY(Transient, NonTransactional)
	bool bEnablePreviewMesh = true;

	/** Enable the use of a preview editable mesh for the preview for this DCRA. */
	UPROPERTY(Transient, NonTransactional)
	bool bEnablePreviewEditableMesh = true;

	/** Determines where the preview settings will be retrieved from. */
	UPROPERTY(Transient, NonTransactional)
	EDisplayClusterConfigurationRootActorPreviewSettingsSource PreviewSetttingsSource = EDisplayClusterConfigurationRootActorPreviewSettingsSource::RootActor;

	/** Freeze preview render.  This will impact editor performance. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Preview", meta = (DisplayName = "Freeze Editor Preview"))
	bool bFreezePreviewRender = false;

	/** Render ICVFX Frustums */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Preview", meta = (DisplayName = "Enable Camera Frustums"))
	bool bPreviewICVFXFrustums = false;

	/** Render ICVFX Frustums */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Preview", AdvancedDisplay, meta = (DisplayName = "Camera Frustum Distance"))
	float PreviewICVFXFrustumsFarDistance = 1000.0f;

	/** Enables the use of 'bPreviewICVFXFrustums' in the game. */
	UPROPERTY()
	bool bPreviewInGameRenderFrustum = true;

#if WITH_EDITORONLY_DATA
	/** Selectively preview a specific viewport or show all/none. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Preview", meta = (DisplayName = "Preview Node"))
	FString PreviewNodeId = DisplayClusterConfigurationStrings::gui::preview::PreviewNodeNone;

	/** Render Mode */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Preview", meta = (DisplayName = "Render Mode"))
	EDisplayClusterConfigurationRenderMode RenderMode = EDisplayClusterConfigurationRenderMode::Mono;
#endif

	/** Tick Per Frame */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Preview", AdvancedDisplay, meta = (DisplayName = "Tick Per Frame", ClampMin = "1", UIMin = "1", ClampMax = "200", UIMax = "200"))
	int TickPerFrame = 1;

	/** Max amount of Viewports Per Frame */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Preview", AdvancedDisplay, meta = (DisplayName = "Viewports Per Frame", ClampMin = "1", UIMin = "1", ClampMax = "200", UIMax = "200"))
	int ViewportsPerFrame = 1;

	/** The maximum dimension of any internal texture for preview. Use less memory for large preview viewports */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Preview", AdvancedDisplay, meta = (DisplayName = "Preview Texture Max Size", ClampMin = "64", UIMin = "64", ClampMax = "4096", UIMax = "4096"))
	int PreviewMaxTextureDimension = 2048;

	/** The included display device nDisplay provides by default */
	UPROPERTY(VisibleDefaultsOnly, Category = "Preview", DisplayName = "Basic Display Device")
	TObjectPtr<UDisplayClusterDisplayDeviceBaseComponent> BasicDisplayDeviceComponent;

	/** Select the default display device class to use when a viewport doesn't have one assigned */
	UPROPERTY(EditDefaultsOnly, Category = "Preview", AdvancedDisplay, DisplayName = "Default Display Device")
	FName DefaultDisplayDeviceName;


#if WITH_EDITORONLY_DATA
	/** Toggles the visibility of the stage's geometry mesh, a smooth, continuous mesh generated and processed from the stage's geometry */
	UPROPERTY(EditInstanceOnly, Category = "Preview", AdvancedDisplay)
	bool bPreviewStageGeometryMesh = false;
#endif

protected:
	/** The default display device to use for preview rendering */
	UPROPERTY(Transient, NonTransactional)
	mutable TObjectPtr<UDisplayClusterDisplayDeviceBaseComponent> DefaultDisplayDeviceComponent;

	/** Line Batchers. All lines to be drawn in the world, but not inside viewports. */
	UPROPERTY(Transient, NonTransactional)
	TObjectPtr<ULineBatchComponent> LineBatcherComponent;

protected:
	/** The name the internal default display device uses. */
	virtual FName GetInternalDisplayDeviceName() const;

//////////////////////////////////////////////////////////////////////////////////////////////
// EDITOR RELATED SETTINGS
//////////////////////////////////////////////////////////////////////////////////////////////

#if WITH_EDITOR
public:
	/** Enable or disable editor render. Preview components may need to be on for texture overrides, but capture and rendering disabled. */
	UE_DEPRECATED(5.4, "This function has been deprecated.")
	void EnableEditorRender(bool bValue) { }

	/** If editor rendering is enabled. */
	UE_DEPRECATED(5.4, "This function has been deprecated.")
	bool IsEditorRenderEnabled() const { return false; }

public:
	DECLARE_DELEGATE(FOnPreviewUpdated);

public:
	// We need tick in Editor
	virtual bool ShouldTickIfViewportsOnly() const override { return true; }

	UE_DEPRECATED(5.4, "This function has been deprecated.")
	FOnPreviewUpdated& GetOnPreviewGenerated() { return DeprecatedPreviewDelegate; }

	UE_DEPRECATED(5.4, "This function has been deprecated.")
	FOnPreviewUpdated& GetOnPreviewDestroyed() { return DeprecatedPreviewDelegate; }

	// return true, if preview enabled for this actor
	UE_DEPRECATED(5.4, "This function has been deprecated.")
	bool IsPreviewEnabled() const { return false; }

	/** Gets whether the preview output is displayed onto the stage actor's screen meshes */
	UE_DEPRECATED(5.4, "This function has been deprecated.")
	bool IsPreviewDrawnToScreens() const { return false; }

	void Constructor_Editor();
	void Destructor_Editor();

	/** Perform a rendering of the DCRA preview. */
	UE_DEPRECATED(5.4, "This function has been deprecated.")
	void RenderPreview_Editor() { };

	void PostLoad_Editor();
	void PostActorCreated_Editor();
	void EndPlay_Editor(const EEndPlayReason::Type EndPlayReason);
	void BeginDestroy_Editor();
	void Destroyed_Editor();
	void RerunConstructionScripts_Editor();

	// Preview components free referenced meshes and materials
	UE_DEPRECATED(5.4, "This function has been deprecated.")
	void ResetPreviewComponents_Editor(bool bInRestoreSceneMaterial) { }

	UE_DEPRECATED(5.4, "This function has been deprecated.")
	class UDisplayClusterPreviewComponent* GetPreviewComponent(const FString& NodeId, const FString& ViewportId) const { return nullptr; }

	UE_DEPRECATED(5.4, "This function has been deprecated.")
	void UpdatePreviewComponents() { }
	UE_DEPRECATED(5.4, "This function has been deprecated.")
	void ReleasePreviewComponents() { }

	/**
	 * Enable the use of a post process render target when bPreviewEnablePostProcess is disabled on the actor. The root actor
	 * will still display a pre post processed preview. This may increase editor overhead.
	 *
	 * Retrieve the post process texture from the preview component with GetRenderTargetTexturePostProcess().
	 *
	 * @param Object The object subscribing to updates
	 * @return The number of subscribers to use post process.
	 */
	UE_DEPRECATED(5.4, "This function has been deprecated.")
	int32 SubscribeToPostProcessRenderTarget(const uint8* Object) { return INDEX_NONE; };

	/**
	 * Unsubscribe a registered object from requiring post process render target updates.
	 *
	 * @param Object The object subscribing to updates. When the counter is zero post process render targets will not be used.
	 * @return The number of subscribers to use post process.
	 */
	UE_DEPRECATED(5.4, "This function has been deprecated.")
	int32 UnsubscribeFromPostProcessRenderTarget(const uint8* Object) { return INDEX_NONE; };

	/** If one or more observers are subscribed to receive post process preview targets. */
	UE_DEPRECATED(5.4, "This function has been deprecated.")
	bool DoObserversNeedPostProcessRenderTarget() const { return false; };
	
	/** When rendering the preview determine which render target should be used for the current frame. */
	UE_DEPRECATED(5.4, "This function has been deprecated.")
	bool ShouldThisFrameOutputPreviewToPostProcessRenderTarget() const { return false; }

	/** Force preview rendering to be enabled regardless of the user's setting until a matching RemovePreviewEnableOverride call is made. */
	UE_DEPRECATED(5.5, "This function has been deprecated.")
	void AddPreviewEnableOverride(const uint8* Object) { };

	/**
	 * Stop forcing preview rendering to be enabled for this caller. If other objects have called AddPreviewEnableOverride, it will remain
	 * forced until they have also removed their overrides.
	 */
	UE_DEPRECATED(5.5, "This function has been deprecated.")
	void RemovePreviewEnableOverride(const uint8* Object) { };

	UE_DEPRECATED(5.4, "This function has been deprecated.")
	float GetPreviewRenderTargetRatioMult() const
	{
		return PreviewRenderTargetRatioMult;
	};

	UE_DEPRECATED(5.4, "This function has been deprecated.")
	IDisplayClusterViewport* FindPreviewViewport(const FString& InViewportId) const
	{
		return nullptr;
	}

	UE_DEPRECATED(5.4, "This function has been deprecated.")
	void GetPreviewRenderTargetableTextures(const TArray<FString>& InViewportNames, TArray<FTextureRHIRef>& OutTextures)
	{ }

	void UpdateInnerFrustumPriority();
	void ResetInnerFrustumPriority();
	
	virtual bool IsSelectedInEditor() const override;
	void SetIsSelectedInEditor(bool bValue);

	// Don't show actor preview in the level viewport when DCRA actor is selected, but none of its children are.
	UE_DEPRECATED(5.4, "This function has been deprecated.")
	virtual bool IsDefaultPreviewEnabled() const override
	{
		return false;
	}

protected:
	virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChainEvent) override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostEditMove(bool bFinished) override;
	virtual void PostTransacted(const FTransactionObjectEvent& TransactionEvent) override;

	/** Called when the asset has been reloaded in the editor. */
	void HandleAssetReload(const EPackageReloadPhase InPackageReloadPhase, FPackageReloadedEvent* InPackageReloadedEvent);

	/** Raised when any object is moved within a level. Used to update the stage's geometry map if any of its components have been moved */
	void OnEndObjectMovement(UObject& InObject);

private:

	/** Flag to indicate if the user is currently interacting with a subobject of CurrentConfigData. */
	bool bIsInteractiveEditingSubobject = false;

	/** Flag to indicate if we need to reregister components. */
	bool bRequiresComponentRefresh = false;

	bool bIsSelectedInEditor = false;

	TWeakPtr<IDisplayClusterConfiguratorBlueprintEditor> ToolkitPtr;

	// UE_DEPRECATED 5.4
	FOnPreviewUpdated DeprecatedPreviewDelegate;
#endif

private:
	// The exclusive lock is a symmetric relationship:
	//   - This DCRA tracks its holders in RootActorExclusiveLockOwners (Add/RemoveRootActorExclusiveLockOwner).
	//   - Each holder can be queried from the other side via IDisplayClusterViewportConfiguration::IsRootActorLockedByThis().
	friend class FDisplayClusterViewportConfiguration;

	/**
	 * Adds InOwner to the exclusive lock holder set.
	 * The DCRA remains locked while at least one live holder is registered.
	 *
	 * @param InOwner  The viewport manager acquiring the lock.
	 */
	void AddRootActorExclusiveLockOwner(const TSharedRef<IDisplayClusterViewportManager, ESPMode::ThreadSafe> InOwner);

	/**
	 * Removes InOwner from the exclusive lock holder set.
	 *
	 * @param InOwner  The viewport manager releasing the lock.
	 */
	void RemoveRootActorExclusiveLockOwner(const TSharedRef<IDisplayClusterViewportManager, ESPMode::ThreadSafe> InOwner);

	/**
	 * Purges stale entries from RootActorExclusiveLockOwners:
	 *  - Expired weak pointers (holder was destroyed).
	 *  - Holders that no longer claim the lock (IsRootActorLockedByThis() returns false).
	 */
	void RemoveRootActorExpiredLockOwners();

	/**
	 * Returns true while at least one live exclusive lock holder is registered.
	 * While locked:
	 *  - Proxy actors (ICVFX panel, StageApp, etc.) may not use this DCRA.
	 *  - Default scene preview rendering is disabled.
	 *  - PIE rendering is disabled.
	 */
	bool IsRootActorExclusivelyLocked() const;

private:
	/** Viewport managers currently holding an exclusive lock on this DCRA.
	 *  Counterpart query: IDisplayClusterViewportConfiguration::IsRootActorLockedByThis(). */
	TSet<TWeakPtr<IDisplayClusterViewportManager, ESPMode::ThreadSafe>> RootActorExclusiveLockOwners;

	/** Mutex for RootActorExclusiveLockOwners. */
	mutable FCriticalSection RootActorExclusiveLockOwnersMutex;
};
