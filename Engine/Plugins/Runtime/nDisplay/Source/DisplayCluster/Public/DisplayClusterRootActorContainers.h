// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DisplayClusterConfigurationTypes_Enums.h"

/**
 * Optional overrides of root actor settings.
 */
struct FDisplayClusterRootActorPropertyOverrides
{
	/** Render this DCRA in game for Standalone/Package builds. */
	TOptional<bool> bPreviewInGameEnable;

	/** Render ICVFX Frustums in game. */
	TOptional<bool> bPreviewInGameRenderFrustum;

	/** Render the scene and display it as a preview on the nDisplay root actor in the editor.  This will impact editor performance. */
	TOptional<bool> bPreviewEnable;
	
	/** Adjust resolution scaling for the editor preview. */
	TOptional<float> PreviewRenderTargetRatioMult;

	/** Enable PostProcess for preview. */
	TOptional<bool> bPreviewEnablePostProcess;

	/** Enable OCIO on preview. */
	TOptional<bool> bPreviewEnableOCIO;

	/** Enables the use of 'HoldoutComposite' plugin for the preview rendering. */
	TOptional<bool> bPreviewEnableHoldoutComposite;

	/** Enable TemporalAA/TSR in preview.
	* Increases memory consumption. This will impact editor performance.
	*/
	TOptional<bool> bPreviewEnableTSR;

	/** Show overlay material on the preview mesh when preview rendering is enabled (UMeshComponent::OverlayMaterial). */
	TOptional<bool> bPreviewEnableOverlayMaterial;

	/** Configure the root actor for Techvis rendering with preview components. */
	TOptional<bool> bEnablePreviewTechvis;

	/** Enable the use of a preview mesh for the preview for this DCRA. */
	TOptional<bool> bEnablePreviewMesh;

	/** Enable the use of a preview editable mesh for the preview for this DCRA. */
	TOptional<bool> bEnablePreviewEditableMesh;

	/** Determines where the preview settings will be retrieved from. */
	TOptional<EDisplayClusterConfigurationRootActorPreviewSettingsSource> PreviewSetttingsSource;

	/** Freeze preview render.  This will impact editor performance. */
	TOptional<bool> bFreezePreviewRender;

	/** Render ICVFX Frustums */
	TOptional<bool> bPreviewICVFXFrustums;

	/** Render ICVFX Frustums */
	TOptional<float> PreviewICVFXFrustumsFarDistance;

	/** Tick Per Frame */
	TOptional<int32> TickPerFrame;

	/** Max amount of Viewports Per Frame */
	TOptional<int32> ViewportsPerFrame;

	/** The maximum dimension of any internal texture for preview. Use less memory for large preview viewports */
	TOptional<int32> PreviewMaxTextureDimension;
};
