// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SceneViewExtension.h"
#include "Render/Viewport/IDisplayClusterViewport.h"

/**
 * Contains information about the context in which this scene view extension will be used.
 */
struct FDisplayClusterSceneViewExtensionContext : public FSceneViewExtensionContext
{
private:
	//~ FSceneViewExtensionContext Interface
	virtual FName GetRTTI() const override { return TEXT("FDisplayClusterSceneViewExtensionContext"); }

	virtual bool IsHMDSupported() const override
	{
		// Disable all HMD extensions for nDisplay render
		return false;
	}

public:
	FDisplayClusterSceneViewExtensionContext()
		: FSceneViewExtensionContext()
	{ }

	FDisplayClusterSceneViewExtensionContext(
		FViewport* InViewport,
		const TSharedRef<const IDisplayClusterViewport, ESPMode::ThreadSafe>& InDisplayClusterViewport)
		: FSceneViewExtensionContext(InViewport)
		, DisplayClusterViewport(InDisplayClusterViewport)
	{ }

	FDisplayClusterSceneViewExtensionContext(
		FSceneInterface* InScene,
		const TSharedRef<const IDisplayClusterViewport, ESPMode::ThreadSafe>& InDisplayClusterViewport)
		: FSceneViewExtensionContext(InScene)
		, DisplayClusterViewport(InDisplayClusterViewport)
	{ }

	/** Returns true if this viewport context refers to the same configuration. */
	bool IsSameDisplayClusterViewportConfiguration(const TSharedRef<const IDisplayClusterViewportConfiguration, ESPMode::ThreadSafe> InConfigurationRef) const
	{
		if (DisplayClusterViewport.IsValid()
			&& DisplayClusterViewport->GetConfigurationRef() == InConfigurationRef)
		{
			return true;
		}

		return false;
	}

public:
	// Reference to the DC viewport
	const TSharedPtr<const IDisplayClusterViewport, ESPMode::ThreadSafe> DisplayClusterViewport;
};
