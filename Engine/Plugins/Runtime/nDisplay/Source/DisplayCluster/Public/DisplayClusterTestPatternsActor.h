// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Actor.h"

#include "Engine/Scene.h"

#include "Cluster/IDisplayClusterClusterManager.h"
#include "DisplayClusterEnums.h"

#include "DisplayClusterTestPatternsActor.generated.h"


class IConsoleVariable;
class UMaterial;
class UMaterialInstanceDynamic;
class UPostProcessComponent;
struct FDisplayClusterConfigViewport;
struct FWeightedBlendable;

struct FDisplayClusterClusterEventJson;


/**
 * Test patterns actor
 */
UCLASS()
class DISPLAYCLUSTER_API ADisplayClusterTestPatternsActor
	: public AActor
{
	GENERATED_BODY()

public:
	ADisplayClusterTestPatternsActor(const FObjectInitializer& ObjectInitializer);

public:
	//////////////////////////////////////////////////////////////////////////////////////////////
	// AActor
	//////////////////////////////////////////////////////////////////////////////////////////////
	virtual void BeginPlay() override;
	virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
	virtual void Tick(float DeltaSeconds) override;

protected:
	enum class EParamType
	{
		TypeScalar,
		TypeVector
	};

	struct FMaterialParameter
	{
		FMaterialParameter(EParamType _Type, const FString& _Value)
			: Type(_Type)
			, Value(_Value)
		{ }

		EParamType Type;
		FString    Value;
	};

protected:
	/** Postprocess component */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "NDisplay")
	TObjectPtr<UPostProcessComponent> PostProcessComponent;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "NDisplay")
	TMap<FString, TObjectPtr<UMaterial>> CalibrationPatterns;

private:
	void InitializeMaterials();
	void InitializeInternals();

	void UpdatePattern(const TMap<FString, FString>& Params);

	UMaterialInstanceDynamic* CreateMaterialInstance(const FString& PatternId);
	FWeightedBlendable CreateWeightedBlendable(const FString& PatternId);
	FWeightedBlendable CreateWeightedBlendable(UMaterialInstanceDynamic* DynamicMaterialInstance);
	FPostProcessSettings CreatePPSettings(const FString& PatternId);
	FPostProcessSettings CreatePPSettings(UMaterialInstanceDynamic* DynamicMaterialInstance);

	void SetupMaterialParameters(UMaterialInstanceDynamic* DynamicMaterialInstance, const TMap<FString, FMaterialParameter>& Params) const;

	void OnConsoleVariableChangedPattern(IConsoleVariable* Var);
	void OnClusterEventHandler(const FDisplayClusterClusterEventJson& Event);

private:
	UPROPERTY(transient)
	TMap<FString, FPostProcessSettings> ViewportPPSettings;

	// Current nDisplay operation mode
	EDisplayClusterOperationMode OperationMode = EDisplayClusterOperationMode::Disabled;
	// Cluster event delegate
	FOnClusterEventJsonListener OnClusterEvent;

	FCriticalSection InternalsSyncScope;
};
