// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/GameViewportClient.h"

#include "DisplayClusterViewportClient.generated.h"

UCLASS()
class DISPLAYCLUSTER_API UDisplayClusterViewportClient
	: public UGameViewportClient
{
	GENERATED_BODY()

public:
	UDisplayClusterViewportClient(FVTableHelper& Helper);
	virtual ~UDisplayClusterViewportClient();

	virtual void Init(struct FWorldContext& WorldContext, UGameInstance* OwningGameInstance, bool bCreateNewAudioDevice = true) override;
	virtual void Draw(FViewport* Viewport, FCanvas* SceneCanvas) override;

	virtual TSharedRef<FSceneViewport> CreateViewport(TSharedPtr<SViewport> InViewportWidget) override;

	/** Always use non-scaled viewports with nDisplay */
	virtual float GetDPIScale() const override
	{
		return 1.f;
	}

protected:
#if WITH_EDITOR
	bool Draw_PIE(FViewport* InViewport, FCanvas* SceneCanvas);
#endif /*WITH_EDITOR*/
};
