// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Object.h"
#include "Subsystems/WorldSubsystem.h"
#include "Engine/EngineBaseTypes.h"

#include "DisplayClusterWorldSubsystem.generated.h"

/** 
* World Subsystem used to react to level and world changes.
* When Concert reloads the packages, streamed levels are removed and re-added without invoiking LoadMap which 
* circumvents FDisplayClusterGameManager::StartScene method invoked inside LoadMap method of DisplayClusterGameEngine.
* This causes issues such as not updating references to DisplayClusterRootActor which causes memory corruption, crashes 
* and graphic corruption. This Subsystem is used to react to changes in number of levels used in the current world 
* and forces DisplayClusterModule to refresh all of its managers.
*/
UCLASS()
class UDisplayClusterWorldSubsystem : public UWorldSubsystem
{
	GENERATED_BODY()
public:

	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	virtual void Deinitialize() override;

public:

	/** Event that is triggered when number of levels is changed. */
	void OnLevelsChanged();
};