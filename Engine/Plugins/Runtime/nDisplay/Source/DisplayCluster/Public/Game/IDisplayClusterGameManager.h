// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"

class ADisplayClusterRootActor;
class UDisplayClusterCameraComponent;
class UDisplayClusterMeshComponent;
class UDisplayClusterRootComponent;
class UDisplayClusterSceneComponent;
class UDisplayClusterScreenComponent;
class UWorld;


/**
 * Public game manager interface
 */
class IDisplayClusterGameManager
{
public:
	virtual ~IDisplayClusterGameManager() = default;

public:
	/**
	* @return - Current root actor
	*/
	virtual ADisplayClusterRootActor* GetRootActor() const = 0;

	/**
	* @return - Current world
	*/
	virtual UWorld* GetWorld() const = 0;

	/**
	* @return - All root actors in world
	*/
	virtual TArray<ADisplayClusterRootActor*> GetAllRootActorsFromWorld(UWorld* InWorld) = 0;
};
