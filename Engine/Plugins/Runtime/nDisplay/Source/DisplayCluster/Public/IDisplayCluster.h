// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"
#include "Modules/ModuleInterface.h"

#include "DisplayClusterEnums.h"


class IDisplayClusterRenderManager;
class IDisplayClusterClusterManager;
class IDisplayClusterConfigManager;
class IDisplayClusterGameManager;
class IDisplayClusterCallbacks;


/**
 * Public module interface
 */
class IDisplayCluster
	: public IModuleInterface
{
public:
	static constexpr const TCHAR* ModuleName = TEXT("DisplayCluster");

public:
	virtual ~IDisplayCluster() = default;

public:
	/**
	* Singleton-like access to this module's interface.  This is just for convenience!
	* Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	*
	* @return Returns singleton instance, loading the module on demand if needed
	*/
	static inline IDisplayCluster& Get()
	{
		//return FModuleManager::LoadModuleChecked<IDisplayCluster>(IDisplayCluster::ModuleName);
		return FModuleManager::GetModuleChecked<IDisplayCluster>(IDisplayCluster::ModuleName);
	}

	/**
	* Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	*
	* @return True if the module is loaded and ready to use
	*/
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded(IDisplayCluster::ModuleName);
	}


	/**
	* [DEPRECATED] Checks if the module has been initialized.
	*
	* @return Is initialized
	*/
	UE_DEPRECATED(5.4, "This function has been deprecated and will be removed soon.")
	virtual bool IsModuleInitialized() const
	{
		return true;
	}

	/**
	* Returns current operation mode.
	*
	* @return Display Cluster operation mode
	*/
	virtual EDisplayClusterOperationMode GetOperationMode() const = 0;

	/**
	* Access to the device manager.
	*
	* @return Current device manager or nullptr
	*/
	virtual IDisplayClusterRenderManager* GetRenderMgr() const = 0;

	/**
	* Access to the cluster manager.
	*
	* @return Current cluster manager or nullptr
	*/
	virtual IDisplayClusterClusterManager* GetClusterMgr() const = 0;

	/**
	* Access to the config manager.
	*
	* @return Current config manager or nullptr
	*/
	virtual IDisplayClusterConfigManager* GetConfigMgr() const = 0;

	/**
	* Access to the game manager.
	*
	* @return Current game manager or nullptr
	*/
	virtual IDisplayClusterGameManager* GetGameMgr() const = 0;

	/**
	* Access to the DisplayCluster callbacks API
	*
	* @return Callbacks API
	*/
	virtual IDisplayClusterCallbacks& GetCallbacks() = 0;
};
