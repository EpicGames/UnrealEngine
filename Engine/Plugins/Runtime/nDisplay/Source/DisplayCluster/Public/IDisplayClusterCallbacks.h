// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DisplayClusterEnums.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_Enums.h"

#include "Delegates/DelegateCombinations.h"
#include "RHIFwd.h"

class FDisplayClusterShaderParameters_ICVFX;
class FRDGBuilder;
class FRHICommandListImmediate;
class FSceneView;
class FSceneViewFamily;
class FSceneViewFamilyContext;
class FViewport;
class IDisplayClusterViewport;
class IDisplayClusterViewportProxy;
class IDisplayClusterViewportManagerProxy;
struct FDisplayClusterShaderParameters_WarpBlend;
struct FPostProcessMaterialInputs;


/**
 * DisplayCluster callbacks API
 */
class IDisplayClusterCallbacks
{
public:
	virtual ~IDisplayClusterCallbacks() = default;

public:
	/** Called on session start **/
	DECLARE_EVENT(IDisplayClusterCallbacks, FDisplayClusterStartSessionEvent);
	virtual FDisplayClusterStartSessionEvent& OnDisplayClusterStartSession() = 0;

	/** Called on session end **/
	DECLARE_EVENT(IDisplayClusterCallbacks, FDisplayClusterEndSessionEvent);
	virtual FDisplayClusterEndSessionEvent& OnDisplayClusterEndSession() = 0;

	/** Called on start scene **/
	DECLARE_EVENT(IDisplayClusterCallbacks, FDisplayClusterStartSceneEvent);
	virtual FDisplayClusterStartSceneEvent& OnDisplayClusterStartScene() = 0;

	/** Called on end scene **/
	DECLARE_EVENT(IDisplayClusterCallbacks, FDisplayClusterEndSceneEvent);
	virtual FDisplayClusterEndSceneEvent& OnDisplayClusterEndScene() = 0;

	/** Called on DisplayCluster StartFrame **/
	DECLARE_EVENT_OneParam(IDisplayClusterCallbacks, FDisplayClusterStartFrameEvent, uint64);
	virtual FDisplayClusterStartFrameEvent& OnDisplayClusterStartFrame() = 0;

	/** Called on DisplayCluster EndFrame **/
	DECLARE_EVENT_OneParam(IDisplayClusterCallbacks, FDisplayClusterEndFrameEvent, uint64);
	virtual FDisplayClusterEndFrameEvent& OnDisplayClusterEndFrame() = 0;

	/** Called on DisplayCluster PreTick **/
	DECLARE_EVENT(IDisplayClusterCallbacks, FDisplayClusterPreTickEvent);
	virtual FDisplayClusterPreTickEvent& OnDisplayClusterPreTick() = 0;

	/** Called on DisplayCluster Tick **/
	DECLARE_EVENT(IDisplayClusterCallbacks, FDisplayClusterTickEvent);
	virtual FDisplayClusterTickEvent& OnDisplayClusterTick() = 0;

	/** Called on DisplayCluster PostTick **/
	DECLARE_EVENT(IDisplayClusterCallbacks, FDisplayClusterPostTickEvent);
	virtual FDisplayClusterPostTickEvent& OnDisplayClusterPostTick() = 0;

	/** Callback triggered when custom present handler was created **/
	DECLARE_EVENT(IDisplayClusterCallbacks, FDisplayClusterCustomPresentSetEvent);
	virtual FDisplayClusterCustomPresentSetEvent& OnDisplayClusterCustomPresentSet() = 0;

	/** Callback triggered before submitting view families to render **/
	DECLARE_EVENT_OneParam(IDisplayClusterCallbacks, FDisplayClusterPreSubmitViewFamilies, TArray<FSceneViewFamilyContext*>&);
	virtual FDisplayClusterPreSubmitViewFamilies& OnDisplayClusterPreSubmitViewFamilies() = 0;

	/** Called before presentation synchronization is initiated **/
	DECLARE_EVENT_OneParam(IDisplayClusterCallbacks, FDisplayClusterPresentationPreSynchronization_RHIThread, FRHIViewport*);
	virtual FDisplayClusterPresentationPreSynchronization_RHIThread& OnDisplayClusterPresentationPreSynchronization_RHIThread() = 0;

	/** Called after presentation synchronization is completed **/
	DECLARE_EVENT_OneParam(IDisplayClusterCallbacks, FDisplayClusterPresentationPostSynchronization_RHIThread, FRHIViewport*);
	virtual FDisplayClusterPresentationPostSynchronization_RHIThread& OnDisplayClusterPresentationPostSynchronization_RHIThread() = 0;

	/** Called after frame presentation **/
	DECLARE_EVENT_OneParam(IDisplayClusterCallbacks, FDisplayClusterFramePresentated_RHIThread, bool bNativePresent);
	virtual FDisplayClusterFramePresentated_RHIThread& OnDisplayClusterFramePresented_RHIThread() = 0;

	/** Notifies about cluster node failure. Called from AnyThread. **/
	DECLARE_EVENT_OneParam(IDisplayClusterCallbacks, FDisplayClusterFailoverNodeDown, const FString&);
	virtual FDisplayClusterFailoverNodeDown& OnDisplayClusterFailoverNodeDown() = 0;

	/** Notifies about new primary node. Called from AnyThread. **/
	DECLARE_EVENT_OneParam(IDisplayClusterCallbacks, FDisplayClusterFailoverPrimaryNodeChanged, const FString&);
	virtual FDisplayClusterFailoverPrimaryNodeChanged& OnDisplayClusterFailoverPrimaryNodeChanged() = 0;

	/** Called after tonemap pass (before applying OCIO transformation) **/
	DECLARE_EVENT_FiveParams(IDisplayClusterCallbacks, FDisplayClusterPostTonemapPass_RenderThread, FRDGBuilder&, const IDisplayClusterViewportProxy*, const FSceneView&, const FPostProcessMaterialInputs&, const uint32);
	virtual FDisplayClusterPostTonemapPass_RenderThread& OnDisplayClusterPostTonemapPass_RenderThread() = 0;

	/** Called once the ViewFamily of this viewport is rendered **/
	DECLARE_EVENT_ThreeParams(IDisplayClusterCallbacks, FDisplayClusterPostRenderViewFamily_RenderThread, FRDGBuilder&, const FSceneViewFamily&, const IDisplayClusterViewportProxy*);
	virtual FDisplayClusterPostRenderViewFamily_RenderThread& OnDisplayClusterPostRenderViewFamily_RenderThread() = 0;

	/** Called once before warping all available viewports **/
	DECLARE_EVENT_TwoParams(IDisplayClusterCallbacks, FDisplayClusterPreWarp_RenderThread, FRHICommandListImmediate&, const IDisplayClusterViewportManagerProxy*);
	virtual FDisplayClusterPreWarp_RenderThread& OnDisplayClusterPreWarp_RenderThread() = 0;

	/** Called before warping a specific viewport **/
	DECLARE_EVENT_TwoParams(IDisplayClusterCallbacks, FDisplayClusterPreWarpViewport_RenderThread, FRHICommandListImmediate&, const IDisplayClusterViewportProxy*);
	virtual FDisplayClusterPreWarpViewport_RenderThread& OnDisplayClusterPreWarpViewport_RenderThread() = 0;

	/** Called once after warping all the viewports **/
	DECLARE_EVENT_TwoParams(IDisplayClusterCallbacks, FDisplayClusterPostWarp_RenderThread, FRHICommandListImmediate&, const IDisplayClusterViewportManagerProxy*);
	virtual FDisplayClusterPostWarp_RenderThread& OnDisplayClusterPostWarp_RenderThread() = 0;

	/** Called after warping a specific viewport **/
	DECLARE_EVENT_TwoParams(IDisplayClusterCallbacks, FDisplayClusterPostWarpViewport_RenderThread, FRHICommandListImmediate&, const IDisplayClusterViewportProxy*);
	virtual FDisplayClusterPostWarpViewport_RenderThread& OnDisplayClusterPostWarpViewport_RenderThread() = 0;

	/** Called after inter-GPU synchronization **/
	DECLARE_EVENT_ThreeParams(IDisplayClusterCallbacks, FDisplayClusterPostCrossGpuTransfer_RenderThread, FRHICommandListImmediate&, const IDisplayClusterViewportManagerProxy*, FViewport*);
	virtual FDisplayClusterPostCrossGpuTransfer_RenderThread& OnDisplayClusterPostCrossGpuTransfer_RenderThread() = 0;

	/** Called to process passthrough media capture **/
	DECLARE_EVENT_TwoParams(IDisplayClusterCallbacks, FDisplayClusterPassthroughMediaCapture_RenderThread, FRHICommandListImmediate&, const IDisplayClusterViewportManagerProxy*);
	virtual FDisplayClusterPassthroughMediaCapture_RenderThread& OnDisplayClusterPassthroughMediaCapture_RenderThread() = 0;

	/** Called to process passthrough media input **/
	DECLARE_EVENT_TwoParams(IDisplayClusterCallbacks, FDisplayClusterPassthroughMediaInput_RenderThread, FRHICommandListImmediate&, const IDisplayClusterViewportManagerProxy*);
	virtual FDisplayClusterPassthroughMediaInput_RenderThread& OnDisplayClusterPassthroughMediaInput_RenderThread() = 0;

	/** Called to let the artificial latency subsystem do its job **/
	DECLARE_EVENT_ThreeParams(IDisplayClusterCallbacks, FDisplayClusterProcessLatency_RenderThread, FRHICommandListImmediate&, const IDisplayClusterViewportManagerProxy*, FViewport*);
	virtual FDisplayClusterProcessLatency_RenderThread& OnDisplayClusterProcessLatency_RenderThread() = 0;

	/** Called for every viewport that has configured content override **/
	DECLARE_EVENT_TwoParams(IDisplayClusterCallbacks, FDisplayClusterPostResolveOverridden_RenderThread, FRHICommandListImmediate&, const IDisplayClusterViewportProxy*);
	virtual FDisplayClusterPostResolveOverridden_RenderThread& OnDisplayClusterPostResolveOverridden_RenderThread() = 0;

	/** Called in the end of the nD rendering chain, right before updating the backbuffer **/
	DECLARE_EVENT_ThreeParams(IDisplayClusterCallbacks, FDisplayClusterPostFrameRender_RenderThread, FRHICommandListImmediate&, const IDisplayClusterViewportManagerProxy*, FViewport*);
	virtual FDisplayClusterPostFrameRender_RenderThread& OnDisplayClusterPostFrameRender_RenderThread() = 0;

	/** (deprecated) Called after backbuffer update **/
	DECLARE_EVENT_ThreeParams(IDisplayClusterCallbacks, FDisplayClusterPostBackbufferUpdate_RenderThread, FRHICommandListImmediate&, const IDisplayClusterViewportManagerProxy*, FViewport*);
	UE_DEPRECATED(5.2, "This event getter has been deprecated. Please use OnDisplayClusterPostBackbufferUpdated_RenderThread()")
	virtual FDisplayClusterPostBackbufferUpdate_RenderThread& OnDisplayClusterPostBackbufferUpdate_RenderThread() = 0;

	/** Called after backbuffer update **/
	DECLARE_EVENT_TwoParams(IDisplayClusterCallbacks, FDisplayClusterPostBackbufferUpdated_RenderThread, FRHICommandListImmediate&, FViewport*);
	virtual FDisplayClusterPostBackbufferUpdated_RenderThread& OnDisplayClusterPostBackbufferUpdated_RenderThread() = 0;

	/** Called before applying ICVFX shaders **/
	DECLARE_EVENT_FourParams(IDisplayClusterCallbacks, FDisplayClusterPreProcessIcvfx_RenderThread, FRHICommandListImmediate&, const IDisplayClusterViewportProxy*, FDisplayClusterShaderParameters_WarpBlend&, FDisplayClusterShaderParameters_ICVFX&);
	virtual FDisplayClusterPreProcessIcvfx_RenderThread& OnDisplayClusterPreProcessIcvfx_RenderThread() = 0;

	/** The viewport can be used by external media, which affects the internal logic of the viewport.
	* Media must use this callback and return their own media states for this viewport.
	*
	* @param InViewport - this viewport wants to know its media state.
	* @param InOutMediaState - the delegate function should raise the desired media state flags to this variable.
	*/
	DECLARE_MULTICAST_DELEGATE_TwoParams(FDisplayClusterUpdateViewportMediaState, IDisplayClusterViewport*, EDisplayClusterViewportMediaState&);
	virtual FDisplayClusterUpdateViewportMediaState& OnDisplayClusterUpdateViewportMediaState() = 0;
};
