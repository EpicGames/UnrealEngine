// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class FArchive;


/**
 * nDisplay serialization interface
 */
class IDisplayClusterSerializable
{
public:

	virtual ~IDisplayClusterSerializable() = default;

public:

	/**
	 * Serialize / deserialize method
	 * 
	 * Like the common UE serialization approach, this one is used for both serialization and
	 * deserialization. To avoid any method ambiguity issues with UObject::Serialize, this one
	 * has the 'DC' postfix. For example, UObject subobjects that also inherit from
	 * IDisplayClusterSerializable, can safely go like this:
	 * 
	 * virtual void SomeUObject::Serialize(FArchive& Ar) override
	 * {
	 *    // Serialization implementation
	 * }
	 * 
	 * virtual void SomeUObject::SerializeDC(FArchive& Ar) override
	 * {
	 *    Serialize(Ar);
	 * }
	 * 
	 * Or vice versa, call 'SerializeDC' from 'Serialize'.
	 */
	virtual void SerializeDC(FArchive& Ar) = 0;
};
