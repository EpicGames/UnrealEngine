// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * DEPRECATED
 * 
 * This interface is deprecated and no longer referenced. Its methods are never invoked and will be removed soon.
 * Refer to the IDisplayClusterSerializable interface, which provides common UE binary serialization.
 */
class IDisplayClusterStringSerializable
{
public:

	virtual ~IDisplayClusterStringSerializable() = default;

public:

	UE_DEPRECATED(5.8, "This method is deprecated and no longer used. Refer to IDisplayClusterSerializable instead.")
	virtual FString SerializeToString() const
	{
		return FString();
	}

	UE_DEPRECATED(5.8, "This method is deprecated and no longer used. Refer to IDisplayClusterSerializable instead.")
	virtual bool DeserializeFromString(const FString& ar)
	{
		return false;
	}
};
