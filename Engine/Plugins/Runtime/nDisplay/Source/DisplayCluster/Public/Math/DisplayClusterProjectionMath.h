// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/** Math utilities for nDisplay projection and frustum calculations. */
struct FDisplayClusterProjectionMath
{
	/**
	 * Calculates the ViewOffset for the eye from the view location.
	 * @param InViewRotation           Rotation of the view; used to derive the eye direction.
	 * @param StereoEyeOffsetDistance  Distance to the eye from the midpoint between the eyes.
	 * @return  Offset vector from the original ViewLocation to the eye.
	 */
	static inline FVector GetStereoEyeOffset(const FRotator& InViewRotation, const float StereoEyeOffsetDistance)
	{
		return InViewRotation.Quaternion().RotateVector(FVector(0.0f, StereoEyeOffsetDistance, 0.0f));
	}

	/** Ensures the frustum range [InOutValue0, InOutValue1] is at least 1 degree wide at the given near distance. */
	static inline void GetNonZeroFrustumRange(double& InOutValue0, double& InOutValue1, double n)
	{
		static const double MinHalfFOVRangeRad = FMath::DegreesToRadians(0.5f);
		static const double MinRangeBase = FMath::Tan(MinHalfFOVRangeRad * 2);

		const double MinRangeValue = n * MinRangeBase;
		if ((InOutValue1 - InOutValue0) < MinRangeValue)
		{
			// Get minimal values from center of range
			const double CenterRad = (FMath::Atan(InOutValue0 / n) + FMath::Atan(InOutValue1 / n)) * 0.5;
			InOutValue0 = n * FMath::Tan(CenterRad - MinHalfFOVRangeRad);
			InOutValue1 = n * FMath::Tan(CenterRad + MinHalfFOVRangeRad);
		}
	}

	/** Converts a frustum angle in degrees to a signed extent on the near plane. */
	template <typename T>
	static T DegreesToNearPlaneExtent(const T AngleDeg, const T ZNear)
	{
		return ZNear * FMath::Tan(FMath::DegreesToRadians(AngleDeg));
	}

	/** Converts a signed extent on the near plane to a frustum angle in degrees. */
	template <typename T>
	static T NearPlaneExtentToDegrees(const T NearPlaneExtent, const T ZNear)
	{
		return ZNear != T(0)
			? FMath::RadiansToDegrees(FMath::Atan(NearPlaneExtent / ZNear))
			: T(0);
	}

	/** Returns true if matrices A and B are nearly equal within the given Tolerance. */
	template <typename T>
	static bool MatricesNearlyEqual(const UE::Math::TMatrix<T>& A, const UE::Math::TMatrix<T>& B, float Tolerance = KINDA_SMALL_NUMBER)
	{
		for (int32 R = 0; R < 4; ++R)
		{
			for (int32 C = 0; C < 4; ++C)
			{
				if (!FMath::IsNearlyEqual(A.M[R][C], B.M[R][C], Tolerance))
				{
					return false;
				}
			}
		}

		return true;
	}

	/**
	 * Builds a perspective projection matrix from asymmetric near-plane extents (off-axis frustum).
	 * Inputs are signed distances from the eye to each frustum edge, measured at the near plane.
	 * The result uses Unreal's Z-inverted left-handed convention.
	 *
	 * @param InLeft    Left edge extent at the near plane (typically negative).
	 * @param InRight   Right edge extent at the near plane (typically positive).
	 * @param InTop     Top edge extent at the near plane (typically positive).
	 * @param InBottom  Bottom edge extent at the near plane (typically negative).
	 * @param InZNear   Near clip plane distance (must be > 0).
	 * @param InZFar    Far clip plane distance. Pass InZNear to use an unlimited far plane.
	 */
	template <typename T>
	static UE::Math::TMatrix<T> MakeAsymmetricProjectionMatrix(const T InLeft, const T InRight, const T InTop, const T InBottom, const T InZNear, const T InZFar)
	{
		const T mx = static_cast<T>(2) * InZNear / (InRight - InLeft);
		const T my = static_cast<T>(2) * InZNear / (InTop - InBottom);

		const T ma = -(InRight + InLeft) / (InRight - InLeft);
		const T mb = -(InTop + InBottom) / (InTop - InBottom);

		// Support unlimited far plane (InZFar == InZNear)
		const T mc = (InZFar == InZNear) ? (static_cast<T>(1) - Z_PRECISION) : (InZFar / (InZFar - InZNear));
		const T md = (InZFar == InZNear) ? (-InZNear * (static_cast<T>(1) - Z_PRECISION)) : (-(InZFar * InZNear) / (InZFar - InZNear));

		const T me = static_cast<T>(1);

		// Normal LHS
		const UE::Math::TMatrix<T> ProjectionMatrix = UE::Math::TMatrix<T>(
			FPlane(mx, 0,  0,  0),
			FPlane(0,  my, 0,  0),
			FPlane(ma, mb, mc, me),
			FPlane(0,  0,  md, 0));

		// Invert Z-axis (UE uses Z-inverted LHS)
		static const UE::Math::TMatrix<T> FlipZ = UE::Math::TMatrix<T>(
			FPlane(1, 0,  0, 0),
			FPlane(0, 1,  0, 0),
			FPlane(0, 0, -1, 0),
			FPlane(0, 0,  1, 1));

		return ProjectionMatrix * FlipZ;
	}

	/**
	 * Builds a perspective projection matrix from asymmetric near-plane extents (off-axis frustum).
	 * @param InFrustumExtents  Near-plane extents as (Left, Right, Top, Bottom).
	 * @param InClipPlanes      Clip distances as (ZNear, ZFar). Pass ZNear in both components for an unlimited far plane.
	 */
	template <typename T>
	static UE::Math::TMatrix<T> MakeAsymmetricProjectionMatrix(const UE::Math::TVector4<T>& InFrustumExtents, const UE::Math::TVector2<T>& InClipPlanes)
	{
		return MakeAsymmetricProjectionMatrix(InFrustumExtents.X, InFrustumExtents.Y, InFrustumExtents.Z, InFrustumExtents.W, InClipPlanes.X, InClipPlanes.Y);
	}
};
