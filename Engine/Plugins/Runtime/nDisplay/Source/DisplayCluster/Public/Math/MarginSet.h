// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Misc/AssertionMacros.h"
#include "Math/IntPoint.h"
#include "Math/UnrealMathUtility.h"
#include "Math/IntRect.h"

namespace UE::Math
{

/**
* Enumerates the sides of a margin set by index.
* Used for indexed access to individual sides (Left, Right, Top, Bottom).
*/
enum class EMarginSetSideIndex : uint8
{
	Left = 0,
	Right = 1,
	Top = 2,
	Bottom = 3,
};

/**
* TMarginSet is a template struct that stores a set of four directional values:
* Left, Right, Top, and Bottom.
*/
template<typename T>
struct alignas(16) TMarginSet
{
public:
	/*
	*    +--------------------------+
	*    |           Top            |
	*    |        +-------+         |
	*    |   Left |       |  Right  |
	*    |        +-------+         |
	*    |         Bottom           |
	*    +--------------------------+
	*/
	T Left;
	T Right;
	T Top;
	T Bottom;

private:
	static constexpr int32 NumSides = 4;

public:
	/**
	* Default constructor.
	*
	* The default margin size is zero on all four sides.
	*/
	TMarginSet()
		: Left(T(0))
		, Right(T(0))
		, Top(T(0))
		, Bottom(T(0))
	{ }

	/**
	 * Constructs a floating-point TMarginSet from a single arithmetic value U.
	 *
	 * Enabled only when T is floating-point and U is a built-in numeric type.
	 * Conversion is explicit if U is not implicitly convertible to T.
	 */
	template <typename U>
	requires (std::is_floating_point_v<T> && std::is_arithmetic_v<std::remove_cvref_t<U>>)
	explicit (!std::is_convertible_v<U, T>)
	TMarginSet(const U InAll)
		: Left(static_cast<T>(InAll))
		, Right(static_cast<T>(InAll))
		, Top(static_cast<T>(InAll))
		, Bottom(static_cast<T>(InAll))
	{ }

	/**
	 * Constructs an integral TMarginSet from a single integral value U.
	 *
	 * Enabled only when both T and U are integral types.
	 * Conversion is explicit if U is not implicitly convertible to T.
	 */
	template <typename IntType>
	requires (std::is_integral_v<T> && std::is_integral_v<IntType>)
	explicit(!std::is_convertible_v<IntType, T>)
	TMarginSet(const IntType InAll)
		: Left(static_cast<T>(InAll))
		, Right(static_cast<T>(InAll))
		, Top(static_cast<T>(InAll))
		, Bottom(static_cast<T>(InAll))
	{ }

	/**
	 * Constructs a floating-point TMarginSet from four values of type U.
	 *
	 * Enabled only when T is floating-point and U is a built-in numeric type.
	 * Conversion is explicit if U is not implicitly convertible to T.
	 */
	template <typename U>
	requires (std::is_floating_point_v<T> && std::is_arithmetic_v<std::remove_cvref_t<U>>)
	explicit (!std::is_convertible_v<U, T>)
	TMarginSet(
		const U InLeft,
		const U InRight,
		const U InTop,
		const U InBottom)
		: Left   (static_cast<T>(InLeft))
		, Right  (static_cast<T>(InRight))
		, Top    (static_cast<T>(InTop))
		, Bottom (static_cast<T>(InBottom))
	{ }

	/**
	 * Constructs an integral TMarginSet from four integral values of type U.
	 *
	 * Enabled only when both T and U are integral types.
	 * Conversion is explicit if U is not implicitly convertible to T.
	 */
	template <typename IntType>
	requires (std::is_integral_v<T> && std::is_integral_v<IntType>)
	explicit (!std::is_convertible_v<IntType, T>)
	TMarginSet(
		const IntType InLeft,
		const IntType InRight,
		const IntType InTop,
		const IntType InBottom)
		: Left   (static_cast<T>(InLeft))
		, Right  (static_cast<T>(InRight))
		, Top    (static_cast<T>(InTop))
		, Bottom (static_cast<T>(InBottom))
	{ }

	/**
	 * Constructs a floating-point TMarginSet from another TMarginSet<U>.
	 *
	 * Enabled only when T is floating-point and U is a built-in numeric type.
	 * Conversion is explicit if U is not implicitly convertible to T.
	 */
	template <typename U>
	requires (std::is_floating_point_v<T> && std::is_arithmetic_v<std::remove_cvref_t<U>>)
	explicit (!std::is_convertible_v<U, T>)
	TMarginSet(const TMarginSet<U>& Other)
		: Left   (static_cast<T>(Other.Left))
		, Right  (static_cast<T>(Other.Right))
		, Top    (static_cast<T>(Other.Top))
		, Bottom (static_cast<T>(Other.Bottom))
	{ }

	/**
	 * Constructs an integral TMarginSet from another TMarginSet<U>.
	 *
	 * Enabled only when both T and U are integral types.
	 * Conversion is explicit if U is not implicitly convertible to T.
	 */
	template <typename IntType>
	requires (std::is_integral_v<T> && std::is_integral_v<IntType>)
	explicit(!std::is_convertible_v<IntType, T>)
	TMarginSet(const TMarginSet<IntType>& Other)
		: Left   (static_cast<T>(Other.Left))
		, Right  (static_cast<T>(Other.Right))
		, Top    (static_cast<T>(Other.Top))
		, Bottom (static_cast<T>(Other.Bottom))
	{ }

	/**
	 * Constructs a floating-point TMarginSet from a TVector4<U>.
	 *
	 * Enabled only when T is floating-point and U is a built-in numeric type.
	 * Conversion is explicit if U is not implicitly convertible to T.
	 */
	template <typename U>
	requires (std::is_floating_point_v<T> && std::is_arithmetic_v<std::remove_cvref_t<U>>)
	explicit (!std::is_convertible_v<U, T>)
	TMarginSet(const TVector4<U>& Vec)
		: Left   (static_cast<T>(Vec.X))
		, Right  (static_cast<T>(Vec.Y))
		, Top    (static_cast<T>(Vec.Z))
		, Bottom (static_cast<T>(Vec.W))
	{ }

	/**
	 * Constructs a TMarginSet from a TIntVector4<U> with integral elements.
	 *
	 * Enabled only when U is an integral type.
	 * Conversion is explicit if U is not implicitly convertible to T.
	 */
	template <typename IntType>
	requires (std::is_integral_v<IntType>)
	explicit (!std::is_convertible_v<IntType, T>)
	TMarginSet(const TIntVector4<IntType>& Vec)
		: Left   (static_cast<T>(Vec.X))
		, Right  (static_cast<T>(Vec.Y))
		, Top    (static_cast<T>(Vec.Z))
		, Bottom (static_cast<T>(Vec.W))
	{ }

public:
	/**
	 * Converts a floating-point TMarginSet to TVector4<U>.
	 *
	 * Enabled only when T is floating-point.
	 * Marked [[nodiscard]] to warn if the result is ignored.
	 */
	template<typename U = T>
	requires (std::is_floating_point_v<U>)
	[[nodiscard]]
	inline operator TVector4<U>() const
	{
		return TVector4<U>(
			Left,
			Right,
			Top,
			Bottom);
	}

	/**
	 * Converts this integral TMarginSet<T> to a TIntVector4<U>.
	 *
	 * Enabled only when both T (this type) and U (defaults to T) are integral.
	 * [[nodiscard]] warns if the result is unused.
	 */
	template<typename IntType = T>
	requires (std::is_integral_v<IntType> && std::is_integral_v<T>)
	[[nodiscard]]
	inline operator TIntVector4<IntType>() const
	{
		return TIntVector4<IntType>(
			Left,
			Right,
			Top,
			Bottom);
	}

public:
	/**
	 * Assigns margins from a TVector4<U> to a floating-point TMarginSet<T>.
	 *
	 * Enabled only when:
	 * - T is floating-point
	 * - U is a built-in numeric type
	 * - U is convertible to T
	 */
	template <typename U>
	requires (std::is_floating_point_v<T> &&
		std::is_arithmetic_v<std::remove_cvref_t<U>> &&
		std::is_convertible_v<U, T>)
	inline TMarginSet& AssignMargins(const TVector4<U>& InVector)
	{
		Left   = static_cast<T>(InVector.X);
		Right  = static_cast<T>(InVector.Y);
		Top    = static_cast<T>(InVector.Z);
		Bottom = static_cast<T>(InVector.W);

		return *this;
	}

	/**
	 * Assigns margins from a TIntVector4<IntType> to this TMarginSet<T>.
	 *
	 * Enabled only when:
	 * - IntType is an integral type
	 * - IntType is convertible to T
	 *
	 * @param InVector  The TIntVector4<IntType> containing the new margin values.
	 * @return          Reference to this TMarginSet<T> after assignment.
	 */
	template <typename IntType>
	requires (std::is_integral_v<IntType> && std::is_convertible_v<IntType, T>)
	inline TMarginSet& AssignMargins(const TIntVector4<IntType>& InVector)
	{
		Left   = static_cast<T>(InVector.X);
		Right  = static_cast<T>(InVector.Y);
		Top    = static_cast<T>(InVector.Z);
		Bottom = static_cast<T>(InVector.W);

		return *this;
	}

	/**
	 * Assigns margins from another TMarginSet<U> to a floating-point TMarginSet<T>.
	 *
	 * Enabled only when:
	 * - T is floating-point
	 * - U is a built-in numeric type
	 * - U is convertible to T
	 */
	template <typename U>
	requires (std::is_floating_point_v<T> &&
		std::is_arithmetic_v<std::remove_cvref_t<U>> &&
		std::is_convertible_v<U, T>)
	inline TMarginSet<T>& AssignMargins(const TMarginSet<U>& Other)
	{
		Left   = static_cast<T>(Other.Left);
		Right  = static_cast<T>(Other.Right);
		Top    = static_cast<T>(Other.Top);
		Bottom = static_cast<T>(Other.Bottom);

		return *this;
	}

	/**
	 * Assigns margins from another TMarginSet<IntType> to an integral TMarginSet<T>.
	 *
	 * Enabled only when:
	 * - T and IntType are integral types
	 * - IntType is convertible to T
	 */
	template <typename IntType>
	requires (std::is_integral_v<T> &&
		std::is_integral_v<IntType> &&
		std::is_convertible_v<IntType, T>)
	inline TMarginSet<T>& AssignMargins(const TMarginSet<IntType>& Other)
	{
		Left   = static_cast<T>(Other.Left);
		Right  = static_cast<T>(Other.Right);
		Top    = static_cast<T>(Other.Top);
		Bottom = static_cast<T>(Other.Bottom);

		return *this;
	}

	/**
	 * Assigns margins from four numeric values of type U to a floating-point TMarginSet<T>.
	 *
	 * Enabled only when:
	 * - T is floating-point
	 * - U is a built-in numeric type
	 * - U is convertible to T
	 */
	template <typename U>
	requires (std::is_floating_point_v<T> &&
		std::is_arithmetic_v<std::remove_cvref_t<U>> &&
		std::is_convertible_v<U, T>)
	inline TMarginSet<T>& AssignMargins(const U InLeft, const U InRight, const  U InTop, const U InBottom)
	{
		Left   = static_cast<T>(InLeft);
		Right  = static_cast<T>(InRight);
		Top    = static_cast<T>(InTop);
		Bottom = static_cast<T>(InBottom);

		return *this;
	}

	/**
	 * Assigns margins from four integral values of type IntType to an integral TMarginSet<T>.
	 *
	 * Enabled only when:
	 * - T and IntType are integral types
	 * - IntType is convertible to T
	 */
	template <typename IntType>
	requires (std::is_integral_v<T> &&
		std::is_integral_v<IntType> &&
		std::is_convertible_v<IntType, T>)
	inline TMarginSet<T>& AssignMargins(
		const IntType InLeft,
		const IntType InRight,
		const IntType InTop,
		const IntType InBottom)
	{
		Left   = static_cast<T>(InLeft);
		Right  = static_cast<T>(InRight);
		Top    = static_cast<T>(InTop);
		Bottom = static_cast<T>(InBottom);

		return *this;
	}

public:
	/**
	 * Multiplies each margin of a TMarginSet<T> by an integral scalar value.
	 *
	 * @param InScale  Integral multiplier applied to all margins.
	 * @return         A TMarginSet<T> where each margin is scaled by InScale.
	 */
	template <typename IntType>
	requires (std::is_integral_v<IntType> &&
		std::is_convertible_v<IntType, T>)
	[[nodiscard]]
	inline TMarginSet<T> operator*(const IntType InScale) const
	{
		const T Scale = static_cast<T>(InScale);
		return TMarginSet<T>(
			Left   * Scale,
			Right  * Scale,
			Top    * Scale,
			Bottom * Scale
		);
	}

	/**
	 * Multiplies each margin of a floating-point TMarginSet<T> by a scalar value.
	 *
	 * @param InScale  Scalar multiplier applied to all margins.
	 * @return         A TMarginSet<T> where each margin is scaled by InScale.
	 */
	template <typename U>
	requires (std::is_floating_point_v<T> &&
		!std::is_integral_v<U> && 
		std::is_arithmetic_v<std::remove_cvref_t<U>>)
	[[nodiscard]]
	inline TMarginSet<T> operator*(const U InScale) const
	{
		const T Scale = static_cast<T>(InScale);
		return TMarginSet<T>(
			Left   * Scale,
			Right  * Scale,
			Top    * Scale,
			Bottom * Scale
		);
	}

	/**
	 * Multiplies each margin of a TMarginSet<T> by the corresponding axis of a TIntPoint<IntType>.
	 * - Left and Right are multiplied by Scale.X
	 * - Top and Bottom are multiplied by Scale.Y
	 *
	 * @param Scale   The TIntPoint<IntType> scale factors (X for horizontal, Y for vertical).
	 * @return        A TMarginSet<T> where each margin is multiplied by the corresponding factor.
	 */
	template <typename IntType>
	requires (std::is_integral_v<IntType>&&
		std::is_convertible_v<IntType, T>)
	[[nodiscard]]
	inline TMarginSet<T> operator*(const TIntPoint<IntType>& Scale) const
	{
		const T ScaleX = static_cast<T>(Scale.X);
		const T ScaleY = static_cast<T>(Scale.Y);

		return TMarginSet<T>(
			Left   * ScaleX,
			Right  * ScaleX,
			Top    * ScaleY,
			Bottom * ScaleY
		);
	}

	/**
	 * Multiplies each margin of a TMarginSet<T> by the corresponding axis of a TIntVector2<IntType>.
	 * - Left and Right are multiplied by Scale.X
	 * - Top and Bottom are multiplied by Scale.Y
	 *
	 * @param Scale   The TIntVector2<IntType> scale factors (X for horizontal, Y for vertical).
	 * @return        A TMarginSet<T> where each margin is scaled by the corresponding factor.
	 */
	template <typename IntType>
	requires (std::is_integral_v<IntType> &&
		std::is_convertible_v<IntType, T>)
	[[nodiscard]]
	inline TMarginSet<T> operator*(const TIntVector2<IntType>& Scale) const
	{
		const T ScaleX = static_cast<T>(Scale.X);
		const T ScaleY = static_cast<T>(Scale.Y);

		return TMarginSet<T>(
			Left   * ScaleX,
			Right  * ScaleX,
			Top    * ScaleY,
			Bottom * ScaleY
		);
	}

	/**
	 * Multiplies each margin of a TMarginSet<T> by the corresponding axis of a TVector2<U>.
	 * - Left and Right are multiplied by Scale.X
	 * - Top and Bottom are multiplied by Scale.Y
	 *
	 * @param Scale   The TVector2<U> scale factors (X for horizontal, Y for vertical).
	 * @return        A TMarginSet<T> where each margin is scaled by the corresponding factor.
	 */
	template <typename U>
	requires (std::is_floating_point_v<T> &&
		std::is_floating_point_v<U> &&
		std::is_convertible_v<U, T>)
	[[nodiscard]]
	inline TMarginSet<T> operator*(const TVector2<U>& Scale) const
	{
		const T ScaleX = static_cast<T>(Scale.X);
		const T ScaleY = static_cast<T>(Scale.Y);

		return TMarginSet<T>(
			Left   * ScaleX,
			Right  * ScaleX,
			Top    * ScaleY,
			Bottom * ScaleY
		);
	}

public:
	/**
	 * Divides each margin of a floating-point TMarginSet<T> by a scalar value.
	 *
	 * @param InDivisor  Scalar divisor applied to all margins.
	 * @return           A TMarginSet<T> where each margin is divided by InDivisor.
	 */
	template <typename U>
	requires (std::is_floating_point_v<T> &&
		std::is_arithmetic_v<std::remove_cvref_t<U>>)
	[[nodiscard]]
	inline TMarginSet<T> operator/(const U InDivisor) const
	{
		const T Divisor = static_cast<T>(InDivisor);

		// Assumes caller ensures Divisor are nonzero
		check(Divisor);

		return UE::Math::TMarginSet<T>(
			Left   / Divisor,
			Right  / Divisor,
			Top    / Divisor,
			Bottom / Divisor
		);
	}
	
	/**
	 * Divides each margin of a TMarginSet<T> by the corresponding axis of a TIntPoint<IntType>.
	 * - Left and Right are divided by Divisor.X
	 * - Top and Bottom are divided by Divisor.Y
	 *
	 * @param Divisor  The TIntPoint<IntType> divisors (X for horizontal, Y for vertical).
	 * @return         A TMarginSet<T> where each margin is divided by the corresponding factor.
	 */
	template <typename IntType>
	requires (std::is_floating_point_v<T>&& 
		std::is_integral_v<IntType>&&
		std::is_convertible_v<IntType, T>)
	[[nodiscard]]
	inline TMarginSet<T> operator/(const TIntPoint<IntType>& Divisor) const
	{
		const T DivisorX = static_cast<T>(Divisor.X);
		const T DivisorY = static_cast<T>(Divisor.Y);

		// Assumes caller ensures DivisorX and DivisorY are nonzero
		check(DivisorX && DivisorY);

		return TMarginSet<T>(
			Left   / DivisorX,
			Right  / DivisorX,
			Top    / DivisorY,
			Bottom / DivisorY
		);
	}

	/**
	 * Divides each margin of a TMarginSet<T> by the corresponding axis of an TIntVector2<IntType>.
	 * - Left/Right are divided by Divisor.X
	 * - Top/Bottom are divided by Divisor.Y
	 *
	 * @param Divisor       The TIntVector2<IntType> divisor (X for horizontal, Y for vertical).
	 * @return              A TMarginSet<T> where each margin is divided by the corresponding divisor.
	 */
	template <typename IntType>
	requires (std::is_floating_point_v<T> && 
		std::is_integral_v<IntType> &&
		std::is_convertible_v<IntType, T>)
	[[nodiscard]]
	inline TMarginSet<T> operator/(const TIntVector2<IntType>& Divisor) const
	{
		const T DivisorX = static_cast<T>(Divisor.X);
		const T DivisorY = static_cast<T>(Divisor.Y);

		// Assumes caller ensures DivisorX and DivisorY are nonzero
		check(DivisorX && DivisorY);

		return TMarginSet<T>(
			Left   / DivisorX,
			Right  / DivisorX,
			Top    / DivisorY,
			Bottom / DivisorY
		);
	}

	/**
	 * Divides each margin of a TMarginSet<T> by the corresponding axis of an TVector2<U>.
	 * - Left/Right are divided by Divisor.X
	 * - Top/Bottom are divided by Divisor.Y
	 *
	 * @param Divisor       The TVector2<U> divisor (X for horizontal, Y for vertical).
	 * @return              A TMarginSet<T> where each margin is divided by the corresponding divisor.
	 */
	template <typename U>
	requires (std::is_floating_point_v<T>&&
		std::is_floating_point_v<U>&&
		std::is_convertible_v<U, T>)
	[[nodiscard]]
	inline TMarginSet<T> operator/(const TVector2<U>& Divisor)
	{
		const T DivisorX = static_cast<T>(Divisor.X);
		const T DivisorY = static_cast<T>(Divisor.Y);

		// Assumes caller ensures Divisor.X and Divisor.Y are nonzero
		check(DivisorX && DivisorY);

		return UE::Math::TMarginSet<T>(
			Left   / DivisorX,
			Right  / DivisorX,
			Top    / DivisorY,
			Bottom / DivisorY
		);
	}

public:
	/**
	 * Expands the given rectangle in-place by the values in this margin set.
	 *
	 * Each margin value increases the corresponding edge outward:
	 * - Left decreases Min.X
	 * - Right increases Max.X
	 * - Top decreases Min.Y
	 * - Bottom increases Max.Y
	 *
	 * This operation can result in negative-sized rectangles if large margins are used.
	 *
	 * @param InOutRect [in,out] The rectangle to expand. Modified in place.
	 */
	template <typename IntType>
	requires (std::is_integral_v<T> &&
		std::is_integral_v<IntType> &&
		std::is_convertible_v<IntType, T>)
	void ApplyExpandToRect(TIntRect<IntType>& InOutRect) const
	{
		InOutRect.Min.X -= Left;
		InOutRect.Max.X += Right;
		InOutRect.Min.Y -= Top;
		InOutRect.Max.Y += Bottom;

		// Clamp to prevent negative size (Min cannot exceed Max).
		InOutRect.Min.X = FMath::Min(InOutRect.Min.X, InOutRect.Max.X);
		InOutRect.Min.Y = FMath::Min(InOutRect.Min.Y, InOutRect.Max.Y);
	}

	/**
	 * Shrinks the given rectangle in-place by the values in this margin set.
	 * Each margin value moves the corresponding edge inward:
	 *   - Left   increases Min.X
	 *   - Right  decreases Max.X
	 *   - Top    increases Min.Y
	 *   - Bottom decreases Max.Y
	 *
	 * If the total margin for any axis exceeds the rectangle's size, the rectangle will collapse to a line or point.
	 * Margin values are clamped as needed to prevent the rectangle from inverting or collapsing to negative size.
	 *
	 * @param InOutRect [in,out] The rectangle to be contracted. Modified in place.
	 */
	template <typename IntType>
		requires (std::is_integral_v<T>&&
		std::is_integral_v<IntType>&&
		std::is_convertible_v<IntType, T>)
	void ApplyClampedInsetToRect(TIntRect<IntType>& InOutRect) const
	{
		const FIntPoint RectSize = InOutRect.Size();

		// Clamp margins to prevent invalid rectangle
		T ClampedLeft   = FMath::Clamp(Left,   0, RectSize.X);
		T ClampedRight  = FMath::Clamp(Right,  0, RectSize.X - ClampedLeft);
		T ClampedTop    = FMath::Clamp(Top,    0, RectSize.Y);
		T ClampedBottom = FMath::Clamp(Bottom, 0, RectSize.Y - ClampedTop);

		// Shrink rect by clamped margins
		InOutRect.Min.X += ClampedLeft;
		InOutRect.Max.X -= ClampedRight;
		InOutRect.Min.Y += ClampedTop;
		InOutRect.Max.Y -= ClampedBottom;

		// Ensure Min does not surpass Max (should already be safe)
		InOutRect.Min.X = FMath::Min(InOutRect.Min.X, InOutRect.Max.X);
		InOutRect.Min.Y = FMath::Min(InOutRect.Min.Y, InOutRect.Max.Y);
	}

public:
	/**
	 * Provides indexed access to a margin side value.
	 *
	 * @param SideIndex  Enum value specifying which margin side to access.
	 * @return           Reference to the corresponding side value.
	 */
	[[nodiscard]] inline T& operator[](EMarginSetSideIndex SideIndex)
	{
		switch (SideIndex)
		{
		case EMarginSetSideIndex::Left:   return Left;
		case EMarginSetSideIndex::Right:  return Right;
		case EMarginSetSideIndex::Top:    return Top;
		case EMarginSetSideIndex::Bottom: return Bottom;
		default:			
			break;
		}

		// Unreachable
		checkNoEntry();

		return Left;
	}

	/**
	 * Provides  read-only indexed access to a margin side value.
	 *
	 * @param SideIndex  Enum value specifying which margin side to access.
	 * @return           Const reference to the corresponding side value.
	 */
	[[nodiscard]] inline const T& operator[](EMarginSetSideIndex SideIndex) const
	{
		switch (SideIndex)
		{
		case EMarginSetSideIndex::Left:   return Left;
		case EMarginSetSideIndex::Right:  return Right;
		case EMarginSetSideIndex::Top:    return Top;
		case EMarginSetSideIndex::Bottom: return Bottom;
		default:
			break;
		}

		// Unreachable
		checkNoEntry();

		return Left;
	}

	/**
	 * Provides indexed access to a margin side value by numeric index.
	 *
	 * @param Index  Zero-based index of the side to access:
	 *               0 = Left, 1 = Right, 2 = Top, 3 = Bottom.
	 * @return       Reference to the corresponding side value.
	 */
	[[nodiscard]] inline T& operator[](int32 Index)
	{	
		check(Index >= 0 && Index < NumSides);
		const EMarginSetSideIndex SideIndex = static_cast<EMarginSetSideIndex>(Index);

		return operator[](SideIndex);
	}

	/**
	 * Provides  read-only indexed access to a margin side value by numeric index.
	 *
	 * @param Index  Zero-based index of the side to access:
	 *               0 = Left, 1 = Right, 2 = Top, 3 = Bottom.
	 * @return       Const reference to the corresponding side value.
	 */
	[[nodiscard]] inline const T& operator[](int32 Index) const
	{
		check(Index >= 0 && Index < NumSides);
		const EMarginSetSideIndex SideIndex = static_cast<EMarginSetSideIndex>(Index);

		return operator[](SideIndex);
	}

	/** Equality operator */
	[[nodiscard]] inline bool operator==(const TMarginSet& Other) const
	{
		return Left == Other.Left && Right == Other.Right &&
			Top == Other.Top && Bottom == Other.Bottom;
	}

	/** Inequality operator */
	[[nodiscard]] inline bool operator!=(const TMarginSet& Other) const
	{
		return !(*this == Other);
	}

public:
	/**
	* Returns a copy of this margin set where all margins floors to the nearest lower int32.
	* 
	* @return A TMarginSet<int32> with each value floored.
	*/
	[[nodiscard]] inline TMarginSet<int32> GetFloorToInt() const
	{
		return TMarginSet<int32>(
			FMath::FloorToInt32(Left),
			FMath::FloorToInt32(Right),
			FMath::FloorToInt32(Top),
			FMath::FloorToInt32(Bottom)
		);
	}

	/**
	* Returns a copy of this margin set where all margins rounds to the nearest int32.
	* 
	* @return A TMarginSet<int32> with all values rounded.
	*/
	[[nodiscard]] inline TMarginSet<int32> GetRoundToInt() const
	{
		return TMarginSet<int32>(
			FMath::RoundToInt32(Left),
			FMath::RoundToInt32(Right),
			FMath::RoundToInt32(Top),
			FMath::RoundToInt32(Bottom)
		);
	}

	/**
	* Returns a copy of this margin set where all margins ceils to the nearest greater or equal int32.
	* 
	* @return A TMarginSet<int32> with each value ceiled.
	*/
	[[nodiscard]] inline TMarginSet<int32> GetCeilToInt() const
	{
		return TMarginSet<int32>(
			FMath::CeilToInt32(Left),
			FMath::CeilToInt32(Right),
			FMath::CeilToInt32(Top),
			FMath::CeilToInt32(Bottom)
		);
	}

public:
	/**
	* Returns a copy of this margin set where all margins clamped the specified [Min, Max] range.
	*
	* @param Min           The minimum allowed value for each margin.
	* @param Max           The maximum allowed value for each margin.
	* @return              A clamped TMarginSet with all values between Min and Max.
	*/
	[[nodiscard]] inline TMarginSet<T> GetClamped(const T Min, const T Max) const
	{
		return TMarginSet<T>(
			FMath::Clamp(Left,   Min, Max),
			FMath::Clamp(Right,  Min, Max),
			FMath::Clamp(Top,    Min, Max),
			FMath::Clamp(Bottom, Min, Max)
		);
	}

	/**
	 * Returns a copy of this margin set with each side individually clamped
	 * between the corresponding sides of the provided minimum and maximum margins.
	 *
	 * For each side:
	 *   - Left   = Clamp(Left,   MinMargins.Left,   MaxMargins.Left)
	 *   - Right  = Clamp(Right,  MinMargins.Right,  MaxMargins.Right)
	 *   - Top    = Clamp(Top,    MinMargins.Top,    MaxMargins.Top)
	 *   - Bottom = Clamp(Bottom, MinMargins.Bottom, MaxMargins.Bottom)
	 *
	 * @param MinMargins Margin set defining the minimum values per side.
	 * @param MaxMargins Margin set defining the maximum values per side.
	 * @return A new TMarginSet with each side clamped to its respective range.
	 */
	[[nodiscard]] inline TMarginSet<T> GetClampedPerSide(const TMarginSet<T>& MinMargins, const TMarginSet& MaxMargins) const
	{
		return TMarginSet<T>(
			FMath::Clamp(Left,   MinMargins.Left,   MaxMargins.Left),
			FMath::Clamp(Right,  MinMargins.Right,  MaxMargins.Right),
			FMath::Clamp(Top,    MinMargins.Top,    MaxMargins.Top),
			FMath::Clamp(Bottom, MinMargins.Bottom, MaxMargins.Bottom)
		);
	}

	/**
	* Returns true if all margins are zero.
	*/
	[[nodiscard]] inline bool IsZero() const
	{
		return Left == T(0)
			&& Right == T(0)
			&& Top == T(0)
			&& Bottom == T(0);
	}

	/**
	 * Returns the minimum value among all four margins (Left, Right, Top, Bottom).
	 *
	 * @return Smallest margin value.
	 */
	[[nodiscard]] inline T GetMin() const
	{
		const T Value1 = FMath::Min(Left, Right);
		const T Value2 = FMath::Min(Top, Bottom);

		return FMath::Min(Value1, Value2);
	}

	/**
	 * Returns the maximum value among all four margins (Left, Right, Top, Bottom).
	 *
	 * @return Largest margin value.
	 */
	[[nodiscard]] inline T GetMax() const
	{
		const T Value1 = FMath::Max(Left, Right);
		const T Value2 = FMath::Max(Top, Bottom);

		return FMath::Max(Value1, Value2);
	}

	/**
	 * Returns a copy of this margin set where all margins is the minimum of this margin and another.
	 *
	 * @param In  The margin set to compare against
	 * @return    A new TMarginSet with the smallest values for each side
	 */
	[[nodiscard]] inline TMarginSet<T> GetMinPerSide(const TMarginSet<T>& In) const
	{
		return TMarginSet<T>(
			FMath::Min(Left,   In.Left),   // Minimum of Left sides
			FMath::Min(Right,  In.Right),  // Minimum of Right sides
			FMath::Min(Top,    In.Top),    // Minimum of Top sides
			FMath::Min(Bottom, In.Bottom)  // Minimum of Bottom sides
		);
	}

	/**
	 * Returns a copy of this margin set where all margins is the maximum of this margin and another.
	 *
	 * @param In  The margin set to compare against
	 * @return    A new TMarginSet with the largest values for each side
	 */
	[[nodiscard]] inline TMarginSet<T> GetMaxPerSide(const TMarginSet<T>& In) const
	{
		return TMarginSet<T>(
			FMath::Max(Left,   In.Left),   // Maximum of Left sides
			FMath::Max(Right,  In.Right),  // Maximum of Right sides
			FMath::Max(Top,    In.Top),    // Maximum of Top sides
			FMath::Max(Bottom, In.Bottom)  // Maximum of Bottom sides
		);
	}

	/**
	 * Returns a new margin set where each side is the maximum of:
	 * - the current value, or
	 * - the input value, if it is non-zero (i.e., treated as defined).
	 *
	 * This allows selectively overriding margins while skipping undefined (zero) inputs.
	 *
	 * @param In The input margin set, where zero values are treated as "undefined".
	 * @return A new margin set with per-side conditional maximums.
	 */
	[[nodiscard]] inline TMarginSet<T> GetMaxPerSideIfDefined(const TMarginSet<T>& In) const
	{
		return TMarginSet<T>(
			In.Left   != static_cast<T>(0) ? FMath::Max(Left,   In.Left)   : Left,
			In.Right  != static_cast<T>(0) ? FMath::Max(Right,  In.Right)  : Right,
			In.Top    != static_cast<T>(0) ? FMath::Max(Top,    In.Top)    : Top,
			In.Bottom != static_cast<T>(0) ? FMath::Max(Bottom, In.Bottom) : Bottom
		);
	}

	/**
	 * Returns a new margin set where each side is the minimum of:
	 * - the current value, or
	 * - the input value, if it is non-zero (i.e., treated as defined).
	 *
	 * This allows selectively overriding margins while skipping undefined (zero) inputs.
	 *
	 * @param In The input margin set, where zero values are treated as "undefined".
	 * @return A new margin set with per-side conditional minimum.
	 */
	[[nodiscard]] inline TMarginSet<T> GetMinPerSideIfDefined(const TMarginSet<T>& In) const
	{
		return TMarginSet<T>(
			In.Left   != static_cast<T>(0) ? FMath::Min(Left,   In.Left)   : Left,
			In.Right  != static_cast<T>(0) ? FMath::Min(Right,  In.Right)  : Right,
			In.Top    != static_cast<T>(0) ? FMath::Min(Top,    In.Top)    : Top,
			In.Bottom != static_cast<T>(0) ? FMath::Min(Bottom, In.Bottom) : Bottom
		);
	}
};

} // namespace UE::Math

UE_DECLARE_LWC_TYPE(MarginSet, 4);
using FIntMarginSet = UE::Math::TMarginSet<int32>;
