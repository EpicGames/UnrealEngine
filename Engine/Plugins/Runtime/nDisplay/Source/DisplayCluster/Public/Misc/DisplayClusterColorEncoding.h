// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/** Color encoding type.  */
enum class EDisplayClusterColorEncoding : uint8
{
	/** Linear color space */
	Linear = 0,

	/** Gamma encoding based on pow(In, Gamma). */
	Gamma,

	/** sRGB color space. */
	sRGB,

	/** ST2084 gamma with saturation (for special use case of nDisplay MediaIO). */
	MediaPQ,
};

/** Color premultiply type.  */
enum class EDisplayClusterColorPremultiply : uint8
{
	/** Color is not changed by alpha. */
	None = 0,

	/** Color is premultiplied by Alpha. */
	Premultiply,

	/** Color is premultiplied by (1-Alpha). */
	InvertPremultiply
};

/** Color encoding data. */
struct FDisplayClusterColorEncoding
{
	/** Constructors. */
	FDisplayClusterColorEncoding() = default;
	FDisplayClusterColorEncoding(
		const EDisplayClusterColorEncoding InColorEncoding,
		const EDisplayClusterColorPremultiply InPremultiply = EDisplayClusterColorPremultiply::None)
		: Encoding(InColorEncoding), Premultiply(InPremultiply) { }
	FDisplayClusterColorEncoding(
		float InGamma,
		const EDisplayClusterColorPremultiply InPremultiply = EDisplayClusterColorPremultiply::None)
		: Encoding(EDisplayClusterColorEncoding::Gamma) , GammaValue(InGamma), Premultiply(InPremultiply) { }

	/** Return equal encoding value. */
	EDisplayClusterColorEncoding GetEqualEncoding() const
	{
		// Linear == Gamma(1.f)
		if (Encoding == EDisplayClusterColorEncoding::Gamma && GammaValue == 1.f)
		{
			return EDisplayClusterColorEncoding::Linear;
		}
		// sRGB == Gamma(0.f) == Gamma(2.2f)
		else if (Encoding == EDisplayClusterColorEncoding::Gamma && (GammaValue <= 0.f || GammaValue == 2.2f))
		{
			return EDisplayClusterColorEncoding::sRGB; // Default gamma means sRGB
		}

		return Encoding;
	}

	/** Returns true if the color encodings are the same. */
	inline bool operator==(const FDisplayClusterColorEncoding& In) const
	{
		if (Premultiply != In.Premultiply)
		{
			return false;
		}

		return IsEqualsGammaEncoding(In);
	}

	/** Compare only gamma encodings. */
	inline bool IsEqualsGammaEncoding(const FDisplayClusterColorEncoding& In) const
	{
		if (Encoding == In.Encoding)
		{
			// Additional rule for `Gamma` value
			if (Encoding != EDisplayClusterColorEncoding::Gamma)
			{
				return true;
			}

			// EDisplayClusterColorEncoding::Gamma
			if (GammaValue == In.GammaValue)
			{
				return true;
			}
		}

		if (GetEqualEncoding() == In.GetEqualEncoding())
		{
			return true;
		}

		return false;
	}

public:
	/** Color space encoding type.  */
	EDisplayClusterColorEncoding Encoding = EDisplayClusterColorEncoding::Gamma;

	/** The gamma value for the `EDisplayClusterColorEncoding::Gamma`.
	* Zero value means default gamma.
	*/
	float GammaValue = 0.f;

	/** Color premultiply modifiers. */
	EDisplayClusterColorPremultiply Premultiply = EDisplayClusterColorPremultiply::None;
};
