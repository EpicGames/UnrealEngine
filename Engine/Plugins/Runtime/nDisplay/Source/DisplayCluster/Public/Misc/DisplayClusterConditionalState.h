// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Delegates/DelegateCombinations.h"


namespace UE::nDisplay::Core
{
	/**
	 * Default NoContext template parameter
	 */
	struct FConditionalState_NoData	{ };


	/**
	 * Conditional State (generic state machine)
	 * 
	 * Manages abstract states based on an abstract input.
	 * 
	 * [SomeInput] -> [ ? ] -> [SomeState]
	 *                  ^
	 *     user provided decision maker
	 * 
	 * Also calls back on every state change.
	 * 
	 * StateDataType:
	 * Any type representing a state. It must be a copyable and
	 * default-constructible type.
	 *
	 * InputDataType:
	 * The type of input data used to evaluate whether a state transition
	 * is required, and if so, which state should be transitioned to.
	 *
	 * ContextDataType:
	 * The type of context data. This represents local storage provided
	 * to the decision-making delegate on each evaluation. This data is never
	 * modified internally; its usage is entirely up to the user.
	 */
	template <typename StateDataType, typename InputDataType, typename ContextDataType = FConditionalState_NoData>
	class FConditionalState
	{
	public:

		/**
		 * Decision making delegate
		 * 
		 * Based on the current state and new input, decides which state to switch on, or remain on the old one
		 * 
		 * @param CurrentState - Current state
		 * @param NewInput - The input data that is affecting current state
		 * @return New state (may be unchanged)
		 */
		DECLARE_DELEGATE_RetVal_ThreeParams(StateDataType, FMakeDecisionDelegate, const StateDataType& /* CurrentState */, const InputDataType& /* NewInput */, ContextDataType& /*ContextData*/);
		FMakeDecisionDelegate OnMakeDecision;

		/**
		 * State change event
		 *
		 * Called every time the state is changed
		 *
		 * @param OldState - Previous state
		 * @param NewState - New state
		 */
		DECLARE_EVENT_TwoParams(FConditionalState, FOnStateChanged, const StateDataType& /* OldState */, const StateDataType& /* NewState */);
		FOnStateChanged OnStateChanged;

	public:

		FConditionalState()
			: CurrentState(StateDataType())
			, ContextData(ContextDataType())
		{ }

		FConditionalState(const StateDataType& InInitialState)
			: CurrentState(InInitialState)
			, ContextData(ContextDataType())
		{ }

		FConditionalState(const StateDataType& InInitialState, const ContextDataType& InContextData)
			: CurrentState(InInitialState)
			, ContextData(InContextData)
		{ }

		virtual ~FConditionalState() = default;

	public:

		/**
		 * Processes incoming input and evaluates whether the current state should change.
		 *
		 * @param NewInput - Input data used to evaluate a potential state transition
		 */
		void Evaluate(const InputDataType& NewInput)
		{
			StateDataType NewState = OnMakeDecision.IsBound()
				? OnMakeDecision.Execute(CurrentState, NewInput, ContextData)
				: CurrentState;

			if (CurrentState != NewState)
			{
				StateDataType OldState = CurrentState;
				CurrentState = NewState;
				OnStateChanged.Broadcast(OldState, NewState);
			}
		}

		/**
		 * Access to the current state
		 * 
		 * @return Current state
		 */
		const StateDataType& GetState() const
		{
			return CurrentState;
		}

	private:

		/** Tracks current state */
		StateDataType CurrentState;

		/** User context data */
		ContextDataType ContextData;
	};
}
