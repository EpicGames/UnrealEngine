// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"


struct FDisplayClusterClusterEventJson;


/**
 * Auxiliary class. Responsible for executing console commands.
 */
class DISPLAYCLUSTER_API FDisplayClusterConsoleExec
{
public:

	/** Runs console commands from JSON cluster events */
	static bool Exec(const FDisplayClusterClusterEventJson& InEvent);

	/** Generic console command runner */
	static bool Exec(const FString& InExecutorName, const FString& InExecCommand);
};
