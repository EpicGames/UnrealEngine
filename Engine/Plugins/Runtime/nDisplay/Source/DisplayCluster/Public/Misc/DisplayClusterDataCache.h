// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"

#include "Misc/DisplayClusterTickableGameObject.h"
#include "Misc/SecureHash.h"

/**
 * Cache template for named data with automatic deletion of unused items after timeout.
 */
template<typename InDataType>
class TDisplayClusterDataCache
{
public:
	/** Data type of stored objects
	 * 
	 * The following methods must be implemented in DataType:
	 * 
	 *   // Return DataCache timeout in frames. A negative value disables the timeout.
	 *   static int32 GetDataCacheTimeOutInFrames();
	 * 
	 *   // Return true if DataCache is enabled.
	 *   static bool IsDataCacheEnabled();
	 *
	 *   // Method for releasing a cached data item, called before its destructor
	 *   void ReleaseDataCacheItem();
	 *
	 *   // Returns the unique name of this texture for DataCache.
	 *   inline const FString& GetDataCacheName() const;
	 */
	typedef InDataType DataType;

	virtual ~TDisplayClusterDataCache()
	{
		Release();
	}

	/** Helper to perform the very common case of hashing an FString into a hex representation. */
	static inline FString HashString(const FString& InString)
	{
		return FMD5::HashAnsiString(*InString.ToLower());
	}

	/** Retrieve an existing object by a unique name. */
	TSharedPtr<DataType, ESPMode::ThreadSafe> Find(const FString& InUniqueDataName)
	{
		if (DataType::IsDataCacheEnabled())
		{
			if (FCachedObject* ExistCachedObject = CachedObjects.FindByPredicate([InUniqueDataName](const FCachedObject& CachedObjectIt)
			{
				return CachedObjectIt.IsNameEqual(InUniqueDataName);
			}))
			{
				return ExistCachedObject->DataRef;
			}
		}

		return nullptr;
	}

	/** Register a new object. */
	void Add(const TSharedPtr<DataType, ESPMode::ThreadSafe>& InDataRef)
	{
		if (DataType::IsDataCacheEnabled())
		{
			CachedObjects.Add(FCachedObject(InDataRef));

			UpdateTickableGameObject();
		}
	}

protected:
	/**
	 * Release the cache
	 */
	void Release()
	{
		CachedObjects.Empty();
		UpdateTickableGameObject();
	}
	 
	 /**
	 * When the number of DataRef references drops to 1, these items will be removed after a timeout.
	 */
	void Tick(float DeltaTime)
	{
		// If the cache is disabled, clear it
		if (!DataType::IsDataCacheEnabled())
		{
			Release();

			return;
		}

		const int32 TimeOutInFrames = DataType::GetDataCacheTimeOutInFrames();

		// Updating the time for all objects in use
		for (int32 Index = 0; Index < CachedObjects.Num(); Index++)
		{
			if (CachedObjects[Index].DataRef.GetSharedReferenceCount() >1 || TimeOutInFrames < 0)
			{
				// When this object used, reset this counter
				CachedObjects[Index].FramesInCachedState = 0;
			}
			else
			{
				// After the data ref counter remains 1, this counter will be incremented every frame
				CachedObjects[Index].FramesInCachedState++;

				// Remove after timout
				if (CachedObjects[Index].FramesInCachedState > TimeOutInFrames)
				{
					// Delete the current element, and reuse the current array index,
					// because the next element will move to the index of the current
					CachedObjects.RemoveAt(Index--);
				}
			}
		}

		UpdateTickableGameObject();
	}

	/** Create or remove tickable game object. */
	inline void UpdateTickableGameObject()
	{
		if (CachedObjects.IsEmpty())
		{
			// Unregister tick event
			if (TickableGameObject.IsValid() && TickHandle.IsValid())
			{
				TickableGameObject->OnTick().Remove(TickHandle);
			}

			TickableGameObject.Reset();
		}
		else
		{
			if (!TickableGameObject.IsValid())
			{
				TickableGameObject = MakeUnique<FDisplayClusterTickableGameObject >();
				TickHandle = TickableGameObject->OnTick().AddRaw(this, &TDisplayClusterDataCache<DataType>::Tick);
			}
		}
	}

private:
	struct FCachedObject
	{
		FCachedObject(const TSharedPtr<DataType, ESPMode::ThreadSafe>& InDataRef)
			: DataRef(InDataRef)
		{ }

		~FCachedObject()
		{
			Release();
		}

		/** Release referenced data. */
		void Release()
		{
			if (DataRef.IsValid())
			{
				// Calls the special release function before the destructor
				DataRef->ReleaseDataCacheItem();
				DataRef.Reset();
			}

			FramesInCachedState = 0;
		}

		inline bool IsNameEqual(const FString& InName) const
		{
			return DataRef.IsValid() && DataRef->GetDataCacheName() == InName;
		}

		// reference to data
		TSharedPtr<DataType, ESPMode::ThreadSafe> DataRef;

		// After the data ref counter remains 1, this counter will be incremented every frame
		int32 FramesInCachedState = 0;
	};

	// Cached objects map
	TArray<FCachedObject> CachedObjects;

	// When CachedObjects is not empty, this ticking object will be created.
	// Also, this object will be deleted when CachedObjects becomes empty.
	TUniquePtr<FDisplayClusterTickableGameObject> TickableGameObject;

	// Event delegate container of the Tickable object
	FDelegateHandle TickHandle;
};
