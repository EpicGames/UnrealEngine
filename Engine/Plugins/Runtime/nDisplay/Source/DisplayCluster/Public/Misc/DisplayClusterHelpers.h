// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "IDisplayCluster.h"
#include "Cluster/IDisplayClusterClusterManager.h"
#include "Config/IDisplayClusterConfigManager.h"
#include "Game/IDisplayClusterGameManager.h"

#include "Misc/DisplayClusterStrings.h"
#include "Misc/DisplayClusterTypesConverter.h"

#include "EngineUtils.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"

#include "Interfaces/IPluginManager.h"

class AActor;
class UDisplayClusterCameraComponent;


namespace DisplayClusterHelpers
{
	//////////////////////////////////////////////////////////////////////////////////////////////
	// String helpers
	//////////////////////////////////////////////////////////////////////////////////////////////
	namespace str
	{
		static inline FString BoolToStr(bool bVal, bool bAsWord = true)
		{
			return (bVal ? 
				(bAsWord ? FString(TEXT("true"))  : FString(TEXT("1"))) :
				(bAsWord ? FString(TEXT("false")) : FString(TEXT("0"))));
		}

		static void TrimStringValue(FString& InLine, bool bTrimQuotes = true)
		{
			// '   = \"  text \"    '
			InLine.TrimStartAndEndInline();
			// '= \"  text \"'
			InLine.RemoveFromStart(FString("="));
			// ' \"  text \"'
			InLine.TrimStartAndEndInline();
			// '\"  text \"'

			if (bTrimQuotes)
			{
				InLine = InLine.TrimQuotes();
				// '  text '
			}

			InLine.TrimStartAndEndInline();
			// 'text'
		}

		static FString TrimStringValue(const FString& InLine, bool bTrimQuotes = true)
		{
			FString TempStr = InLine;
			TrimStringValue(TempStr, bTrimQuotes);
			return TempStr;
		}

		// Parses string items separated by specified separator into array
		// Example: item1, item2,item3  ,  item4 => {item1, item2, item3, item4}
		template<typename TVal>
		static void StrToArray(const FString& InData, const FString& InSeparator, TArray<TVal>& OutData, bool bCullEmpty = true)
		{
			TArray<FString> TempData;
			InData.ParseIntoArray(TempData, *InSeparator, bCullEmpty);

			for (FString& Item : TempData)
			{
				TrimStringValue(Item, false);
			}

			for (const FString& Item : TempData)
			{
				if (!bCullEmpty && Item.IsEmpty())
				{
					OutData.AddDefaulted();
				}
				else
				{
					OutData.Add(DisplayClusterTypesConverter::template FromString<TVal>(Item));
				}
			}
		}

		// Exports array data to a string
		// Example: {item1, item2, item3, item4} => "item1,item2,item3,item4"
		template<typename TVal>
		static FString ArrayToStr(const TArray<TVal>& InData, const FString& InSeparator = FString(DisplayClusterStrings::common::ArrayValSeparator), bool bAddQuotes = true)
		{
			const FString Quotes("\"");

			FString ResultStr;
			ResultStr.Reserve(255);

			if (bAddQuotes)
			{
				ResultStr = Quotes;
			}

			for (const auto& it : InData)
			{
				ResultStr += FString::Printf(TEXT("%s%s"), *DisplayClusterTypesConverter::template ToString(it), *InSeparator);
			}

			if (InSeparator.Len() > 0 && InData.Num() > 0)
			{
				ResultStr.RemoveAt(ResultStr.Len() - InSeparator.Len(), InSeparator.Len(), bAddQuotes ? EAllowShrinking::No : EAllowShrinking::Yes);
			}

			if (bAddQuotes)
			{
				ResultStr += Quotes;
			}

			return ResultStr;
		}

		// Parses string items separated by specified separator into a set
		// Example: item1, item2, item3, item2, item1  ,  item4 => {item1, item2, item3, item4}
		template<typename TVal>
		static void StrToSet(const FString& InData, const FString& InSeparator, TSet<TVal>& OutData, bool bCullEmpty = true)
		{
			TArray<FString> TempData;
			StrToArray<TVal>(InData, InSeparator, TempData, bCullEmpty);
			OutData.Reset();
			OutData.Append(TempData);
		}

		// Exports set to a string
		// Example: {item1, item2, item3, item4} => "item1,item2,item3,item4"
		template<typename TVal>
		static FString SetToStr(const TSet<TVal>& InData, const FString& InSeparator = FString(DisplayClusterStrings::common::ArrayValSeparator), bool bAddQuotes = true)
		{
			return ArrayToStr<TVal>(InData.Array(), InSeparator, bAddQuotes);
		}

		// Parses string of key-value pairs separated by specified separator into map
		// Example: "key1=val1 key2=val2 key3=val3" => {{key1, val2}, {key2, val2}, {key3, val3}}
		template<typename TKey, typename TVal>
		void StrToMap(
			const FString& InData,
			TMap<TKey, TVal>& OutData,
			const FString& InPairSeparator   = FString(DisplayClusterStrings::common::PairSeparator),
			const FString& InKeyValSeparator = FString(DisplayClusterStrings::common::KeyValSeparator))
		{
			TArray<FString> StrPairs;
			DisplayClusterHelpers::str::template StrToArray<FString>(InData, InPairSeparator, StrPairs);

			for (const FString& StrPair : StrPairs)
			{
				FString StrKey;
				FString StrVal;

				if (StrPair.Split(InKeyValSeparator, &StrKey, &StrVal, ESearchCase::IgnoreCase))
				{
					TrimStringValue(StrKey);
					TrimStringValue(StrVal);

					OutData.Emplace(DisplayClusterTypesConverter::template FromString<TKey>(StrKey), DisplayClusterTypesConverter::template FromString<TVal>(StrVal));
				}
			}
		}

		// Exports map data to a string
		// Example: {{key1,val1},{key2,val2},{key3,val3}} => "key1=val1 key2=val2 key3=var3"
		template<typename TKey, typename TVal>
		FString MapToStr(
			const TMap<TKey, TVal>& InData,
			const FString& InPairSeparator   = FString(DisplayClusterStrings::common::PairSeparator),
			const FString& InKeyValSeparator = FString(DisplayClusterStrings::common::KeyValSeparator),
			bool bAddQuoutes = true)
		{
			static const auto Quotes = TEXT("\"");
			
			FString ResultStr;
			ResultStr.Reserve(255);

			if (bAddQuoutes)
			{
				ResultStr = Quotes;
			}

			for (const auto& Pair : InData)
			{
				ResultStr = FString::Printf(TEXT("%s%s%s%s%s"),
					*ResultStr,
					*DisplayClusterTypesConverter::template ToString(Pair.Key),
					*InKeyValSeparator,
					*DisplayClusterTypesConverter::template ToString(Pair.Value),
					*InPairSeparator);
			}

			if (InPairSeparator.Len() > 0 && InData.Num() > 0)
			{
				ResultStr.RemoveAt(ResultStr.Len() - InPairSeparator.Len(), InPairSeparator.Len(), bAddQuoutes ? EAllowShrinking::No : EAllowShrinking::Yes);
			}

			if (bAddQuoutes)
			{
				ResultStr += Quotes;
			}

			return ResultStr;
		}

		// Extracts value either from a command line string or any other line that matches the same format
		// Example: extracting value of param2
		// "param1=value1 param2=value2 param3=value3" => value2
		template<typename T>
		static bool ExtractValue(const FString& InLine, const FString& InParamName, T& OutValue, bool bInTrimQuotes = true)
		{
			FString TempVal;
			const FString EqToken("=");

			// Trim argument name and add '=' to the end
			FString FullParamName = InParamName.TrimStartAndEnd();
			FullParamName = (FullParamName.EndsWith(EqToken) ? FullParamName : FullParamName + EqToken);
			if (FParse::Value(*InLine, *FullParamName, TempVal, false))
			{
				TrimStringValue(TempVal, bInTrimQuotes);
				OutValue = DisplayClusterTypesConverter::template FromString<T>(TempVal);
				return true;
			}

			return false;
		}

		// Extracts array value either from a command line string or any other line that matches the same format
		// Example: extracting array value of param2
		// "param1=value1 param2="a,b,c,d" param3=value3" => {a,b,c,d}
		template<typename TVal>
		static bool ExtractArray(const FString& InLine, const FString& InParamName, const FString& InSeparator, TArray<TVal>& OutValue)
		{
			FString TempVal;

			if (ExtractValue(InLine, InParamName, TempVal, false))
			{
				DisplayClusterHelpers::str::template StrToArray<TVal>(TempVal, InSeparator, OutValue);
				return true;
			}

			return false;
		}

		// Extracts map value either from a command line string or any other line that matches the same format
		// Example: extracting map value of param2
		// "param1=value1 param2="a:1,b:7,c:22" param3=value3" => {{a,1},{b,7}{c,22}}
		template<typename TKey, typename TVal>
		static bool ExtractMap(
			const FString& InLine,
			const FString& InParamName,
			TMap<TKey, TVal>& OutData,
			const FString& InPairSeparator = FString(DisplayClusterStrings::common::PairSeparator),
			const FString& InKeyValSeparator = FString(DisplayClusterStrings::common::KeyValSeparator))
		{
			TArray<FString> TempPairs;
			if (!ExtractArray(InLine, InParamName, InPairSeparator, TempPairs))
			{
				return false;
			}

			for (const FString& StrPair : TempPairs)
			{
				StrToMap(StrPair, OutData, InPairSeparator, InKeyValSeparator);
			}

			return true;
		}
	};


	//////////////////////////////////////////////////////////////////////////////////////////////
	// Map helpers
	//////////////////////////////////////////////////////////////////////////////////////////////
	namespace map
	{
		// Extracts value from TMap<FString, TVal>, returns true if Ok
		template<typename TVal>
		bool ExtractValue(const TMap<FString, TVal>& InMap, const FString& InKey, TVal& OutValue, ESearchCase::Type SearchCase = ESearchCase::IgnoreCase)
		{
			for (const auto& it : InMap)
			{
				if (InKey.Equals(it.Key, SearchCase))
				{
					OutValue = it.Value;
					return true;
				}
			}

			return false;
		}

		// Extracts value from TMap<FString, FString>, returns value if found, otherwise default value
		template<typename TVal>
		TVal ExtractValue(const TMap<FString, TVal>& InMap, const FString& InKey, const TVal& DefaultValue, ESearchCase::Type SearchCase = ESearchCase::IgnoreCase)
		{
			TVal TempVal;
			return (DisplayClusterHelpers::map::template ExtractValue(InMap, InKey, TempVal, SearchCase) ? TempVal : DefaultValue);
		}

		// Extracts value from TMap<FString, FString>, returns true and TReturn converted from string, otherwise false
		template<typename TReturn>
		bool ExtractValueFromString(const TMap<FString, FString>& InMap, const FString& InKey, TReturn& OutValue, ESearchCase::Type SearchCase = ESearchCase::IgnoreCase)
		{
			FString TempValue;
			if (DisplayClusterHelpers::map::template ExtractValue(InMap, InKey, TempValue, SearchCase))
			{
				OutValue = DisplayClusterTypesConverter::template FromString<TReturn>(TempValue);
				return true;
			}

			return false;
		}

		// Extracts value from TMap<FString, FString> and converts it to TReturn. If no value found, the default value is returned
		template<typename TReturn>
		TReturn ExtractValueFromString(const TMap<FString, FString>& InMap, const FString& InKey, const TReturn& DefaultValue, ESearchCase::Type SearchCase = ESearchCase::IgnoreCase)
		{
			FString TempValue;
			if (DisplayClusterHelpers::map::template ExtractValue(InMap, InKey, TempValue, SearchCase))
			{
				return DisplayClusterTypesConverter::template FromString<TReturn>(TempValue);
			}

			return DefaultValue;
		}

		// Extracts array from sting map value
		template<typename TVal>
		bool ExtractArrayFromString(
			const TMap<FString, FString>& InMap,
			const FString& InKey, TArray<TVal>& OutArray,
			const FString& InSeparator = DisplayClusterStrings::common::ArrayValSeparator,
			bool bCullEmpty = true,
			ESearchCase::Type SearchCase = ESearchCase::IgnoreCase)
		{
			FString TempValue;
			if (DisplayClusterHelpers::map::template ExtractValue(InMap, InKey, TempValue, SearchCase))
			{
				 DisplayClusterHelpers::str::template StrToArray(TempValue, InSeparator, OutArray, bCullEmpty);
				return true;
			}

			return false;
		}

		// Extracts map from sting map value
		template<typename TKey, typename TVal>
		bool ExtractMapFromString(
			const TMap<FString, FString>& InMap,
			const FString& InKey,
			TMap<TKey, TVal>& OutMap,
			const FString& InPairSeparator   = FString(DisplayClusterStrings::common::PairSeparator),
			const FString& InKeyValSeparator = FString(DisplayClusterStrings::common::KeyValSeparator),
			ESearchCase::Type SearchCase = ESearchCase::IgnoreCase)
		{
			FString TempValue;
			if (DisplayClusterHelpers::map::template ExtractValue(InMap, InKey, TempValue, SearchCase))
			{
				DisplayClusterHelpers::str::template StrToMap(TempValue, OutMap, InPairSeparator, InKeyValSeparator);
				return true;
			}

			return false;
		}
	}

	//////////////////////////////////////////////////////////////////////////////////////////////
	// Game helpers
	//////////////////////////////////////////////////////////////////////////////////////////////
	namespace game
	{
		template<typename T>
		static void FindAllActors(UWorld* World, TArray<T*>& Out)
		{
			for (TActorIterator<AActor> It(World, T::StaticClass()); It; ++It)
			{
				T* Actor = Cast<T>(*It);
				if (IsValid(Actor))
				{
					Out.Add(Actor);
				}
			}
		}
	}

	//////////////////////////////////////////////////////////////////////////////////////////////
	// File system helpers
	//////////////////////////////////////////////////////////////////////////////////////////////
	namespace filesystem
	{
		// This helper function looks for a config file provided with relative path. It uses
		// different base directories that are typically used in different runtime environments.
		static FString GetFullPathForConfig(const FString& RelativeConfig)
		{
			if (!FPaths::IsRelative(RelativeConfig))
			{
				return RelativeConfig;
			}

			TArray<FString> LookupRoots;

			// Editor (configurator)
			LookupRoots.Emplace(FPaths::LaunchDir());
			// PIE
			LookupRoots.Emplace(FPaths::ProjectDir());

			for (const FString& Root : LookupRoots)
			{
				const FString AbsoluteRoot   = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*Root);
				const FString AbsoluteConfig = FPaths::ConvertRelativePathToFull(Root, RelativeConfig);

				if (FPaths::FileExists(AbsoluteConfig))
				{
					return AbsoluteConfig;
				}
			}

			// Not found
			return RelativeConfig;
		}

		/** Get ordered search base dirs for resources. */
		static TArray<FString> GetOrderedConfigResourceDirs()
		{
			TArray<FString> OutDirs;

			// First look in the configuration directory
			IDisplayClusterConfigManager* const ConfigMgr = IDisplayCluster::Get().GetConfigMgr();
			if (ConfigMgr)
			{
				const FString ConfigPath = ConfigMgr->GetConfigPath();
				if (!ConfigPath.IsEmpty())
				{
					OutDirs.Add(FPaths::GetPath(ConfigPath));
				}
			}

			// Then look in the project directory
			// This will allow us to use relative paths for external files inside the project
			OutDirs.Add(FPaths::ConvertRelativePathToFull(FPaths::ProjectDir()));

			// Finally, look in the root directory of the UE.
			OutDirs.Add(FPaths::RootDir());

			return OutDirs;
		}

		/**
		* Getting the relative path to an external file
		*/
		static FString GetRelativePathForConfigResource(const FString& ResourceFullPath)
		{
			FString CleanResourceFullPath = DisplayClusterHelpers::str::TrimStringValue(ResourceFullPath);
			FPaths::NormalizeFilename(CleanResourceFullPath);

			if (!CleanResourceFullPath.IsEmpty())
			{
				// Tries to create relative paths to allowed directories:
				for (const FString& ResourcePathIt : GetOrderedConfigResourceDirs())
				{
					if (CleanResourceFullPath.StartsWith(ResourcePathIt))
					{
						// Creates a relative path to the files in the project directory
						return CleanResourceFullPath.Mid(ResourcePathIt.Len());
					}
				}
			}

			return CleanResourceFullPath;
		}

		static FString GetFullPathForConfigResource(const FString& ResourcePath)
		{
			FString CleanResourcePath = DisplayClusterHelpers::str::TrimStringValue(ResourcePath);
			FPaths::NormalizeFilename(CleanResourcePath);

			if (FPaths::IsRelative(CleanResourcePath))
			{
				// Process base dirs in order:
				for (const FString& ResourcePathIt : GetOrderedConfigResourceDirs())
				{
					const FString FullPath = FPaths::ConvertRelativePathToFull(ResourcePathIt, CleanResourcePath);
					if (FPaths::FileExists(FullPath))
					{
						return FullPath;
					}
				}
			}
			else
			{
				return CleanResourcePath;
			}

			return FString();
		}

		/** Returns the full path to the DLL located in the ThirdParty directory in the nDisplay plugin.
		* @param InRelativePathForThirdPartyDll - relative path to dll from ThirdParty root
		* @return full path to file
		*/
		static FString GetFullPathForThirdPartyDLL(const FString& InRelativePathForThirdPartyDll)
		{
			// Get the DLL from the nDisplay plugin directory
			const FString PluginDir = IPluginManager::Get().FindPlugin(TEXT("nDisplay"))->GetBaseDir();
			const FString FullPath = FPaths::Combine(PluginDir, TEXT("Source/ThirdParty"), InRelativePathForThirdPartyDll);
			if (FPaths::FileExists(FullPath))
			{
				return FullPath;
			}

			// Package/Standalone builds may use a different directory for the ThirdParty DLL.
			// Todo: add ThirdParty DLL search for Package/Standalone builds.

			return InRelativePathForThirdPartyDll;
		}
	}
};
