// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Components/SceneComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/GameEngine.h"
#include "EngineUtils.h"

#if WITH_EDITOR
#include "Editor.h"
#endif

#include "DisplayClusterObjectRef.generated.h"

class DISPLAYCLUSTER_API FDisplayClusterActorRef
{
public:
	FDisplayClusterActorRef()
	{}

	FDisplayClusterActorRef(const FDisplayClusterActorRef& InActorRef)
		: WorldPtr(InActorRef.WorldPtr)
		, ActorClassName(InActorRef.ActorClassName)
		, ActorClassPtr(InActorRef.ActorClassPtr)
		, ActorName(InActorRef.ActorName)
		, ActorPtr(InActorRef.ActorPtr)
	{}

	virtual ~FDisplayClusterActorRef()
	{ }

public:
	bool IsDefinedSceneActor() const
	{
		return !ActorClassName.IsEmpty() && !ActorName.IsNone();
	}

	/** Returns the current weak ptr of the scene actor object. */
	const TWeakObjectPtr<AActor> GetSceneActorWeakPtr() const
	{
		FScopeLock lock(&DataGuard);
		if (!IsDefinedSceneActor())
		{
			return nullptr;
		}

		return ActorPtr;
	}

	// Return actor object ptr.
	// For killed object ptr, reset and find actor new object ptr by name and save to [mutable] ActorPtr
	AActor* GetOrFindSceneActor() const
	{
		FScopeLock lock(&DataGuard);

		if (!IsDefinedSceneActor())
		{
			return nullptr;
		}

		if (!ActorPtr.IsValid())
		{
			ActorPtr.Reset();

			// Do not do anything after the exit request.
			if (IsEngineExitRequested())
			{
				return nullptr;
			}

			// Reload the current world and actor class if necessary.
			if (UpdateWorldPtr() && UpdateActorClassPtr())
			{
				// Find actor object
				for (TActorIterator<AActor> It(WorldPtr.Get(), ActorClassPtr.Get(), EActorIteratorFlags::SkipPendingKill); It; ++It)
				{
					AActor* Actor = *It;
					if (Actor != nullptr && !Actor->IsTemplate() && Actor->GetFName() == ActorName)
					{
						ActorPtr = TWeakObjectPtr<AActor>(Actor);
						return Actor;
					}
				}
			}
			// Actor or class not found. Removed from scene?
		}

		return ActorPtr.Get();
	}

	bool SetSceneActor(const AActor* InActor)
	{
		FScopeLock lock(&DataGuard);

		ResetSceneActor();

		if (InActor)
		{
			UClass* ActorClass = InActor->GetClass();
			if (ActorClass != nullptr)
			{
				ActorClassName = ActorClass->GetPathName();
				ActorName = InActor->GetFName();

				WorldPtr = TWeakObjectPtr<UWorld>(InActor->GetWorld());
				ActorClassPtr = TWeakObjectPtr<UClass>(ActorClass);

				// Ignore const
				ActorPtr = TWeakObjectPtr<AActor>((AActor*)InActor);

				return true;
			}
		}

		return false;
	}

	void ResetSceneActor()
	{
		FScopeLock lock(&DataGuard);

		ActorClassName.Empty();
		ActorName = FName();

		WorldPtr.Reset();
		ActorClassPtr.Reset();
		ActorPtr.Reset();
	}

private:
	// Change mutable WorldPtr for re-created world
	bool UpdateWorldPtr() const
	{
		if (!WorldPtr.IsValid())
		{
			UWorld* CurrentWorld = nullptr;

#if WITH_EDITOR
			// Prevent a crash when exiting the UE application and calling this function after deleting GEditor.
			if (GIsEditor && IsValid(GEditor))
			{
				CurrentWorld = GEditor->GetEditorWorldContext().World();
			}
			else
#endif
			if (UGameEngine* GameEngine = Cast<UGameEngine>(GEngine))
			{
				CurrentWorld = IsValid(GameEngine) ? GameEngine->GetGameWorld() : nullptr;
			}

			if (!IsValid(CurrentWorld))
			{
				WorldPtr.Reset();

				return false;
			}

			WorldPtr = TWeakObjectPtr<UWorld>(CurrentWorld);
		}

		return true;
	}

	// Find actor new object ptr by name and save to ActorPtr
	bool UpdateActorClassPtr() const
	{
		UClass* ActorClass = ActorClassName.IsEmpty() ? nullptr : StaticLoadClass(UObject::StaticClass(), nullptr, *ActorClassName, NULL, LOAD_None, NULL);
		if (!ActorClass)
		{
			ActorClassPtr.Reset();
			return false;
		}

		ActorClassPtr = TWeakObjectPtr<UClass>(ActorClass);

		return true;
	}

private:
	// Save world object ptr
	mutable TWeakObjectPtr<UWorld> WorldPtr;

	// Find actor class object ptr by name and save to ActorClassPtr
	FString ActorClassName;
	mutable TWeakObjectPtr<UClass> ActorClassPtr;

	// Find actor new object ptr by name and save to ActorPtr
	FName   ActorName;
	mutable TWeakObjectPtr<AActor> ActorPtr;

protected:
	// Multi-thread data guard
	mutable FCriticalSection DataGuard;
};

class DISPLAYCLUSTER_API FDisplayClusterSceneComponentRef
	: public FDisplayClusterActorRef
{
public:
	FDisplayClusterSceneComponentRef()
		: FDisplayClusterActorRef()
	{ }

	FDisplayClusterSceneComponentRef(const FDisplayClusterSceneComponentRef& InComponentRef)
		: FDisplayClusterActorRef(InComponentRef)
		, ComponentName(InComponentRef.ComponentName)
		, ComponentPtr(InComponentRef.ComponentPtr)
	{}

	FDisplayClusterSceneComponentRef(USceneComponent* InComponent)
	{
		SetSceneComponent(InComponent);
	}

public:
	bool IsDefinedSceneComponent() const
	{
		return !ComponentName.IsNone() && IsDefinedSceneActor();
	}

	bool IsSceneComponentValid() const
	{
		return ComponentPtr.IsValid() && !ComponentPtr.IsStale();
	}

	// Return component object ptr.
	// For killed object ptr, reset and find component new object ptr by name and save to [mutable] ComponentPtr
	USceneComponent* GetOrFindSceneComponent() const;

	bool SetSceneComponent(USceneComponent* const InComponent)
	{
		FScopeLock lock(&DataGuard);

		ResetSceneComponent();

		if (InComponent != nullptr && SetSceneActor(InComponent->GetOwner()))
		{
			ComponentName = InComponent->GetFName();
			ComponentPtr = TWeakObjectPtr<USceneComponent>(InComponent);
			return true;
		}

		return false;
	}

	void ResetSceneComponent()
	{
		FScopeLock lock(&DataGuard);

		ResetSceneActor();

		ComponentName = FName();
		ComponentPtr.Reset();
	}

	bool IsEqualsComponentName(const FName& InComponentName) const
	{
		return ComponentName == InComponentName;
	}

private:
	// Find component new object ptr by name and save to ComponentPtr
	FName   ComponentName;
	mutable TWeakObjectPtr<USceneComponent> ComponentPtr;
};

USTRUCT(meta = (HideChildren))
struct DISPLAYCLUSTER_API FDisplayClusterComponentRef
{
	GENERATED_BODY()
	
	FDisplayClusterComponentRef() {}
	
	FDisplayClusterComponentRef(const FString& InName)
	{
		Name = InName;
	}
	
	UPROPERTY(VisibleAnywhere, Category = Component)
	FString Name;
	
	FORCEINLINE bool operator==(const FDisplayClusterComponentRef& OtherRef) const
	{
		return Name == OtherRef.Name;
	}
};