// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"


namespace DisplayClusterStrings
{
	// Common strings
	namespace common
	{
		static constexpr const TCHAR* PairSeparator     = TEXT(" ");
		static constexpr const TCHAR* KeyValSeparator   = TEXT("=");
		static constexpr const TCHAR* ArrayValSeparator = TEXT(",");
	}

	// Command line arguments
	namespace args
	{
		static constexpr const TCHAR* Cluster    = TEXT("dc_cluster");
		static constexpr const TCHAR* Node       = TEXT("dc_node");
		static constexpr const TCHAR* Config     = TEXT("dc_cfg");
		static constexpr const TCHAR* NodeList   = TEXT("dc_nodes");
		static constexpr const TCHAR* NodeRole   = TEXT("dc_role");

		// Cluster node roles
		namespace role
		{
			static constexpr const TCHAR* Primary   = TEXT("primary");
			static constexpr const TCHAR* Secondary = TEXT("secondary");
			static constexpr const TCHAR* Backup    = TEXT("backup");
		}

		// Stereo device types (command line values)
		namespace dev
		{
			static constexpr const TCHAR* QBS  = TEXT("quad_buffer_stereo");
			static constexpr const TCHAR* TB   = TEXT("dc_dev_top_bottom");
			static constexpr const TCHAR* SbS  = TEXT("dc_dev_side_by_side");
			static constexpr const TCHAR* Mono = TEXT("dc_dev_mono");
		}
	}

	// Log strings
	namespace log
	{
		static constexpr const TCHAR* Found     = TEXT("found");
		static constexpr const TCHAR* NotFound  = TEXT("not found");
	}

	// System cluster events
	namespace cluster_events
	{
		// Common system events data
		static constexpr const TCHAR* EventCategory = TEXT("nDisplay");
		static constexpr const TCHAR* EventType     = TEXT("control");

		// Specific events
		static constexpr const TCHAR* EvtQuitName   = TEXT("quit");
	}

	namespace uri_args
	{
		static constexpr const TCHAR* ClusterId = TEXT("ClusterId=");
		static constexpr const TCHAR* PrimaryNodeId = TEXT("PrimaryNodeId=");
		static constexpr const TCHAR* PrimaryNodePort = TEXT("PrimaryNodePort=");
		static constexpr const TCHAR* ClusterNodesNum = TEXT("ClusterNodesCount=");
	}
};
