// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Tickable.h"

/**
 * Tickable game object: has an OnTick() callback to start all subscribers
 */
class DISPLAYCLUSTER_API FDisplayClusterTickableGameObject : public FTickableGameObject
{
public:
	FDisplayClusterTickableGameObject() = default;
	virtual ~FDisplayClusterTickableGameObject() = default;

public:
	/** Callback on tick **/
	DECLARE_EVENT_OneParam(FDisplayClusterTickableGameObject, FTickEvent, float);
	FTickEvent& OnTick()
	{
		return TickEvent;
	}

	virtual bool IsTickableInEditor() const override
	{
		return true;
	}

	virtual void Tick(float DeltaTime) override;

	virtual ETickableTickType GetTickableTickType() const override
	{
		return ETickableTickType::Always;
	}

	virtual TStatId GetStatId() const override
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FDisplayClusterTickableGameObject, STATGROUP_Tickables);
	}

private:
	FTickEvent TickEvent;
};
