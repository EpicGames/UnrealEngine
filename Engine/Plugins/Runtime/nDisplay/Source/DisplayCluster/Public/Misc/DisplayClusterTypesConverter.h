// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Timecode.h"
#include "Misc/FrameRate.h"
#include "Misc/QualifiedFrameTime.h"

#include "DisplayClusterEnums.h"


/**
 * Auxiliary class with different type conversion functions
 */
class DisplayClusterTypesConverter
{
public:
	//////////////////////////////////////////////////////////////////////////////////////////////
	// TYPE --> STRING
	//////////////////////////////////////////////////////////////////////////////////////////////
	template <typename ConvertFrom>
	static FString ToString(const ConvertFrom& From);

	//////////////////////////////////////////////////////////////////////////////////////////////
	// STRING --> TYPE
	//////////////////////////////////////////////////////////////////////////////////////////////
	template <typename ConvertTo>
	static ConvertTo FromString(const FString& From);

	//////////////////////////////////////////////////////////////////////////////////////////////
	// TYPE --> HEX STRING
	//////////////////////////////////////////////////////////////////////////////////////////////
	template <typename ConvertFrom>
	static FString ToHexString(const ConvertFrom& From);

	//////////////////////////////////////////////////////////////////////////////////////////////
	// HEX STRING --> TYPE
	//////////////////////////////////////////////////////////////////////////////////////////////
	template <typename ConvertTo>
	static ConvertTo FromHexString(const FString& From);
};


//////////////////////////////////////////////////////////////////////////////////////////////
// Forward declarations
//////////////////////////////////////////////////////////////////////////////////////////////
template <> FString DisplayClusterTypesConverter::ToHexString  <float>(const float&   From);
template <> float   DisplayClusterTypesConverter::FromHexString<float>(const FString& From);


//////////////////////////////////////////////////////////////////////////////////////////////
// TYPE --> STRING
//////////////////////////////////////////////////////////////////////////////////////////////
template <> inline FString DisplayClusterTypesConverter::ToString<>(const FString& From)    { return From; }
template <> inline FString DisplayClusterTypesConverter::ToString<>(const bool& From)       { return From ? FString(TEXT("true")) : FString(TEXT("false")); }
template <> inline FString DisplayClusterTypesConverter::ToString<>(const int8& From)       { return FString::FromInt(From); }
template <> inline FString DisplayClusterTypesConverter::ToString<>(const uint8& From)      { return ToString(static_cast<int8>(From)); }
template <> inline FString DisplayClusterTypesConverter::ToString<>(const int32& From)      { return FString::FromInt(From); }
template <> inline FString DisplayClusterTypesConverter::ToString<>(const uint32& From)     { return ToString(static_cast<int32>(From)); }
template <> inline FString DisplayClusterTypesConverter::ToString<>(const float& From)      { return FString::SanitizeFloat(From); }
template <> inline FString DisplayClusterTypesConverter::ToString<>(const double& From)     { return FString::Printf(TEXT("%lf"), From); }
template <> inline FString DisplayClusterTypesConverter::ToString<>(const FRotator& From)   { return From.ToString(); }
template <> inline FString DisplayClusterTypesConverter::ToString<>(const FMatrix& From)    { return From.ToString(); }
template <> inline FString DisplayClusterTypesConverter::ToString<>(const FQuat& From)      { return From.ToString(); }
template <> inline FString DisplayClusterTypesConverter::ToString<>(const FIntPoint& From)  { return From.ToString(); }

// The TVector::ToString() function uses "%3.3f" which compresses the value. A proprietary function is used to get a full-size float.
template <> inline FString DisplayClusterTypesConverter::ToString<>(const FVector& From)
{
	return FString::Printf(TEXT("X=%f Y=%f Z=%f"), From.X, From.Y, From.Z);
}

// The TVector2D::ToString() function uses "%3.3f" which compresses the value. A proprietary function is used to get a full-size float.
template <> inline FString DisplayClusterTypesConverter::ToString<>(const FVector2D& From)
{
	return FString::Printf(TEXT("X=%f Y=%f"), From.X, From.Y);
}

// We can't just use FTimecode ToString as that loses information.
template <> inline FString DisplayClusterTypesConverter::ToString<>(const FTimecode& From)
{
	return FString::Printf(TEXT("%d;%d;%d;%d;%d"), From.bDropFrameFormat ? 1 : 0, From.Hours, From.Minutes, From.Seconds, From.Frames);
}

template <> inline FString DisplayClusterTypesConverter::ToString<>(const FFrameRate& From)
{
	return FString::Printf(TEXT("%d;%d"), From.Numerator, From.Denominator);
}

template <> inline FString DisplayClusterTypesConverter::ToString<>(const FQualifiedFrameTime& From)
{
	return FString::Printf(TEXT("%d;%s;%d;%d"), From.Time.GetFrame().Value, *ToHexString(From.Time.GetSubFrame()), From.Rate.Numerator, From.Rate.Denominator);
}

template <> inline FString DisplayClusterTypesConverter::ToString<>(const EDisplayClusterOperationMode& From)
{
	switch (From)
	{
	case EDisplayClusterOperationMode::Cluster:
		return FString("cluster");
	case EDisplayClusterOperationMode::Editor:
		return FString("editor");
	case EDisplayClusterOperationMode::Disabled:
		return FString("disabled");
	default:
		return FString("unknown");
	}
}

template <> inline FString DisplayClusterTypesConverter::ToString<>(const EDisplayClusterSyncGroup& From)
{
	switch (From)
	{
	case EDisplayClusterSyncGroup::PreTick:
		return FString("PreTick");
	case EDisplayClusterSyncGroup::Tick:
		return FString("Tick");
	case EDisplayClusterSyncGroup::PostTick:
		return FString("PostTick");
	}

	return FString("Tick");
}


//////////////////////////////////////////////////////////////////////////////////////////////
// STRING --> TYPE
//////////////////////////////////////////////////////////////////////////////////////////////
template <> inline FString   DisplayClusterTypesConverter::FromString<> (const FString& From) { return From; }
template <> inline bool      DisplayClusterTypesConverter::FromString<> (const FString& From) { return (From == FString("1") || (From.Equals(FString("true"), ESearchCase::IgnoreCase))); }
template <> inline int8      DisplayClusterTypesConverter::FromString<> (const FString& From) { return FCString::Atoi(*From); }
template <> inline uint8     DisplayClusterTypesConverter::FromString<> (const FString& From) { return static_cast<uint8>(FromString<int8>(From)); }
template <> inline int32     DisplayClusterTypesConverter::FromString<> (const FString& From) { return FCString::Atoi(*From); }
template <> inline uint32    DisplayClusterTypesConverter::FromString<> (const FString& From) { return static_cast<uint32>(FromString<int32>(From)); }
template <> inline float     DisplayClusterTypesConverter::FromString<> (const FString& From) { return FCString::Atof(*From); }
template <> inline double    DisplayClusterTypesConverter::FromString<> (const FString& From) { return FCString::Atod(*From); }
template <> inline FVector   DisplayClusterTypesConverter::FromString<> (const FString& From) { FVector vec;  vec.InitFromString(From); return vec; }
template <> inline FVector2D DisplayClusterTypesConverter::FromString<> (const FString& From) { FVector2D vec;  vec.InitFromString(From); return vec; }
template <> inline FRotator  DisplayClusterTypesConverter::FromString<> (const FString& From) { FRotator rot; rot.InitFromString(From); return rot; }

template <> inline FMatrix   DisplayClusterTypesConverter::FromString<>(const FString& From)
{
	FMatrix ResultMatrix = FMatrix::Identity;
	FPlane  Planes[4];

	int32 IdxStart = 0;
	int32 IdxEnd = 0;
	for (int PlaneNum = 0; PlaneNum < 4; ++PlaneNum)
	{
		IdxStart = From.Find(FString(TEXT("[")), ESearchCase::IgnoreCase, ESearchDir::FromStart, IdxEnd);
		IdxEnd   = From.Find(FString(TEXT("]")), ESearchCase::IgnoreCase, ESearchDir::FromStart, IdxStart);
		if (IdxStart == INDEX_NONE || IdxEnd == INDEX_NONE || (IdxEnd - IdxStart) < 8)
		{
			return ResultMatrix;
		}

		FString StrPlane = From.Mid(IdxStart + 1, IdxEnd - IdxStart - 1);

		int32 StrLen = 0;
		int32 NewStrLen = -1;
		while (NewStrLen < StrLen)
		{
			StrLen = StrPlane.Len();
			StrPlane.ReplaceInline(TEXT("  "), TEXT(" "));
			NewStrLen = StrPlane.Len();
		}

		TArray<FString> StrPlaneValues;
		StrPlane.ParseIntoArray(StrPlaneValues, TEXT(" "));
		if (StrPlaneValues.Num() != 4)
		{
			return ResultMatrix;
		}

		FMatrix::FReal PlaneValues[4] = { 0.f };
		for (int i = 0; i < 4; ++i)
		{
			PlaneValues[i] = FromString<FMatrix::FReal>(StrPlaneValues[i]);
		}

		Planes[PlaneNum] = FPlane(PlaneValues[0], PlaneValues[1], PlaneValues[2], PlaneValues[3]);
	}

	ResultMatrix = FMatrix(Planes[0], Planes[1], Planes[2], Planes[3]);

	return ResultMatrix;
}

template <> inline FQuat DisplayClusterTypesConverter::FromString<>(const FString& From)
{
	FQuat Result = FQuat::Identity;
	const bool bSuccessful
		=  FParse::Value(*From, TEXT("X="), Result.X)
		&& FParse::Value(*From, TEXT("Y="), Result.Y)
		&& FParse::Value(*From, TEXT("Z="), Result.Z)
		&& FParse::Value(*From, TEXT("W="), Result.W);

	if (!bSuccessful)
	{
		return FQuat::Identity;
	}

	return Result;
}

template <> inline FIntPoint DisplayClusterTypesConverter::FromString<>(const FString& From)
{
	FIntPoint Result = FIntPoint::ZeroValue;
	const bool bSuccessful
		=  FParse::Value(*From, TEXT("X="), Result.X)
		&& FParse::Value(*From, TEXT("Y="), Result.Y);

	if (!bSuccessful)
	{
		return FIntPoint::ZeroValue;
	}

	return Result;
}

template <> inline FTimecode DisplayClusterTypesConverter::FromString<> (const FString& From)
{
	FTimecode timecode;

	TArray<FString> parts;
	parts.Reserve(5);
	const int32 found = From.ParseIntoArray(parts, TEXT(";"));

	// We are expecting 5 "parts" - DropFrame, Hours, Minutes, Seconds, Frames.
	if (found == 5)
	{
		timecode.bDropFrameFormat = FromString<bool>(parts[0]);
		timecode.Hours   = FromString<int32>(parts[1]);
		timecode.Minutes = FromString<int32>(parts[2]);
		timecode.Seconds = FromString<int32>(parts[3]);
		timecode.Frames  = FromString<int32>(parts[4]);
	}

	return timecode;
}

template <> inline FFrameRate DisplayClusterTypesConverter::FromString<> (const FString& From)
{
	FFrameRate frameRate;

	TArray<FString> parts;
	parts.Reserve(2);
	const int32 found = From.ParseIntoArray(parts, TEXT(";"));

	// We are expecting 2 "parts" - Numerator, Denominator.
	if (found == 2)
	{
		frameRate.Numerator   = FromString<int32>(parts[0]);
		frameRate.Denominator = FromString<int32>(parts[1]);
	}

	return frameRate;
}

template <> inline FQualifiedFrameTime DisplayClusterTypesConverter::FromString<>(const FString& From)
{
	FQualifiedFrameTime frameTime;

	TArray<FString> parts;
	parts.Reserve(4);
	const int32 found = From.ParseIntoArray(parts, TEXT(";"));

	// We are expecting 4 "parts" - Frame, SubFrame, Numerator, Denominator.
	if (found == 4)
	{
		frameTime.Time = FFrameTime(FromString<int32>(parts[0]), FromHexString<float>(parts[1]));
		frameTime.Rate.Numerator   = FromString<int32>(parts[2]);
		frameTime.Rate.Denominator = FromString<int32>(parts[3]);
	}

	return frameTime;
}

template <> inline EDisplayClusterSyncGroup DisplayClusterTypesConverter::FromString<>(const FString& From)
{
	if (From.Equals(TEXT("PreTick")))
	{
		return EDisplayClusterSyncGroup::PreTick;
	}
	else if (From.Equals(TEXT("Tick")))
	{
		return EDisplayClusterSyncGroup::Tick;
	}
	else if (From.Equals(TEXT("PostTick")))
	{
		return EDisplayClusterSyncGroup::PostTick;
	}

	return EDisplayClusterSyncGroup::Tick;
}


//////////////////////////////////////////////////////////////////////////////////////////////
// TYPE --> HEX STRING
//////////////////////////////////////////////////////////////////////////////////////////////
template <> inline FString DisplayClusterTypesConverter::ToHexString<>(const float& From)
{
	return BytesToHex(reinterpret_cast<const uint8*>(&From), sizeof(float));
}

template <> inline FString DisplayClusterTypesConverter::ToHexString<>(const double& From)
{
	return BytesToHex(reinterpret_cast<const uint8*>(&From), sizeof(double));
}

template <> inline FString DisplayClusterTypesConverter::ToHexString<>(const FVector& From)
{
	return FString::Printf(TEXT("X=%s Y=%s Z=%s"), 
		*DisplayClusterTypesConverter::template ToHexString<>(From.X),
		*DisplayClusterTypesConverter::template ToHexString<>(From.Y),
		*DisplayClusterTypesConverter::template ToHexString<>(From.Z));
}

template <> inline FString DisplayClusterTypesConverter::ToHexString<>(const FVector2D& From)
{
	return FString::Printf(TEXT("X=%s Y=%s"),
		*DisplayClusterTypesConverter::template ToHexString<>(From.X),
		*DisplayClusterTypesConverter::template ToHexString<>(From.Y));
}

template <> inline FString DisplayClusterTypesConverter::ToHexString<>(const FRotator& From)
{
	return FString::Printf(TEXT("P=%s Y=%s R=%s"),
		*DisplayClusterTypesConverter::template ToHexString<>(From.Pitch),
		*DisplayClusterTypesConverter::template ToHexString<>(From.Yaw),
		*DisplayClusterTypesConverter::template ToHexString<>(From.Roll));
}

template <> inline FString DisplayClusterTypesConverter::ToHexString<>(const FMatrix& From)
{
	FString Result;

	for (int i = 0; i < 4; ++i)
	{
		Result += FString::Printf(TEXT("[%s %s %s %s] "),
			*DisplayClusterTypesConverter::template ToHexString<>(From.M[i][0]),
			*DisplayClusterTypesConverter::template ToHexString<>(From.M[i][1]),
			*DisplayClusterTypesConverter::template ToHexString<>(From.M[i][2]),
			*DisplayClusterTypesConverter::template ToHexString<>(From.M[i][3]));
	}

	return Result;
}

template <> inline FString DisplayClusterTypesConverter::ToHexString<>(const FTransform& From)
{
	return FString::Printf(TEXT("%s|%s|%s"),
		*DisplayClusterTypesConverter::template ToHexString<>(From.GetLocation()),
		*DisplayClusterTypesConverter::template ToHexString<>(From.GetRotation().Rotator()),
		*DisplayClusterTypesConverter::template ToHexString<>(From.GetScale3D()));
}

template <> inline FString DisplayClusterTypesConverter::ToHexString<>(const FQuat& From)
{
	return FString::Printf(TEXT("X=%s Y=%s Z=%s W=%s"),
		*DisplayClusterTypesConverter::template ToHexString<>(From.X),
		*DisplayClusterTypesConverter::template ToHexString<>(From.Y),
		*DisplayClusterTypesConverter::template ToHexString<>(From.Z),
		*DisplayClusterTypesConverter::template ToHexString<>(From.W));
}



//////////////////////////////////////////////////////////////////////////////////////////////
// HEX STRING --> TYPE
//////////////////////////////////////////////////////////////////////////////////////////////
template <> inline float DisplayClusterTypesConverter::FromHexString<>(const FString& From)
{
	float Result = 0.f;
	checkSlow(From.Len() == 2 * sizeof(Result));
	HexToBytes(From, reinterpret_cast<uint8*>(&Result));
	return Result;
}

template <> inline double DisplayClusterTypesConverter::FromHexString<>(const FString& From)
{
	double Result = 0.f;
	checkSlow(From.Len() == 2 * sizeof(Result));
	HexToBytes(From, reinterpret_cast<uint8*>(&Result));
	return Result;
}

template <> inline FVector DisplayClusterTypesConverter::FromHexString<>(const FString& From)
{
	FString X, Y, Z;
	FVector Result = FVector::ZeroVector;

	const bool bSuccessful = FParse::Value(*From, TEXT("X="), X) && FParse::Value(*From, TEXT("Y="), Y) && FParse::Value(*From, TEXT("Z="), Z);
	if (bSuccessful)
	{
		Result.X = DisplayClusterTypesConverter::template FromHexString<FVector::FReal>(X);
		Result.Y = DisplayClusterTypesConverter::template FromHexString<FVector::FReal>(Y);
		Result.Z = DisplayClusterTypesConverter::template FromHexString<FVector::FReal>(Z);
	}

	return Result;
}

template <> inline FVector2D DisplayClusterTypesConverter::FromHexString<>(const FString& From)
{
	FString X, Y;
	FVector2D Result = FVector2D::ZeroVector;

	const bool bSuccessful = FParse::Value(*From, TEXT("X="), X) && FParse::Value(*From, TEXT("Y="), Y);
	if (bSuccessful)
	{
		Result.X = DisplayClusterTypesConverter::template FromHexString<FVector2D::FReal>(X);
		Result.Y = DisplayClusterTypesConverter::template FromHexString<FVector2D::FReal>(Y);
	}

	return Result;
}

template <> inline FRotator DisplayClusterTypesConverter::FromHexString<>(const FString& From)
{
	FString P, Y, R;
	FRotator Result = FRotator::ZeroRotator;

	const bool bSuccessful = FParse::Value(*From, TEXT("P="), P) && FParse::Value(*From, TEXT("Y="), Y) && FParse::Value(*From, TEXT("R="), R);
	if (bSuccessful)
	{
		Result.Pitch = DisplayClusterTypesConverter::template FromHexString<FRotator::FReal>(P);
		Result.Yaw   = DisplayClusterTypesConverter::template FromHexString<FRotator::FReal>(Y);
		Result.Roll  = DisplayClusterTypesConverter::template FromHexString<FRotator::FReal>(R);
	}

	return Result;
}

template <> inline FMatrix DisplayClusterTypesConverter::FromHexString<>(const FString& From)
{
	FMatrix ResultMatrix = FMatrix::Identity;
	FPlane  Planes[4];

	int32 IdxStart = 0;
	int32 IdxEnd = 0;
	for (int PlaneNum = 0; PlaneNum < 4; ++PlaneNum)
	{
		IdxStart = From.Find(FString(TEXT("[")), ESearchCase::IgnoreCase, ESearchDir::FromStart, IdxEnd);
		IdxEnd   = From.Find(FString(TEXT("]")), ESearchCase::IgnoreCase, ESearchDir::FromStart, IdxStart);
		if (IdxStart == INDEX_NONE || IdxEnd == INDEX_NONE || (IdxEnd - IdxStart) < 8)
		{
			return ResultMatrix;
		}

		FString StrPlane = From.Mid(IdxStart + 1, IdxEnd - IdxStart - 1);

		int32 StrLen = 0;
		int32 NewStrLen = -1;
		while (NewStrLen < StrLen)
		{
			StrLen = StrPlane.Len();
			StrPlane.ReplaceInline(TEXT("  "), TEXT(" "));
			NewStrLen = StrPlane.Len();
		}

		TArray<FString> StrPlaneValues;
		StrPlane.ParseIntoArray(StrPlaneValues, TEXT(" "));
		if (StrPlaneValues.Num() != 4)
		{
			return ResultMatrix;
		}

		FMatrix::FReal PlaneValues[4] = { 0.f };
		for (int i = 0; i < 4; ++i)
		{
			PlaneValues[i] = DisplayClusterTypesConverter::template FromHexString<FMatrix::FReal>(StrPlaneValues[i]);
		}

		Planes[PlaneNum] = FPlane(PlaneValues[0], PlaneValues[1], PlaneValues[2], PlaneValues[3]);
	}

	ResultMatrix = FMatrix(Planes[0], Planes[1], Planes[2], Planes[3]);

	return ResultMatrix;
}

template <> inline FTransform DisplayClusterTypesConverter::FromHexString<>(const FString& From)
{
	TArray<FString> ComponentStrings;
	From.ParseIntoArray(ComponentStrings, TEXT("|"), true);
	if (ComponentStrings.Num() != 3)
	{
		return FTransform::Identity;
	}

	const FVector  ParsedTranslation = DisplayClusterTypesConverter::template FromHexString<FVector>(ComponentStrings[0]);
	const FRotator ParsedRotation    = DisplayClusterTypesConverter::template FromHexString<FRotator>(ComponentStrings[1]);
	const FVector  ParsedScale       = DisplayClusterTypesConverter::template FromHexString<FVector>(ComponentStrings[2]);

	const FTransform Result(ParsedRotation, ParsedTranslation, ParsedScale);

	return Result;
}

template <> inline FQuat DisplayClusterTypesConverter::FromHexString<>(const FString& From)
{
	FString X, Y, Z, W;
	FQuat Result = FQuat::Identity;

	const bool bSuccessful = FParse::Value(*From, TEXT("X="), X) && FParse::Value(*From, TEXT("Y="), Y) && FParse::Value(*From, TEXT("Z="), Z) && FParse::Value(*From, TEXT("W="), W);
	if (bSuccessful)
	{
		Result.X = DisplayClusterTypesConverter::template FromHexString<FQuat::FReal>(X);
		Result.Y = DisplayClusterTypesConverter::template FromHexString<FQuat::FReal>(Y);
		Result.Z = DisplayClusterTypesConverter::template FromHexString<FQuat::FReal>(Z);
		Result.W = DisplayClusterTypesConverter::template FromHexString<FQuat::FReal>(W);
	}

	return Result;
}
