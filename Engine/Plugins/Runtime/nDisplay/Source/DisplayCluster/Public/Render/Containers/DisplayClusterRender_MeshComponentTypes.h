// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

// Support special mesh proxy geometry funcs
enum class EDisplayClusterRender_MeshComponentProxyDataFunc : uint8
{
	Disabled = 0,
	OutputRemapScreenSpace
};

// Assigned geometry type
enum class EDisplayClusterRender_MeshComponentGeometrySource: uint8
{
	Disabled = 0,
	StaticMeshComponentRef,
	ProceduralMeshComponentRef,
	ProceduralMeshSection,
	StaticMeshAsset,
	MeshGeometry,
};

/** The vertex data used to filter a texture. */
struct FDisplayClusterMeshVertex
{
	FVector4 Position;
	FVector2D UV;
	FVector2D UV_Chromakey;
};

// Map source geometry UVs to DCMeshComponent UVs
struct FDisplayClusterMeshUVs
{
	enum class ERemapTarget: uint8
	{
		Base = 0,
		Chromakey,
		COUNT
	};

	FDisplayClusterMeshUVs() = default;

	FDisplayClusterMeshUVs(const int32 InBaseUVIndex, const int32 InChromakeyUVIndex)
	{
		RemapValue[(uint8)ERemapTarget::Base]      = InBaseUVIndex;
		RemapValue[(uint8)ERemapTarget::Chromakey] = InChromakeyUVIndex;
	}

	const int32 operator[](const ERemapTarget RemapTarget) const
	{
		check((uint8)RemapTarget < (uint8)ERemapTarget::COUNT);

		return RemapValue[(uint8)RemapTarget];
	}

	const int32 GetSourceGeometryUVIndex(const ERemapTarget RemapTarget, const int32 MaxValue, const int32 DefaultValue) const
	{
		const int32 CurrentValue = operator[](RemapTarget);

		// Replace INDEX_NONE to default value
		const int32 Result = (CurrentValue == INDEX_NONE) ? DefaultValue : CurrentValue;

		// Return UV in range 0..MaxValue
		return (Result >= 0 && Result < MaxValue) ? Result : 0;
	}

private:
	int32 RemapValue[(uint8)ERemapTarget::COUNT] = {INDEX_NONE, INDEX_NONE};
};
