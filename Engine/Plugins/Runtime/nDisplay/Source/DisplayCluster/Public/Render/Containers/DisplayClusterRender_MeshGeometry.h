// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

// Helper: load mesh geometry from OBJ file

enum class EDisplayClusterRender_MeshGeometryFormat : uint8
{
	OBJ = 0,
};

enum class EDisplayClusterRender_MeshGeometryCreateType : uint8
{
	Passthrough = 0,
};


class DISPLAYCLUSTER_API FDisplayClusterRender_MeshGeometry
{
public:
	FDisplayClusterRender_MeshGeometry() = default;
	FDisplayClusterRender_MeshGeometry(EDisplayClusterRender_MeshGeometryCreateType CreateType);
	FDisplayClusterRender_MeshGeometry(const FDisplayClusterRender_MeshGeometry& In);
	~FDisplayClusterRender_MeshGeometry() = default;

public:
	// Load geometry from OBJ file
	bool LoadFromFile(const FString& FullPathFileName, EDisplayClusterRender_MeshGeometryFormat Format = EDisplayClusterRender_MeshGeometryFormat::OBJ);
	// Test purpose: create square geometry
	void CreatePassthrough();

public:
	TArray<FVector>   Vertices;
	TArray<FVector>   Normal;
	TArray<FVector2D> UV;
	TArray<FVector2D> ChromakeyUV; // Optional
	TArray<int32>     Triangles;
};
