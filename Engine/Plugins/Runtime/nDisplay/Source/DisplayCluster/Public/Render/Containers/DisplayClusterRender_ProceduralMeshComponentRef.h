// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Misc/DisplayClusterObjectRef.h"

class UProceduralMeshComponent;

class DISPLAYCLUSTER_API FDisplayClusterRender_ProceduralMeshComponentRef
	: public FDisplayClusterSceneComponentRef
{
public:
	FDisplayClusterRender_ProceduralMeshComponentRef()
		: FDisplayClusterSceneComponentRef()
	{ }

	FDisplayClusterRender_ProceduralMeshComponentRef(const FDisplayClusterRender_ProceduralMeshComponentRef& In)
		: FDisplayClusterSceneComponentRef(In)
		, bIsProceduralMeshGeometryDirty(In.bIsProceduralMeshGeometryDirty)
	{ }

	// Get or find the referenced ProceduralMesh component
	UProceduralMeshComponent* GetOrFindProceduralMeshComponent() const;

	bool SetProceduralMeshComponentRef(UProceduralMeshComponent* InProceduralMeshComponent);
	void ResetProceduralMeshComponentRef();

	bool IsProceduralMeshGeometryDirty() const;
	void MarkProceduralMeshGeometryDirty() const;
	void ResetProceduralMeshGeometryDirty() const;

private:
	mutable bool bIsProceduralMeshGeometryDirty = false;
};
