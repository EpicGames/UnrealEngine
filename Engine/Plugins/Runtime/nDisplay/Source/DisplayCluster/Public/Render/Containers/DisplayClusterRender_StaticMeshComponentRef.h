// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Misc/DisplayClusterObjectRef.h"

class UStaticMeshComponent;

class DISPLAYCLUSTER_API FDisplayClusterRender_StaticMeshComponentRef
	: public FDisplayClusterSceneComponentRef
{
public:
	FDisplayClusterRender_StaticMeshComponentRef()
		: FDisplayClusterSceneComponentRef()
	{ }

	FDisplayClusterRender_StaticMeshComponentRef(const FDisplayClusterRender_StaticMeshComponentRef& In)
		: FDisplayClusterSceneComponentRef(In)
		, bIsStaticMeshGeometryDirty(In.bIsStaticMeshGeometryDirty)
		, MeshGeometryName(In.MeshGeometryName)
	{ }

	// Get or find the referenced StaticMesh component
	UStaticMeshComponent* GetOrFindStaticMeshComponent() const;

	bool SetStaticMeshComponentRef(UStaticMeshComponent* InStaticMeshComponent);
	void ResetStaticMeshComponentRef();

	bool IsStaticMeshGeometryDirty() const;
	void MarkStaticMeshGeometryDirty() const;
	void ResetStaticMeshGeometryDirty() const;

private:
	mutable bool bIsStaticMeshGeometryDirty = false;
	// Compares the assigned static mesh geometry by name, and raise for changed [mutable] bIsStaticMeshGeometryDirty
	FName MeshGeometryName;
};
