// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Render/Containers/DisplayClusterRender_MeshComponentTypes.h"
#include "Render/Containers/DisplayClusterRender_MeshGeometry.h"

class USceneComponent;
class UMeshComponent;
class UStaticMesh;
class UStaticMeshComponent;
class UProceduralMeshComponent;

class IDisplayClusterRender_MeshComponentProxy;

struct FStaticMeshLODResources;
struct FProcMeshSection;

class DISPLAYCLUSTER_API IDisplayClusterRender_MeshComponent
{
public:
	virtual ~IDisplayClusterRender_MeshComponent() = default;

public:
	/**
	* Assign static mesh component and origin to this container
	* Then send geometry from the StaticMesh component to proxy
	*
	* @param InStaticMeshComponent - Pointer to the StaticMesh component with geometry for warp
	* @param InUVs                 - Map source geometry UVs to DCMeshComponent UVs
	* @param InOriginComponent     - (optional) stage origin point component
	* @param InLODIndex            - (optional) use geometry from LOD index
	*/
	virtual void AssignStaticMeshComponentRefs(UStaticMeshComponent* InStaticMeshComponent, const FDisplayClusterMeshUVs& InUVs, USceneComponent* InOriginComponent = nullptr, int32 InLODIndex = 0) = 0;

	/**
	* Assign procedural mesh component, section index and origin to this container
	* Then send geometry from the ProceduralMesh(SectionIndex) to proxy
	*
	* @param InProceduralMeshComponent - Pointer to the ProceduralMesh component with geometry for warp
	* @param InUVs                     - Map source geometry UVs to DCMeshComponent UVs
	* @param InOriginComponent         - (optional) stage origin point component
	* @param InSectionIndex            - (optional) The section index in ProceduralMesh
	*/
	virtual void AssignProceduralMeshComponentRefs(UProceduralMeshComponent* InProceduralMeshComponent, const FDisplayClusterMeshUVs& InUVs, USceneComponent* InOriginComponent = nullptr, const int32 InSectionIndex = 0) = 0;

	/**
	* Assign procedural mesh section geometry
	* Then send geometry from the ProcMeshSection to proxy
	*
	* @param InProcMeshSection - Pointer to the Proce mesh section geometry
	* @param InUVs             - Map source geometry UVs to DCMeshComponent UVs
	*/
	virtual void AssignProceduralMeshSection(const FProcMeshSection& InProcMeshSection, const FDisplayClusterMeshUVs& InUVs) = 0;

	/**
	* Assign static mesh geometry
	* Then send geometry from the StaticMesh to proxy
	*
	* @param InStaticMesh - Pointer to the StaticMesh geometry
	* @param InUVs        - Map source geometry UVs to DCMeshComponent UVs
	* @param InLODIndex   - (optional) use geometry from LOD index
	*/
	virtual void AssignStaticMesh(const UStaticMesh* InStaticMesh, const FDisplayClusterMeshUVs& InUVs, int32 InLODIndex = 0) = 0;

	/**
	* Assign mesh geometry
	* Then send geometry to proxy
	*
	* @param InMeshGeometry - Pointer to the source geometry data
	*/
	virtual void AssignMeshGeometry(const FDisplayClusterRender_MeshGeometry* InMeshGeometry) = 0;

	/**
	* Create mesh geometry on render thread (debug purpose)
	*
	* @param InMeshGeometry - Pointer to the source geometry data
	* @param InDataFunc     - Geometry function type
	*/
	virtual void AssignMeshGeometry_RenderThread(const FDisplayClusterRender_MeshGeometry* InMeshGeometry, const EDisplayClusterRender_MeshComponentProxyDataFunc InDataFunc) const = 0;

	/**
	* Release refs, proxy, RHI.
	*
	*/
	virtual void ReleaseMeshComponent() = 0;

	/**
	* Release proxy
	*
	*/
	virtual void ReleaseProxyGeometry() = 0;

	/**
	* Get referenced Origin component object
	*
	* @return - pointer to the Origin component
	*/
	virtual USceneComponent* GetOriginComponent() const = 0;

	/**
	* Get referenced Mesh component object
	*
	* @return - pointer to the Mesh component
	*/
	virtual UMeshComponent* GetMeshComponent() const = 0;

	/**
	* Get referenced StaticMesh component object
	*
	* @return - pointer to the StaticMesh component
	*/
	virtual UStaticMeshComponent* GetStaticMeshComponent() const = 0;

	/**
	* Get referenced StaticMesh object
	*
	* @return - pointer to the StaticMesh object
	*/
	virtual const UStaticMesh* GetStaticMesh() const = 0;

	/**
	* Get referenced ProceduralMesh component object
	*
	* @return - pointer to the ProceduralMesh component
	*/
	virtual UProceduralMeshComponent* GetProceduralMeshComponent() const = 0;

	/**
	* Get geometry from assigned StaticMesh component
	*
	* @param LodIndex - (optional) lod index
	*
	* @return - pointer to the static mesh geometry
	*/
	virtual const FStaticMeshLODResources* GetStaticMeshComponentLODResources(int32 InLODIndex = 0) const = 0;

	/**
	* Get geometry from assigned ProceduralMesh section by index
	*
	* @param SectionIndex - Geometry source section index
	*
	* @return - pointer to the geometry section data
	*/
	virtual const FProcMeshSection* GetProceduralMeshComponentSection(const int32 SectionIndex = 0) const = 0;

	/**
	* Get nDisplay mesh component proxy object
	*
	* @return - pointer to the nDisplay mesh component proxy
	*/
	virtual IDisplayClusterRender_MeshComponentProxy* GetMeshComponentProxy_RenderThread() const = 0;

	/**
	* Set geometry preprocess function. This function change geometry prepared for proxy
	*
	* @param InDataFunc - geometry function type
	*/
	virtual void SetGeometryFunc(const EDisplayClusterRender_MeshComponentProxyDataFunc InDataFunc) = 0;

	/**
	* Return true if referenced component geometry changed
	*/
	virtual bool IsMeshComponentRefGeometryDirty() const = 0;

	/**
	* Mark referenced component geometry as changed
	*/
	virtual void MarkMeshComponentRefGeometryDirty() const = 0;

	/**
	* Clear referenced component geometry changed flag
	*/
	virtual void ResetMeshComponentRefGeometryDirty() const = 0;

	/**
	* return true, if referenced component name is equal
	*/
	virtual bool EqualsMeshComponentName(const FName& InMeshComponentName) const = 0;

	/**
	* return assigned geometry source type
	*/
	virtual EDisplayClusterRender_MeshComponentGeometrySource GetGeometrySource() const = 0;
};
