// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "RHI.h"
#include "RHICommandList.h"

class DISPLAYCLUSTER_API IDisplayClusterRender_MeshComponentProxy
{
public:
	virtual ~IDisplayClusterRender_MeshComponentProxy() = default;

public:
	/**
	* Initialize RHI for mesh render
	* 
	* return - true, if the mesh can be rendered
	*/
	virtual bool BeginRender_RenderThread(FRHICommandListImmediate& RHICmdList, FGraphicsPipelineStateInitializer& GraphicsPSOInit) const = 0;

	/**
	* Render mesh geometry
	*
	* return - true, if the mesh rendered
	*/
	virtual bool FinishRender_RenderThread(FRHICommandListImmediate& RHICmdList) const = 0;

	/**
	* Return true, if mesh valid and can be rendered
	*/
	virtual bool IsEnabled_RenderThread() const = 0;
};
