// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "RHI.h"
#include "RHIResources.h"
#include "RenderResource.h"

/**
 * A runtime texture that is created based on data in memory 
  */
class DISPLAYCLUSTER_API IDisplayClusterRender_Texture
{
public:
	virtual ~IDisplayClusterRender_Texture() = default;

public:
	/** Get TSharePtr from this. */
	virtual TSharedPtr<IDisplayClusterRender_Texture, ESPMode::ThreadSafe> ToSharedPtr() = 0;
	virtual TSharedPtr<const IDisplayClusterRender_Texture, ESPMode::ThreadSafe> ToSharedPtr() const = 0;

	/**
	* Create texture from memory data
	*
	* @param InTextureData    - Pointer to texture data in memory
	* @param InComponentDepth - Components per pixel
	* @param InBitDepth       - Bit per component
	* @param InWidth          - Texture width in pixels
	* @param InHeight         - Texture height in pixels
	* @param bInHasCPUAccess  - has CPU access to texture data, dont release from memory
	*
	* @return - true, if the texture resource was created successfully
	*/
	virtual void CreateTexture(const void* InTextureData, const uint32 InComponentDepth, const uint32 InBitDepth, uint32_t InWidth, uint32_t InHeight, bool bInHasCPUAccess = false) = 0;

	/** True if the texture resource is initialized. */
	virtual bool IsEnabled() const = 0;

	/** Get ptr to resource data in memory (The texture must be created with bInHasCPUAccess set to true). */
	virtual void* GetData() const = 0;

	/** Get texture width. */
	virtual uint32_t GetWidth() const = 0;

	/** Get texture height. */
	virtual uint32_t GetHeight() const = 0;
	
	/** Get texture pixel format. */
	virtual EPixelFormat GetPixelFormat() const = 0;

	/** Get number of components. */
	virtual uint32 GetComponentDepth() const = 0;

	/** Get the RHI resource. */
	virtual FRHITexture* GetRHITexture() const = 0;
};
