// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "StereoRendering.h"

#include "Render/Viewport/RenderFrame/DisplayClusterRenderFrameEnums.h"

class IDisplayClusterPresentation;
struct FDisplayClusterRenderViewContext;
class FDisplayClusterRenderFrame;
class IDisplayClusterProjectionPolicy;
class UWorld;
class FViewport;


/**
 * nDisplay render device interface
 */
class IDisplayClusterRenderDevice : public IStereoRendering
{
public:
	virtual ~IDisplayClusterRenderDevice() = default;

public:

	/**
	* Device initialization
	*
	* @return - true if success
	*/
	virtual bool Initialize()
	{ return true; }

	/**
	* Called on a scene start to allow a rendering device to initialize any world related content
	*/
	virtual void StartScene(UWorld* World)
	{ }

	/**
	* Called before scene Tick
	*/
	virtual void PreTick(float DeltaSeconds)
	{ }

	/**
	* Called before unload current level
	*/
	virtual void EndScene()
	{ }

	/**
	* Update settings from root actor config data, and build new frame structure
	* This function also updates all viewport settings from config.
	* Must be called before the InitializeNewFrame() function.
	*/
	virtual bool BeginNewFrame(FViewport* InViewport, UWorld* InWorld, FDisplayClusterRenderFrame& OutRenderFrame) = 0;

	/**
	* Initialize internal data to render a new frame
	* Initialize references to DCRA objects.
	* Must be called before viewports start rendering
	* [ This pair of functions InitializeNewFrame\FinalizeNewFrame should always be called together. ]
	*/
	virtual void InitializeNewFrame() = 0;

	/**
	* Finish rendering the frame and compose the viewports for the final RTT (Warp&Blend, OCIO, ICVFX, etc.)
	* Release references to DCRA objects.
	* Must be called after all viewports have been rendered.
	* [ This pair of functions InitializeNewFrame\FinalizeNewFrame should always be called together. ]
	*/
	virtual void FinalizeNewFrame() = 0;

	/**
	* Returns current presentation handler
	*
	* @return - nullptr if failed
	*/
	virtual IDisplayClusterPresentation* GetPresentation() const = 0;

};
