// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class IDisplayClusterRenderDevice;


/**
 * nDisplay rendering device factory interface
 */
class IDisplayClusterRenderDeviceFactory
{
public:
	virtual ~IDisplayClusterRenderDeviceFactory() = default;

public:
	/**
	* Creates a rendering device instance
	*
	* @param InDeviceType - device type that has been specified on registering the factory (may be useful if the same factory is responsible for multiple device types)
	*
	* @return - rendering device
	*/
	virtual TSharedPtr<IDisplayClusterRenderDevice, ESPMode::ThreadSafe> Create(const FString& InDeviceType) = 0;
};
