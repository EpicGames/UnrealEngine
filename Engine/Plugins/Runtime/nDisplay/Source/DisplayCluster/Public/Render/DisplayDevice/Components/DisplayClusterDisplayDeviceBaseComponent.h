// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Render/DisplayDevice/Containers/DisplayClusterDisplayDevice_Enums.h"
#include "Components/SceneComponent.h"
#include "Materials/MaterialInstanceDynamic.h"

#include "DisplayClusterDisplayDeviceBaseComponent.generated.h"

class UMaterial;
class FSceneView;
class FSceneViewFamily;
class UMeshComponent;
class UDisplayClusterCameraComponent;
class UWorld;

class IDisplayClusterDisplayDeviceProxy;
class IDisplayClusterViewportPreview;
class IDisplayClusterViewportConfiguration;

/**
 * Display Device Components can be added to nDisplay root actors and assigned to viewport nodes to allow additional
 * processing on the preview material.
 */
UCLASS(Abstract, ClassGroup = (DisplayCluster), HideCategories = (Transform, Rendering, Tags, Activation, Cooking, Physics, LOD, AssetUserData, Navigation))
class DISPLAYCLUSTER_API UDisplayClusterDisplayDeviceBaseComponent
	: public USceneComponent
{
	GENERATED_BODY()
public:
	UDisplayClusterDisplayDeviceBaseComponent();

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	
public:
	/** Receive material by type
	* 
	* @param InMeshType     - mesh type
	* @param InMaterialType - the type of material being requested
	*/
	virtual TObjectPtr<UMaterial> GetDisplayDeviceMaterial(const EDisplayClusterDisplayDeviceMeshType InMeshType, const EDisplayClusterDisplayDeviceMaterialType InMaterialType) const;
	
	/** Perform any operations on the  material instance, such as setting parameter values.
	* 
	* @param InViewportPreview  - current viewport
	* @param InMeshType         - mesh type
	* @param InMaterialType     - type of material being requested
	* @param InMeshComponent - mesh component to be updated
	* @param InMeshMaterialInstance - material instance that used on this mesh
	*/
	virtual void OnUpdateDisplayDeviceMeshAndMaterialInstance(IDisplayClusterViewportPreview& InViewportPreview, const EDisplayClusterDisplayDeviceMeshType InMeshType, const EDisplayClusterDisplayDeviceMaterialType InMaterialType, UMeshComponent* InMeshComponent, UMaterialInstanceDynamic* InMeshMaterialInstance) const;

	/** Update and Get proxy object for rendering thread.
	* 
	* @param InConfiguration - current viewport configuration
	*/
	virtual TSharedPtr<IDisplayClusterDisplayDeviceProxy, ESPMode::ThreadSafe> GetDisplayDeviceProxy(IDisplayClusterViewportConfiguration& InConfiguration);

	/** Setup view for this Display Device
	 *
	 * @param InViewport      - the current viewport
	 * @param ContextNum      - context index (eye)
	 * @param InOutViewFamily - [In,Out] ViewFamily.
	 * @param InOutView       - [In,Out] View.
	 *
	 * @return - none.
	 */
	virtual void SetupSceneView(const IDisplayClusterViewportPreview& InViewportPreview, uint32 ContextNum, FSceneViewFamily& InOutViewFamily, FSceneView& InOutView) const;

protected:
	/** Updating the proxy object for the current configuration.
	* If the current configuration does not allow for a display device, this function will remove the DisplayDeviceProxy variable.
	*
	* @param InConfiguration - current viewport configuration
	*/
	virtual void UpdateDisplayDeviceProxyImpl(IDisplayClusterViewportConfiguration& InConfiguration);

	/** Returns true if the display device should be used for rendering in nDisplay */
	virtual bool ShouldUseDisplayDevice(IDisplayClusterViewportConfiguration& InConfiguration) const;

protected:
	/** If render passes are enabled. */
	UPROPERTY(EditAnywhere, Category=RenderPass)
	bool bEnableRenderPass = false;
	
	/** The material to assign to the static mesh when preview is disabled. */
	UPROPERTY(EditAnywhere, Category = Material, NoClear)
	TObjectPtr<UMaterial> MeshMaterial = nullptr;

	/** The material the preview components will create a material instance from when rendering the nDisplay preview. */
	UPROPERTY(EditAnywhere, Category=Material, NoClear)
	TObjectPtr<UMaterial> PreviewMeshMaterial = nullptr;

	/** [techvis] The material the preview components will create a material instance from when rendering the nDisplay preview. */
	UPROPERTY(EditAnywhere, Category = Material, NoClear)
	TObjectPtr<UMaterial> PreviewMeshTechvisMaterial = nullptr;

protected:
	/** Display device proxy object. */
	TSharedPtr<IDisplayClusterDisplayDeviceProxy, ESPMode::ThreadSafe> DisplayDeviceProxy;
};
