// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Render/DisplayDevice/Components/DisplayClusterDisplayDeviceBaseComponent.h"
#include "OpenColorIOColorSpace.h"

#include "DisplayClusterDisplayDeviceComponent.generated.h"

class UMaterial;
class UMaterialInstanceDynamic;
class UTexture;
class UTextureRenderTarget2D;

/**
* Display Device Component with OCIO render pass
*/
UCLASS(ClassGroup = (DisplayCluster), meta = (BlueprintSpawnableComponent, DisplayName = "NDisplay Display Device"))
class DISPLAYCLUSTER_API UDisplayClusterDisplayDeviceComponent
	: public UDisplayClusterDisplayDeviceBaseComponent
{
	GENERATED_BODY()

public:
	UDisplayClusterDisplayDeviceComponent();

	//~BEGIN UDisplayClusterDisplayDeviceBaseComponent
	virtual void OnUpdateDisplayDeviceMeshAndMaterialInstance(IDisplayClusterViewportPreview& InViewportPreview, const EDisplayClusterDisplayDeviceMeshType InMeshType, const EDisplayClusterDisplayDeviceMaterialType InMaterialType, UMeshComponent* InMeshComponent, UMaterialInstanceDynamic* InMeshMaterialInstance) const override;
	virtual void UpdateDisplayDeviceProxyImpl(IDisplayClusterViewportConfiguration& InConfiguration) override;
	//~~END UDisplayClusterDisplayDeviceBaseComponent

protected:
	/** Adjust the exposure for the emissive input. */
	UPROPERTY(EditAnywhere, Category=Material)
	float Exposure = 0.f;

	/** Adjust the gamma for the emissive input. */
	UPROPERTY(EditAnywhere, Category = Material)
	float Gamma = 1.f;

	/** Color grading settings. */
	UPROPERTY(EditAnywhere, Category = RenderPass, meta = (DisplayAfter="bEnableRenderPass"))
	FOpenColorIOColorConversionSettings ColorConversionSettings;
};
