// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
* Display Device materials
*/
enum class EDisplayClusterDisplayDeviceMaterialType : uint8
{
	// Preview on mesh
	PreviewMeshMaterial,

	// Preview on mesh for techvis
	PreviewMeshTechvisMaterial,
};

/**
* Display Device mesh type
*/
enum class EDisplayClusterDisplayDeviceMeshType : uint8
{
	// Default mesh without preview
	DefaultMesh = 0,

	// Preview mesh
	PreviewMesh,

	// Preview editable mesh
	PreviewEditableMesh,

};
