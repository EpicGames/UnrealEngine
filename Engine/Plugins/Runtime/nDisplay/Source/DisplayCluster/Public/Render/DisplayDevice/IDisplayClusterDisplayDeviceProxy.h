// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RHI.h"
#include "RHIResources.h"
#include "EngineUtils.h"
#include "ScreenRendering.h"
#include "ScreenPass.h"
#include "PostProcess/PostProcessMaterialInputs.h"

class FDisplayClusterViewport_Context;
class FRDGBuilder;
class FSceneView;
class IDisplayClusterShadersTextureUtils;
struct FDisplayClusterShadersTextureUtilsSettings;
struct FPostProcessMaterialInputs;
struct FScreenPassTexture;

/**
 * Display Device Proxy object interface
 * [rendering thread]
 */
class DISPLAYCLUSTER_API IDisplayClusterDisplayDeviceProxy
{
public:
	virtual ~IDisplayClusterDisplayDeviceProxy() = default;

public:
	/** Is final pass used
	*/
	virtual bool HasFinalPass_RenderThread() const
	{
		return false;
	}

	/** Add render pass at the end of nDisplay pipeline.
	 *
	 * @param InTextureUtilsSettings - A settings for render
	 * @param InTextureUtils         - A container of resources for processing.
	 *
	 * @return - true if success.
	 */
	virtual bool AddFinalPass_RenderThread(const FDisplayClusterShadersTextureUtilsSettings& InTextureUtilsSettings, const TSharedRef<IDisplayClusterShadersTextureUtils, ESPMode::ThreadSafe>& InTextureUtils) const
	{
		return false;
	}

	/** Allow callback OnPostProcessPassAfterFXAA.  */
	virtual bool ShouldUsePostProcessPassAfterFXAA() const
	{ return false; }

	/** Callback OnPostProcessPassAfterFXAA.
	 *
	 * @param GraphBuilder - RDG interface
	 * @param View         - Scene View
	 * @param Inputs       - PP Input resources
	 * @param ContextNum   - viewport context index
	 *
	 * @return - Screen pass texture
	 */
	virtual FScreenPassTexture OnPostProcessPassAfterFXAA_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& View, const FPostProcessMaterialInputs& Inputs, const uint32 ContextNum)
	{ return Inputs.ReturnUntouchedSceneColorForPostProcessing(GraphBuilder); }

	/** Allow callback OnPostProcessPassAfterSSRInput. */
	virtual bool ShouldUsePostProcessPassAfterSSRInput() const
	{ return false; }

	/** Callback OnPostProcessPassAfterSSRInput.
	 *
	 * @param GraphBuilder - RDG interface
	 * @param View         - Scene View
	 * @param Inputs       - PP Input resources
	 * @param ContextNum   - viewport context index
	 *
	 * @return - Screen pass texture
	 */
	virtual FScreenPassTexture OnPostProcessPassAfterSSRInput_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& View, const FPostProcessMaterialInputs& Inputs, const uint32 ContextNum)
	{ return Inputs.ReturnUntouchedSceneColorForPostProcessing(GraphBuilder); }

	/** Allow callback OnPostProcessPassAfterTonemap. */
	virtual bool ShouldUsePostProcessPassTonemap() const
	{ return false; }

	/** Callback OnPostProcessPassAfterTonemap.
	 *
	 * @param GraphBuilder - RDG interface
	 * @param View         - Scene View
	 * @param Inputs       - PP Input resources
	 * @param ContextNum   - viewport context index
	 *
	 * @return - Screen pass texture
	 */
	virtual FScreenPassTexture OnPostProcessPassAfterTonemap_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& View, const FPostProcessMaterialInputs& Inputs, const uint32 ContextNum)
	{ return Inputs.ReturnUntouchedSceneColorForPostProcessing(GraphBuilder); }

	/** Add render pass at the end of nDisplay pipeline.
	 *
	 * @param GraphBuilder      - RDG interface.
	 * @param InViewportContext - DC viewport context.
	 * @param InputTextureRHI   - Source texture.
	 * @param InputRect         - Source rec.
	 * @param OutputTextureRHI  - Destination texture.
	 * @param OutputRect        - Destination rec.
	 *
	 * @return - true if success.
	 */
	UE_DEPRECATED(5.6, "This function has been deprecated. Please use 'AddFinalPass_RenderThread()'.")
	virtual bool AddFinalPass_RenderThread(FRDGBuilder& GraphBuilder, const FDisplayClusterViewport_Context& InViewportContext,
		FRHITexture* InputTextureRHI, const FIntRect& InputRect, FRHITexture* OutputTextureRHI, const FIntRect& OutputRect) const
	{
		return false;
	}
};
