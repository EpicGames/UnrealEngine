// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHIFwd.h"
#include "Math/MathFwd.h"


/**
 * GUI controller interface
 * 
 * Represents a mediator that controls how Slate/UMG output is rendered
 * to the backbuffer. It also provides access to the texture containing
 * the GUI (the so-called GUI layer).
 */
class IDisplayClusterGUILayerController
{
public:

	virtual ~IDisplayClusterGUILayerController() = default;

public:

	/**
	 * Supported operation modes
	 */
	enum class EGuiOutputMode : uint8
	{
		Disabled   = 0, // GUI layer controller is completely disabled
		Hidden     = 1, // Never render to the output
		Whole      = 2, // Render in the output (backbuffer) space like UE natively does
		Propagated = 3, // Per-viewport GUI output
	};

public:

	/** Return current operation mode */
	virtual EGuiOutputMode GetOutputMode() const = 0;

	/** Returns true if GUI controller is currently enabled */
	virtual bool IsEnabled() const = 0;

	/** Returns current GUI layer texture size, or {0, 0} if disabled. */
	virtual FIntPoint GetGuiLayerTextureSize() const = 0;

	/** Returns current GUI layer texture if available, otherwise invalid reference */
	virtual FTextureRHIRef GetGuiLayerTexture_RenderThread() = 0;
};
