// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class IDisplayClusterGUILayerController;
class IDisplayClusterRenderDevice;
class IDisplayClusterPostProcess;
class IDisplayClusterPostProcessFactory;
class IDisplayClusterRender_MeshComponent;
class IDisplayClusterRender_Texture;
class IDisplayClusterProjectionPolicy;
class IDisplayClusterProjectionPolicyFactory;
class IDisplayClusterWarpPolicyFactory;
class IDisplayClusterRenderDeviceFactory;
class IDisplayClusterRenderSyncPolicy;
class IDisplayClusterRenderSyncPolicyFactory;
class IDisplayClusterVblankMonitor;
class IDisplayClusterViewportManager; 
class UCineCameraComponent;
struct FPostProcessSettings;


/**
 * Public render manager interface
 */
class IDisplayClusterRenderManager
{
public:
	virtual ~IDisplayClusterRenderManager() = default;

public:
	// Post-process operation wrapper
	struct FDisplayClusterPPInfo
	{
		FDisplayClusterPPInfo(TSharedPtr<IDisplayClusterPostProcess, ESPMode::ThreadSafe>& InOperation, int32 InPriority)
			: Operation(InOperation)
			, Priority(InPriority)
		{ }

		TSharedPtr<IDisplayClusterPostProcess, ESPMode::ThreadSafe> Operation;
		int32 Priority;
	};

public:
	/**
	* Returns current rendering device
	*
	* @return - nullptr if failed
	*/
	virtual IDisplayClusterRenderDevice* GetRenderDevice() const = 0;

	/**
	* Registers rendering device factory
	*
	* @param DeviceType - Type of rendering device
	* @param Factory    - Factory instance
	*
	* @return - True if success
	*/
	virtual bool RegisterRenderDeviceFactory(const FString& DeviceType, TSharedPtr<IDisplayClusterRenderDeviceFactory>& Factory) = 0;

	/**
	* Unregisters rendering device factory
	*
	* @param DeviceType - Type of rendering device
	*
	* @return - True if success
	*/
	virtual bool UnregisterRenderDeviceFactory(const FString& DeviceType) = 0;

	/**
	* Registers synchronization policy factory
	*
	* @param SyncPolicyType - Type of synchronization policy
	* @param Factory        - Factory instance
	*
	* @return - True if success
	*/
	virtual bool RegisterSynchronizationPolicyFactory(const FString& SyncPolicyType, TSharedPtr<IDisplayClusterRenderSyncPolicyFactory>& Factory) = 0;

	/**
	* Unregisters synchronization policy factory
	*
	* @param SyncPolicyType - Type of synchronization policy
	*
	* @return - True if success
	*/
	virtual bool UnregisterSynchronizationPolicyFactory(const FString& SyncPolicyType) = 0;

	/**
	* Returns currently active rendering synchronization policy object
	*
	* @return - Rendering synchronization policy object
	*/
	virtual TSharedPtr<IDisplayClusterRenderSyncPolicy> GetCurrentSynchronizationPolicy() = 0;

	/**
	* Registers projection policy factory
	*
	* @param ProjectionType - Type of projection data (MPCDI etc.)
	* @param Factory        - Factory instance
	*
	* @return - True if success
	*/
	virtual bool RegisterProjectionPolicyFactory(const FString& ProjectionType, TSharedPtr<IDisplayClusterProjectionPolicyFactory>& Factory) = 0;

	/**
	* Unregisters projection policy factory
	*
	* @param ProjectionType - Type of projection policy
	*
	* @return - True if success
	*/
	virtual bool UnregisterProjectionPolicyFactory(const FString& ProjectionType) = 0;

	/**
	* Returns a projection policy factory of specified type (if it has been registered previously)
	*
	* @param ProjectionType - Projection policy type
	*
	* @return - Projection policy factory pointer or nullptr if not registered
	*/
	virtual TSharedPtr<IDisplayClusterProjectionPolicyFactory> GetProjectionPolicyFactory(const FString& ProjectionType) = 0;

	/**
	* Returns all registered projection policy types
	*
	* @param OutPolicyIDs - (out) array to put registered IDs
	*/
	virtual void GetRegisteredProjectionPolicies(TArray<FString>& OutPolicyIDs) const = 0;

	/**
	* Registers PostProcess factory
	*
	* @param PostProcessType - Type of PostProcess data
	* @param Factory        - Factory instance
	*
	* @return - True if success
	*/
	virtual bool RegisterPostProcessFactory(const FString& PostProcessType, TSharedPtr<IDisplayClusterPostProcessFactory>& Factory) = 0;

	/**
	* Unregisters PostProcess factory
	*
	* @param PostProcessType - Type of post process
	*
	* @return - True if success
	*/
	virtual bool UnregisterPostProcessFactory(const FString& PostProcessType) = 0;

	/**
	* Returns a postprocess factory of specified type (if it has been registered previously)
	*
	* @param PostProcessType - PostProcess type
	*
	* @return - PostProcess factory pointer or nullptr if not registered
	*/
	virtual TSharedPtr<IDisplayClusterPostProcessFactory> GetPostProcessFactory(const FString& PostProcessType) = 0;

	/**
	* Returns all registered postprocess types
	*
	* @param OutPostProcessIDs - (out) array to put registered IDs
	*/
	virtual void GetRegisteredPostProcess(TArray<FString>& OutPostProcessIDs) const = 0;

	/**
	* Registers warp policy factory
	*
	* @param InWarpPolicyType - Type of warp policy
	* @param Factory          - Factory instance
	*
	* @return - True if success
	*/
	virtual bool RegisterWarpPolicyFactory(const FString& InWarpPolicyType, TSharedPtr<IDisplayClusterWarpPolicyFactory>& Factory) = 0;

	/**
	* Unregisters warp policy factory
	*`
	* @param InWarpPolicyType - Type of warp policy
	*
	* @return - True if success
	*/
	virtual bool UnregisterWarpPolicyFactory(const FString& InWarpPolicyType) = 0;

	/**
	* Returns a warp policy factory of specified type (if it has been registered previously)
	*
	* @param InWarpPolicyType - Warp policy type
	*
	* @return - Warp policy factory pointer or nullptr if not registered
	*/
	virtual TSharedPtr<IDisplayClusterWarpPolicyFactory> GetWarpPolicyFactory(const FString& InWarpPolicyType) = 0;

	/**
	* Returns all registered warp policy types
	*
	* @param OutWarpPolicyIDs - (out) array to put registered IDs
	*/
	virtual void GetRegisteredWarpPolicies(TArray<FString>& OutWarpPolicyIDs) const = 0;

	/**
	* Returns V-blank monitor interface
	*/
	virtual TSharedPtr<IDisplayClusterVblankMonitor, ESPMode::ThreadSafe> GetVblankMonitor() = 0;

	/**
	* @return - Current viewport manager from root actor
	*/
	virtual IDisplayClusterViewportManager* GetViewportManager() const = 0;

	/**
	* @return - new mesh component object
	*/
	virtual TSharedPtr<IDisplayClusterRender_MeshComponent, ESPMode::ThreadSafe> CreateMeshComponent() const = 0;

	/**
	 * Get or create a cached texture with unique name
	 */
	virtual TSharedPtr<IDisplayClusterRender_Texture, ESPMode::ThreadSafe> GetOrCreateCachedTexture(const FString& InUniqueTextureName) const = 0;

	/**
	 * Returns GUI layer controller interface
	 */
	virtual IDisplayClusterGUILayerController& GetGuiLayerController() const = 0;
};
