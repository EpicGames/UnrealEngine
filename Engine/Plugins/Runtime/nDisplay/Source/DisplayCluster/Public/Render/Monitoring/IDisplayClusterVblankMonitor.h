// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"


/**
 * V-blank monitor interface
 */
class IDisplayClusterVblankMonitor
{
public:
	virtual ~IDisplayClusterVblankMonitor() = default;

public:
	/** Start monitoring */
	virtual bool StartMonitoring() = 0;

	/** Stop monitoring */
	virtual bool StopMonitoring() = 0;

	/** Returns true if currently monitoring */
	virtual bool IsMonitoring() = 0;

	/** Returns true if last and next expected V-blank timestamps are known */
	virtual bool IsVblankTimeDataAvailable() = 0;

	/** Returns last V-blank timestamp */
	virtual double GetLastVblankTime() = 0;

	/** Returns next expected V-blank timestamp */
	virtual double GetNextVblankTime() = 0;
};
