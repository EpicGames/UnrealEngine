// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "RHI.h"
#include "RHIResources.h"
#include "RHICommandList.h"

class IDisplayClusterViewportManager;
class IDisplayClusterViewportManagerProxy;
class IDisplayClusterViewportProxy;
class FDisplayClusterRenderFrame;

/**
 * nDisplay post-process interface
 */
class IDisplayClusterPostProcess
{
public:
	virtual ~IDisplayClusterPostProcess() = default;

public:
	/**
	* Return postprocess name
	*/
	virtual const FString& GetId() const = 0;

	/**
	* Return postprocess order
	*/
	virtual int32 GetOrder() const = 0;

	/**
	* Return postprocess type
	*/
	virtual const FString& GetType() const = 0;

	/**
	* Return postprocess configuration
	*/
	virtual const TMap<FString, FString>& GetParameters() const = 0;

	/**
	* Update postprocess internal data from game thread
	*/
	virtual void Tick()
	{ }

	/**
	* Check postprocess settings changes
	*
	* @param InConfigurationProjectionPolicy - new settings
	*
	* @return - True if found changes
	*/
	virtual bool IsConfigurationChanged(const struct FDisplayClusterConfigurationPostprocess* InConfigurationPostprocess) const = 0;

	/**
	* Called each time a new game level starts
	*
	*/
	virtual bool HandleStartScene(IDisplayClusterViewportManager* InViewportManager) = 0;

	/**
	* Called when current level is going to be closed (i.e. before loading a new map)
	*
	*/
	virtual void HandleEndScene(IDisplayClusterViewportManager* InViewportManager) = 0;

	/**
	* Called each time a new frame setup
	*
	*/
	virtual void HandleSetupNewFrame(IDisplayClusterViewportManager* InViewportManager)
	{ }

	/**
	* Called each time a new frame begin
	*
	*/
	virtual void HandleBeginNewFrame(IDisplayClusterViewportManager* InViewportManager, FDisplayClusterRenderFrame& InOutRenderFrame)
	{ }

	/**
	* Called each time a render thread multi-viewport composing begin
	*
	*/
	virtual void HandleRenderFrameSetup_RenderThread(FRHICommandListImmediate& RHICmdList, const IDisplayClusterViewportManagerProxy* InViewportManagerProxy)
	{ }

	/**
	* Called each time a render thread before UpdateFrameResources()
	*
	*/
	virtual void HandleBeginUpdateFrameResources_RenderThread(FRHICommandListImmediate& RHICmdList, const IDisplayClusterViewportManagerProxy* InViewportManagerProxy)
	{ }

	/**
	* Called every time the render thread runs after the entire warpblend has completed.
	*/
	virtual void HandleUpdateFrameResourcesAfterWarpBlend_RenderThread(FRHICommandListImmediate& RHICmdList, const IDisplayClusterViewportManagerProxy* InViewportManagerProxy)
	{ }

	/**
	* Called each time a render thread after UpdateFrameResources()
	*
	*/
	virtual void HandleEndUpdateFrameResources_RenderThread(FRHICommandListImmediate& RHICmdList, const IDisplayClusterViewportManagerProxy* InViewportManagerProxy)
	{ }

	/**
	* Returns if an interface implementation processes each view region before warp&blend
	* the corresponded function will be called once per viewport
	*
	* @return - true if required
	*/
	virtual bool IsPostProcessViewBeforeWarpBlendRequired() const
	{ return false; }

	/**
	* PP operation on a view region before warp&blend (if available for the current projection policy)
	* the corresponded function will be called once per viewport
	*
	* @param RHICmdList - RHI command list
	* @param ViewportProxy - viewport proxy interface
	*/
	virtual void PerformPostProcessViewBeforeWarpBlend_RenderThread(FRHICommandListImmediate& RHICmdList, const IDisplayClusterViewportProxy* ViewportProxy) const
	{ }

	/**
	* Returns if an interface implementation processes each view region after warp&blend
	* the corresponded function will be called once per viewport
	*
	* @return - true if required
	*/
	virtual bool IsPostProcessViewAfterWarpBlendRequired() const
	{ return false; }

	/**
	* PP operation on a view region after warp&blend (if available for the current projection policy)
	* the corresponded function will be called once per viewport
	*
	* @param RHICmdList - RHI command list
	* @param ViewportProxy - viewport proxy interface
	*/
	virtual void PerformPostProcessViewAfterWarpBlend_RenderThread(FRHICommandListImmediate& RHICmdList, const IDisplayClusterViewportProxy* ViewportProxy) const
	{ }

	/**
	* Returns if an interface implementation processes output frames after warp&blend
	*
	* @return - true if required
	*/
	virtual bool IsPostProcessFrameAfterWarpBlendRequired() const
	{ return false; }

	/**
	* Request additional frame targetable resource from viewport manager
	*
	* @return - true if resource required
	*/
	virtual bool ShouldUseAdditionalFrameTargetableResource() const
	{ return false; }

	/**
	* PP operation on a frame region after warp&blend
	*
	* @param RHICmdList - RHI command list
	* @param FrameTargets - Frame textures array (1 for mono, 2 for stereo)
	*/
	virtual void PerformPostProcessFrameAfterWarpBlend_RenderThread(FRHICommandListImmediate& RHICmdList, const TArray<FRHITexture*>* InFrameTargets = nullptr, const TArray<FRHITexture*>* InAdditionalFrameTargets = nullptr) const
	{ }
};
