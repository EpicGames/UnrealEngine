// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class IDisplayClusterPostProcess;


/**
 * nDisplay output post process factory interface
 */
class IDisplayClusterPostProcessFactory
{
public:
	virtual ~IDisplayClusterPostProcessFactory() = default;

public:
	/**
	* Creates a output post process instance
	*
	* @param PostProcessPolicyId - ID of a postprocess
	* @param InConfigurationPostProcess - postprocess configuration
	*
	* @return - Post process instance
	*/
	virtual TSharedPtr<IDisplayClusterPostProcess, ESPMode::ThreadSafe> Create(const FString& PostProcessId, const struct FDisplayClusterConfigurationPostprocess* InConfigurationPostProcess) = 0;
	
	/** Special rules for creating a PP instance.
	* Note: This function must be called before Create() on the nDisplay side.
	* (useful to avoid unnecessary nDisplay log messages).
	*
	* For example, some postprocess types may use an instance as a singleton.
	* In this case, Create() returns nullptr if the PP instance already exists,
	* and as a result nDisplay issues an error message to the log, and then avoids creating this PP in the future.
	*
	* @param InConfigurationPostProcess - postprocess configuration
	*
	* @return true if a postprocess with this type can be created at this time.
	*/
	virtual bool CanBeCreated(const FDisplayClusterConfigurationPostprocess* InConfigurationPostProcess) const
	{
		return true;
	}
};
