// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"


/**
 * Public display cluster presentation interface
 */
class IDisplayClusterPresentation
{
public:
	virtual ~IDisplayClusterPresentation() = default;
};
