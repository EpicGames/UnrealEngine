// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FRHICommandListImmediate;
class IDisplayClusterViewport;
class IDisplayClusterViewportProxy;
class IDisplayClusterWarpBlend;
class IDisplayClusterWarpPolicy;
class UMeshComponent;
class USceneComponent;
class UCameraComponent;
struct FDisplayClusterConfigurationProjection;
struct FMinimalViewInfo;


/**
 * The type of media usage for the viewport.
 */
enum class EDisplayClusterProjectionPolicyFlags : uint8
{
	// No flags
	None = 0,

	/** This projection policy use eye tracking (ViewPoint).
	* 
	* This means that the InOutViewLocation and InOutViewRotation arguments in CalculateView()
	* will contain the position of the viewpoint in world space, not the camera.
	*
	* Camera position is the value passed from IStereoRenderingDevice.
	* This value is usually based on the active engine camera,
	* which can be overridden in ISceneViewExtension::SetupViewPoint().
	* (see FDisplayClusterViewportManagerViewPointExtension::SetupViewPoint())
	* 
	* Most nDisplay projection policies (simple, mesh, easyblend, etc.) ignore this camera position
	* and use their own values or DCRA components to compute the view.
	* In this case, this flag replaces the IDisplayClusterProjectionPolicy::CalculateView() input arguments with the position of the
	* of the DCRA ViewPoint component instead of the camera position.
	* And the projection policy may not implement this logic.
	* (see IDisplayClusterViewport::GetViewPointCameraEye())
	*
	* Note:
	* On the nDisplay side, the camera position can already be overridden using the ViewPoint position for some cases:
	* UDisplayClusterCameraComponent::GetDesiredView()
	*  -> GetTargetCameraDesiredViewInternal()
	*  -> IsViewPointOverrideCameraPosition()
	*/
	UseEyeTracking = 1 << 0,

	/** Projection policies can use local space and custom geometric units.
	* 
	* Therefore, InOutViewLocation and InOutViewRotation must be converted from world space
	* to the projection policy space and back again.
	* 
	* The projection policy space is formed from the Origin component and scaling units.
	* see GetOriginComponent(), GetGeometryToMeters(), GetOriginLocationOffsetInGeometryUnits().
	*
	* This conversion will look like this:
	*   World -> ProjectionPolicy
	*     ScaleWorldToGeometry = GeometryToMeters / WorldToMeters
	*     PrjPolicyLocation = OriginToWorld.InverseTransformPosition(WorldSpaceLocation) * ScaleWorldToGeometry
	*     PrjPolicyRotation = OriginToWorld.InverseTransformRotation(WorldSpaceRotation)
	* and vice versa.
	*/
	UseLocalSpace = 1 << 1,

	/** This projection policy supports "Follow Camera" feature.
	* To support this feature, the projection policy must return an output ViewLocation in the DCRA zone.
	* 
	* Usually these values are calculated based on objects such as these:
	* ViewPoint component, ICVFXCamera component, CineCamera actor referenced by ICVFXCamera, etc.
	*/
	EnableFollowCameraFeature = 1 << 2,
};
ENUM_CLASS_FLAGS(EDisplayClusterProjectionPolicyFlags);

/**
 * nDisplay projection policy
 */
class IDisplayClusterProjectionPolicy
{
public:
	virtual ~IDisplayClusterProjectionPolicy() = default;

public:
	/**
	* Return projection policy instance ID
	*/
	virtual const FString& GetId() const = 0;

	/**
	* Return projection policy type
	*/
	virtual const FString& GetType() const = 0;

	/**
	* Return projection policy configuration
	*/
	virtual const TMap<FString, FString>& GetParameters() const = 0;

	/**
	 * Return Origin point component used by this viewport
	 * This component is used to convert from the local DCRA space to the local projection policy space,
	 * which contains the calibrated geometry data used for the warp.
	 */
	virtual USceneComponent* const GetOriginComponent() const
	{
		return nullptr;
	}

	/**
	 * Returns the camera component driving this projection policy, or nullptr if the policy does not use one.
	 */
	virtual UCameraComponent* GetCameraComponent() const
	{
		return nullptr;
	}

	/**
	* If the projection policy uses a custom scale internally, this function returns that value.
	* The value means how many geometry units are in one meter.
	* 
	* @param InViewport      - (in) a owner viewport
	* @param InWorldToMeters - (in) UE world units per meter
	*/
	virtual float GetGeometryToMeters(const IDisplayClusterViewport* InViewport, const float InWorldToMeters) const
	{
		// UE world units are used by default.
		return InWorldToMeters;
	}

	/** Returns Origin's location in geometry units.
	* 
	* The Origin component must be configured in UE units, but the calibration tools use their own units.
	* Therefore, the user must perform calculations to correctly set the location of the Origin component within the UE.
	* Typically, this approach leads to mistakes in the DCRA setup.
	* To avoid this, the projection policy can receive the Origin Location parameter in geometric units.
	* This function returns the value of this parameter.
	* 
	* @param InViewport - (in) a owner viewport
	*/
	virtual FVector GetOriginLocationOffsetInGeometryUnits(const IDisplayClusterViewport* InViewport) const
	{
		// A zero vector means it is not used.
		return FVector::ZeroVector;
	}

	/**
	* Send projection policy game thread data to render thread proxy
	* called once per frame from FDisplayClusterViewportManager::FinalizeNewFrame
	*/
	virtual void UpdateProxyData(IDisplayClusterViewport* InViewport)
	{ }

	/**
	* Called each time a new game level starts
	*
	* @param InViewport - a owner viewport
	*/
	virtual bool HandleStartScene(IDisplayClusterViewport* InViewport)
	{
		return true;
	}

	/**
	* Called when current level is going to be closed (i.e. before loading a new map)
	*
	* @param InViewport - a owner viewport
	*/
	virtual void HandleEndScene(IDisplayClusterViewport* InViewport)
	{ }

	/**
	 * Called after the base configuration is completed, but before the ICVFX
	 *
	 * @param InViewport - a owner viewport
	 */
	virtual void PostUpdateBaseConfiguration(IDisplayClusterViewport* InViewport) const
	{ }

	/**
	 * Called before FDisplayClusterViewport::UpdateFrameContexts()
	 * From this function, the policy can override any viewport settings (custom overscan, etc).
	 * 
	 * @param InViewport - a owner viewport
	 */
	virtual void BeginUpdateFrameContexts(IDisplayClusterViewport* InViewport) const
	{ }

	/**
	 * Called after FDisplayClusterViewport::UpdateFrameContexts()
	 * 
	 * @param InViewport - a owner viewport
	 */
	virtual void EndUpdateFrameContexts(IDisplayClusterViewport* InViewport) const
	{ }

	/**
	* Set warp policy for this projection
	*
	* @param InWarpPolicy - the warp policy instance
	*/
	virtual void SetWarpPolicy(IDisplayClusterWarpPolicy* InWarpPolicy)
	{ }

	/**
	* Get warp policy
	*/
	virtual IDisplayClusterWarpPolicy* GetWarpPolicy() const
	{
		return nullptr;
	}

	/**
	* Get warp policy on rendering thread
	*/
	virtual IDisplayClusterWarpPolicy* GetWarpPolicy_RenderThread() const
	{
		return nullptr;
	}

	// Handle request for additional render targetable resource inside viewport api for projection policy
	virtual bool ShouldUseAdditionalTargetableResource(const IDisplayClusterViewport* InViewport) const
	{ 
		return false; 
	}

	/**
	* Returns true if the policy supports input mip-textures.
	* Use a mip texture for smoother deformation on curved surfaces.
	*
	* @return - true, if mip-texture is supported by the policy implementation
	*/
	virtual bool ShouldUseSourceTextureWithMips(const IDisplayClusterViewport* InViewport) const
	{
		return false;
	}

	/** Returns true if this policy supports ICVFX rendering
	 * 
	 * @param InViewport - a owner viewport
	 */
	virtual bool ShouldSupportICVFX(const IDisplayClusterViewport* InViewport) const
	{
		return false;
	}

	// Return true, if camera projection visible for this viewport geometry
	// ICVFX Performance : if camera frame not visible on this node, disable render for this camera
	virtual bool IsCameraProjectionVisible(const FRotator& InViewRotation, const FVector& InViewLocation, const FMatrix& InProjectionMatrix)
	{
		return true;
	}

	/**
	* Check projection policy settings changes
	*
	* @param InConfigurationProjectionPolicy - new settings
	*
	* @return - True if found changes
	*/
	virtual bool IsConfigurationChanged(const struct FDisplayClusterConfigurationProjection* InConfigurationProjectionPolicy) const = 0;

	/** Override view from this projection policy
	 *
	 * @param InViewport                 - a owner viewport
	 * @param InDeltaTime                - delta time in current frame
	 * @param InOutViewInfo              - ViewInfo data
	 * @param OutCustomNearClippingPlane - Custom NCP, or a value less than zero if not defined.
	 */
	virtual void SetupProjectionViewPoint(IDisplayClusterViewport* InViewport, const float InDeltaTime, FMinimalViewInfo& InOutViewInfo, float* OutCustomNearClippingPlane = nullptr)
	{ }

	/* Returns true if this projection policy should use post-processes from the ViewPoint component. **/
	virtual bool ShouldUseViewPointComponentPostProcesses(IDisplayClusterViewport* InViewport) const
	{
		return true;
	}

	/** Projection policy can override PP
	* 
	* @param InViewport - a owner viewport
	*/
	virtual void UpdatePostProcessSettings(IDisplayClusterViewport* InViewport)
	{ }

	/**
	* Projection policy can modify the clipping planes.
	* 
	* @param InViewport - (in) a owner viewport
	* @param InOutNCP   - (in, out) Near clipping plane
	* @param InOutFCP   - (in, out) Far clipping plane
	*/
	virtual void ApplyClippingPlanesOverrides(const IDisplayClusterViewport* InViewport, float& InOutNCP, float& InOutFCP) const
	{ }

	/**
	* Return Projection policy flags
	*
	* @param InViewport   - (in) a owner viewport
	* @param InContextNum - Index of view that is being processed for this viewport
	*/
	virtual EDisplayClusterProjectionPolicyFlags GetProjectionPolicyFlags(const IDisplayClusterViewport* InViewport, const uint32 InContextNum) const
	{
		return EDisplayClusterProjectionPolicyFlags::None;
	}

	/** Calculate view projection data
	* 
	* @param InViewport        - a owner viewport
	* @param InContextNum      - Index of view that is being processed for this viewport
	* @param InOutViewLocation - (in/out) View location with ViewOffset (i.e. left eye pre-computed location)
	* @param InOutViewRotation - (in/out) View rotation
	* @param ViewOffset        - Offset applied ot a camera location that gives us InOutViewLocation (i.e. right offset in world to compute right eye location)
	* @param WorldToMeters     - Current world scale (units (cm) in meter)
	* @param NCP               - Distance to the near clipping plane
	* @param FCP               - Distance to the far  clipping plane
	*
	* @return - True if success
	*/
	virtual bool CalculateView(IDisplayClusterViewport* InViewport, const uint32 InContextNum, FVector& InOutViewLocation, FRotator& InOutViewRotation, const FVector& ViewOffset, const float WorldToMeters, const float NCP, const float FCP) = 0;

	/** Gets projection matrix
	* 
	* @param InViewport   - a owner viewport
	* @param InContextNum - Index of view that is being processed for this viewport
	* @param OutPrjMatrix - (out) projection matrix
	*
	* @return - True if success
	*/
	virtual bool GetProjectionMatrix(IDisplayClusterViewport* InViewport, const uint32 InContextNum, FMatrix& OutPrjMatrix) = 0;

	/** Returns true if the frustum has been rotated to fit the context size.
	* 
	* @param InViewport   - a owner viewport
	* @param InContextNum - Index of view that is being processed for this viewport
	* 
	* @return true is success.
	*/
	virtual bool IsFrustumRotatedToFitContextSize(IDisplayClusterViewport* InViewport, const uint32 InContextNum)
	{
		return false;
	}

	/** Returns true if the frustum has been rotated to fit the context size.
	*
	* @param InViewportProxy - The owning viewport proxy
	* @param InContextNum - Index of view that is being processed for this viewport
	*
	* @return True if succeeded.
	*/
	virtual bool IsFrustumRotatedToFitContextSize_RenderThread(const IDisplayClusterViewportProxy* InViewportProxy, const uint32 InContextNum)
	{
		return false;
	}

	/**
	* Returns if a policy provides warp&blend feature
	*
	* @return - True if warp&blend operations are supported by the policy implementation
	*/
	virtual bool IsWarpBlendSupported(const IDisplayClusterViewport* InViewport) const
	{
		return false;
	}

	/**
	* Returns if a policy provides warp&blend feature
	* (
	*
	* @return - True if warp&blend operations are supported by the policy implementation
	*/
	virtual bool IsWarpBlendSupported_RenderThread(const IDisplayClusterViewportProxy* InViewportProxy) const
	{
		return false;
	}

	/**
	* This function can override the size of the RenderTarget texture for the viewport.
	*
	* @param InViewport          - viewport to override RTT size.
	* @param OutRenderTargetSize - (out) the desired RTT size.
	*
	* @return is true if the RTT size should be overridden.
	*/
	virtual bool GetCustomRenderTargetSize(const IDisplayClusterViewport* InViewport, FIntPoint& OutRenderTargetSize) const
	{
		return false;
	}

	/**
	* This function controls the RTT size multipliers - if false is returned, all modifiers should be ignored.
	* Here is the list of ignored modifiers:
	*    RenderTargetAdaptRatio, RenderTargetRatio,
	*    ClusterRenderTargetRatioMult,
	*    ClusterICVFXOuterViewportRenderTargetRatioMult, ClusterICVFXInnerViewportRenderTargetRatioMult
	* 
	* @param InViewport - the DC viewport.
	*/
	virtual bool ShouldUseAnySizeScaleForRenderTarget(const IDisplayClusterViewport* InViewport) const
	{
		return true;
	}

	/**
	* Initializing the projection policy logic for the current frame before applying warp blending.
	* Have to be called only if IsWarpBlendSupported_RenderThread() returns true
	*
	* @param RHICmdList      - RHI commands
	* @param InViewportProxy - viewport proxy
	*
	*/
	virtual void BeginWarpBlend_RenderThread(FRHICommandListImmediate& RHICmdList, const IDisplayClusterViewportProxy* InViewportProxy)
	{ }

	/**
	* Performs Warp and Blend on the rendering thread
	* Have to be called only if IsWarpBlendSupported_RenderThread() returns true
	*
	* @param RHICmdList      - RHI commands
	* @param InViewportProxy - viewport proxy
	*
	*/
	virtual void ApplyWarpBlend_RenderThread(FRHICommandListImmediate& RHICmdList, const IDisplayClusterViewportProxy* InViewportProxy)
	{ }

	/**
	* Completing the projection policy logic for the current frame after applying warp blending.
	* Have to be called only if IsWarpBlendSupported_RenderThread() returns true
	*
	* @param ViewIdx      - Index of view that is being processed for this viewport
	* @param RHICmdList   - RHI commands
	* @param SrcTexture   - Source texture
	* @param ViewportRect - Region of the SrcTexture to perform warp&blend operations
	*
	*/
	virtual void EndWarpBlend_RenderThread(FRHICommandListImmediate& RHICmdList, const IDisplayClusterViewportProxy* InViewportProxy)
	{ }

	/**
	* Return warpblend interface of this policy
	*
	* @param OutWarpBlendInterface - output interface ref
	* 
	* @return - true if output interface valid
	*/
	virtual bool GetWarpBlendInterface(TSharedPtr<IDisplayClusterWarpBlend, ESPMode::ThreadSafe>& OutWarpBlendInterface) const
	{
		return false;
	}

	/**
	* Return warpblend interface proxy of this policy
	*
	* @param OutWarpBlendInterfaceProxy - output interface ref
	*
	* @return - true if output interface valid
	*/
	virtual bool GetWarpBlendInterface_RenderThread(TSharedPtr<IDisplayClusterWarpBlend, ESPMode::ThreadSafe>& OutWarpBlendInterfaceProxy) const
	{
		return false;
	}

	/**
	* Override copying 'InternalRenderTargetResource' to 'InputShaderResource'.
	* The same behavior as for function bellow  is expected:
	*     ResolveResources_RenderThread(RHICmdList, InSourceViewportProxy, InSrcResourceType, InDstResourceType);
	*
	* @param InViewportProxy       - This ViewportProxy will get the result
	* @param InSourceViewportProxy - This ViewportProxy will provide the input resource
	* 
	* @return - true, if copying is overridden.
	*/
	virtual bool ResolveInternalRenderTargetResource_RenderThread(FRHICommandListImmediate& RHICmdList, const IDisplayClusterViewportProxy* InViewportProxy, const IDisplayClusterViewportProxy* InSourceViewportProxy)
	{
		return false;
	}

	/**
	* Ask projection policy instance if it has any mesh based preview
	*
	* @param InViewport - a owner viewport
	* @return - True if mesh based preview is available
	*/
	virtual bool HasPreviewMesh(IDisplayClusterViewport* InViewport)
	{
		return false;
	}

	/**
	* Build preview mesh
	* This MeshComponent cannot be moved freely.
	* This MeshComponent is attached to the geometry from the real world via Origin.
	* When the screen geometry in the real world changes position, the position of this component must also be changed.
	*
	* @param InViewport - Projection specific parameters.
	* @param bOutIsRootActorComponent - return true, if used custom root actor component. return false, if created unique temporary component
	*/
	virtual UMeshComponent* GetOrCreatePreviewMeshComponent(IDisplayClusterViewport* InViewport, bool& bOutIsRootActorComponent)
	{
		return nullptr;
	}

	/**
	 * Return Origin point component used by preview mesh
	 * 
	 * @param InViewport - a owner viewport
	 */
	virtual USceneComponent* const GetPreviewMeshOriginComponent(IDisplayClusterViewport* InViewport) const
	{
		return nullptr;
	}

	/**
	* Ask projection policy instance if it has any Editable mesh based preview
	* 
	* @param InViewport - a owner viewport
	* @return - True if mesh based preview is available
	*/
	virtual bool HasPreviewEditableMesh(IDisplayClusterViewport* InViewport)
	{
		return false;
	}

	/**
	* Build preview Editable mesh
	* This MeshComponent is a copy of the preview mesh and can be moved freely with the UI visualization.
	*
	* @param InViewport - a owner viewport
	*/
	virtual UMeshComponent* GetOrCreatePreviewEditableMeshComponent(IDisplayClusterViewport* InViewport)
	{
		return nullptr;
	}

	/**
	 * Return Origin point component used by preview Editable mesh
	 * 
	 * @param InViewport - a owner viewport
	 */
	virtual USceneComponent* const GetPreviewEditableMeshOriginComponent(IDisplayClusterViewport* InViewport) const
	{
		return nullptr;
	}

	//////////// UE_DEPRECATED 5.3 ////////////

	// This policy can support ICVFX rendering
	UE_DEPRECATED(5.3, "This function has been deprecated. Please use 'ShouldSupportICVFX(IDisplayClusterViewport*)'.")
	virtual bool ShouldSupportICVFX() const
	{
		return false;
	}

	//////////// UE_DEPRECATED 5.4 ////////////

	/**
	* Ask projection policy instance if it has any mesh based preview
	*
	* @return - True if mesh based preview is available
	*/
	UE_DEPRECATED(5.4, "This function has been deprecated. Please use 'HasPreviewMesh(IDisplayClusterViewport*)'.")
	virtual bool HasPreviewMesh()
	{
		return false;
	}

	/**
	* Returns if a policy provides warp&blend feature
	*
	* @return - True if warp&blend operations are supported by the policy implementation
	*/
	UE_DEPRECATED(5.6, "This function has been deprecated. Please use 'IsWarpBlendSupported(IDisplayClusterViewport*) or IsWarpBlendSupported_RenderThread(IDisplayClusterViewportProxy*)'.")
	virtual bool IsWarpBlendSupported()
	{
		return false;
	}

	// Handle request for additional render targetable resource inside viewport api for projection policy
	UE_DEPRECATED(5.6, "This function has been deprecated. Please use 'ShouldUseAdditionalTargetableResource(IDisplayClusterViewport*).")
	virtual bool ShouldUseAdditionalTargetableResource() const
	{
		return false;
	}

	/**
	* Returns true if the policy supports input mip-textures.
	* Use a mip texture for smoother deformation on curved surfaces.
	*
	* @return - true, if mip-texture is supported by the policy implementation
	*/
	UE_DEPRECATED(5.6, "This function has been deprecated. Please use 'ShouldUseSourceTextureWithMips(IDisplayClusterViewport*).")
	virtual bool ShouldUseSourceTextureWithMips() const
	{
		return false;
	}

};
