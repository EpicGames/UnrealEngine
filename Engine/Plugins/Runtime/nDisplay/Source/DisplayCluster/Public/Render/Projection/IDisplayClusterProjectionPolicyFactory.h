// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class IDisplayClusterProjectionPolicy;


/**
 * nDisplay projection policy factory interface
 */
class IDisplayClusterProjectionPolicyFactory
{
public:
	virtual ~IDisplayClusterProjectionPolicyFactory() = default;

public:
	/**
	* Creates a projection policy instance
	*
	* @param PolicyType - Projection policy type, same as specified on registration (useful if the same factory is registered for multiple projection types)
	* @param ProjectionPolicyId - ID of a policy
	* @param InConfigurationProjectionPolicy - policy configuration
	*
	* @return - Projection policy instance
	*/
	virtual TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe> Create(const FString& ProjectionPolicyId, const struct FDisplayClusterConfigurationProjection* InConfigurationProjectionPolicy) = 0;
	
};
