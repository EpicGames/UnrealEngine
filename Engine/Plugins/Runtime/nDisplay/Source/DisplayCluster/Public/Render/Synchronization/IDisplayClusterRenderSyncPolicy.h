// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RHIFwd.h"

/**
 * Public render synchronization policy interface
 */
class IDisplayClusterRenderSyncPolicy
{
public:
	virtual ~IDisplayClusterRenderSyncPolicy() = default;

public:
	/**
	* Perform any non-constructor initialization if needed
	*
	* @return - true if initialized successfully
	*/
	virtual bool Initialize()
	{
		return true;
	}

	/**
	 * Returns the name of the sync policy
	 */
	virtual FName GetName() const = 0;

	/**
	* Synchronizes rendering threads in a cluster (optionally presents a frame)
	*
	* @param InOutSyncInterval - Sync interval (VSync)
	*
	* @return - true if we a caller needs to present frame by its own
	*/
	virtual bool SynchronizeClusterRendering(FRHIViewport* ViewportRHI, int32& InOutSyncInterval) = 0;

	/**
	 * Returns true if policy instance support "WaitForVBlank" feature that is
	 * used on the higher levels
	 */
	virtual bool IsWaitForVBlankFeatureSupported()
	{
		return false;
	}

	/**
	 * Wait unless the next V-blank occurs. This function may be called after
	 * IsWaitForVBlankFeatureSupported returned true. Otherwise it will never be called.
	*
	* @return - true if v-blank awaiting was successfull
	 */
	virtual bool WaitForVBlank(FRHIViewport* Viewport)
	{
		return false;
	}
};
