// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class IDisplayClusterRenderSyncPolicy;


/**
 * nDisplay rendering synchronization policy factory interface
 */
class IDisplayClusterRenderSyncPolicyFactory
{
public:
	virtual ~IDisplayClusterRenderSyncPolicyFactory() = default;

public:
	/**
	* Creates a rendering device instance
	*
	* @param InPolicyType - Synchronization policy type that has been specified on registering the factory (may be useful if the same factory is responsible for multiple policies types)
	*
	* @return - rendering device
	*/
	virtual TSharedPtr<IDisplayClusterRenderSyncPolicy> Create(const FString& InPolicyType, const TMap<FString, FString>& Parameters) = 0;
};
