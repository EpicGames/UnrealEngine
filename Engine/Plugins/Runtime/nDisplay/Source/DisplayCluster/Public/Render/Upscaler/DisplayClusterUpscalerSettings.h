// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_Enums.h"
#include "StructUtils/PropertyBag.h"

/** Determines AntiAliasing method. */
enum class EDisplayClusterUpscalerAntiAliasingMethod : uint8
{
	// Use default AA from the project settings
	Default = 0,

	// Disable AA
	None,

	// Fast Approximate Anti-Aliasing (FXAA)
	FXAA,

	// Multisample Anti-Aliasing (MSAA)
	// Only supported with forward shading.  MSAA sample count is controlled by r.MSAACount.
	MSAA,

	// Temporal Anti-Aliasing (TAA)
	TAA,

	// Temporal Super-Resolution (TSR)
	TSR,
};

/**
 * nDisplay Upscaler settings
 */
class FDisplayClusterUpscalerSettings
{
public:
	FDisplayClusterUpscalerSettings() = default;

public:
	// Determines AntiAliasing method.
	EDisplayClusterUpscalerAntiAliasingMethod AntiAliasingMethod = EDisplayClusterUpscalerAntiAliasingMethod::Default;

	// Custom custom upscaler name
	FName CustomUpscalerName;

	// Custom upscaler settings
	FInstancedPropertyBag CustomUpscalerSettings;

	// parameters for the Functor logic(IUpscalerModularFeature::AddSceneViewExtensionIsActiveFunctor()).
	// Enable to use upscaler ViewExtension
	bool bIsActive = true;
};
