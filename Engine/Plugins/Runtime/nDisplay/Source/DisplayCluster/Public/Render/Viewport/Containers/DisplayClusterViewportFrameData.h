// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Camera/CameraTypes.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_Context.h"

/**
 * Per-frame snapshot of a viewport's render state: eye contexts and cached camera view info.
 */
struct FDisplayClusterViewportFrameData
{
	/** Reset to an empty/uninitialized state, ready to be populated for the next frame. */
	inline void Reset()
	{
		FrameNumber = 0;
		Contexts.Reset();
		CachedCameraViewInfo.Reset();
	}

	/** Returns true if no contexts have been populated for this frame yet. */
	inline bool IsEmpty() const
	{
		return Contexts.IsEmpty();
	}

public:
	/** Frame number this snapshot was captured on. */
	uint32 FrameNumber = 0;

	/** Per-eye render contexts (mono, stereo left, stereo right).
	 *  mutable: updated by the render thread through the const IDisplayClusterViewportProxy interface. */
	mutable TArray<FDisplayClusterViewport_Context> Contexts;

	/** Cached camera view info for FDisplayClusterViewport::GetCameraViewPoint(). */
	TOptional<FMinimalViewInfo> CachedCameraViewInfo;
};
