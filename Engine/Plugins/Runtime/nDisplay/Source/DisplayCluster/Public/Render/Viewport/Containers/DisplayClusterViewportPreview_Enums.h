// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * Runtime configuration from DCRA.
 */
enum class EDisplayClusterViewportPreviewFlags : uint8
{
	None = 0,

	// the RTT has been changed
	HasChangedPreviewRTT = 1 << 0,

	// This viewport has a valid preview texture.
	HasValidPreviewRTT = 1 << 1,

	// Preview mesh material instance has been changed.
	HasChangedPreviewMeshMaterialInstance = 1 << 2,

	// Preview editable mesh material instance has been changed.
	HasChangedPreviewEditableMeshMaterialInstance = 1 << 3,
};
ENUM_CLASS_FLAGS(EDisplayClusterViewportPreviewFlags);
