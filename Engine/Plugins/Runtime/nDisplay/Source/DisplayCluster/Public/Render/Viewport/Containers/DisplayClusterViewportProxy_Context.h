// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
* The viewport proxy context (rendering thread data)
*/
struct FDisplayClusterViewportProxy_Context
{
	// The number of context
	uint32  ContextNum = 0;

	// Profile description from ViewFamily
	FString ViewFamilyProfileDescription;
};
