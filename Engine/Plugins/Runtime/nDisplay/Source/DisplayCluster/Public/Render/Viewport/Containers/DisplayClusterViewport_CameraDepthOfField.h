// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/WeakObjectPtr.h"

class FSceneView;
class UTexture2D;

/**
* Settings for the camera depth of field blur.
*/
class FDisplayClusterViewport_CameraDepthOfField
{
public:
	/** Sets the appropriate depth of field settings in the scene view */
	void SetupSceneView(FSceneView& InOutView) const;

	/** At the beginning of each frame, all settings must be restored to default. */
	void BeginUpdateSettings()
	{
		// Reset all values to default values.
		*this = FDisplayClusterViewport_CameraDepthOfField();
	}

public:
	/** Indicates whether depth of field correction is enabled for the ICVFX camera */
	bool bEnableDepthOfFieldCompensation = false;

	/** The distance from the ICVFX camera to the wall it is shooting against */
	float DistanceToWall = 0.0;

	/** An offset applied to the distance to wall value */
	float DistanceToWallOffset = 0.0;

	/** Look-up texture that encodes the specific amount of compensation used for each combination of wall distance and object distance */
	TWeakObjectPtr<UTexture2D> CompensationLUT;
};