// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
* Camera motion blur mode.
*/
enum class EDisplayClusterViewport_CameraMotionBlur : uint8
{
	Undefined = 0,
	Off,
	On,
	Override
};

/**
* Settings for the CameraMotion blur.
*/
struct FDisplayClusterViewport_CameraMotionBlur
{
	EDisplayClusterViewport_CameraMotionBlur Mode = EDisplayClusterViewport_CameraMotionBlur::Undefined;

	FVector  CameraLocation = FVector::ZeroVector;
	FRotator CameraRotation = FRotator::ZeroRotator;

	float    TranslationScale = 1.f;
};
