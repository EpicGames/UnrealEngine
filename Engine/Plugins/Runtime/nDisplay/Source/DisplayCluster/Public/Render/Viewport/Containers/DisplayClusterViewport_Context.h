// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "StereoRendering.h"
#include "ShowFlags.h"
#include "Camera/CameraTypes.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_Enums.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_EyeAdaptation.h"

/**
 * Viewport context with cahched data and states
 */
class FDisplayClusterViewport_Context
{
public:
	FDisplayClusterViewport_Context(const uint32 InContextNum, const EStereoscopicPass InStereoscopicPass, const int32 InStereoViewIndex)
		: ContextNum(InContextNum)
		, StereoscopicPass(InStereoscopicPass)
		, StereoViewIndex(InStereoViewIndex)
	{ }

	/** Returns the effective projection matrix for rendering. */
	inline const FMatrix& GetRenderProjectionMatrix() const
	{
		return ProjectionData.bUseOverscan ?
			OverscanProjectionMatrix :
			ProjectionMatrix;
	}

	/** Returns the effective near-plane frustum extents for rendering (X=Left, Y=Right, Z=Top, W=Bottom), in world units. */
	inline const FVector4& GetRenderNearPlaneExtents() const
	{
		return ProjectionData.bUseOverscan
			? ProjectionData.OverscanProjectionAngles
			: ProjectionData.ProjectionAngles;
	}

	/**
	 * Returns the viewport content rect within the render target, excluding overscan padding.
	 */
	inline const FIntRect& GetViewportContentRect() const
	{
		return ProjectionData.bUseOverscan
			? OverscanInnerRenderTargetRect 
			: RenderTargetRect;
	}

public:
	/**
	* Declare all bitfields together to avoid alignment and padding.
	*/

	// True if this ViewData have a valid data.
	uint8 bValidData : 1 = 0;

	// Is this data have been calculated.
	uint8 bCalculated : 1 = 0;

	// Enables nDisplay's native implementation of cross-GPU transfer.
	// This disables cross-GPU transfer by default for nDisplay viewports in FSceneViewFamily structure.
	uint8 bOverrideCrossGPUTransfer : 1 = 0;

	// Disable render for this viewport (Overlay)
	uint8 bDisableRender : 1 = 0;

public:
	// Index of context (eye #)
	const uint32            ContextNum;

	// References to IStereoRendering
	EStereoscopicPass StereoscopicPass;
	int32             StereoViewIndex;

	// Cached ViewInfo
	TOptional<FMinimalViewInfo> CachedViewPoint;

	struct FCachedViewData
	{
		/**
		* Here's the order of data transformation:
		*
		* IStereoRenderingDevice
		* -> {CameraOrigin,CameraRotation}
		*    IDisplayClusterViewport::CalculateViewData()
		*      []->{EyeLocation, EyeRotation}
		* 		   IDisplayClusterProjectionPolicy::CalculateView()
		*      []-> {ViewLocation, ViewRotation}
		* <- {ViewLocation, ViewRotation}
		* IStereoRenderingDevice
		*/

		// Camera location in the world space
		// This value receives nDisplay from IStereoRenderingDevice
		FVector CameraOrigin = FVector::ZeroVector;

		// Camera rotation in the world space
		// This value receives nDisplay from IStereoRenderingDevice
		FRotator CameraRotation = FRotator::ZeroRotator;

		// The eye location  in the world space
		// This value is used as an argument to IDisplayClusterProjectionPolicy::CalculateView()
		FVector EyeOrigin = FVector::ZeroVector;

		// The eye rotation in the world space
		// This value is used as an argument to IDisplayClusterProjectionPolicy::CalculateView()
		FRotator EyeRotation = FRotator::ZeroRotator;

		// Camera head (pivot) position in world space.
		// Equals RenderLocation minus the stereo eye offset; identical to RenderLocation for mono rendering.
		FVector RenderCameraLocation = FVector::ZeroVector;

		// Render eye position in world space; the actual view origin passed to the scene renderer.
		// Differs from RenderCameraLocation by the stereo eye offset in stereo contexts.
		FVector RenderLocation = FVector::ZeroVector;

		// View rotation in world space; shared by both the camera head and the render eye.
		FRotator RenderRotation = FRotator::ZeroRotator;
	};

	// View data for this context
	FCachedViewData ViewData;

	/** Cached projection data */
	struct FCachedProjectionData
	{
		bool bValid = false;

		// Is overscan used
		bool bUseOverscan = false;

		// Near-plane frustum extents for rendering in world units.
		// ( X=Left, Y=Right, Z=Top, W=Bottom )
		FVector4 ProjectionAngles;

		// Near-plane frustum extents for rendering with overscan in world units.
		// ( X=Left, Y=Right, Z=Top, W=Bottom )
		FVector4 OverscanProjectionAngles;

		// Clipping planes
		double ZNear = 0.f;
		double ZFar = 0.f;
	};

	// Cached projection values
	// This values updated from function FDisplayClusterViewport::CalculateProjectionMatrix()
	FCachedProjectionData ProjectionData;

	// UE scale: how many UE Units in one meter
	float WorldToMeters = 100.f;

	// Geometry scale: how many Geometry Units in one meter
	float GeometryToMeters = 100.f;

	// Origin component to world transform
	// This component is defined in the projection policy
	FTransform OriginToWorld = FTransform::Identity;

	// RootActor to world transform
	FTransform RootActorToWorld = FTransform::Identity;

	// View location used in world space
	FVector  ViewLocation = FVector::ZeroVector;

	// View rotation used in world space
	FRotator ViewRotation = FRotator::ZeroRotator;

	// Projection Matrix
	FMatrix ProjectionMatrix = FMatrix::Identity;

	// Overscan Projection Matrix (internal use)
	FMatrix OverscanProjectionMatrix = FMatrix::Identity;
	
	/** Additional data for the Depth of Field (DoF). */
	struct FDepthOfFieldSettings
	{
		// Focal length of the Depth of Field effect camera in mm.
		float SensorFocalLength = 0.f;

		// This is the squeeze factor for the DOF, which emulates the properties of anamorphic lenses.
		float SqueezeFactor = 1.f;

	} DepthOfField;

	// GPU index for this context render target
	int32 GPUIndex = INDEX_NONE;

	// Inner rectangle inside RenderTargetRect
	FIntRect OverscanInnerRenderTargetRect;

	// Location and size on a render target texture
	FIntRect RenderTargetRect;

	// Context size
	FIntPoint ContextSize;

	// Location and size on a frame target texture
	FIntRect FrameTargetRect;

	// Tile location and size in the source viewport
	FIntRect TileDestRect;

	// Buffer ratio
	float CustomBufferRatio = 1;

	// Mips number for additional MipsShader resources
	int32 NumMips = 1;

	/**
	* Viewport context data for rendering thread
	*/
	struct FRenderThreadData
	{
		FRenderThreadData()
			: EngineShowFlags(ESFIM_All0)
		{ }
		
		// GPUIndex used to render this context.
		int32 GPUIndex = INDEX_NONE;

		// Display gamma used to render this context
		float EngineDisplayGamma = 2.2f;

		// Engine flags used to render this context
		FEngineShowFlags EngineShowFlags;

		// EyeAdaptation data
		FDisplayClusterViewport_EyeAdaptationData EyeAdaptationData;

		// The value from FSceneViewFamily::FrameNumber
		uint32 FrameNumber = 0;
	};

	// This data updated only on rendering thread
	FRenderThreadData RenderThreadData;
};
