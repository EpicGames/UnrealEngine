// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DisplayClusterViewport_Enums.h"
#include "Math/MarginSet.h"

/**
* Custom frustum settings of viewport
*/
struct FDisplayClusterViewport_CustomFrustumSettings
	: public FMarginSet
{
	// Enable custom frustum
	uint8 bEnabled : 1 = 0;

	// Enable adaptive resolution
	uint8 bAdaptResolution : 1 = 0;

	// Units type of values
	EDisplayClusterViewport_FrustumUnit Unit = EDisplayClusterViewport_FrustumUnit::Pixels;
};
