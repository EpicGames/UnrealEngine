// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * Types of DC viewport resources
 */
enum class EDisplayClusterViewportResourceType : uint8
{
	// Undefined resource type
	Unknown = 0,

	// This RTT is used to render the scene for the viewport eye context to the specified area on this texture.
	// The mGPU rendering with cross-GPU transfer can be used for this texture.
	// This resource has an area that includes the overscan extension.
	InternalRenderTargetEntireRectResource, /** Just for Internal use */

	/**
	* Same RTT texture resource, but with the final value of the inner area on the texture (after overscan, etc.).
	*/
	InternalRenderTargetResource, /** Just for Internal use */


	/**
	 * Internal textures of the DC rendering pipeline.
	 */

	// Unique Viewport contexts shader resource. No regions used at this point
	// The image source for this resource can be: 'InternalRenderTargetResource', 'OverrideTexture', 'OverrideViewport', etc.
	// This is the entry point to the DC rendering pipeline
	InputShaderResource,

	// Special resource with mips texture, generated from 'InputShaderResource' after postprocess
	MipsShaderResource,

	// Additional targetable resource, used by logic (external warpblend, blur, etc)
	// This resource is used as an additional RTT for 'InputShaderResource'.
	// It contains different contents depending on the point in time: 'PostprocessRTT','ViewportOutputRemap','AfterWarpBlend' etc.
	AdditionalTargetableResource,


	/**
	 * Textures for warp and blend
	 */
	 
	 // The viewport texture before warpblend (InputShaderResource)
	// This resource is used as the source image for the projection policy.
	BeforeWarpBlendTargetableResource,

	// The viewport texture after warpblend (AdditionalTargetableResource or InputShaderResource).
	// This resource is used as the output image in the projection policy.
	AfterWarpBlendTargetableResource,


	/**
	 * Output textures
	 */

	// This is the entry point into the output texture rendering.('OutputFrameTargetableResource' or 'OutputPreviewTargetableResource')
	// The rendering time of this image in the DC rendering pipeline is right after warp&blend.
	OutputTargetableResource,


	/**
	 * Output textures for preview rendering.
	 */

	// If DCRA uses a preview, this texture will be used instead of 'Frame' textures
	OutputPreviewTargetableResource,


	/**
	 * Output 'Frame' textures for cluster rendering
	 * The 'Frame' resources are used to compose all the viewports into a single texture.
	 * There is a separate 'frame' texture for each eye context.
	 * And at the end of the frame these resources are copied to the backbuffer.
	 * 
	 * Projection policy render output to this resources into viewport region
	 * (Context frame region saved FDisplayClusterViewport_Context::FrameTargetRect)
	 */

	// This texture contains the results of the DC rendering, and is copied directly to the backbuffer texture at the end of the frame.
	// Each eye is in a separate texture.
	OutputFrameTargetableResource,

	// This resource is used as an additional RTT for 'OutputFrameTargetableResource'.
	// It contains different contents depending on the point in time: 'FramePostprocessRTT','OutputRemap', etc.
	AdditionalFrameTargetableResource,
};

/**
 * Viewport capture mode
 * This mode affects many viewport rendering settings.
 */
enum class EDisplayClusterViewportCaptureMode : uint8
{
	// Use current scene format, no alpha
	Default = 0,

	// use small BGRA 8bit texture with alpha for masking
	Chromakey,

	// use hi-res float texture with alpha for compisiting
	Lightcard,

	// Special hi-res mode for movie pipeline
	MoviePipeline,
};

/**
 * Viewport can be overridden by another one.
 * This mode determines how many resources will be overridden.
 */
enum class EDisplayClusterViewportOverrideMode : uint8
{
	// Do not override this viewport from the other one (Render viewport; create all resources)
	None = 0,

	// Override internalRTT from the other viewport (Don't render this viewport; Don't create RTT resource)
	// Useful for custom PP on the same InRTT. (OCIO per-viewport\node)
	InternalRTT,

	// Overrides only internal viewport resources
	// The output RTT is not overridden and must use rendering.
	InternalViewportResources,

	// Override all - clone viewport (Dont render; Don't create resources;)
	All
};

/**
* Type of unit for frustum
*/
enum class EDisplayClusterViewport_FrustumUnit: uint8
{
	// 1 unit = 1 pixel
	Pixels = 0,

	// 1 unit = 1 per cent
	Percent
};

/**
* Type of DCRA used by configuration
*/
enum class EDisplayClusterRootActorType : uint8
{
	// This DCRA will be used to render previews. The meshes and preview materials are created at runtime.
	Preview = 1 << 0,

	// A reference to DCRA in the scene, used as a source for math calculations and references.
	// Locations in the scene and math data are taken from this DCRA.
	Scene = 1 << 1,

	// Reference to DCRA, used as a source of configuration data from DCRA and its components.
	Configuration = 1 << 2,

	// This value can only be used in very specific cases:
	// For function GetRootActor() : Return any of the DCRAs that are not nullptr, in ascending order of type: Preview, Scene, Configuration.
	// For function SetRootActor() : Sets all references to DRCA to the specified value.
	Any = Preview | Scene | Configuration,
};
ENUM_CLASS_FLAGS(EDisplayClusterRootActorType);

/**
 * The type of media usage for the viewport.
 */
enum class EDisplayClusterViewportMediaState : uint8
{
	// This viewport does not use media.
	None = 0,

	// This viewport will be captured by a media device.
	Capture = 1 << 0,

	// Custom OCIO transformation is expected on the receiving side
	CaptureLateOCIO = 1 << 1,

	// This viewport is overridden by a media device.
	Input = 1 << 2,

	// Custom OCIO transformation is expected on receiving
	InputLateOCIO = 1 << 3,
};
ENUM_CLASS_FLAGS(EDisplayClusterViewportMediaState);

/**
* A set of flags defining the post-process parameters to be received from the camera.
*/
enum class EDisplayClusterViewportCameraPostProcessFlags : uint8
{
	// Ignore all post-processing from the camera.
	None = 0,

	// Use the PP settings from the specified camera.
	// (Supported by Camera, CineCamera and ICVFXCamera components).
	EnablePostProcess = 1 << 0,

	// Enable the CineCamera DoF PP settings from the specified camera.
	// (Supported by CineCamera and ICVFXCamera components).
	EnableDepthOfField = 1 << 1,

	// Use the custom NearClippingPlane value from the specified cine camera.
	// (Supported by CineCamera and ICVFXCamera components).
	EnableNearClippingPlane = 1 << 2,

	// Use the DC ColorGrading from the specified ICVFX camera.
	// (Supported by ICVFXCamera component only).
	EnableICVFXColorGrading = 1 << 3,

	// Use the DC Motion Blur settings from the specified ICVFX camera.
	// (Supported by ICVFXCamera component only).
	EnableICVFXMotionBlur = 1 << 4,

	// Use the DC Depth-Of-Field settings from the specified ICVFX camera.
	// (Supported by ICVFXCamera component only).
	EnableICVFXDepthOfFieldCompensation = 1 << 5,

	// Apply all possible post-processing from the camera.
	All = 0xFF
};
ENUM_CLASS_FLAGS(EDisplayClusterViewportCameraPostProcessFlags);

/**
* Rules for customizing nDisplay views and view families for the renderer.
*/
enum class EDisplayClusterViewportRenderingFlags : uint8
{
	// No flags
	None = 0,

	// Stereo rendering: Change screen percentage method to raw output when doing dynamic resolution with VR if not using TAA upsample.
	StereoRendering = 1 << 0,
};
ENUM_CLASS_FLAGS(EDisplayClusterViewportRenderingFlags);

/**
 * Viewport usage type flags.
 */
enum class EDisplayClusterViewportFlags : uint8
{
	/** The standard viewport does not have any special flags. */
	None = 0,

	/** Tile viewport. */
	TileViewport = 1 << 0,

	/** ICVFX viewports. */
	ICVFX_InnerFrustum = 1 << 1,
	ICVFX_ChromaKey    = 1 << 2,
	ICVFX_LightCard    = 1 << 3,
	ICVFX_UVLightCard  = 1 << 4,

	/** ICVFX render modifiers. */
	ICVFX_Modifier_RenderOverInnerFrustum  = 1 << 5,
	ICVFX_Modifier_RenderUnderInnerFrustum = 1 << 6,

	/** Combined mask for any ICVFX viewport role (excludes modifiers). */
	ICVFX_AnyRole =
	  ICVFX_InnerFrustum
	| ICVFX_ChromaKey
	| ICVFX_LightCard
	| ICVFX_UVLightCard,
};
ENUM_CLASS_FLAGS(EDisplayClusterViewportFlags);
