// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * Runtime configuration from DCRA.
 */
enum class EDisplayClusterViewportICVFXFlags : uint16
{
	None = 0,

	// Allow to use ICVFX for this viewport (Must be supported by projection policy)
	Enable = 1 << 0,

	// Disable incamera render to this viewport
	DisableCamera = 1 << 1,

	// Disable chromakey render to this viewport
	DisableChromakey = 1 << 2,

	// Disable chromakey markers render to this viewport
	DisableChromakeyMarkers = 1 << 3,

	// Disable lightcard render to this viewport
	DisableLightcard = 1 << 4,

	// lightcard render always under the InCamera
	LightcardAlwaysUnder = 1 << 5,

	// lightcard render always over the InCamera
	LightcardAlwaysOver = 1 << 6,

	// Lightcard rendering mode is determined from the stage settings.
	LightcardUseStageSettings = 1 << 7,

	// Mask that get only flags that define the light card render mode.
	LightcardRenderModeMask = LightcardAlwaysUnder | LightcardAlwaysOver | LightcardUseStageSettings,

	/** The order in which the ICVFX cameras are composited over is reversed. Useful for time-multiplexed displays. */
	ReverseCameraPriority = 1 << 8,

	/** Indicates that the In-Camera viewport is not visible because no target viewports are assigned. */
	CameraHasNoTargetViewports = 1 << 9,
};
ENUM_CLASS_FLAGS(EDisplayClusterViewportICVFXFlags);

/**
 * This flag raised only from icvfx manager.
*/
enum class EDisplayClusterViewportRuntimeICVFXFlags: uint16
{
	None = 0,

	// Enable use icvfx only from projection policy for this viewport.
	Target = 1 << 0,

	// viewport ICVFX usage
	InCamera    = 1 << 1,
	Chromakey   = 1 << 2,
	Lightcard   = 1 << 3,
	UVLightcard = 1 << 4,

	// Additional flags marking the position of the viewport relative to the in-frustum.
	OverInFrustum = 1 << 5,
	UnderInFrustum = 1 << 6,

	// This viewport used as internal icvfx compositing resource (created and deleted inside icvfx logic)
	InternalResource = 1 << 14,

	// Mark unused icvfx dynamic viewports
	Unused = 1 << 15,
};
ENUM_CLASS_FLAGS(EDisplayClusterViewportRuntimeICVFXFlags);
