// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * Eye Adaptation data for nDisplay viewport.
 */
struct FDisplayClusterViewport_EyeAdaptationData
{
	/** Return true if this data can be used. */
	inline bool CanBeUsed() const
	{
		return FMath::IsFinite(LastAverageSceneLuminance)
			&& LastAverageSceneLuminance > 0.0f;
	}

	// The last valid average scene luminance for eye adaptation (exposure compensation curve).
	// Contains the value from FSceneView::GetLastAverageSceneLuminance().
	// The value 0.0f means that there is no luminance data available.
	float LastAverageSceneLuminance = 0.0f;
};
