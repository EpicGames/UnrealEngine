// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DisplayClusterViewport_Enums.h"

#include "Math/MarginSet.h"

/**
* Overscan settings of viewport
*/
struct FDisplayClusterViewport_OverscanSettings
	: public FMarginSet
{
	/**
	 * Configures symmetric viewport overscan from an overscan fraction.
	 *
	 * @param InOverscanFraction  Per-side overscan fraction [0..1].
	 * @param bInOversize         If true, the render target is enlarged to fit the overscan;
	 *                            otherwise overscan renders within the existing target.
	 */
	inline void SetSymmetricOverscan(const float InOverscanFraction, bool bInOversize)
	{
		if (FMath::IsNearlyZero(InOverscanFraction) || !FMath::IsFinite(InOverscanFraction) || InOverscanFraction < 0)
		{
			bEnabled = false;
		}
		else
		{
			// Clamp to [0, 1] and apply uniformly to all four margins.
			const float OverscanFraction = FMath::Clamp(InOverscanFraction, 0.0f, 1.0f);

			AssignMargins(FMarginSet(OverscanFraction));

			bEnabled = true;
			bOversize = bInOversize;
			Unit = EDisplayClusterViewport_FrustumUnit::Percent;
		}
	}

	// Enable overscan
	uint8 bEnabled : 1 = 0;

	// Set to True to render at the overscan resolution, set to false to render at the resolution in the configuration and scale for overscan
	uint8 bOversize : 1 = 0;

	// Blends the overlapping (overscan) regions between adjacent tiles.
	// The size (in pixels) must be less than or equal to the overscan size,
	// ensuring a smooth transition with no visible seams.
	uint8 bApplyTileEdgeBlend : 1 = 0;

	// Percentage of the overscan (0.0–1.0) to use for edge blending.
	float TileEdgeBlendPercentage = 0.0f;

	// Units type of overscan values
	EDisplayClusterViewport_FrustumUnit Unit = EDisplayClusterViewport_FrustumUnit::Pixels;
};
