// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "ShaderParameters/DisplayClusterShaderParameters_Override.h"
#include "ShaderParameters/DisplayClusterShaderParameters_PostprocessBlur.h"
#include "ShaderParameters/DisplayClusterShaderParameters_GenerateMips.h"

/**
* Additional DC viewport rendering settings
*/
class FDisplayClusterViewport_PostRenderSettings
{
public:
	FDisplayClusterViewport_PostRenderSettings()
	{}

	~FDisplayClusterViewport_PostRenderSettings()
	{}

	inline void SetParameters(const FDisplayClusterViewport_PostRenderSettings& InPostRenderSettings)
	{
		Replace.SetParameters(InPostRenderSettings.Replace);
		PostprocessBlur = InPostRenderSettings.PostprocessBlur;
		GenerateMips = InPostRenderSettings.GenerateMips;
	}

	inline void BeginUpdateSettings()
	{
		Replace.Reset();
		PostprocessBlur.Reset();
		GenerateMips.Reset();
	}

public:
	FDisplayClusterShaderParameters_Override        Replace;
	FDisplayClusterShaderParameters_PostprocessBlur PostprocessBlur;
	FDisplayClusterShaderParameters_GenerateMips    GenerateMips;
};
