// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_Enums.h"
#include "Render/Viewport/RenderFrame/DisplayClusterRenderFrameEnums.h"

/**
* Viewport preview-in-scene rendering settings.
*/
struct FDisplayClusterViewport_PreviewSettings
{
	// The IDisplayClusterViewportManagerPreview::UpdateEntireClusterPreviewRender() function will use this rendering mode
	// For special rendering cases, set a different value, such as EDisplayClusterRenderFrameMode::PreviewProxyHitInScene
	EDisplayClusterRenderFrameMode EntireClusterPreviewRenderMode = EDisplayClusterRenderFrameMode::PreviewInScene;

	// Render the scene and display it as a preview on the nDisplay root actor in the editor.  This will impact editor performance.
	uint8 bPreviewEnable : 1 = 0;

	// Render this DCRA in game for Standalone/Package builds.
	uint8 bPreviewInGameEnable : 1 = 0;
	uint8 bPreviewInGameRenderFrustum : 1 = 0;

	/** Enables the use of 'HoldoutComposite' plugin for the preview rendering. */
	uint8 bPreviewEnableHoldoutComposite : 1 = 1;

	/** Enable TemporalAA/TSR in preview.
	* Increases memory consumption. This will impact editor performance.
	*/
	uint8 bPreviewEnableTSR : 1 = 0;

	// Preview uses techvis mode for rendering
	uint8 bEnablePreviewTechvis : 1 = 0;

	// Enable/Disable preview rendering. When disabled preview image freeze
	uint8 bFreezePreviewRender : 1 = 0;

	// Hack preview gamma.
	// In a scene, PostProcess always renders on top of the preview textures.
	// But in it, PostProcess is also rendered with the flag turned off.
	uint8 bPreviewEnablePostProcess : 1 = 0;

	// Enable OCIO on preview
	uint8 bPreviewEnableOCIO : 1 = 0;

	// Show overlay material on the preview mesh when preview rendering is enabled (UMeshComponent::OverlayMaterial)
	uint8 bPreviewEnableOverlayMaterial : 1 = 1;

	// Allows you to process preview meshes inside DCRA (get or create a mesh from a projection policy, update materials on the preview mesh, etc.).
	uint8 bEnablePreviewMesh : 1 = 0;

	// Allows you to process preview editable meshes inside DCRA (get or create a mesh from a projection policy, update materials on the preview editable mesh, etc.).
	uint8 bEnablePreviewEditableMesh : 1 = 0;

	// Render ICVFX Frustums
	uint8 bPreviewICVFXFrustums : 1 = 0;

public:
	float PreviewICVFXFrustumsFarDistance = 1000.0f;

	// Preview RTT size multiplier
	float PreviewRenderTargetRatioMult = 1.f;

	// The maximum dimension of any texture for preview
	// Limit preview textures max size
	int32 PreviewMaxTextureDimension = 2048;

	// Tick Per Frame
	int32 TickPerFrame = 1;

	// Max amount of Viewports Per Frame
	int32 ViewportsPerFrame = 1;

	// The DisplayDevice component will be obtained from the RootActor with the specified type
	EDisplayClusterRootActorType DisplayDeviceRootActorType = EDisplayClusterRootActorType::Configuration;
};
