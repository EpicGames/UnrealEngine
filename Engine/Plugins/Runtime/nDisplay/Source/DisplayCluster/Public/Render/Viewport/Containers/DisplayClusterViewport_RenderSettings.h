// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DisplayClusterViewport_OverscanSettings.h"
#include "DisplayClusterViewport_CustomFrustumSettings.h"
#include "DisplayClusterViewport_TileSettings.h"

#include "Render/Viewport/Containers/DisplayClusterViewport_Enums.h"
#include "Render/Upscaler/DisplayClusterUpscalerSettings.h"

/**
 * nDisplay viewport render settings.
 * These are runtime settings, updated every frame from the cluster configuration.
 */
class FDisplayClusterViewport_RenderSettings
{
public:
	// Assigned camera. If empty, the currently active camera must be used
	FString CameraId;

	// Assigned display device. If empty, the currently active display device must be used
	FString DisplayDeviceId;

	// Location and size on a backbuffer.
	FIntRect Rect;

public:
	// Enable this viewport and related resources rendering
	bool bEnable = true;

	// This viewport visible on final frame texture (backbuffer)
	bool bVisible = true;

	// Skip rendering for this viewport
	bool bSkipRendering = false;

	// Freeze viewport resources, skip rendering internal viewport resources. But still use it for final compositing
	bool bFreezeRendering = false;

	// Render alpha channel from input texture to warp output
	bool bWarpBlendRenderAlphaChannel = false;

	// Disable CustomFrustum feature from viewport settings
	bool bDisableCustomFrustumFeature = false;

	// Viewport custom frustum settings
	FDisplayClusterViewport_CustomFrustumSettings CustomFrustumSettings;

	// Disable viewport overscan feature from settings
	bool bDisableFrustumOverscanFeature = false;

	// Viewport frustum overscan settings
	FDisplayClusterViewport_OverscanSettings OverscanSettings;

	// Viewport tile rendering settings
	FDisplayClusterViewport_TileSettings TileSettings;

	// Useful to render some viewports in mono, then copied to stereo backbuffers identical image
	bool bForceMono = false;

	// Enable cross-GPU transfer for this viewport.
	// It may be disabled in some configurations. For example, when using offscreen rendering with TextureShare,
	// cross-gpu transfer can be disabled for this viewport to improve performance, because when transfer is called,
	// it freezes the GPUs until synchronization is reached.
	// (TextureShare uses its own implementation of the crossGPU transfer for the shared textures.)
	bool bEnableCrossGPUTransfer = true;

	// Performance, Multi-GPU: Asign GPU for viewport rendering. The Value '-1' used to default gpu mapping
	int32 GPUIndex = -1;

	// Performance, Multi-GPU: Customize GPU for stereo mode second view (EYE_RIGHT)
	int32 StereoGPUIndex = -1;

	// Allow ScreenPercentage 
	float BufferRatio = 1;

	// Performance: Render target base resolution multiplier
	float RenderTargetRatio = 1;

	// Performance: Render target adaptive resolution multiplier
	float RenderTargetAdaptRatio = 1;

	// Upscaler settings
	FDisplayClusterUpscalerSettings UpscalerSettings;

	// Viewport can overlap each other on backbuffer. This value uses to sorting order
	int32 OverlapOrder = 0;

	// Special capture modes (chromakey, lightcard) change RTT format and render flags
	EDisplayClusterViewportCaptureMode CaptureMode = EDisplayClusterViewportCaptureMode::Default;

public:
	/** Resets the viewport settings. This function is called every frame at the beginning of the frame. */
	inline void BeginUpdateSettings()
	{
		bVisible = true;
		bEnable = true;
		bSkipRendering = false;
		bFreezeRendering = false;
		bWarpBlendRenderAlphaChannel = false;

		OverscanSettings = FDisplayClusterViewport_OverscanSettings();
		CustomFrustumSettings = FDisplayClusterViewport_CustomFrustumSettings();

		CaptureMode = EDisplayClusterViewportCaptureMode::Default;

		ViewportOverrideMode = EDisplayClusterViewportOverrideMode::None;
		ViewportOverrideId.Empty();
	}

	/** Finishes setting the viewport in the game thread. Called once per frame at the end. */
	inline void FinishUpdateSettings()
	{ }

	/** Returns true if the viewport is assigned to a parent viewport. */
	inline bool IsViewportHasParent() const
	{
		return !ParentViewportId.IsEmpty();
	}

	/** Get the name of the parent viewport. */
	inline const FString& GetParentViewportId() const
	{
		return ParentViewportId;
	}

	/** Assign parent viewport to this.
	* The main idea is copiing some render settings and math from the parent viewport.
	* This is used for a 'link' projection policy to render LC and CK from the same frustum assigned as the parent (outer for LC, incamera for CK).
	* Also, child viewports are only updated when the parent viewport is updated (sorted in ViewportManager/Proxy).
	* 
	* @param InParentViewportId - parent viewport name
	* @param InParentSettings   - parent viewport rendering settings.
	*/
	inline void AssignParentViewport(const FString& InParentViewportId, const FDisplayClusterViewport_RenderSettings& InParentSettings)
	{
		ParentViewportId = InParentViewportId;

		// Inherit values from parent viewport:
		CameraId = InParentSettings.CameraId;
		Rect = InParentSettings.Rect;

		bForceMono = InParentSettings.bForceMono;

		GPUIndex = (GPUIndex < 0) ? InParentSettings.GPUIndex : GPUIndex;
		StereoGPUIndex = (StereoGPUIndex < 0) ? InParentSettings.StereoGPUIndex : StereoGPUIndex;
	}

	/** The viewport can be overridden from another viewport. This function returns true if it is. */
	inline bool IsViewportOverridden() const
	{
		return ViewportOverrideMode != EDisplayClusterViewportOverrideMode::None && !ViewportOverrideId.IsEmpty();
	}

	/** Getting the override mode that is currently in use. */
	inline EDisplayClusterViewportOverrideMode GetViewportOverrideMode() const
	{
		return IsViewportOverridden() ? ViewportOverrideMode : EDisplayClusterViewportOverrideMode::None;
	}

	/** Get the name of the viewport used as the image source. */
	inline const FString& GetViewportOverrideId() const
	{
		return ViewportOverrideId;
	}

	/** Set an override for viewport images from another viewport.
	* 
	* @param InViewportOverrideId   - The name of the viewport used as the image source.
	* @param InViewportOverrideMode - Override mode, which defines its rules.
	*/
	inline void SetViewportOverride(const FString& InViewportOverrideId, const EDisplayClusterViewportOverrideMode InViewportOverrideMode = EDisplayClusterViewportOverrideMode::All)
	{
		ViewportOverrideMode = InViewportOverrideMode;
		ViewportOverrideId = InViewportOverrideId;
	}

	/** Returns true if the media state of this viewport is set to any of the requested states.*/
	inline bool HasAnyMediaStates(const EDisplayClusterViewportMediaState InMediaStates) const
	{
		return EnumHasAnyFlags(MediaState, InMediaStates);
	}

	/** Returns current media states. */
	inline EDisplayClusterViewportMediaState GetMediaStates() const
	{
		return MediaState;
	}

	/** Set media state for this viewport. */
	inline void AssignMediaStates(const EDisplayClusterViewportMediaState InMediaStates)
	{
		MediaState = InMediaStates;
	}

protected:
	// Parent viewport name
	FString ParentViewportId;

	// Override resources from another viewport. The name of the viewport used as the image source.
	FString ViewportOverrideId;

	// Override mode, which defines its rules.
	EDisplayClusterViewportOverrideMode ViewportOverrideMode = EDisplayClusterViewportOverrideMode::None;

	// Viewport can be used by external media, and this affects its rules.
	// This variable is always updated at the beginning of each viewport configuration.
	EDisplayClusterViewportMediaState MediaState = EDisplayClusterViewportMediaState::None;
};
