// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Render/Viewport/Containers/DisplayClusterViewport_EnumsICVFX.h"
#include "ShaderParameters/DisplayClusterShaderParameters_ICVFX.h"

/**
 * ICVFX settings for viewport
 */
class FDisplayClusterViewport_RenderSettingsICVFX
{
public:
	// ICVFX logic flags
	EDisplayClusterViewportICVFXFlags         Flags = EDisplayClusterViewportICVFXFlags::None;

	// Internal logic ICVFX flags
	EDisplayClusterViewportRuntimeICVFXFlags  RuntimeFlags = EDisplayClusterViewportRuntimeICVFXFlags::None;

	// Compact ICVFX viewport settings
	FDisplayClusterShaderParameters_ICVFX ICVFX;

public:
	inline void BeginUpdateSettings()
	{
		Flags = EDisplayClusterViewportICVFXFlags::None;

		ICVFX.Reset();

		// Reset target flag
		EnumRemoveFlags(RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::Target);
	}

	// Implement copy ref and arrays
	inline void SetParameters(const FDisplayClusterViewport_RenderSettingsICVFX& InParameters)
	{
		Flags = InParameters.Flags;
		RuntimeFlags = InParameters.RuntimeFlags;

		ICVFX.SetParameters(InParameters.ICVFX);
	}
};
