// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/EnumClassFlags.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_OverscanSettings.h"

/**
 * Define the viewport type for tile rendering.
 */
enum class EDisplayClusterViewportTileType: uint8
{
	// This viewport dont use tile rendering
	None = 0,

	// This viewport will be split into several tiled viewports.
	// These tiled viewports will be used for rendering, after which the resulting images will be composited back into this viewport.
	Source,

	// One of many tiled viewports that is used to render an image fragment for the Source viewport.
	Tile,

	// This is an internal type that is used during the process of reallocation in the viewport.
	UnusedTile
};

/**
 * Additional tile settings represented as a bitmask
 */
enum class EDisplayClusterViewportTileFlags : uint8
{
	None = 0,

	// [Media]  Allow this tile to render when unbound 
	// 
	// This flag is passed directly from the FDisplayClusterConfigurationMediaICVFX::ClusterNodesToRenderUnboundTiles.
	// 
	// Controls if current node was explicitly allowed to render unbound tiles.
	// `Unbound tiles` are tile viewports that are not connected to MediaIO.
	// 
	// Here are comments on this logic from FDisplayClusterViewport::IsRenderEnabledByMedia()
	//
	// In | Out | Render (offscreen) | Render (onscreen)
	// ------------------------------------------------------------------
	//  0 |  0  |        0           | EDisplayClusterViewportTileFlags::AllowUnboundRender
	//  0 |  1  |        1           | 1
	//  1 |  0  |        0           | 0
	//  1 |  1  |        0           | 0
	AllowUnboundRender = 1 << 0,

	// Optimize overscan values for edge tiles.
	// 
	// Overscan is not applied on the outer edges of the image — only where tiles meet.
	// Example layout (tiles with overscan overlap marked as "++"):
	//
	//  +------------+-----------+
	//  | Tile0_0  ++|++ Tile1_0 |
	//  |     ++     |     ++    |
	//  +------------+-----------+
	//  |     ++     |     ++    |
	//  | Tile0_1  ++|++ Tile1_1 |
	//  +------------+-----------+
	OptimizeTileOverscan = 1 << 1,
};
ENUM_CLASS_FLAGS(EDisplayClusterViewportTileFlags);

/**
 * nDisplay viewport tile settings.
 * These are runtime settings, updated every frame from the cluster configuration.
 */
struct FDisplayClusterViewport_TileSettings
{
public:
	FDisplayClusterViewport_TileSettings() = default;

	/** Setup as source. */
	FDisplayClusterViewport_TileSettings(const FIntPoint& InSize, const FDisplayClusterViewport_OverscanSettings& InOverscanSettings, const EDisplayClusterViewportTileFlags InTileFlags)
		: Type(EDisplayClusterViewportTileType::Source)
		, Size(InSize)
		, OverscanSettings(InOverscanSettings)
		, TileFlags(InTileFlags)
	{ }

	/** Setup as tile. */
	FDisplayClusterViewport_TileSettings(const FString& InSourceViewportId, const FIntPoint& InPos, const FIntPoint& InSize, const EDisplayClusterViewportTileFlags InTileFlags)
		: Type(EDisplayClusterViewportTileType::Tile)
		, Size(InSize)
		, Pos(InPos)
		, SourceViewportId(InSourceViewportId)
		, TileFlags(InTileFlags)
	{ }

	/** Returns the current viewport type for tile rendering. */
	EDisplayClusterViewportTileType GetType() const
	{
		return Type;
	}

	/** Return true if this viewport is internal. */
	bool IsInternalViewport() const
	{
		return Type == EDisplayClusterViewportTileType::Tile;
	}

	/** Get Size value. */
	const FIntPoint& GetSize() const
	{
		return Size;
	}

	/** Get Pos value. */
	const FIntPoint& GetPos() const
	{
		return Pos;
	}

	/** Get SourceViewportId. */
	const FString& GetSourceViewportId() const
	{
		return SourceViewportId;
	}

	/** Get overscan settings for tile rendering. */
	const FDisplayClusterViewport_OverscanSettings& GetOverscanSettings() const
	{
		return OverscanSettings;
	}

	/** Returns tile flags. */
	EDisplayClusterViewportTileFlags GetTileFlags() const
	{
		return TileFlags;
	}

	/** Checks if any of specified tile flags are set. */
	const bool HasAnyTileFlags(const EDisplayClusterViewportTileFlags RequestedTileFlags) const
	{
		return EnumHasAnyFlags(TileFlags, RequestedTileFlags);
	}

	/** Set tile state to be used. */
	inline void SetTileStateToBeUsed(bool bUsed)
	{
		if(bUsed && Type == EDisplayClusterViewportTileType::UnusedTile)
		{
			Type = EDisplayClusterViewportTileType::Tile;
		}

		if (!bUsed && Type == EDisplayClusterViewportTileType::Tile)
		{
			Type = EDisplayClusterViewportTileType::UnusedTile;
		}
	}

private:
	// Define the viewport type for tile rendering.
	EDisplayClusterViewportTileType Type = EDisplayClusterViewportTileType::None;

	// Tile size for the 'source' viewport
	FIntPoint Size = FIntPoint::ZeroValue;

	// Tile index for the 'tile' viewport
	FIntPoint Pos = FIntPoint::ZeroValue;

	// Tile source viewport name
	FString SourceViewportId;

	// Overscan settings for tile rendering.
	FDisplayClusterViewport_OverscanSettings OverscanSettings;

	// Extra tile flags
	EDisplayClusterViewportTileFlags TileFlags = EDisplayClusterViewportTileFlags::None;
};
