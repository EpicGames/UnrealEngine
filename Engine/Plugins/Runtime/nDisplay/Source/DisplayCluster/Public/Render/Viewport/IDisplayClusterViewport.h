// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Render/Viewport/IDisplayClusterViewport_CustomPostProcessSettings.h"
#include "Render/Viewport/IDisplayClusterViewportConfiguration.h"
#include "Render/Viewport/IDisplayClusterViewportPreview.h"

#include "Render/Viewport/Containers/DisplayClusterViewport_OverscanSettings.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_PostRenderSettings.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_RenderSettings.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_RenderSettingsICVFX.h"
#include "Render/Viewport/Containers/DisplayClusterViewportFrameData.h"

#include "Render/Viewport/RenderFrame/DisplayClusterRenderFrameEnums.h"

#include "Engine/EngineTypes.h"

class UCameraComponent;
class UWorld;
class FSceneViewStateReference;
struct FMinimalViewInfo;
struct FOpenColorIOColorConversionSettings;


/**
 * nDisplay: Viewport (interface for GameThread)
 */
class DISPLAYCLUSTER_API IDisplayClusterViewport
{
public:
	virtual ~IDisplayClusterViewport() = default;

public:
	/** Get TSharedPtr from self. */
	virtual TSharedPtr<IDisplayClusterViewport, ESPMode::ThreadSafe> ToSharedPtr() = 0;
	virtual TSharedPtr<const IDisplayClusterViewport, ESPMode::ThreadSafe> ToSharedPtr() const = 0;

	/** Internal functions. Get TSharedRef from Self. */
	virtual TSharedRef<class FDisplayClusterViewport, ESPMode::ThreadSafe> ToSharedRef() = 0;
	virtual TSharedRef<const class FDisplayClusterViewport, ESPMode::ThreadSafe> ToSharedRef() const = 0;

	/** Returns the usage flags of the viewport. */
	virtual EDisplayClusterViewportFlags GetViewportFlags() const = 0;

	/** Returns the base viewport identifier (not guaranteed to be unique across ICVFX viewports).
	* Note: For ICVFX viewports returns:
	*       - The name of the ICVFX camera for InCamera/Chromakey.
	*       - The name of the Outer viewport for Lightcards.
	*/
	virtual FString GetBaseId() const = 0;

	/** Returns the unique viewport identifier, guaranteed to be unique across the entire cluster. */
	virtual FString GetId() const = 0;

	/** Returns the identifier of the cluster node that owns this viewport. */
	virtual FString GetClusterNodeId() const = 0;

	/** Get viewport manager configuration interface. */
	virtual IDisplayClusterViewportConfiguration& GetConfiguration() = 0;

	/** [const] Get viewport manager configuration interface. */
	virtual const IDisplayClusterViewportConfiguration& GetConfiguration() const = 0;

	/** Get viewport manager configuration interface reference. */
	virtual const TSharedRef<IDisplayClusterViewportConfiguration, ESPMode::ThreadSafe> GetConfigurationRef() = 0;

	/** [const] Get viewport manager configuration interface reference. */
	virtual const TSharedRef<const IDisplayClusterViewportConfiguration, ESPMode::ThreadSafe>  GetConfigurationRef() const = 0;

	/** Get viewport preview API */
	virtual IDisplayClusterViewportPreview& GetViewportPreview() const = 0;

	/** Get viewport proxy object. */
	virtual TSharedRef<class IDisplayClusterViewportProxy, ESPMode::ThreadSafe> GetViewportProxy() const = 0;

	/** Get main viewport render settings. */
	virtual const FDisplayClusterViewport_RenderSettings&      GetRenderSettings() const = 0;

	/** Get ICVFX render settings. */
	virtual const FDisplayClusterViewport_RenderSettingsICVFX& GetRenderSettingsICVFX() const = 0;

	/** Get post-render settings. */
	virtual const FDisplayClusterViewport_PostRenderSettings&  GetPostRenderSettings() const = 0;

	/** Returns true if both viewports use the same OCIO. */
	virtual bool UseSameOCIO(const IDisplayClusterViewport* InViewport) const = 0;

	/** Calculate projection matrices for viewport context
	 * (also created overscan matrices, etc)
	 * 
	 * @param InContextNum   - viewport eye context index
	 * @param Left           - frustum left angle
	 * @param Right          - frustum right angle
	 * @param Top            - frustum top angle
	 * @param Bottom         - frustum bottom angle
	 * @param NCP            - Near clipping plane
	 * @param FCP            - Far  clipping plane
	 * @param bIsAnglesInput - true, if projection angles in degrees
	 */
	virtual void CalculateProjectionMatrix(const uint32 InContextNum, float Left, float Right, float Top, float Bottom, float ZNear, float ZFar, bool bIsAnglesInput) = 0;

	/** Calculate viewport projection
	 * 
	 * @param InContextNum      - viewport eye context index
	 * @param InOutViewLocation - (in, out) View location
	 * @param InOutViewRotation - (in,out) View rotator
	 * @param WorldToMeters     - UE world scale
	 * 
	 * @return - true, if the calculation of the viewport projection is successful.
	 */
	virtual bool CalculateView(const uint32 InContextNum, FVector& InOutViewLocation, FRotator& InOutViewRotation, const float WorldToMeters) = 0;


	/** Get projection matrix for viewport context
	 * 
	 * @param InContextNum - viewport eye context index
	 * @param OutPrjMatrix - (out) Projection  matrix
	 * 
	 * @return true, if projection matrix valid.
	 */
	virtual bool GetProjectionMatrix(const uint32 InContextNum, FMatrix& OutPrjMatrix) = 0;

	/** Calculate View data
	 * Updates data within the FDisplayClusterViewport_Context.
	 * This function calls IDisplayClusterProjectionPolicy::CalculateView() and IDisplayClusterProjectionPolicy::GetProjectionMatrix().
	 * 
	 * Note: Called once per frame from IDisplayClusterViewport::CalculateView().
	 *
	 * @param InContextNum      - (in) viewport eye context index
	 * @param InOutViewLocation - (in) View location
	 * @param InOutViewRotation - (in) View rotator
	 * @param WorldToMeters     - UE world scale
	 *
	 * @return true on success
	*/
	virtual bool CalculateViewData(const uint32 InContextNum, const FVector& InViewLocation, const FRotator& InViewRotation, const float WorldToMeters) = 0;

	/** Setup viewpoint for this viewport
	 *
	 * @param InOutViewInfo - [in\out] viewinfo
	 * 
	 * @return - true if there is an internal viewpoint for the given viewport.
	 */
	virtual bool SetupViewPoint(const uint32 InContextNum, struct FMinimalViewInfo& InOutViewInfo) = 0;

	/** Return view point camera component for this viewport.
	* @param InRootActorType - the root type
	*/
	virtual class UDisplayClusterCameraComponent* GetViewPointCameraComponent(const EDisplayClusterRootActorType InRootActorType) const = 0;

	/** Return a DisplayDevice component for this viewport.
	* @param InRootActorType - the root type
	*/
	virtual class UDisplayClusterDisplayDeviceBaseComponent* GetDisplayDeviceComponent(const EDisplayClusterRootActorType InRootActorType) const = 0;

	/** Retrieves the view position from the ViewPoint component used by this viewport.
	 * The CalculateView() function receives arguments that may not match this position because they can be overridden.
	 * Therefore, projection policies that expect values from the ViewPoint component must use this function to get the correct values.
	 */
	virtual bool GetViewPointCameraEye(const uint32 InContextNum, FVector& OutViewLocation, FRotator& OutViewRotation, FVector& OutViewOffset) = 0;

	/** Get the distance from the eye to the viewpoint location.
	 *
	 * @param InContextNum - eye context of this viewport
	 */
	virtual float GetStereoEyeOffsetDistance(const uint32 InContextNum) = 0;

	/** Get projection policy ref. */
	virtual const TSharedPtr<class IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe>& GetProjectionPolicy() const = 0;

	/** Get viewport contexts */
	virtual const TArray<FDisplayClusterViewport_Context>& GetContexts() const = 0;

	/** Get viewport contexts captured during the previous frame. */
	virtual const TArray<FDisplayClusterViewport_Context>& GetPrevFrameContexts() const = 0;

	/** Get the frame data snapshot captured for the current frame. */
	virtual const FDisplayClusterViewportFrameData& GetFrameData() const = 0;

	/** Get the frame data snapshot captured during the previous frame. */
	virtual const FDisplayClusterViewportFrameData& GetPrevFrameData() const = 0;

	/**
	 * Returns the ViewState for the specified viewport context, creating it if necessary.
	 *
	 * @param InContextNum Eye context index for this viewport.
	 *
	 * @return Shared pointer to the corresponding ViewState, or nullptr if ViewState usage is not allowed for this viewport.
	 */
	virtual TSharedPtr<FSceneViewStateReference, ESPMode::ThreadSafe> GetOrCreateViewState(uint32 InContextNum) = 0;

	/** Get clipping planes used by this viewport
	 * 
	 * @return Clipping planes as FVector2D(ZNear, ZFar)
	 */
	virtual FVector2D GetClippingPlanes() const = 0;

	/**
	 * Returns OCIO configuration if any set
	 * 
	 * @return true if OCIO is configured
	 */
	virtual bool GetOCIOConversionSettings(FOpenColorIOColorConversionSettings& OutOCIOConversionSettings) const = 0;

	/**
	* [const] Get custom postprocess settings for this viewport.
	*/
	virtual const IDisplayClusterViewport_CustomPostProcessSettings& GetViewport_CustomPostProcessSettings() const = 0;

	/**
	* Get custom postprocess settings for this viewport.
	*/
	virtual IDisplayClusterViewport_CustomPostProcessSettings& GetViewport_CustomPostProcessSettings() = 0;

	/**
	* Setup scene view for rendering specified Context.
	*/
	virtual void SetupSceneView(uint32 ContextNum, class UWorld* World, class FSceneViewFamily& InViewFamily, FSceneView& InView) const = 0;

	/**
	* Change render setting for viewport.
	* This function must be called in a valid moment in time.
	*/
	virtual void SetRenderSettings(const FDisplayClusterViewport_RenderSettings& InRenderSettings) = 0;

	/**
	* Change viewport contexts data.
	* This function must be called in a valid moment in time.
	*/
	virtual void SetContexts(TArray<FDisplayClusterViewport_Context>& InContexts) = 0;

	/** Make asymmetrical projection matrix for nDisplay. */
	static FMatrix MakeProjectionMatrix(float InLeft, float InRight, float InTop, float InBottom, float ZNear, float ZFar);

	/** Get View from CameraComponent
	*
	* @param InCameraComponent          - Ptr to the camera component used for rendering. if this ptr is nullptr, then the active player camera is used
	* @param InDeltaTime                - delta time in current frame
	* @param bUseCameraPostprocess      - If false, then InOutViewInfo.PostProcessBlendWeight is set to 0.
	* @param InOutViewInfo              - ViewPoint data
	* @param OutCustomNearClippingPlane - Custom NCP, or a value less than zero if not defined.
	*
	* @return false if the viewpoint data cannot be retrieved.
	*/
	static bool GetCameraComponentView(UCameraComponent* InCameraComponent, const float InDeltaTime, const bool bUseCameraPostprocess, FMinimalViewInfo& InOutViewInfo, float* OutCustomNearClippingPlane = nullptr);

	/** Get View from current player camera
	*
	* @param InWorld                    - (in)       In this world, an active game camera will be searched for.
	* @param bUseCameraPostprocess      - (in)       if false, then InOutViewInfo.PostProcessBlendWeight is set to 0.
	* @param InOutViewInfo              - (in,out)   ViewPoint data
	* @param OutCustomNearClippingPlane - (opt, out) Custom NCP, or a value less than zero if not defined.
	*
	* @return false if the viewpoint data cannot be retrieved.
	*/
	static bool GetPlayerCameraView(UWorld* InWorld, const bool bUseCameraPostprocess, FMinimalViewInfo& InOutViewInfo, float* OutCustomNearClippingPlane = nullptr);

	///////////////// UE_DEPRECATED 5.3 ///////////////////

	UE_DEPRECATED(5.3, "This function has beend deprecate. Please use 'GetViewportManager()'.")
	virtual class IDisplayClusterViewportManager& GetOwner() const
	{
		check(false);
	
		return *GetConfiguration().GetViewportManager();
	}

	/** Calculate viewport projection (This function has been deprecated.)
	 *
	 * @param InContextNum      - viewport eye context index
	 * @param InOutViewLocation - (in, out) View location
	 * @param InOutViewRotation - (in,out) View rotator
	 * @param ViewOffset         - eye offset from view location
	 * @param WorldToMeters     - UE world scale
	 * @param NCP - Near clipping plane
	 * @param FCP - Far clipping plane
	 *
	 * @return - true, if the calculation of the viewport projection is successful.
	 */
	UE_DEPRECATED(5.3, "This function has been deprecated. Please use 'CalculateView()'.")
	virtual bool CalculateView(const uint32 InContextNum, FVector& InOutViewLocation, FRotator& InOutViewRotation, const FVector& ViewOffset, const float WorldToMeters, const float NCP, const float FCP)
	{
		return false;
	}

	///////////////// UE_DEPRECATED 5.4 ///////////////////
	/** Return view point camera component for this viewport. */
	UE_DEPRECATED(5.4, "This function has been deprecated. Please use 'GetViewPointCameraComponent(EDisplayClusterRootActorType)'.")
	virtual class UDisplayClusterCameraComponent* GetViewPointCameraComponent() const
	{
		return nullptr;
	}

	/** Return the viewport manager that owns this viewport */
	UE_DEPRECATED(5.4, "This function has been deprecated. Please use 'GetConfiguration()'.")
	virtual class IDisplayClusterViewportManager* GetViewportManager() const
	{
		return nullptr;
	}

	/** Gets a reference to the DCRA with actual configuration inside. */
	UE_DEPRECATED(5.4, "This function has been deprecated. Please use 'GetConfiguration()'.")
	virtual class ADisplayClusterRootActor* GetRootActor() const
	{
		return nullptr;
	}

	/** Return current render mode. */
	UE_DEPRECATED(5.4, "This function has been deprecated. Please use 'GetConfiguration()'.")
	virtual EDisplayClusterRenderFrameMode GetRenderMode() const
	{
		return EDisplayClusterRenderFrameMode::Unknown;
	}

	/** Return current world. */
	UE_DEPRECATED(5.4, "This function has been deprecated. Please use 'GetConfiguration()'.")
	virtual class UWorld* GetCurrentWorld() const
	{
		return nullptr;
	}

	/** Returns true if the scene is open now (The current world is assigned and DCRA has already initialized for it). */
	UE_DEPRECATED(5.4, "This function has been deprecated. Please use 'GetConfiguration()'.")
	virtual bool IsSceneOpened() const
	{
		return false;
	}

	/** Returns true if the current world type is equal to one of the input types. */
	UE_DEPRECATED(5.4, "This function has been deprecated. Please use 'GetConfiguration()'.")
	virtual bool IsCurrentWorldHasAnyType(const EWorldType::Type InWorldType1, const EWorldType::Type InWorldType2 = EWorldType::None, const EWorldType::Type InWorldType3 = EWorldType::None) const
	{
		return false;
	}

	/** Setup viewpoint for this viewport
	 *
	 * @param InOutViewInfo - [in\out] viewinfo
	 *
	 * @return - true if there is an internal viewpoint for the given viewport.
	 */
	UE_DEPRECATED(5.5, "This function has been deprecated. Please use 'SetupViewPoint()'.")
	virtual bool SetupViewPoint(struct FMinimalViewInfo& InOutViewInfo)
	{
		return false;
	}

};
