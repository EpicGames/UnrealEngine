// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "Render/Viewport/RenderFrame/DisplayClusterRenderFrameEnums.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_PreviewSettings.h"

class UWorld;
class UDisplayClusterConfigurationData;
class ADisplayClusterRootActor;
class IDisplayClusterViewportManager;
struct FDisplayClusterConfigurationICVFX_StageSettings;
struct FDisplayClusterConfigurationRenderFrame;

/**
 * Viewport manager configuration.
 */
class DISPLAYCLUSTER_API IDisplayClusterViewportConfiguration
{
public:
	virtual ~IDisplayClusterViewportConfiguration() = default;

public:
	/**
	 *  Sets a reference to the DCRA's
	 * @param InRootActorType - Type of the DCRA
	 * @param InRootActor     - a ref to DCRA
	 */
	virtual void SetRootActor(const EDisplayClusterRootActorType InRootActorType, const ADisplayClusterRootActor* InRootActor) = 0;

	/**
	 * Applies or releases an exclusive lock on all DCRAs currently referenced by this configuration.
	 * DCRAs assigned later via SetRootActor() are also locked/unlocked automatically.
	 * Counterpart: ADisplayClusterRootActor::IsRootActorExclusivelyLocked().
	 *
	 * @param bLocked  When true, locks all referenced DCRAs; when false, releases the lock.
	 */
	virtual void SetExclusiveLockOnRootActors(const bool bLocked) = 0;

	/**
	 * Returns true if this configuration currently holds an exclusive lock on InRootActor.
	 * Counterpart: ADisplayClusterRootActor::RootActorExclusiveLockOwners.
	 *
	 * @param InRootActor  The DCRA to query.
	 */
	virtual bool IsRootActorLockedByThis(const ADisplayClusterRootActor* InRootActor) const = 0;

	/**
	* Assign new preview settings for rendering previews.
	*
	* @param InPreviewSettings - a new preview settings
	*/
	virtual void SetPreviewSettings(const FDisplayClusterViewport_PreviewSettings& InPreviewSettings) = 0;

	/** Gets the current preview settings. */
	virtual const FDisplayClusterViewport_PreviewSettings& GetPreviewSettings() const = 0;

	/**
	 * Enables external overscan, bypassing nDisplay's own overscan settings.
	 * When enabled, a symmetric overscan is applied from InOverscanOverride, or from
	 * FMinimalViewInfo.GetOverscan() when InOverscanOverride is nullptr.
	 *
	 * @param bInEnable          - When true, nDisplay's own overscan is ignored and the external value is used.
	 * @param InOverscanOverride - (opt) Per-side overscan fraction [0..1]. When nullptr, read from FMinimalViewInfo.GetOverscan().
	 */
	virtual void SetExternalOverscan(bool bInEnable, const float* InOverscanOverride = nullptr) = 0;

	/**
	 * Scales the render resolution of all viewports uniformly.
	 *
	 * @param InScale - Resolution scale factor [0..1]. 1.0 = full resolution, 0.5 = half resolution.
	 */
	virtual void SetRenderResolutionScale(const float InScale) = 0;

	/**
	* Update\Create\Delete local node viewports
	* Updating the configuration to render a ClusterNode in the specified mode
	*
	* @param InRenderMode          - Render mode
	* @param InWorld               - ptr to the world to be rendered
	* @param InClusterNodeId       - cluster node for rendering
	* @param InFrameNumberOverride - (opt) If set, use this value instead of GFrameNumber.
	*/
	virtual bool UpdateConfigurationForClusterNode(
		EDisplayClusterRenderFrameMode InRenderMode,
		const UWorld* InWorld,
		const FString& InClusterNodeId,
		const uint32* InFrameNumberOverride = nullptr) = 0;

	/**
	* Update\Create\Delete local node viewports
	* Updating the configuration to render a list of viewports in a given mode
	*
	* @param InRenderMode          - Render mode
	* @param InWorld               - ptr to the world to be rendered
	* @param InViewportNames       - Viewports names for next frame
	* @param InFrameNumberOverride - (opt) If set, use this value instead of GFrameNumber.
	*/
	virtual bool UpdateConfigurationForViewportsList(
		EDisplayClusterRenderFrameMode InRenderMode,
		const UWorld* InWorld,
		const TArray<FString>& InViewportNames,
		const uint32* InFrameNumberOverride = nullptr) = 0;

	/** Release the current configuration and free resources. */
	virtual void ReleaseConfiguration() = 0;

public:
	/** Return the configuration proxy object. */
	virtual const class IDisplayClusterViewportConfigurationProxy& GetProxy() const = 0;

	/** Return the viewport manager that used by this configuration. */
	virtual IDisplayClusterViewportManager* GetViewportManager() const = 0;

	/**
	 *  Gets a reference to the current world being rendered in DCRA
	 */
	virtual UWorld* GetCurrentWorld() const = 0;

	/** Return value from the DCRA function GetWorldDeltaSeconds(). */
	virtual float GetRootActorWorldDeltaSeconds(const EDisplayClusterRootActorType InRootActorType = EDisplayClusterRootActorType::Scene) const = 0;

	/**
	 * Gets a reference to the DCRA by type.
	 * If a DCRA with the specified type is not assigned, the default DCRA is used.
	 * 
	 * @param InRootActorType - Type of the DCRA
	 */
	virtual ADisplayClusterRootActor* GetRootActor(const EDisplayClusterRootActorType InRootActorType) const = 0;

	/**
	 * Gets a configuration data from the Configuration RootActor
	 */
	virtual const UDisplayClusterConfigurationData* GetConfigurationData() const = 0;

	/**
	 * Gets a configuration stage settings from the Configuration RootActor
	 */
	virtual const FDisplayClusterConfigurationICVFX_StageSettings* GetStageSettings() const = 0;

	/**
	 * Gets a configuration render settings from the Configuration RootActor
	 */
	virtual const FDisplayClusterConfigurationRenderFrame* GetConfigurationRenderFrameSettings() const = 0;

	/**
	* Returns true if the current world type is equal to one of the input types.
	*/
	virtual bool IsCurrentWorldHasAnyType(
		const EWorldType::Type InWorldType1,
		const EWorldType::Type InWorldType2 = EWorldType::None,
		const EWorldType::Type InWorldType3 = EWorldType::None
	) const = 0;

	/**
	* Returns true if the given DCRA has a world type equal to one of the input types.
	*/
	virtual bool IsRootActorWorldHasAnyType(
		const EDisplayClusterRootActorType InRootActorType,
		const EWorldType::Type InWorldType1,
		const EWorldType::Type InWorldType2 = EWorldType::None,
		const EWorldType::Type InWorldType3 = EWorldType::None
	) const = 0;

	/** 
	* Returns true if the scene is open now (The current world is assigned and DCRA has already initialized for it).
	*/
	virtual bool IsSceneOpened() const = 0;

	/** Returns true if ICVFX rendering can be used. */
	virtual bool IsICVFXEnabled() const = 0;

	/**
	* Returns true if media can be used.
	*
	* Note: DCRA media is available only in cluster mode
	* and should be ignored during preview rendering.
	*/
	virtual bool IsMediaAvailable() const = 0;

	/** Returns true if preview rendering mode is used. */
	virtual bool IsPreviewRendering() const = 0;

	/** Returns true if this DCRA is using cluster rendering. */
	virtual bool IsClusterRendering() const = 0;

	/** Returns true if MoviePipeline is used for rendering. */
	virtual bool IsMoviePipelineRendering() const = 0;

	/** Returns true, if Techvis is used. */
	virtual bool IsTechvisEnabled() const = 0;

	/** Returns true if the DCRA preview feature in Standalone/Package builds is used. */
	virtual bool IsPreviewInGameEnabled() const = 0;

	/**
	* Returns true if this DCRA is under an exclusive lock.
	* While exclusively locked:
	*  - No proxy actors may use this DCRA simultaneously.
	*  - Default scene preview rendering is disabled.
	*  - PIE rendering is disabled.
	*
	* Intended for scenarios where a system (e.g. Movie Render Queue) requires
	* sole ownership of the DCRA without interference.
	*/
	virtual bool IsRootActorExclusivelyLocked() const = 0;

	/**
	 * Checks whether the current cluster node is rendering offscreen (headless).
	 * 
	 * Note: The DCRA can render in both Preview and Runtime Cluster modes at the same time.
	 *       This is why this function is DCRA-specific rather than global.
	 *
	 * Usage:
	 *   - Editor/In-Game Preview: Determined by UDisplayClusterConfigurationClusterNode::bRenderHeadless
	 *   - Cluster Runtime:        Determined by the "RenderOffscreen" command-line argument
	 *
	 * @return true if rendering offscreen (headless), false otherwise.
	 */
	virtual bool IsClusterNodeRenderingOffscreen() const = 0;

	/** Returns the rendering mode for PIE. */
	virtual EDisplayClusterRenderFrameMode GetRenderModeForPIE() const = 0;

	/** Return current cluster node id. */
	virtual const FString& GetClusterNodeId() const = 0;

	/** Return current value for WorldToMeters. */
	virtual const float GetWorldToMeters() const = 0;

	/** Return number of views per viewport. */
	virtual int32 GetViewPerViewportAmount() const = 0;

};
