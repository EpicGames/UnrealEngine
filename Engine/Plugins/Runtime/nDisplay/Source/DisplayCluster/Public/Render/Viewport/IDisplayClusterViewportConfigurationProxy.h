// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Render/Viewport/RenderFrame/DisplayClusterRenderFrameEnums.h"

class IDisplayClusterViewportManagerProxy;

/**
 * Viewport manager proxy configuration.
 */
class DISPLAYCLUSTER_API IDisplayClusterViewportConfigurationProxy
{
public:
	virtual ~IDisplayClusterViewportConfigurationProxy() = default;

public:
	/** Return the viewport manager that used by this configuration. */
	virtual IDisplayClusterViewportManagerProxy* GetViewportManagerProxy_RenderThread() const = 0;

	/** Returns true if preview rendering mode is used. */
	virtual bool IsPreviewRendering_RenderThread() const = 0;

	/** Returns true if ICVFX rendering can be used. */
	virtual bool IsICVFXEnabled_RenderThread() const = 0;

	/** Return current render mode. */
	virtual EDisplayClusterRenderFrameMode GetRenderMode_RenderThread() const = 0;

	/** Return current cluster node id. */
	virtual const FString& GetClusterNodeId_RenderThread() const = 0;
};
