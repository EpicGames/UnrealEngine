// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Render/Viewport/IDisplayClusterViewport.h"
#include "Render/Viewport/IDisplayClusterViewportManagerPreview.h"
#include "Render/Viewport/RenderFrame/DisplayClusterRenderFrameEnums.h"

#include "SceneView.h"

class IDisplayClusterWarpPolicy;
class UWorld;
class FViewport;
class FSceneViewFamilyContext;
class FSceneView;
class ADisplayClusterRootActor;
class UDisplayClusterConfigurationViewport;
class IDisplayClusterViewportManagerProxy;
class FReferenceCollector;
class FDisplayClusterRenderFrame;
struct FDisplayClusterRenderFrameTarget;
struct FDisplayClusterRenderFrameTargetViewFamily;

/**
 * nDisplay ViewportManager (interface for GameThread)
 */
class DISPLAYCLUSTER_API IDisplayClusterViewportManager
{
public:
	/**
	* Must be called before the destructor.
	* Because all subsystems connected to this class via WeakPtr will not work when we are in destructor.
	*/
	virtual void Release() = 0;

	virtual ~IDisplayClusterViewportManager() = default;

public:
	/** Static viewport manager instance constructor.
	* This function is needed for external modules that may need their own instance of ViewportManager, 
	* which will be used to generate preview images with their own rendering settings.
	*/
	static TSharedRef<IDisplayClusterViewportManager, ESPMode::ThreadSafe> CreateViewportManager();

	/** Get TSharedPtr from self. */
	virtual TSharedPtr<IDisplayClusterViewportManager, ESPMode::ThreadSafe> ToSharedPtr() = 0;
	virtual TSharedPtr<const IDisplayClusterViewportManager, ESPMode::ThreadSafe> ToSharedPtr() const = 0;

	/** Internal functions. Get TSharedRef from Self. */
	virtual TSharedRef<class FDisplayClusterViewportManager, ESPMode::ThreadSafe> ToSharedRef() = 0;
	virtual TSharedRef<const class FDisplayClusterViewportManager, ESPMode::ThreadSafe> ToSharedRef() const = 0;

	/** Get viewport manager proxy interface. */
	virtual const IDisplayClusterViewportManagerProxy* GetProxy() const = 0;
	virtual       IDisplayClusterViewportManagerProxy* GetProxy() = 0;

	/** Get viewport manager configuration interface. */
	virtual IDisplayClusterViewportConfiguration& GetConfiguration() = 0;
	virtual const IDisplayClusterViewportConfiguration& GetConfiguration() const = 0;

	/** Get viewport manager configuration interface reference. */
	virtual const TSharedRef<IDisplayClusterViewportConfiguration, ESPMode::ThreadSafe> GetConfigurationRef() = 0;
	virtual const TSharedRef<const IDisplayClusterViewportConfiguration, ESPMode::ThreadSafe>  GetConfigurationRef() const = 0;


	/** Get viewport manager preview API */
	virtual IDisplayClusterViewportManagerPreview& GetViewportManagerPreview() = 0;
	virtual const IDisplayClusterViewportManagerPreview& GetViewportManagerPreview() const = 0;

	/**
	* Initialize new frame for all viewports on game thread, and update context, render resources with viewport new settings
	* And finally build render frame structure and send to render thread proxy viewport objects
	* [Game thread func]
	*
	* @param InViewport          - target viewport
	* @param OutRenderFrame      - output render frame container
	*
	* @return - true, if success
	*/
	virtual bool BeginNewFrame(FViewport* InViewport, FDisplayClusterRenderFrame& OutRenderFrame) = 0;

	/**
	* Initialize frame for render on game thread
	* [Game thread func]
	*
	*/
	virtual void InitializeNewFrame() = 0;

	/**
	* Finalize frame for render on game thread
	* [Game thread func]
	*
	*/
	virtual void FinalizeNewFrame() = 0;

	/**
	* Create constructor values for viewfamily, using rules
	* [Game thread func]
	*
	* @param InFrameTarget           - frame target
	* @param InScene                 - scene
	* @param InEngineShowFlags       - starting showflags
	* @param bInAdditionalViewFamily - flag indicating this is an additional view family
	* 
	* @return - The ConstructionValues object that was created, meant to initialize a view family context.
	*/
	virtual FSceneViewFamily::ConstructionValues CreateViewFamilyConstructionValues(
		const FDisplayClusterRenderFrameTarget& InFrameTarget,
		FSceneInterface* InScene,
		FEngineShowFlags InEngineShowFlags,
		const bool bInAdditionalViewFamily
	) const = 0;

	/** This function sets the engine flags corresponding to the rendering mode. */
	static void SetupEngineShowFlags(const EDisplayClusterViewportCaptureMode InMode, FEngineShowFlags& InOutEngineShowFlags);

	/** Sets the ViewFamily and SceneView properties correspond to the rendering mode. */
	static void SetupSceneView(const EDisplayClusterViewportCaptureMode InMode, FSceneViewFamily& InOutViewFamily, FSceneView& InOutView);

	/**
	* Initialize view family, using rules
	* [Game thread func]
	*
	* @param InFrameTarget          - frame target
	* @param InFrameViewFamily      - the frame data for the view family
	* @param OutRenderOutViewFamily - (in, out) The view family that will be configured.
	*
	*/
	virtual void ConfigureViewFamily(
		const FDisplayClusterRenderFrameTarget& InFrameTarget,
		const FDisplayClusterRenderFrameTargetViewFamily& InFrameViewFamily,
		FSceneViewFamilyContext& InOutViewFamily) = 0;

	/**
	* Post-Initialize view family, using rules
	* Should be called after all 'FSceneView' have been added.
	* This function configures the screen percentage.
	* [Game thread func]
	*
	* @param InFrameTarget            - frame target
	* @param InFrameViewFamily        - the frame data for the view family
	* @param InOutRenderOutViewFamily - (in, out) The view family that will be configured.
	* @param InOutViews               - (in, out) The views in the family that will be configured.
	* @param InRenderingFlags         - (in, opt) Rules for customizing nDisplay views and view families for the renderer.
	* @param InDPIScale               - (in, opt ) DPI scale value
	*
	*/
	virtual void PostConfigureViewFamily(
		const FDisplayClusterRenderFrameTarget& InFrameTarget,
		const FDisplayClusterRenderFrameTargetViewFamily& InFrameViewFamily,
		FSceneViewFamilyContext& InOutViewFamily,
		const TArray<FSceneView*>& InOutViews,
		const EDisplayClusterViewportRenderingFlags InRenderingFlags = EDisplayClusterViewportRenderingFlags::None,
		const float InDPIScale = 1.f) = 0;

	/** Send to render thread. */
	virtual void RenderFrame(FViewport* InViewport) = 0;

	/** Add internal DCVM objects to the reference collector. */
	virtual void AddReferencedObjects(FReferenceCollector& Collector) = 0;

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	/**
	* Find viewport object by name
	* [Game thread func]
	*
	* @param ViewportId - Viewport name
	*
	* @return - viewport object ref
	*/
	virtual IDisplayClusterViewport* FindViewport(const FString& InViewportId) const = 0;

	/**
	* Find viewport object and context number by stereoscopic pass index
	* [Game thread func]
	*
	* @param StereoViewIndex - stereoscopic view index
	* @param OutContextNum - context number
	*
	* @return - viewport object ref
	*/
	virtual IDisplayClusterViewport* FindViewport(const int32 StereoViewIndex, uint32* OutContextNum = nullptr) const = 0;
	
	/**
	* Return all exist viewports objects for current cluster node
	* [Game thread func]
	*
	* @return - arrays with viewport objects refs
	*/
	virtual const TArrayView<TSharedPtr<IDisplayClusterViewport, ESPMode::ThreadSafe>> GetCurrentRenderFrameViewports() const = 0;

	/**
	* Return entire cluster viewports objects
	* [Game thread func]
	*
	* @return - arrays with viewport objects refs
	*/
	virtual const TArrayView<TSharedPtr<IDisplayClusterViewport, ESPMode::ThreadSafe>> GetEntireClusterViewports() const = 0;

	/**
	 * Returns the viewports from the entire cluster associated with the specified warp policy
	 * [Game thread func]
	 *
	 * @param InWarpPolicy - The warp policy to get viewports for
	 * @return - A list of viewports associated with the warp policy
	 */
	virtual TArray<TSharedPtr<IDisplayClusterViewport, ESPMode::ThreadSafe>> GetEntireClusterViewportsForWarpPolicy(const TSharedPtr<IDisplayClusterWarpPolicy>& InWarpPolicy) const = 0;

	/**
	* Mark the geometry of the referenced component(s) as dirty (ProceduralMesh, etc)
	* [Game thread func]
	*
	* @param InComponentName - (optional) Unique component name
	*/
	virtual void MarkComponentGeometryDirty(const FName InComponentName = NAME_None) = 0;

	/**
	* Call LocalPlayer->CalcSceneView()
	* [Game thread func]
	*
	* @param LocalPlayer - local player
	* @param ViewFamily - output view struct
	* @param OutViewLocation - output actor location
	* @param OutViewRotation - output actor rotation
	* @param Viewport - current client viewport
	* @param ViewDrawer - optional drawing in the view
	* @param StereoViewIndex - index of the view when using stereoscopy
	*/
	virtual class FSceneView* CalcSceneView(
		class ULocalPlayer* LocalPlayer,
		class FSceneViewFamily* ViewFamily,
		FVector& OutViewLocation,
		FRotator& OutViewRotation,
		class FViewport* Viewport,
		class FViewElementDrawer* ViewDrawer,
		int32 StereoViewIndex) = 0;

	///////////////// UE_DEPRECATED 5.3 ///////////////////

	UE_DEPRECATED(5.3, "This function has been deprecated. Please use 'GetCurrentRenderFrameViewports'.")
		virtual const TArrayView<IDisplayClusterViewport*> GetViewports() const
	{
		return TArrayView<IDisplayClusterViewport*>();
	}

	///////////////// UE_DEPRECATED 5.4 ///////////////////
	
	/** Return current render mode. */
	UE_DEPRECATED(5.4, "This function has been deprecated. Please use 'GetConfiguration()'.")
	virtual EDisplayClusterRenderFrameMode GetRenderMode() const
	{
		return EDisplayClusterRenderFrameMode::Unknown;
	}

	UE_DEPRECATED(5.4, "This function has been deprecated. Please use 'GetConfiguration()'.")
	virtual UWorld* GetCurrentWorld() const
	{
		return nullptr;
	}

	UE_DEPRECATED(5.4, "This function has been deprecated. Please use 'GetConfiguration()'.")
	virtual ADisplayClusterRootActor* GetRootActor() const
	{
		return nullptr;
	}

	/**
	* Returns true if the scene is open now (The current world is assigned and DCRA has already initialized for it).
	* [Game thread func]
	*/
	UE_DEPRECATED(5.4, "This function has been deprecated. Please use 'GetConfiguration()'.")
	virtual bool IsSceneOpened() const
	{
		return false;
	}

	/**
	* Update\Create\Delete local node viewports
	* Update ICVFX configuration from root actor components
	* [Game thread func]
	*
	* @param InRenderMode     - Render mode
	* @param InClusterNodeId  - cluster node for rendering
	* @param InRootActorPtr   - reference to RootActor with actual configuration inside
	* @param InPreviewSettings - support preview rendering
	*
	* @return - true, if success
	*/
	UE_DEPRECATED(5.4, "This function has been deprecated. Please use 'GetConfiguration()'.")
	virtual bool UpdateConfiguration(EDisplayClusterRenderFrameMode InRenderMode, const FString& InClusterNodeId, class ADisplayClusterRootActor* InRootActorPtr, const struct FDisplayClusterPreviewSettings* InPreviewSettings = nullptr)
	{
		return false;
	}

	/**
	* Update\Create\Delete viewports for frame. For rendering outside of cluster nodes
	* Update ICVFX configuration from root actor components
	* [Game thread func]
	*
	* @param InRenderMode    - Render mode
	* @param InViewportNames - Viewports names for next frame
	* @param InRootActorPtr  - reference to RootActor with actual configuration inside
	*
	* @return - true, if success
	*/
	UE_DEPRECATED(5.4, "This function has been deprecated. Please use 'GetConfiguration()'.")
	virtual bool UpdateCustomConfiguration(EDisplayClusterRenderFrameMode InRenderMode, const TArray<FString>& InViewportNames, class ADisplayClusterRootActor* InRootActorPtr)
	{
		return false;
	}

	/**
	* Initialize new frame for all viewports on game thread, and update context, render resources with viewport new settings
	* And finally build render frame structure and send to render thread proxy viewport objects
	* [Game thread func]
	*
	* @param InViewport          - target viewport
	* @param OutRenderFrame      - output render frame container
	*
	* @return - true, if success
	*/
	UE_DEPRECATED(5.4, "This function has been deprecated. Please use 'BeginNewFrame()'.")
	virtual bool BeginNewFrame(FViewport* InViewport, UWorld* InWorld, FDisplayClusterRenderFrame& OutRenderFrame)
	{
		return false;
	}


	/**
	* Render in editor (preview)
	* [Game thread func]
	*
	* @param InRenderFrame - render frame setup
	* @param InViewport
	* @param InFirstViewportNum - begin render from this viewport in frame
	* @param InViewportsAmount - max viewports for render
	* @param OutViewportsAmount - total viewport rendered
	* @param bOutFrameRendered - true, if cluster node composition pass done (additional pass after last viewport is rendered)
	*
	* @return - true, if render success
	*/
	UE_DEPRECATED(5.4, "This function has been deprecated. Please use 'GetViewportPreview().Render()'.")
	virtual bool RenderInEditor(FDisplayClusterRenderFrame& InRenderFrame, FViewport* InViewport, const uint32 InFirstViewportNum, const int32 InViewportsAmount, int32& OutViewportsAmount, bool& bOutFrameRendered)
	{
		return false;
	}
};
