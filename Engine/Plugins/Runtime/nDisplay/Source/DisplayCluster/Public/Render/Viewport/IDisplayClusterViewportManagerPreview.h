// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Render/Viewport/IDisplayClusterViewportConfiguration.h"
#include "Render/Viewport/IDisplayClusterViewportPreview.h"

class FDisplayClusterRenderFrame;
class FViewport;
class FCanvas;

/**
* Viewport manager preview rendering interface
*/
class DISPLAYCLUSTER_API IDisplayClusterViewportManagerPreview
{
public:
	virtual ~IDisplayClusterViewportManagerPreview() = default;

public:
	/** Get TSharedPtr from self. */
	virtual TSharedPtr<IDisplayClusterViewportManagerPreview, ESPMode::ThreadSafe> ToSharedPtr() = 0;
	virtual TSharedPtr<const IDisplayClusterViewportManagerPreview, ESPMode::ThreadSafe> ToSharedPtr() const = 0;

	/** Get viewport manager configuration interface. */
	virtual IDisplayClusterViewportConfiguration& GetConfiguration() = 0;
	virtual const IDisplayClusterViewportConfiguration& GetConfiguration() const = 0;

	/** A preview of the entire cluster is rendered using the tick-per-frame and viewport-per-frame settings.
	* This function should be called several times until the entire cluster is rendered.
	* The rendering is distributed over multiple frames to improve performance.
	* The rendering of multiple instances is also distributed to improve performance.
	* 
	* Note: When rendering the entire cluster, the CurrentWorld value will always be overridden by the GetWorld() value obtained from SceneRootActor.
	* 
	* @param bEnablePreviewRendering - to render or stop rendering.
	*/
	virtual void UpdateEntireClusterPreviewRender(bool bEnablePreviewRendering) = 0;

	/** Reset preview rendering for entire cluster. */
	virtual void ResetEntireClusterPreviewRendering() = 0;

	/**
	* Initialization of preview rendering for a cluster node
	* 
	* @param InRenderMode    - Rendering mode
	* @param InWorld         - World to render
	* @param InClusterNodeId - cluster node name
	* @param InViewport      - Output viewport
	* 
	* @return true if success
	*/
	virtual bool InitializeClusterNodePreview(const EDisplayClusterRenderFrameMode InRenderMode, UWorld* InWorld, const FString& InClusterNodeId, FViewport* InViewport) = 0;

	/**
	* Render specified ammount of viewport for current cluster node
	*
	* @param InViewportsAmmount - Total ammount of viewports that should be rendered on this frame
	*                             INDEX_NONE means render all viewports of this node
	* @param InViewport         - (opt) the dest window
	*
	* @return a value equal to InViewportsAmmount minus the total number of rendered viewports.
	*/
	virtual int32 RenderClusterNodePreview(const int32 InViewportsAmmount, FViewport* InViewport = nullptr, FCanvas* SceneCanvas = nullptr) = 0;

	/**
	* Returns all cluster viewport objects that are visible, displayed on the output texture, and used in the preview.
	* [Game thread func]
	*
	* @return - arrays with viewport preview objects refs
	*/
	virtual const TArray<TSharedPtr<IDisplayClusterViewportPreview, ESPMode::ThreadSafe>> GetEntireClusterPreviewViewports() const = 0;

	DECLARE_DELEGATE(FOnOnEntireClusterPreviewGenerated);
	/** Rendering preview of the entire cluster node is complete. */
	virtual FOnOnEntireClusterPreviewGenerated& GetOnEntireClusterPreviewGenerated() = 0;

	DECLARE_DELEGATE_OneParam(FOnOnClusterNodePreviewGenerated, const FString&);
	/** Rendering preview of the current cluster node is complete. */
	virtual FOnOnClusterNodePreviewGenerated& GetOnClusterNodePreviewGenerated() = 0;
};
