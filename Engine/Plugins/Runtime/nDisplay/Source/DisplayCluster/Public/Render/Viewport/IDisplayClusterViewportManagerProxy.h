// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Render/Viewport/IDisplayClusterViewportProxy.h"

/**
 * nDisplay ViewportManagerProxy (interface for RenderThread)
 */
class DISPLAYCLUSTER_API IDisplayClusterViewportManagerProxy
{
public:
	virtual ~IDisplayClusterViewportManagerProxy() = default;

public:
	/** Get TSharedPtr from self. */
	virtual TSharedPtr<IDisplayClusterViewportManagerProxy, ESPMode::ThreadSafe> ToSharedPtr() = 0;
	virtual TSharedPtr<const IDisplayClusterViewportManagerProxy, ESPMode::ThreadSafe> ToSharedPtr() const = 0;

	/** Get viewport manager proxy configuration interface. */
	virtual const IDisplayClusterViewportConfigurationProxy& GetConfigurationProxy() const = 0;

	/**
	* Find viewport render thread proxy object by name
	* [Rendering thread func]
	*
	* @param ViewportId - Viewport name
	*
	* @return - viewport proxy object ref
	*/
	virtual IDisplayClusterViewportProxy* FindViewport_RenderThread(const FString& InViewportId) const = 0;

	/**
	* Find viewport render thread proxy object and context number by stereoscopic pass index
	* [Rendering thread func]
	*
	* @param StereoViewIndex - stereoscopic view index
	* @param OutContextNum - context number
	*
	* @return - viewport render thread proxy object ref
	*/
	virtual IDisplayClusterViewportProxy* FindViewport_RenderThread(const int32 StereoViewIndex, uint32* OutContextNum = nullptr) const = 0;

	/**
	* Return all exist viewports render thread proxy objects
	* [Rendering thread func]
	*
	* @return - arrays with viewport render thread proxy objects refs
	*/
	virtual const TArrayView<TSharedPtr<IDisplayClusterViewportProxy, ESPMode::ThreadSafe>> GetViewports_RenderThread() const = 0;

	/**
	* Return render frame targets for current frame
	* [Rendering thread func]
	*
	* @param OutRenderFrameTargets - frame RTTs (left, right)
	* @param OutTargetOffsets - frames offset on backbuffer
	* @param OutAdditionalFrameResources - (optional) array with additional render targetable resources (requested externally FDisplayClusterRenderFrameSettings::bShouldUseAdditionalTargetableFrameResource)
	*
	* @return - true if success
	*/
	virtual bool GetFrameTargets_RenderThread(TArray<FRHITexture*>& OutFrameResources, TArray<FIntPoint>& OutTargetOffsets, TArray<FRHITexture*>* OutAdditionalFrameResources=nullptr) const = 0;

	/**
	* Resolve to backbuffer
	* [Rendering thread func]
	*
	* @param InContextNum - renderframe source context num
	* @param DestArrayIndex - dest array index on backbuffer
	* @param WindowSize - dest backbuffer window size
	*
	* @return - true if success
	*/
	virtual bool ResolveFrameTargetToBackBuffer_RenderThread(FRHICommandListImmediate& RHICmdList, const uint32 InContextNum, const int32 DestArrayIndex, FRHITexture* DstBackBuffer, FVector2D WindowSize) const = 0;

	///////////////// UE_DEPRECATED 5.4 ///////////////////

	/** Return current render mode. */
	UE_DEPRECATED(5.4, "This function has been deprecated. Please use 'GetConfigurationProxy()'.")
	virtual EDisplayClusterRenderFrameMode GetRenderMode() const
	{
		return EDisplayClusterRenderFrameMode::Unknown;
	}
};

