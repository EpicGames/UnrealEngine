// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Render/Viewport/IDisplayClusterViewportConfiguration.h"
#include "Render/Viewport/Containers/DisplayClusterViewportPreview_Enums.h"

class IDisplayClusterViewport;
class UMeshComponent;
class UTextureRenderTarget2D;
class UMaterialInstanceDynamic;

/**
* Provides access to the resources used to render the viewport preview.
*/
class DISPLAYCLUSTER_API IDisplayClusterViewportPreview
{
public:
	virtual ~IDisplayClusterViewportPreview() = default;

public:
	/** Get TSharedPtr from self. */
	virtual TSharedPtr<IDisplayClusterViewportPreview, ESPMode::ThreadSafe> ToSharedPtr() = 0;
	virtual TSharedPtr<const IDisplayClusterViewportPreview, ESPMode::ThreadSafe> ToSharedPtr() const = 0;

	/** Get viewport manager configuration interface. */
	virtual IDisplayClusterViewportConfiguration& GetConfiguration() = 0;
	virtual const IDisplayClusterViewportConfiguration& GetConfiguration() const = 0;

	/** Gets the viewport name. */
	virtual FString GetId() const = 0;

	/** Gets the cluster node name. */
	virtual FString GetClusterNodeId() const = 0;

	/** Returns the viewport that owns this object. */
	virtual TSharedPtr<IDisplayClusterViewport, ESPMode::ThreadSafe> GetOwningViewport() const = 0;

	/** Returns true if the preview has any of the requested flags. */
	virtual bool HasAnyFlags(const EDisplayClusterViewportPreviewFlags InPreviewFlags) const = 0;

	/** Get viewport preview texture. */
	virtual UTextureRenderTarget2D* GetPreviewTextureRenderTarget2D() const = 0;

	/** Get viewport preview mesh. */
	virtual UMeshComponent* GetPreviewMeshComponent() const = 0;

	/** Get viewport preview editable mesh.*/
	virtual UMeshComponent* GetPreviewEditableMeshComponent() const = 0;

	/** Deprecated. The base implementation returns nullptr. Use GetOwningViewport() instead. */
	UE_DEPRECATED(5.8, "This function is deprecated. Use GetOwningViewport() instead.")
	virtual IDisplayClusterViewport* GetViewport() const
	{ return nullptr; };
};
