// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Render/Viewport/Containers/DisplayClusterViewport_Enums.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_Context.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_RenderSettings.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_RenderSettingsICVFX.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_PostRenderSettings.h"
#include "Render/Viewport/Containers/DisplayClusterViewportProxy_Context.h"
#include "Render/Viewport/RenderFrame/DisplayClusterRenderFrameEnums.h"
#include "Render/Viewport/Containers/DisplayClusterViewportFrameData.h"
#include "Render/Viewport/IDisplayClusterViewportConfigurationProxy.h"

#include "Misc/DisplayClusterColorEncoding.h"

#include "RHI.h"
#include "RHICommandList.h"
#include "RHIResources.h"

/**
 * nDisplay: ViewportProxy (interface for RenderThread)
 */
class DISPLAYCLUSTER_API IDisplayClusterViewportProxy
{
public:
	virtual ~IDisplayClusterViewportProxy() = default;

public:
	/** Get TSharedPtr from self. */
	virtual TSharedPtr<IDisplayClusterViewportProxy, ESPMode::ThreadSafe> ToSharedPtr() = 0;
	virtual TSharedPtr<const IDisplayClusterViewportProxy, ESPMode::ThreadSafe> ToSharedPtr() const = 0;

	/** Get TSharedRef from self. */
	virtual TSharedRef<IDisplayClusterViewportProxy, ESPMode::ThreadSafe> ToSharedRef() = 0;
	virtual TSharedRef<const IDisplayClusterViewportProxy, ESPMode::ThreadSafe> ToSharedRef() const = 0;

	/** Returns the usage flags of the viewport. */
	virtual EDisplayClusterViewportFlags GetViewportFlags() const = 0;

	/** Returns the base viewport identifier (not guaranteed to be unique across ICVFX viewports).
	 * Note: For ICVFX viewports returns:
	 *       - The name of the ICVFX camera for InCamera/Chromakey.
	 *       - The name of the Outer viewport for Lightcards.
	 */
	virtual FString GetBaseId() const = 0;

	/** Returns the unique viewport identifier, guaranteed to be unique across the entire cluster. */
	virtual FString GetId() const = 0;

	/** Returns the identifier of the cluster node that owns this viewport. */
	virtual FString GetClusterNodeId() const = 0;

	/** Get viewport manager configuration interface. */
	virtual const IDisplayClusterViewportConfigurationProxy& GetConfigurationProxy() const = 0;

	/** Get projection policy ref. */
	virtual const TSharedPtr<class IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe>& GetProjectionPolicy_RenderThread() const = 0;

	/** Get main viewport render settings. */
	virtual const FDisplayClusterViewport_RenderSettings& GetRenderSettings_RenderThread() const = 0;

	/** Get ICVFX render settings. */
	virtual const FDisplayClusterViewport_RenderSettingsICVFX& GetRenderSettingsICVFX_RenderThread() const = 0;

	/** Get post-render settings. */
	virtual const FDisplayClusterViewport_PostRenderSettings& GetPostRenderSettings_RenderThread() const = 0;

	/** Return viewport proxy contexts data. */
	virtual const TArray<FDisplayClusterViewport_Context>& GetContexts_RenderThread() const = 0;

	/** Get the frame data snapshot captured for the current frame. */
	virtual const FDisplayClusterViewportFrameData& GetFrameData_RenderThread() const = 0;

	/** Get the frame data snapshot captured during the previous frame. */
	virtual const FDisplayClusterViewportFrameData& GetPrevFrameData_RenderThread() const = 0;

	/**
	* Return color-encoding information for the resource
	* 
	* @param InResourceType   - (in) resource type (RTT, Shader, MIPS, etc)
	*/
	virtual FDisplayClusterColorEncoding GetResourceColorEncoding_RenderThread(const EDisplayClusterViewportResourceType InResourceType) const = 0;

	/** Get viewport resources by type
	 *
	 * @param InResourceType - resource type (RTT, Shader, MIPS, etc)
	 * @param OutResources   - [Out] RHI resources array for all contexts
	 *
	 * @return - true if success
	 */
	virtual bool GetResources_RenderThread(const EDisplayClusterViewportResourceType InResourceType, TArray<FRHITexture*>& OutResources) const = 0;

	/** Get viewport resources with rects by type
	 *
	 * @param InResourceType - resource type (RTT, Shader, MIPS, etc)
	 * @param OutResources   - [Out] RHI resources array for all contexts
	 * @param OutRects       - [Out] RHI resources rects array for all contexts
	 *
	 * @return - true if success
	 */
	virtual bool GetResourcesWithRects_RenderThread(const EDisplayClusterViewportResourceType InResourceType, TArray<FRHITexture*>& OutResources, TArray<FIntRect>& OutRects) const = 0;

	/** Set a per-frame resource override for the given resource type.
	 * The override shadows the normal resource-fetch path for this frame only and is cleared automatically at the start of the next frame.
	 * Pass an empty InResources to remove any existing override for InResourceType.
	 *
	 * @param InResourceType - resource type to override (RTT, Shader, MIPS, etc)
	 * @param InResources    - RHI resources to use instead of the viewport's own, one entry per context; pass empty to clear
	 * @param InOptRects     - [optional] subrect within each resource; if null, the full texture extent is used
	 *
	 * @return - true if the override was stored (or cleared) successfully
	 * @see GetResourcesOverride_RenderThread
	 */
	virtual bool SetResourcesOverride_RenderThread(const EDisplayClusterViewportResourceType InResourceType, const TArray<FRHITexture*>& InResources, const TArray<FIntRect>* InOptRects = nullptr) const = 0;

	/** Get the active per-frame resource override for the given resource type, if one was set this frame via SetResourcesOverride_RenderThread.
	 *
	 * @param InResourceType - resource type to query (RTT, Shader, MIPS, etc)
	 * @param OutResources   - [Out] overriding RHI resources, one entry per context
	 * @param OutOptRects    - [optional, Out] subrects for each resource; not written if null
	 *
	 * @return - true if an override is active for InResourceType this frame
	 * @see SetResourcesOverride_RenderThread
	 */
	virtual bool GetResourcesOverride_RenderThread(const EDisplayClusterViewportResourceType InResourceType, TArray<FRHITexture*>& OutResources, TArray<FIntRect>* OutOptRects = nullptr) const = 0;

	/** Copy resource contexts by type
	 *
	 * @param RHICmdList         - RHIinterface
	 * @param InputResourceType  - Input resource type (RTT, Shader, MIPS, etc)
	 * @param OutputResourceType - Output resource type (RTT, Shader, MIPS, etc)
	 * @param InContextNum       - [optional] the type of context to copy (by default, all contexts are copied).
	 *
	 * @return - true if success
	 */
	virtual bool ResolveResources_RenderThread(FRHICommandListImmediate& RHICmdList, const EDisplayClusterViewportResourceType InputResourceType, const EDisplayClusterViewportResourceType OutputResourceType, const int32 InContextNum = INDEX_NONE) const = 0;

	/** Copy resource contexts by type
	 *
	 * @param RHICmdList                 - RHIinterface
	 * @param InputResourceViewportProxy - The 'InputResourceType' resource will be obtained from this object
	 * @param InputResourceType          - Input resource type (RTT, Shader, MIPS, etc)
	 * @param OutputResourceType         - Output resource type (RTT, Shader, MIPS, etc)
	 * @param InContextNum               - [optional] the type of context to copy (by default, all contexts are copied).
	 *
	 * @return - true if success
	 */
	virtual bool ResolveResources_RenderThread(FRHICommandListImmediate& RHICmdList, IDisplayClusterViewportProxy* InputResourceViewportProxy, const EDisplayClusterViewportResourceType InputResourceType, const EDisplayClusterViewportResourceType OutputResourceType, const int32 InContextNum = INDEX_NONE) const = 0;

	/**
	* Change render setting for viewport.
	* This function must be called in a valid moment in time.
	*/
	virtual void SetRenderSettings_RenderThread(const FDisplayClusterViewport_RenderSettings& InRenderSettings) const = 0;

	/**
	* Change viewport contexts data.
	* This function must be called in a valid moment in time.
	*/
	virtual void SetContexts_RenderThread(const TArray<FDisplayClusterViewport_Context>& InContexts) const = 0;

	///////////////// UE_DEPRECATED 5.3 ///////////////////

	/** Return output resource type (support preview, remap, etc). */
	UE_DEPRECATED(5.3, "This function has beend deprecate. Please use the new enumeration values.")
	virtual EDisplayClusterViewportResourceType GetOutputResourceType_RenderThread() const
	{
		return EDisplayClusterViewportResourceType::OutputTargetableResource;
	}

	///////////////// UE_DEPRECATED 5.4 ///////////////////

	/** Return current render mode. */
	UE_DEPRECATED(5.4, "This function has been deprecated. Please use 'GetConfigurationProxy().GetRenderMode_RenderThread()'.")
	virtual EDisplayClusterRenderFrameMode GetRenderMode() const
	{
		return EDisplayClusterRenderFrameMode::Unknown;
	}

	/** Returns ptr to ViewportManagerProxy (the owner of this viewport proxy) if it still exists. */
	UE_DEPRECATED(5.4, "This function has been deprecated. Please use 'GetConfigurationProxy().GetViewportManagerProxy_RenderThread'.")
	virtual const class IDisplayClusterViewportManagerProxy* GetViewportManagerProxy_RenderThread() const
	{
		return nullptr;
	}

	/** Get render frame settings from viewport proxy owner, if it still exists. */
	UE_DEPRECATED(5.4, "This function has been deprecated.")
	virtual const struct FDisplayClusterRenderFrameSettings* GetRenderFrameSettings_RenderThread() const
	{
		return nullptr;
	}
};
