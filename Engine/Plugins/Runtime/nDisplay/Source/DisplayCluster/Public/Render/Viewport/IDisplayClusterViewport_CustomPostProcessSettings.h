// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class IDisplayClusterViewport;
struct FPostProcessSettings;

/**
* DC Viewport Postprocess interface
*/
class DISPLAYCLUSTER_API IDisplayClusterViewport_CustomPostProcessSettings
{
public:
	virtual ~IDisplayClusterViewport_CustomPostProcessSettings() = default;

public:
	/** The viewport's custom PostProcesses are blended in a specific order, which is specified by the values below.*/
	enum class ERenderPass : uint8
	{
		/** These PP settings will be applied when the StartFinalPostprocessSettings() function is called */
		Start = 0,

		/** These PP settings will be applied when the OverrideFinalPostprocessSettings() function is called */
		Override,

		/** These PP settings will be applied when the EndFinalPostprocessSettings() function is called.
		* The `Final` and `FinalPerViewport` are always applied together.*/
		Final,

		/** This rendering pass is for the container only, to separate ICVFX ColorGrading into a separate pass.
		* Note: The value is ignored by ApplyCustomPostProcess(). */
		FinalPerViewport,
	};

public:
	/** Add custom postprocess. */
	virtual void AddCustomPostProcess(const ERenderPass InRenderPass, const FPostProcessSettings& InSettings, float BlendWeight, bool bSingleFrame) = 0;

	/** Deletes the custom postprocess for the specified rendering pass. */
	virtual void RemoveCustomPostProcess(const ERenderPass InRenderPass) = 0;

	/** Apply postprocess for the specified rendering pass to the InOutStartPostProcessingSettings.
	* 
	* @param InViewport       - DC viewport that is rendered with these settings.
	* @param InContextNum     - Index of view that is being processed for this viewport
	* @param InRenderPass     - PP rendering pass to be used. (The value of 'FinalPerViewport' is ignored.)
	* @param InOutPPSettings  - (in,out) PostProcess settings that need to be changed.
	* @param InOutBlendWeight - (opt, in, out) PostProcess weight parameter that need to be changed.
	* 
	* @return true, if the postprocess settings have been overridden.
	*/
	virtual bool ApplyCustomPostProcess(IDisplayClusterViewport* InViewport, const uint32 InContextNum, const ERenderPass InRenderPass, FPostProcessSettings& InOutPPSettings, float* InOutBlendWeight = nullptr) const = 0;

public:
	/** Override postprocess, if defined. */
	UE_DEPRECATED(5.5, "This function has been deprecated. Please use 'ApplyCustomPostProcess()'.")
	virtual bool DoPostProcess(const ERenderPass InRenderPass, struct FPostProcessSettings* OutSettings, float* OutBlendWeight = nullptr) const
	{
		return false;
	}

};
