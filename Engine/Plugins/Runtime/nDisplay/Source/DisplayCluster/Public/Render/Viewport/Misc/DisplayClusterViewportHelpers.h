// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DisplayClusterRootActor.h"
#include "Render/Viewport/IDisplayClusterViewportConfiguration.h"

/**
* Helper functions for the DisplayClusterViewport
*/
namespace UE::DisplayClusterViewportHelpers
{
	/** Find a component by name from a RootActor of the specified type.
	*
	* @param InConfiguration - current configuration
	* @param InRootActorType - the search is performed within the DCRA of the given type.
	* @param InComponentName - the name of the component
	*/
	template<class TComp>
	TComp* GetRootActorComponentByName(const IDisplayClusterViewportConfiguration& InConfiguration, const EDisplayClusterRootActorType InRootActorType, const FString& InComponentName)
	{
		if (InComponentName.IsEmpty())
		{
			return nullptr;
		}

		if (const ADisplayClusterRootActor* RequestedRootActor = InConfiguration.GetRootActor(InRootActorType))
		{
			if (TComp* Component = RequestedRootActor->GetComponentByName<TComp>(InComponentName))
			{
				return Component;
			}
		}

		return nullptr;
	}

	/** Find a component by name from a RootActor of the specified type.
	*
	* @param ComponentOfRootActor
	* @param InComponentName - the name of the component
	*/
	template<class TComp>
	TComp* GetOwnerRootActorComponentByName(USceneComponent* ComponentOfRootActor, const FString& InComponentName)
	{
		if (InComponentName.IsEmpty())
		{
			return nullptr;
		}

		if (AActor* OwnerActor = ComponentOfRootActor->GetOwner())
		{
			if (OwnerActor->IsA<ADisplayClusterRootActor>())
			{
				if (ADisplayClusterRootActor* OwnerRootActor = Cast<ADisplayClusterRootActor>(OwnerActor))
				{
					if (TComp* OutComponent = OwnerRootActor->GetComponentByName<TComp>(InComponentName))
					{
						return OutComponent;
					}
				}
			}
		}

		return nullptr;
	}

	/** Return the same component (by class and name) from a DCRA of the specified type.
	* If no other component exists, or if the current DCRA type matches the requested one,
	* the input component is returned.
	*
	* @param InRootActorType - the search is performed within the DCRA of the given type.
	* @param InComponent     - The component that is used for the search.
	*
	* @return A component with the same name and class belonging to a DCRA of this type.
	*/
	template<class TComp>
	const TComp& GetMatchingComponentFromRootActor(const IDisplayClusterViewportConfiguration& InConfiguration, const EDisplayClusterRootActorType InRootActorType, const TComp& InComponent)
	{
		if (const ADisplayClusterRootActor* RequestedRootActor = InConfiguration.GetRootActor(InRootActorType))
		{
			// If the current DCRA type does not match the requested one.
			if (RequestedRootActor != InComponent.GetOwner())
			{
				if (const TComp* OutComponent = RequestedRootActor->GetComponentByName<TComp>(InComponent.GetName()))
				{
					return *OutComponent;
				}
			}
		}

		return InComponent;
	}
};
