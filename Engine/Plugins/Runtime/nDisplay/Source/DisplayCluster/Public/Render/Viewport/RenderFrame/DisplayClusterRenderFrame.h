// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DisplayClusterRenderFrameEnums.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_RenderSettings.h"

#include "SceneViewExtension.h"

class IDisplayClusterViewport;
class IDisplayClusterViewportManager;
class FDisplayClusterViewportResource;

/**
 * nDisplay: DCViewport context for render in UE View
 */
struct FDisplayClusterRenderFrameTargetView
{
	/** Is this viewport context can be rendered. */
	inline bool IsViewportContextCanBeRendered() const
	{
		return !bDisableRender && !bFreezeRendering;
	}

	// Viewport context index for this view
	uint32 ContextNum = 0;

	// Some viewports may skip rendering (for example: overridden)
	bool bDisableRender = false;

	// In special cases, viewports can reuse an existing RTT image
	// For example: the preview is rendered over several frames, so the already rendered viewports remain "frozen" and can be used during subsequent frames.
	bool bFreezeRendering = false;

	// Viewport game-thread data
	TSharedPtr<IDisplayClusterViewport, ESPMode::ThreadSafe> Viewport;
};

/**
 * nDisplay: A group of DCViewports that can be render within a single viewfamily
 */
struct FDisplayClusterRenderFrameTargetViewFamily
{
	// Customize ScreenPercentage feature for viewfamily
	float CustomBufferRatio = 1;

	// ViewFamily Upscaler settings
	FDisplayClusterUpscalerSettings UpscalerSettings;

	// Extensions that can modify view parameters
	TArray<FSceneViewExtensionRef> ViewExtensions;

	// Viewports, rendered at once for this family
	TArray<FDisplayClusterRenderFrameTargetView> Views;
};

/**
 * nDisplay: Target texture for rendering.
 * Contains structured information on how to render DCViewports via viewfamilies in RTT
 */
struct FDisplayClusterRenderFrameTarget
{
	// Discard some RTT (when view render disabled)
	// Also when RTT Atlasing used, this viewports excluded from atlas map (reduce size)
	bool bShouldUseRenderTarget = true;

	// required Render target size (resource can be bigger)
	FIntPoint RenderTargetSize;

	// Viewport capture mode
	// This mode affects many viewport rendering settings.
	EDisplayClusterViewportCaptureMode CaptureMode = EDisplayClusterViewportCaptureMode::Default;

	// Render target resource ref
	TSharedPtr<FDisplayClusterViewportResource, ESPMode::ThreadSafe> RenderTargetResource;

	// UE Viewport used to render
	FViewport* Viewport = nullptr;

	// Families, rendered on this target
	TArray<FDisplayClusterRenderFrameTargetViewFamily> ViewFamilies;
};

/**
 * nDisplay: Render frame container
 * The final frame is composed of DCViewports, which are rendered in the correct order on a few RTTs.
 * Contains all information about how to render all DCViewports for the current frame and the settings for it.
 */
class FDisplayClusterRenderFrame
{
public:
	/** Returns a pointer to the DCViewportManager used for this frame. */
	IDisplayClusterViewportManager* GetViewportManager() const
	{
		return ViewportManagerWeakPtr.IsValid() ? ViewportManagerWeakPtr.Pin().Get() : nullptr;
	}

public:
	// Render frame to this targets
	TArray<FDisplayClusterRenderFrameTarget> RenderTargets;
	
	// Frame rect on final backbuffer
	FIntRect FrameRect;

	// Desired numbers of view
	int32 DesiredNumberOfViews = 0;

	// Ref to the DC Viewport Manager
	TWeakPtr<IDisplayClusterViewportManager, ESPMode::ThreadSafe> ViewportManagerWeakPtr;
};
