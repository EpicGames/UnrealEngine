// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * Defines the rendering output mode.
 */
enum class EDisplayClusterRenderFrameMode : uint8
{
	Unknown = 0,

	// Monoscopic rendering to a backbuffer texture.
	Mono,

	// Stereoscopic rendering to a layered backbuffer texture (quad-buffer).
	Stereo,

	// Stereoscopic rendering packed into one texture:
	// left eye in the left half, right eye in the right half.
	SideBySide,

	// Stereoscopic rendering packed into one texture:
	// left eye in the top half, right eye in the bottom half.
	TopBottom,

	// Monoscopic rendering in Play In Editor (PIE).
	PIE_Mono,

	// Stereoscopic rendering in PIE packed into one texture:
	// left eye in the left half, right eye in the right half.
	PIE_SideBySide,

	// Stereoscopic rendering in PIE packed into one texture:
	// left eye in the top half, right eye in the bottom half.
	PIE_TopBottom,

	// Monoscopic rendering for Movie Render Queue (MRQ).
	MRQ_Mono,

	// Stereoscopic rendering for Movie Render Queue (MRQ).
	MRQ_Stereo,

	// Monoscopic rendering for Movie Render Graph (MRG).
	MRG_Mono,

	// Stereoscopic rendering for Movie Render Graph (MRG).
	MRG_Stereo,

	// Rendering mode used for in-scene preview.
	PreviewInScene,

	// ProxyHit rendering mode used for in-scene preview.
	// Not implemented.
	PreviewProxyHitInScene,
};