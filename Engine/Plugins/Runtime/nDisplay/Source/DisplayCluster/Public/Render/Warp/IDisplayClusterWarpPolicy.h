// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Render/DisplayDevice/Containers/DisplayClusterDisplayDevice_Enums.h"

class IDisplayClusterViewport;
class IDisplayClusterViewportManager;
class IDisplayClusterViewportPreview;
class UMaterialInstanceDynamic;
class UMeshComponent;

/**
 * Warp policy interface
 * Customize warp math for projection policies.
 * Multiple projections can use the same warp policy.
 */
class IDisplayClusterWarpPolicy
{
public:
	virtual ~IDisplayClusterWarpPolicy() = default;

public:
	/** Get TSharedPtr from self. */
	virtual TSharedPtr<IDisplayClusterWarpPolicy, ESPMode::ThreadSafe> ToSharedPtr() = 0;
	virtual TSharedPtr<const IDisplayClusterWarpPolicy, ESPMode::ThreadSafe> ToSharedPtr() const = 0;

	/**
	* Return warp policy instance ID
	*/
	virtual const FString& GetId() const = 0;

	/**
	* Return warp policy type
	*/
	virtual const FString& GetType() const = 0;


	/** Returns true if this policy supports ICVFX rendering
	 * 
	 * @param InViewport - a owner viewport
	 * 
	 * @return - true if this warp policy supports ICVFX
	 */
	virtual bool ShouldSupportICVFX(const IDisplayClusterViewport* InViewport) const
	{
		return false;
	}

	/**
	* Handle new frame for viewports group.
	* This function is called once per frame at the beginning, immediately after the viewport setup is complete.
	* 
	* @param InViewports - a group of viewports from the entire cluster that use this WarpInterface
	*/
	virtual void HandleNewFrame(const TArray<TSharedPtr<IDisplayClusterViewport, ESPMode::ThreadSafe>>& InViewports)
	{ }

	/**
	* Tick function
	* The positions of editable preview components may be updated on each frame.
	* Use this function to handle these updates.
	* 
	* @param InViewportManager - Viewport manager interface
	* @param DeltaSeconds      - delta time
	*/
	virtual void Tick(IDisplayClusterViewportManager* InViewportManager, float DeltaSeconds)
	{ }

	/** Should override frustum for viewport context
	 * 
	 * @param InViewport - a owner viewport
	 * @param ContextNum - viewport eye context index
	 * 
	 * @return - true, if the CalcFrustumOverrideFunc() function will be used
	 */
	virtual bool ShouldOverrideCalcFrustum(IDisplayClusterViewport* InViewport)
	{
		return false;
	}

	/** Override frustum for viewport context
	 * This function is called only when the ShouldOverrideCalcFrustum() function returns true
	 * 
	 * @param InViewport - a owner viewport
	 * @param ContextNum - viewport eye context index
	 * 
	 * @return - true if frustum is overridden.
	 */
	virtual bool OverrideCalcFrustum(IDisplayClusterViewport* InViewport, const uint32 ContextNum)
	{
		return false;
	}

	/** Call before CalcFrustum()
	 *
	 * @param InViewport - a owner viewport
	 * @param ContextNum - viewport eye context index
	 */
	virtual void BeginCalcFrustum(IDisplayClusterViewport* InViewport, const uint32 ContextNum)
	{ }

	/** Call after CalcFrustum()
	 *
	 * @param InViewport - a owner viewport
	 * @param ContextNum - viewport eye context index
	 */
	virtual void EndCalcFrustum(IDisplayClusterViewport* InViewport, const uint32 ContextNum)
	{ }

	/**
	* Ask warp  policy instance if it has any Editable mesh based preview
	* @param InViewport - a owner viewport
	* @return - True if mesh based preview is available
	*/
	virtual bool HasPreviewEditableMesh(IDisplayClusterViewport* InViewport)
	{
		return false;
	}

	/** Perform any operations on the  mesh and material instance, such as setting parameter values.
	* 
	* @param InViewport - current viewport
	* @param InMeshType - mesh type
	* @param InMaterialType - type of material being requested
	* @param InMeshComponent - mesh component to be updated
	* @param InMeshMaterialInstance - material instance that used on this mesh
	*/
	virtual void OnUpdateDisplayDeviceMeshAndMaterialInstance(IDisplayClusterViewportPreview& InViewportPreview, const EDisplayClusterDisplayDeviceMeshType InMeshType, const EDisplayClusterDisplayDeviceMaterialType InMaterialType, UMeshComponent* InMeshComponent, UMaterialInstanceDynamic* InMeshMaterialInstance) const
	{ }
};
