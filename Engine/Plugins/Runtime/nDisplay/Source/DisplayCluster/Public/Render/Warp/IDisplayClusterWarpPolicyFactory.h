// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class IDisplayClusterWarpPolicy;

/**
 * nDisplay warp policy factory interface
 */
class IDisplayClusterWarpPolicyFactory
{
public:
	virtual ~IDisplayClusterWarpPolicyFactory() = default;

public:
	/**
	* Creates a warp policy instance
	*
	* @param InWarpPolicyType - Warp policy type, same as specified on registration (useful if the same factory is registered for multiple warp policy types)
	* @param InWarpPolicyName - Warp policy name
	*
	* @return - Warp policy instance
	*/
	virtual TSharedPtr<IDisplayClusterWarpPolicy, ESPMode::ThreadSafe> Create(const FString& InWarpPolicyType, const FString& InWarpPolicyName) = 0;
};
