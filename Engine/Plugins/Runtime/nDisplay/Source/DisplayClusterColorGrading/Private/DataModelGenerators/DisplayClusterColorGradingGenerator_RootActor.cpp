// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterColorGradingGenerator_RootActor.h"

#include "DisplayClusterColorGradingStyle.h"
#include "IDisplayClusterColorGrading.h"
#include "IDisplayClusterColorGradingDrawerSingleton.h"

#include "DisplayClusterRootActor.h"
#include "Components/DisplayClusterICVFXCameraComponent.h"

#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterConfigurationTypes_ICVFX.h"
#include "DisplayClusterConfigurationTypes_Postprocess.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailCustomization.h"
#include "IDetailTreeNode.h"
#include "IPropertyRowGenerator.h"
#include "PropertyHandle.h"
#include "PropertyPathHelpers.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "DisplayClusterColorGrading"

#define GET_MEMBER_NAME_ARRAY_CHECKED(ClassName, MemberName, Index) FName(FString(GET_MEMBER_NAME_STRING_CHECKED(ClassName, MemberName)).Replace(TEXT(#Index), *FString::FromInt(Index), ESearchCase::CaseSensitive))

FColorGradingEditorDataModel::FColorGradingGroup FDisplayClusterColorGradingGenerator_ColorGradingRenderingSettings::CreateColorGradingGroup(const TSharedPtr<IPropertyHandle>& GroupPropertyHandle, bool bOverrideResetToDefault)
{
	FColorGradingEditorDataModel::FColorGradingGroup ColorGradingGroup;
	ColorGradingGroup.DisplayName = GroupPropertyHandle->GetPropertyDisplayName();
	ColorGradingGroup.GroupPropertyHandle = GroupPropertyHandle;

	ColorGradingGroup.ColorGradingElements.Add(CreateColorGradingElement(GroupPropertyHandle, TEXT("Global"), LOCTEXT("ColorGrading_GlobalLabel", "Global"), bOverrideResetToDefault));
	ColorGradingGroup.ColorGradingElements.Add(CreateColorGradingElement(GroupPropertyHandle, TEXT("Shadows"), LOCTEXT("ColorGrading_ShadowsLabel", "Shadows"), bOverrideResetToDefault));
	ColorGradingGroup.ColorGradingElements.Add(CreateColorGradingElement(GroupPropertyHandle, TEXT("Midtones"), LOCTEXT("ColorGrading_MidtonesLabel", "Midtones"), bOverrideResetToDefault));
	ColorGradingGroup.ColorGradingElements.Add(CreateColorGradingElement(GroupPropertyHandle, TEXT("Highlights"), LOCTEXT("ColorGrading_HighlightsLabel", "Highlights"), bOverrideResetToDefault));

	ColorGradingGroup.DetailsViewCategories =
	{
		TEXT("DetailView_Exposure"),
		TEXT("DetailView_ColorGrading"),
		TEXT("DetailView_WhiteBalance"),
		TEXT("DetailView_Misc")
	};

	return ColorGradingGroup;
}

FColorGradingEditorDataModel::FColorGradingElement FDisplayClusterColorGradingGenerator_ColorGradingRenderingSettings::CreateColorGradingElement(
	const TSharedPtr<IPropertyHandle>& GroupPropertyHandle,
	FName ElementPropertyName,
	FText ElementLabel,
	bool bOverrideResetToDefault)
{
	FColorGradingEditorDataModel::FColorGradingElement ColorGradingElement;
	ColorGradingElement.DisplayName = ElementLabel;

	TSharedPtr<IPropertyHandle> ElementPropertyHandle = GroupPropertyHandle->GetChildHandle(ElementPropertyName);
	if (ElementPropertyHandle.IsValid() && ElementPropertyHandle->IsValidHandle())
	{
		ColorGradingElement.SaturationPropertyHandle = ElementPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_ColorGradingSettings, Saturation));
		ColorGradingElement.ContrastPropertyHandle = ElementPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_ColorGradingSettings, Contrast));
		ColorGradingElement.GammaPropertyHandle = ElementPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_ColorGradingSettings, Gamma));
		ColorGradingElement.GainPropertyHandle = ElementPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_ColorGradingSettings, Gain));
		ColorGradingElement.OffsetPropertyHandle = ElementPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_ColorGradingSettings, Offset));

		if (bOverrideResetToDefault)
		{
			auto ResetPropertyColorValue = [](TSharedPtr<IPropertyHandle> InPropertyHandle, FVector4 InDefaultValue)
			{
				// Set the value with an interactive, non-transactable change first to avoid invoking any post edit change events on each component
				// (Vector properties use property handles to set each component's value), in case the object owning the property becomes invalid
				// on post edit change events (such as construction script created objects)
				InPropertyHandle->SetValue(InDefaultValue, EPropertyValueSetFlags::InteractiveChange | EPropertyValueSetFlags::NotTransactable);

				// Now update the value normally to invoke the usual notification/transaction pipeline
				InPropertyHandle->SetValue(InDefaultValue, EPropertyValueSetFlags::ResetToDefault);
			};
			
			// If we are overriding the reset to default, assume we want to reset the color values to the defaults defined in FDisplayClusterConfigurationViewport_ColorGradingSettings
			ColorGradingElement.SaturationResetToDefaultOverride = FResetToDefaultOverride::Create(FResetToDefaultHandler::CreateLambda(ResetPropertyColorValue, FVector4::One()));
			ColorGradingElement.ContrastResetToDefaultOverride = FResetToDefaultOverride::Create(FResetToDefaultHandler::CreateLambda(ResetPropertyColorValue, FVector4::One()));
			ColorGradingElement.GammaResetToDefaultOverride = FResetToDefaultOverride::Create(FResetToDefaultHandler::CreateLambda(ResetPropertyColorValue, FVector4::One()));
			ColorGradingElement.GainResetToDefaultOverride = FResetToDefaultOverride::Create(FResetToDefaultHandler::CreateLambda(ResetPropertyColorValue, FVector4::One()));
			ColorGradingElement.OffsetResetToDefaultOverride = FResetToDefaultOverride::Create(FResetToDefaultHandler::CreateLambda(ResetPropertyColorValue, FVector4::Zero()));
		}
	}

	return ColorGradingElement;
}

TSharedPtr<IDetailTreeNode> FDisplayClusterColorGradingGenerator_ColorGradingRenderingSettings::FindPropertyTreeNode(const TSharedRef<IDetailTreeNode>& Node, const FCachedPropertyPath& PropertyPath)
{
	if (Node->GetNodeType() == EDetailNodeType::Item)
	{
		if (Node->GetNodeName() == PropertyPath.GetLastSegment().GetName())
		{
			TSharedPtr<IPropertyHandle> FoundPropertyHandle = Node->CreatePropertyHandle();
			FString FoundPropertyPath = FoundPropertyHandle->GeneratePathToProperty();

			if (PropertyPath == FoundPropertyPath)
			{
				return Node;
			}
		}
		
		return nullptr;
	}
	else
	{
		TArray<TSharedRef<IDetailTreeNode>> Children;
		Node->GetChildren(Children);
		for (const TSharedRef<IDetailTreeNode>& Child : Children)
		{
			if (TSharedPtr<IDetailTreeNode> PropertyTreeNode = FindPropertyTreeNode(Child, PropertyPath))
			{
				return PropertyTreeNode;
			}
		}

		return nullptr;
	}
}

TSharedPtr<IPropertyHandle> FDisplayClusterColorGradingGenerator_ColorGradingRenderingSettings::FindPropertyHandle(IPropertyRowGenerator& PropertyRowGenerator, const FCachedPropertyPath& PropertyPath)
{
	const TArray<TSharedRef<IDetailTreeNode>>& RootNodes = PropertyRowGenerator.GetRootTreeNodes();

	for (const TSharedRef<IDetailTreeNode>& RootNode : RootNodes)
	{
		if (TSharedPtr<IDetailTreeNode> PropertyTreeNode = FindPropertyTreeNode(RootNode, PropertyPath))
		{
			return PropertyTreeNode->CreatePropertyHandle();
		}
	}

	return nullptr;
}

#define CREATE_PROPERTY_PATH(RootObjectClass, PropertyPath) FCachedPropertyPath(GET_MEMBER_NAME_STRING_CHECKED(RootObjectClass, PropertyPath))

TSharedPtr<IPropertyHandle> MakePropertyTransactional(const TSharedPtr<IPropertyHandle>& PropertyHandle)
{
	if (PropertyHandle.IsValid())
	{
		PropertyHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([PropertyHandle]
		{
			TArray<UObject*> OuterObjects;
			PropertyHandle->GetOuterObjects(OuterObjects);
			for (UObject* Object : OuterObjects)
			{
				if (!Object)
				{
					continue;
				}

				if (!Object->HasAnyFlags(RF_Transactional))
				{
					Object->SetFlags(RF_Transactional);
				}

				SaveToTransactionBuffer(Object, false);
				SnapshotTransactionBuffer(Object);
			}
		}));
	}

	return PropertyHandle;
}

TSharedRef<IColorGradingEditorDataModelGenerator> FDisplayClusterColorGradingGenerator_RootActor::MakeInstance()
{
	return MakeShareable(new FDisplayClusterColorGradingGenerator_RootActor());
}

/**
 * A detail customization that picks out only the necessary properties needed to display a root actor in the color grading drawer and hides all other properties
 * Also organizes the properties into custom categories that can be easily displayed in the color grading drawer
 */
class FRootActorDetailsCustomization : public IDetailCustomization
{
public:
	FRootActorDetailsCustomization(const TSharedRef<FColorGradingEditorDataModel>& InColorGradingDataModel)
		: ColorGradingDataModel(InColorGradingDataModel)
	{ }

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override
	{
		TArray<FName> Categories;
		DetailBuilder.GetCategoryNames(Categories);

		for (const FName& Category : Categories)
		{
			DetailBuilder.HideCategory(Category);
		}

		UClass* ConfigDataClass = UDisplayClusterConfigurationData::StaticClass();

		IDetailCategoryBuilder& ColorGradingCategoryBuilder = DetailBuilder.EditCategory(TEXT("ColorGradingCategory"));
		ColorGradingCategoryBuilder.AddProperty(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationData, StageSettings.EntireClusterColorGrading), ConfigDataClass));
		ColorGradingCategoryBuilder.AddProperty(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationData, StageSettings.PerViewportColorGrading), ConfigDataClass));
		ColorGradingCategoryBuilder.AddProperty(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationData, StageSettings.bEnableInnerFrustums), ConfigDataClass));

		AddColorGradingDetailProperties(DetailBuilder);
		AddDetailsPanelProperties(DetailBuilder);

		DetailBuilder.SortCategories([](const TMap<FName, IDetailCategoryBuilder*>& CategoryMap)
		{
			const TMap<FName, int32> SortOrder =
			{
				{ TEXT("DetailView_PerViewport"), 0},
				{ TEXT("DetailView_Exposure"), 1},
				{ TEXT("DetailView_ColorGrading"), 2},
				{ TEXT("DetailView_WhiteBalance"), 3},
				{ TEXT("DetailView_Misc"), 4}
			};

			for (const TPair<FName, int32>& SortPair : SortOrder)
			{
				if (CategoryMap.Contains(SortPair.Key))
				{
					CategoryMap[SortPair.Key]->SetSortOrder(SortPair.Value);
				}
			}
		});
	}

private:
	void AddColorGradingDetailProperties(IDetailLayoutBuilder& DetailBuilder)
	{
		auto AddColorGradingSettings = [&DetailBuilder](const TSharedRef<IPropertyHandle>& ColorGradingSettingsHandle, bool bOverrideResetToDefault)
		{
			IDetailCategoryBuilder& DetailExposureCategoryBuilder = DetailBuilder.EditCategory(TEXT("DetailView_Exposure"), LOCTEXT("DetailView_ExposureDisplayName", "Exposure"));
			DetailExposureCategoryBuilder.AddProperty(MakePropertyTransactional(ColorGradingSettingsHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_ColorGradingRenderingSettings, AutoExposureBias))));

			IDetailCategoryBuilder& DetailColorGradingCategoryBuilder = DetailBuilder.EditCategory(TEXT("DetailView_ColorGrading"), LOCTEXT("DetailView_ColorGradingDisplayName", "Color Grading"));
			DetailColorGradingCategoryBuilder.AddProperty(MakePropertyTransactional(ColorGradingSettingsHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_ColorGradingRenderingSettings, ColorCorrectionShadowsMax))));
			DetailColorGradingCategoryBuilder.AddProperty(MakePropertyTransactional(ColorGradingSettingsHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_ColorGradingRenderingSettings, ColorCorrectionHighlightsMin))));
			DetailColorGradingCategoryBuilder.AddProperty(MakePropertyTransactional(ColorGradingSettingsHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_ColorGradingRenderingSettings, ColorCorrectionHighlightsMax))));

			IDetailCategoryBuilder& DetailWhiteBalanceCategoryBuilder = DetailBuilder.EditCategory(TEXT("DetailView_WhiteBalance"), LOCTEXT("DetailView_WhiteBalanceDisplayName", "White Balance"));
			DetailWhiteBalanceCategoryBuilder.AddProperty(ColorGradingSettingsHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_ColorGradingWhiteBalanceSettings, TemperatureType)));
			
			if (bOverrideResetToDefault)
			{
				// Due to the way the editor handles resetting properties that are in structs inside an array, we must override the reset to default
				// behavior to ensure the value we reset to is a sensible value and not just zero
				DetailWhiteBalanceCategoryBuilder.AddProperty(MakePropertyTransactional(ColorGradingSettingsHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_ColorGradingWhiteBalanceSettings, WhiteTemp))))
					.OverrideResetToDefault(FResetToDefaultOverride::Create(FResetToDefaultHandler::CreateLambda([](TSharedPtr<IPropertyHandle> InPropertyHandle)
					{
						InPropertyHandle->SetValue(6500.0f);
					})));
			}
			else
			{
				DetailWhiteBalanceCategoryBuilder.AddProperty(MakePropertyTransactional(ColorGradingSettingsHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_ColorGradingWhiteBalanceSettings, WhiteTemp))));
			}
			
			DetailWhiteBalanceCategoryBuilder.AddProperty(MakePropertyTransactional(ColorGradingSettingsHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_ColorGradingWhiteBalanceSettings, WhiteTint))));

			IDetailCategoryBuilder& DetailMiscCategoryBuilder = DetailBuilder.EditCategory(TEXT("DetailView_Misc"), LOCTEXT("DetailView_MiscDisplayName", "Misc"));
			DetailMiscCategoryBuilder.AddProperty(MakePropertyTransactional(ColorGradingSettingsHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_ColorGradingMiscSettings, BlueCorrection))));
			DetailMiscCategoryBuilder.AddProperty(MakePropertyTransactional(ColorGradingSettingsHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_ColorGradingMiscSettings, ExpandGamut))));
			DetailMiscCategoryBuilder.AddProperty(ColorGradingSettingsHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_ColorGradingMiscSettings, SceneColorTint)));
		};

		UClass* ConfigDataClass = UDisplayClusterConfigurationData::StaticClass();
		const int32 GroupIndex = ColorGradingDataModel.IsValid() ? ColorGradingDataModel.Pin()->GetSelectedColorGradingGroupIndex() : INDEX_NONE;
		uint32 ArraySize = 0;

		TSharedRef<IPropertyHandle> ArrayPropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationData, StageSettings.PerViewportColorGrading), ConfigDataClass);
		check(ArrayPropertyHandle->AsArray());
		ArrayPropertyHandle->AsArray()->GetNumElements(ArraySize);

		if (GroupIndex > 0 && GroupIndex <= (int32)ArraySize)
		{
			const int32 Index = GroupIndex - 1;
			IDetailCategoryBuilder& PerNodeSettingsCategoryBuilder = DetailBuilder.EditCategory(TEXT("DetailView_PerViewport"), LOCTEXT("DetailView_PerViewportDisplayName", "Per-Viewport Settings"));
			PerNodeSettingsCategoryBuilder.AddProperty(DetailBuilder.GetProperty(GET_MEMBER_NAME_ARRAY_CHECKED(UDisplayClusterConfigurationData, StageSettings.PerViewportColorGrading[Index].bIsEntireClusterEnabled, Index), ConfigDataClass));

			const bool bOverrideResetToDefault = true;
			AddColorGradingSettings(DetailBuilder.GetProperty(GET_MEMBER_NAME_ARRAY_CHECKED(UDisplayClusterConfigurationData, StageSettings.PerViewportColorGrading[Index].ColorGradingSettings, Index), ConfigDataClass), bOverrideResetToDefault);
		}
		else
		{
			const bool bOverrideResetToDefault = false;
			AddColorGradingSettings(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationData, StageSettings.EntireClusterColorGrading.ColorGradingSettings), ConfigDataClass), bOverrideResetToDefault);
		}
	}

	void AddDetailsPanelProperties(IDetailLayoutBuilder& DetailBuilder)
	{
		auto AddProperty = [&DetailBuilder](IDetailCategoryBuilder& Category, FName PropertyName, bool bExpandChildProperties = false)
		{
			TSharedRef<IPropertyHandle> PropertyHandle = DetailBuilder.GetProperty(PropertyName, ADisplayClusterRootActor::StaticClass());

			if (bExpandChildProperties)
			{
				PropertyHandle->SetInstanceMetaData(TEXT("ShowOnlyInnerProperties"), TEXT("1"));
			}

			Category.AddProperty(PropertyHandle);
		};

		IDetailCategoryBuilder& ViewportsCategoryBuilder = DetailBuilder.EditCategory(TEXT("CustomViewportsCategory"), LOCTEXT("CustomViewportsCategoryLabel", "Viewports"));
		AddProperty(ViewportsCategoryBuilder, TEXT("OuterViewportUpscalerSettingsRef"));
		AddProperty(ViewportsCategoryBuilder, TEXT("ViewportScreenPercentageMultiplierRef"));
		AddProperty(ViewportsCategoryBuilder, TEXT("FreezeRenderOuterViewportsRef"));

		IDetailCategoryBuilder& InnerFrustumCategoryBuilder = DetailBuilder.EditCategory(TEXT("CustomICVFXCategory"), LOCTEXT("CustomICVFXCategoryLabel", "In-Camera VFX"));
		AddProperty(InnerFrustumCategoryBuilder, TEXT("GlobalInnerFrustumUpscalerSettingsRef"));
		AddProperty(InnerFrustumCategoryBuilder, TEXT("ShowInnerFrustumOverlapsRef"));
		AddProperty(InnerFrustumCategoryBuilder, GET_MEMBER_NAME_CHECKED(ADisplayClusterRootActor, InnerFrustumPriority));


		IDetailCategoryBuilder& ViewportChromakeyCategoryBuilder = DetailBuilder.EditCategory(TEXT("CustomViewportChromakeyCategory"), LOCTEXT("CustomViewportChromakeyCategoryLabel", "Chromakey"));
		AddProperty(ViewportChromakeyCategoryBuilder, TEXT("GlobalChromakeyColorRef"));

		IDetailCategoryBuilder& ViewportChromakeyMarkersCategoryBuilder = DetailBuilder.EditCategory(TEXT("CustomViewportChromakeyMarkersCategory"), LOCTEXT("CustomViewportChromakeyMarkersCategoryLabel", "Chromakey Markers"));
		AddProperty(ViewportChromakeyMarkersCategoryBuilder, TEXT("GlobalChromakeyMarkersRef"), true);
	}

private:
	TWeakPtr<FColorGradingEditorDataModel> ColorGradingDataModel;
};

/** A property customizer that culls unneeded properties from the FDisplayClusterConfigurationViewport_EntireClusterColorGrading struct to help speed up property node tree generation */
class FFastEntireClusterColorGradingCustomization : public IPropertyTypeCustomization
{
public:
	FFastEntireClusterColorGradingCustomization(const TSharedRef<FColorGradingEditorDataModel>& InColorGradingDataModel)
		: ColorGradingDataModel(InColorGradingDataModel)
	{ }

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override
	{
		HeaderRow.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		];
	}

	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override
	{
		const int32 GroupIndex = ColorGradingDataModel.IsValid() ? ColorGradingDataModel.Pin()->GetSelectedColorGradingGroupIndex() : INDEX_NONE;

		StructBuilder.AddProperty(StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_EntireClusterColorGrading, bEnableEntireClusterColorGrading)).ToSharedRef());

		if (GroupIndex < 1)
		{
			StructBuilder.AddProperty(StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_EntireClusterColorGrading, ColorGradingSettings)).ToSharedRef());
		}
	}

private:
	TWeakPtr<FColorGradingEditorDataModel> ColorGradingDataModel;
};

/** A property customizer that culls unneeded properties from the FDisplayClusterConfigurationViewport_PerViewportColorGrading struct to help speed up property node tree generation */
class FFastPerViewportColorGradingCustomization : public IPropertyTypeCustomization
{
public:
	FFastPerViewportColorGradingCustomization(const TSharedRef<FColorGradingEditorDataModel>& InColorGradingDataModel)
		: ColorGradingDataModel(InColorGradingDataModel)
	{ }

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override
	{
		HeaderRow.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		];
	}

	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override
	{
		const int32 ArrayIndex = StructPropertyHandle->GetIndexInArray();
		const int32 GroupIndex = ColorGradingDataModel.IsValid() ? ColorGradingDataModel.Pin()->GetSelectedColorGradingGroupIndex() : INDEX_NONE;

		StructBuilder.AddProperty(StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_PerViewportColorGrading, bIsEnabled)).ToSharedRef());
		StructBuilder.AddProperty(StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_PerViewportColorGrading, Name)).ToSharedRef());

		if (GroupIndex == ArrayIndex + 1)
		{
			StructBuilder.AddProperty(StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_PerViewportColorGrading, ColorGradingSettings)).ToSharedRef());
		}
	}

private:
	TWeakPtr<FColorGradingEditorDataModel> ColorGradingDataModel;
};

void FDisplayClusterColorGradingGenerator_RootActor::Initialize(const TSharedRef<class FColorGradingEditorDataModel>& ColorGradingDataModel, const TSharedRef<IPropertyRowGenerator>& PropertyRowGenerator)
{
	PropertyRowGenerator->RegisterInstancedCustomPropertyTypeLayout(FDisplayClusterConfigurationViewport_EntireClusterColorGrading::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateLambda([ColorGradingDataModel]
	{
		return MakeShared<FFastEntireClusterColorGradingCustomization>(ColorGradingDataModel);
	}));

	PropertyRowGenerator->RegisterInstancedCustomPropertyTypeLayout(FDisplayClusterConfigurationViewport_PerViewportColorGrading::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateLambda([ColorGradingDataModel]
	{
		return MakeShared<FFastPerViewportColorGradingCustomization>(ColorGradingDataModel);
	}));

	PropertyRowGenerator->RegisterInstancedCustomPropertyLayout(ADisplayClusterRootActor::StaticClass(), FOnGetDetailCustomizationInstance::CreateLambda([ColorGradingDataModel]
	{
		return MakeShared<FRootActorDetailsCustomization>(ColorGradingDataModel);
	}));
}

void FDisplayClusterColorGradingGenerator_RootActor::Destroy(const TSharedRef<class FColorGradingEditorDataModel>& ColorGradingDataModel, const TSharedRef<IPropertyRowGenerator>& PropertyRowGenerator)
{
	PropertyRowGenerator->UnregisterInstancedCustomPropertyTypeLayout(FDisplayClusterConfigurationViewport_EntireClusterColorGrading::StaticStruct()->GetFName());
	PropertyRowGenerator->UnregisterInstancedCustomPropertyTypeLayout(FDisplayClusterConfigurationViewport_PerViewportColorGrading::StaticStruct()->GetFName());
	PropertyRowGenerator->UnregisterInstancedCustomPropertyLayout(ADisplayClusterRootActor::StaticClass());
}

void FDisplayClusterColorGradingGenerator_RootActor::GenerateDataModel(IPropertyRowGenerator& PropertyRowGenerator, FColorGradingEditorDataModel& OutColorGradingDataModel)
{
	RootActors.Empty();

	const TArray<TWeakObjectPtr<UObject>>& SelectedObjects = PropertyRowGenerator.GetSelectedObjects();
	for (const TWeakObjectPtr<UObject>& SelectedObject : SelectedObjects)
	{
		if (SelectedObject.IsValid() && SelectedObject->IsA<ADisplayClusterRootActor>())
		{
			TWeakObjectPtr<ADisplayClusterRootActor> SelectedRootActor = CastChecked<ADisplayClusterRootActor>(SelectedObject.Get());
			RootActors.Add(SelectedRootActor);
		}
	}

	// Add a color grading group for the root actor's "EntireClusterColorGrading" property
	if (TSharedPtr<IPropertyHandle> EntireClusterColorGradingHandle = FindPropertyHandle(PropertyRowGenerator, CREATE_PROPERTY_PATH(UDisplayClusterConfigurationData, StageSettings.EntireClusterColorGrading)))
	{
		FColorGradingEditorDataModel::FColorGradingGroup EntireClusterGroup = CreateColorGradingGroup(EntireClusterColorGradingHandle);
		EntireClusterGroup.EditConditionPropertyHandle = EntireClusterColorGradingHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_EntireClusterColorGrading, bEnableEntireClusterColorGrading));

		EntireClusterGroup.GroupHeaderWidget = SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(0, 0, 4, 0)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(EntireClusterGroup.DisplayName)
						.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						CreateViewportComboBox(INDEX_NONE)
					];

		OutColorGradingDataModel.ColorGradingGroups.Add(EntireClusterGroup);
	}

	// Add a color grading group for each element in the root actor's "PerViewportColorGrading" array
	if (TSharedPtr<IPropertyHandle> PerViewportColorGradingHandle = FindPropertyHandle(PropertyRowGenerator, CREATE_PROPERTY_PATH(UDisplayClusterConfigurationData, StageSettings.PerViewportColorGrading)))
	{
		check(PerViewportColorGradingHandle->AsArray().IsValid());

		uint32 NumGroups;
		if (PerViewportColorGradingHandle->AsArray()->GetNumElements(NumGroups) == FPropertyAccess::Success)
		{
			for (int32 Index = 0; Index < (int32)NumGroups; ++Index)
			{
				TSharedRef<IPropertyHandle> PerViewportElementHandle = PerViewportColorGradingHandle->AsArray()->GetElement(Index);

				// Due to the way the editor handles resetting properties that are in structs inside an array, we must override the reset to default
				// behavior to ensure the value we reset to is a sensible value and not just zero
				const bool bOverrideResetToDefault = true;
				FColorGradingEditorDataModel::FColorGradingGroup PerViewportGroup = CreateColorGradingGroup(PerViewportElementHandle, bOverrideResetToDefault);
				PerViewportGroup.bCanBeDeleted = true;
				PerViewportGroup.bCanBeRenamed = true;
				PerViewportGroup.EditConditionPropertyHandle = PerViewportElementHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_PerViewportColorGrading, bIsEnabled));

				PerViewportGroup.DetailsViewCategories.Add(TEXT("DetailView_PerViewport"));

				TSharedPtr<IPropertyHandle> NamePropertyHandle = PerViewportElementHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_PerViewportColorGrading, Name));
				if (NamePropertyHandle.IsValid() && NamePropertyHandle->IsValidHandle())
				{
					NamePropertyHandle->GetValue(PerViewportGroup.DisplayName);
				}

				PerViewportGroup.GroupHeaderWidget = SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(0, 0, 4, 0)
					.VAlign(VAlign_Center)
					[
						SNew(SInlineEditableTextBlock)
						.Cursor(EMouseCursor::TextEditBeam)
						.Font_Lambda([NamePropertyHandle]
						{
							FText Name = FText::GetEmpty();

							if (NamePropertyHandle.IsValid() && NamePropertyHandle->IsValidHandle())
							{
								NamePropertyHandle->GetValue(Name);
							}

							if (Name.IsEmpty())
							{
								return FAppStyle::Get().GetFontStyle("NormalFontBoldItalic");
							}

							return FAppStyle::Get().GetFontStyle("NormalFontBold");
						})
						.OnTextCommitted_Lambda([NamePropertyHandle](const FText& InText, ETextCommit::Type TextCommitType)
						{
							if (NamePropertyHandle.IsValid() && NamePropertyHandle->IsValidHandle())
							{
								NamePropertyHandle->SetValue(InText);
								IDisplayClusterColorGrading::Get().GetColorGradingDrawerSingleton().RefreshColorGradingDrawers();
							}
						})
						.Text_Lambda([NamePropertyHandle]()
						{
							FText Name = FText::GetEmpty();

							if (NamePropertyHandle.IsValid() && NamePropertyHandle->IsValidHandle())
							{
								NamePropertyHandle->GetValue(Name);
							}

							if (Name.IsEmpty())
							{
								return LOCTEXT("UnnamedPerViewportLabel", "Unnamed");
							}

							return Name;
						})
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						CreateViewportComboBox(Index)
					];

				OutColorGradingDataModel.ColorGradingGroups.Add(PerViewportGroup);
			}
		}
	}

	OutColorGradingDataModel.bShowColorGradingGroupToolBar = true;
	OutColorGradingDataModel.ColorGradingGroupToolBarWidget = SNew(SButton)
		.ButtonStyle(FAppStyle::Get(), "SimpleButton")
		.OnClicked(this, &FDisplayClusterColorGradingGenerator_RootActor::AddColorGradingGroup)
		.ContentPadding(FMargin(1.0f, 0.0f))
		.VAlign(VAlign_Center)
		[
			SNew(SImage)
			.Image(FAppStyle::GetBrush("Icons.PlusCircle"))
			.ColorAndOpacity(FSlateColor::UseForeground())
		];

	OutColorGradingDataModel.OnColorGradingGroupDeleted().AddSP(this, &FDisplayClusterColorGradingGenerator_RootActor::DeleteColorGradingGroup);
	OutColorGradingDataModel.OnColorGradingGroupRenamed().AddSP(this, &FDisplayClusterColorGradingGenerator_RootActor::RenameColorGradingGroup);
}

FReply FDisplayClusterColorGradingGenerator_RootActor::AddColorGradingGroup()
{
	for (const TWeakObjectPtr<ADisplayClusterRootActor>& RootActor : RootActors)
	{
		if (RootActor.IsValid())
		{
			if (UDisplayClusterConfigurationData* ConfigData = RootActor->GetConfigData())
			{
				FScopedTransaction Transaction(LOCTEXT("AddViewportColorGradingGroupTransaction", "Add Viewport Group"));

				RootActor->Modify();
				ConfigData->Modify();

				FDisplayClusterConfigurationViewport_PerViewportColorGrading NewColorGradingGroup;
				NewColorGradingGroup.Name = LOCTEXT("NewViewportColorGradingGroupName", "NewViewportGroup");

				ConfigData->StageSettings.PerViewportColorGrading.Add(NewColorGradingGroup);

				IDisplayClusterColorGrading::Get().GetColorGradingDrawerSingleton().RefreshColorGradingDrawers();
			}
		}
	}

	return FReply::Handled();
}

void FDisplayClusterColorGradingGenerator_RootActor::DeleteColorGradingGroup(int32 GroupIndex)
{
	for (const TWeakObjectPtr<ADisplayClusterRootActor>& RootActor : RootActors)
	{
		if (RootActor.IsValid())
		{
			if (UDisplayClusterConfigurationData* ConfigData = RootActor->GetConfigData())
			{
				// Use GroupIndex - 1 since the entire cluster color grading is group 0
				if (ConfigData->StageSettings.PerViewportColorGrading.IsValidIndex(GroupIndex - 1))
				{
					FScopedTransaction Transaction(LOCTEXT("DeleteViewportColorGradingGroupTransaction", "Delete Viewport Group"));

					RootActor->Modify();
					ConfigData->Modify();

					ConfigData->StageSettings.PerViewportColorGrading.RemoveAt(GroupIndex - 1);

					IDisplayClusterColorGrading::Get().GetColorGradingDrawerSingleton().RefreshColorGradingDrawers();
				}
			}
		}
	}
}

void FDisplayClusterColorGradingGenerator_RootActor::RenameColorGradingGroup(int32 GroupIndex, const FText& NewName)
{
	for (const TWeakObjectPtr<ADisplayClusterRootActor>& RootActor : RootActors)
	{
		if (RootActor.IsValid())
		{
			if (UDisplayClusterConfigurationData* ConfigData = RootActor->GetConfigData())
			{
				// Use GroupIndex - 1 since the entire cluster color grading is group 0
				if (ConfigData->StageSettings.PerViewportColorGrading.IsValidIndex(GroupIndex - 1))
				{
					FScopedTransaction Transaction(LOCTEXT("RenameViewportColorGradingGroupTransaction", "Rename Viewport Group"));

					RootActor->Modify();
					ConfigData->Modify();

					ConfigData->StageSettings.PerViewportColorGrading[GroupIndex - 1].Name = NewName;

					IDisplayClusterColorGrading::Get().GetColorGradingDrawerSingleton().RefreshColorGradingDrawers();
				}
			}
		}
	}
}

TSharedRef<SWidget> FDisplayClusterColorGradingGenerator_RootActor::CreateViewportComboBox(int32 PerViewportColorGradingIndex) const
{
	return SNew(SComboButton)
		.HasDownArrow(true)
		.OnGetMenuContent(this, &FDisplayClusterColorGradingGenerator_RootActor::GetViewportComboBoxMenu, PerViewportColorGradingIndex)
		.ButtonContent()
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0, 0, 4, 0)
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.Image(FDisplayClusterColorGradingStyle::Get().GetBrush("ColorGradingDrawer.Viewports"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]

			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(this, &FDisplayClusterColorGradingGenerator_RootActor::GetViewportComboBoxText, PerViewportColorGradingIndex)
			]
		];
}

FText FDisplayClusterColorGradingGenerator_RootActor::GetViewportComboBoxText(int32 PerViewportColorGradingIndex) const
{
	// For now, only support displaying actual data when a single root actor is selected
	if (RootActors.Num() == 1)
	{
		TWeakObjectPtr<ADisplayClusterRootActor> RootActor = RootActors[0];

		if (RootActor.IsValid())
		{
			if (UDisplayClusterConfigurationData* ConfigData = RootActor->GetConfigData())
			{
				// If a valid per-viewport color grading group is passed in, determine the number of viewports associated with that group; otherwise,
				// count the total viewports in the configuration
				int32 NumViewports = 0;
				if (PerViewportColorGradingIndex > INDEX_NONE && PerViewportColorGradingIndex < ConfigData->StageSettings.PerViewportColorGrading.Num())
				{
					FDisplayClusterConfigurationViewport_PerViewportColorGrading& PerViewportColorGrading = ConfigData->StageSettings.PerViewportColorGrading[PerViewportColorGradingIndex];
					NumViewports = PerViewportColorGrading.ApplyPostProcessToObjects.Num();
				}
				else
				{
					for (const TTuple<FString, TObjectPtr<UDisplayClusterConfigurationClusterNode>>& Node : ConfigData->Cluster->Nodes)
					{
						if (Node.Value)
						{
							NumViewports += Node.Value->Viewports.Num();
						}
					}
				}

				return FText::Format(LOCTEXT("PerViewportColorGradingGroup_NumViewports", "{0} {0}|plural(one=Viewport,other=Viewports)"), FText::AsNumber(NumViewports));
			}
		}
	}
	else if (RootActors.Num() > 1)
	{
		return LOCTEXT("MultipleValuesSelectedLabel", "Multiple Values");
	}

	return FText::GetEmpty();
}

TSharedRef<SWidget> FDisplayClusterColorGradingGenerator_RootActor::GetViewportComboBoxMenu(int32 PerViewportColorGradingIndex) const
{
	FMenuBuilder MenuBuilder(false, nullptr);

	// For now, only support displaying actual data when a single root actor is selected
	if (RootActors.Num() == 1)
	{
		TWeakObjectPtr<ADisplayClusterRootActor> RootActor = RootActors[0];
		if (RootActor.IsValid())
		{
			if (UDisplayClusterConfigurationData* ConfigData = RootActor->GetConfigData())
			{
				// Extract all viewport names from the configuration data, so they can be sorted alphabetically
				TArray<FString> ViewportNames;

				for (const TTuple<FString, TObjectPtr<UDisplayClusterConfigurationClusterNode>>& Node : ConfigData->Cluster->Nodes)
				{
					if (Node.Value)
					{
						for (const TTuple<FString, TObjectPtr<UDisplayClusterConfigurationViewport>>& Viewport : Node.Value->Viewports)
						{
							ViewportNames.Add(Viewport.Key);
						}
					}
				}

				ViewportNames.Sort();

				const bool bForEntireCluster = PerViewportColorGradingIndex == INDEX_NONE;
				FDisplayClusterConfigurationViewport_PerViewportColorGrading* PerViewportColorGrading = !bForEntireCluster ? 
					&ConfigData->StageSettings.PerViewportColorGrading[PerViewportColorGradingIndex] : nullptr;

				MenuBuilder.BeginSection(TEXT("ViewportSection"), LOCTEXT("ViewportsMenuSectionLabel", "Viewports"));
				for (const FString& ViewportName : ViewportNames)
				{
					MenuBuilder.AddMenuEntry(FText::FromString(ViewportName),
						FText::GetEmpty(),
						FSlateIcon(),
						FUIAction(
							FExecuteAction::CreateLambda([ConfigData , PerViewportColorGrading, ViewportName, InRootActor=RootActor.Get()]
							{
								if (PerViewportColorGrading)
								{
									if (PerViewportColorGrading->ApplyPostProcessToObjects.Contains(ViewportName))
									{
										FScopedTransaction Transaction(LOCTEXT("RemoveViewportFromColorGradingGroupTransaction", "Remove Viewport from Group"));
										InRootActor->Modify();
										ConfigData->Modify();

										PerViewportColorGrading->ApplyPostProcessToObjects.Remove(ViewportName);
									}
									else
									{
										FScopedTransaction Transaction(LOCTEXT("AddViewportToColorGradingGroupTransaction", "Add Viewport to Group"));
										InRootActor->Modify();
										ConfigData->Modify();

										PerViewportColorGrading->ApplyPostProcessToObjects.Add(ViewportName);
									}
								}
							}),
							FCanExecuteAction::CreateLambda([PerViewportColorGrading] { return PerViewportColorGrading != nullptr; }),
							FGetActionCheckState::CreateLambda([PerViewportColorGrading, ViewportName]
							{
								// If the menu is for the EntireCluster group (PerViewportColorGrading is null), all viewport list items should be checked
								if (PerViewportColorGrading)
								{
									const bool bIsViewportInGroup = PerViewportColorGrading->ApplyPostProcessToObjects.Contains(ViewportName);
									return bIsViewportInGroup ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
								}
								else
								{
									return ECheckBoxState::Checked;
								}
							})
						),
						NAME_None,
						EUserInterfaceActionType::ToggleButton);
				}
				MenuBuilder.EndSection();
			}
		}
	}

	return MenuBuilder.MakeWidget();
}

TSharedRef<IColorGradingEditorDataModelGenerator> FDisplayClusterColorGradingGenerator_ICVFXCamera::MakeInstance()
{
	return MakeShareable(new FDisplayClusterColorGradingGenerator_ICVFXCamera());
}

/**
 * A detail customization that picks out only the necessary properties needed to display a ICVFX camera component in the color grading drawer and hides all other properties
 * Also organizes the properties into custom categories that can be easily displayed in the color grading drawer
 */
class FICVFXCameraColorGradingCustomization : public IDetailCustomization
{
public:
	FICVFXCameraColorGradingCustomization(const TSharedRef<FColorGradingEditorDataModel>& InColorGradingDataModel)
		: ColorGradingDataModel(InColorGradingDataModel)
	{ }

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override
	{
		TArray<FName> Categories;
		DetailBuilder.GetCategoryNames(Categories);

		for (const FName& Category : Categories)
		{
			DetailBuilder.HideCategory(Category);
		}

		// TransformCommon is a custom category that doesn't get returned by GetCategoryNames that also needs to be hidden
		DetailBuilder.HideCategory(TEXT("TransformCommon"));

		IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory(TEXT("ColorGradingCategory"));

		CategoryBuilder.AddProperty(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UDisplayClusterICVFXCameraComponent, CameraSettings.bEnable)));
		CategoryBuilder.AddProperty(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UDisplayClusterICVFXCameraComponent, CameraSettings.CustomFrustum.bEnable)));
		CategoryBuilder.AddProperty(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UDisplayClusterICVFXCameraComponent, CameraSettings.Chromakey.bEnable)));
		CategoryBuilder.AddProperty(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UDisplayClusterICVFXCameraComponent, CameraSettings.AllNodesColorGrading)));
		CategoryBuilder.AddProperty(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UDisplayClusterICVFXCameraComponent, CameraSettings.PerNodeColorGrading)));

		AddColorGradingDetailProperties(DetailBuilder);
		AddDetailsPanelProperties(DetailBuilder);

		DetailBuilder.SortCategories([](const TMap<FName, IDetailCategoryBuilder*>& CategoryMap)
		{
			const TMap<FName, int32> SortOrder =
			{
				{ TEXT("DetailView_AllNodes"), 0},
				{ TEXT("DetailView_PerNode"), 1},
				{ TEXT("DetailView_Exposure"), 2},
				{ TEXT("DetailView_ColorGrading"), 3},
				{ TEXT("DetailView_WhiteBalance"), 4},
				{ TEXT("DetailView_Misc"), 5}
			};

			for (const TPair<FName, int32>& SortPair : SortOrder)
			{
				if (CategoryMap.Contains(SortPair.Key))
				{
					CategoryMap[SortPair.Key]->SetSortOrder(SortPair.Value);
				}
			}
		});
	}

private:
	void AddColorGradingDetailProperties(IDetailLayoutBuilder& DetailBuilder)
	{
		auto AddColorGradingSettings = [&DetailBuilder](const TSharedRef<IPropertyHandle>& ColorGradingSettingsHandle, bool bOverrideResetToDefault)
		{
			IDetailCategoryBuilder& DetailExposureCategoryBuilder = DetailBuilder.EditCategory(TEXT("DetailView_Exposure"), LOCTEXT("DetailView_ExposureDisplayName", "Exposure"));
			DetailExposureCategoryBuilder.AddProperty(MakePropertyTransactional(ColorGradingSettingsHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_ColorGradingRenderingSettings, AutoExposureBias))));

			IDetailCategoryBuilder& DetailColorGradingCategoryBuilder = DetailBuilder.EditCategory(TEXT("DetailView_ColorGrading"), LOCTEXT("DetailView_ColorGradingDisplayName", "Color Grading"));
			DetailColorGradingCategoryBuilder.AddProperty(MakePropertyTransactional(ColorGradingSettingsHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_ColorGradingRenderingSettings, ColorCorrectionShadowsMax))));
			DetailColorGradingCategoryBuilder.AddProperty(MakePropertyTransactional(ColorGradingSettingsHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_ColorGradingRenderingSettings, ColorCorrectionHighlightsMin))));
			DetailColorGradingCategoryBuilder.AddProperty(MakePropertyTransactional(ColorGradingSettingsHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_ColorGradingRenderingSettings, ColorCorrectionHighlightsMax))));

			IDetailCategoryBuilder& DetailWhiteBalanceCategoryBuilder = DetailBuilder.EditCategory(TEXT("DetailView_WhiteBalance"), LOCTEXT("DetailView_WhiteBalanceDisplayName", "White Balance"));
			DetailWhiteBalanceCategoryBuilder.AddProperty(ColorGradingSettingsHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_ColorGradingWhiteBalanceSettings, TemperatureType)));

			if (bOverrideResetToDefault)
			{
				// Due to the way the editor handles resetting properties that are in structs inside an array, we must override the reset to default
				// behavior to ensure the value we reset to is a sensible value and not just zero
				DetailWhiteBalanceCategoryBuilder.AddProperty(MakePropertyTransactional(ColorGradingSettingsHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_ColorGradingWhiteBalanceSettings, WhiteTemp))))
					.OverrideResetToDefault(FResetToDefaultOverride::Create(FResetToDefaultHandler::CreateLambda([](TSharedPtr<IPropertyHandle> InPropertyHandle)
					{
						InPropertyHandle->SetValue(6500.0f);
					})));
			}
			else
			{
				DetailWhiteBalanceCategoryBuilder.AddProperty(MakePropertyTransactional(ColorGradingSettingsHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_ColorGradingWhiteBalanceSettings, WhiteTemp))));
			}
			
			DetailWhiteBalanceCategoryBuilder.AddProperty(MakePropertyTransactional(ColorGradingSettingsHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_ColorGradingWhiteBalanceSettings, WhiteTint))));

			IDetailCategoryBuilder& DetailMiscCategoryBuilder = DetailBuilder.EditCategory(TEXT("DetailView_Misc"), LOCTEXT("DetailView_MiscDisplayName", "Misc"));
			DetailMiscCategoryBuilder.AddProperty(MakePropertyTransactional(ColorGradingSettingsHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_ColorGradingMiscSettings, BlueCorrection))));
			DetailMiscCategoryBuilder.AddProperty(MakePropertyTransactional(ColorGradingSettingsHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_ColorGradingMiscSettings, ExpandGamut))));
			DetailMiscCategoryBuilder.AddProperty(ColorGradingSettingsHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_ColorGradingMiscSettings, SceneColorTint)));
		};

		const int32 GroupIndex = ColorGradingDataModel.IsValid() ? ColorGradingDataModel.Pin()->GetSelectedColorGradingGroupIndex() : INDEX_NONE;
		uint32 ArraySize = 0;

		TSharedPtr<IPropertyHandle> ArrayPropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UDisplayClusterICVFXCameraComponent, CameraSettings.PerNodeColorGrading));
		check(ArrayPropertyHandle->AsArray());
		ArrayPropertyHandle->AsArray()->GetNumElements(ArraySize);

		if (GroupIndex > 0 && GroupIndex <= (int32)ArraySize)
		{
			const int32 Index = GroupIndex - 1;
			IDetailCategoryBuilder& PerNodeSettingsCategoryBuilder = DetailBuilder.EditCategory(TEXT("DetailView_PerNode"), LOCTEXT("DetailView_PerNodeDisplayName", "Per-Node Settings"));
			PerNodeSettingsCategoryBuilder.AddProperty(DetailBuilder.GetProperty(GET_MEMBER_NAME_ARRAY_CHECKED(UDisplayClusterICVFXCameraComponent, CameraSettings.PerNodeColorGrading[Index].bEntireClusterColorGrading, Index)));
			PerNodeSettingsCategoryBuilder.AddProperty(DetailBuilder.GetProperty(GET_MEMBER_NAME_ARRAY_CHECKED(UDisplayClusterICVFXCameraComponent, CameraSettings.PerNodeColorGrading[Index].bAllNodesColorGrading, Index)));

			const bool bOverrideResetToDefault = true;
			AddColorGradingSettings(DetailBuilder.GetProperty(GET_MEMBER_NAME_ARRAY_CHECKED(UDisplayClusterICVFXCameraComponent, CameraSettings.PerNodeColorGrading[Index].ColorGradingSettings, Index)), bOverrideResetToDefault);
		}
		else
		{
			IDetailCategoryBuilder& AllNodeSettingsCategoryBuilder = DetailBuilder.EditCategory(TEXT("DetailView_AllNodes"), LOCTEXT("DetailView_AllNodesDisplayName", "All Nodes Settings"));
			AllNodeSettingsCategoryBuilder.AddProperty(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UDisplayClusterICVFXCameraComponent, CameraSettings.AllNodesColorGrading.bEnableEntireClusterColorGrading)));

			const bool bOverrideResetToDefault = false;
			AddColorGradingSettings(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UDisplayClusterICVFXCameraComponent, CameraSettings.AllNodesColorGrading.ColorGradingSettings)), bOverrideResetToDefault);
		}
	}

	void AddDetailsPanelProperties(IDetailLayoutBuilder& DetailBuilder)
	{
		auto AddProperty = [&DetailBuilder](IDetailCategoryBuilder& Category, FName PropertyName, bool bExpandChildProperties = false)
		{
			TSharedRef<IPropertyHandle> PropertyHandle = DetailBuilder.GetProperty(PropertyName, UDisplayClusterICVFXCameraComponent::StaticClass());

			if (bExpandChildProperties)
			{
				PropertyHandle->SetInstanceMetaData(TEXT("ShowOnlyInnerProperties"), TEXT("1"));
			}

			Category.AddProperty(PropertyHandle);
		};

		IDetailCategoryBuilder& ICVFXCategoryBuilder = DetailBuilder.EditCategory(TEXT("CustomICVFXCategory"), LOCTEXT("CustomICVFXCategoryLabel", "In-Camera VFX"));
		AddProperty(ICVFXCategoryBuilder, TEXT("UpscalerSettingsRef"));
		AddProperty(ICVFXCategoryBuilder, TEXT("BufferRatioRef"));
		AddProperty(ICVFXCategoryBuilder, TEXT("ExternalCameraActorRef"));
		AddProperty(ICVFXCategoryBuilder, TEXT("HiddenICVFXViewportsRef"));

		IDetailCategoryBuilder& SoftEdgeCategoryBuilder = DetailBuilder.EditCategory(TEXT("CustomSoftEdgeCategory"), LOCTEXT("CustomSoftEdgeCategoryLabel", "Soft Edge"));
		AddProperty(SoftEdgeCategoryBuilder, TEXT("SoftEdgeRef"), true);

		IDetailCategoryBuilder& BorderCategoryBuilder = DetailBuilder.EditCategory(TEXT("CustomBorderCategory"), LOCTEXT("CustomBorderCategoryLabel", "Border"));
		AddProperty(BorderCategoryBuilder, TEXT("BorderRef"), true);

		IDetailCategoryBuilder& OverscanCategoryBuilder = DetailBuilder.EditCategory(TEXT("CustomOverscanCategory"), LOCTEXT("CustomOverscanCategoryLabel", "Inner Frustum Overscan"));
		AddProperty(OverscanCategoryBuilder, TEXT("CustomFrustumRef"), true);

		IDetailCategoryBuilder& ChromakeyCategoryBuilder = DetailBuilder.EditCategory(TEXT("CustomChromakeyCategory"), LOCTEXT("CustomChromakeyCategoryLabel", "Chromakey"));
		AddProperty(ChromakeyCategoryBuilder, TEXT("ChromakeyTypeRef"));
		AddProperty(ChromakeyCategoryBuilder, TEXT("ChromakeySettingsSourceRef"));
		AddProperty(ChromakeyCategoryBuilder, TEXT("ChromakeyColorRef"));

		IDetailCategoryBuilder& ChromakeyMarkersCategoryBuilder = DetailBuilder.EditCategory(TEXT("CustomChromakeyMarkersCategory"), LOCTEXT("CustomChromakeyMarkersCategoryLabel", "ChromakeyMarkers"));
		AddProperty(ChromakeyMarkersCategoryBuilder, TEXT("ChromakeyMarkersRef"), true);
	}

private:
	TWeakPtr<FColorGradingEditorDataModel> ColorGradingDataModel;
};

/** A property customizer that culls unneeded properties from the FDisplayClusterConfigurationViewport_AllNodesColorGrading struct to help speed up property node tree generation */
class FFastAllNodesColorGradingCustomization : public IPropertyTypeCustomization
{
public:
	FFastAllNodesColorGradingCustomization(const TSharedRef<FColorGradingEditorDataModel>& InColorGradingDataModel)
		: ColorGradingDataModel(InColorGradingDataModel)
	{ }

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override
	{
		HeaderRow.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		];
	}

	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override
	{
		const int32 GroupIndex = ColorGradingDataModel.IsValid() ? ColorGradingDataModel.Pin()->GetSelectedColorGradingGroupIndex() : INDEX_NONE;

		StructBuilder.AddProperty(StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_AllNodesColorGrading, bEnableInnerFrustumAllNodesColorGrading)).ToSharedRef());

		if (GroupIndex < 1)
		{
			StructBuilder.AddProperty(StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_AllNodesColorGrading, ColorGradingSettings)).ToSharedRef());
		}
	}

private:
	TWeakPtr<FColorGradingEditorDataModel> ColorGradingDataModel;
};

/** A property customizer that culls unneeded properties from the FDisplayClusterConfigurationViewport_PerNodeColorGrading struct to help speed up property node tree generation */
class FFastPerNodeColorGradingCustomization : public IPropertyTypeCustomization
{
public:
	FFastPerNodeColorGradingCustomization(const TSharedRef<FColorGradingEditorDataModel>& InColorGradingDataModel)
		: ColorGradingDataModel(InColorGradingDataModel)
	{ }

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override
	{
		HeaderRow.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		];
	}

	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override
	{
		const int32 ArrayIndex = StructPropertyHandle->GetIndexInArray();
		const int32 GroupIndex = ColorGradingDataModel.IsValid() ? ColorGradingDataModel.Pin()->GetSelectedColorGradingGroupIndex() : INDEX_NONE;

		StructBuilder.AddProperty(StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_PerNodeColorGrading, bIsEnabled)).ToSharedRef());
		StructBuilder.AddProperty(StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_PerNodeColorGrading, Name)).ToSharedRef());

		if (GroupIndex == ArrayIndex + 1)
		{
			StructBuilder.AddProperty(StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_PerNodeColorGrading, ColorGradingSettings)).ToSharedRef());
		}
	}

private:
	TWeakPtr<FColorGradingEditorDataModel> ColorGradingDataModel;
};

void FDisplayClusterColorGradingGenerator_ICVFXCamera::Initialize(const TSharedRef<class FColorGradingEditorDataModel>& ColorGradingDataModel, const TSharedRef<IPropertyRowGenerator>& PropertyRowGenerator)
{
	PropertyRowGenerator->RegisterInstancedCustomPropertyTypeLayout(FDisplayClusterConfigurationViewport_AllNodesColorGrading::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateLambda([ColorGradingDataModel]
	{
		return MakeShared<FFastAllNodesColorGradingCustomization>(ColorGradingDataModel);
	}));

	PropertyRowGenerator->RegisterInstancedCustomPropertyTypeLayout(FDisplayClusterConfigurationViewport_PerNodeColorGrading::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateLambda([ColorGradingDataModel]
	{
		return MakeShared<FFastPerNodeColorGradingCustomization>(ColorGradingDataModel);
	}));

	PropertyRowGenerator->RegisterInstancedCustomPropertyLayout(UDisplayClusterICVFXCameraComponent::StaticClass(), FOnGetDetailCustomizationInstance::CreateLambda([ColorGradingDataModel]
	{
		return MakeShared<FICVFXCameraColorGradingCustomization>(ColorGradingDataModel);
	}));
}

void FDisplayClusterColorGradingGenerator_ICVFXCamera::Destroy(const TSharedRef<class FColorGradingEditorDataModel>& ColorGradingDataModel, const TSharedRef<IPropertyRowGenerator>& PropertyRowGenerator)
{
	PropertyRowGenerator->UnregisterInstancedCustomPropertyTypeLayout(FDisplayClusterConfigurationViewport_AllNodesColorGrading::StaticStruct()->GetFName());
	PropertyRowGenerator->UnregisterInstancedCustomPropertyTypeLayout(FDisplayClusterConfigurationViewport_PerNodeColorGrading::StaticStruct()->GetFName());
	PropertyRowGenerator->UnregisterInstancedCustomPropertyLayout(UDisplayClusterICVFXCameraComponent::StaticClass());
}

void FDisplayClusterColorGradingGenerator_ICVFXCamera::GenerateDataModel(IPropertyRowGenerator& PropertyRowGenerator, FColorGradingEditorDataModel& OutColorGradingDataModel)
{
	CameraComponents.Empty();

	const TArray<TWeakObjectPtr<UObject>>& SelectedObjects = PropertyRowGenerator.GetSelectedObjects();
	for (const TWeakObjectPtr<UObject>& SelectedObject : SelectedObjects)
	{
		if (SelectedObject.IsValid() && SelectedObject->IsA<UDisplayClusterICVFXCameraComponent>())
		{
			TWeakObjectPtr<UDisplayClusterICVFXCameraComponent> SelectedCameraComponent = CastChecked<UDisplayClusterICVFXCameraComponent>(SelectedObject.Get());
			CameraComponents.Add(SelectedCameraComponent);
		}
	}

	// Add a color grading group for the camera's "AllNodesColorGrading" property
	if (TSharedPtr<IPropertyHandle> AllNodesColorGradingHandle = FindPropertyHandle(PropertyRowGenerator, CREATE_PROPERTY_PATH(UDisplayClusterICVFXCameraComponent, CameraSettings.AllNodesColorGrading)))
	{
		FColorGradingEditorDataModel::FColorGradingGroup AllNodesGroup = CreateColorGradingGroup(AllNodesColorGradingHandle);
		AllNodesGroup.EditConditionPropertyHandle = AllNodesColorGradingHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_AllNodesColorGrading, bEnableInnerFrustumAllNodesColorGrading));

		AllNodesGroup.DetailsViewCategories.Add(TEXT("DetailView_AllNodes"));

		AllNodesGroup.GroupHeaderWidget = SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0, 0, 4, 0)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(AllNodesGroup.DisplayName)
				.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				CreateNodeComboBox(INDEX_NONE)
			];

		OutColorGradingDataModel.ColorGradingGroups.Add(AllNodesGroup);
	}

	// Add a color grading group for each element in the camera's "PerNodeColorGrading" array
	if (TSharedPtr<IPropertyHandle> PerNodeColorGradingHandle = FindPropertyHandle(PropertyRowGenerator, CREATE_PROPERTY_PATH(UDisplayClusterICVFXCameraComponent, CameraSettings.PerNodeColorGrading)))
	{
		check(PerNodeColorGradingHandle->AsArray().IsValid());

		uint32 NumGroups;
		if (PerNodeColorGradingHandle->AsArray()->GetNumElements(NumGroups) == FPropertyAccess::Success)
		{
			for (int32 Index = 0; Index < (int32)NumGroups; ++Index)
			{
				TSharedRef<IPropertyHandle> PerNodeElementHandle = PerNodeColorGradingHandle->AsArray()->GetElement(Index);

				// Due to the way the editor handles resetting properties that are in structs inside an array, we must override the reset to default
				// behavior to ensure the value we reset to is a sensible value and not just zero
				const bool bOverrideResetToDefault = true;
				FColorGradingEditorDataModel::FColorGradingGroup PerNodeGroup = CreateColorGradingGroup(PerNodeElementHandle, bOverrideResetToDefault);
				PerNodeGroup.bCanBeDeleted = true;
				PerNodeGroup.bCanBeRenamed = true;
				PerNodeGroup.EditConditionPropertyHandle = PerNodeElementHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_PerNodeColorGrading, bIsEnabled));

				PerNodeGroup.DetailsViewCategories.Add(TEXT("DetailView_PerNode"));

				TSharedPtr<IPropertyHandle> NamePropertyHandle = PerNodeElementHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_PerNodeColorGrading, Name));
				if (NamePropertyHandle.IsValid() && NamePropertyHandle->IsValidHandle())
				{
					NamePropertyHandle->GetValue(PerNodeGroup.DisplayName);
				}

				PerNodeGroup.GroupHeaderWidget = SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(0, 0, 4, 0)
					.VAlign(VAlign_Center)
					[
						SNew(SInlineEditableTextBlock)
						.Cursor(EMouseCursor::TextEditBeam)
						.Font_Lambda([NamePropertyHandle]
						{
							FText Name = FText::GetEmpty();

							if (NamePropertyHandle.IsValid() && NamePropertyHandle->IsValidHandle())
							{
								NamePropertyHandle->GetValue(Name);
							}

							if (Name.IsEmpty())
							{
								return FAppStyle::Get().GetFontStyle("NormalFontBoldItalic");
							}

							return FAppStyle::Get().GetFontStyle("NormalFontBold");
						})
						.OnTextCommitted_Lambda([NamePropertyHandle](const FText& InText, ETextCommit::Type TextCommitType)
						{
							if (NamePropertyHandle.IsValid() && NamePropertyHandle->IsValidHandle())
							{
								NamePropertyHandle->SetValue(InText);
								IDisplayClusterColorGrading::Get().GetColorGradingDrawerSingleton().RefreshColorGradingDrawers();
							}
						})
						.Text_Lambda([NamePropertyHandle]()
						{
							FText Name = FText::GetEmpty();

							if (NamePropertyHandle.IsValid() && NamePropertyHandle->IsValidHandle())
							{
								NamePropertyHandle->GetValue(Name);
							}

							if (Name.IsEmpty())
							{
								return LOCTEXT("UnnamedPerNodeLabel", "Unnamed");
							}

							return Name;
						})
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						CreateNodeComboBox(Index)
					];

				OutColorGradingDataModel.ColorGradingGroups.Add(PerNodeGroup);
			}
		}
	}

	OutColorGradingDataModel.bShowColorGradingGroupToolBar = true;
	OutColorGradingDataModel.ColorGradingGroupToolBarWidget = SNew(SButton)
		.ButtonStyle(FAppStyle::Get(), "SimpleButton")
		.OnClicked(this, &FDisplayClusterColorGradingGenerator_ICVFXCamera::AddColorGradingGroup)
		.ContentPadding(FMargin(1.0f, 0.0f))
		.VAlign(VAlign_Center)
		[
			SNew(SImage)
			.Image(FAppStyle::GetBrush("Icons.PlusCircle"))
			.ColorAndOpacity(FSlateColor::UseForeground())
		];

	OutColorGradingDataModel.OnColorGradingGroupDeleted().AddSP(this, &FDisplayClusterColorGradingGenerator_ICVFXCamera::DeleteColorGradingGroup);
	OutColorGradingDataModel.OnColorGradingGroupRenamed().AddSP(this, &FDisplayClusterColorGradingGenerator_ICVFXCamera::RenameColorGradingGroup);
}

FReply FDisplayClusterColorGradingGenerator_ICVFXCamera::AddColorGradingGroup()
{
	for (const TWeakObjectPtr<UDisplayClusterICVFXCameraComponent>& CameraComponent : CameraComponents)
	{
		if (CameraComponent.IsValid())
		{
			FScopedTransaction Transaction(LOCTEXT("AddNodeColorGradingGroupTransaction", "Add Node Group"));
			CameraComponent->Modify();

			FDisplayClusterConfigurationViewport_PerNodeColorGrading NewColorGradingGroup;
			NewColorGradingGroup.Name = LOCTEXT("NewNodeColorGradingGroupName", "NewNodeGroup");

			CameraComponent->CameraSettings.PerNodeColorGrading.Add(NewColorGradingGroup);

			IDisplayClusterColorGrading::Get().GetColorGradingDrawerSingleton().RefreshColorGradingDrawers();
		}
	}

	return FReply::Handled();
}

void FDisplayClusterColorGradingGenerator_ICVFXCamera::DeleteColorGradingGroup(int32 GroupIndex)
{
	for (const TWeakObjectPtr<UDisplayClusterICVFXCameraComponent>& CameraComponent : CameraComponents)
	{
		if (CameraComponent.IsValid())
		{
			// Use GroupIndex - 1 since the entire cluster color grading is group 0
			if (CameraComponent->CameraSettings.PerNodeColorGrading.IsValidIndex(GroupIndex - 1))
			{
				FScopedTransaction Transaction(LOCTEXT("DeleteNodeColorGradingGroupTransaction", "Delete Node Group"));
				CameraComponent->Modify();

				CameraComponent->CameraSettings.PerNodeColorGrading.RemoveAt(GroupIndex - 1);

				IDisplayClusterColorGrading::Get().GetColorGradingDrawerSingleton().RefreshColorGradingDrawers();
			}
		}
	}
}

void FDisplayClusterColorGradingGenerator_ICVFXCamera::RenameColorGradingGroup(int32 GroupIndex, const FText& NewName)
{
	for (const TWeakObjectPtr<UDisplayClusterICVFXCameraComponent>& CameraComponent : CameraComponents)
	{
		if (CameraComponent.IsValid())
		{
			// Use GroupIndex - 1 since the entire cluster color grading is group 0
			if (CameraComponent->CameraSettings.PerNodeColorGrading.IsValidIndex(GroupIndex - 1))
			{
				FScopedTransaction Transaction(LOCTEXT("RenameNodeColorGradingGroupTransaction", "Rename Node Group"));
				CameraComponent->Modify();

				CameraComponent->CameraSettings.PerNodeColorGrading[GroupIndex - 1].Name = NewName;

				IDisplayClusterColorGrading::Get().GetColorGradingDrawerSingleton().RefreshColorGradingDrawers();
			}
		}
	}
}

TSharedRef<SWidget> FDisplayClusterColorGradingGenerator_ICVFXCamera::CreateNodeComboBox(int32 PerNodeColorGradingIndex) const
{
	return SNew(SComboButton)
		.HasDownArrow(true)
		.OnGetMenuContent(this, &FDisplayClusterColorGradingGenerator_ICVFXCamera::GetNodeComboBoxMenu, PerNodeColorGradingIndex)
		.ButtonContent()
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0, 0, 4, 0)
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.Image(FDisplayClusterColorGradingStyle::Get().GetBrush("ColorGradingDrawer.Nodes"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]

			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(this, &FDisplayClusterColorGradingGenerator_ICVFXCamera::GetNodeComboBoxText, PerNodeColorGradingIndex)
			]
		];
}

FText FDisplayClusterColorGradingGenerator_ICVFXCamera::GetNodeComboBoxText(int32 PerNodeColorGradingIndex) const
{
	// For now, only support displaying actual data when a single camera component is selected
	if (CameraComponents.Num() == 1)
	{
		TWeakObjectPtr<UDisplayClusterICVFXCameraComponent> CameraComponent = CameraComponents[0];
		if (CameraComponent.IsValid())
		{
			int32 NumNodes = 0;
			if (PerNodeColorGradingIndex > INDEX_NONE && PerNodeColorGradingIndex < CameraComponent->CameraSettings.PerNodeColorGrading.Num())
			{
				const FDisplayClusterConfigurationViewport_PerNodeColorGrading& PerNodeColorGrading = CameraComponent->CameraSettings.PerNodeColorGrading[PerNodeColorGradingIndex];
				NumNodes = PerNodeColorGrading.ApplyPostProcessToObjects.Num();
			}
			else
			{
				if (ADisplayClusterRootActor* ParentRootActor = Cast<ADisplayClusterRootActor>(CameraComponent->GetOwner()))
				{
					if (UDisplayClusterConfigurationData* ConfigData = ParentRootActor->GetConfigData())
					{
						NumNodes = ConfigData->Cluster->Nodes.Num();
					}
				}
			}

			return FText::Format(LOCTEXT("PerNodeColorGradingGroup_NumViewports", "{0} {0}|plural(one=Node,other=Nodes)"), FText::AsNumber(NumNodes));
		}
	}
	else if (CameraComponents.Num() > 1)
	{
		return LOCTEXT("MultipleValuesSelectedLabel", "Multiple Values");
	}

	return FText::GetEmpty();
}

TSharedRef<SWidget> FDisplayClusterColorGradingGenerator_ICVFXCamera::GetNodeComboBoxMenu(int32 PerNodeColorGradingIndex) const
{

	FMenuBuilder MenuBuilder(false, nullptr);

	// For now, only support displaying actual data when a single camera component is selected
	if (CameraComponents.Num() == 1)
	{
		TWeakObjectPtr<UDisplayClusterICVFXCameraComponent> CameraComponent = CameraComponents[0];
		if (CameraComponent.IsValid())
		{
			if (ADisplayClusterRootActor* ParentRootActor = Cast<ADisplayClusterRootActor>(CameraComponent->GetOwner()))
			{
				if (UDisplayClusterConfigurationData* ConfigData = ParentRootActor->GetConfigData())
				{
					TArray<FString> NodeNames;
					ConfigData->Cluster->Nodes.GetKeys(NodeNames);

					NodeNames.Sort();

					const bool bForAllNodes = PerNodeColorGradingIndex == INDEX_NONE;
					FDisplayClusterConfigurationViewport_PerNodeColorGrading* PerNodeColorGrading = !bForAllNodes ?
						&CameraComponent->CameraSettings.PerNodeColorGrading[PerNodeColorGradingIndex] : nullptr;

					MenuBuilder.BeginSection(TEXT("NodeSection"), LOCTEXT("NodeMenuSectionLabel", "Nodes"));
					for (const FString& NodeName : NodeNames)
					{
						MenuBuilder.AddMenuEntry(FText::FromString(NodeName),
							FText::GetEmpty(),
							FSlateIcon(),
							FUIAction(
								FExecuteAction::CreateLambda([PerNodeColorGrading, NodeName, InCameraComponent=CameraComponent.Get()]()
								{
									if (PerNodeColorGrading)
									{
										if (PerNodeColorGrading->ApplyPostProcessToObjects.Contains(NodeName))
										{
											FScopedTransaction Transaction(LOCTEXT("RemoveNodeFromColorGradingGroupTransaction", "Remove Node from Group"));
											InCameraComponent->Modify();

											PerNodeColorGrading->ApplyPostProcessToObjects.Remove(NodeName);
										}
										else
										{
											FScopedTransaction Transaction(LOCTEXT("AddNodeToColorGradingGroupTransaction", "Add Node to Group"));
											InCameraComponent->Modify();

											PerNodeColorGrading->ApplyPostProcessToObjects.Add(NodeName);
										}
									}
								}),
								FCanExecuteAction::CreateLambda([PerNodeColorGrading] { return PerNodeColorGrading != nullptr; }),
								FGetActionCheckState::CreateLambda([PerNodeColorGrading, NodeName]()
								{
									// If the menu is for the AllNodes group (PerNodeColorGrading is null), all node list items should be checked
									if (PerNodeColorGrading)
									{
										const bool bIsNodeInGroup = PerNodeColorGrading->ApplyPostProcessToObjects.Contains(NodeName);
										return bIsNodeInGroup ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
									}
									else
									{
										return ECheckBoxState::Checked;
									}
								})
							),
							NAME_None,
							EUserInterfaceActionType::ToggleButton);
					}
					MenuBuilder.EndSection();
				}
			}
		}
	}

	return MenuBuilder.MakeWidget();
}

#undef LOCTEXT_NAMESPACE
