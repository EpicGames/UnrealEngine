// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ColorGradingEditorDataModel.h"

#include "Input/Reply.h"

class ADisplayClusterRootActor;
class IDetailTreeNode;
class IPropertyHandle;
class IPropertyRowGenerator;
class FDisplayClusterColorGradingDataModel;
class UDisplayClusterICVFXCameraComponent;
struct FCachedPropertyPath;

/** Base generator for any object that needs a color grading data model generated from an FColorGradingRenderingSettings struct */
class FDisplayClusterColorGradingGenerator_ColorGradingRenderingSettings : public IColorGradingEditorDataModelGenerator
{
protected:
	/** Creates a new color grading group structure from the specified group property handle, which finds and connects the appropriate property handles for the color wheels and details view */
	FColorGradingEditorDataModel::FColorGradingGroup CreateColorGradingGroup(const TSharedPtr<IPropertyHandle>& GroupPropertyHandle, bool bOverrideResetToDefault = false);

	/** Creates a color grading element structure from the specified property handle, whose child properties are expected to be colors with the ColorGradingMode metadata set */
	FColorGradingEditorDataModel::FColorGradingElement CreateColorGradingElement(const TSharedPtr<IPropertyHandle>& GroupPropertyHandle, FName ElementPropertyName, FText ElementLabel, bool bOverrideResetToDefault = false);

	/** Recursively searches the detail tree hierarchy for a property detail tree node whose name matches the specified name */
	TSharedPtr<IDetailTreeNode> FindPropertyTreeNode(const TSharedRef<IDetailTreeNode>& Node, const FCachedPropertyPath& PropertyPath);

	/** Finds a property handle in the specified property row generator whose name matches the specified name */
	TSharedPtr<IPropertyHandle> FindPropertyHandle(IPropertyRowGenerator& PropertyRowGenerator, const FCachedPropertyPath& PropertyPath);
};

/** Color grading data model generator for an nDisplay root actor */
class FDisplayClusterColorGradingGenerator_RootActor : public FDisplayClusterColorGradingGenerator_ColorGradingRenderingSettings
{
public:
	static TSharedRef<IColorGradingEditorDataModelGenerator> MakeInstance();

	//~ IDisplayClusterColorGradingDataModelGenerator interface
	virtual void Initialize(const TSharedRef<class FColorGradingEditorDataModel>& ColorGradingDataModel, const TSharedRef<IPropertyRowGenerator>& PropertyRowGenerator) override;
	virtual void Destroy(const TSharedRef<class FColorGradingEditorDataModel>& ColorGradingDataModel, const TSharedRef<IPropertyRowGenerator>& PropertyRowGenerator) override;
	virtual void GenerateDataModel(IPropertyRowGenerator& PropertyRowGenerator, FColorGradingEditorDataModel& OutColorGradingDataModel) override;
	//~ End IDisplayClusterColorGradingDataModelGenerator interface

private:
	/** OnClick delegate handler for the "+" button that is added to the color grading group toolbar */
	FReply AddColorGradingGroup();

	/** Raised when a color grading group is being deleted */
	void DeleteColorGradingGroup(int32 GroupIndex);

	/** Raised when a color grading group is being renamed */
	void RenameColorGradingGroup(int32 GroupIndex, const FText& NewName);

	/** Creates the combo box used to display which viewports are part of the specified color grading group */
	TSharedRef<SWidget> CreateViewportComboBox(int32 PerViewportColorGradingIndex) const;

	/** Gets the display text for the viewports list combo box for the PerViewportsGroup at the specified index */
	FText GetViewportComboBoxText(int32 PerViewportColorGradingIndex) const;

	/** Generates a menu widget for the viewports list combo box for the PerViewportsGroup at the specified index */
	TSharedRef<SWidget> GetViewportComboBoxMenu(int32 PerViewportColorGradingIndex) const;

private:
	/** A list of root actors that are being represented by the data model */
	TArray<TWeakObjectPtr<ADisplayClusterRootActor>> RootActors;
};

/** Color grading data model generator for an nDisplay ICVFX camera component */
class FDisplayClusterColorGradingGenerator_ICVFXCamera : public FDisplayClusterColorGradingGenerator_ColorGradingRenderingSettings
{
public:
	static TSharedRef<IColorGradingEditorDataModelGenerator> MakeInstance();

	//~ IDisplayClusterColorGradingDataModelGenerator interface
	virtual void Initialize(const TSharedRef<class FColorGradingEditorDataModel>& InColorGradingDataModel, const TSharedRef<IPropertyRowGenerator>& InPropertyRowGenerator) override;
	virtual void Destroy(const TSharedRef<class FColorGradingEditorDataModel>& InColorGradingDataModel, const TSharedRef<IPropertyRowGenerator>& InPropertyRowGenerator) override;
	virtual void GenerateDataModel(IPropertyRowGenerator& PropertyRowGenerator, FColorGradingEditorDataModel& OutColorGradingDataModel) override;
	//~ End IDisplayClusterColorGradingDataModelGenerator interface

private:
	/** OnClick delegate handler for the "+" button that is added to the color grading group toolbar */
	FReply AddColorGradingGroup();

	/** Raised when a color grading group is being deleted */
	void DeleteColorGradingGroup(int32 GroupIndex);

	/** Raised when a color grading group is being renamed */
	void RenameColorGradingGroup(int32 GroupIndex, const FText& NewName);

	/** Creates the combo box used to display which nodes are part of the specified color grading group */
	TSharedRef<SWidget> CreateNodeComboBox(int32 PerNodeColorGradingIndex) const;

	/** Gets the display text for the nodes list combo box from the specified PerNodesGroup property handle */
	FText GetNodeComboBoxText(int32 PerNodeColorGradingIndex) const;

	/** Generates a menu widget for the nodes list combo box for the PerNodesGroup at the specified index */
	TSharedRef<SWidget> GetNodeComboBoxMenu(int32 PerNodeColorGradingIndex) const;

private:
	/** A list of camera components that are being represented by the data model */
	TArray<TWeakObjectPtr<UDisplayClusterICVFXCameraComponent>> CameraComponents;
};