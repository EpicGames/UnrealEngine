// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterColorGradingCommands.h"

#define LOCTEXT_NAMESPACE "DisplayClusterColorGrading"

void FDisplayClusterColorGradingCommands::RegisterCommands()
{
	UI_COMMAND(OpenColorGradingDrawer, "Open Color Grading Drawer", "Opens the color grading drawer from the status bar", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::SpaceBar));
}

#undef LOCTEXT_NAMESPACE
