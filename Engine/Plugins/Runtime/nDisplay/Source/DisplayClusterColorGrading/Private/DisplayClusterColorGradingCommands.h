// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DisplayClusterColorGradingStyle.h"
#include "Framework/Commands/Commands.h"
#include "Styling/AppStyle.h"

/** Command list for the color grading drawer */
class FDisplayClusterColorGradingCommands
	: public TCommands<FDisplayClusterColorGradingCommands>
{
public:
	FDisplayClusterColorGradingCommands()
		: TCommands<FDisplayClusterColorGradingCommands>(TEXT("DisplayClusterColorGrading"),
			NSLOCTEXT("Contexts", "DisplayClusterColorGrading", "Display Cluster Color Grading"), NAME_None, FDisplayClusterColorGradingStyle::Get().GetStyleSetName())
	{ }

	virtual void RegisterCommands() override;

public:
	TSharedPtr<FUICommandInfo> OpenColorGradingDrawer;
};