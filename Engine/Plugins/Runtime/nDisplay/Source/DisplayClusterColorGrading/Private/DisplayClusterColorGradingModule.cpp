// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterColorGradingModule.h"

#include "DisplayClusterColorGradingCommands.h"
#include "DataModelGenerators/DisplayClusterColorGradingGenerator_RootActor.h"
#include "Drawer/DisplayClusterColorGradingDrawerSingleton.h"
#include "Drawer/SDisplayClusterColorGradingDrawer.h"

#include "DisplayClusterRootActor.h"
#include "Components/DisplayClusterICVFXCameraComponent.h"

#include "ColorCorrectRegion.h"
#include "ColorGradingMixerObjectFilterRegistry.h"
#include "Engine/PostProcessVolume.h"

#define LOCTEXT_NAMESPACE "DisplayClusterColorGrading"

void FDisplayClusterColorGradingModule::StartupModule()
{
	ColorGradingDrawerSingleton = MakeUnique<FDisplayClusterColorGradingDrawerSingleton>();

	FColorGradingEditorDataModel::RegisterColorGradingDataModelGenerator<ADisplayClusterRootActor>(
		FGetDetailsDataModelGenerator::CreateStatic(&FDisplayClusterColorGradingGenerator_RootActor::MakeInstance));

	FColorGradingEditorDataModel::RegisterColorGradingDataModelGenerator<UDisplayClusterICVFXCameraComponent>(
		FGetDetailsDataModelGenerator::CreateStatic(&FDisplayClusterColorGradingGenerator_ICVFXCamera::MakeInstance));

	FColorGradingMixerObjectFilterRegistry::RegisterActorClassToPlace(ADisplayClusterRootActor::StaticClass());

	FColorGradingMixerObjectFilterRegistry::RegisterObjectClassToFilter(ADisplayClusterRootActor::StaticClass());
	FColorGradingMixerObjectFilterRegistry::RegisterObjectClassToFilter(UDisplayClusterICVFXCameraComponent::StaticClass());

	FDisplayClusterColorGradingCommands::Register();
}

void FDisplayClusterColorGradingModule::ShutdownModule()
{
	ColorGradingDrawerSingleton.Reset();
}

IDisplayClusterColorGradingDrawerSingleton& FDisplayClusterColorGradingModule::GetColorGradingDrawerSingleton() const
{
	return *ColorGradingDrawerSingleton.Get();
}

IMPLEMENT_MODULE(FDisplayClusterColorGradingModule, DisplayClusterColorGrading);

#undef LOCTEXT_NAMESPACE
