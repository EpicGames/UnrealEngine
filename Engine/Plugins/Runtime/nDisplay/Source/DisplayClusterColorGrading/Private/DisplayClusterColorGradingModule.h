// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDisplayClusterColorGrading.h"

class FDisplayClusterColorGradingDrawerSingleton;

/**
 * Display Cluster Color Grading module
 */
class FDisplayClusterColorGradingModule : public IDisplayClusterColorGrading
{
public:
	//~ IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface interface

	//~ IDisplayClusterColorGrading interface
	virtual IDisplayClusterColorGradingDrawerSingleton& GetColorGradingDrawerSingleton() const override;
	//~ End IDisplayClusterColorGrading interface

private:
	/** The color grading drawer singleton, which manages the color grading drawer widget */
	TUniquePtr<FDisplayClusterColorGradingDrawerSingleton> ColorGradingDrawerSingleton;
};
