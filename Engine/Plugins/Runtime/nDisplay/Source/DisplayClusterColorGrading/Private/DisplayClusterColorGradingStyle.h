// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyle.h"
#include "Styling/ToolBarStyle.h"


/** Styleset for the nDisplay color grading UI elements */
class FDisplayClusterColorGradingStyle final : public FSlateStyleSet
{
public:

	FDisplayClusterColorGradingStyle()
		: FSlateStyleSet("DisplayClusterColorGradingStyle")
	{
		const FVector2D Icon16x16(16.0f, 16.0f);

		SetParentStyleName(FAppStyle::GetAppStyleSetName());

		// Set miscellaneous icons
		SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Runtime/nDisplay/Content/Icons/"));
		SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));

		Set("ColorGradingDrawer.Icon", new IMAGE_BRUSH_SVG("OperatorPanel/Colors", Icon16x16));
		Set("ColorGradingDrawer.Viewports", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Viewports", Icon16x16));
		Set("ColorGradingDrawer.Nodes", new IMAGE_BRUSH_SVG("Cluster/ClusterNode", Icon16x16));

		FSlateStyleRegistry::RegisterSlateStyle(*this);
	}

	virtual ~FDisplayClusterColorGradingStyle()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	}

	static FDisplayClusterColorGradingStyle& Get()
	{
		static FDisplayClusterColorGradingStyle Inst;
		return Inst;
	}
};