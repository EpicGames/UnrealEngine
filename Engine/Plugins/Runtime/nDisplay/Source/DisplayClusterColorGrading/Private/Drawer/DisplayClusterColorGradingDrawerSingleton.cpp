// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterColorGradingDrawerSingleton.h"

#include "DisplayClusterColorGradingCommands.h"
#include "DisplayClusterColorGradingStyle.h"

#include "DisplayClusterRootActor.h"

#include "IDisplayClusterOperator.h"
#include "IDisplayClusterOperatorViewModel.h"
#include "DisplayClusterOperatorStatusBarExtender.h"
#include "Drawer/SDisplayClusterColorGradingDrawer.h"

#include "Editor.h"
#include "IColorGradingEditor.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/LayoutExtender.h"
#include "Framework/Docking/TabManager.h"
#include "IPropertyRowGenerator.h"
#include "LevelEditor.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "SColorGradingPanel.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "DisplayClusterColorGrading"

const FName FDisplayClusterColorGradingDrawerSingleton::ColorGradingDrawerId = TEXT("ColorGradingDrawer");
const FName FDisplayClusterColorGradingDrawerSingleton::ColorGradingDrawerTab = TEXT("ColorGradingDrawerTab");

FDisplayClusterColorGradingDrawerSingleton::FDisplayClusterColorGradingDrawerSingleton()
{
	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(ColorGradingDrawerTab, FOnSpawnTab::CreateRaw(this, &FDisplayClusterColorGradingDrawerSingleton::SpawnColorGradingDrawerTab))
		.SetIcon(FSlateIcon(FDisplayClusterColorGradingStyle::Get().GetStyleSetName(), "ColorGradingDrawer.Icon"))
		.SetDisplayName(LOCTEXT("ColorGradingDarwerTab_DisplayName", "Color Grading"))
		.SetTooltipText(LOCTEXT("ColorGradingDarwerTab_Tooltip", "Color grading controls for in-camera VFX"))
		.SetMenuType(ETabSpawnerMenuType::Hidden);

	IDisplayClusterOperator::Get().OnRegisterLayoutExtensions().AddRaw(this, &FDisplayClusterColorGradingDrawerSingleton::ExtendOperatorTabLayout);
	IDisplayClusterOperator::Get().OnRegisterStatusBarExtensions().AddRaw(this, &FDisplayClusterColorGradingDrawerSingleton::ExtendOperatorStatusBar);
	IDisplayClusterOperator::Get().OnAppendOperatorPanelCommands().AddRaw(this, &FDisplayClusterColorGradingDrawerSingleton::AppendOperatorPanelCommands);

	IDisplayClusterOperator::Get().GetOperatorViewModel()->OnActiveRootActorChanged().AddRaw(this, &FDisplayClusterColorGradingDrawerSingleton::OnActiveRootActorChanged);
	IDisplayClusterOperator::Get().GetOperatorViewModel()->OnDetailObjectsChanged().AddRaw(this, &FDisplayClusterColorGradingDrawerSingleton::OnDetailObjectsChanged);
}

FDisplayClusterColorGradingDrawerSingleton::~FDisplayClusterColorGradingDrawerSingleton()
{
	IDisplayClusterOperator::Get().OnRegisterLayoutExtensions().RemoveAll(this);
	IDisplayClusterOperator::Get().OnRegisterStatusBarExtensions().RemoveAll(this);
	IDisplayClusterOperator::Get().OnAppendOperatorPanelCommands().RemoveAll(this);
	IDisplayClusterOperator::Get().GetOperatorViewModel()->OnActiveRootActorChanged().RemoveAll(this);
	IDisplayClusterOperator::Get().GetOperatorViewModel()->OnDetailObjectsChanged().RemoveAll(this);

	if (FSlateApplication::IsInitialized())
	{
		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(ColorGradingDrawerTab);
	}
}

void FDisplayClusterColorGradingDrawerSingleton::DockColorGradingDrawer()
{
	if (TSharedPtr<FTabManager> OperatorPanelTabManager = IDisplayClusterOperator::Get().GetOperatorViewModel()->GetTabManager())
	{
		if (TSharedPtr<SDockTab> ExistingTab = OperatorPanelTabManager->FindExistingLiveTab(ColorGradingDrawerTab))
		{
			IDisplayClusterOperator::Get().ForceDismissDrawers();
			ExistingTab->ActivateInParent(ETabActivationCause::SetDirectly);
		}
		else
		{
			OperatorPanelTabManager->TryInvokeTab(ColorGradingDrawerTab);
		}
	}
}

void FDisplayClusterColorGradingDrawerSingleton::RefreshColorGradingDrawers()
{
	if (ColorGradingDrawer.IsValid())
	{
		ColorGradingDrawer.Pin()->Refresh();
	}

	if (TSharedPtr<FTabManager> OperatorPanelTabManager = IDisplayClusterOperator::Get().GetOperatorViewModel()->GetTabManager())
	{
		if (TSharedPtr<SDockTab> ExistingTab = OperatorPanelTabManager->FindExistingLiveTab(ColorGradingDrawerTab))
		{
			TSharedRef<SDisplayClusterColorGradingDrawer> DockedDrawer = StaticCastSharedRef<SDisplayClusterColorGradingDrawer>(ExistingTab->GetContent());
			DockedDrawer->Refresh();
		}
	}

	// Update any top-level color grading drawers open in the level editor
	FLevelEditorModule& LevelEditor = FModuleManager::Get().LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	if (TSharedPtr<SDockTab> ExistingTab = LevelEditor.GetLevelEditorTabManager()->FindExistingLiveTab(IColorGradingEditor::Get().GetColorGradingTabSpawnerId()))
	{
		TSharedRef<SColorGradingPanel> ColorGradingPanel = StaticCastSharedRef<SColorGradingPanel>(ExistingTab->GetContent());
		ColorGradingPanel->Refresh();
	}
}

TSharedRef<SWidget> FDisplayClusterColorGradingDrawerSingleton::CreateDrawerContent(bool bIsInDrawer, bool bCopyStateFromActiveDrawer)
{
	if (bIsInDrawer)
	{
		TSharedPtr<SDisplayClusterColorGradingDrawer> Drawer = ColorGradingDrawer.IsValid() ? ColorGradingDrawer.Pin() : nullptr;

		if (!Drawer.IsValid())
		{
			Drawer = SNew(SDisplayClusterColorGradingDrawer, true);
			ColorGradingDrawer = Drawer;
		}

		if (PreviousColorGradingPanelState.IsSet())
		{
			ColorGradingDrawer.Pin()->SetColorGradingPanelState(PreviousColorGradingPanelState.GetValue());
			PreviousColorGradingPanelState.Reset();
		}
		else
		{
			ColorGradingDrawer.Pin()->SelectOperatorRootActor();
		}

		return Drawer.ToSharedRef();
	}
	else
	{
		TSharedRef<SDisplayClusterColorGradingDrawer> NewDrawer = SNew(SDisplayClusterColorGradingDrawer, false);

		if (bCopyStateFromActiveDrawer && ColorGradingDrawer.IsValid())
		{
			NewDrawer->SetColorGradingPanelState(ColorGradingDrawer.Pin()->GetColorGradingPanelState());
		}

		return NewDrawer;
	}
}

TSharedRef<SDockTab> FDisplayClusterColorGradingDrawerSingleton::SpawnColorGradingDrawerTab(const FSpawnTabArgs& SpawnTabArgs)
{
	const TSharedRef<SDockTab> MajorTab = SNew(SDockTab)
		.TabRole(ETabRole::NomadTab);

	MajorTab->SetContent(CreateDrawerContent(false, true));

	return MajorTab;
}

void FDisplayClusterColorGradingDrawerSingleton::ExtendOperatorTabLayout(FLayoutExtender& InExtender)
{
	FTabManager::FTab NewTab(FTabId(ColorGradingDrawerTab, ETabIdFlags::SaveLayout), ETabState::ClosedTab);
	InExtender.ExtendStack(IDisplayClusterOperator::Get().GetAuxilliaryOperatorExtensionId(), ELayoutExtensionPosition::After, NewTab);
}

void FDisplayClusterColorGradingDrawerSingleton::ExtendOperatorStatusBar(FDisplayClusterOperatorStatusBarExtender& StatusBarExtender)
{
	FWidgetDrawerConfig ColorGradingDrawerConfig(ColorGradingDrawerId);

	ColorGradingDrawerConfig.GetDrawerContentDelegate.BindRaw(this, &FDisplayClusterColorGradingDrawerSingleton::CreateDrawerContent, true, false);
	ColorGradingDrawerConfig.OnDrawerDismissedDelegate.BindRaw(this, &FDisplayClusterColorGradingDrawerSingleton::SaveDrawerState);
	ColorGradingDrawerConfig.ButtonText = LOCTEXT("ColorGradingDrawer_ButtonText", "Color Grading");
	ColorGradingDrawerConfig.Icon = FDisplayClusterColorGradingStyle::Get().GetBrush("ColorGradingDrawer.Icon");

	StatusBarExtender.AddWidgetDrawer(ColorGradingDrawerConfig);
}

void FDisplayClusterColorGradingDrawerSingleton::AppendOperatorPanelCommands(TSharedRef<FUICommandList> OperatorPanelCommandList)
{
	OperatorPanelCommandList->MapAction(
		FDisplayClusterColorGradingCommands::Get().OpenColorGradingDrawer,
		FExecuteAction::CreateRaw(this, &FDisplayClusterColorGradingDrawerSingleton::OpenColorGradingDrawer)
	);
}

void FDisplayClusterColorGradingDrawerSingleton::OpenColorGradingDrawer()
{
	IDisplayClusterOperator::Get().ToggleDrawer(ColorGradingDrawerId);
}

void FDisplayClusterColorGradingDrawerSingleton::SaveDrawerState(const TSharedPtr<SWidget>& DrawerContent)
{
	if (ColorGradingDrawer.IsValid())
	{
		PreviousColorGradingPanelState = ColorGradingDrawer.Pin()->GetColorGradingPanelState();
	}
	else
	{
		PreviousColorGradingPanelState.Reset();
	}
}

void FDisplayClusterColorGradingDrawerSingleton::OnActiveRootActorChanged(ADisplayClusterRootActor* NewRootActor)
{
	// Clear the previous drawer state when the active root actor is changed, since it is most likely invalid
	PreviousColorGradingPanelState.Reset();
}

void FDisplayClusterColorGradingDrawerSingleton::OnDetailObjectsChanged(const TArray<UObject*>& NewObjects)
{
	// Clear the previous drawer state when the selected detail objects have changed
	PreviousColorGradingPanelState.Reset();
}

#undef LOCTEXT_NAMESPACE