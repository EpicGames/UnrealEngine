// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDisplayClusterColorGradingDrawerSingleton.h"
#include "Drawer/SDisplayClusterColorGradingDrawer.h"

class FDisplayClusterOperatorStatusBarExtender;
class FLayoutExtender;
class FSpawnTabArgs;
class FUICommandList;
class SWidget;
class SDockTab;

/** A singleton used to manage and store persistent state for the color grading drawer */
class FDisplayClusterColorGradingDrawerSingleton : public IDisplayClusterColorGradingDrawerSingleton
{
public:
	/** The ID of the color grading drawer when registered with the nDisplay operator panel's status bar */
	static const FName ColorGradingDrawerId;

	/** The ID of the color grading drawer when docked in the nDisplay operator panel's tab manager */
	static const FName ColorGradingDrawerTab;

public:
	FDisplayClusterColorGradingDrawerSingleton();
	virtual ~FDisplayClusterColorGradingDrawerSingleton();

	/** Docks the color grading drawer in the nDisplay operator window */
	virtual void DockColorGradingDrawer() override;

	/** Refreshes the UI of any open color grading drawers */
	virtual void RefreshColorGradingDrawers() override;

private:
	/** Creates a new drawer widget to place in a drawer or in a tab */
	TSharedRef<SWidget> CreateDrawerContent(bool bIsInDrawer, bool bCopyStateFromActiveDrawer);

	/** Tab spawn delegate handler used to create the drawer tab when the drawer is docked in the operator panel */
	TSharedRef<SDockTab> SpawnColorGradingDrawerTab(const FSpawnTabArgs& SpawnTabArgs);

	/** Tab extender delegate callback that registers the tab spawner with the operator panel's tab manager */
	void ExtendOperatorTabLayout(FLayoutExtender& InExtender);

	/** Status bar extender delegate callback that registers the drawer spawner with the operator panel's status bar */
	void ExtendOperatorStatusBar(FDisplayClusterOperatorStatusBarExtender& StatusBarExtender);

	/** Delegate callback that appends the operator panel command list to add color grading drawer commands */
	void AppendOperatorPanelCommands(TSharedRef<FUICommandList> OperatorPanelCommandList);

	/** Opens the color grading drawer */
	void OpenColorGradingDrawer();

	/** Delegate callback when the drawer is closed to save the drawer state */
	void SaveDrawerState(const TSharedPtr<SWidget>& DrawerContent);

	/** Delegate callback that is raised when the active root actor of the operator panel has changed */
	void OnActiveRootActorChanged(ADisplayClusterRootActor* NewRootActor);

	/** Delegate callback that is raised when the list of objects displayed in the operator panel's details panel has changed */
	void OnDetailObjectsChanged(const TArray<UObject*>& NewObjects);

private:
	/** A weak pointer to the active color grading drawer that is open */
	TWeakPtr<SDisplayClusterColorGradingDrawer> ColorGradingDrawer;

	/** The color grading panel state when the last instance of the color grading drawer was dismissed */
	TOptional<FColorGradingPanelState> PreviousColorGradingPanelState;
};