// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDisplayClusterColorGradingDrawer.h"

#include "ColorGradingEditorDataModel.h"
#include "DisplayClusterColorGradingStyle.h"
#include "Engine/Blueprint.h"
#include "IDisplayClusterColorGrading.h"
#include "IDisplayClusterColorGradingDrawerSingleton.h"

#include "IDisplayClusterOperator.h"
#include "IDisplayClusterOperatorViewModel.h"

#include "DisplayClusterRootActor.h"
#include "Components/DisplayClusterICVFXCameraComponent.h"
#include "DisplayClusterConfigurationTypes.h"
#include "SelectionInterface/DisplayClusterObjectMixerSelectionInterface.h"

#include "ColorCorrectRegion.h"
#include "ColorCorrectWindow.h"
#include "Engine/PostProcessVolume.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Modules/ModuleManager.h"
#include "ScopedTransaction.h"
#include "SColorGradingPanel.h"
#include "Styling/AppStyle.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

#define LOCTEXT_NAMESPACE "DisplayClusterColorGrading"

SDisplayClusterColorGradingDrawer::~SDisplayClusterColorGradingDrawer()
{
	OperatorViewModel = IDisplayClusterOperator::Get().GetOperatorViewModel();
	OperatorViewModel->OnActiveRootActorChanged().RemoveAll(this);
	OperatorViewModel->OnDetailObjectsChanged().RemoveAll(this);
}

void SDisplayClusterColorGradingDrawer::Construct(const FArguments& InArgs, bool bInIsInDrawer)
{
	bIsInDrawer = bInIsInDrawer;
	OperatorViewModel = IDisplayClusterOperator::Get().GetOperatorViewModel();
	OperatorViewModel->OnActiveRootActorChanged().AddSP(this, &SDisplayClusterColorGradingDrawer::OnActiveRootActorChanged);
	
	TSharedRef<SVerticalBox> ColorGradingObjectListBox = SNew(SVerticalBox);

	TSharedPtr<SWidget> SharedThis = AsShared();

	ChildSlot
	[
		SAssignNew(MainPanel, SColorGradingPanel)
		.SelectionInterface(MakeShared<FDisplayClusterObjectMixerSelectionInterface>())
		.OverrideWorld(this, &SDisplayClusterColorGradingDrawer::GetOperatorWorld)
		.IsInDrawer(bInIsInDrawer)
		.OnDocked_Lambda([]() {
			IDisplayClusterColorGrading::Get().GetColorGradingDrawerSingleton().DockColorGradingDrawer();
		})
		.ActorFilter([SharedThis, this](const AActor* Actor) -> bool {
			if (SharedThis.IsValid() && OperatorViewModel.IsValid() && Actor && Actor->IsA<ADisplayClusterRootActor>())
			{
				return Actor == OperatorViewModel->GetRootActor();
			}

			return true;
		})
	];
}

void SDisplayClusterColorGradingDrawer::Refresh()
{
	FColorGradingPanelState PanelState = GetColorGradingPanelState();

	if (MainPanel.IsValid())
	{
		MainPanel->Refresh();
	}

	SetColorGradingPanelState(PanelState);
}

void SDisplayClusterColorGradingDrawer::PostUndo(bool bSuccess)
{
	if (bSuccess)
	{
		Refresh();
	}
}

void SDisplayClusterColorGradingDrawer::PostRedo(bool bSuccess)
{
	if (bSuccess)
	{
		Refresh();
	}
}

FColorGradingPanelState SDisplayClusterColorGradingDrawer::GetColorGradingPanelState() const
{
	FColorGradingPanelState PanelState;

	if (MainPanel.IsValid())
	{
		MainPanel->GetPanelState(PanelState);
	}

	return PanelState;
}

void SDisplayClusterColorGradingDrawer::SetColorGradingPanelState(const FColorGradingPanelState& InPanelState)
{
	if (MainPanel.IsValid())
	{
		MainPanel->SetPanelState(InPanelState);
	}
}

void SDisplayClusterColorGradingDrawer::SelectOperatorRootActor()
{
	if (!MainPanel.IsValid())
	{
		return;
	}

	if (OperatorViewModel->HasRootActor())
	{
		MainPanel->SetSelectedObjects({ OperatorViewModel->GetRootActor() });
	}
	else
	{
		MainPanel->SetSelectedObjects({});
	}
}

FText SDisplayClusterColorGradingDrawer::GetCurrentLevelName() const
{
	if (OperatorViewModel->HasRootActor())
	{
		if (UWorld* World = OperatorViewModel->GetRootActor()->GetWorld())
		{
			return FText::FromString(World->GetMapName());
		}
	}

	return FText::GetEmpty();
}

FText SDisplayClusterColorGradingDrawer::GetCurrentRootActorName() const
{
	if (OperatorViewModel->HasRootActor())
	{
		return FText::FromString(OperatorViewModel->GetRootActor()->GetActorLabel());
	}

	return FText::GetEmpty();
}

void SDisplayClusterColorGradingDrawer::OnActiveRootActorChanged(ADisplayClusterRootActor* NewRootActor)
{
	Refresh();
	SelectOperatorRootActor();
}

UWorld* SDisplayClusterColorGradingDrawer::GetOperatorWorld() const
{
	if (OperatorViewModel.IsValid())
	{
		if (OperatorViewModel->HasRootActor())
		{
			return OperatorViewModel->GetRootActor()->GetWorld();
		}
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE