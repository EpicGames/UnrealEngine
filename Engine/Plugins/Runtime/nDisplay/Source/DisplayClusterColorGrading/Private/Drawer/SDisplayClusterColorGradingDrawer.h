// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ColorGradingPanelState.h"
#include "CoreMinimal.h"
#include "EditorUndoClient.h"
#include "Styling/SlateTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class ADisplayClusterRootActor;
class FColorGradingEditorDataModel;
class FDisplayClusterOperatorStatusBarExtender;
class IDisplayClusterOperatorViewModel;
class IPropertyRowGenerator;
class SColorGradingPanel;
class SDisplayClusterColorGradingDetailsPanel;
class SHorizontalBox;
class SInlineEditableTextBlock;

/** Color grading drawer widget, which displays a list of color gradable items, and the color wheel panel */
class SDisplayClusterColorGradingDrawer : public SCompoundWidget, public FEditorUndoClient
{
public:
	~SDisplayClusterColorGradingDrawer();

	SLATE_BEGIN_ARGS(SDisplayClusterColorGradingDrawer)
	{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, bool bInIsInDrawer);

	/** Refreshes the drawer's UI to match the current state of the level and active root actor */
	void Refresh();

	//~ FEditorUndoClient interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
	//~ End FEditorUndoClient interface

	/** Gets the state of the color grading panel UI */
	FColorGradingPanelState GetColorGradingPanelState() const;

	/** Sets the state of the color grading panel UI */
	void SetColorGradingPanelState(const FColorGradingPanelState& InPanelState);

	/** Sets the color grading panel's selected object to the operator panel's selected root actor */
	void SelectOperatorRootActor();

private:
	/** Gets the name of the current level the active root actor is in */
	FText GetCurrentLevelName() const;

	/** Gets the name of the active root actor */
	FText GetCurrentRootActorName() const;

	/** Raised when the user has changed the active root actor selected in the nDisplay operator panel */
	void OnActiveRootActorChanged(ADisplayClusterRootActor* NewRootActor);

	/** Get the world of the current root actor in the ICVFX panel */
	UWorld* GetOperatorWorld() const;

private:
	/** The operator panel's view model */
	TSharedPtr<IDisplayClusterOperatorViewModel> OperatorViewModel;

	/** The panel containing the color grading wheels or object details */
	TSharedPtr<SColorGradingPanel> MainPanel;

	/** Gets whether this widget is in a drawer or docked in a tab */
	bool bIsInDrawer = false;

	/** Indicates that the drawer should refresh itself on the next tick */
	bool bRefreshOnNextTick = false;

	/** Indicates if the color grading data model should update when a list item selection has changed */
	bool bUpdateDataModelOnSelectionChanged = true;

	/** Delegate handle for the OnActiveRootActorChanged delegate */
	FDelegateHandle ActiveRootActorChangedHandle;
};
