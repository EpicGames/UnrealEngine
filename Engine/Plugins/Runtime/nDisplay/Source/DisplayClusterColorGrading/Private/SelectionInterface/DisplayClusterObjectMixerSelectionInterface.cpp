// Copyright Epic Games, Inc. All Rights Reserved.

#include "SelectionInterface/DisplayClusterObjectMixerSelectionInterface.h"

#include "Algo/Transform.h"
#include "Editor/UnrealEdEngine.h"
#include "Engine/Selection.h"
#include "IDisplayClusterOperator.h"
#include "IDisplayClusterOperatorViewModel.h"

FDisplayClusterObjectMixerSelectionInterface::FDisplayClusterObjectMixerSelectionInterface()
{
	IDisplayClusterOperator::Get().GetOperatorViewModel()->OnOutlinerSelectionChanged().AddRaw(
		this, &FDisplayClusterObjectMixerSelectionInterface::OnOutlinerSelectionChanged
	);
}

FDisplayClusterObjectMixerSelectionInterface::~FDisplayClusterObjectMixerSelectionInterface()
{
	IDisplayClusterOperator::Get().GetOperatorViewModel()->OnOutlinerSelectionChanged().RemoveAll(this);
}

void FDisplayClusterObjectMixerSelectionInterface::SelectActors(const TArray<AActor*>& InSelectedActors, bool bShouldSelect, bool bSelectEvenIfHidden)
{
	TSharedRef<IDisplayClusterOperatorViewModel> Operator = IDisplayClusterOperator::Get().GetOperatorViewModel();

	Operator->SelectActors(InSelectedActors, bShouldSelect);
}

void FDisplayClusterObjectMixerSelectionInterface::SelectComponents(const TArray<UActorComponent*>& InSelectedComponents, bool bShouldSelect, bool bSelectEvenIfHidden)
{
	TSharedRef<IDisplayClusterOperatorViewModel> Operator = IDisplayClusterOperator::Get().GetOperatorViewModel();
	TArray<UObject*> DetailObjects;

	if (bShouldSelect)
	{
		DetailObjects.Append(InSelectedComponents);
	}
	else
	{
		Algo::Transform(Operator->GetDetailObjects(), DetailObjects, [](const TWeakObjectPtr<UObject>& Object) { return Object.Get(); });

		for (UActorComponent* Component : InSelectedComponents)
		{
			DetailObjects.RemoveSingle(Component);
		}
	}

	Operator->ShowDetailsForObjects(DetailObjects);
}

TArray<AActor*> FDisplayClusterObjectMixerSelectionInterface::GetSelectedActors() const
{
	return SelectedActors;
}

TArray<UActorComponent*> FDisplayClusterObjectMixerSelectionInterface::GetSelectedComponents() const
{
	TArray<UActorComponent*> OutComponents;

	Algo::TransformIf(
		IDisplayClusterOperator::Get().GetOperatorViewModel()->GetDetailObjects(),
		OutComponents,
		[](const TWeakObjectPtr<UObject>& Object) -> bool
		{
			return Object.IsValid() && Object->IsA<UActorComponent>();
		},
		[](const TWeakObjectPtr<UObject>& Object) -> UActorComponent*
		{
			return Cast<UActorComponent>(Object.Get());
		}
	);

	return MoveTemp(OutComponents);
}

void FDisplayClusterObjectMixerSelectionInterface::OnOutlinerSelectionChanged(const TArray<AActor*>& Actors)
{
	SelectedActors = Actors;
	SelectionChanged.Broadcast();
}
