// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SelectionInterface/IObjectMixerSelectionInterface.h"

/**
 * Provides an interface for the ObjectMixer to access DisplayCluster operator panel selections
 */
class FDisplayClusterObjectMixerSelectionInterface : public IObjectMixerSelectionInterface
{
public:

	FDisplayClusterObjectMixerSelectionInterface();
	virtual ~FDisplayClusterObjectMixerSelectionInterface() override;

	//~ Begin IObjectMixerSelectionInterface interface
	virtual void SelectActors(const TArray<AActor*>& InSelectedActors, bool bShouldSelect, bool bSelectEvenIfHidden) override;
	virtual void SelectComponents(const TArray<UActorComponent*>& InSelectedComponents, bool bShouldSelect, bool bSelectEvenIfHidden) override;
	virtual void SelectSubObjects(const TArray<UObject*>& InSelectedSubObjects, bool bShouldSelect, bool bSelectEvenIfHidden) override { }
	virtual TArray<AActor*> GetSelectedActors() const override;
	virtual TArray<UActorComponent*> GetSelectedComponents() const override;
	virtual TArray<UObject*> GetSelectedSubObjects() const override { return { }; }
	virtual FOnSelectionChanged& OnSelectionChanged() override { return SelectionChanged; }
	//~ End IObjectMixerSelectionInterface interface

private:
	void OnOutlinerSelectionChanged(const TArray<AActor*>& Actors);

	FOnSelectionChanged SelectionChanged;

	/** Last reported list of actors selected in the operator panel's outliner */
	TArray<AActor*> SelectedActors;
};