// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class IPropertyRowGenerator;

/** A singleton used to manage and store persistent state for the color grading drawer */
class IDisplayClusterColorGradingDrawerSingleton
{
public:
	/** Docks the color grading drawer in the nDisplay operator window */
	virtual void DockColorGradingDrawer() = 0;

	/** Refreshes the UI of any open color grading drawers */
	virtual void RefreshColorGradingDrawers() = 0;
};