// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class DisplayClusterConfiguration : ModuleRules
{
	public DisplayClusterConfiguration(ReadOnlyTargetRules ROTargetRules) : base(ROTargetRules)
	{
		PublicIncludePathModuleNames.AddRange(
			new string[] {
				"DisplayCluster",
				"DisplayClusterProjection",
				"DisplayClusterShaders",
			});

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"ActorLayerUtilities",
				"CinematicCamera",
				"MediaAssets",
				"MediaIOCore",
				"OpenColorIO",
			});

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"Json",
				"JsonUtilities",
			});
	}
}
