// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfigurationLog.h"


// Module-wide log categories
DEFINE_LOG_CATEGORY(LogDisplayClusterConfiguration);
