// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"


// Module-wide log categories
DECLARE_LOG_CATEGORY_EXTERN(LogDisplayClusterConfiguration, Log, All);
