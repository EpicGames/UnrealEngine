// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfigurationMgr.h"

#include "Formats/IDisplayClusterConfigurationDataParser.h"
#include "Formats/JSON426/DisplayClusterConfigurationJsonParser_426.h"
#include "Formats/JSON427/DisplayClusterConfigurationJsonParser_427.h"
#include "Formats/JSON500/DisplayClusterConfigurationJsonParser_500.h"

#include "VersionChecker/DisplayClusterConfigurationVersionChecker.h"

#include "DisplayClusterConfigurationLog.h"
#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterConfigurationUtils.h"
#include "DisplayClusterConfigurationVersion.h"

#include "Misc/DisplayClusterHelpers.h"
#include "Misc/Paths.h"


// Alias for current config scheme
namespace ACTUAL_CONFIG_SCHEME = JSON500;


FDisplayClusterConfigurationMgr& FDisplayClusterConfigurationMgr::Get()
{
	static FDisplayClusterConfigurationMgr Instance;
	return Instance;
}

EDisplayClusterConfigurationVersion FDisplayClusterConfigurationMgr::GetConfigVersion(const FString& FilePath)
{
	FString ConfigFile = FilePath.TrimStartAndEnd();

	// Process relative paths
	if (FPaths::IsRelative(ConfigFile))
	{
		ConfigFile = DisplayClusterHelpers::filesystem::GetFullPathForConfig(ConfigFile);
	}

	// Detect version
	TUniquePtr<FDisplayClusterConfigurationVersionChecker> VersionChecker = MakeUnique<FDisplayClusterConfigurationVersionChecker>();
	return VersionChecker->GetConfigVersion(ConfigFile);
}

UDisplayClusterConfigurationData* FDisplayClusterConfigurationMgr::LoadConfig(const FString& FilePath, UObject* Owner)
{
	FString ConfigFile = FilePath.TrimStartAndEnd();

	if (FPaths::IsRelative(ConfigFile))
	{
		ConfigFile = DisplayClusterHelpers::filesystem::GetFullPathForConfig(ConfigFile);
	}

	if (!FPaths::FileExists(ConfigFile))
	{
		UE_LOGF(LogDisplayClusterConfiguration, Error, "File not found: %ls", *ConfigFile);
		return nullptr;
	}

	// Detect config version and instantiate a proper config parser
	TUniquePtr<IDisplayClusterConfigurationDataParser> Parser;
	TUniquePtr<FDisplayClusterConfigurationVersionChecker> VersionChecker = MakeUnique<FDisplayClusterConfigurationVersionChecker>();
	switch (VersionChecker->GetConfigVersion(ConfigFile))
	{
	case EDisplayClusterConfigurationVersion::Version_426:
		Parser = MakeUnique<JSON426::FDisplayClusterConfigurationJsonParser>();
		break;

	case EDisplayClusterConfigurationVersion::Version_427:
		Parser = MakeUnique<JSON427::FDisplayClusterConfigurationJsonParser>();
		break;

	case EDisplayClusterConfigurationVersion::Version_500:
		Parser = MakeUnique<JSON500::FDisplayClusterConfigurationJsonParser>();
		break;

	case EDisplayClusterConfigurationVersion::Unknown:
	default:
		UE_LOGF(LogDisplayClusterConfiguration, Error, "No parser implemented for file: %ls", *ConfigFile);
		return nullptr;
	}

	// Load configuration data
	UDisplayClusterConfigurationData* Config = Parser->LoadData(ConfigFile);

	// Apply any upgrades if necessary
	if (IsValid(Config))
	{
		FDisplayClusterConfigurationUtils::UpdateToLatest(Config);
	}

	return Config;
}

bool FDisplayClusterConfigurationMgr::SaveConfig(const UDisplayClusterConfigurationData* Config, const FString& FilePath)
{
	// Save to json with current config scheme
	TUniquePtr<IDisplayClusterConfigurationDataParser> Parser = MakeUnique<ACTUAL_CONFIG_SCHEME::FDisplayClusterConfigurationJsonParser>();
	check(Parser);
	return Parser->SaveData(Config, FilePath);
}

bool FDisplayClusterConfigurationMgr::ConfigAsString(const UDisplayClusterConfigurationData* Config, FString& OutString)
{
	// Stringify to json with current config scheme
	TUniquePtr<IDisplayClusterConfigurationDataParser> Parser = MakeUnique<ACTUAL_CONFIG_SCHEME::FDisplayClusterConfigurationJsonParser>();
	check(Parser);
	return Parser->AsString(Config, OutString);
}

UDisplayClusterConfigurationData* FDisplayClusterConfigurationMgr::CreateDefaultStandaloneConfigData()
{
	// Not supported yet
	return nullptr;
}
