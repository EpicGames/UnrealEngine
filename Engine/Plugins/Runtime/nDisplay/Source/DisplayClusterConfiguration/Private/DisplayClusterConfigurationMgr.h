// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DisplayClusterConfigurationVersion.h"

class UDisplayClusterConfigurationData;


/**
 * Display Cluster configuration manager
 */
class FDisplayClusterConfigurationMgr
{
protected:
	FDisplayClusterConfigurationMgr() = default;
	~FDisplayClusterConfigurationMgr() = default;

public:
	// Singletone getter
	static FDisplayClusterConfigurationMgr& Get();

public:
	EDisplayClusterConfigurationVersion GetConfigVersion(const FString& FilePath);
	UDisplayClusterConfigurationData* LoadConfig(const FString& FilePath, UObject* Owner = nullptr);
	bool SaveConfig(const UDisplayClusterConfigurationData* Config, const FString& FilePath);
	bool ConfigAsString(const UDisplayClusterConfigurationData* Config, FString& OutString);
	UDisplayClusterConfigurationData* CreateDefaultStandaloneConfigData();
};
