// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfigurationModule.h"
#include "DisplayClusterConfigurationMgr.h"

#include "Misc/CommandLine.h"
#include "Modules/ModuleManager.h"

#include "Engine/Engine.h"
#include "Engine/Console.h"

#include "DisplayClusterConfigurationLog.h"
#include "DisplayClusterConfigurationStrings.h"

#include "Misc/DisplayClusterHelpers.h"


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterConfiguration
//////////////////////////////////////////////////////////////////////////////////////////////

void FDisplayClusterConfigurationModule::SetIsSnapshotTransacting(bool bIsSnapshotState)
{
	bIsSnapshot = bIsSnapshotState;
}

bool FDisplayClusterConfigurationModule::IsTransactingSnapshot() const
{
	return bIsSnapshot;
}

EDisplayClusterConfigurationVersion FDisplayClusterConfigurationModule::GetConfigVersion(const FString& FilePath)
{
	return FDisplayClusterConfigurationMgr::Get().GetConfigVersion(FilePath);
}

UDisplayClusterConfigurationData* FDisplayClusterConfigurationModule::LoadConfig(const FString& FilePath, UObject* Owner)
{
	return FDisplayClusterConfigurationMgr::Get().LoadConfig(FilePath, Owner);
}

bool FDisplayClusterConfigurationModule::SaveConfig(const UDisplayClusterConfigurationData* Config, const FString& FilePath)
{
	return FDisplayClusterConfigurationMgr::Get().SaveConfig(Config, FilePath);
}

bool FDisplayClusterConfigurationModule::ConfigAsString(const UDisplayClusterConfigurationData* Config, FString& OutString) const
{
	return FDisplayClusterConfigurationMgr::Get().ConfigAsString(Config, OutString);
}

IMPLEMENT_MODULE(FDisplayClusterConfigurationModule, DisplayClusterConfiguration);
