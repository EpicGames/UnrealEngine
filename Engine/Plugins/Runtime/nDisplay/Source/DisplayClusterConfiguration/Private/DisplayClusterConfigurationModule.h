// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDisplayClusterConfiguration.h"

class UDisplayClusterConfigurationData;


/**
 * Display Cluster configuration module
 */
class FDisplayClusterConfigurationModule :
	public IDisplayClusterConfiguration
{
public:
	//////////////////////////////////////////////////////////////////////////////////////////////
	// IDisplayClusterConfiguration
	//////////////////////////////////////////////////////////////////////////////////////////////

	virtual void SetIsSnapshotTransacting(bool bIsSnapshot) override;
	virtual bool IsTransactingSnapshot() const override;
	virtual EDisplayClusterConfigurationVersion GetConfigVersion(const FString& FilePath) override;
	virtual UDisplayClusterConfigurationData* LoadConfig(const FString& FilePath, UObject* Owner = nullptr) override;
	virtual bool SaveConfig(const UDisplayClusterConfigurationData* Config, const FString& FilePath) override;
	virtual bool ConfigAsString(const UDisplayClusterConfigurationData* Config, FString& OutString) const override;

private:
	/** Snapshot state */
	bool bIsSnapshot = false;
};
