// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfigurationTypes.h"

#include "DisplayClusterConfigurationLog.h"
#include "DisplayClusterConfigurationStrings.h"
#include "DisplayClusterConfigurationTypes_Base.h"
#include "DisplayClusterConfigurationTypes_Viewport.h"
#include "DisplayClusterConfigurationTypes_ICVFX.h"
#include "DisplayClusterConfigurationTypes_PostRender.h"
#include "DisplayClusterConfigurationUtils.h"
#include "DisplayClusterProjectionStrings.h"

#include "Engine/StaticMesh.h"
#include "UObject/Package.h"

#if WITH_EDITOR
#include "Kismet2/CompilerResultsLog.h"
#endif

#define SAVE_MAP_TO_ARRAY(Map, DestArray) \
	for (const auto& KeyVal : Map) \
	{ \
		auto Component = KeyVal.Value; \
		if(Component) \
		{ \
			DestArray.AddUnique(Component); \
		} \
	} \

#define SAVE_MAP(Map) \
	SAVE_MAP_TO_ARRAY(Map, OutObjects); \


FIntRect FDisplayClusterReplaceTextureCropRectangle::ToRect() const
{
	return FIntRect(FIntPoint(Origin.X, Origin.Y), FIntPoint(Origin.X + Size.W, Origin.Y + Size.H));
}

FIntRect FDisplayClusterConfigurationRectangle::ToRect() const
{
	return FIntRect(FIntPoint(X, Y), FIntPoint(X + W, Y + H));
}

UDisplayClusterConfigurationData::UDisplayClusterConfigurationData()
{
}

UDisplayClusterConfigurationClusterNode* UDisplayClusterConfigurationData::GetPrimaryNode() const
{
	if (Cluster)
	{
		return Cluster->GetNode(Cluster->PrimaryNode.Id);
	}

	return nullptr;
}

UDisplayClusterConfigurationClusterNode* UDisplayClusterConfigurationData::GetNode(const FString& NodeId) const
{
	if (Cluster)
	{
		return Cluster->GetNode(NodeId);
	}

	return nullptr;
}

UDisplayClusterConfigurationViewport* UDisplayClusterConfigurationData::GetViewport(const FString& NodeId, const FString& ViewportId) const
{
	UDisplayClusterConfigurationClusterNode* Node = Cluster->GetNode(NodeId);
	if (Node)
	{
		TObjectPtr<UDisplayClusterConfigurationViewport>* Viewport = Node->Viewports.Find(ViewportId);
		if (Viewport)
		{
			return *Viewport;
		}
	}
	return nullptr;
}

void UDisplayClusterConfigurationData::ForEachViewport(const TFunction<void(const TObjectPtr<UDisplayClusterConfigurationViewport>&)>& Function) const
{
	if (Cluster)
	{
		for (const TTuple<FString, TObjectPtr<UDisplayClusterConfigurationClusterNode>>& ClusterNode : Cluster->Nodes)
		{
			if (ClusterNode.Value)
			{
				for (const TTuple<FString, TObjectPtr<UDisplayClusterConfigurationViewport>>& Viewport : ClusterNode.Value->Viewports)
				{
					if (Viewport.Value)
					{
						Function(Viewport.Value);
					}
				}
			}
		}
	}
}

bool UDisplayClusterConfigurationData::AssignPostprocess(const FString& NodeId, const FString& PostprocessId, const FString& Type, TMap<FString, FString> Parameters, int32 Order)
{
	if (Cluster && Cluster->Nodes.Contains(NodeId))
	{
		if (!PostprocessId.IsEmpty())
		{
			FDisplayClusterConfigurationPostprocess PostprocessData;
			PostprocessData.Type = Type;
			PostprocessData.Parameters.Append(Parameters);
			PostprocessData.Order = Order;

			Cluster->Nodes[NodeId]->Postprocess.Emplace(PostprocessId, PostprocessData);

			return true;
		}
	}

	return false;
}

bool UDisplayClusterConfigurationData::RemovePostprocess(const FString& NodeId, const FString& PostprocessId)
{
	if (Cluster && Cluster->Nodes.Contains(NodeId))
	{
		if (Cluster->Nodes[NodeId]->Postprocess.Contains(PostprocessId))
		{
			Cluster->Nodes[NodeId]->Postprocess.Remove(PostprocessId);
			return true;
		}
	}

	return false;
}

bool UDisplayClusterConfigurationData::GetPostprocess(const FString& NodeId, const FString& PostprocessId, FDisplayClusterConfigurationPostprocess& OutPostprocess) const
{
	if (!Cluster)
	{
		return false;
	}

	const UDisplayClusterConfigurationClusterNode* Node = Cluster->GetNode(NodeId);

	if (Node)
	{
		const FDisplayClusterConfigurationPostprocess* PostprocessOperation = Node->Postprocess.Find(PostprocessId);
		if (PostprocessOperation)
		{
			OutPostprocess = *PostprocessOperation;
			return true;
		}
	}

	return false;
}

bool UDisplayClusterConfigurationData::GetProjectionPolicy(const FString& NodeId, const FString& ViewportId, FDisplayClusterConfigurationProjection& OutProjection) const
{
	const UDisplayClusterConfigurationViewport* Viewport = GetViewport(NodeId, ViewportId);
	if (Viewport)
	{
		OutProjection = Viewport->ProjectionPolicy;
		return true;
	}

	return false;
}

#if WITH_EDITORONLY_DATA

const TSet<FString> UDisplayClusterConfigurationData::RenderSyncPolicies =
{
	TEXT("Ethernet"),
	TEXT("Nvidia"),
	TEXT("None")
};

const TSet<FString> UDisplayClusterConfigurationData::InputSyncPolicies =
{
	TEXT("ReplicatePrimary"),
	TEXT("None")
};

const TSet<FString> UDisplayClusterConfigurationData::ProjectionPolicies =
{
	TEXT("Camera"),
	TEXT("Mesh"),
	TEXT("MPCDI"),
	TEXT("EasyBlend"),
	TEXT("DomeProjection"),
	TEXT("VIOSO"),
	TEXT("Manual"),
	TEXT("Reference"),
};

#endif

FDisplayClusterConfigurationProjection::FDisplayClusterConfigurationProjection()
{
	Type = TEXT("Mesh");
}

bool FDisplayClusterConfigurationProjection::ShouldDisplayICVFXCategory() const
{
	return Type.Equals(DisplayClusterProjectionStrings::projection::Mesh, ESearchCase::IgnoreCase)
		|| Type.Equals(DisplayClusterProjectionStrings::projection::MPCDI, ESearchCase::IgnoreCase)
		|| Type.Equals(DisplayClusterProjectionStrings::projection::Reference, ESearchCase::IgnoreCase);
}

constexpr float UDisplayClusterConfigurationViewport::ViewportMinimumSize = 1.0f;
constexpr float UDisplayClusterConfigurationViewport::ViewportMaximumSize = 15360.0f;

UDisplayClusterConfigurationViewport::UDisplayClusterConfigurationViewport()
{
#if WITH_EDITORONLY_DATA
	bIsVisible = true;
	bIsUnlocked = true;
#endif
}

void UDisplayClusterConfigurationViewport::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (!IsValid(RenderSettings.Media.MediaInput.MediaSource) && IsValid(RenderSettings.Media.MediaSource))
		{
			RenderSettings.Media.MediaInput.MediaSource = RenderSettings.Media.MediaSource;
			RenderSettings.Media.MediaSource = nullptr;
		}

		if (RenderSettings.Media.MediaOutputs.IsEmpty() && IsValid(RenderSettings.Media.MediaOutput))
		{
			RenderSettings.Media.MediaOutputs.Add({ RenderSettings.Media.MediaOutput , RenderSettings.Media.OutputSyncPolicy });
			RenderSettings.Media.MediaOutput = nullptr;
			RenderSettings.Media.OutputSyncPolicy = nullptr;
		}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITOR

	// 5.8+ backward compatibility path for 'Simple' projection policy
	FDisplayClusterConfigurationUtils::RedirectSimpleToMeshPolicy(this);
}

#if WITH_EDITOR
void UDisplayClusterConfigurationViewport::PreEditChange(FEditPropertyChain& PropertyAboutToChange)
{
	if (TDoubleLinkedList<FProperty*>::TDoubleLinkedListNode* ActiveMemberNode = PropertyAboutToChange.
		GetActiveMemberNode())
	{
		if (const FStructProperty* StructProperty = CastField<FStructProperty>(ActiveMemberNode->GetValue()))
		{
			if (StructProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationViewport, Region))
			{
				if (DisablePreviewTexture())
				{
					bIsManagingPreviewTexture = true;
				}
			}
		}
	}
	
	Super::PreEditChange(PropertyAboutToChange);
}

void UDisplayClusterConfigurationViewport::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
	{
		if (TDoubleLinkedList<FProperty*>::TDoubleLinkedListNode* ActiveMemberNode = PropertyChangedEvent.
			PropertyChain.GetActiveMemberNode())
		{
			if (const FStructProperty* StructProperty = CastField<FStructProperty>(ActiveMemberNode->GetValue()))
			{
				if (StructProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationViewport, Region))
				{
					if (bIsManagingPreviewTexture)
					{
						EnablePreviewTexture();
					}
				}
			}
		}
	}
	
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	OnPostEditChangeChainProperty.Broadcast(PropertyChangedEvent);
}

void UDisplayClusterConfigurationViewport::EnablePreviewTexture()
{
	bAllowPreviewTexture = true;
	bIsManagingPreviewTexture = false;
}

bool UDisplayClusterConfigurationViewport::DisablePreviewTexture()
{
	if (bAllowPreviewTexture)
	{
		bAllowPreviewTexture = false;
		return true;
	}
	
	return false;
}

void UDisplayClusterConfigurationViewport::OnPreCompile(FCompilerResultsLog& MessageLog)
{
	Super::OnPreCompile(MessageLog);

	if (!bAllowPreviewTexture)
	{
		// Verify correct rendering value is applied. This branch shouldn't be hit as long as the Region
		// struct always has a final PostEditChangeProperty called without a change type of Interactive, but may
		// be hit when compiling after this viewport was deleted and then the deletion undone.
		EnablePreviewTexture();
	}
}

void UDisplayClusterConfigurationClusterNode::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	OnPostEditChangeChainProperty.Broadcast(PropertyChangedEvent);
}

void UDisplayClusterConfigurationHostDisplayData::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	OnPostEditChangeChainProperty.Broadcast(PropertyChangedEvent);
}
#endif

void UDisplayClusterConfigurationClusterNode::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (!IsValid(Media.MediaInput.MediaSource) && IsValid(Media.MediaSource))
		{
			Media.MediaInput.MediaSource = Media.MediaSource;
			Media.MediaSource = nullptr;
		}

		if (Media.MediaOutputs.IsEmpty() && IsValid(Media.MediaOutput))
		{
			Media.MediaOutputs.Add({ Media.MediaOutput, Media.OutputSyncPolicy });
			Media.MediaOutput = nullptr;
			Media.OutputSyncPolicy = nullptr;
		}

		if (MediaSettings.MediaOutputs.IsEmpty() && Media.MediaOutputs.Num() > 0)
		{
			MediaSettings.bEnable = Media.bEnable;
			MediaSettings.MediaOutputs = Media.MediaOutputs;
			Media.bEnable = false;
			Media.MediaOutputs.Empty();
		}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITOR
}

void UDisplayClusterConfigurationClusterNode::Serialize(FArchive& Ar)
{
	// When loading, overwrite Media settings with defaults unless this is the archetype
	if (Ar.IsLoading() && !IsTemplate())
	{
		const FDisplayClusterConfigurationMediaNodeBackbuffer MediaSettingsOriginal = MediaSettings;
		Super::Serialize(Ar);
		MediaSettings = MediaSettingsOriginal;
	}
	else
	{
		Super::Serialize(Ar);
	}
}

UDisplayClusterConfigurationClusterNode::UDisplayClusterConfigurationClusterNode()
	: bIsSoundEnabled(false)
#if WITH_EDITORONLY_DATA
	, bIsVisible(true)
	, bIsUnlocked(true)
#endif
{
}

UDisplayClusterConfigurationHostDisplayData::UDisplayClusterConfigurationHostDisplayData()
	: bIsVisible(true)
	, bIsUnlocked(true)
{
	SetFlags(RF_Public);
}

void UDisplayClusterConfigurationClusterNode::GetObjectsToExport(TArray<UObject*>& OutObjects)
{
	Super::GetObjectsToExport(OutObjects);
	SAVE_MAP(Viewports);
}

void UDisplayClusterConfigurationCluster::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (Failover.FailoverPolicy == EDisplayClusterConfigurationFailoverPolicy::DropSecondaryNodesOnly)
	{
		Failover.bEnabled = true;
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITOR
}

void UDisplayClusterConfigurationCluster::GetObjectsToExport(TArray<UObject*>& OutObjects)
{
	Super::GetObjectsToExport(OutObjects);
	SAVE_MAP(Nodes);
}

void UDisplayClusterConfigurationData_Base::Serialize(FArchive& Ar)
{
#if WITH_EDITOR
	if (Ar.IsSaving() && !Ar.IsTransacting())
	{
		/*
		 * We need to set everything to public so it can be saved & referenced properly with the object.
		 * The object ownership doesn't seem to be correct at this stage either and is sometimes
		 * owned by the main data object, but since subobjects embed subobjects the correct parent
		 * should be set prior to save.
		 */

		SetFlags(RF_Public);
		ClearFlags(RF_Transient);
		
		ExportedObjects.Reset();
		GetObjectsToExport(MutableView(ExportedObjects));
		for (UObject* Object : ExportedObjects)
		{
			if (!ensure(Object != nullptr))
			{
				UE_LOGF(LogDisplayClusterConfiguration, Warning, "Null object passed to GetObjectsToExport");
				continue;
			}
			if (Object->GetOuter() != this)
			{
				Object->Rename(nullptr, this, REN_DoNotDirty | REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
			}
			Object->SetFlags(RF_Public);
			Object->ClearFlags(RF_Transient);
		}
	}
#endif
	
	Super::Serialize(Ar);
}

void UDisplayClusterConfigurationScene::GetObjectsToExport(TArray<UObject*>& OutObjects)
{
	Super::GetObjectsToExport(OutObjects);
	
	SAVE_MAP(Xforms);
	SAVE_MAP(Screens);
	SAVE_MAP(Cameras);
}

FDisplayClusterConfigurationPrimaryNodePorts::FDisplayClusterConfigurationPrimaryNodePorts()
	: ClusterSync        (41001)
	, ClusterEventsJson  (41003)
	, ClusterEventsBinary(41004)
{
}

FDisplayClusterConfigurationClusterSync::FDisplayClusterConfigurationClusterSync()
{
	using namespace DisplayClusterConfigurationStrings::config;
	RenderSyncPolicy.Type = cluster::render_sync::Ethernet;
	InputSyncPolicy.Type  = cluster::input_sync::InputSyncPolicyReplicatePrimary;
}

FDisplayClusterConfigurationNetworkSettings::FDisplayClusterConfigurationNetworkSettings()
	: ConnectRetriesAmount     (300)
	, ConnectRetryDelay        (1000) // 1 second between attempts
	, GameStartBarrierTimeout  (1000 * 60 * 60) // 1 hour
	, FrameStartBarrierTimeout (1000 * 60 * 30) // 30 minutes
	, FrameEndBarrierTimeout   (1000 * 60 * 30) // 30 minutes
	, RenderSyncBarrierTimeout (1000 * 60 * 30) // 30 minutes
{
}

FDisplayClusterConfigurationFailoverSettings::FDisplayClusterConfigurationFailoverSettings()
	: bEnabled(false)
{
}

void UDisplayClusterConfigurationViewport::GetReferencedMeshNames(TArray<FString>& OutMeshNames) const
{
	// Collect all mesh references from projection policies
	for (const TPair<FString, FString>& It : ProjectionPolicy.Parameters)
	{
		if (ProjectionPolicy.Type.Compare(DisplayClusterProjectionStrings::projection::Mesh, ESearchCase::IgnoreCase) == 0)
		{
			if (It.Key.Compare(DisplayClusterProjectionStrings::cfg::mesh::Component, ESearchCase::IgnoreCase) == 0)
			{
				OutMeshNames.Add(It.Value);
			}
		}
	}
}

void UDisplayClusterConfigurationClusterNode::GetViewportIds(TArray<FString>& OutViewportIds) const
{
	Viewports.GenerateKeyArray(OutViewportIds);
}

UDisplayClusterConfigurationViewport* UDisplayClusterConfigurationClusterNode::GetViewport(const FString& ViewportId) const
{
	TObjectPtr<UDisplayClusterConfigurationViewport> const* Viewport = Viewports.Find(ViewportId);
	return Viewport ? *Viewport : nullptr;
}

void UDisplayClusterConfigurationClusterNode::GetReferencedMeshNames(TArray<FString>& OutMeshNames) const
{
	for (const TPair<FString, TObjectPtr<UDisplayClusterConfigurationViewport>>& It : Viewports)
	{
		if (It.Value == nullptr)
		{
			// Could be null temporarily during a reimport when this is called.
			continue;
		}
		It.Value->GetReferencedMeshNames(OutMeshNames);
	}
}

void UDisplayClusterConfigurationCluster::GetNodeIds(TArray<FString>& OutNodeIds) const
{
	Nodes.GenerateKeyArray(OutNodeIds);
}

UDisplayClusterConfigurationClusterNode* UDisplayClusterConfigurationCluster::GetNode(const FString& NodeId) const
{
	TObjectPtr<UDisplayClusterConfigurationClusterNode> const* Node = Nodes.Find(NodeId);
	return Node ? *Node : nullptr;
}

void UDisplayClusterConfigurationCluster::GetReferencedMeshNames(TArray<FString>& OutMeshNames) const
{
	for (const TPair<FString, TObjectPtr<UDisplayClusterConfigurationClusterNode>>& It : Nodes)
	{
		if (It.Value == nullptr)
		{
			// Could be null temporarily during a reimport when this is called.
			continue;
		}
		It.Value->GetReferencedMeshNames(OutMeshNames);
	}
}

void UDisplayClusterConfigurationData::GetReferencedMeshNames(TArray<FString>& OutMeshNames) const
{
	if (Cluster != nullptr)
	{
		Cluster->GetReferencedMeshNames(OutMeshNames);
	}
}

uint32 UDisplayClusterConfigurationData::GetNumberOfClusterNodes() const
{
	return Cluster ? static_cast<uint32>(Cluster->Nodes.Num()) : 0;
}

FString UDisplayClusterConfigurationData::GetPrimaryNodeAddress() const
{
	if (Cluster)
	{
		const FString& PrimaryNodeId = Cluster->PrimaryNode.Id;
		if (UDisplayClusterConfigurationClusterNode* const PrimaryNode = Cluster->GetNode(PrimaryNodeId))
		{
			return PrimaryNode->Host;
		}
	}

	return FString();
}

UDisplayClusterConfigurationData* UDisplayClusterConfigurationData::CreateNewConfigData(UObject* Owner, EObjectFlags ObjectFlags)
{
	UDisplayClusterConfigurationData* NewConfigData = NewObject<UDisplayClusterConfigurationData>(Owner ? Owner : GetTransientPackage(), NAME_None,
		ObjectFlags | RF_ArchetypeObject | RF_Public | RF_Transactional);

	NewConfigData->Scene = NewObject<UDisplayClusterConfigurationScene>(NewConfigData, NAME_None,
		RF_ArchetypeObject | RF_Public | RF_Transactional);

	NewConfigData->Cluster = NewObject<UDisplayClusterConfigurationCluster>(NewConfigData, NAME_None,
		RF_ArchetypeObject | RF_Public | RF_Transactional);

	return NewConfigData;
}

#if WITH_EDITOR
void UDisplayClusterConfigurationData::PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent)
{
	auto ChainContainsProperty = [&PropertyChangedEvent](const FName& InPropertyName)
	{
		TDoubleLinkedList<FProperty*>::TDoubleLinkedListNode* CurrentNode = PropertyChangedEvent.PropertyChain.GetActiveNode();
		while (CurrentNode)
		{
			if (FProperty* Property = CurrentNode->GetValue())
			{
				if (Property->GetFName() == InPropertyName)
				{
					return  true;
				}
			}

			CurrentNode = CurrentNode->GetPrevNode();
		}

		return false;
	};
	
	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	if (PropertyName == GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationUpscalerSettings, MethodName)
		&& ChainContainsProperty(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_StageSettings, OuterViewportUpscalerSettings)))
	{
		Modify();
		
		// All viewport upscaling methods should match the global outer viewport upscaling method
		for (TTuple<FString, TObjectPtr<UDisplayClusterConfigurationClusterNode>>& Node : Cluster->Nodes)
		{
			for (TTuple<FString, TObjectPtr<UDisplayClusterConfigurationViewport>>& Viewport : Node.Value->Viewports)
			{
				Viewport.Value->Modify();
				
				Viewport.Value->RenderSettings.UpscalerSettings.MethodName = StageSettings.OuterViewportUpscalerSettings.MethodName;
				Viewport.Value->RenderSettings.UpscalerSettings.EditingData = StageSettings.OuterViewportUpscalerSettings.EditingData;
				Viewport.Value->RenderSettings.UpscalerSettings.ParameterOverrideGuids.Empty();
			}
		}
	}
	else if (ChainContainsProperty(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationUpscalerSettings, EditingData))
		&& ChainContainsProperty(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_StageSettings, OuterViewportUpscalerSettings)))
	{
		Modify();
		
		// A property in the upscaler settings property bag has changed. Propagate the global property bag values to the viewports, considering viewport overrides
		for (TTuple<FString, TObjectPtr<UDisplayClusterConfigurationClusterNode>>& Node : Cluster->Nodes)
		{
			for (TTuple<FString, TObjectPtr<UDisplayClusterConfigurationViewport>>& Viewport : Node.Value->Viewports)
			{
				Viewport.Value->Modify();
				
				Viewport.Value->RenderSettings.UpscalerSettings.EditingData.MigrateToNewBagInstanceWithOverrides(
					StageSettings.OuterViewportUpscalerSettings.EditingData,
					Viewport.Value->RenderSettings.UpscalerSettings.ParameterOverrideGuids);
			}
		}
	}
	
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif
