// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfigurationTypes_ICVFX.h"
#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterConfigurationUtils.h"
#include "IDisplayCluster.h"
#include "Camera/CameraTypes.h"
#include "CineCameraComponent.h"
#include "CineCameraActor.h"



///////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterConfigurationICVFX_ChromakeyMarkers
///////////////////////////////////////////////////////////////////////////////////////
FDisplayClusterConfigurationICVFX_ChromakeyMarkers::FDisplayClusterConfigurationICVFX_ChromakeyMarkers()
{
	// Default marker texture
	const FString TexturePath = TEXT("/nDisplay/Textures/T_TrackingMarker_A.T_TrackingMarker_A");
	MarkerTileRGBA = Cast<UTexture2D>(FSoftObjectPath(TexturePath).TryLoad());
}

///////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterConfigurationICVFX_CameraRenderSettings
///////////////////////////////////////////////////////////////////////////////////////
FDisplayClusterConfigurationICVFX_CameraRenderSettings::FDisplayClusterConfigurationICVFX_CameraRenderSettings()
{
	// Setup incamera defaults:
	GenerateMips.bAutoGenerateMips = true;
}

void FDisplayClusterConfigurationICVFX_CameraRenderSettings::SetupViewInfo(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings, FMinimalViewInfo& InOutViewInfo) const
{
	// CameraSettings can disable postprocess from this camera
	if (!bUseCameraComponentPostprocess)
	{
		InOutViewInfo.PostProcessSettings = FPostProcessSettings();
		InOutViewInfo.PostProcessBlendWeight = 0.0f;
	}
}

///////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterConfigurationICVFX_LightcardCustomOCIO
///////////////////////////////////////////////////////////////////////////////////////
const FOpenColorIOColorConversionSettings* FDisplayClusterConfigurationICVFX_LightcardCustomOCIO::FindOCIOConfiguration(const FString& InViewportId) const
{
	// Note: Lightcard OCIO is enabled from the drop-down menu, so we ignore AllViewportsOCIOConfiguration.bIsEnabled (the property isn't exposed)
	
	// Per viewport OCIO:
	for (const FDisplayClusterConfigurationOCIOProfile& OCIOProfileIt : PerViewportOCIOProfiles)
	{
		if (OCIOProfileIt.IsEnabledForObject(InViewportId))
		{
			return &OCIOProfileIt.ColorConfiguration;
		}
	}

	return &AllViewportsOCIOConfiguration.ColorConfiguration;
}

///////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterConfigurationICVFX_ViewportOCIO
///////////////////////////////////////////////////////////////////////////////////////
const FOpenColorIOColorConversionSettings* FDisplayClusterConfigurationICVFX_ViewportOCIO::FindOCIOConfiguration(const FString& InViewportId) const
{
	if (AllViewportsOCIOConfiguration.bIsEnabled)
	{
		// Per viewport OCIO:
		for (const FDisplayClusterConfigurationOCIOProfile& OCIOProfileIt : PerViewportOCIOProfiles)
		{
			if (OCIOProfileIt.IsEnabledForObject(InViewportId))
			{
				return &OCIOProfileIt.ColorConfiguration;
			}
		}

		return &AllViewportsOCIOConfiguration.ColorConfiguration;
	}

	return nullptr;
}

///////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterConfigurationICVFX_CameraOCIO
///////////////////////////////////////////////////////////////////////////////////////
const FOpenColorIOColorConversionSettings* FDisplayClusterConfigurationICVFX_CameraOCIO::FindOCIOConfiguration(const FString& InClusterNodeId) const
{
	if (AllNodesOCIOConfiguration.bIsEnabled)
	{
		// Per node OCIO:
		for (const FDisplayClusterConfigurationOCIOProfile& OCIOProfileIt : PerNodeOCIOProfiles)
		{
			if (OCIOProfileIt.IsEnabledForObject(InClusterNodeId))
			{
				return &OCIOProfileIt.ColorConfiguration;
			}
		}

		return &AllNodesOCIOConfiguration.ColorConfiguration;
	}

	return nullptr;
}

bool FDisplayClusterConfigurationICVFX_CameraOCIO::IsChromakeyViewportSettingsEqual(const FString& InClusterNodeId1, const FString& InClusterNodeId2) const
{
	return IsInnerFrustumViewportSettingsEqual(InClusterNodeId1, InClusterNodeId2);
}

bool FDisplayClusterConfigurationICVFX_CameraOCIO::IsInnerFrustumViewportSettingsEqual(const FString& InClusterNodeId1, const FString& InClusterNodeId2) const
{
	if (AllNodesOCIOConfiguration.bIsEnabled)
	{
		for (const FDisplayClusterConfigurationOCIOProfile& OCIOProfileIt : PerNodeOCIOProfiles)
		{
			if (OCIOProfileIt.bIsEnabled)
			{
				const FString* CustomNode1 = OCIOProfileIt.ApplyOCIOToObjects.FindByPredicate([ClusterNodeId = InClusterNodeId1](const FString& InClusterNodeId)
					{
						return ClusterNodeId.Equals(InClusterNodeId, ESearchCase::IgnoreCase);
					});

				const FString* CustomNode2 = OCIOProfileIt.ApplyOCIOToObjects.FindByPredicate([ClusterNodeId = InClusterNodeId2](const FString& InClusterNodeId)
					{
						return ClusterNodeId.Equals(InClusterNodeId, ESearchCase::IgnoreCase);
					});

				if (CustomNode1 && CustomNode2)
				{
					// equal custom settings
					return true;
				}

				if (CustomNode1 || CustomNode2)
				{
					// one of node has custom settings
					return false;
				}
			}
		}
	}

	return true;
}

///////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterConfigurationICVFX_LightcardOCIO
///////////////////////////////////////////////////////////////////////////////////////
const FOpenColorIOColorConversionSettings* FDisplayClusterConfigurationICVFX_LightcardOCIO::FindOCIOConfiguration(const FString& InViewportId, const FDisplayClusterConfigurationICVFX_ViewportOCIO& InViewportOCIO) const
{
	switch (LightcardOCIOMode)
	{
	case EDisplayClusterConfigurationViewportLightcardOCIOMode::nDisplay:
		// Use Viewport OCIO
		return InViewportOCIO.FindOCIOConfiguration(InViewportId);

	case EDisplayClusterConfigurationViewportLightcardOCIOMode::Custom:
		// Use custom OCIO
		return CustomOCIO.FindOCIOConfiguration(InViewportId);

	default:
		// No OCIO for Light Cards
		break;
	}

	return nullptr;
}

///////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterConfigurationICVFX_CameraSettings
///////////////////////////////////////////////////////////////////////////////////////
FDisplayClusterConfigurationICVFX_CameraSettings::FDisplayClusterConfigurationICVFX_CameraSettings()
{
	AllNodesColorGrading.bEnableEntireClusterColorGrading = true;
}

ACineCameraActor* FDisplayClusterConfigurationICVFX_CameraSettings::GetExternalCineCameraActor() const
{
	ACineCameraActor* ExternalCineCameraActor = ExternalCameraActor.Get();
	if (IsValid(ExternalCineCameraActor))
	{
		return ExternalCineCameraActor;
	}

	return nullptr;
}

UCineCameraComponent* FDisplayClusterConfigurationICVFX_CameraSettings::GetExternalCineCameraComponent() const
{
	if (ACineCameraActor* ExternalCineCameraActor = GetExternalCineCameraActor())
	{
		UCineCameraComponent* ExternalCineCameraComponent = ExternalCineCameraActor->GetCineCameraComponent();
		if (IsValid(ExternalCineCameraComponent))
		{
			return ExternalCineCameraComponent;
		}
	}

	return nullptr;
}

bool FDisplayClusterConfigurationICVFX_CameraSettings::IsICVFXEnabled(const UDisplayClusterConfigurationData& InConfigurationData, const FString& InClusterNodeId) const
{
	// All rules in the FDisplayClusterViewport::IsRenderEnabledByMedia()
		return bEnable;
}

const FOpenColorIOColorConversionSettings* FDisplayClusterConfigurationICVFX_CameraSettings::FindInnerFrustumOCIOConfiguration(const FString& InClusterNodeId) const
{
	return CameraOCIO.FindOCIOConfiguration(InClusterNodeId);
}

const FOpenColorIOColorConversionSettings* FDisplayClusterConfigurationICVFX_CameraSettings::FindChromakeyOCIOConfiguration(const FString& InClusterNodeId) const
{
	// Always use incamera OCIO
	return CameraOCIO.FindOCIOConfiguration(InClusterNodeId);
}

bool FDisplayClusterConfigurationICVFX_CameraSettings::IsInnerFrustumViewportSettingsEqual(const FString& InClusterNodeId1, const FString& InClusterNodeId2) const
{
	return CameraOCIO.IsInnerFrustumViewportSettingsEqual(InClusterNodeId1, InClusterNodeId2);
}

bool FDisplayClusterConfigurationICVFX_CameraSettings::IsChromakeyViewportSettingsEqual(const FString& InClusterNodeId1, const FString& InClusterNodeId2) const
{
	return CameraOCIO.IsChromakeyViewportSettingsEqual(InClusterNodeId1, InClusterNodeId2);
}

float FDisplayClusterConfigurationICVFX_CameraSettings::GetCameraBufferRatio(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings) const
{
	return BufferRatio;
}

void FDisplayClusterConfigurationICVFX_CameraSettings::GetCameraUpscalerSettings(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings, FDisplayClusterUpscalerSettings& OutUpscalerSettings) const
{
	UpscalerSettings.GetUpscalerSettings(
		&InStageSettings.GlobalInnerFrustumUpscalerSettings,
		OutUpscalerSettings);
}

void FDisplayClusterConfigurationICVFX_CameraSettings::SetupViewInfo(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings, FMinimalViewInfo& InOutViewInfo)
{
	RenderSettings.SetupViewInfo(InStageSettings, InOutViewInfo);
	CustomFrustum.SetupViewInfo(InStageSettings, *this, InOutViewInfo);
	CameraMotionBlur.SetupViewInfo(InStageSettings, InOutViewInfo);
}

///////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterConfigurationICVFX_StageSettings
///////////////////////////////////////////////////////////////////////////////////////
const FOpenColorIOColorConversionSettings* FDisplayClusterConfigurationICVFX_StageSettings::FindViewportOCIOConfiguration(const FString& InViewportId) const
{
	return ViewportOCIO.FindOCIOConfiguration(InViewportId);
}

const FOpenColorIOColorConversionSettings* FDisplayClusterConfigurationICVFX_StageSettings::FindLightcardOCIOConfiguration(const FString& InViewportId) const
{
	return Lightcard.LightcardOCIO.FindOCIOConfiguration(InViewportId, ViewportOCIO);
}

EDisplayClusterShaderParametersICVFX_CameraOverlappingRenderMode FDisplayClusterConfigurationICVFX_StageSettings::GetCameraOverlappingRenderMode() const
{
	if (bEnableInnerFrustumChromakeyOverlap)
	{
		return EDisplayClusterShaderParametersICVFX_CameraOverlappingRenderMode::FinalPass;
	}

	return EDisplayClusterShaderParametersICVFX_CameraOverlappingRenderMode::None;
}

///////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterConfigurationICVFX_ChromakeySettings
///////////////////////////////////////////////////////////////////////////////////////
EDisplayClusterShaderParametersICVFX_ChromakeySource FDisplayClusterConfigurationICVFX_ChromakeySettings::GetChromakeyType(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings) const
{
	if (bEnable)
	{
		switch (ChromakeyType)
		{
		case EDisplayClusterConfigurationICVFX_ChromakeyType::InnerFrustum:
			return EDisplayClusterShaderParametersICVFX_ChromakeySource::FrameColor;

		case EDisplayClusterConfigurationICVFX_ChromakeyType::CustomChromakey:
			return EDisplayClusterShaderParametersICVFX_ChromakeySource::ChromakeyLayers;

		case EDisplayClusterConfigurationICVFX_ChromakeyType::Disabled:
			return EDisplayClusterShaderParametersICVFX_ChromakeySource::Default;

		default:
			break;
		}
	}

	return EDisplayClusterShaderParametersICVFX_ChromakeySource::Disabled;
}

FDisplayClusterConfigurationICVFX_ChromakeyRenderSettings* FDisplayClusterConfigurationICVFX_ChromakeySettings::GetWritableChromakeyRenderSettings(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings)
{
	// Note: Here we can add an override of the CK rendering settings from StageSetings
	return &ChromakeyRenderTexture;
}

const FDisplayClusterConfigurationICVFX_ChromakeyRenderSettings* FDisplayClusterConfigurationICVFX_ChromakeySettings::GetChromakeyRenderSettings(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings) const
{
	// Note: Here we can add an override of the CK rendering settings from StageSetings
	return &ChromakeyRenderTexture;
}

const FLinearColor& FDisplayClusterConfigurationICVFX_ChromakeySettings::GetChromakeyColor(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings) const
{
	switch(ChromakeySettingsSource)
	{
	case EDisplayClusterConfigurationICVFX_ChromakeySettingsSource::Viewport:
		// Override Chromakey color from stage settings
		return InStageSettings.GlobalChromakey.ChromakeyColor;

	default:
		break;
	}

	// Use Chromakey color from camera
	return ChromakeyColor;
}

const FLinearColor& FDisplayClusterConfigurationICVFX_ChromakeySettings::GetOverlapChromakeyColor(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings) const
{
	// Note: Here we can add an override of the CK overlap area color from camera

	// Use overlay color from stage settings
	return InStageSettings.GlobalChromakey.ChromakeyColor;
}

const FDisplayClusterConfigurationICVFX_ChromakeyMarkers* FDisplayClusterConfigurationICVFX_ChromakeySettings::ImplGetChromakeyMarkers(const FDisplayClusterConfigurationICVFX_ChromakeyMarkers* InValue) const
{
	// Chromakey markers require texture
	if (!InValue || !InValue->bEnable || InValue->MarkerTileRGBA == nullptr)
	{
		return nullptr;
	}

	// This CK markers can be used
	return InValue;
}

const FDisplayClusterConfigurationICVFX_ChromakeyMarkers* FDisplayClusterConfigurationICVFX_ChromakeySettings::GetChromakeyMarkers(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings) const
{
	switch (ChromakeySettingsSource)
	{
		case EDisplayClusterConfigurationICVFX_ChromakeySettingsSource::Viewport:
			// Use global CK markers
			return ImplGetChromakeyMarkers(&InStageSettings.GlobalChromakey.ChromakeyMarkers);

		default:
			break;
	}

	// Use CK markers from camera
	return ImplGetChromakeyMarkers(&ChromakeyMarkers);
}

const FDisplayClusterConfigurationICVFX_ChromakeyMarkers* FDisplayClusterConfigurationICVFX_ChromakeySettings::GetOverlapChromakeyMarkers(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings) const
{
	// Note: Here we can add an override of the CK overlap markers from camera
	FDisplayClusterConfigurationICVFX_ChromakeyMarkers const* OutChromakeyMarkers = &InStageSettings.GlobalChromakey.ChromakeyMarkers;

	// Use CK overlap markers from stage settings:
	return ImplGetChromakeyMarkers(OutChromakeyMarkers);
}

///////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterConfigurationICVFX_VisibilityList
///////////////////////////////////////////////////////////////////////////////////////
bool FDisplayClusterConfigurationICVFX_VisibilityList::IsVisibilityListValid() const
{
	for (const FString& ComponentNameIt : RootActorComponentNames)
	{
		if (!ComponentNameIt.IsEmpty())
		{
			return true;
		}
	}

	for (const TSoftObjectPtr<AActor>& ActorSOPtrIt : Actors)
	{
		if (ActorSOPtrIt.IsValid())
		{
			return true;
		}
	}

	for (const FActorLayer& ActorLayerIt : ActorLayers)
	{
		if (!ActorLayerIt.Name.IsNone())
		{
			return true;
		}
	}

	for (const TSoftObjectPtr<AActor>& AutoAddedActor : AutoAddedActors)
	{
		if (AutoAddedActor.IsValid())
		{
			return true;
		}
	}

	return false;
}

///////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterConfigurationICVFX_LightcardSettings
///////////////////////////////////////////////////////////////////////////////////////
bool FDisplayClusterConfigurationICVFX_LightcardSettings::ShouldUseLightCard(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings) const
{
	if (!bEnable)
	{
		// Don't use the lightcard if it is disabled
		return false;
	}

	if (RenderSettings.Replace.bAllowReplace)
	{
		if (RenderSettings.Replace.SourceTexture == nullptr)
		{
			// LightcardSettings.Override require source texture.
			return false;
		}
	}

	// Lightcard require layers for render
	return ShowOnlyList.IsVisibilityListValid();
}

bool FDisplayClusterConfigurationICVFX_LightcardSettings::ShouldUseUVLightCard(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings) const
{
	//Note: Here we can add custom rules for UV lightcards
	return ShouldUseLightCard(InStageSettings);
}

EDisplayClusterShaderParametersICVFX_LightCardRenderMode FDisplayClusterConfigurationICVFX_LightcardSettings::GetLightCardRenderModeOverride(const UDisplayClusterConfigurationViewport* InViewportConfiguration) const
{
	if (!bEnable || (InViewportConfiguration && !InViewportConfiguration->ICVFX.bAllowICVFX))
	{
		// When ICVFX is disabled we don't override lightcards rendering mode
		return EDisplayClusterShaderParametersICVFX_LightCardRenderMode::None;
	}

	if (InViewportConfiguration && InViewportConfiguration->ICVFX.LightcardRenderMode != EDisplayClusterConfigurationICVFX_OverrideLightcardRenderMode::Default)
	{
		// Use overridden values from the viewport:
		switch (InViewportConfiguration->ICVFX.LightcardRenderMode)
		{
		case EDisplayClusterConfigurationICVFX_OverrideLightcardRenderMode::Over:
			return EDisplayClusterShaderParametersICVFX_LightCardRenderMode::Over;

		case EDisplayClusterConfigurationICVFX_OverrideLightcardRenderMode::Under:
			return EDisplayClusterShaderParametersICVFX_LightCardRenderMode::Under;

		default:
			break;
		}
	}

	return EDisplayClusterShaderParametersICVFX_LightCardRenderMode::None;
}

EDisplayClusterShaderParametersICVFX_LightCardRenderMode FDisplayClusterConfigurationICVFX_LightcardSettings::GetLightCardRenderMode(const EDisplayClusterConfigurationICVFX_PerLightcardRenderMode InPerLightcardRenderMode, const UDisplayClusterConfigurationViewport* InViewportConfiguration) const
{
	if (!bEnable || (InViewportConfiguration && !InViewportConfiguration->ICVFX.bAllowICVFX))
	{
		// When ICVFX is disabled we don't render lightcards
		return EDisplayClusterShaderParametersICVFX_LightCardRenderMode::None;
	}

	if (InViewportConfiguration && InViewportConfiguration->ICVFX.LightcardRenderMode != EDisplayClusterConfigurationICVFX_OverrideLightcardRenderMode::Default)
	{
		// Use overridden values from the viewport:
		switch (InViewportConfiguration->ICVFX.LightcardRenderMode)
		{
		case EDisplayClusterConfigurationICVFX_OverrideLightcardRenderMode::Over:
			return EDisplayClusterShaderParametersICVFX_LightCardRenderMode::Over;

		case EDisplayClusterConfigurationICVFX_OverrideLightcardRenderMode::Under:
			return EDisplayClusterShaderParametersICVFX_LightCardRenderMode::Under;

		default:
			break;
		}

		return EDisplayClusterShaderParametersICVFX_LightCardRenderMode::None;
	}

	// Per-lightcard render mode:
	switch (InPerLightcardRenderMode)
	{
	case EDisplayClusterConfigurationICVFX_PerLightcardRenderMode::Under:
		return EDisplayClusterShaderParametersICVFX_LightCardRenderMode::Under;

	case EDisplayClusterConfigurationICVFX_PerLightcardRenderMode::Over:
		return EDisplayClusterShaderParametersICVFX_LightCardRenderMode::Over;

	default:
		break;
	}

	// Use global lightcard settings:
	switch (Blendingmode)
	{
	case EDisplayClusterConfigurationICVFX_LightcardRenderMode::Under:
		return EDisplayClusterShaderParametersICVFX_LightCardRenderMode::Under;

	default:
		break;
	};

	// By default, lightcards are rendered in "Over" mode.
	return EDisplayClusterShaderParametersICVFX_LightCardRenderMode::Over;
}

///////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterConfigurationICVFX_ChromakeyRenderSettings
///////////////////////////////////////////////////////////////////////////////////////
bool FDisplayClusterConfigurationICVFX_ChromakeyRenderSettings::ShouldUseChromakeyViewport(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings) const
{
	if (Replace.bAllowReplace)
	{
		if (Replace.SourceTexture == nullptr)
		{
			// ChromakeyRender Override require source texture.
			return false;
		}
	}

	// ChromakeyRender requires actors for render.
	return ShowOnlyList.IsVisibilityListValid();
}

void FDisplayClusterConfigurationICVFX_CameraDepthOfField::UpdateDynamicCompensationLUT()
{
	CompensationLUT.LoadSynchronous();
	if (CompensationLUT)
	{
		FSharedImageConstRef CPUTextureRef = CompensationLUT->GetCPUCopy();
		if (CPUTextureRef.IsValid() && CPUTextureRef->Format == ERawImageFormat::R32F)
		{
			TArrayView64<const float> SrcPixels = CPUTextureRef->AsR32F();
			TArray64<FFloat16> DestPixels;
			DestPixels.AddZeroed(SrcPixels.Num());

			for (int32 Index = 0; Index < SrcPixels.Num(); ++Index)
			{
				// Scale the offset encoded in the LUT so that the final CoC when computed in the DoF pipeline is scaled by the gain.
				// The actual new offset needed to accomplish this comes from the following equation:
				// c * (CoC_obj + CoC_off) = CoC_obj + newOffset =>
				// newOffset = (1 - c) * CoC_obj + c * CoC_off
				
				const float ObjectCoC = Index / 32.0f + 1;
				const float Offset = SrcPixels[Index];
				const float ScaledOffset = (1 - DepthOfFieldGain) * ObjectCoC + DepthOfFieldGain * Offset;
				DestPixels[Index] = ScaledOffset;
			}

			TArrayView<uint8> PixelsView((uint8*)DestPixels.GetData(), int64(DestPixels.Num() * sizeof(FFloat16)));

			// Texture format is assumed to be greyscale (PF_G8), and we must disable sRGB on the texture to ensure the raw byte value, which encodes
			// the offset in pixels, is passed unmodified to the depth of field shader
			if (UTexture2D* NewTexture = UTexture2D::CreateTransient(CPUTextureRef->GetWidth(), CPUTextureRef->GetHeight(), PF_R16F, NAME_None, PixelsView))
			{
				DynamicCompensationLUT = NewTexture;
				DynamicCompensationLUT->SRGB = 0;
				return;
			}
		}
	}

	DynamicCompensationLUT = nullptr;
}

void FDisplayClusterConfigurationICVFX_CameraCustomFrustum::SetupViewInfo(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings, const FDisplayClusterConfigurationICVFX_CameraSettings& InCameraSettings, FMinimalViewInfo& InOutViewInfo) const
{
	// Since Circle of confusion is directly proportional to aperature, with wider FOV focal length needs to be shortened by the same amount as FOV.
	// Adapting the FOV of the nDisplay viewport to DoF is already done in FDisplayClusterViewport_CustomPostProcessSettings::ConfigurePostProcessSettingsForViewport().
}

float FDisplayClusterConfigurationICVFX_CameraCustomFrustum::GetCameraFieldOfViewMultiplier(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings) const
{
	if (bEnable)
	{
		return FieldOfViewMultiplier;
	}

	return  1.f;
}

float FDisplayClusterConfigurationICVFX_CameraCustomFrustum::GetCameraAdaptResolutionRatio(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings) const
{
	if (bAdaptResolution)
	{
		return GetCameraFieldOfViewMultiplier(InStageSettings);
	}

	// Don't use an adaptive resolution multiplier
	return 1.f;
}

void FDisplayClusterConfigurationICVFX_CameraMotionBlur::SetupViewInfo(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings, FMinimalViewInfo& InOutViewInfo) const
{
	// Add postprocess blur settings to viewinfo PP
	if (MotionBlurPPS.bReplaceEnable)
	{
		// Send camera postprocess to override
		InOutViewInfo.PostProcessBlendWeight = 1.0f;

		InOutViewInfo.PostProcessSettings.MotionBlurAmount = MotionBlurPPS.MotionBlurAmount;
		InOutViewInfo.PostProcessSettings.bOverride_MotionBlurAmount = true;

		InOutViewInfo.PostProcessSettings.MotionBlurMax = MotionBlurPPS.MotionBlurMax;
		InOutViewInfo.PostProcessSettings.bOverride_MotionBlurMax = true;

		InOutViewInfo.PostProcessSettings.MotionBlurPerObjectSize = MotionBlurPPS.MotionBlurPerObjectSize;
		InOutViewInfo.PostProcessSettings.bOverride_MotionBlurPerObjectSize = true;
	}
}

///////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterConfigurationICVFX_CameraDepthOfField
///////////////////////////////////////////////////////////////////////////////////////
UTexture2D* FDisplayClusterConfigurationICVFX_CameraDepthOfField::GetCompensationLUT(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings) const
{
	if (DynamicCompensationLUT)
	{
		return ToRawPtr(DynamicCompensationLUT);
	}
	
	if (CompensationLUT.IsValid())
	{
		return CompensationLUT.Get();
	}

	return nullptr;
}
