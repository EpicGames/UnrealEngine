// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfigurationTypes_Media.h"


///////////////////////////////////////////////////
// FDisplayClusterConfigurationMediaNodeBackbuffer

bool FDisplayClusterConfigurationMediaNodeBackbuffer::IsMediaOutputAssigned() const
{
	// Just see if any FullFrame media output instances are assigned
	bool bAnyMediaAssigned = MediaOutputs.ContainsByPredicate([](const FDisplayClusterConfigurationMediaOutput& Item)
		{
			return IsValid(Item.MediaOutput);
		});

	// If no FullFrame, let's see if there are any tiles configured
	if (!bAnyMediaAssigned)
	{
		bAnyMediaAssigned = TiledMediaOutputs.ContainsByPredicate([](const FDisplayClusterConfigurationMediaUniformTileOutput& Item)
			{
				return IsValid(Item.MediaOutput);
			});
	}

	return bAnyMediaAssigned;
}


///////////////////////////////////////////////////
// FDisplayClusterConfigurationMediaViewport

bool FDisplayClusterConfigurationMediaViewport::IsMediaInputAssigned() const
{
	return IsValid(MediaInput.MediaSource);
}

bool FDisplayClusterConfigurationMediaViewport::IsMediaOutputAssigned() const
{
	// Just see if any media output instances are assigned
	const bool bAnyMediaAssigned = MediaOutputs.ContainsByPredicate([](const FDisplayClusterConfigurationMediaOutput& Item)
		{
			return IsValid(Item.MediaOutput);
		});

	return bAnyMediaAssigned;
}


///////////////////////////////////////////////////
// FDisplayClusterConfigurationMediaICVFX

bool FDisplayClusterConfigurationMediaICVFX::ShouldMediaICVFXSplitIntoTiles() const
{
	// Nothing to do if media is not enabled
	if (!bEnable)
	{
		return false;
	}

	// Nothing to do if media tiling is not used
	if (SplitType != EDisplayClusterConfigurationMediaSplitType::UniformTiles)
	{
		return false;
	}

	// Validate the layout
	if (!FDisplayClusterConfigurationTile_Settings::IsValid(TiledSplitLayout))
	{
		return false;
	}

	// Allow the use of tile splitting.
	return true;
}

EDisplayClusterViewportTileFlags FDisplayClusterConfigurationMediaICVFX::GetMediaICVFXTileFlags(const FString& NodeId) const
{
	// Find if this cluster node is allowed to render unbound tiles
	const bool bAllowRenderUnbound = ClusterNodesToRenderUnboundTiles.ItemNames.ContainsByPredicate([&NodeId](const FString& Item)
		{
			return NodeId.Equals(Item, ESearchCase::IgnoreCase);
		});

	// Generate flags
	EDisplayClusterViewportTileFlags OutTileFlags = EDisplayClusterViewportTileFlags::None;
	if (bAllowRenderUnbound)
	{
		OutTileFlags |= EDisplayClusterViewportTileFlags::AllowUnboundRender;
	}

	return OutTileFlags;
}

bool FDisplayClusterConfigurationMediaICVFX::HasAnyMediaInputAssigned(const FString& NodeId, EDisplayClusterConfigurationMediaSplitType InSplitType) const
{
	// Nothing to do if a different split type requested
	if (SplitType != InSplitType)
	{
		return false;
	}

	if (InSplitType == EDisplayClusterConfigurationMediaSplitType::FullFrame)
	{
		const bool bHasInput = IsValid(GetMediaSource(NodeId));
		return bHasInput;
	}
	else if (InSplitType == EDisplayClusterConfigurationMediaSplitType::UniformTiles)
	{
		TArray<FDisplayClusterConfigurationMediaUniformTileInput> InputTiles;
		GetMediaInputTiles(NodeId, InputTiles);

		// Find any tile with a media source assigned
		const bool bFoundInputTile = InputTiles.ContainsByPredicate([](const FDisplayClusterConfigurationMediaUniformTileInput& Tile)
			{
				return IsValid(Tile.MediaSource);
			});

		return bFoundInputTile;
	}
	else
	{
		checkNoEntry();
	}

	return false;
}

bool FDisplayClusterConfigurationMediaICVFX::HasAnyMediaOutputAssigned(const FString& NodeId, EDisplayClusterConfigurationMediaSplitType InSplitType) const
{
	if (SplitType != InSplitType)
	{
		return false;
	}

	if (InSplitType == EDisplayClusterConfigurationMediaSplitType::FullFrame)
	{
		const TArray<FDisplayClusterConfigurationMediaOutputGroup> NodeOutputGroups = GetMediaOutputGroups(NodeId);

		const bool bFoundOutput = NodeOutputGroups.ContainsByPredicate([](const FDisplayClusterConfigurationMediaOutputGroup& OutputGroup)
			{
				return IsValid(OutputGroup.MediaOutput);
			});

		return bFoundOutput;
	}
	else if (InSplitType == EDisplayClusterConfigurationMediaSplitType::UniformTiles)
	{
		TArray<FDisplayClusterConfigurationMediaUniformTileOutput> OutputTiles;
		GetMediaOutputTiles(NodeId, OutputTiles);

		const bool bFoundOutputTile = OutputTiles.ContainsByPredicate([](const FDisplayClusterConfigurationMediaUniformTileOutput& Tile)
			{
				return IsValid(Tile.MediaOutput);
			});

		return bFoundOutputTile;
	}
	else
	{
		checkNoEntry();
	}

	return false;
}

//@note: Full-frame input - we don't expect the same node ID to be used multiple times (in different groups)
UMediaSource* FDisplayClusterConfigurationMediaICVFX::GetMediaSource(const FString& NodeId) const
{
	if (SplitType != EDisplayClusterConfigurationMediaSplitType::FullFrame)
	{
		return nullptr;
	}

	// Look up for the first group that contains node ID specified
	const FDisplayClusterConfigurationMediaInputGroup* const FoundGroup = MediaInputGroups.FindByPredicate([&NodeId](const FDisplayClusterConfigurationMediaInputGroup& MediaInputGroup)
		{
			return MediaInputGroup.ClusterNodes.ItemNames.ContainsByPredicate([&NodeId](const FString& Item)
				{
					return Item.Equals(NodeId, ESearchCase::IgnoreCase);
				});
		});

	if (FoundGroup)
	{
		return FoundGroup->MediaSource;
	}

	return nullptr;
}

//@note: Full-frame output, it's allowed to use the same node ID in different groups
TArray<FDisplayClusterConfigurationMediaOutputGroup> FDisplayClusterConfigurationMediaICVFX::GetMediaOutputGroups(const FString& NodeId) const
{
	if (SplitType != EDisplayClusterConfigurationMediaSplitType::FullFrame)
	{
		return TArray<FDisplayClusterConfigurationMediaOutputGroup>();
	}

	return MediaOutputGroups.FilterByPredicate([&NodeId](const FDisplayClusterConfigurationMediaOutputGroup& Item)
		{
			return Item.ClusterNodes.ItemNames.ContainsByPredicate([&NodeId](const FString& Item)
				{
					return Item.Equals(NodeId, ESearchCase::IgnoreCase);
				});
		});
}

//@note: Tiled input, it's allowed to use the same node ID in different groups
bool FDisplayClusterConfigurationMediaICVFX::GetMediaInputTiles(const FString& NodeId, TArray<FDisplayClusterConfigurationMediaUniformTileInput>& OutTiles) const
{
	if (SplitType != EDisplayClusterConfigurationMediaSplitType::UniformTiles)
	{
		return false;
	}

	// Find all input groups bound to the node ID specified
	const TArray<FDisplayClusterConfigurationMediaTiledInputGroup> FoundInputGroups = TiledMediaInputGroups.FilterByPredicate([&NodeId](const FDisplayClusterConfigurationMediaTiledInputGroup& TiledInputGroup)
		{
			return TiledInputGroup.ClusterNodes.ItemNames.ContainsByPredicate([&NodeId](const FString& Item)
				{
					return Item.Equals(NodeId, ESearchCase::IgnoreCase);
				});
		});

	// Combine result out of the filtered groups
	for (const FDisplayClusterConfigurationMediaTiledInputGroup& TiledInputGroup : FoundInputGroups)
	{
		OutTiles.Append(TiledInputGroup.Tiles);
	}

	return true;
}

//@note: Tiled output, it's allowed to use the same node ID in different groups
bool FDisplayClusterConfigurationMediaICVFX::GetMediaOutputTiles(const FString& NodeId, TArray<FDisplayClusterConfigurationMediaUniformTileOutput>& OutTiles) const
{
	if (SplitType != EDisplayClusterConfigurationMediaSplitType::UniformTiles)
	{
		return false;
	}

	// Find all output groups bound to the node ID specified
	const TArray<FDisplayClusterConfigurationMediaTiledOutputGroup> FoundOutputGroups = TiledMediaOutputGroups.FilterByPredicate([&NodeId](const FDisplayClusterConfigurationMediaTiledOutputGroup& TiledOutputGroup)
		{
			return TiledOutputGroup.ClusterNodes.ItemNames.ContainsByPredicate([&NodeId](const FString& Item)
				{
					return Item.Equals(NodeId, ESearchCase::IgnoreCase);
				});
		});

	// Combine result out of the filtered groups
	for (const FDisplayClusterConfigurationMediaTiledOutputGroup& TiledOutputGroup : FoundOutputGroups)
	{
		OutTiles.Append(TiledOutputGroup.Tiles);
	}

	return true;
}
