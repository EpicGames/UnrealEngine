// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfigurationTypes_OCIO.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"

///////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterConfigurationOCIOConfiguration
///////////////////////////////////////////////////////////////////////////////////////
bool FDisplayClusterConfigurationOCIOConfiguration::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);

	// Don't actually serialize, just write the custom version for PostSerialize
	return false;
}

void FDisplayClusterConfigurationOCIOConfiguration::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		if (Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) < FUE5ReleaseStreamObjectVersion::OpenColorIODisabledDisplayConfigurationDefault)
		{
			// Retain previous behavior: enabled only when the settings were valid
			if (ColorConfiguration.IsValid())
			{
				bIsEnabled = true;
			}
		}
	}
}

///////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterConfigurationOCIOProfile
///////////////////////////////////////////////////////////////////////////////////////
bool FDisplayClusterConfigurationOCIOProfile::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);

	// Don't actually serialize, just write the custom version for PostSerialize
	return false;
}

void FDisplayClusterConfigurationOCIOProfile::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		if (Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) < FUE5ReleaseStreamObjectVersion::OpenColorIODisabledDisplayConfigurationDefault)
		{
			// Retain previous behavior: enabled only when the settings were valid
			if (ColorConfiguration.IsValid())
			{
				bIsEnabled = true;
			}
		}
	}
}

bool FDisplayClusterConfigurationOCIOProfile::IsEnabledForObject(const FString& InObjectId) const
{
	if (bIsEnabled)
	{
		for (const FString& ViewportNameIt : ApplyOCIOToObjects)
		{
			if (InObjectId.Compare(ViewportNameIt, ESearchCase::IgnoreCase) == 0)
			{
				return true;
			}
		}
	}

	return false;
}
