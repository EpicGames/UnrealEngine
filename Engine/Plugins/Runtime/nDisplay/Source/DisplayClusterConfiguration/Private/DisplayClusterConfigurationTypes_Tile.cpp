// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfigurationTypes_Tile.h"
#include "DisplayClusterConfigurationTypes.h"


///////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterConfigurationViewport_Tile
///////////////////////////////////////////////////////////////////////////////////////
bool FDisplayClusterConfigurationTile_Settings::IsEnabled(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings) const
{
	// Note: InStageSettings can be used to disable tile rendering for the entire stage.

	if (!bEnabled)
	{
		// This property enables tile rendering.
		return false;
	}

	if (!FDisplayClusterConfigurationTile_Settings::IsValid(Layout))
	{
		// Invalid layout settings
		return false;
	}

	return true;
}

bool FDisplayClusterConfigurationTile_Settings::IsValid(const FIntPoint& InTilesLayout)
{
	// Ignore wrong values
	if (InTilesLayout.X < 1 || InTilesLayout.Y < 1)
	{
		return false;
	}

	// Ignore 1x1 case
	if (InTilesLayout.X == 1 && InTilesLayout.Y == 1)
	{
		return false;
	}

	return true;
}
