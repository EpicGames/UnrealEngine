// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfigurationTypes_Upscaler.h"
#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterConfigurationTypes_Enums.h"

#include "UObject/EnumProperty.h"
#include "UObject/UnrealType.h"

namespace UE::DisplayClusterConfiguration::UpscalerSettings
{
	/** Get configuration data for the current upscaling method. */
	static inline void GetConfigurationData(const FDisplayClusterConfigurationUpscalerSettings& InSettings, FInstancedPropertyBag& OutPropertyBag)
	{
		OutPropertyBag = InSettings.EditingData;
	}

	/** Convert type EDisplayClusterConfigurationUpscalingMethod->EDisplayClusterUpscalerAntiAliasingMethod */
	static inline EDisplayClusterUpscalerAntiAliasingMethod ToAntiAliasingMethod(const EDisplayClusterConfigurationUpscalingMethod InUpscalingMethod)
	{
		// Get AA method
		switch (InUpscalingMethod)
		{
		case EDisplayClusterConfigurationUpscalingMethod::Disable: return EDisplayClusterUpscalerAntiAliasingMethod::None;

		case EDisplayClusterConfigurationUpscalingMethod::FXAA: return EDisplayClusterUpscalerAntiAliasingMethod::FXAA;
		case EDisplayClusterConfigurationUpscalingMethod::MSAA: return EDisplayClusterUpscalerAntiAliasingMethod::MSAA;
		case EDisplayClusterConfigurationUpscalingMethod::TAA:  return EDisplayClusterUpscalerAntiAliasingMethod::TAA;
		case EDisplayClusterConfigurationUpscalingMethod::TSR:  return EDisplayClusterUpscalerAntiAliasingMethod::TSR;

		default:
			break;
		}

		return EDisplayClusterUpscalerAntiAliasingMethod::Default;
	};

};

bool FDisplayClusterConfigurationUpscalerSettings::GetUpscalerSettings(
	const FDisplayClusterConfigurationUpscalerSettings* InDefaultUpscalerSettings,
	FDisplayClusterUpscalerSettings& OutUpscalerSettings) const
{
	using namespace UE::DisplayClusterConfiguration::UpscalerSettings;

	// Find method name in the EDisplayClusterConfigurationUpscalingMethod
	if (const UEnum* EnumClass = StaticEnum<EDisplayClusterConfigurationUpscalingMethod>())
	{
		for (int32 EnumElementIndex = 0; EnumElementIndex < EnumClass->NumEnums(); ++EnumElementIndex)
		{
			if (EnumClass->GetNameStringByIndex(EnumElementIndex) == MethodName
#if WITH_METADATA
				&& !EnumClass->HasMetaData(TEXT("Hidden"), EnumElementIndex)
#endif /** WITH_METADATA */
				)
			{
				OutUpscalerSettings.AntiAliasingMethod = ToAntiAliasingMethod(static_cast<EDisplayClusterConfigurationUpscalingMethod>(EnumElementIndex));

				// Use default settings
				if (OutUpscalerSettings.AntiAliasingMethod == EDisplayClusterUpscalerAntiAliasingMethod::Default && InDefaultUpscalerSettings)
				{
					return InDefaultUpscalerSettings->GetUpscalerSettings(nullptr, OutUpscalerSettings);
				}

				// Custom upscaler not used.
				OutUpscalerSettings.CustomUpscalerName = NAME_None;
				GetConfigurationData(*this, OutUpscalerSettings.CustomUpscalerSettings);

				return true;
			}
		}
	}

	// Otherwise use custom upscaler
	{
		// Custom upscaler uses its own antialiasing rules.
		OutUpscalerSettings.AntiAliasingMethod = EDisplayClusterUpscalerAntiAliasingMethod::Default;

		// Set custom upscaler name and PropertyBag with settings.
		OutUpscalerSettings.CustomUpscalerName = MethodName;
		GetConfigurationData(*this, OutUpscalerSettings.CustomUpscalerSettings);
	}

	return MethodName != NAME_None;
}