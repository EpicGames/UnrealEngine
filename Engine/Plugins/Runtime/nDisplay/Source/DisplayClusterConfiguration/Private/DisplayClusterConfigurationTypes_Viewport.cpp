// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfigurationTypes_Viewport.h"
#include "DisplayClusterConfigurationTypes_ICVFX.h"
#include "DisplayClusterConfigurationUtils.h"

///////////////////////////////////////////////////////////////////////////////////////
// UDisplayClusterConfigurationViewport
///////////////////////////////////////////////////////////////////////////////////////
EDisplayClusterViewportICVFXFlags UDisplayClusterConfigurationViewport::GetViewportICVFXFlags(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings) const
{
	EDisplayClusterViewportICVFXFlags OutFlags = EDisplayClusterViewportICVFXFlags::None;
	if (ICVFX.bAllowICVFX)
	{
		EnumAddFlags(OutFlags, EDisplayClusterViewportICVFXFlags::Enable);
	}

	// Override camera render mode
	EDisplayClusterConfigurationICVFX_OverrideCameraRenderMode UsedCameraRenderMode = ICVFX.CameraRenderMode;
	if (!ICVFX.bAllowInnerFrustum || !InStageSettings.bEnableInnerFrustums)
	{
		UsedCameraRenderMode = EDisplayClusterConfigurationICVFX_OverrideCameraRenderMode::Disabled;
	}

	switch (UsedCameraRenderMode)
	{
	// Disable camera frame render for this viewport
	case EDisplayClusterConfigurationICVFX_OverrideCameraRenderMode::Disabled:
		EnumAddFlags(OutFlags, EDisplayClusterViewportICVFXFlags::DisableCamera | EDisplayClusterViewportICVFXFlags::DisableChromakey | EDisplayClusterViewportICVFXFlags::DisableChromakeyMarkers);
		break;

	// Disable chromakey render for this viewport
	case EDisplayClusterConfigurationICVFX_OverrideCameraRenderMode::DisableChromakey:
		EnumAddFlags(OutFlags, EDisplayClusterViewportICVFXFlags::DisableChromakey | EDisplayClusterViewportICVFXFlags::DisableChromakeyMarkers);
		break;

	// Disable chromakey markers render for this viewport
	case EDisplayClusterConfigurationICVFX_OverrideCameraRenderMode::DisableChromakeyMarkers:
		EnumAddFlags(OutFlags, EDisplayClusterViewportICVFXFlags::DisableChromakeyMarkers);
		break;

	default:
		break;
	}

	// Disable lightcards rendering
	const EDisplayClusterShaderParametersICVFX_LightCardRenderMode LightCardRenderMode = InStageSettings.Lightcard.GetLightCardRenderMode(EDisplayClusterConfigurationICVFX_PerLightcardRenderMode::Default, this);
	if (LightCardRenderMode == EDisplayClusterShaderParametersICVFX_LightCardRenderMode::None)
	{
		EnumAddFlags(OutFlags, EDisplayClusterViewportICVFXFlags::DisableLightcard);
	}

	// Per-viewport lightcard
	const EDisplayClusterShaderParametersICVFX_LightCardRenderMode LightCardRenderModeOverride = InStageSettings.Lightcard.GetLightCardRenderModeOverride(this);
	switch (LightCardRenderModeOverride)
	{
	case EDisplayClusterShaderParametersICVFX_LightCardRenderMode::Over:
		EnumAddFlags(OutFlags, EDisplayClusterViewportICVFXFlags::LightcardAlwaysOver);
		break;

	case EDisplayClusterShaderParametersICVFX_LightCardRenderMode::Under:
		EnumAddFlags(OutFlags, EDisplayClusterViewportICVFXFlags::LightcardAlwaysUnder);
		break;

	default:
		EnumAddFlags(OutFlags, EDisplayClusterViewportICVFXFlags::LightcardUseStageSettings);
		break;
	}

	// Reverse camera order when the option is set in the viewport.
	if (ICVFX.bReverseCameraPriority)
	{
		EnumAddFlags(OutFlags, EDisplayClusterViewportICVFXFlags::ReverseCameraPriority);
	}

	return OutFlags;
}

EDisplayClusterShaderParametersICVFX_ChromakeySource UDisplayClusterConfigurationViewport::GetViewportChromakeyType(
	const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings,
	const FString& InCameraId,
	const struct FDisplayClusterConfigurationICVFX_CameraSettings& InCameraSettings) const
{
	// Get ICVFX camera settings:
	const EDisplayClusterShaderParametersICVFX_ChromakeySource CameraChromakeySource = InCameraSettings.Chromakey.GetChromakeyType(InStageSettings);

	const EDisplayClusterViewportICVFXFlags ICVFXFlags = GetViewportICVFXFlags(InStageSettings);
	const EDisplayClusterConfigurationICVFX_OverrideChromakeyType OverrideChromakeySource = ICVFX.GetOverrideChromakeyType(InCameraId);

	if (CameraChromakeySource == EDisplayClusterShaderParametersICVFX_ChromakeySource::Disabled
	|| OverrideChromakeySource == EDisplayClusterConfigurationICVFX_OverrideChromakeyType::Disabled
	|| EnumHasAnyFlags(ICVFXFlags, EDisplayClusterViewportICVFXFlags::DisableCamera | EDisplayClusterViewportICVFXFlags::DisableChromakey)
	)
	{
		return EDisplayClusterShaderParametersICVFX_ChromakeySource::Disabled;
	}

	// Override chromakey from viewport
	switch (OverrideChromakeySource)
	{
		case EDisplayClusterConfigurationICVFX_OverrideChromakeyType::InnerFrustum:
			return EDisplayClusterShaderParametersICVFX_ChromakeySource::FrameColor;

		case EDisplayClusterConfigurationICVFX_OverrideChromakeyType::CustomChromakey:
			return EDisplayClusterShaderParametersICVFX_ChromakeySource::ChromakeyLayers;

		default:
			break;
	}

	// Default By default CK is disabled.
	if(CameraChromakeySource == EDisplayClusterShaderParametersICVFX_ChromakeySource::Default)
	{
		return EDisplayClusterShaderParametersICVFX_ChromakeySource::Disabled;
	}

	return CameraChromakeySource;
}

EDisplayClusterConfigurationICVFX_OverrideChromakeyType FDisplayClusterConfigurationViewport_ICVFX::GetOverrideChromakeyType(const FString& CameraId) const
{
	if (const EDisplayClusterConfigurationICVFX_OverrideChromakeyType* OverrideChromakeyTypePtr = PerCameraOverrideChromakeyType.Find(CameraId))
	{
		return *OverrideChromakeyTypePtr;
	}

	return OverrideChromakeyType;
}