// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfigurationTypes_ViewportRemap.h"

bool FDisplayClusterConfigurationViewport_RemapData::IsValid() const
{
	bool bIsRotating = IsRotating();
	bool bIsFlipping = IsFlipping();
	bool bIsTranslating = !ViewportRegion.ToRect().IsEmpty() || !OutputRegion.ToRect().IsEmpty();

	return bIsRotating || bIsFlipping || bIsTranslating;
}

bool FDisplayClusterConfigurationViewport_RemapData::IsRotating() const
{
	const float NormalizedAngle = FRotator::NormalizeAxis(Angle);
	return FMath::Abs(NormalizedAngle) > KINDA_SMALL_NUMBER;
}

bool FDisplayClusterConfigurationViewport_RemapData::IsFlipping() const
{
	return bFlipH || bFlipV;
}