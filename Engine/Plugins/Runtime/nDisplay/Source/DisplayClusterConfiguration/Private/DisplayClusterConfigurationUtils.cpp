// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfigurationUtils.h"

#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterProjectionStrings.h"

#include "UObject/Object.h"
#include "UObject/UObjectThreadContext.h"


bool FDisplayClusterConfigurationUtils::IsSerializingTemplate(const FArchive& Ar)
{
	const FUObjectSerializeContext* SerializeContext = FUObjectThreadContext::Get().GetSerializeContext();
	const UObject* SerializedObject = SerializeContext ? SerializeContext->SerializedObject : nullptr;

	// We assume that if the SerializedObject is null, it is indicative of a template.
	return SerializedObject ? SerializedObject->IsTemplate() : true;
}

void FDisplayClusterConfigurationUtils::UpdateToLatest(UDisplayClusterConfigurationData* Config)
{
	// Ignore invalid input
	if (!IsValid(Config) || !IsValid(Config->Cluster))
	{
		return;
	}

	// Iterate through every cluster node
	for (TPair<FString, TObjectPtr<UDisplayClusterConfigurationClusterNode>>& ClusterNode : Config->Cluster->Nodes)
	{
		if (!IsValid(ClusterNode.Value))
		{
			continue;
		}

		// Iterate though every viewport
		for (TPair<FString, TObjectPtr<UDisplayClusterConfigurationViewport>>& Viewport : ClusterNode.Value->Viewports)
		{
			if (!IsValid(Viewport.Value))
			{
				continue;
			}

			// 'simple' -> 'mesh' policy migration
			RedirectSimpleToMeshPolicy(Viewport.Value);
		}
	}
}


void FDisplayClusterConfigurationUtils::RedirectSimpleToMeshPolicy(UDisplayClusterConfigurationViewport* ViewportCfg)
{
	// Ignore inalid input
	if (!IsValid(ViewportCfg))
	{
		return;
	}

	// 5.8+ backward compatibility path for 'Simple' projection policy. Replace it with 'Mesh' policy.
	{
		FDisplayClusterConfigurationProjection& PolicyCfg = ViewportCfg->ProjectionPolicy;
		const bool bIsSimplePolicy = PolicyCfg.Type.Equals(TEXT("simple"), ESearchCase::IgnoreCase);
		const bool bIsMeshPolicy   = PolicyCfg.Type.Equals(DisplayClusterProjectionStrings::projection::Mesh, ESearchCase::IgnoreCase);

		// Check both 'Mesh' and 'Simple' as 'Mesh' also replaces the default value
		if (bIsSimplePolicy || bIsMeshPolicy)
		{
			// 'Screen' property was the property of 'Simple' policy. We can use it as a reliable sign
			// that this instance needs to be upgraded.
			FString ScreenId;
			if (PolicyCfg.Parameters.RemoveAndCopyValue(TEXT("screen"), ScreenId))
			{
				// Rename the key. Screen name will simply become a mesh component name used in 'Mesh' policy
				PolicyCfg.Parameters.Emplace(DisplayClusterProjectionStrings::cfg::mesh::Component, MoveTemp(ScreenId));

				// Only 'Mesh' policy since 5.8
				PolicyCfg.Type = DisplayClusterProjectionStrings::projection::Mesh;
				if (PolicyCfg.Type.Len() > 0)
				{
					// Explicitly capitalize the first character to improve UI presentation.
					PolicyCfg.Type[0] = FChar::ToUpper(PolicyCfg.Type[0]);
				}
			}
		}
	}
}
