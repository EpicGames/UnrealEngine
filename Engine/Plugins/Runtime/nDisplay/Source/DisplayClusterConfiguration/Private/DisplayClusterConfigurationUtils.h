// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


class FArchive;
class UDisplayClusterConfigurationData;
class UDisplayClusterConfigurationViewport;


/**
 * Utility functions
 */
class FDisplayClusterConfigurationUtils
{
private:
	FDisplayClusterConfigurationUtils() = default;

public:
	// Returns true if we're serializing a template or archetype. False otherwise.
	static bool IsSerializingTemplate(const FArchive& Ar);

	/**
	 * Updates the config data container to the latest version. It's called internally
	 * after loading the configuration from an external source.
	 * 
	 * @param Config - Configuration data container
	 */
	static void UpdateToLatest(UDisplayClusterConfigurationData* Config);

	/**
	 * Migrates 'simple' to 'mesh' projection policy
	 * 
	 * @param ViewportCfg - Viewport configuration to modify
	 */
	static void RedirectSimpleToMeshPolicy(UDisplayClusterConfigurationViewport* ViewportCfg);
};
