// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class UDisplayClusterConfigurationData;


class IDisplayClusterConfigurationDataParser
{
public:
	virtual ~IDisplayClusterConfigurationDataParser() = default;

public:
	// Load data from a specified file
	virtual UDisplayClusterConfigurationData* LoadData(const FString& FilePath, UObject* Owner = nullptr) = 0;

	// Save data to a specified file
	virtual bool SaveData(const UDisplayClusterConfigurationData* ConfigData, const FString& FilePath) = 0;

	// Convert configuration to string
	virtual bool AsString(const UDisplayClusterConfigurationData* ConfigData, FString& OutString) = 0;
};
