// Copyright Epic Games, Inc. All Rights Reserved.

#include "Formats/JSON426/DisplayClusterConfigurationJsonParser_426.h"
#include "Formats/JSON426/DisplayClusterConfigurationJsonHelpers_426.h"

#include "Misc/DisplayClusterHelpers.h"

#include "DisplayClusterConfigurationLog.h"
#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterConfigurationStrings.h"

#include "JsonObjectConverter.h"

#include "Misc/FileHelper.h"
#include "Misc/Paths.h"


namespace JSON426
{
	UDisplayClusterConfigurationData* FDisplayClusterConfigurationJsonParser::LoadData(const FString& FilePath, UObject* Owner)
	{
		ConfigDataOwner = Owner;

		FString JsonText;

		// Load json text to the string object
		if (!FFileHelper::LoadFileToString(JsonText, *FilePath))
		{
			UE_LOGF(LogDisplayClusterConfiguration, Error, "Couldn't read file: %ls", *FilePath);
			return nullptr;
		}

		UE_LOGF(LogDisplayClusterConfiguration, Log, "nDisplay json configuration: %ls", *JsonText);

		// Parse the string object
		if (!FJsonObjectConverter::JsonObjectStringToUStruct<FDisplayClusterConfigurationJsonContainer_426>(JsonText, &JsonData, 0, 0))
		{
			UE_LOGF(LogDisplayClusterConfiguration, Error, "Couldn't deserialize json file: %ls", *FilePath);
			return nullptr;
		}

		ConfigFile = FilePath;

		// Finally, convert the data to nDisplay internal types
		return ConvertDataToInternalTypes();
	}

	bool FDisplayClusterConfigurationJsonParser::SaveData(const UDisplayClusterConfigurationData* ConfigData, const FString& FilePath)
	{
		UE_LOGF(LogDisplayClusterConfiguration, Error, "Export to JSON 426 is not supported. Use JSON 427 exporter.");
		return false;
	}

	bool FDisplayClusterConfigurationJsonParser::AsString(const UDisplayClusterConfigurationData* ConfigData, FString& OutString)
	{
		UE_LOGF(LogDisplayClusterConfiguration, Error, "Export to JSON 426 is not supported. Use JSON 427 exporter.");
		return false;
	}

	UDisplayClusterConfigurationData* FDisplayClusterConfigurationJsonParser::ConvertDataToInternalTypes()
	{
		UDisplayClusterConfigurationData* Config = UDisplayClusterConfigurationData::CreateNewConfigData(ConfigDataOwner, RF_MarkAsRootSet);
		check(Config && Config->Scene && Config->Cluster);

		// Fill metadata
		Config->Meta.ImportDataSource = EDisplayClusterConfigurationDataSource::Json;
		Config->Meta.ImportFilePath = ConfigFile;

		const FDisplayClusterConfigurationJsonNdisplay_426& CfgJson = JsonData.nDisplay;

		// Info
		Config->Info.Version = CfgJson.Version;
		Config->Info.Description = CfgJson.Description;

		// Scene
		{
			// Xforms
			for (const auto& CfgComp : CfgJson.Scene.Xforms)
			{
				UDisplayClusterConfigurationSceneComponentXform* Comp = NewObject<UDisplayClusterConfigurationSceneComponentXform>(Config, NAME_None, RF_Transactional);
				check(Comp);

				// General
				Comp->ParentId = CfgComp.Value.Parent;
				Comp->Location = FDisplayClusterConfigurationJsonVector_426::ToVector(CfgComp.Value.Location) * 100.f;
				Comp->Rotation = FDisplayClusterConfigurationJsonRotator_426::ToRotator(CfgComp.Value.Rotation);

				Config->Scene->Xforms.Emplace(CfgComp.Key, Comp);
			}

			// Screens
			for (const auto& CfgComp : CfgJson.Scene.Screens)
			{
				UDisplayClusterConfigurationSceneComponentScreen* Comp = NewObject<UDisplayClusterConfigurationSceneComponentScreen>(Config, NAME_None, RF_Transactional);
				check(Comp);

				// General
				Comp->ParentId = CfgComp.Value.Parent;
				Comp->Location = FDisplayClusterConfigurationJsonVector_426::ToVector(CfgComp.Value.Location) * 100.f;
				Comp->Rotation = FDisplayClusterConfigurationJsonRotator_426::ToRotator(CfgComp.Value.Rotation);
				// Screen specific
				Comp->Size = FDisplayClusterConfigurationJsonSizeFloat_426::ToVector(CfgComp.Value.Size) * 100.f;

				Config->Scene->Screens.Emplace(CfgComp.Key, Comp);
			}

			// Cameras
			for (const auto& CfgComp : CfgJson.Scene.Cameras)
			{
				UDisplayClusterConfigurationSceneComponentCamera* Comp = NewObject<UDisplayClusterConfigurationSceneComponentCamera>(Config, NAME_None, RF_Transactional);
				check(Comp);

				// General
				Comp->ParentId = CfgComp.Value.Parent;
				Comp->Location = FDisplayClusterConfigurationJsonVector_426::ToVector(CfgComp.Value.Location) * 100.f;
				Comp->Rotation = FDisplayClusterConfigurationJsonRotator_426::ToRotator(CfgComp.Value.Rotation);
				// Camera specific
				Comp->InterpupillaryDistance = CfgComp.Value.InterpupillaryDistance * 100.f;
				Comp->bSwapEyes = CfgComp.Value.SwapEyes;
				Comp->StereoOffset = DisplayClusterConfigurationJsonHelpers::FromString<EDisplayClusterConfigurationEyeStereoOffset>(CfgComp.Value.StereoOffset);

				Config->Scene->Cameras.Emplace(CfgComp.Key, Comp);
			}
		}

		// Cluster
		{
			// Primary node
			{
				Config->Cluster->PrimaryNode.Id = CfgJson.Cluster.MasterNode.Id;

				const uint16* ClusterSyncPort = CfgJson.Cluster.MasterNode.Ports.Find(DisplayClusterConfigurationStrings::config::cluster::ports::PortClusterSync);
				Config->Cluster->PrimaryNode.Ports.ClusterSync = (ClusterSyncPort ? *ClusterSyncPort : 41001);

				const uint16* ClusterEventsJsonPort = CfgJson.Cluster.MasterNode.Ports.Find(DisplayClusterConfigurationStrings::config::cluster::ports::PortClusterEventsJson);
				Config->Cluster->PrimaryNode.Ports.ClusterEventsJson = (ClusterEventsJsonPort ? *ClusterEventsJsonPort : 41003);

				const uint16* ClusterEventsBinaryPort = CfgJson.Cluster.MasterNode.Ports.Find(DisplayClusterConfigurationStrings::config::cluster::ports::PortClusterEventsBinary);
				Config->Cluster->PrimaryNode.Ports.ClusterEventsBinary = (ClusterEventsBinaryPort ? *ClusterEventsBinaryPort : 41004);
			}

			// Cluster sync
			{
				// Native input sync
				Config->Cluster->Sync.InputSyncPolicy.Type = CfgJson.Cluster.Sync.InputSyncPolicy.Type;
				Config->Cluster->Sync.InputSyncPolicy.Parameters = CfgJson.Cluster.Sync.InputSyncPolicy.Parameters;

				// Render sync
				Config->Cluster->Sync.RenderSyncPolicy.Type = CfgJson.Cluster.Sync.RenderSyncPolicy.Type;
				Config->Cluster->Sync.RenderSyncPolicy.Parameters = CfgJson.Cluster.Sync.RenderSyncPolicy.Parameters;
			}

			// Network
			{
				Config->Cluster->Network.ConnectRetriesAmount = DisplayClusterHelpers::map::template ExtractValueFromString(
					CfgJson.Cluster.Network, FString(DisplayClusterConfigurationStrings::config::cluster::network::NetConnectRetriesAmount), (uint32)15);

				Config->Cluster->Network.ConnectRetryDelay = DisplayClusterHelpers::map::template ExtractValueFromString(
					CfgJson.Cluster.Network, FString(DisplayClusterConfigurationStrings::config::cluster::network::NetConnectRetryDelay), (uint32)1000);

				Config->Cluster->Network.GameStartBarrierTimeout = DisplayClusterHelpers::map::template ExtractValueFromString(
					CfgJson.Cluster.Network, FString(DisplayClusterConfigurationStrings::config::cluster::network::NetGameStartBarrierTimeout), (uint32)30000);

				Config->Cluster->Network.FrameStartBarrierTimeout = DisplayClusterHelpers::map::template ExtractValueFromString(
					CfgJson.Cluster.Network, FString(DisplayClusterConfigurationStrings::config::cluster::network::NetFrameStartBarrierTimeout), (uint32)5000);

				Config->Cluster->Network.FrameEndBarrierTimeout = DisplayClusterHelpers::map::template ExtractValueFromString(
					CfgJson.Cluster.Network, FString(DisplayClusterConfigurationStrings::config::cluster::network::NetFrameEndBarrierTimeout), (uint32)5000);

				Config->Cluster->Network.RenderSyncBarrierTimeout = DisplayClusterHelpers::map::template ExtractValueFromString(
					CfgJson.Cluster.Network, FString(DisplayClusterConfigurationStrings::config::cluster::network::NetRenderSyncBarrierTimeout), (uint32)5000);
			}

			// Cluster nodes
			for (const auto& CfgNode : CfgJson.Cluster.Nodes)
			{
				UDisplayClusterConfigurationClusterNode* Node = NewObject<UDisplayClusterConfigurationClusterNode>(Config->Cluster, *CfgNode.Key, RF_Transactional);
				check(Node);

				// Base parameters
				Node->Host = CfgNode.Value.Host;
				Node->bIsSoundEnabled = CfgNode.Value.Sound;
				Node->bIsFullscreen = CfgNode.Value.FullScreen;
				Node->WindowRect = FDisplayClusterConfigurationRectangle(CfgNode.Value.Window.X, CfgNode.Value.Window.Y, CfgNode.Value.Window.W, CfgNode.Value.Window.H);

				// Viewports
				for (const auto& CfgViewport : CfgNode.Value.Viewports)
				{
					UDisplayClusterConfigurationViewport* Viewport = NewObject<UDisplayClusterConfigurationViewport>(Node, *CfgViewport.Key, RF_Transactional | RF_ArchetypeObject | RF_Public);
					check(Viewport);

					// Base parameters
					Viewport->RenderSettings.BufferRatio = CfgViewport.Value.BufferRatio;
					Viewport->Camera = CfgViewport.Value.Camera;
					Viewport->Region = FDisplayClusterConfigurationRectangle(CfgViewport.Value.Region.X, CfgViewport.Value.Region.Y, CfgViewport.Value.Region.W, CfgViewport.Value.Region.H);
					Viewport->GPUIndex = CfgViewport.Value.GPUIndex;

					// Projection policy
					Viewport->ProjectionPolicy.Type = CfgViewport.Value.ProjectionPolicy.Type;
					Viewport->ProjectionPolicy.Parameters = CfgViewport.Value.ProjectionPolicy.Parameters;

					// Add this viewport
					Node->Viewports.Emplace(CfgViewport.Key, Viewport);
				}

				// Postprocess
				for (const auto& CfgPostprocess : CfgNode.Value.Postprocess)
				{
					FDisplayClusterConfigurationPostprocess PostprocessOperation;

					PostprocessOperation.Type = CfgPostprocess.Value.Type;
					PostprocessOperation.Parameters = CfgPostprocess.Value.Parameters;

					Node->Postprocess.Emplace(CfgPostprocess.Key, PostprocessOperation);
				}

				// Store new cluster node
				Config->Cluster->Nodes.Emplace(CfgNode.Key, Node);
			}
		}

		// Custom parameters
		Config->CustomParameters = CfgJson.CustomParameters;

		// Diagnostics
		Config->Diagnostics.bSimulateLag = CfgJson.Diagnostics.SimulateLag;
		Config->Diagnostics.MinLagTime = CfgJson.Diagnostics.MinLagTime;
		Config->Diagnostics.MaxLagTime = CfgJson.Diagnostics.MaxLagTime;

		return Config;
	}

	bool FDisplayClusterConfigurationJsonParser::ConvertDataToExternalTypes(const UDisplayClusterConfigurationData* Config)
	{
		// Not supported anymore
		return false;
	}
}
