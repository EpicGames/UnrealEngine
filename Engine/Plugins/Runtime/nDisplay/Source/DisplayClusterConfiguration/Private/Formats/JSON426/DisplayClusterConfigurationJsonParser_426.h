// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Formats/IDisplayClusterConfigurationDataParser.h"
#include "Formats/JSON426/DisplayClusterConfigurationJsonTypes_426.h"

class UDisplayClusterConfigurationData;


namespace JSON426
{
	/**
	 * Config parser for JSON based config files
	 */
	class FDisplayClusterConfigurationJsonParser
		: public IDisplayClusterConfigurationDataParser
	{
	public:
		FDisplayClusterConfigurationJsonParser()  = default;
		~FDisplayClusterConfigurationJsonParser() = default;

	public:
		// Load data from a specified file
		virtual UDisplayClusterConfigurationData* LoadData(const FString& FilePath, UObject* Owner = nullptr) override;

		// Save data to a specified file
		virtual bool SaveData(const UDisplayClusterConfigurationData* ConfigData, const FString& FilePath) override;

		// Convert configuration to string
		virtual bool AsString(const UDisplayClusterConfigurationData* ConfigData, FString& OutString) override;

	protected:
		// [import data] Fill generic data container with parsed information
		UDisplayClusterConfigurationData* ConvertDataToInternalTypes();
		// [export data] Extract data from generic container to specific containers
		bool ConvertDataToExternalTypes(const UDisplayClusterConfigurationData* InData);

	private:
		// Intermediate temporary data (Json types)
		FDisplayClusterConfigurationJsonContainer_426 JsonData;
		// Source file
		FString ConfigFile;
		// Owner for config data UObject we'll create on success
		UObject* ConfigDataOwner = nullptr;
	};
}
