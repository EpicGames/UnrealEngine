// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DisplayClusterConfigurationStrings.h"
#include "DisplayClusterConfigurationJsonTypes_426.generated.h"


USTRUCT()
struct FDisplayClusterConfigurationJsonRectangle_426
{
	GENERATED_BODY()

public:
	FDisplayClusterConfigurationJsonRectangle_426()
		: X(0), Y(0), W(0), H(0)
	{ }

	FDisplayClusterConfigurationJsonRectangle_426(int32 _X, int32 _Y, int32 _W, int32 _H)
		: X(_X), Y(_Y), W(_W), H(_H)
	{ }

public:
	UPROPERTY()
	int32 X;

	UPROPERTY()
	int32 Y;

	UPROPERTY()
	int32 W;

	UPROPERTY()
	int32 H;
};


USTRUCT()
struct FDisplayClusterConfigurationJsonVector_426
{
	GENERATED_BODY()

public:
	FDisplayClusterConfigurationJsonVector_426()
		: FDisplayClusterConfigurationJsonVector_426(0.f, 0.f, 0.f)
	{ }

	FDisplayClusterConfigurationJsonVector_426(float _X, float _Y, float _Z)
		: X(_X), Y(_Y), Z(_Z)
	{ }

	FDisplayClusterConfigurationJsonVector_426(const FVector& Vector)
		: FDisplayClusterConfigurationJsonVector_426(Vector.X, Vector.Y, Vector.Z)
	{ }

public:
	UPROPERTY()
	float X;

	UPROPERTY()
	float Y;

	UPROPERTY()
	float Z;

public:
	static FVector ToVector(const FDisplayClusterConfigurationJsonVector_426& Data)
	{
		return FVector(Data.X, Data.Y, Data.Z);
	}

	static FDisplayClusterConfigurationJsonVector_426 FromVector(const FVector& Vector)
	{
		return FDisplayClusterConfigurationJsonVector_426(Vector.X, Vector.Y, Vector.Z);
	}
};


USTRUCT()
struct FDisplayClusterConfigurationJsonRotator_426
{
	GENERATED_BODY()

public:
	FDisplayClusterConfigurationJsonRotator_426()
		: FDisplayClusterConfigurationJsonRotator_426(0.f, 0.f, 0.f)
	{ }

	FDisplayClusterConfigurationJsonRotator_426(float P, float Y, float R)
		: Pitch(P), Yaw(Y), Roll(R)
	{ }

	FDisplayClusterConfigurationJsonRotator_426(const FRotator& Rotator)
		: FDisplayClusterConfigurationJsonRotator_426(Rotator.Pitch, Rotator.Yaw, Rotator.Roll)
	{ }

public:
	UPROPERTY()
	float Pitch;

	UPROPERTY()
	float Yaw;

	UPROPERTY()
	float Roll;

public:
	static FRotator ToRotator(const FDisplayClusterConfigurationJsonRotator_426& Data)
	{
		return FRotator(Data.Pitch, Data.Yaw, Data.Roll);
	}

	static FDisplayClusterConfigurationJsonRotator_426 FromRotator(const FRotator& Rotator)
	{
		return FDisplayClusterConfigurationJsonRotator_426(Rotator.Pitch, Rotator.Yaw, Rotator.Roll);
	}
};


USTRUCT()
struct FDisplayClusterConfigurationJsonSizeInt_426
{
	GENERATED_BODY()

public:
	FDisplayClusterConfigurationJsonSizeInt_426()
		: FDisplayClusterConfigurationJsonSizeInt_426(0, 0)
	{ }

	FDisplayClusterConfigurationJsonSizeInt_426(int W, int H)
		: Width(W), Height(H)
	{ }

public:
	UPROPERTY()
	int Width;

	UPROPERTY()
	int Height;

public:
	static FIntPoint ToPoint(const FDisplayClusterConfigurationJsonSizeInt_426& Data)
	{
		return FIntPoint(Data.Width, Data.Height);
	}

	static FDisplayClusterConfigurationJsonSizeInt_426 FromPoint(const FIntPoint& Point)
	{
		return FDisplayClusterConfigurationJsonSizeInt_426(Point.X, Point.Y);
	}
};


USTRUCT()
struct FDisplayClusterConfigurationJsonSizeFloat_426
{
	GENERATED_BODY()

public:
	FDisplayClusterConfigurationJsonSizeFloat_426()
		: FDisplayClusterConfigurationJsonSizeFloat_426(0.f, 0.f)
	{ }

	FDisplayClusterConfigurationJsonSizeFloat_426(float W, float H)
		: Width(W), Height(H)
	{ }

	FDisplayClusterConfigurationJsonSizeFloat_426(const FVector2D& Vector)
		: FDisplayClusterConfigurationJsonSizeFloat_426(Vector.X, Vector.Y)
	{ }

public:
	UPROPERTY()
	float Width;

	UPROPERTY()
	float Height;

public:
	static FVector2D ToVector(const FDisplayClusterConfigurationJsonSizeFloat_426& Data)
	{
		return FVector2D(Data.Width, Data.Height);
	}

	static FDisplayClusterConfigurationJsonSizeFloat_426 FromVector(const FVector2D& Vector)
	{
		return FDisplayClusterConfigurationJsonSizeFloat_426(Vector.X, Vector.Y);
	}
};


USTRUCT()
struct FDisplayClusterConfigurationJsonPolymorphicEntity_426
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FString Type;

	UPROPERTY()
	TMap<FString, FString> Parameters;
};


USTRUCT()
struct FDisplayClusterConfigurationJsonSceneComponent_426
{
	GENERATED_BODY()
};

USTRUCT()
struct FDisplayClusterConfigurationJsonSceneComponentXform_426
	: public FDisplayClusterConfigurationJsonSceneComponent_426
{
	GENERATED_BODY()

public:
	FDisplayClusterConfigurationJsonSceneComponentXform_426()
		: FDisplayClusterConfigurationJsonSceneComponentXform_426(FString(), FVector::ZeroVector, FRotator::ZeroRotator)
	{ }

	FDisplayClusterConfigurationJsonSceneComponentXform_426(const FString& InParent, const FVector& InLocation, const FRotator& InRotation)
		: Parent(InParent)
		, Location(InLocation)
		, Rotation(InRotation)
	{ }

public:
	UPROPERTY()
	FString Parent;

	UPROPERTY()
	FDisplayClusterConfigurationJsonVector_426 Location;

	UPROPERTY()
	FDisplayClusterConfigurationJsonRotator_426 Rotation;
};

USTRUCT()
struct FDisplayClusterConfigurationJsonSceneComponentCamera_426
	: public FDisplayClusterConfigurationJsonSceneComponentXform_426
{
	GENERATED_BODY()

public:
	FDisplayClusterConfigurationJsonSceneComponentCamera_426()
		: FDisplayClusterConfigurationJsonSceneComponentCamera_426(FString(), FVector::ZeroVector, FRotator::ZeroRotator, 0.064f, false, DisplayClusterConfigurationStrings::config::scene::camera::CameraStereoOffsetNone)
	{ }

	FDisplayClusterConfigurationJsonSceneComponentCamera_426(const FString& InParent, const FVector& InLocation, const FRotator& InRotation, float InInterpupillaryDistance, bool bInSwapEyes, FString InStereoOffset)
		: FDisplayClusterConfigurationJsonSceneComponentXform_426(InParent, InLocation, InRotation)
		, InterpupillaryDistance(InInterpupillaryDistance)
		, SwapEyes(bInSwapEyes)
		, StereoOffset(InStereoOffset)
	{ }

public:
	UPROPERTY()
	float InterpupillaryDistance;

	UPROPERTY()
	bool SwapEyes;

	UPROPERTY()
	FString StereoOffset;
};

USTRUCT()
struct FDisplayClusterConfigurationJsonSceneComponentScreen_426
	: public FDisplayClusterConfigurationJsonSceneComponentXform_426
{
	GENERATED_BODY()

public:
	FDisplayClusterConfigurationJsonSceneComponentScreen_426()
		: FDisplayClusterConfigurationJsonSceneComponentScreen_426(FString(), FVector::ZeroVector, FRotator::ZeroRotator, FVector2D::ZeroVector)
	{ }

	FDisplayClusterConfigurationJsonSceneComponentScreen_426(const FString& InParent, const FVector& InLocation, const FRotator& InRotation, const FVector2D& InSize)
		: FDisplayClusterConfigurationJsonSceneComponentXform_426(InParent, InLocation, InRotation)
		, Size(InSize)
	{ }

public:
	UPROPERTY()
	FDisplayClusterConfigurationJsonSizeFloat_426 Size;
};

USTRUCT()
struct FDisplayClusterConfigurationJsonScene_426
{
	GENERATED_BODY()

public:
	UPROPERTY()
	TMap<FString, FDisplayClusterConfigurationJsonSceneComponentXform_426> Xforms;

	UPROPERTY()
	TMap<FString, FDisplayClusterConfigurationJsonSceneComponentCamera_426> Cameras;

	UPROPERTY()
	TMap<FString, FDisplayClusterConfigurationJsonSceneComponentScreen_426> Screens;
};

USTRUCT()
struct FDisplayClusterConfigurationJsonPrimaryNode_426
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FString Id;

	UPROPERTY()
	TMap<FString, uint16> Ports;
};

USTRUCT()
struct FDisplayClusterConfigurationJsonClusterSyncPolicy_426
	: public FDisplayClusterConfigurationJsonPolymorphicEntity_426
{
	GENERATED_BODY()
};

USTRUCT()
struct FDisplayClusterConfigurationJsonClusterSync_426
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FDisplayClusterConfigurationJsonClusterSyncPolicy_426 RenderSyncPolicy;

	UPROPERTY()
	FDisplayClusterConfigurationJsonClusterSyncPolicy_426 InputSyncPolicy;
};

USTRUCT()
struct FDisplayClusterConfigurationJsonPostprocess_426
	: public FDisplayClusterConfigurationJsonPolymorphicEntity_426
{
	GENERATED_BODY()
};

USTRUCT()
struct FDisplayClusterConfigurationJsonProjectionPolicy_426
	: public FDisplayClusterConfigurationJsonPolymorphicEntity_426
{
	GENERATED_BODY()
};

USTRUCT()
struct FDisplayClusterConfigurationJsonViewport_426
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FString Camera;

	UPROPERTY()
	float BufferRatio = 1.0f;

	UPROPERTY()
	int32 GPUIndex = 0;

	UPROPERTY()
	bool AllowCrossGPUTransfer = false;

	UPROPERTY()
	bool IsShared = false;

	UPROPERTY()
	FDisplayClusterConfigurationJsonRectangle_426 Region;

	UPROPERTY()
	FDisplayClusterConfigurationJsonProjectionPolicy_426 ProjectionPolicy;
};

USTRUCT()
struct FDisplayClusterConfigurationJsonClusterNode_426
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FString Host;

	UPROPERTY()
	bool Sound = true;

	UPROPERTY()
	bool FullScreen = false;

	UPROPERTY()
	FDisplayClusterConfigurationJsonRectangle_426 Window;

	UPROPERTY()
	TMap<FString, FDisplayClusterConfigurationJsonPostprocess_426> Postprocess;

	UPROPERTY()
	TMap<FString, FDisplayClusterConfigurationJsonViewport_426> Viewports;
};

USTRUCT()
struct FDisplayClusterConfigurationJsonCluster_426
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FDisplayClusterConfigurationJsonPrimaryNode_426 MasterNode;

	UPROPERTY()
	FDisplayClusterConfigurationJsonClusterSync_426 Sync;

	UPROPERTY()
	TMap<FString, FString> Network;

	UPROPERTY()
	TMap<FString, FDisplayClusterConfigurationJsonClusterNode_426> Nodes;
};

USTRUCT()
struct FDisplayClusterConfigurationJsonDiagnostics_426
{
	GENERATED_BODY()

public:
	UPROPERTY()
	bool SimulateLag = false;

	UPROPERTY()
	float MinLagTime = 0.01f;

	UPROPERTY()
	float MaxLagTime = 0.3f;
};


// "nDisplay" category structure
USTRUCT()
struct FDisplayClusterConfigurationJsonNdisplay_426
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FString Description;

	UPROPERTY()
	FString Version;

	UPROPERTY()
	FString AssetPath;

	UPROPERTY()
	FDisplayClusterConfigurationJsonScene_426 Scene;

	UPROPERTY()
	FDisplayClusterConfigurationJsonCluster_426 Cluster;

	UPROPERTY()
	TMap<FString, FString> CustomParameters;

	UPROPERTY()
	FDisplayClusterConfigurationJsonDiagnostics_426 Diagnostics;
};

// The main nDisplay configuration structure. It's supposed to extract nDisplay related data from a collecting JSON file.
USTRUCT()
struct FDisplayClusterConfigurationJsonContainer_426
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FDisplayClusterConfigurationJsonNdisplay_426 nDisplay;
};
