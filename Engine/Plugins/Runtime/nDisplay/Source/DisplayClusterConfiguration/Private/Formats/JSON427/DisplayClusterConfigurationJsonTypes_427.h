// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DisplayClusterConfigurationStrings.h"
#include "DisplayClusterConfigurationJsonTypes_427.generated.h"


USTRUCT()
struct FDisplayClusterConfigurationJsonRectangle_427
{
	GENERATED_BODY()

public:
	FDisplayClusterConfigurationJsonRectangle_427()
		: X(0), Y(0), W(0), H(0)
	{ }

	FDisplayClusterConfigurationJsonRectangle_427(int32 _X, int32 _Y, int32 _W, int32 _H)
		: X(_X), Y(_Y), W(_W), H(_H)
	{ }

public:
	UPROPERTY()
	int32 X;

	UPROPERTY()
	int32 Y;

	UPROPERTY()
	int32 W;

	UPROPERTY()
	int32 H;
};


USTRUCT()
struct FDisplayClusterConfigurationJsonVector_427
{
	GENERATED_BODY()

public:
	FDisplayClusterConfigurationJsonVector_427()
		: FDisplayClusterConfigurationJsonVector_427(0.f, 0.f, 0.f)
	{ }

	FDisplayClusterConfigurationJsonVector_427(float _X, float _Y, float _Z)
		: X(_X), Y(_Y), Z(_Z)
	{ }

	FDisplayClusterConfigurationJsonVector_427(const FVector& Vector)
		: FDisplayClusterConfigurationJsonVector_427(Vector.X, Vector.Y, Vector.Z)
	{ }

public:
	UPROPERTY()
	float X;

	UPROPERTY()
	float Y;

	UPROPERTY()
	float Z;

public:
	static FVector ToVector(const FDisplayClusterConfigurationJsonVector_427& Data)
	{
		return FVector(Data.X, Data.Y, Data.Z);
	}

	static FDisplayClusterConfigurationJsonVector_427 FromVector(const FVector& Vector)
	{
		return FDisplayClusterConfigurationJsonVector_427(Vector.X, Vector.Y, Vector.Z);
	}
};


USTRUCT()
struct FDisplayClusterConfigurationJsonRotator_427
{
	GENERATED_BODY()

public:
	FDisplayClusterConfigurationJsonRotator_427()
		: FDisplayClusterConfigurationJsonRotator_427(0.f, 0.f, 0.f)
	{ }

	FDisplayClusterConfigurationJsonRotator_427(float P, float Y, float R)
		: Pitch(P), Yaw(Y), Roll(R)
	{ }

	FDisplayClusterConfigurationJsonRotator_427(const FRotator& Rotator)
		: FDisplayClusterConfigurationJsonRotator_427(Rotator.Pitch, Rotator.Yaw, Rotator.Roll)
	{ }

public:
	UPROPERTY()
	float Pitch;

	UPROPERTY()
	float Yaw;

	UPROPERTY()
	float Roll;

public:
	static FRotator ToRotator(const FDisplayClusterConfigurationJsonRotator_427& Data)
	{
		return FRotator(Data.Pitch, Data.Yaw, Data.Roll);
	}

	static FDisplayClusterConfigurationJsonRotator_427 FromRotator(const FRotator& Rotator)
	{
		return FDisplayClusterConfigurationJsonRotator_427(Rotator.Pitch, Rotator.Yaw, Rotator.Roll);
	}
};


USTRUCT()
struct FDisplayClusterConfigurationJsonSizeInt_427
{
	GENERATED_BODY()

public:
	FDisplayClusterConfigurationJsonSizeInt_427()
		: FDisplayClusterConfigurationJsonSizeInt_427(0, 0)
	{ }

	FDisplayClusterConfigurationJsonSizeInt_427(int W, int H)
		: Width(W), Height(H)
	{ }

public:
	UPROPERTY()
	int Width;

	UPROPERTY()
	int Height;

public:
	static FIntPoint ToPoint(const FDisplayClusterConfigurationJsonSizeInt_427& Data)
	{
		return FIntPoint(Data.Width, Data.Height);
	}

	static FDisplayClusterConfigurationJsonSizeInt_427 FromPoint(const FIntPoint& Point)
	{
		return FDisplayClusterConfigurationJsonSizeInt_427(Point.X, Point.Y);
	}
};


USTRUCT()
struct FDisplayClusterConfigurationJsonSizeFloat_427
{
	GENERATED_BODY()

public:
	FDisplayClusterConfigurationJsonSizeFloat_427()
		: FDisplayClusterConfigurationJsonSizeFloat_427(0.f, 0.f)
	{ }

	FDisplayClusterConfigurationJsonSizeFloat_427(float W, float H)
		: Width(W), Height(H)
	{ }

	FDisplayClusterConfigurationJsonSizeFloat_427(const FVector2D& Vector)
		: FDisplayClusterConfigurationJsonSizeFloat_427(Vector.X, Vector.Y)
	{ }

public:
	UPROPERTY()
	float Width;

	UPROPERTY()
	float Height;

public:
	static FVector2D ToVector(const FDisplayClusterConfigurationJsonSizeFloat_427& Data)
	{
		return FVector2D(Data.Width, Data.Height);
	}

	static FDisplayClusterConfigurationJsonSizeFloat_427 FromVector(const FVector2D& Vector)
	{
		return FDisplayClusterConfigurationJsonSizeFloat_427(Vector.X, Vector.Y);
	}
};


USTRUCT()
struct FDisplayClusterConfigurationJsonPolymorphicEntity_427
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FString Type;

	UPROPERTY()
	TMap<FString, FString> Parameters;
};


USTRUCT()
struct FDisplayClusterConfigurationJsonMisc_427
{
	GENERATED_BODY()

public:
	FDisplayClusterConfigurationJsonMisc_427()
		: bFollowLocalPlayerCamera(false)
		, bExitOnEsc(true)
	{ }

public:
	UPROPERTY()
	bool bFollowLocalPlayerCamera;

	UPROPERTY()
	bool bExitOnEsc;
};


USTRUCT()
struct FDisplayClusterConfigurationJsonSceneComponent_427
{
	GENERATED_BODY()
};


USTRUCT()
struct FDisplayClusterConfigurationJsonSceneComponentXform_427
	: public FDisplayClusterConfigurationJsonSceneComponent_427
{
	GENERATED_BODY()

public:
	FDisplayClusterConfigurationJsonSceneComponentXform_427()
		: FDisplayClusterConfigurationJsonSceneComponentXform_427(FString(), FVector::ZeroVector, FRotator::ZeroRotator)
	{ }

	FDisplayClusterConfigurationJsonSceneComponentXform_427(const FString& InParentId, const FVector& InLocation, const FRotator& InRotation)
		: ParentId(InParentId)
		, Location(InLocation)
		, Rotation(InRotation)
	{ }

public:
	UPROPERTY()
	FString ParentId;

	UPROPERTY()
	FDisplayClusterConfigurationJsonVector_427 Location;

	UPROPERTY()
	FDisplayClusterConfigurationJsonRotator_427 Rotation;
};


USTRUCT()
struct FDisplayClusterConfigurationJsonSceneComponentCamera_427
	: public FDisplayClusterConfigurationJsonSceneComponentXform_427
{
	GENERATED_BODY()

public:
	FDisplayClusterConfigurationJsonSceneComponentCamera_427()
		: FDisplayClusterConfigurationJsonSceneComponentCamera_427(FString(), FVector::ZeroVector, FRotator::ZeroRotator, 6.4f, false, DisplayClusterConfigurationStrings::config::scene::camera::CameraStereoOffsetNone)
	{ }

	FDisplayClusterConfigurationJsonSceneComponentCamera_427(const FString& InParentId, const FVector& InLocation, const FRotator& InRotation, float InInterpupillaryDistance, bool bInSwapEyes, const FString& InStereoOffset)
		: FDisplayClusterConfigurationJsonSceneComponentXform_427(InParentId, InLocation, InRotation)
		, InterpupillaryDistance(InInterpupillaryDistance)
		, SwapEyes(bInSwapEyes)
		, StereoOffset(InStereoOffset)
	{ }

public:
	UPROPERTY()
	float InterpupillaryDistance;

	UPROPERTY()
	bool SwapEyes;

	UPROPERTY()
	FString StereoOffset;
};


USTRUCT()
struct FDisplayClusterConfigurationJsonSceneComponentScreen_427
	: public FDisplayClusterConfigurationJsonSceneComponentXform_427
{
	GENERATED_BODY()

public:
	FDisplayClusterConfigurationJsonSceneComponentScreen_427()
		: FDisplayClusterConfigurationJsonSceneComponentScreen_427(FString(), FVector::ZeroVector, FRotator::ZeroRotator, FVector2D(1.f, 1.f))
	{ }

	FDisplayClusterConfigurationJsonSceneComponentScreen_427(const FString& InParentId, const FVector& InLocation, const FRotator& InRotation, const FVector2D& InSize)
		: FDisplayClusterConfigurationJsonSceneComponentXform_427(InParentId, InLocation, InRotation)
		, Size(InSize)
	{ }

public:
	UPROPERTY()
	FDisplayClusterConfigurationJsonSizeFloat_427 Size;
};


USTRUCT()
struct FDisplayClusterConfigurationJsonScene_427
{
	GENERATED_BODY()

public:
	UPROPERTY()
	TMap<FString, FDisplayClusterConfigurationJsonSceneComponentXform_427> Xforms;

	UPROPERTY()
	TMap<FString, FDisplayClusterConfigurationJsonSceneComponentCamera_427> Cameras;

	UPROPERTY()
	TMap<FString, FDisplayClusterConfigurationJsonSceneComponentScreen_427> Screens;
};


USTRUCT()
struct FDisplayClusterConfigurationJsonPrimaryNode_427
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FString Id;

	UPROPERTY()
	TMap<FString, uint16> Ports;
};

USTRUCT()
struct FDisplayClusterConfigurationJsonClusterSyncPolicy_427
	: public FDisplayClusterConfigurationJsonPolymorphicEntity_427
{
	GENERATED_BODY()
};

USTRUCT()
struct FDisplayClusterConfigurationJsonClusterSync_427
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FDisplayClusterConfigurationJsonClusterSyncPolicy_427 RenderSyncPolicy;

	UPROPERTY()
	FDisplayClusterConfigurationJsonClusterSyncPolicy_427 InputSyncPolicy;
};

USTRUCT()
struct FDisplayClusterConfigurationJsonPostprocess_427
	: public FDisplayClusterConfigurationJsonPolymorphicEntity_427
{
	GENERATED_BODY()
};

USTRUCT()
struct FDisplayClusterConfigurationJsonProjectionPolicy_427
	: public FDisplayClusterConfigurationJsonPolymorphicEntity_427
{
	GENERATED_BODY()
};

USTRUCT()
struct FDisplayClusterConfigurationJsonOverscan_427
{
	GENERATED_BODY()

public:
	UPROPERTY()
	bool bEnabled = false;

	UPROPERTY()
	FString Mode = "";

	UPROPERTY()
	int32 Left = 0;

	UPROPERTY()
	int32 Right = 0;

	UPROPERTY()
	int32 Top = 0;

	UPROPERTY()
	int32 Bottom = 0;

	UPROPERTY()
	bool Oversize = 0;
};

USTRUCT()
struct FDisplayClusterConfigurationJsonViewport_427
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FString Camera;

	UPROPERTY()
	float BufferRatio = 1.0f;

	UPROPERTY()
	int32 GPUIndex = 0;

	UPROPERTY()
	bool AllowCrossGPUTransfer = false;

	UPROPERTY()
	bool IsShared = false;

	UPROPERTY()
	FDisplayClusterConfigurationJsonOverscan_427 Overscan;

	UPROPERTY()
	FDisplayClusterConfigurationJsonRectangle_427 Region;

	UPROPERTY()
	FDisplayClusterConfigurationJsonProjectionPolicy_427 ProjectionPolicy;
};

USTRUCT()
struct FDisplayClusterConfigurationFramePostProcess_OutputRemap_427
{
	GENERATED_BODY()

public:
	UPROPERTY()
	bool bEnable = false;

	UPROPERTY()
	FString DataSource;

	UPROPERTY()
	FString StaticMeshAsset;

	UPROPERTY()
	FString ExternalFile;
};

USTRUCT()
struct FDisplayClusterConfigurationJsonClusterNode_427
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FString Host;

	UPROPERTY()
	bool Sound = true;

	UPROPERTY()
	bool FullScreen = false;

	UPROPERTY()
	FDisplayClusterConfigurationJsonRectangle_427 Window;

	UPROPERTY()
	TMap<FString, FDisplayClusterConfigurationJsonPostprocess_427> Postprocess;

	UPROPERTY()
	TMap<FString, FDisplayClusterConfigurationJsonViewport_427> Viewports;

	UPROPERTY()
	FDisplayClusterConfigurationFramePostProcess_OutputRemap_427 OutputRemap;
};

USTRUCT()
struct FDisplayClusterConfigurationJsonCluster_427
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FDisplayClusterConfigurationJsonPrimaryNode_427 MasterNode;

	UPROPERTY()
	FDisplayClusterConfigurationJsonClusterSync_427 Sync;

	UPROPERTY()
	TMap<FString, FString> Network;

	UPROPERTY()
	TMap<FString, FDisplayClusterConfigurationJsonClusterNode_427> Nodes;
};

USTRUCT()
struct FDisplayClusterConfigurationJsonDiagnostics_427
{
	GENERATED_BODY()

public:
	UPROPERTY()
	bool SimulateLag = false;

	UPROPERTY()
	float MinLagTime = 0.01f;

	UPROPERTY()
	float MaxLagTime = 0.3f;
};


// "nDisplay" category structure
USTRUCT()
struct FDisplayClusterConfigurationJsonNdisplay_427
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FString Description;

	UPROPERTY()
	FString Version;

	UPROPERTY()
	FString AssetPath;

	UPROPERTY()
	FDisplayClusterConfigurationJsonMisc_427 Misc;

	UPROPERTY()
	FDisplayClusterConfigurationJsonScene_427 Scene;

	UPROPERTY()
	FDisplayClusterConfigurationJsonCluster_427 Cluster;

	UPROPERTY()
	TMap<FString, FString> CustomParameters;

	UPROPERTY()
	FDisplayClusterConfigurationJsonDiagnostics_427 Diagnostics;
};

// The main nDisplay configuration structure. It's supposed to extract nDisplay related data from a collecting JSON file.
USTRUCT()
struct FDisplayClusterConfigurationJsonContainer_427
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FDisplayClusterConfigurationJsonNdisplay_427 nDisplay;
};
