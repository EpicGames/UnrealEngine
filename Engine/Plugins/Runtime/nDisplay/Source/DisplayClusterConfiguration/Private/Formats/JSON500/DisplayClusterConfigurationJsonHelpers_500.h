// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DisplayClusterConfigurationLog.h"
#include "DisplayClusterConfigurationStrings.h"
#include "DisplayClusterConfigurationTypes.h"


namespace JSON500
{
	// since we cannot modify public headers we need to put those string in private headers temporary
	namespace config
	{
		namespace node
		{
			namespace viewport
			{
				namespace overscan
				{
					static constexpr const TCHAR* OverscanNone = TEXT("none");
					static constexpr const TCHAR* OverscanPixels = TEXT("pixels");
					static constexpr const TCHAR* OverscanPercent = TEXT("percent");
				}
			}
		}
	}

	/**
	 * Auxiliary class with different type conversion functions
	 */
	class DisplayClusterConfigurationJsonHelpers
	{
	public:
		//////////////////////////////////////////////////////////////////////////////////////////////
		// TYPE --> STRING
		//////////////////////////////////////////////////////////////////////////////////////////////
		template <typename ConvertFrom>
		static FString ToString(const ConvertFrom& From);

		//////////////////////////////////////////////////////////////////////////////////////////////
		// STRING --> TYPE
		//////////////////////////////////////////////////////////////////////////////////////////////
		template <typename ConvertTo>
		static ConvertTo FromString(const FString& From);
	};


	// EDisplayClusterConfigurationEyeStereoOffset
	template <>
	inline FString DisplayClusterConfigurationJsonHelpers::ToString<>(const EDisplayClusterConfigurationEyeStereoOffset& From)
	{
		switch (From)
		{
		case EDisplayClusterConfigurationEyeStereoOffset::None:
			return DisplayClusterConfigurationStrings::config::scene::camera::CameraStereoOffsetNone;

		case EDisplayClusterConfigurationEyeStereoOffset::Left:
			return DisplayClusterConfigurationStrings::config::scene::camera::CameraStereoOffsetLeft;

		case EDisplayClusterConfigurationEyeStereoOffset::Right:
			return DisplayClusterConfigurationStrings::config::scene::camera::CameraStereoOffsetRight;

		default:
			UE_LOGF(LogDisplayClusterConfiguration, Error, "Unexpected camera stereo offset type");
			break;
		}

		return DisplayClusterConfigurationStrings::config::scene::camera::CameraStereoOffsetNone;
	}

	template <>
	inline EDisplayClusterConfigurationEyeStereoOffset DisplayClusterConfigurationJsonHelpers::FromString<>(const FString& From)
	{
		EDisplayClusterConfigurationEyeStereoOffset Result = EDisplayClusterConfigurationEyeStereoOffset::None;

		if (From.Equals(DisplayClusterConfigurationStrings::config::scene::camera::CameraStereoOffsetNone, ESearchCase::IgnoreCase))
		{
			Result = EDisplayClusterConfigurationEyeStereoOffset::None;
		}
		else if (From.Equals(DisplayClusterConfigurationStrings::config::scene::camera::CameraStereoOffsetLeft, ESearchCase::IgnoreCase))
		{
			Result = EDisplayClusterConfigurationEyeStereoOffset::Left;
		}
		else if (From.Equals(DisplayClusterConfigurationStrings::config::scene::camera::CameraStereoOffsetRight, ESearchCase::IgnoreCase))
		{
			Result = EDisplayClusterConfigurationEyeStereoOffset::Right;
		}

		return Result;
	}

	// EDisplayClusterConfigurationViewportOverscanMode
	template <>
	inline FString DisplayClusterConfigurationJsonHelpers::ToString<>(const EDisplayClusterConfigurationViewportOverscanMode& InOverscanMode)
	{
		switch (InOverscanMode)
		{
		case EDisplayClusterConfigurationViewportOverscanMode::Pixels:
			return JSON500::config::node::viewport::overscan::OverscanPixels;

		case EDisplayClusterConfigurationViewportOverscanMode::Percent:
			return JSON500::config::node::viewport::overscan::OverscanPercent;

		default:
			UE_LOGF(LogDisplayClusterConfiguration, Error, "Unexpected overscan type");
			break;
		}

		return JSON500::config::node::viewport::overscan::OverscanNone;
	}

	template <>
	inline EDisplayClusterConfigurationViewportOverscanMode DisplayClusterConfigurationJsonHelpers::FromString<>(const FString& InOverscanModeString)
	{
		EDisplayClusterConfigurationViewportOverscanMode Result = EDisplayClusterConfigurationViewportOverscanMode::Percent;

		if (InOverscanModeString.Equals(JSON500::config::node::viewport::overscan::OverscanPixels, ESearchCase::IgnoreCase))
		{
			Result = EDisplayClusterConfigurationViewportOverscanMode::Pixels;
		}
		else if (InOverscanModeString.Equals(JSON500::config::node::viewport::overscan::OverscanPercent, ESearchCase::IgnoreCase))
		{
			Result = EDisplayClusterConfigurationViewportOverscanMode::Percent;
		}

		return Result;
	}
}