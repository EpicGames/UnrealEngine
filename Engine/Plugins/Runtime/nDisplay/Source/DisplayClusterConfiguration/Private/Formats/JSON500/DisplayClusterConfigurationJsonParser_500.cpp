// Copyright Epic Games, Inc. All Rights Reserved.

#include "Formats/JSON500/DisplayClusterConfigurationJsonParser_500.h"
#include "Formats/JSON500/DisplayClusterConfigurationJsonHelpers_500.h"

#include "Misc/DisplayClusterHelpers.h"

#include "DisplayClusterConfigurationLog.h"
#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterConfigurationStrings.h"

#include "JsonObjectConverter.h"

#include "Misc/FileHelper.h"
#include "Misc/Paths.h"


namespace JSON500
{
	UDisplayClusterConfigurationData* FDisplayClusterConfigurationJsonParser::LoadData(const FString& FilePath, UObject* Owner)
	{
		ConfigDataOwner = Owner;

		FString JsonText;

		// Load json text to the string object
		if (!FFileHelper::LoadFileToString(JsonText, *FilePath))
		{
			UE_LOGF(LogDisplayClusterConfiguration, Error, "Couldn't read file: %ls", *FilePath);
			return nullptr;
		}

		UE_LOGF(LogDisplayClusterConfiguration, Log, "nDisplay json configuration: %ls", *JsonText);

		// Parse the string object
		if (!FJsonObjectConverter::JsonObjectStringToUStruct<FDisplayClusterConfigurationJsonContainer_500>(JsonText, &JsonData, 0, 0))
		{
			UE_LOGF(LogDisplayClusterConfiguration, Error, "Couldn't deserialize json file: %ls", *FilePath);
			return nullptr;
		}

		ConfigFile = FilePath;

		// Finally, convert the data to nDisplay internal types
		return ConvertDataToInternalTypes();
	}

	bool FDisplayClusterConfigurationJsonParser::SaveData(const UDisplayClusterConfigurationData* ConfigData, const FString& FilePath)
	{
		FString JsonTextOut;

		// Convert to json string
		if (!AsString(ConfigData, JsonTextOut))
		{
			return false;
		}

		// Save json string to a file
		if (!FFileHelper::SaveStringToFile(JsonTextOut, *FilePath))
		{
			UE_LOGF(LogDisplayClusterConfiguration, Error, "Couldn't save data to file: %ls", *FilePath);
			return false;
		}

		UE_LOGF(LogDisplayClusterConfiguration, Log, "Configuration data has been successfully saved to file: %ls", *FilePath);

		return true;
	}

	bool FDisplayClusterConfigurationJsonParser::AsString(const UDisplayClusterConfigurationData* ConfigData, FString& OutString)
	{
		// Convert nDisplay internal types to json types
		if (!ConvertDataToExternalTypes(ConfigData))
		{
			UE_LOGF(LogDisplayClusterConfiguration, Error, "Couldn't convert data to json data types");
			return false;
		}

		// Serialize json types to json string
		if (!FJsonObjectConverter::UStructToJsonObjectString<FDisplayClusterConfigurationJsonContainer_500>(JsonData, OutString))
		{
			UE_LOGF(LogDisplayClusterConfiguration, Error, "Couldn't serialize data to json");
			return false;
		}

		return true;
	}

	UDisplayClusterConfigurationData* FDisplayClusterConfigurationJsonParser::ConvertDataToInternalTypes()
	{
		UDisplayClusterConfigurationData* Config = UDisplayClusterConfigurationData::CreateNewConfigData(ConfigDataOwner, RF_MarkAsRootSet);
		check(Config && Config->Scene && Config->Cluster);

		// Fill metadata
		Config->Meta.ImportDataSource = EDisplayClusterConfigurationDataSource::Json;
		Config->Meta.ImportFilePath = ConfigFile;

		const FDisplayClusterConfigurationJsonNdisplay_500& CfgJson = JsonData.nDisplay;

		// Info
		Config->Info.Version = CfgJson.Version;
		Config->Info.Description = CfgJson.Description;
		Config->Info.AssetPath = CfgJson.AssetPath;

		// Misc
		Config->bFollowLocalPlayerCamera = CfgJson.Misc.bFollowLocalPlayerCamera;
		Config->bExitOnEsc = CfgJson.Misc.bExitOnEsc;
		Config->bOverrideViewportsFromExternalConfig = CfgJson.Misc.bOverrideViewportsFromExternalConfig;
		Config->bOverrideTransformsFromExternalConfig = CfgJson.Misc.bOverrideTransformsFromExternalConfig;
		Config->RenderFrameSettings.ClusterICVFXOuterViewportBufferRatioMult = CfgJson.Misc.ClusterOuterViewportsBufferRatio;

		// Scene
		{
			// Cameras
			for (const TPair<FString, FDisplayClusterConfigurationJsonSceneComponentCamera_500>& CfgComp : CfgJson.Scene.Cameras)
			{
				UDisplayClusterConfigurationSceneComponentCamera* Comp = NewObject<UDisplayClusterConfigurationSceneComponentCamera>(Config->Scene, *CfgComp.Key, RF_Transactional);
				check(Comp);

				// General
				Comp->ParentId = CfgComp.Value.ParentId;
				Comp->Location = FDisplayClusterConfigurationJsonVector_500::ToVector(CfgComp.Value.Location);
				Comp->Rotation = FDisplayClusterConfigurationJsonRotator_500::ToRotator(CfgComp.Value.Rotation);
				// Camera specific
				Comp->InterpupillaryDistance = CfgComp.Value.InterpupillaryDistance;
				Comp->bSwapEyes = CfgComp.Value.SwapEyes;
				Comp->StereoOffset = DisplayClusterConfigurationJsonHelpers::FromString<EDisplayClusterConfigurationEyeStereoOffset>(CfgComp.Value.StereoOffset);

				Config->Scene->Cameras.Emplace(CfgComp.Key, Comp);
			}

			// Screens
			for (const TPair<FString, FDisplayClusterConfigurationJsonSceneComponentScreen_500>& CfgComp : CfgJson.Scene.Screens)
			{
				UDisplayClusterConfigurationSceneComponentScreen* Comp = NewObject<UDisplayClusterConfigurationSceneComponentScreen>(Config->Scene, *CfgComp.Key, RF_Transactional);
				check(Comp);

				// General
				Comp->ParentId = CfgComp.Value.ParentId;
				Comp->Location = FDisplayClusterConfigurationJsonVector_500::ToVector(CfgComp.Value.Location);
				Comp->Rotation = FDisplayClusterConfigurationJsonRotator_500::ToRotator(CfgComp.Value.Rotation);
				// Screen specific
				Comp->Size = FDisplayClusterConfigurationJsonSizeFloat_500::ToVector(CfgComp.Value.Size);

				Config->Scene->Screens.Emplace(CfgComp.Key, Comp);
			}

			// Xforms
			for (const TPair<FString, FDisplayClusterConfigurationJsonSceneComponentXform_500>& CfgComp : CfgJson.Scene.Xforms)
			{
				UDisplayClusterConfigurationSceneComponentXform* Comp = NewObject<UDisplayClusterConfigurationSceneComponentXform>(Config->Scene, *CfgComp.Key, RF_Transactional);
				check(Comp);

				// General
				Comp->ParentId = CfgComp.Value.ParentId;
				Comp->Location = FDisplayClusterConfigurationJsonVector_500::ToVector(CfgComp.Value.Location);
				Comp->Rotation = FDisplayClusterConfigurationJsonRotator_500::ToRotator(CfgComp.Value.Rotation);

				Config->Scene->Xforms.Emplace(CfgComp.Key, Comp);
			}
		}

		// Cluster
		{
			// Primary node
			{
				Config->Cluster->PrimaryNode.Id = CfgJson.Cluster.PrimaryNode.Id;

				const uint16* ClusterSyncPort = CfgJson.Cluster.PrimaryNode.Ports.Find(DisplayClusterConfigurationStrings::config::cluster::ports::PortClusterSync);
				Config->Cluster->PrimaryNode.Ports.ClusterSync = (ClusterSyncPort ? *ClusterSyncPort : 41001);

				const uint16* ClusterEventsJsonPort = CfgJson.Cluster.PrimaryNode.Ports.Find(DisplayClusterConfigurationStrings::config::cluster::ports::PortClusterEventsJson);
				Config->Cluster->PrimaryNode.Ports.ClusterEventsJson = (ClusterEventsJsonPort ? *ClusterEventsJsonPort : 41003);

				const uint16* ClusterEventsBinaryPort = CfgJson.Cluster.PrimaryNode.Ports.Find(DisplayClusterConfigurationStrings::config::cluster::ports::PortClusterEventsBinary);
				Config->Cluster->PrimaryNode.Ports.ClusterEventsBinary = (ClusterEventsBinaryPort ? *ClusterEventsBinaryPort : 41004);
			}

			// Cluster sync
			{
				// Native input sync
				Config->Cluster->Sync.InputSyncPolicy.Type = CfgJson.Cluster.Sync.InputSyncPolicy.Type;
				Config->Cluster->Sync.InputSyncPolicy.Parameters = CfgJson.Cluster.Sync.InputSyncPolicy.Parameters;

				// Render sync
				Config->Cluster->Sync.RenderSyncPolicy.Type = CfgJson.Cluster.Sync.RenderSyncPolicy.Type;
				Config->Cluster->Sync.RenderSyncPolicy.Parameters = CfgJson.Cluster.Sync.RenderSyncPolicy.Parameters;
			}

			// Network
			{
				Config->Cluster->Network.ConnectRetriesAmount = DisplayClusterHelpers::map::template ExtractValueFromString(
					CfgJson.Cluster.Network, FString(DisplayClusterConfigurationStrings::config::cluster::network::NetConnectRetriesAmount), static_cast<uint32>(Config->Cluster->Network.ConnectRetriesAmount));

				Config->Cluster->Network.ConnectRetryDelay = DisplayClusterHelpers::map::template ExtractValueFromString(
					CfgJson.Cluster.Network, FString(DisplayClusterConfigurationStrings::config::cluster::network::NetConnectRetryDelay), static_cast<uint32>(Config->Cluster->Network.ConnectRetryDelay));

				Config->Cluster->Network.GameStartBarrierTimeout = DisplayClusterHelpers::map::template ExtractValueFromString(
					CfgJson.Cluster.Network, FString(DisplayClusterConfigurationStrings::config::cluster::network::NetGameStartBarrierTimeout), static_cast<uint32>(Config->Cluster->Network.GameStartBarrierTimeout));

				Config->Cluster->Network.FrameStartBarrierTimeout = DisplayClusterHelpers::map::template ExtractValueFromString(
					CfgJson.Cluster.Network, FString(DisplayClusterConfigurationStrings::config::cluster::network::NetFrameStartBarrierTimeout), static_cast<uint32>(Config->Cluster->Network.FrameStartBarrierTimeout));

				Config->Cluster->Network.FrameEndBarrierTimeout = DisplayClusterHelpers::map::template ExtractValueFromString(
					CfgJson.Cluster.Network, FString(DisplayClusterConfigurationStrings::config::cluster::network::NetFrameEndBarrierTimeout), static_cast<uint32>(Config->Cluster->Network.FrameEndBarrierTimeout));

				Config->Cluster->Network.RenderSyncBarrierTimeout = DisplayClusterHelpers::map::template ExtractValueFromString(
					CfgJson.Cluster.Network, FString(DisplayClusterConfigurationStrings::config::cluster::network::NetRenderSyncBarrierTimeout), static_cast<uint32>(Config->Cluster->Network.RenderSyncBarrierTimeout));
			}

			// Failover
			{
				Config->Cluster->Failover.bEnabled = CfgJson.Cluster.Failover.bEnabled;
				Config->Cluster->Failover.PrimaryBackups.ItemNames = CfgJson.Cluster.Failover.PrimaryBackups;
				Config->Cluster->Failover.CriticalNodes.ItemNames  = CfgJson.Cluster.Failover.CriticalNodes;
			}

			// Cluster nodes
			for (const TPair<FString, FDisplayClusterConfigurationJsonClusterNode_500>& CfgNode : CfgJson.Cluster.Nodes)
			{
				UDisplayClusterConfigurationClusterNode* Node = NewObject<UDisplayClusterConfigurationClusterNode>(Config->Cluster, *CfgNode.Key, RF_Transactional);
				check(Node);

				// Base parameters
				Node->Host = CfgNode.Value.Host;
				Node->bIsSoundEnabled = CfgNode.Value.Sound;
				Node->bIsFullscreen = CfgNode.Value.FullScreen;
				Node->bRenderHeadless = CfgNode.Value.RenderHeadless;
				Node->GraphicsAdapter = CfgNode.Value.GraphicsAdapter;
				Node->bEnableTextureShare = CfgNode.Value.TextureShare;
				Node->WindowRect = FDisplayClusterConfigurationRectangle(CfgNode.Value.Window.X, CfgNode.Value.Window.Y, CfgNode.Value.Window.W, CfgNode.Value.Window.H);

				// Viewports
				for (const TPair<FString, FDisplayClusterConfigurationJsonViewport_500>& CfgViewport : CfgNode.Value.Viewports)
				{
					UDisplayClusterConfigurationViewport* Viewport = NewObject<UDisplayClusterConfigurationViewport>(Node, *CfgViewport.Key, RF_Transactional | RF_ArchetypeObject | RF_Public);
					check(Viewport);

					// Base parameters
					Viewport->RenderSettings.BufferRatio = CfgViewport.Value.BufferRatio;
					Viewport->Camera = CfgViewport.Value.Camera;
					Viewport->Region = FDisplayClusterConfigurationRectangle(CfgViewport.Value.Region.X, CfgViewport.Value.Region.Y, CfgViewport.Value.Region.W, CfgViewport.Value.Region.H);
					Viewport->GPUIndex = CfgViewport.Value.GPUIndex;

					// Projection policy
					Viewport->ProjectionPolicy.Type = CfgViewport.Value.ProjectionPolicy.Type;
					Viewport->ProjectionPolicy.Parameters = CfgViewport.Value.ProjectionPolicy.Parameters;

					Viewport->RenderSettings.Overscan.Mode = DisplayClusterConfigurationJsonHelpers::FromString<EDisplayClusterConfigurationViewportOverscanMode>(CfgViewport.Value.Overscan.Mode);
					Viewport->RenderSettings.Overscan.bEnabled = CfgViewport.Value.Overscan.bEnabled;
					Viewport->RenderSettings.Overscan.bOversize = CfgViewport.Value.Overscan.Oversize;
					Viewport->RenderSettings.Overscan.Left = CfgViewport.Value.Overscan.Left;
					Viewport->RenderSettings.Overscan.Right = CfgViewport.Value.Overscan.Right;
					Viewport->RenderSettings.Overscan.Top = CfgViewport.Value.Overscan.Top;
					Viewport->RenderSettings.Overscan.Bottom = CfgViewport.Value.Overscan.Bottom;

					// Add this viewport
					Node->Viewports.Emplace(CfgViewport.Key, Viewport);
				}

				// Postprocess
				for (const TPair<FString, FDisplayClusterConfigurationJsonPostprocess_500>& CfgPostprocess : CfgNode.Value.Postprocess)
				{
					FDisplayClusterConfigurationPostprocess PostprocessOperation;

					PostprocessOperation.Type = CfgPostprocess.Value.Type;
					PostprocessOperation.Parameters = CfgPostprocess.Value.Parameters;

					Node->Postprocess.Emplace(CfgPostprocess.Key, PostprocessOperation);
				}

				// Output remap
				Node->OutputRemap.bEnable = CfgNode.Value.OutputRemap.bEnable;
				Node->OutputRemap.DataSource = (CfgNode.Value.OutputRemap.DataSource == "file") ? EDisplayClusterConfigurationFramePostProcess_OutputRemapSource::ExternalFile : EDisplayClusterConfigurationFramePostProcess_OutputRemapSource::StaticMesh;
				Node->OutputRemap.ExternalFile = CfgNode.Value.OutputRemap.ExternalFile;

				if (!CfgNode.Value.OutputRemap.StaticMeshAsset.IsEmpty())
				{
					Node->OutputRemap.StaticMesh = Cast<UStaticMesh>(StaticLoadObject(UStaticMesh::StaticClass(), nullptr, *CfgNode.Value.OutputRemap.StaticMeshAsset));
				}

				// Store new cluster node
				Config->Cluster->Nodes.Emplace(CfgNode.Key, Node);
			}
		}

		// Custom parameters
		Config->CustomParameters = CfgJson.CustomParameters;

		// Diagnostics
		Config->Diagnostics.bSimulateLag = CfgJson.Diagnostics.SimulateLag;
		Config->Diagnostics.MinLagTime = CfgJson.Diagnostics.MinLagTime;
		Config->Diagnostics.MaxLagTime = CfgJson.Diagnostics.MaxLagTime;

		return Config;
	}

	bool FDisplayClusterConfigurationJsonParser::ConvertDataToExternalTypes(const UDisplayClusterConfigurationData* Config)
	{
		if (!(Config && Config->Scene && Config->Cluster))
		{
			UE_LOGF(LogDisplayClusterConfiguration, Error, "nullptr detected in the configuration data");
			return false;
		}

		FDisplayClusterConfigurationJsonNdisplay_500& Json = JsonData.nDisplay;

		// Info
		Json.Description = Config->Info.Description;
		Json.Version   = FString("5.00"); // Use hard-coded version number to be able to detect JSON config version on import
		Json.AssetPath = Config->Info.AssetPath;

		// Misc
		Json.Misc.bFollowLocalPlayerCamera = Config->bFollowLocalPlayerCamera;
		Json.Misc.bExitOnEsc = Config->bExitOnEsc;
		Json.Misc.bOverrideViewportsFromExternalConfig = Config->bOverrideViewportsFromExternalConfig;
		Json.Misc.bOverrideTransformsFromExternalConfig = Config->bOverrideTransformsFromExternalConfig;
		Json.Misc.ClusterOuterViewportsBufferRatio = Config->RenderFrameSettings.ClusterICVFXOuterViewportBufferRatioMult;

		// Scene
		{
			// Cameras
			for (const TPair<FString, TObjectPtr<UDisplayClusterConfigurationSceneComponentCamera>>& Comp : Config->Scene->Cameras)
			{
				FDisplayClusterConfigurationJsonSceneComponentCamera_500 CfgComp;

				// General
				CfgComp.ParentId = Comp.Value->ParentId;
				CfgComp.Location = FDisplayClusterConfigurationJsonVector_500::FromVector(Comp.Value->Location);
				CfgComp.Rotation = FDisplayClusterConfigurationJsonRotator_500::FromRotator(Comp.Value->Rotation);
				// Camera specific
				CfgComp.InterpupillaryDistance = Comp.Value->InterpupillaryDistance;
				CfgComp.SwapEyes = Comp.Value->bSwapEyes;
				CfgComp.StereoOffset = DisplayClusterConfigurationJsonHelpers::ToString(Comp.Value->StereoOffset);

				Json.Scene.Cameras.Emplace(Comp.Key, CfgComp);
			}

			// Screens
			for (const TPair<FString, TObjectPtr<UDisplayClusterConfigurationSceneComponentScreen>>& Comp : Config->Scene->Screens)
			{
				FDisplayClusterConfigurationJsonSceneComponentScreen_500 CfgComp;

				// General
				CfgComp.ParentId = Comp.Value->ParentId;
				CfgComp.Location = FDisplayClusterConfigurationJsonVector_500::FromVector(Comp.Value->Location);
				CfgComp.Rotation = FDisplayClusterConfigurationJsonRotator_500::FromRotator(Comp.Value->Rotation);
				// Screen specific
				CfgComp.Size = FDisplayClusterConfigurationJsonSizeFloat_500::FromVector(Comp.Value->Size);

				Json.Scene.Screens.Emplace(Comp.Key, CfgComp);
			}

			// Xforms
			for (const TPair<FString, TObjectPtr<UDisplayClusterConfigurationSceneComponentXform>>& Comp : Config->Scene->Xforms)
			{
				FDisplayClusterConfigurationJsonSceneComponentScreen_500 CfgComp;

				// General
				CfgComp.ParentId = Comp.Value->ParentId;
				CfgComp.Location = FDisplayClusterConfigurationJsonVector_500::FromVector(Comp.Value->Location);
				CfgComp.Rotation = FDisplayClusterConfigurationJsonRotator_500::FromRotator(Comp.Value->Rotation);

				Json.Scene.Xforms.Emplace(Comp.Key, CfgComp);
			}
		}

		// Cluster
		{
			// Primary node
			{
				Json.Cluster.PrimaryNode.Id = Config->Cluster->PrimaryNode.Id;
				Json.Cluster.PrimaryNode.Ports.Emplace(DisplayClusterConfigurationStrings::config::cluster::ports::PortClusterSync,         Config->Cluster->PrimaryNode.Ports.ClusterSync);
				Json.Cluster.PrimaryNode.Ports.Emplace(DisplayClusterConfigurationStrings::config::cluster::ports::PortClusterEventsJson,   Config->Cluster->PrimaryNode.Ports.ClusterEventsJson);
				Json.Cluster.PrimaryNode.Ports.Emplace(DisplayClusterConfigurationStrings::config::cluster::ports::PortClusterEventsBinary, Config->Cluster->PrimaryNode.Ports.ClusterEventsBinary);
			}

			// Cluster sync
			{
				// Native input sync
				Json.Cluster.Sync.InputSyncPolicy.Type = Config->Cluster->Sync.InputSyncPolicy.Type;
				Json.Cluster.Sync.InputSyncPolicy.Parameters = Config->Cluster->Sync.InputSyncPolicy.Parameters;

				// Render sync
				Json.Cluster.Sync.RenderSyncPolicy.Type = Config->Cluster->Sync.RenderSyncPolicy.Type;
				Json.Cluster.Sync.RenderSyncPolicy.Parameters = Config->Cluster->Sync.RenderSyncPolicy.Parameters;
			}

			// Network
			{
				Json.Cluster.Network.Emplace(DisplayClusterConfigurationStrings::config::cluster::network::NetConnectRetriesAmount,
					DisplayClusterTypesConverter::template ToString(Config->Cluster->Network.ConnectRetriesAmount));

				Json.Cluster.Network.Emplace(DisplayClusterConfigurationStrings::config::cluster::network::NetConnectRetryDelay,
					DisplayClusterTypesConverter::template ToString(Config->Cluster->Network.ConnectRetryDelay));

				Json.Cluster.Network.Emplace(DisplayClusterConfigurationStrings::config::cluster::network::NetGameStartBarrierTimeout,
					DisplayClusterTypesConverter::template ToString(Config->Cluster->Network.GameStartBarrierTimeout));

				Json.Cluster.Network.Emplace(DisplayClusterConfigurationStrings::config::cluster::network::NetFrameStartBarrierTimeout,
					DisplayClusterTypesConverter::template ToString(Config->Cluster->Network.FrameStartBarrierTimeout));

				Json.Cluster.Network.Emplace(DisplayClusterConfigurationStrings::config::cluster::network::NetFrameEndBarrierTimeout,
					DisplayClusterTypesConverter::template ToString(Config->Cluster->Network.FrameEndBarrierTimeout));

				Json.Cluster.Network.Emplace(DisplayClusterConfigurationStrings::config::cluster::network::NetRenderSyncBarrierTimeout,
					DisplayClusterTypesConverter::template ToString(Config->Cluster->Network.RenderSyncBarrierTimeout));
			}

			// Failover
			{
				Json.Cluster.Failover.bEnabled = Config->Cluster->Failover.bEnabled;
				Json.Cluster.Failover.PrimaryBackups = Config->Cluster->Failover.PrimaryBackups.ItemNames;
				Json.Cluster.Failover.CriticalNodes  = Config->Cluster->Failover.CriticalNodes.ItemNames;
			}

			// Cluster nodes
			for (const TPair<FString, TObjectPtr<UDisplayClusterConfigurationClusterNode>>& CfgNode : Config->Cluster->Nodes)
			{
				FDisplayClusterConfigurationJsonClusterNode_500 Node;

				// Base parameters
				Node.Host = CfgNode.Value->Host;
				Node.Sound = CfgNode.Value->bIsSoundEnabled;
				Node.FullScreen = CfgNode.Value->bIsFullscreen;
				Node.RenderHeadless = CfgNode.Value->bRenderHeadless;
				Node.GraphicsAdapter = CfgNode.Value->GraphicsAdapter;
				Node.TextureShare = CfgNode.Value->bEnableTextureShare;
				Node.Window = FDisplayClusterConfigurationJsonRectangle_500(CfgNode.Value->WindowRect.X, CfgNode.Value->WindowRect.Y, CfgNode.Value->WindowRect.W, CfgNode.Value->WindowRect.H);

				// Viewports
				for (const TPair<FString, TObjectPtr<UDisplayClusterConfigurationViewport>>& CfgViewport : CfgNode.Value->Viewports)
				{
					FDisplayClusterConfigurationJsonViewport_500 Viewport;

					// Base parameters
					Viewport.Camera = CfgViewport.Value->Camera;
					Viewport.Region = FDisplayClusterConfigurationJsonRectangle_500(CfgViewport.Value->Region.X, CfgViewport.Value->Region.Y, CfgViewport.Value->Region.W, CfgViewport.Value->Region.H);
					Viewport.GPUIndex = CfgViewport.Value->GPUIndex;
					Viewport.BufferRatio = CfgViewport.Value->RenderSettings.BufferRatio;

					// Projection policy
					Viewport.ProjectionPolicy.Type = CfgViewport.Value->ProjectionPolicy.Type;
					Viewport.ProjectionPolicy.Parameters = CfgViewport.Value->ProjectionPolicy.Parameters;

					// Overscan
					Viewport.Overscan.Mode = DisplayClusterConfigurationJsonHelpers::ToString(CfgViewport.Value->RenderSettings.Overscan.Mode);
					Viewport.Overscan.bEnabled = CfgViewport.Value->RenderSettings.Overscan.bEnabled;
					Viewport.Overscan.Oversize = CfgViewport.Value->RenderSettings.Overscan.bOversize;
					Viewport.Overscan.Left = CfgViewport.Value->RenderSettings.Overscan.Left;
					Viewport.Overscan.Right = CfgViewport.Value->RenderSettings.Overscan.Right;
					Viewport.Overscan.Top = CfgViewport.Value->RenderSettings.Overscan.Top;
					Viewport.Overscan.Bottom = CfgViewport.Value->RenderSettings.Overscan.Bottom;
					
					// Save this viewport
					Node.Viewports.Emplace(CfgViewport.Key, Viewport);
				}

				// Postprocess
				for (const TPair<FString, FDisplayClusterConfigurationPostprocess>& CfgPostprocess : CfgNode.Value->Postprocess)
				{
					FDisplayClusterConfigurationJsonPostprocess_500 PostprocessOperation;

					PostprocessOperation.Type = CfgPostprocess.Value.Type;
					PostprocessOperation.Parameters = CfgPostprocess.Value.Parameters;

					Node.Postprocess.Emplace(CfgPostprocess.Key, PostprocessOperation);
				}

				// OutputRemap
				{
					Node.OutputRemap.bEnable = CfgNode.Value->OutputRemap.bEnable;
					Node.OutputRemap.DataSource = (CfgNode.Value->OutputRemap.DataSource == EDisplayClusterConfigurationFramePostProcess_OutputRemapSource::ExternalFile) ? "file" : "mesh";
					Node.OutputRemap.ExternalFile = CfgNode.Value->OutputRemap.ExternalFile;

					if (CfgNode.Value->OutputRemap.StaticMesh)
					{
						Node.OutputRemap.StaticMeshAsset = CfgNode.Value->OutputRemap.StaticMesh->GetPathName();
					}
				}

				// Store new cluster node
				Json.Cluster.Nodes.Emplace(CfgNode.Key, Node);
			}
		}

		// Custom parameters
		Json.CustomParameters = Config->CustomParameters;

		// Diagnostics
		Json.Diagnostics.SimulateLag = Config->Diagnostics.bSimulateLag;
		Json.Diagnostics.MinLagTime = Config->Diagnostics.MinLagTime;
		Json.Diagnostics.MaxLagTime = Config->Diagnostics.MaxLagTime;

		return true;
	}
}
