// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DisplayClusterConfigurationStrings.h"
#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterConfigurationJsonTypes_500.generated.h"


USTRUCT()
struct FDisplayClusterConfigurationJsonRectangle_500
{
	GENERATED_BODY()

public:
	FDisplayClusterConfigurationJsonRectangle_500()
		: X(0), Y(0), W(0), H(0)
	{ }

	FDisplayClusterConfigurationJsonRectangle_500(int32 _X, int32 _Y, int32 _W, int32 _H)
		: X(_X), Y(_Y), W(_W), H(_H)
	{ }

public:
	UPROPERTY()
	int32 X;

	UPROPERTY()
	int32 Y;

	UPROPERTY()
	int32 W;

	UPROPERTY()
	int32 H;
};


USTRUCT()
struct FDisplayClusterConfigurationJsonVector_500
{
	GENERATED_BODY()

public:
	FDisplayClusterConfigurationJsonVector_500()
		: FDisplayClusterConfigurationJsonVector_500(0.f, 0.f, 0.f)
	{ }

	FDisplayClusterConfigurationJsonVector_500(float _X, float _Y, float _Z)
		: X(_X), Y(_Y), Z(_Z)
	{ }

	FDisplayClusterConfigurationJsonVector_500(const FVector& Vector)
		: FDisplayClusterConfigurationJsonVector_500(Vector.X, Vector.Y, Vector.Z)
	{ }

public:
	UPROPERTY()
	float X;

	UPROPERTY()
	float Y;

	UPROPERTY()
	float Z;

public:
	static FVector ToVector(const FDisplayClusterConfigurationJsonVector_500& Data)
	{
		return FVector(Data.X, Data.Y, Data.Z);
	}

	static FDisplayClusterConfigurationJsonVector_500 FromVector(const FVector& Vector)
	{
		return FDisplayClusterConfigurationJsonVector_500(Vector.X, Vector.Y, Vector.Z);
	}
};


USTRUCT()
struct FDisplayClusterConfigurationJsonRotator_500
{
	GENERATED_BODY()

public:
	FDisplayClusterConfigurationJsonRotator_500()
		: FDisplayClusterConfigurationJsonRotator_500(0.f, 0.f, 0.f)
	{ }

	FDisplayClusterConfigurationJsonRotator_500(float P, float Y, float R)
		: Pitch(P), Yaw(Y), Roll(R)
	{ }

	FDisplayClusterConfigurationJsonRotator_500(const FRotator& Rotator)
		: FDisplayClusterConfigurationJsonRotator_500(Rotator.Pitch, Rotator.Yaw, Rotator.Roll)
	{ }

public:
	UPROPERTY()
	float Pitch;

	UPROPERTY()
	float Yaw;

	UPROPERTY()
	float Roll;

public:
	static FRotator ToRotator(const FDisplayClusterConfigurationJsonRotator_500& Data)
	{
		return FRotator(Data.Pitch, Data.Yaw, Data.Roll);
	}

	static FDisplayClusterConfigurationJsonRotator_500 FromRotator(const FRotator& Rotator)
	{
		return FDisplayClusterConfigurationJsonRotator_500(Rotator.Pitch, Rotator.Yaw, Rotator.Roll);
	}
};


USTRUCT()
struct FDisplayClusterConfigurationJsonSizeInt_500
{
	GENERATED_BODY()

public:
	FDisplayClusterConfigurationJsonSizeInt_500()
		: FDisplayClusterConfigurationJsonSizeInt_500(0, 0)
	{ }

	FDisplayClusterConfigurationJsonSizeInt_500(int W, int H)
		: Width(W), Height(H)
	{ }

public:
	UPROPERTY()
	int Width;

	UPROPERTY()
	int Height;

public:
	static FIntPoint ToPoint(const FDisplayClusterConfigurationJsonSizeInt_500& Data)
	{
		return FIntPoint(Data.Width, Data.Height);
	}

	static FDisplayClusterConfigurationJsonSizeInt_500 FromPoint(const FIntPoint& Point)
	{
		return FDisplayClusterConfigurationJsonSizeInt_500(Point.X, Point.Y);
	}
};


USTRUCT()
struct FDisplayClusterConfigurationJsonSizeFloat_500
{
	GENERATED_BODY()

public:
	FDisplayClusterConfigurationJsonSizeFloat_500()
		: FDisplayClusterConfigurationJsonSizeFloat_500(0.f, 0.f)
	{ }

	FDisplayClusterConfigurationJsonSizeFloat_500(float W, float H)
		: Width(W), Height(H)
	{ }

	FDisplayClusterConfigurationJsonSizeFloat_500(const FVector2D& Vector)
		: FDisplayClusterConfigurationJsonSizeFloat_500(Vector.X, Vector.Y)
	{ }

public:
	UPROPERTY()
	float Width;

	UPROPERTY()
	float Height;

public:
	static FVector2D ToVector(const FDisplayClusterConfigurationJsonSizeFloat_500& Data)
	{
		return FVector2D(Data.Width, Data.Height);
	}

	static FDisplayClusterConfigurationJsonSizeFloat_500 FromVector(const FVector2D& Vector)
	{
		return FDisplayClusterConfigurationJsonSizeFloat_500(Vector.X, Vector.Y);
	}
};


USTRUCT()
struct FDisplayClusterConfigurationJsonPolymorphicEntity_500
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FString Type;

	UPROPERTY()
	TMap<FString, FString> Parameters;
};


USTRUCT()
struct FDisplayClusterConfigurationJsonMisc_500
{
	GENERATED_BODY()

public:
	FDisplayClusterConfigurationJsonMisc_500()
		: bFollowLocalPlayerCamera(false)
		, bExitOnEsc(true)
		, bOverrideViewportsFromExternalConfig(false)
		, bOverrideTransformsFromExternalConfig(false)
		, ClusterOuterViewportsBufferRatio(0.66f)
	{ }

public:
	UPROPERTY()
	bool bFollowLocalPlayerCamera;

	UPROPERTY()
	bool bExitOnEsc;

	UPROPERTY()
	bool bOverrideViewportsFromExternalConfig;

	UPROPERTY()
	bool bOverrideTransformsFromExternalConfig;

	UPROPERTY()
	float ClusterOuterViewportsBufferRatio;
};


USTRUCT()
struct FDisplayClusterConfigurationJsonSceneComponent_500
{
	GENERATED_BODY()
};


USTRUCT()
struct FDisplayClusterConfigurationJsonSceneComponentXform_500
	: public FDisplayClusterConfigurationJsonSceneComponent_500
{
	GENERATED_BODY()

public:
	FDisplayClusterConfigurationJsonSceneComponentXform_500()
		: FDisplayClusterConfigurationJsonSceneComponentXform_500(FString(), FVector::ZeroVector, FRotator::ZeroRotator)
	{ }

	FDisplayClusterConfigurationJsonSceneComponentXform_500(const FString& InParentId, const FVector& InLocation, const FRotator& InRotation)
		: ParentId(InParentId)
		, Location(InLocation)
		, Rotation(InRotation)
	{ }

public:
	UPROPERTY()
	FString ParentId;

	UPROPERTY()
	FDisplayClusterConfigurationJsonVector_500 Location;

	UPROPERTY()
	FDisplayClusterConfigurationJsonRotator_500 Rotation;
};


USTRUCT()
struct FDisplayClusterConfigurationJsonSceneComponentCamera_500
	: public FDisplayClusterConfigurationJsonSceneComponentXform_500
{
	GENERATED_BODY()

public:
	FDisplayClusterConfigurationJsonSceneComponentCamera_500()
		: FDisplayClusterConfigurationJsonSceneComponentCamera_500(FString(), FVector::ZeroVector, FRotator::ZeroRotator, 6.4f, false, DisplayClusterConfigurationStrings::config::scene::camera::CameraStereoOffsetNone)
	{ }

	FDisplayClusterConfigurationJsonSceneComponentCamera_500(const FString& InParentId, const FVector& InLocation, const FRotator& InRotation, float InInterpupillaryDistance, bool bInSwapEyes, const FString& InStereoOffset)
		: FDisplayClusterConfigurationJsonSceneComponentXform_500(InParentId, InLocation, InRotation)
		, InterpupillaryDistance(InInterpupillaryDistance)
		, SwapEyes(bInSwapEyes)
		, StereoOffset(InStereoOffset)
	{ }

public:
	UPROPERTY()
	float InterpupillaryDistance;

	UPROPERTY()
	bool SwapEyes;

	UPROPERTY()
	FString StereoOffset;
};


USTRUCT()
struct FDisplayClusterConfigurationJsonSceneComponentScreen_500
	: public FDisplayClusterConfigurationJsonSceneComponentXform_500
{
	GENERATED_BODY()

public:
	FDisplayClusterConfigurationJsonSceneComponentScreen_500()
		: FDisplayClusterConfigurationJsonSceneComponentScreen_500(FString(), FVector::ZeroVector, FRotator::ZeroRotator, FVector2D(1.f, 1.f))
	{ }

	FDisplayClusterConfigurationJsonSceneComponentScreen_500(const FString& InParentId, const FVector& InLocation, const FRotator& InRotation, const FVector2D& InSize)
		: FDisplayClusterConfigurationJsonSceneComponentXform_500(InParentId, InLocation, InRotation)
		, Size(InSize)
	{ }

public:
	UPROPERTY()
	FDisplayClusterConfigurationJsonSizeFloat_500 Size;
};


USTRUCT()
struct FDisplayClusterConfigurationJsonScene_500
{
	GENERATED_BODY()

public:
	UPROPERTY()
	TMap<FString, FDisplayClusterConfigurationJsonSceneComponentXform_500> Xforms;

	UPROPERTY()
	TMap<FString, FDisplayClusterConfigurationJsonSceneComponentCamera_500> Cameras;

	UPROPERTY()
	TMap<FString, FDisplayClusterConfigurationJsonSceneComponentScreen_500> Screens;
};


USTRUCT()
struct FDisplayClusterConfigurationJsonPrimaryNode_500
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FString Id;

	UPROPERTY()
	TMap<FString, uint16> Ports;
};


USTRUCT()
struct FDisplayClusterConfigurationJsonClusterSyncPolicy_500
	: public FDisplayClusterConfigurationJsonPolymorphicEntity_500
{
	GENERATED_BODY()
};


USTRUCT()
struct FDisplayClusterConfigurationJsonClusterSync_500
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FDisplayClusterConfigurationJsonClusterSyncPolicy_500 RenderSyncPolicy;

	UPROPERTY()
	FDisplayClusterConfigurationJsonClusterSyncPolicy_500 InputSyncPolicy;
};


USTRUCT()
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationFailoverSettings_500
{
	GENERATED_BODY()

public:

	/** Enable/disable failover subsystem */
	UPROPERTY()
	bool bEnabled = false;

	/** Prioritized nodes for primary node replacement. Used in the same order as listed. */
	UPROPERTY()
	TArray<FString> PrimaryBackups;

	/** Critical nodes not allowed to fail. */
	UPROPERTY()
	TArray<FString> CriticalNodes;
};


USTRUCT()
struct FDisplayClusterConfigurationJsonPostprocess_500
	: public FDisplayClusterConfigurationJsonPolymorphicEntity_500
{
	GENERATED_BODY()
};


USTRUCT()
struct FDisplayClusterConfigurationJsonProjectionPolicy_500
	: public FDisplayClusterConfigurationJsonPolymorphicEntity_500
{
	GENERATED_BODY()
};


USTRUCT()
struct FDisplayClusterConfigurationJsonOverscan_500
{
	GENERATED_BODY()

public:
	UPROPERTY()
	bool bEnabled = false;

	UPROPERTY()
	FString Mode = "";

	UPROPERTY()
	int32 Left = 0;

	UPROPERTY()
	int32 Right = 0;

	UPROPERTY()
	int32 Top = 0;

	UPROPERTY()
	int32 Bottom = 0;

	UPROPERTY()
	bool Oversize = 0;
};


USTRUCT()
struct FDisplayClusterConfigurationJsonViewport_500
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FString Camera;

	UPROPERTY()
	float BufferRatio = 1.0f;

	UPROPERTY()
	int32 GPUIndex = 0;

	UPROPERTY()
	bool AllowCrossGPUTransfer = false;

	UPROPERTY()
	bool IsShared = false;

	UPROPERTY()
	FDisplayClusterConfigurationJsonOverscan_500 Overscan;

	UPROPERTY()
	FDisplayClusterConfigurationJsonRectangle_500 Region;

	UPROPERTY()
	FDisplayClusterConfigurationJsonProjectionPolicy_500 ProjectionPolicy;
};


USTRUCT()
struct FDisplayClusterConfigurationFramePostProcess_OutputRemap_500
{
	GENERATED_BODY()

public:
	UPROPERTY()
	bool bEnable = false;

	UPROPERTY()
	FString DataSource;

	UPROPERTY()
	FString StaticMeshAsset;

	UPROPERTY()
	FString ExternalFile;
};


USTRUCT()
struct FDisplayClusterConfigurationJsonClusterNode_500
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FString Host;

	UPROPERTY()
	bool Sound = true;

	UPROPERTY()
	bool FullScreen = false;

	UPROPERTY()
	bool RenderHeadless = false;

	UPROPERTY()
	int32 GraphicsAdapter = -1;

	UPROPERTY()
	bool TextureShare = false;

	UPROPERTY()
	FDisplayClusterConfigurationJsonRectangle_500 Window;

	UPROPERTY()
	TMap<FString, FDisplayClusterConfigurationJsonPostprocess_500> Postprocess;

	UPROPERTY()
	TMap<FString, FDisplayClusterConfigurationJsonViewport_500> Viewports;

	UPROPERTY()
	FDisplayClusterConfigurationFramePostProcess_OutputRemap_500 OutputRemap;
};


USTRUCT()
struct FDisplayClusterConfigurationJsonCluster_500
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FDisplayClusterConfigurationJsonPrimaryNode_500 PrimaryNode;

	UPROPERTY()
	FDisplayClusterConfigurationJsonClusterSync_500 Sync;

	UPROPERTY()
	TMap<FString, FString> Network;

	UPROPERTY()
	FDisplayClusterConfigurationFailoverSettings_500 Failover;

	UPROPERTY()
	TMap<FString, FDisplayClusterConfigurationJsonClusterNode_500> Nodes;
};


USTRUCT()
struct FDisplayClusterConfigurationJsonDiagnostics_500
{
	GENERATED_BODY()

public:
	UPROPERTY()
	bool SimulateLag = false;

	UPROPERTY()
	float MinLagTime = 0.01f;

	UPROPERTY()
	float MaxLagTime = 0.3f;
};


// "nDisplay" category structure
USTRUCT()
struct FDisplayClusterConfigurationJsonNdisplay_500
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FString Description;

	UPROPERTY()
	FString Version;

	UPROPERTY()
	FString AssetPath;

	UPROPERTY()
	FDisplayClusterConfigurationJsonMisc_500 Misc;

	UPROPERTY()
	FDisplayClusterConfigurationJsonScene_500 Scene;

	UPROPERTY()
	FDisplayClusterConfigurationJsonCluster_500 Cluster;

	UPROPERTY()
	TMap<FString, FString> CustomParameters;

	UPROPERTY()
	FDisplayClusterConfigurationJsonDiagnostics_500 Diagnostics;
};


// The main nDisplay configuration structure. It's supposed to extract nDisplay related data from a collecting JSON file.
USTRUCT()
struct FDisplayClusterConfigurationJsonContainer_500
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FDisplayClusterConfigurationJsonNdisplay_500 nDisplay;
};
