// Copyright Epic Games, Inc. All Rights Reserved.

#include "VersionChecker/DisplayClusterConfigurationVersionChecker.h"
#include "VersionChecker/DisplayClusterConfigurationVersionCheckerTypes.h"

#include "DisplayClusterConfigurationLog.h"
#include "DisplayClusterConfigurationStrings.h"

#include "JsonObjectConverter.h"

#include "Misc/FileHelper.h"
#include "Misc/Paths.h"


EDisplayClusterConfigurationVersion FDisplayClusterConfigurationVersionChecker::GetConfigVersion(const FString& FilePath) const
{
	// First, detect either it's an old text based file or a newer JSON
	EConfigFileType FileType = GetConfigFileType(FilePath);

	switch (FileType)
	{
	case EConfigFileType::Json:
		return GetConfigVersionJson(FilePath);

	default:
		return EDisplayClusterConfigurationVersion::Unknown;
	}
}

EDisplayClusterConfigurationVersion FDisplayClusterConfigurationVersionChecker::GetConfigVersionJson(const FString& FilePath) const
{
	FString JsonText;

	// Load json text to the string object
	if (!FFileHelper::LoadFileToString(JsonText, *FilePath))
	{
		UE_LOGF(LogDisplayClusterConfiguration, Error, "Couldn't read file: %ls", *FilePath);
		return EDisplayClusterConfigurationVersion::Unknown;
	}

	// Parse the string object
	FDisplayClusterConfigurationVersionContainer JsonData;
	if (!FJsonObjectConverter::JsonObjectStringToUStruct<FDisplayClusterConfigurationVersionContainer>(JsonText, &JsonData, 0, 0))
	{
		UE_LOGF(LogDisplayClusterConfiguration, Error, "Couldn't deserialize json file: %ls", *FilePath);
		return EDisplayClusterConfigurationVersion::Unknown;
	}

	// Now detect the JSON config version
	if (JsonData.nDisplay.Version.Equals(TEXT("23")) ||   // 426 was exporting .ndisplay configs with version "23"
		JsonData.nDisplay.Version.Equals(TEXT("4.23")) || // This and the following are just in case somebody manually has changed the version field
		JsonData.nDisplay.Version.Equals(TEXT("26")) ||
		JsonData.nDisplay.Version.Equals(TEXT("4.26")))
	{
		return EDisplayClusterConfigurationVersion::Version_426;
	}
	else if (JsonData.nDisplay.Version.Equals(TEXT("4.27")))
	{
		return EDisplayClusterConfigurationVersion::Version_427;
	}
	else if (JsonData.nDisplay.Version.Equals(TEXT("5.00")))
	{
		return EDisplayClusterConfigurationVersion::Version_500;
	}

	// Otherwise it's something unknown
	return EDisplayClusterConfigurationVersion::Unknown;
}

FDisplayClusterConfigurationVersionChecker::EConfigFileType FDisplayClusterConfigurationVersionChecker::GetConfigFileType(const FString& InConfigPath) const
{
	const FString Extension = FPaths::GetExtension(InConfigPath).ToLower();
	if (Extension.Equals(FString(DisplayClusterConfigurationStrings::file::FileExtJson), ESearchCase::IgnoreCase))
	{
		UE_LOGF(LogDisplayClusterConfiguration, Log, "JSON config: %ls", *InConfigPath);
		return EConfigFileType::Json;
	}

	UE_LOGF(LogDisplayClusterConfiguration, Warning, "Unknown file extension: %ls", *Extension);
	return EConfigFileType::Unknown;
}
