// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DisplayClusterConfigurationVersion.h"


/**
* Version checker for nDisplay config files
*/
class FDisplayClusterConfigurationVersionChecker
{
public:
	FDisplayClusterConfigurationVersionChecker() = default;
	~FDisplayClusterConfigurationVersionChecker() = default;

public:
	// Returns version of a specified config file
	EDisplayClusterConfigurationVersion GetConfigVersion(const FString& FilePath) const;

protected:
	// Helper function to deal with JSON files that might be of different version
	EDisplayClusterConfigurationVersion GetConfigVersionJson(const FString& FilePath) const;

protected:
	enum class EConfigFileType
	{
		Unknown,
		Json
	};

	EConfigFileType GetConfigFileType(const FString& InConfigPath) const;
};
