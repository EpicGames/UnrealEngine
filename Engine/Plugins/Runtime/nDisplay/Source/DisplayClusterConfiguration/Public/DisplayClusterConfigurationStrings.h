// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

namespace DisplayClusterConfigurationStrings
{
	// GUI
	namespace gui
	{
		namespace preview
		{
			static constexpr const TCHAR* PreviewNodeNone = TEXT("None");
		}
	}

	// Property Categories
	namespace categories
	{
		static constexpr const TCHAR* DefaultCategory              = TEXT("NDisplay");

		static constexpr const TCHAR* ClusterCategory              = TEXT("NDisplay Cluster");
		static constexpr const TCHAR* ClusterConfigurationCategory = TEXT("NDisplay Cluster Configuration");
		static constexpr const TCHAR* ClusterPostprocessCategory   = TEXT("Post Process");
		static constexpr const TCHAR* ColorGradingCategory         = TEXT("Color Grading");
		static constexpr const TCHAR* CameraColorGradingCategoryOrig   = TEXT("Inner Frustum Color Grading");
		static constexpr const TCHAR* CameraColorGradingCategory   = TEXT("Color Grading Inner Frustum");
		static constexpr const TCHAR* ChromaKeyCategory            = TEXT("Chromakey");
		static constexpr const TCHAR* LightcardCategory            = TEXT("Light Cards");
		static constexpr const TCHAR* OCIOCategory                 = TEXT("OCIO");
		static constexpr const TCHAR* ExposureCategory             = TEXT("Exposure");
		static constexpr const TCHAR* MediaCategory                = TEXT("Media");
		static constexpr const TCHAR* TileCategory                 = TEXT("Tile Rendering");
		static constexpr const TCHAR* OverrideCategory             = TEXT("Texture Replacement");
		static constexpr const TCHAR* ViewportsCategory            = TEXT("Viewports");
		
		static constexpr const TCHAR* InCameraVFXCategory          = TEXT("In Camera VFX");
		static constexpr const TCHAR* InnerFrustumCategory         = TEXT("Inner Frustum");

		static constexpr const TCHAR* ICVFXCameraCategoryOrig      = TEXT("ICVFX Camera");
		static constexpr const TCHAR* ICVFXCameraCategory          = TEXT("Camera ICVFX");

		static constexpr const TCHAR* ConfigurationCategory        = TEXT("Configuration");
		static constexpr const TCHAR* PreviewCategory              = TEXT("Preview");
		static constexpr const TCHAR* AdvancedCategory             = TEXT("Advanced");
		static constexpr const TCHAR* TextureShareCategory         = TEXT("Texture Share");
		static constexpr const TCHAR* NetworkCategory              = TEXT("Network");
		static constexpr const TCHAR* RenderingCategory            = TEXT("Rendering");
		static constexpr const TCHAR* StereoCategory               = TEXT("Stereo");

		static constexpr const TCHAR* ViewPointStereoCategory              = TEXT("Stereo");
		static constexpr const TCHAR* ViewPointCameraPostProcessCategory   = TEXT("Camera Settings");
		static constexpr const TCHAR* ViewPointInFrustumProjectionCategory = TEXT("Frustum Fit");
	}

	// Config file extensions
	namespace file
	{
		static constexpr const TCHAR* FileExtJson = TEXT("ndisplay");
	}

	namespace config
	{
		namespace cluster
		{
			namespace ports
			{
				static constexpr const TCHAR* PortClusterSync         = TEXT("ClusterSync");
				static constexpr const TCHAR* PortClusterEventsJson   = TEXT("ClusterEventsJson");
				static constexpr const TCHAR* PortClusterEventsBinary = TEXT("ClusterEventsBinary");
			}

			namespace network
			{
				static constexpr const TCHAR* NetConnectRetriesAmount     = TEXT("ConnectRetriesAmount");
				static constexpr const TCHAR* NetConnectRetryDelay        = TEXT("ConnectRetryDelay");
				static constexpr const TCHAR* NetGameStartBarrierTimeout  = TEXT("GameStartBarrierTimeout");
				static constexpr const TCHAR* NetFrameStartBarrierTimeout = TEXT("FrameStartBarrierTimeout");
				static constexpr const TCHAR* NetFrameEndBarrierTimeout   = TEXT("FrameEndBarrierTimeout");
				static constexpr const TCHAR* NetRenderSyncBarrierTimeout = TEXT("RenderSyncBarrierTimeout");
			}

			namespace input_sync
			{
				static constexpr const TCHAR* InputSyncPolicyNone             = TEXT("None");
				static constexpr const TCHAR* InputSyncPolicyReplicatePrimary = TEXT("ReplicatePrimary");
			}

			namespace render_sync
			{
				static constexpr const TCHAR* None              = TEXT("none");
				static constexpr const TCHAR* Ethernet          = TEXT("ethernet");
				static constexpr const TCHAR* EthernetBarrier   = TEXT("ethernet_barrier");

				// NVIDIA Swap Barrier (old)
				static constexpr const TCHAR* Nvidia            = TEXT("nvidia");
				static constexpr const TCHAR* NvidiaSwapBarrier = TEXT("swap_barrier");
				static constexpr const TCHAR* NvidiaSwapGroup   = TEXT("swap_group");

				// NVIDIA Present Barrier (new)
				static constexpr const TCHAR* NvidiaPresentBarrier = TEXT("nvidia_pb");

				// Always use 'none' for headless rendering
				static constexpr const TCHAR* HeadlessRenderingSyncPolicy = None;
			}
		}

		namespace scene
		{
			namespace camera
			{
				static constexpr const TCHAR* CameraStereoOffsetNone  = TEXT("none");
				static constexpr const TCHAR* CameraStereoOffsetLeft  = TEXT("left");
				static constexpr const TCHAR* CameraStereoOffsetRight = TEXT("right");
			}
		}
	}
};
