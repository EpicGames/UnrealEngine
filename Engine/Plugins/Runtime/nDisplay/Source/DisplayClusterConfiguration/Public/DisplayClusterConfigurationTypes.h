// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"

#include "DisplayClusterConfigurationTypes_Base.h"
#include "DisplayClusterConfigurationTypes_ICVFX.h"
#include "DisplayClusterConfigurationTypes_Viewport.h"
#include "DisplayClusterConfigurationTypes_OutputRemap.h"
#include "DisplayClusterEditorPropertyReference.h"

#include "DisplayClusterConfigurationVersion.h"

#include "DisplayClusterConfigurationTypes.generated.h"

class UStaticMesh;
struct FPropertyChangedChainEvent;


USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationInfo
{
	GENERATED_BODY()

public:
	FDisplayClusterConfigurationInfo()
		: Version(DisplayClusterConfiguration::GetCurrentConfigurationSchemeMarker())
	{ }

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay)
	FString Description;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = NDisplay)
	FString Version;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = NDisplay)
	FString AssetPath;
};

// Scene hierarchy
UCLASS()
class DISPLAYCLUSTERCONFIGURATION_API UDisplayClusterConfigurationSceneComponent
	: public UDisplayClusterConfigurationData_Base
{
	GENERATED_BODY()

public:
	UDisplayClusterConfigurationSceneComponent()
		: UDisplayClusterConfigurationSceneComponent(FString(), FVector::ZeroVector, FRotator::ZeroRotator)
	{ }

	UDisplayClusterConfigurationSceneComponent(const FString& InParentId, const FVector& InLocation, const FRotator& InRotation)
		: ParentId(InParentId)
		, Location(InLocation)
		, Rotation(InRotation)
	{ }

public:
	UPROPERTY(EditAnywhere, Category = NDisplay)
	FString ParentId;

	UPROPERTY(EditAnywhere, Category = NDisplay)
	FVector Location;

	UPROPERTY(EditAnywhere, Category = NDisplay)
	FRotator Rotation;
};

UCLASS()
class DISPLAYCLUSTERCONFIGURATION_API UDisplayClusterConfigurationSceneComponentXform
	: public UDisplayClusterConfigurationSceneComponent
{
	GENERATED_BODY()

public:
	UDisplayClusterConfigurationSceneComponentXform()
		: UDisplayClusterConfigurationSceneComponentXform(FString(), FVector::ZeroVector, FRotator::ZeroRotator)
	{ }

	UDisplayClusterConfigurationSceneComponentXform(const FString& InParentId, const FVector& InLocation, const FRotator& InRotation)
		: UDisplayClusterConfigurationSceneComponent(InParentId , InLocation, InRotation)
	{ }
};

UCLASS()
class DISPLAYCLUSTERCONFIGURATION_API UDisplayClusterConfigurationSceneComponentScreen
	: public UDisplayClusterConfigurationSceneComponent
{
	GENERATED_BODY()

public:
	UDisplayClusterConfigurationSceneComponentScreen()
		: UDisplayClusterConfigurationSceneComponentScreen(FString(), FVector::ZeroVector, FRotator::ZeroRotator, FVector2D(100.f, 100.f))
	{ }

	UDisplayClusterConfigurationSceneComponentScreen(const FString& InParentId, const FVector& InLocation, const FRotator& InRotation, const FVector2D& InSize)
		: UDisplayClusterConfigurationSceneComponent(InParentId, InLocation, InRotation)
		, Size(InSize)
	{ }

public:
	UPROPERTY(EditAnywhere, Category = NDisplay)
	FVector2D Size;
};

UCLASS()
class DISPLAYCLUSTERCONFIGURATION_API UDisplayClusterConfigurationSceneComponentCamera
	: public UDisplayClusterConfigurationSceneComponent
{
	GENERATED_BODY()

public:
	UDisplayClusterConfigurationSceneComponentCamera()
		: UDisplayClusterConfigurationSceneComponentCamera(FString(), FVector::ZeroVector, FRotator::ZeroRotator, 6.4f, false, EDisplayClusterConfigurationEyeStereoOffset::None)
	{ }

	UDisplayClusterConfigurationSceneComponentCamera(const FString& InParentId, const FVector& InLocation, const FRotator& InRotation,
		float InInterpupillaryDistance, bool bInSwapEyes, EDisplayClusterConfigurationEyeStereoOffset InStereoOffset)
		: UDisplayClusterConfigurationSceneComponent(InParentId, InLocation, InRotation)
		, InterpupillaryDistance(InInterpupillaryDistance)
		, bSwapEyes(bInSwapEyes)
		, StereoOffset(InStereoOffset)
	{ }

public:
	UPROPERTY(EditAnywhere, Category = NDisplay)
	float InterpupillaryDistance;

	UPROPERTY(EditAnywhere, Category = NDisplay)
	bool bSwapEyes;

	UPROPERTY(EditAnywhere, Category = NDisplay)
	EDisplayClusterConfigurationEyeStereoOffset StereoOffset;
};

UCLASS()
class DISPLAYCLUSTERCONFIGURATION_API UDisplayClusterConfigurationScene
	: public UDisplayClusterConfigurationData_Base
{
	GENERATED_BODY()

public:
	UPROPERTY()
	TMap<FString, TObjectPtr<UDisplayClusterConfigurationSceneComponentXform>> Xforms;

	UPROPERTY()
	TMap<FString, TObjectPtr<UDisplayClusterConfigurationSceneComponentScreen>> Screens;

	UPROPERTY()
	TMap<FString, TObjectPtr<UDisplayClusterConfigurationSceneComponentCamera>> Cameras;

protected:
	virtual void GetObjectsToExport(TArray<UObject*>& OutObjects) override;

};


////////////////////////////////////////////////////////////////
// Cluster
USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationPrimaryNodePorts
{
	GENERATED_BODY()

public:
	FDisplayClusterConfigurationPrimaryNodePorts();

public:
	/** Advanced: TCP port for nDisplay internal services */
	UPROPERTY(EditAnywhere, Category = NDisplay, meta = (ClampMin = "1024", ClampMax = "65535", UIMin = "1024", UIMax = "65535"))
	uint16 ClusterSync;

	/** Advanced: network port for Json Cluster Events */
	UPROPERTY(EditAnywhere, Category = NDisplay, meta = (ClampMin = "1024", ClampMax = "65535", UIMin = "1024", UIMax = "65535"))
	uint16 ClusterEventsJson;

	/** Advanced: network port for Binary Cluster Events */
	UPROPERTY(EditAnywhere, Category = NDisplay, meta = (ClampMin = "1024", ClampMax = "65535", UIMin = "1024", UIMax = "65535"))
	uint16 ClusterEventsBinary;
};

USTRUCT(Blueprintable)
struct FDisplayClusterConfigurationPrimaryNode
{
	GENERATED_BODY()

public:
	UPROPERTY(BlueprintReadOnly, Category = NDisplay)
	FString Id;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Configuration", meta = (DisplayName = "Primary Node Ports"))
	FDisplayClusterConfigurationPrimaryNodePorts Ports;
};

USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationRenderSyncPolicy
	: public FDisplayClusterConfigurationPolymorphicEntity
{
	GENERATED_BODY()
};

USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationInputSyncPolicy
	: public FDisplayClusterConfigurationPolymorphicEntity
{
	GENERATED_BODY()
};

USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationClusterSync
{
	GENERATED_BODY()

public:
	FDisplayClusterConfigurationClusterSync();
	
public:
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Configuration)
	FDisplayClusterConfigurationRenderSyncPolicy RenderSyncPolicy;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Configuration)
	FDisplayClusterConfigurationInputSyncPolicy InputSyncPolicy;
};


USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationNetworkSettings
{
	GENERATED_BODY()

public:

	FDisplayClusterConfigurationNetworkSettings();

public:
	/** Advanced: amount of times nDisplay tries to reconnect before dropping */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (ClampMin = "1", ClampMax = "500", UIMin = "1", UIMax = "500"))
	int32 ConnectRetriesAmount;

	/** Advanced: delay in between connection retries */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (ClampMin = "1", ClampMax = "5000", UIMin = "1", UIMax = "5000"))
	int32 ConnectRetryDelay;

	/** Advanced: timeout for Game Thread Barrier */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (ClampMin = "5000", UIMin = "5000"))
	int32 GameStartBarrierTimeout;

	/** Advanced: timeout value for Start Frame Barrier */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (ClampMin = "1", UIMin = "1"))
	int32 FrameStartBarrierTimeout;

	/** Advanced: timeout value for End Frame Barrier */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (ClampMin = "1", UIMin = "1"))
	int32 FrameEndBarrierTimeout;

	/** Advanced: timeout value for Render Sync */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (ClampMin = "1", UIMin = "1"))
	int32 RenderSyncBarrierTimeout;
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationFailoverSettings
{
	GENERATED_BODY()

public:

	FDisplayClusterConfigurationFailoverSettings();

public:

	/** Enable/disable failover subsystem */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Failover, meta = (DisplayName = "Enable Failover"))
	bool bEnabled;

	/** Prioritized nodes for primary node replacement. Used in the same order as listed. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Failover, meta = (DisplayName = "Prioritized Primary Backups", EditCondition = "bEnabled", ClusterItemType = ClusterNodes))
	FDisplayClusterConfigurationClusterItemReferenceList PrimaryBackups;

	/** Critical cluster nodes. None of these are allowed to fail. If even one fails, the cluster will self-terminate. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Failover, meta = (DisplayName = "Critical Nodes", EditCondition = "bEnabled", ClusterItemType = ClusterNodes))
	FDisplayClusterConfigurationClusterItemReferenceList CriticalNodes;

public:

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.6, "This property has been deprecated.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "This property has been deprecated."))
	EDisplayClusterConfigurationFailoverPolicy FailoverPolicy = EDisplayClusterConfigurationFailoverPolicy::Disabled;
#endif // WITH_EDITORONLY_DATA
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS

USTRUCT()
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationExternalImage
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = NDisplay)
	FString ImagePath;
};

UCLASS(Blueprintable)
class DISPLAYCLUSTERCONFIGURATION_API UDisplayClusterConfigurationClusterNode
	: public UDisplayClusterConfigurationData_Base
{
	GENERATED_BODY()

public:
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPostEditChangeChainProperty, const FPropertyChangedChainEvent&);

	FOnPostEditChangeChainProperty OnPostEditChangeChainProperty;

public:
	UDisplayClusterConfigurationClusterNode();

	UFUNCTION(BlueprintPure, Category = "NDisplay|Configuration")
	void GetViewportIds(TArray<FString>& OutViewportIds) const;

	UFUNCTION(BlueprintPure, Category = "NDisplay|Configuration")
	UDisplayClusterConfigurationViewport* GetViewport(const FString& ViewportId) const;

	// Return all references to meshes from policy, and other
	UFUNCTION(BlueprintPure, Category = "NDisplay|Configuration")
	void GetReferencedMeshNames(TArray<FString>& OutMeshNames) const;
	
	//~ Begin UObject interface
public:
	virtual void PostLoad() override;
	virtual void Serialize(FArchive& Ar) override;

#if WITH_EDITOR
	virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject interface

public:
	/** IP address of this specific cluster Node */
	UPROPERTY(EditAnywhere, BlueprintReadonly, Category = "Network", meta = (DisplayName = "Host IP Address"))
	FString Host;
	
	/** Enables or disables sound on nDisplay primary Node */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Configuration", meta = (DisplayName = "Enable Sound"))
	bool bIsSoundEnabled;

	/** Enables application window native fullscreen support */
	UPROPERTY(EditAnywhere, BlueprintReadonly, Category = "Configuration", meta = (DisplayName = "Fullscreen"))
	bool bIsFullscreen;

	/** Defines the application window size in pixels */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Configuration", meta = (DisplayName = "Window", DisplayMode = "Compound", FixedAspectRatioProperty = "bFixedAspectRatio"))
	FDisplayClusterConfigurationRectangle WindowRect;

	/** Output remapping settings for the selected cluster node */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Configuration", meta = (DisplayName = "Output Remapping"))
	FDisplayClusterConfigurationFramePostProcess_OutputRemap OutputRemap;

	/** Activates headless rendering for this cluster node */
	UPROPERTY(EditAnywhere, BlueprintReadonly, Category = "Configuration", meta = (DisplayName = "Headless Rendering"))
	bool bRenderHeadless = false;

	/** Hint for setting the r.GraphicsAdapter CVar when launching this cluster node. Note that this is distinct from the GPU Node Indices assigned to viewports. */
	UPROPERTY(EditAnywhere, BlueprintReadonly, Category = "Configuration", meta = (DisplayName = "Graphics Adapter"))
	int32 GraphicsAdapter = -1;

	/** Enables texture sharing for this cluster node */
	UPROPERTY(EditAnywhere, BlueprintReadonly, Category = "Configuration", meta = (DisplayName = "Enable Texture Share"))
	bool bEnableTextureShare = false;

#if WITH_EDITORONLY_DATA
	/** Locks the application window aspect ratio for easier resizing */
	UPROPERTY(EditAnywhere, Category = "Configuration", meta = (HideProperty))
	bool bFixedAspectRatio;
#endif

	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, EditFixedSize, Instanced, Category = "Configuration", meta = (DisplayThumbnail = false))
	TMap<FString, TObjectPtr<UDisplayClusterConfigurationViewport>> Viewports;

	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Configuration", meta = (DisplayName = "Custom Output Settings", DisplayThumbnail = false, ShowInnerProperties))
	TMap<FString, FDisplayClusterConfigurationPostprocess> Postprocess;

	// Media settings
	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Category = "Media", meta = (DisplayName = "Media", ShowOnlyInnerProperties))
	FDisplayClusterConfigurationMediaNodeBackbuffer MediaSettings;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditDefaultsOnly, Category = "Configuration", meta = (HideProperty))
	bool bIsVisible;

	UPROPERTY(EditDefaultsOnly, Category = "Configuration", meta = (HideProperty))
	bool bIsUnlocked;

	/** Binds a background preview image for easier output mapping */
	UPROPERTY(EditDefaultsOnly, Category = "Configuration")
	FDisplayClusterConfigurationExternalImage PreviewImage;
#endif

public:

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.4, "This property has been deprecated. Please use 'MediaSettings'.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "This property has been deprecated. Please use 'MediaSettings'."))
	FDisplayClusterConfigurationMediaViewport Media;
#endif

protected:
	virtual void GetObjectsToExport(TArray<UObject*>& OutObjects) override;
};

UCLASS(Blueprintable)
class DISPLAYCLUSTERCONFIGURATION_API UDisplayClusterConfigurationHostDisplayData : public UObject
{
	GENERATED_BODY()

public:
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPostEditChangeChainProperty, const FPropertyChangedChainEvent&);

	FOnPostEditChangeChainProperty OnPostEditChangeChainProperty;

public:
	UDisplayClusterConfigurationHostDisplayData();

private:
	#if WITH_EDITOR
	// UObject interface
	virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
	// End of UObject interface
	#endif

public:
	/** Custom name for the Host PC. No effect on nDisplay */
	UPROPERTY(EditAnywhere, Category = "Configuration")
	FText HostName;

	/** Arbitrary position of the Host PC in 2D workspace. No effect on nDisplay */
	UPROPERTY(EditAnywhere, Category = "Configuration", meta = (EditCondition = "bAllowManualPlacement"))
	FVector2D Position;

	/** Disables the automatic placement of Host PCs */
	UPROPERTY(EditAnywhere, Category = "Configuration")
	bool bAllowManualPlacement;

	/** Resolution of Host PC in pixels */
	UPROPERTY(EditAnywhere, Category = "Configuration", meta = (EditCondition = "bAllowManualSizing", AllowPreserveRatio))
	FVector2D HostResolution;

	/** Allows to manually resize the Host PC resolution */
	UPROPERTY(EditAnywhere, Category = "Configuration")
	bool bAllowManualSizing;

	/** Specify coordinates of the Host PC origin in relation to OS configuration */
	UPROPERTY(EditAnywhere, Category = "Configuration")
	FVector2D Origin;

	/** Specify custom and arbitrary color for a given Host PC */
	UPROPERTY(EditAnywhere, Category = "Configuration")
	FLinearColor Color;
	
	UPROPERTY()
	bool bIsVisible;

	UPROPERTY()
	bool bIsUnlocked;
};

UCLASS(Blueprintable)
class DISPLAYCLUSTERCONFIGURATION_API UDisplayClusterConfigurationCluster
	: public UDisplayClusterConfigurationData_Base
{
	GENERATED_BODY()

public:
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Configuration", meta = (ShowOnlyInnerProperties))
	FDisplayClusterConfigurationPrimaryNode PrimaryNode;

	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Configuration", meta = (ShowOnlyInnerProperties))
	FDisplayClusterConfigurationClusterSync Sync;

	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Configuration")
	FDisplayClusterConfigurationNetworkSettings Network;

	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Configuration")
	FDisplayClusterConfigurationFailoverSettings Failover;

	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, EditFixedSize, Instanced, Category = NDisplay, meta = (DisplayThumbnail = false, HideProperty))
	TMap<FString, TObjectPtr<UDisplayClusterConfigurationClusterNode>> Nodes;

#if WITH_EDITORONLY_DATA
	UPROPERTY(Instanced)
	TMap<FString, TObjectPtr<UDisplayClusterConfigurationHostDisplayData>> HostDisplayData;
#endif

public:

	//~ Begin UObject interface
	virtual void PostLoad() override;
	//~ End UObject interface

protected:
	virtual void GetObjectsToExport(TArray<UObject*>& OutObjects) override;

public:
	// Return all references to meshes from policy, and other
	UFUNCTION(BlueprintPure, Category = "NDisplay|Configuration")
	void GetReferencedMeshNames(TArray<FString>& OutMeshNames) const;

	// Nodes API
	UFUNCTION(BlueprintPure, Category = "NDisplay|Configuration")
	void GetNodeIds(TArray<FString>& OutNodeIds) const;

	UFUNCTION(BlueprintPure, Category = "NDisplay|Configuration")
	UDisplayClusterConfigurationClusterNode* GetNode(const FString& NodeId) const;
};

USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationDiagnostics
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = NDisplay)
	bool bSimulateLag = false;

	UPROPERTY(EditAnywhere, Category = NDisplay)
	float MinLagTime = 0.01f;

	UPROPERTY(EditAnywhere, Category = NDisplay)
	float MaxLagTime = 0.3f;
};

struct FDisplayClusterConfigurationDataMetaInfo
{
	EDisplayClusterConfigurationDataSource ImportDataSource;
	FString ImportFilePath;
};

////////////////////////////////////////////////////////////////
// Main configuration data container
UCLASS(Blueprintable, BlueprintType, PerObjectConfig, config = EditorPerProjectUserSettings)
class DISPLAYCLUSTERCONFIGURATION_API UDisplayClusterConfigurationData
	: public UDisplayClusterConfigurationData_Base
{
	GENERATED_BODY()

public:
	UDisplayClusterConfigurationData();

public:

	/** Returns primary node configuration. Otherwise nullptr. */
	UFUNCTION(BlueprintPure, Category = "NDisplay|Configuration")
	UDisplayClusterConfigurationClusterNode* GetPrimaryNode() const;

	/** Returns specific node configuration. Otherwise nullptr. */
	UFUNCTION(BlueprintPure, Category = "NDisplay|Configuration")
	UDisplayClusterConfigurationClusterNode* GetNode(const FString& NodeId) const;

	// Viewports API
	UFUNCTION(BlueprintPure, Category = "NDisplay|Configuration")
	UDisplayClusterConfigurationViewport* GetViewport(const FString& NodeId, const FString& ViewportId) const;

	/** Execute a function on each viewport in each cluster node */
	void ForEachViewport(const TFunction<void(const TObjectPtr<UDisplayClusterConfigurationViewport>&)>& Function) const;
	
	/**
	* Update\Create node postprocess
	*
	* @param PostprocessId - Unique postprocess name
	* @param Type          - Postprocess type id
	* @param Parameters    - Postprocess parameters
	* @param Order         - Control the rendering order of post-processing. Larger value is displayed last
	*
	* @return - true, if success
	*/
	UFUNCTION(BlueprintCallable, Category = "NDisplay|Configuration")
	bool AssignPostprocess(const FString& NodeId, const FString& PostprocessId, const FString& Type, TMap<FString, FString> Parameters, int32 Order = -1);

	UFUNCTION(BlueprintCallable, Category = "NDisplay|Configuration")
	bool RemovePostprocess(const FString& NodeId, const FString& PostprocessId);

	UFUNCTION(BlueprintCallable, Category = "NDisplay|Configuration")
	bool GetPostprocess(const FString& NodeId, const FString& PostprocessId, FDisplayClusterConfigurationPostprocess& OutPostprocess) const;

	UFUNCTION(BlueprintCallable, Category = "NDisplay|Configuration")
	bool GetProjectionPolicy(const FString& NodeId, const FString& ViewportId, FDisplayClusterConfigurationProjection& OutProjection) const;

	// Return all references to meshes from policy, and other
	UFUNCTION(BlueprintCallable, Category = "NDisplay|Configuration")
	void GetReferencedMeshNames(TArray<FString>& OutMeshNames) const;

public:
	/** Returns number of nodes in current cluster */
	uint32 GetNumberOfClusterNodes() const;

	/** Returns primary node IP address of current cluster */
	FString GetPrimaryNodeAddress() const;

public:
	FDisplayClusterConfigurationDataMetaInfo Meta;

	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Category = Configuration, meta = (DisplayAfter = "RenderFrameSettings"))
	FDisplayClusterConfigurationInfo Info;

	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Transient, Category = NDisplay, meta = (HideProperty))
	TObjectPtr<UDisplayClusterConfigurationScene> Scene;

	UPROPERTY(VisibleInstanceOnly, BlueprintReadOnly, Instanced, Category = NDisplay)
	TObjectPtr<UDisplayClusterConfigurationCluster> Cluster;

	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Category = Configuration, meta = (DisplayAfter = "Diagnostics"))
	TMap<FString, FString> CustomParameters;

	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Category = Configuration, meta = (DisplayAfter = "Info"))
	FDisplayClusterConfigurationDiagnostics Diagnostics;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Configuration, meta = (HidePropertyInstanceOnly, DisplayAfter = "DefaultFrameSizeRef"))
	FDisplayClusterConfigurationRenderFrame RenderFrameSettings;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (HideProperty))
	FDisplayClusterConfigurationICVFX_StageSettings StageSettings;

	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Category = Configuration, meta = (DisplayName = "Follow Local Player Camera", DisplayAfter = "CustomParameters"))
	bool bFollowLocalPlayerCamera = false;

	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Category = Configuration, meta = (DisplayName = "Exit When ESC Pressed", DisplayAfter = "bFollowLocalPlayerCamera"))
	bool bExitOnEsc = true;

	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Category = Configuration, meta = (DisplayName = "Override Viewports From .ndisplay", DisplayAfter = "bExitOnEsc"))
	bool bOverrideViewportsFromExternalConfig = false;

	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Category = Configuration, meta = (DisplayName = "Override Transforms From .ndisplay", DisplayAfter = "bOverrideViewportsFromExternalConfig"))
	bool bOverrideTransformsFromExternalConfig = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Configuration, meta = (DisplayName = "Global Media Settings"))
	FDisplayClusterConfigurationGlobalMediaSettings MediaSettings;

	/** Create empty config data. */
	static UDisplayClusterConfigurationData* CreateNewConfigData(UObject* Owner = nullptr, EObjectFlags ObjectFlags = RF_NoFlags);
#if WITH_EDITORONLY_DATA

public:
	UPROPERTY(config)
	FString PathToConfig;

	/** The path used when originally importing. */
	UPROPERTY()
	FString ImportedPath;

public:
	const static TSet<FString> RenderSyncPolicies;

	const static TSet<FString> InputSyncPolicies;

	const static TSet<FString> ProjectionPolicies;
	
//////////////////////////////////////////////////////////////////////////////////////////////
// Details Panel Property Referencers
//////////////////////////////////////////////////////////////////////////////////////////////
private:
	UPROPERTY(EditDefaultsOnly, Transient, Category = Configuration, meta = (PropertyPath = "StageSettings.DefaultFrameSize"))
	FDisplayClusterEditorPropertyReference DefaultFrameSizeRef;

#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
public:
	// UObject interface
	virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent) override;
	// End of UObject interface
#endif
};
