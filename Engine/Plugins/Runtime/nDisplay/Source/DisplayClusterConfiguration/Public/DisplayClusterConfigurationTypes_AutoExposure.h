// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"

#include "DisplayClusterConfigurationTypes_Enums.h"

#include "DisplayClusterConfigurationTypes_AutoExposure.generated.h"

USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfiguration_AutoExposureMeteringSettings
{
	GENERATED_BODY()

public:
	/**
	 * List of allowed outer viewports for auto-exposure metering when the override is enabled.
	 * If no viewports are specified, all outer viewports are used for metering.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Exposure", meta = (DisplayName = "Viewports"))
	TArray<FString> Viewports;

	/**
	 * List of allowed ICVFX cameras for auto-exposure metering when the override is enabled.
	 * If no ICVFX cameras are specified, all ICVFX cameras are used for metering.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Exposure", meta = (DisplayName = "ICVFX Cameras"))
	TArray<FString> ICVFXCameras;

	/**
	 * List of allowed cluster nodes for auto-exposure metering when the override is enabled.
	 * If no nodes are specified, all cluster nodes can be used for metering.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Exposure", meta = (DisplayName = "Nodes"))
	TArray<FString> Nodes;
};

USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfiguration_AutoExposureExcludeSettings
{
	GENERATED_BODY()

public:
	/** Outer viewports excluded from nDisplay auto-exposure (they use the default exposure behavior). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Exposure", meta = (DisplayName = "Viewports"))
	TArray<FString> Viewports;

	/** ICVFX cameras excluded from nDisplay auto-exposure (they use the default exposure behavior). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Exposure", meta = (DisplayName = "ICVFX Cameras"))
	TArray<FString> ICVFXCameras;

	/** Cluster nodes excluded from nDisplay auto-exposure (they use the default exposure behavior). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Exposure", meta = (DisplayName = "Nodes"))
	TArray<FString> Nodes;
};

USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfiguration_AutoExposureSettings
{
	GENERATED_BODY()

public:
	/** Enable/disable AutoExposure override. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Exposure", meta = (DisplayName = "Enable Auto Exposure Override"))
	bool bEnabled = false;

	/**
	 * Metering settings for auto-exposure metering when the override is enabled.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Exposure", meta = (DisplayName = "Meter Auto Exposure on Items"))
	FDisplayClusterConfiguration_AutoExposureMeteringSettings MeteringSettings;

	/**
	 * Exclude settings for auto-exposure.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Exposure", meta = (DisplayName = "Exclude Items from Auto Exposure"))
	FDisplayClusterConfiguration_AutoExposureExcludeSettings ExcludeSettings;
};
