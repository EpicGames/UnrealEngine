// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"

#include "DisplayClusterConfigurationTypes_Base.generated.h"

/**
 * Texture Replace Crop Origin parameter container
 */
USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FTextureCropOrigin
{
	GENERATED_BODY()

public:
	// Replace texture origin X location, in pixels
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (DisplayName = "X"))
	int32 X = 0;

	// Replace texture origin Y position, in pixels
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (DisplayName = "Y"))
	int32 Y = 0;
};

/**
 * Texture Replace Crop Size parameter container
 */
USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FTextureCropSize
{
	GENERATED_BODY()

public:
	// Replace texture crop width, in pixels
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (DisplayName = "W"))
	int32 W = 0;

	// Replace texture crop height, in pixels
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (DisplayName = "H"))
	int32 H = 0;
};

/**
 * Texture Replace Crop parameters container
 */
USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterReplaceTextureCropRectangle
{
	GENERATED_BODY()

public:
	FIntRect ToRect() const;

public:
	/** Texture Crop Origin */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (DisplayName = "Texture Crop Origin"))
	FTextureCropOrigin Origin;

	/** Texture Crop Size */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (DisplayName = "Texture Crop Size"))
	FTextureCropSize Size;
};

/**
 * All configuration UObjects should inherit from this class.
 */
USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationRectangle
{
	GENERATED_BODY()

public:
	FDisplayClusterConfigurationRectangle()
		: X(0), Y(0), W(0), H(0)
	{ }

	FDisplayClusterConfigurationRectangle(int32 _X, int32 _Y, int32 _W, int32 _H)
		: X(_X), Y(_Y), W(_W), H(_H)
	{ }

	FIntRect ToRect() const;

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay)
	int32 X;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay)
	int32 Y;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay)
	int32 W;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay)
	int32 H;
};

/**
 * All configuration UObjects should inherit from this class.
 */
UCLASS()
class DISPLAYCLUSTERCONFIGURATION_API UDisplayClusterConfigurationData_Base
	: public UObject
{
	GENERATED_BODY()

public:

	static FName GetExportedObjectsMemberName()
	{
		return GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationData_Base, ExportedObjects);
	}
	
	// UObject
	virtual void Serialize(FArchive& Ar) override;
	// ~UObject

#if WITH_EDITOR
protected:
	friend class FDisplayClusterConfiguratorKismetCompilerContext;
	
	/** Called by nDisplay compiler prior to compilation. */
	virtual void OnPreCompile(class FCompilerResultsLog& MessageLog) {}
#endif
	
protected:
	/** Called before saving to collect objects which should be exported as a sub object block. */
	virtual void GetObjectsToExport(TArray<UObject*>& OutObjects) {}

private:
	UPROPERTY(Export)
	TArray<TObjectPtr<UObject>> ExportedObjects;
};

USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationPolymorphicEntity
{
	GENERATED_BODY()

public:
	// Polymorphic entity type
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = NDisplay)
	FString Type;

	// Generic parameters map
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = NDisplay)
	TMap<FString, FString> Parameters;

#if WITH_EDITORONLY_DATA
	/**
	 * When a custom policy is selected from the details panel.
	 * This is needed in the event a custom policy is selected
	 * but the custom type is a default policy. This allows users
	 * to further customize default policies if necessary.
	 *
	 * EditAnywhere is required so we can manipulate the property
	 * through a handle. Details will hide it from showing.
	 */
	UPROPERTY(EditAnywhere, Category = NDisplay)
	bool bIsCustom = false;
#endif
};

USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationProjection
	: public FDisplayClusterConfigurationPolymorphicEntity
{
	GENERATED_BODY()

public:
	FDisplayClusterConfigurationProjection();

	/**
	* The ICVFX properties of the viewport will only be displayed in the UI if this function returns true.
	*/
	bool ShouldDisplayICVFXCategory() const;
};

USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationPostprocess
	: public FDisplayClusterConfigurationPolymorphicEntity
{
	GENERATED_BODY()

public:
	// Control postprocess rendering order. Bigger value rendered last
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = NDisplay)
	int32 Order = -1;
};

USTRUCT(BlueprintType)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationClusterItemReferenceList
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = NDisplay)
	TArray<FString> ItemNames;
};
