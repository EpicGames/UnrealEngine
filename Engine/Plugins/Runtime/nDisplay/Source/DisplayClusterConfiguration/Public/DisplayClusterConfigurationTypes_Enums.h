// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"

#include "DisplayClusterConfigurationTypes_Enums.generated.h"


UENUM()
enum class EDisplayClusterConfigurationDataSource : uint8
{
	Text UMETA(DisplayName = "Text file"),
	Json UMETA(DisplayName = "JSON file")
};

UENUM()
enum class UE_DEPRECATED(5.6, "Removed as unnecessary in the new failover implementation") EDisplayClusterConfigurationFailoverPolicy : uint8
{
	// No failover operations performed. The whole cluster gets terminated in case of any error
	Disabled                          UMETA(DisplayName = "Disabled"),

	// This policy allows to drop any secondary node out of cluster in case it's failed,
	// and let the others continue working. However, the whole cluster will be terminated if primary node fails.
	DropSecondaryNodesOnly            UMETA(DisplayName = "Drop S-node on fail"),
};

UENUM()
enum class EDisplayClusterConfigurationEyeStereoOffset : uint8
{
	None  UMETA(DisplayName = "No offset"),
	Left  UMETA(DisplayName = "Left eye of a stereo pair"),
	Right UMETA(DisplayName = "Right eye of a stereo pair")
};

UENUM()
enum class EDisplayClusterConfiguration_PostRenderBlur : uint8
{
	// Not use blur postprocess
	None      UMETA(DisplayName = "None"),

	// Blur viewport using Gaussian method
	Gaussian  UMETA(DisplayName = "Gaussian"),

	// Blur viewport using Dilate method
	Dilate    UMETA(DisplayName = "Dilate"),
};

/** Indicates the type of chromakey that is used within an ICVFX camera's inner frustum */
UENUM()
enum class EDisplayClusterConfigurationICVFX_ChromakeyType : uint8
{
	/** The entire inner frustum is rendered as chromakey */
	InnerFrustum       UMETA(DisplayName = "Inner Frustum"),

	/** Only actors specified in the custom chromakey actor list are rendered as chromakey */
	CustomChromakey    UMETA(DisplayName = "Custom Chromakey"),

	/**
	* Disable chromakey.
	* This value can be overridden by the viewport ICVFX settings,
	* by changing the ‘Override Chromakey Type’ property.
	*/
	Disabled            UMETA(DisplayName = "Disabled"),
};

/** Indicates the source of the chromakey settings when an ICVFX camera's inner frustum is rendered as chromakey */
UENUM()
enum class EDisplayClusterConfigurationICVFX_ChromakeySettingsSource : uint8
{
	/** The nDisplay stage actor's chromakey settings are used */
	Viewport       UMETA(DisplayName = "Use nDisplay Viewport Chromakey"),
	
	/** The ICVFX camera's chromakey settings are used */
	ICVFXCamera    UMETA(DisplayName = "Use ICVFX Camera Chromakey"),
};

UENUM()
enum class EDisplayClusterConfigurationICVFX_ChromakeySource : uint8
{
	// Disable chromakey rendering
	None                   UMETA(DisplayName = "None"),

	// Fill whole camera frame with chromakey color
	FrameColor             UMETA(DisplayName = "Chromakey Color"),

	// Capture to texture RTT from scene layers
	ChromakeyRenderTexture UMETA(DisplayName = "Chromakey Render Texture"),
};

/** How to render Light Cards in relation to the inner frustum. */
UENUM()
enum class EDisplayClusterConfigurationICVFX_LightcardRenderMode : uint8
{
	/** Render Light Cards over the inner frustum.  Light Cards can be directly visible in camera. */
	Over    UMETA(DisplayName = "Lightcard Over Frustum"),

	/** Render Light Cards under the inner frustum. Light Cards will not be directly visible in camera. */
	Under   UMETA(DisplayName = "Lightcard Under Frustum"),
};

/** How to render a Light Card Actor in relation to the inner frustum. */
UENUM()
enum class EDisplayClusterConfigurationICVFX_PerLightcardRenderMode : uint8
{
	/** It is determined by nDisplay settings. */
	Default UMETA(DisplayName = "Default"),

	/** Render Light Card Actor over the inner frustum.  Light Card Actor can be directly visible in camera. */
	Over    UMETA(DisplayName = "Lightcard Over Frustum"),

	/** Render Light Card Actor under the inner frustum. Light Card Actor will not be directly visible in camera. */
	Under   UMETA(DisplayName = "Lightcard Under Frustum"),
};

/** Override the lightcard rendering order for the viewport. */
UENUM()
enum class EDisplayClusterConfigurationICVFX_OverrideLightcardRenderMode : uint8
{
	// Use global lightcard mode from StageSettings for this viewport
	Default    UMETA(DisplayName = "Default"),

	// Disable lightcard rendering for this viewport
	Disabled    UMETA(DisplayName = "Disabled"),

	/** Always render the In-Camera frame over lightcards for this viewport. */
	Over    UMETA(DisplayName = "Lightcard Over Frustum"),

	/** Always render the In-Camera frame under lightcards for this viewport. */
	Under   UMETA(DisplayName = "Lightcard Under Frustum"),
};

/** Override the chromakey type for the viewport. */
UENUM()
enum class EDisplayClusterConfigurationICVFX_OverrideChromakeyType : uint8
{
	// Use the chromakey type from ICVFX camera
	Default    UMETA(DisplayName = "Default"),

	// The entire inner frustum is rendered as chromakey for this viewport
	InnerFrustum       UMETA(DisplayName = "Inner Frustum"),

	/** Only actors specified in the custom chromakey actor list are rendered as chromakey */
	CustomChromakey    UMETA(DisplayName = "Custom Chromakey"),

	// Disable chromakey
	Disabled UMETA(DisplayName = "Disabled Chromakey"),
};

UENUM()
enum class EDisplayClusterConfigurationICVFX_OverrideCameraRenderMode : uint8
{
	// Use default rendering rules
	Default                 UMETA(DisplayName = "Default"),

	// Disable camera frame render for this viewport
	Disabled                UMETA(DisplayName = "Disabled"),

	// Disable chromakey render for this viewport
	DisableChromakey        UMETA(DisplayName = "Disabled Chromakey"),

	// Disable chromakey markers render for this viewport
	DisableChromakeyMarkers UMETA(DisplayName = "Disabled Markers"),
};

UENUM()
enum class EDisplayClusterConfigurationViewport_StereoMode : uint8
{
	// Render incamera frame over lightcard
	Default  UMETA(DisplayName = "Default"),

	// Force monoscopic render mode for this viewport (performance)
	ForceMono   UMETA(DisplayName = "Force Mono"),
};

UENUM()
enum class EDisplayClusterConfigurationRenderFamilyMode : uint8
{
	// Render all viewports to unique RenderTargets
	None                          UMETA(DisplayName = "Disabled"),

	// Merge views by ViewFamilyGroupNum
	AllowMergeForGroups           UMETA(DisplayName = "Groups"),

	// Merge views by ViewFamilyGroupNum and stereo
	AllowMergeForGroupsAndStereo  UMETA(DisplayName = "GroupsAndStereo"),

	// Use rules to merge views to minimal num of families (separate by: buffer_ratio, viewExtension, max RTT size)
	MergeAnyPossible              UMETA(DisplayName = "AnyPossible"),
};

UENUM()
enum class EDisplayClusterConfigurationCameraMotionBlurMode : uint8
{
	/** Subtract blur due to all global motion of the ICVFX camera, but preserve blur from object motion. */
	Off          UMETA(DisplayName = "All Camera Blur Off"),

	/** Allow blur from camera motion. This option should not normally be used for shooting, but may be useful for diagnostic purposes. */
	On           UMETA(DisplayName = "ICVFX Camera Blur On"),

	/** Subtract blur due to motion of the ICVFX camera relative to the nDisplay root, but preserve blur from both object motion and movement of the nDisplay root, which can be animated to represent vehicular motion through an environment. */
	Override     UMETA(DisplayName = "ICVFX Camera Blur Off"),
};

UENUM()
enum class EDisplayClusterConfigurationViewportOverscanMode : uint8
{
	Pixels   UMETA(DisplayName = "Pixels"),
	Percent  UMETA(DisplayName = "Percent")
};

/**
* Defines how the overscan blend width is interpreted.
* Allows blending between tiles to be specified in
* a percentage of the overscan size, or disabled entirely.
*/
UENUM()
enum class EDisplayClusterConfigurationViewportOverscanBlendMode : uint8
{
	/** Disable blending across overscan region */
	Disable     UMETA(DisplayName = "Disable"),

	/** Blend width defined as 25% of the overscan size */
	Percent25   UMETA(DisplayName = "25%"),

	/** Blend width defined as 50% of the overscan size */
	Percent50   UMETA(DisplayName = "50%"),

	/** Blend width defined as 75% of the overscan size */
	Percent75   UMETA(DisplayName = "75%"),
};


UENUM()
enum class EDisplayClusterConfigurationRenderMode : uint8
{
	Mono        UMETA(DisplayName = "Mono"),
	SideBySide  UMETA(DisplayName = "Stereo: Side By Side"),
	TopBottom   UMETA(DisplayName = "Stereo: Top Bottom")
};

UENUM()
enum class EDisplayClusterConfigurationViewportCustomFrustumMode : uint8
{
	Percent  UMETA(DisplayName = "Percentage"),
	Pixels   UMETA(DisplayName = "Pixels")
};

UENUM()
enum class EDisplayClusterConfigurationViewportLightcardOCIOMode : uint8
{
	/** Use nDisplay Viewport OCIO. */
	nDisplay UMETA(DisplayName = "Use nDisplay Viewport OCIO"),

	/** Use Custom Light Cards OCIO. */
	Custom UMETA(DisplayName = "Use Custom Light Cards OCIO"),

	/** None. */
	None UMETA(DisplayName = "None"),
};

/** Determines where the data for RootActor GetPreviewSettings() will come from: */
UENUM()
enum class EDisplayClusterConfigurationRootActorPreviewSettingsSource : uint8
{
	// The preview settings will be obtained from the RootActor properties.
	RootActor,

	// The preview settings will be retrieved from DC ViewportManagerConfiguration.
	// This option is used when the preview settings need to be customized in a different way, but not to change the RootActor properties.
	Configuration
};

/** Determines Upscaling method.
* Not used as an enum in the UI.
* Used in the FDisplayClusterConfigurationUpscalerSettingsDetailCustomization.
*/
UENUM()
enum class EDisplayClusterConfigurationUpscalingMethod : uint8
{
	/** Use configuration project settings. */
	Default = 0 UMETA(DisplayName = "Default"),

	/** No Antialiasing. */
	Disable UMETA(DisplayName = "Disable"),

	/** Fast Approximate Anti-Aliasing. */
	FXAA UMETA(DisplayName = "Fast Approximate Anti-Aliasing (FXAA)"),

	/** Multisample Anti-Aliasing. (Only supported with forward shading.) */
	MSAA UMETA(DisplayName = "Multisample Anti-Aliasing (MSAA)"),

	/** Temporal Anti-Aliasing. */
	TAA UMETA(DisplayName = "Temporal Anti-Aliasing (TAA)"),

	/** Temporal Super-Resolution. Use TSR upscaler with built-in AntiAliasing. */
	TSR UMETA(DisplayName = "Temporal Super-Resolution (TSR)"),
};
