// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"

#include "Components/ActorComponent.h"
#include "ActorLayerUtilities.h"

#include "CineCameraActor.h"

#include "DisplayClusterConfigurationTypes_AutoExposure.h"
#include "DisplayClusterConfigurationTypes_Upscaler.h"
#include "DisplayClusterConfigurationTypes_Media.h"
#include "DisplayClusterConfigurationTypes_PostRender.h"
#include "DisplayClusterConfigurationTypes_Postprocess.h"
#include "DisplayClusterConfigurationTypes_OCIO.h"
#include "DisplayClusterConfigurationTypes_ViewportOverscan.h"

#include "Containers/DisplayClusterShader_Enums.h"

#include "DisplayClusterConfigurationTypes_ICVFX.generated.h"

class UCineCameraComponent;
class UDisplayClusterConfigurationViewport;

USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationICVFX_LightcardCustomOCIO
{
	GENERATED_BODY()

public:
	/** Return LightCard OCIO configuration for the specified viewport. Return nullptr if no OCIO. */
	const FOpenColorIOColorConversionSettings* FindOCIOConfiguration(const FString& InViewportId) const;

public:
	/** Apply this OpenColorIO configuration to all viewports. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Light Card", meta = (DisplayName = "All Viewports Color Configuration", ConfigurationMode = "Viewports"))
	FDisplayClusterConfigurationOCIOConfiguration AllViewportsOCIOConfiguration;

	/** Apply an OpenColorIO configuration on a per-viewport or group-of-viewports basis. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Light Card", meta = (DisplayName = "Per-Viewport OCIO Overrides", ConfigurationMode = "Viewports"))
	TArray<FDisplayClusterConfigurationOCIOProfile> PerViewportOCIOProfiles;
};

USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationICVFX_ViewportOCIO
{
	GENERATED_BODY()

public:
	/** Return OCIO configuration for the specified viewport. Return nullptr if no OCIO. */
	const FOpenColorIOColorConversionSettings* FindOCIOConfiguration(const FString& InViewportId) const;

public:
	/** Apply this OpenColorIO configuration to all viewports. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OCIO", meta = (DisplayName = "All Viewports Color Configuration", ConfigurationMode = "Viewports"))
	FDisplayClusterConfigurationOCIOConfiguration AllViewportsOCIOConfiguration;

	/** Apply an OpenColorIO configuration on a per-viewport or group-of-viewports basis. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OCIO", meta = (DisplayName = "Per-Viewport OCIO Overrides", ConfigurationMode = "Viewports"))
	TArray<FDisplayClusterConfigurationOCIOProfile> PerViewportOCIOProfiles;
};

USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationICVFX_CameraOCIO
{
	GENERATED_BODY()

public:
	/** Return InCamera OCIO configuration for the specified cluster node. Return nullptr if no OCIO. */
	const FOpenColorIOColorConversionSettings* FindOCIOConfiguration(const FString& InClusterNodeId) const;

	/** Returns true if the InCamera OCIO configuration is the same for the input nodes. */
	bool IsInnerFrustumViewportSettingsEqual(const FString& InClusterNodeId1, const FString& InClusterNodeId2) const;

	/** Returns true if the Chromakey OCIO configuration is the same for the input nodes. */
	bool IsChromakeyViewportSettingsEqual(const FString& InClusterNodeId1, const FString& InClusterNodeId2) const;

public:
	/** OCIO Display look configuration for all nodes */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OCIO", meta = (DisplayName = "All Nodes Color Configuration", ConfigurationMode = "ClusterNodes"))
	FDisplayClusterConfigurationOCIOConfiguration AllNodesOCIOConfiguration;

	/** Apply an OpenColorIO configuration on a per-node or group-of-nodes basis. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OCIO", meta = (DisplayName = "Per-Node OCIO Overrides", ConfigurationMode = "ClusterNodes"))
	TArray<FDisplayClusterConfigurationOCIOProfile> PerNodeOCIOProfiles;
};

USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationICVFX_LightcardOCIO
{
	GENERATED_BODY()

public:
	/** Return LightCard OCIO configuration for the specified viewport. Return nullptr if no OCIO. */
	const FOpenColorIOColorConversionSettings* FindOCIOConfiguration(const FString& InViewportId, const FDisplayClusterConfigurationICVFX_ViewportOCIO& InViewportOCIO) const;

public:
	/** Light Cards OCIO mode. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Light Card|OCIO", meta = (DisplayName = "Light Cards OCIO"))
	EDisplayClusterConfigurationViewportLightcardOCIOMode LightcardOCIOMode = EDisplayClusterConfigurationViewportLightcardOCIOMode::nDisplay;

	/** Custom OpenColorIO configuration for Light Cards. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Light Card|OCIO", meta = (DisplayName = "Custom Light Cards OCIO"))
	FDisplayClusterConfigurationICVFX_LightcardCustomOCIO CustomOCIO;
};

USTRUCT(Blueprintable, meta = (DefaultSubstitutions = "LayersTooltip = Actor Layers, ActorsTooltip = Actor references"))
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationICVFX_VisibilityList
{
	GENERATED_BODY()

public:
	/** returns true if there is at least one valid value in the visibility list. */
	bool IsVisibilityListValid() const;

public:
	/** Actor layers.
	* Warning: This option has a very expensive performance cost, especially on big projects.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (DisplayName = "Layers"))
	TArray<FActorLayer> ActorLayers;

	/** Actor references */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (ToolTip = "{ActorsTooltip}"))
	TArray<TSoftObjectPtr<AActor>> Actors;

	/** Reference to RootActor components by names */
	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Category = NDisplay)
	TArray<FString> RootActorComponentNames; //@todo change link, now by names

	/** Actors added automatically during a session */
	TArray<TSoftObjectPtr<AActor>> AutoAddedActors;
};

/** Custom resolution settings for ICVFX. */
USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationICVFX_CustomSize
{
	GENERATED_BODY()

public:
	/** Enabling this option will allow these custom settings to be used instead of the default settings. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (DisplayName = "Use Custom Resolution"))
	bool bUseCustomSize = false;

	/** Custom Width, in pixels. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (DisplayName = "Custom Width", ClampMin = "32", UIMin = "32", EditCondition = "bUseCustomSize"))
	int CustomWidth = 2560;

	/** Custom Height, in pixels. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (DisplayName = "Custom Height", ClampMin = "32", UIMin = "32", EditCondition = "bUseCustomSize"))
	int CustomHeight = 1440;

	/** Automatically adapts resolution to be proportional
	* to the filmback aspect ratio for the ICVFX Camera
	* while continuing to render the same total amount of
	* pixels as specified by the size. Enabling this can help
	* avoid visual artifacts without affecting the
	* performance budget.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (DisplayName = "Adapt Resolution to Match Filmback Aspect Ratio", EditCondition = "bUseCustomSize"))
	bool bAdaptSize = true;
};

/** Default resolution settings for ICVFX. */
USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationICVFX_Size
{
	GENERATED_BODY()

public:
	/** The default width of In-Cameras, in pixels. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (DisplayName = "Default Width", ClampMin = "32", UIMin = "32"))
	int Width = 2560;

	/** The default height of In-Cameras, in pixels. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (DisplayName = "Default Height", ClampMin = "32", UIMin = "32"))
	int Height = 1440;

	/** Automatically adapts the default frame resolution to be proportional
	* to the filmback aspect ratio for the ICVFX Camera
	* while continuing to render the same total amount of
	* pixels as specified by the size. Enabling this can help
	* avoid visual artifacts without affecting the
	* performance budget.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (DisplayName = "Adapt Resolution to Match Filmback Aspect Ratio"))
	bool bAdaptSize = true;
};

USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationICVFX_OverlayAdvancedRenderSettings
{
	GENERATED_BODY()

public:
	// Allow ScreenPercentage 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (ClampMin = "0.05", UIMin = "0.05", ClampMax = "10.0", UIMax = "10.0"))
	float BufferRatio = 1;

	// Performance: Render to scale RTT, resolved with shader to viewport (Custom value)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (ClampMin = "0.01", UIMin = "0.01", ClampMax = "1.0", UIMax = "1.0"))
	float RenderTargetRatio = 1;

	/**
	* Specifies the GPU index for the ICVFX viewport (LC or CK).
	* A value of '-1' means using the same GPU index as defined in the base viewport:
	* - The In-Camera viewport is used as the base viewport for the Chromakey (CK) viewport.
	* - An outer viewport is used as the base viewport for the Light Card (LC) viewport.
	* Used to improve rendering performance by spreading the load across multiple GPUs.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (DisplayName = "GPU Index", ClampMin = "-1", UIMin = "-1", ClampMax = "8", UIMax = "8"))
	int GPUIndex = INDEX_NONE;

	/**
	* Specifies the GPU index for the ICVFX viewport (LC or CK) in stereo rendering for the second eye.
	* A value of '-1' means to use the value from the GPU Index parameter. (the same value is used for both eyes).
	* Used to improve rendering performance by spreading the load across multiple GPUs.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (DisplayName = "Stereo GPU Index", ClampMin = "-1", UIMin = "-1", ClampMax = "8", UIMax = "8"))
	int StereoGPUIndex = INDEX_NONE;

	// Performance: force monoscopic render, resolved to stereo viewport
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay)
	EDisplayClusterConfigurationViewport_StereoMode StereoMode = EDisplayClusterConfigurationViewport_StereoMode::Default;

	// Experimental: Support special frame builder mode - merge viewports to single viewfamily by group num
	// [not implemented yet]
	UPROPERTY()
	int RenderFamilyGroup = -1;
};

USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationICVFX_ChromakeyRenderSettings
{
	GENERATED_BODY()

public:
	// Note: We need to explicitly disable warnings on these constructors/operators for clang to be happy with deprecated variables
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FDisplayClusterConfigurationICVFX_ChromakeyRenderSettings() = default;
	~FDisplayClusterConfigurationICVFX_ChromakeyRenderSettings() = default;
	FDisplayClusterConfigurationICVFX_ChromakeyRenderSettings(const FDisplayClusterConfigurationICVFX_ChromakeyRenderSettings&) = default;
	FDisplayClusterConfigurationICVFX_ChromakeyRenderSettings(FDisplayClusterConfigurationICVFX_ChromakeyRenderSettings&&) = default;
	FDisplayClusterConfigurationICVFX_ChromakeyRenderSettings& operator=(const FDisplayClusterConfigurationICVFX_ChromakeyRenderSettings&) = default;
	FDisplayClusterConfigurationICVFX_ChromakeyRenderSettings& operator=(FDisplayClusterConfigurationICVFX_ChromakeyRenderSettings&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

public:
	/** true if chromakey viewport is used in  ICVFX. */
	bool ShouldUseChromakeyViewport(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings) const;

public:
	/** Set to True to use custom chromakey content. */
	UE_DEPRECATED(5.3, "Use the ChromakeyType enum in FDisplayClusterConfigurationICVFX_ChromakeySettings instead")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use the Chromakey Type enum in the chromakey settings instead"))
	bool bEnable_DEPRECATED = false;

	// Replace the texture of the camera viewport from this chromakey RTT
	UPROPERTY(BlueprintReadWrite,Category = NDisplay)
	bool bReplaceCameraViewport = false;

	/** Custom resolution of the chromakey RTT. If it is not used, the Default Frame Resolution value is used by default. */
	UE_DEPRECATED(5.5, "Use the 'ChromakeySizeMult' instead")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use the 'Chromakey Resolution Multiplier' instead"))
	FDisplayClusterConfigurationICVFX_CustomSize CustomSize_DEPRECATED;

	/** Resolution multiplier for RTT chromakey. The default is the ICVFX camera frame resolution. */
	UPROPERTY(BlueprintReadWrite, Category = NDisplay, meta = (DisplayName = "Chromakey Resolution Multiplier", ClampMin = "0.05", UIMin = "0.05", ClampMax = "1.0", UIMax = "1.0"))
	float ChromakeySizeMult = 1.f;

	/** Content specified here will be overridden to use the chromakey color specified and include chromakey markers if enabled. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (DisplayName = "Custom Chromakey Content"))
	FDisplayClusterConfigurationICVFX_VisibilityList ShowOnlyList;

	// Replace viewport render from source texture
	UPROPERTY(BlueprintReadWrite, Category = NDisplay)
	FDisplayClusterConfigurationPostRender_Override Replace;

	/** Apply blur to the Custom Chromakey content. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (DisplayName = "Post Process Blur"))
	FDisplayClusterConfigurationPostRender_BlurPostprocess PostprocessBlur;

	UPROPERTY(BlueprintReadWrite, Category = NDisplay)
	FDisplayClusterConfigurationPostRender_GenerateMips GenerateMips;

	// Advanced render settings
	UPROPERTY(BlueprintReadWrite, Category = NDisplay)
	FDisplayClusterConfigurationICVFX_OverlayAdvancedRenderSettings AdvancedRenderSettings;
};

USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationICVFX_ChromakeyMarkers
{
	GENERATED_BODY()

public:
	FDisplayClusterConfigurationICVFX_ChromakeyMarkers();

public:
	/** True to display Chromakey Markers within the inner frustum. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (DisplayName = "Enable Chromakey Markers"))
	bool bEnable = true;

	/** Marker Color */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay)
	FLinearColor MarkerColor = { 0.f, 0.25f, 0.f, 1.f };

	/** Texture to use as the chromakey marker tile. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay)
	TObjectPtr<UTexture> MarkerTileRGBA = nullptr;

	/** Scale value for the size of each chromakey marker tile. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (ClampMin = "0", UIMin = "0", DisplayName = "Marker Scale"))
	float MarkerSizeScale = 0.5;

	/** Distance value between each chromakey marker tile. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (ClampMin = "0", UIMin = "0"))
	float MarkerTileDistance = 1.5;

	/** Offset value for the chromakey marker tiles, normalized to the tile distance.  Adjust placement of the chromakey markers within the composition of the camera framing.  Whole numbers will offset chromakey markers by a cyclical amount and have no visual change. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay)
	FVector2D MarkerTileOffset = { 0, 0 };
};

USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationICVFX_ChromakeySettings
{
	GENERATED_BODY()

public:
	FDisplayClusterConfigurationICVFX_ChromakeySettings()
		// Default chromakey color is (0,128,0)
		: ChromakeyColor(0, 0.5f, 0)
	{ }

public:
	/** Get Type of Chromakey: disabled, filling all internal frustum or rendering Chromakey actors. */
	EDisplayClusterShaderParametersICVFX_ChromakeySource GetChromakeyType(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings) const;

	/** Get the Chromakey render settings
	 * Return nullptr if Chromakey: is disabled or its type does not use rendering
	 */
	const FDisplayClusterConfigurationICVFX_ChromakeyRenderSettings* GetChromakeyRenderSettings(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings) const;

	/** Get the writable Chromakey render settings
	 * Return nullptr if Chromakey: is disabled or its type does not use rendering
	 */
	FDisplayClusterConfigurationICVFX_ChromakeyRenderSettings* GetWritableChromakeyRenderSettings(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings);

	/** Get the Chromakey color value */
	const FLinearColor& GetChromakeyColor(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings) const;

	/** Get the Chromakey overlap color value */
	const FLinearColor& GetOverlapChromakeyColor(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings) const;

	/** Get the Chromakey markers settings
	 * Return nullptr if Chromakey markers is :disabled
	 */
	const FDisplayClusterConfigurationICVFX_ChromakeyMarkers* GetChromakeyMarkers(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings) const;
	
	/** Get the Chromakey overlap markers settings
	 * Return nullptr if Chromakey overlap markers is disabled
	 */
	const FDisplayClusterConfigurationICVFX_ChromakeyMarkers* GetOverlapChromakeyMarkers(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings) const;

private:
	/** Checks the input parameter and returns it or nullptr if it cannot be used. */
	const FDisplayClusterConfigurationICVFX_ChromakeyMarkers* ImplGetChromakeyMarkers(const FDisplayClusterConfigurationICVFX_ChromakeyMarkers* InValue) const;

public:
	/** Set to True to fill the inner frustum with the specified Chromakey Color. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (DisplayName = "Enable Inner Frustum Chromakey"))
	bool bEnable = false;

	/** The type of chromakey to use */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay)
	EDisplayClusterConfigurationICVFX_ChromakeyType ChromakeyType = EDisplayClusterConfigurationICVFX_ChromakeyType::InnerFrustum;

	/** The source of the chromakey settings, either the settings on the ICVFX camera or the global settings on the root actor */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (DisplayName = "ICVFX Camera Chromakey"))
	EDisplayClusterConfigurationICVFX_ChromakeySettingsSource ChromakeySettingsSource = EDisplayClusterConfigurationICVFX_ChromakeySettingsSource::Viewport;

	/** Chromakey Color */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay)
	FLinearColor ChromakeyColor;

	/** Configure a custom chromakey based on content that will appear in the inner frustum, rather than the entire inner frustum. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (DisplayName = "Custom Chromakey"))
	FDisplayClusterConfigurationICVFX_ChromakeyRenderSettings ChromakeyRenderTexture;

	/** Display Chromakey Markers to facilitate camera tracking in post production. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay)
	FDisplayClusterConfigurationICVFX_ChromakeyMarkers ChromakeyMarkers;
};

/** Chromakey settings that are global for all ICVFX cameras in a root actor */
USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationICVFX_GlobalChromakeySettings
{
	GENERATED_BODY()

public:
	FDisplayClusterConfigurationICVFX_GlobalChromakeySettings()
		// Default chromakey color is (0,128,0)
		: ChromakeyColor(0, 0.5f, 0)
	{ }

public:
	/** Chromakey Color */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay)
	FLinearColor ChromakeyColor;

	/** Display Chromakey Markers to facilitate camera tracking in post production. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay)
	FDisplayClusterConfigurationICVFX_ChromakeyMarkers ChromakeyMarkers;
};

USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationICVFX_LightcardRenderSettings
{
	GENERATED_BODY()

public:
	// override the texture of the target viewport from this lightcard RTT
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay)
	bool bReplaceViewport = false;

	// Override viewport render from source texture
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay)
	FDisplayClusterConfigurationPostRender_Override Replace;

	UPROPERTY()
	FDisplayClusterConfigurationPostRender_BlurPostprocess PostprocessBlur;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay)
	FDisplayClusterConfigurationPostRender_GenerateMips GenerateMips;

	// Advanced render settings
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay)
	FDisplayClusterConfigurationICVFX_OverlayAdvancedRenderSettings AdvancedRenderSettings;
};

USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationICVFX_LightcardSettings
{
	GENERATED_BODY()

public:
	/* Returns true if the use of the LightCard is allowed */
	bool ShouldUseLightCard(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings) const;

	/* Returns true if the use of the UVLightCard is allowed */
	bool ShouldUseUVLightCard(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings) const;

	/** Get lightcard render mode. */
	EDisplayClusterShaderParametersICVFX_LightCardRenderMode GetLightCardRenderMode(const EDisplayClusterConfigurationICVFX_PerLightcardRenderMode InPerLightcardRenderMode, const UDisplayClusterConfigurationViewport* InViewportConfiguration = nullptr) const;

	/** Get LC render mode override. */
	EDisplayClusterShaderParametersICVFX_LightCardRenderMode GetLightCardRenderModeOverride(const UDisplayClusterConfigurationViewport* InViewportConfiguration) const;

public:
	/** Enable Light Cards */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Light Cards", meta = (DisplayName = "Enable Light Cards"))
	bool bEnable = true;

	/** Enable\Disable freeze rendering for lightcards when outer viewports rendering also freezed. This will impact performance. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Light Cards", meta = (DisplayName = "Ignore Freezing of Outer Viewports for Lightcards"))
	bool bIgnoreOuterViewportsFreezingForLightcards = true;

	/** Specify how to render Light Cards in relation to the inner frustum. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Light Cards", meta = (DisplayName = "Blending Mode"))
	EDisplayClusterConfigurationICVFX_LightcardRenderMode Blendingmode = EDisplayClusterConfigurationICVFX_LightcardRenderMode::Under;

	// Render actors from this layers to lightcard textures
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Light Cards", meta = (DisplayName = "Light Cards Content", ToolTip = "Content specified here will be treated as a Light Card and adhere to the Blending Mode setting", Substitutions = "LayersTooltip = Layers containing Light Cards, ActorsTooltip = Light Card Actors"))
	FDisplayClusterConfigurationICVFX_VisibilityList ShowOnlyList;

	// Configure global render settings for this viewport
	UPROPERTY(BlueprintReadWrite, Category = NDisplay)
	FDisplayClusterConfigurationICVFX_LightcardRenderSettings RenderSettings;

	/** OpenColorIO configuration for the lightcards. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OCIO", meta = (DisplayName = "Lightcard OCIO"))
	FDisplayClusterConfigurationICVFX_LightcardOCIO LightcardOCIO;

	// Enable using outer viewport Color Grading from DCRA for lightcard rendering
	UPROPERTY()
	bool bEnableOuterViewportColorGrading = false;
};

USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationICVFX_CameraAdvancedRenderSettings
{
	GENERATED_BODY()

public:
	// Performance: Render to scale RTT, resolved with shader to viewport (Custom value)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (ClampMin = "0.01", UIMin = "0.01", ClampMax = "1.0", UIMax = "1.0"))
	float RenderTargetRatio = 1;

	/**
	* Specifies the GPU index for the ICVFX camera viewport.
	* Value '-1' means do not use multi-GPU
	* Used to improve rendering performance by spreading the load across multiple GPUs.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (DisplayName = "GPU Index", ClampMin = "-1", UIMin = "-1", ClampMax = "8", UIMax = "8"))
	int GPUIndex = INDEX_NONE;

	/**
	* Specifies the GPU index for the ICVFX camera viewport in stereo rendering for the second eye.
	* A value of '-1' means to use the value from the GPU Index parameter. (the same value is used for both eyes).
	* Used to improve rendering performance by spreading the load across multiple GPUs.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (DisplayName = "Stereo GPU Index", ClampMin = "-1", UIMin = "-1", ClampMax = "8", UIMax = "8"))
	int StereoGPUIndex = INDEX_NONE;

	// Performance: force monoscopic render, resolved to stereo viewport
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay)
	EDisplayClusterConfigurationViewport_StereoMode StereoMode = EDisplayClusterConfigurationViewport_StereoMode::Default;

	// Experimental: Support special frame builder mode - merge viewports to single viewfamily by group num
	// [not implemented yet]
	UPROPERTY()
	int RenderFamilyGroup = -1;
};

USTRUCT(BlueprintType)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationICVFX_CameraRenderSettings
{
	GENERATED_BODY()

public:
	FDisplayClusterConfigurationICVFX_CameraRenderSettings();

public:
	/** Custom resolution of the ICVFX Camera. If it is not used, the Default Frame Resolution value is used by default. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (DisplayName = "Custom Frame Resolution"))
	FDisplayClusterConfigurationICVFX_CustomSize CustomFrameSize;

	// Camera render order, bigger value is over
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay)
	int RenderOrder = -1;

	UPROPERTY()
	FDisplayClusterConfigurationViewport_CustomPostprocess CustomPostprocess;

	// Use postprocess settings from camera component
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay)
	bool bUseCameraComponentPostprocess = true;

	// Replace viewport render from source texture
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (DisplayName = "Replace Output With a Texture"))
	FDisplayClusterConfigurationPostRender_Override Replace;

	// Media settings
	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Category = NDisplay, meta = (DisplayName = "Media", ShowOnlyInnerProperties))
	FDisplayClusterConfigurationMediaICVFX Media;

	UPROPERTY()
	FDisplayClusterConfigurationPostRender_BlurPostprocess PostprocessBlur;

	/** Mipmapping can help avoid visual artifacts when the inner frustum is rendered at a lower resolution than specified in the configuration and is smaller on screen than the available pixels on the display device. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (DisplayName = "Mipmapping", Simplified))
	FDisplayClusterConfigurationPostRender_GenerateMips GenerateMips;

	// Advanced render settings
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay)
	FDisplayClusterConfigurationICVFX_CameraAdvancedRenderSettings AdvancedRenderSettings;

public:
	/** Propagates general render related settings to the view info. */
	void SetupViewInfo(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings, FMinimalViewInfo& InOutViewInfo) const;
};


USTRUCT(BlueprintType)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationICVFX_CameraMotionBlurOverridePPS
{
	GENERATED_BODY()

public:
	/** If enabled, override the overall motion blur settings that would otherwise come from the current post-process volume or Cine Camera. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = NDisplay, meta = (DisplayName = "Enable Settings Override"))
	bool bReplaceEnable = false;

	/** Strength of motion blur, 0:off. */
	UPROPERTY(interp, BlueprintReadWrite, Category = NDisplay, meta = (ClampMin = "0.0", ClampMax = "1.0", DisplayName = "Intensity"))
	float MotionBlurAmount = 1;

	/** Max distortion caused by motion blur in percent of the screen width, 0:off */
	UPROPERTY(interp, BlueprintReadWrite, Category = NDisplay, meta = (ClampMin = "0.0", ClampMax = "100.0", DisplayName = "Max"))
	float MotionBlurMax = 50;

	/** The minimum projected screen radius for a primitive to be drawn in the velocity pass.Percentage of screen width, smaller numbers cause more draw calls, default: 4 % */
	UPROPERTY(interp, BlueprintReadWrite, Category = NDisplay, meta = (ClampMin = "0.0", UIMax = "100.0", DisplayName = "Per Object Size"))
	float MotionBlurPerObjectSize = 4;
};

USTRUCT(BlueprintType)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationICVFX_CameraMotionBlur
{
	GENERATED_BODY()

public:
	/** Specify the motion blur mode for the inner frustum, correcting for the motion of the camera. Blur due to camera motion will be incorrectly doubled in the physically exposed image if there is already camera blur applied to the inner frustum. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay)
	EDisplayClusterConfigurationCameraMotionBlurMode MotionBlurMode = EDisplayClusterConfigurationCameraMotionBlurMode::Override;

	/** Translation Scale */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay)
	float TranslationScale = 1.f;

	/** Motion Blur Settings Override */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (DisplayName = "Motion Blur Settings Override"))
	FDisplayClusterConfigurationICVFX_CameraMotionBlurOverridePPS MotionBlurPPS;

public:
	/** Propagates Motion Blur related settings to the view info. */
	void SetupViewInfo(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings, FMinimalViewInfo& InOutViewInfo) const;
};

USTRUCT(BlueprintType)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationICVFX_CameraDepthOfField
{
	GENERATED_BODY()

public:
	/** Get the CompensationLUT texture corresponding to the current settings.
	* 
	* @param InStageSettings - it can be used for global LUT settings.
	*/
	UTexture2D* GetCompensationLUT(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings) const;

public:
	/** Enables depth of field correction on the wall, which dynamically adjusts the size of the defocus circle of confusion to compensate for the real-world camera blur when shooting the wall */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = NDisplay)
	bool bEnableDepthOfFieldCompensation = false;

	/** Allows the ICVFX camera to automatically compute its distance from the stage walls using ray casting every tick */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = NDisplay)
	bool bAutomaticallySetDistanceToWall = true;

	/** The distance from the ICVFX camera to the wall it is pointing at */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = NDisplay, meta=(EditCondition="!bAutomaticallySetDistanceToWall"))
	float DistanceToWall = 0.0;

	/** An offset applied to DistanceToWall (applied regardless of whether DistanceToWall is automatically set) */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = NDisplay)
	float DistanceToWallOffset = 0.0;

	/** A gain factor that scales the amount of depth of field blur rendered on the wall */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = NDisplay, meta = (UIMin=0.0, UIMax=10.0, ClampMin=0.0, ClampMax=10.0))
	float DepthOfFieldGain = 1.0;

	/** Look-up texture that encodes the specific amount of compensation used for each combination of wall distance and object distance */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = NDisplay)
	TSoftObjectPtr<UTexture2D> CompensationLUT = TSoftObjectPtr<UTexture2D>(FSoftObjectPath(TEXT("/nDisplay/DepthOfField/T_LUT_PointRMS.T_LUT_PointRMS")));

	/** Actual LUT to use with the depth of field pipeline, copied and modified from CompensationLUT */
	UPROPERTY(Transient)
	TObjectPtr<UTexture2D> DynamicCompensationLUT = nullptr;

public:
	/** Processes the compensation LUT by adding any needed DoF gain and writes the result to the dynamic compensation LUT texture */
	void UpdateDynamicCompensationLUT();
};

USTRUCT(BlueprintType)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationICVFX_CameraSoftEdge
{
	GENERATED_BODY()

	/** Adjust blur amount to the top and bottom edges of the inner frustum. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (DisplayName = "Top and Bottom", ClampMin = "0.0", UIMin = "0.0", ClampMax = "1.0", UIMax = "1.0"))
	float Vertical = 0.f;

	/** Adjust blur amount to the left and right side edges of the inner frustum. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (DisplayName = "Sides", ClampMin = "0.0", UIMin = "0.0", ClampMax = "1.0", UIMax = "1.0"))
	float Horizontal = 0.f;

	/** Feather. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (DisplayName = "Feather", ClampMin = "0.0", UIMin = "0.0", ClampMax = "1.0", UIMax = "1.0"))
	float Feather = 1.f;
};

USTRUCT(BlueprintType)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationICVFX_CameraBorder
{
	GENERATED_BODY()

	/** Enable Inner Frustum Border. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (DisplayName = "Enable Inner Frustum Border"))
	bool Enable = false;

	/** Adjust border width to the top and bottom edges of the inner frustum. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (DisplayName = "Border Width", ClampMin = "0.0", UIMin = "0.0", ClampMax = "1.0", UIMax = "1.0"))
	float Thickness = 0.05f;

	/** Adjust color of the border edges of the inner frustum. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (DisplayName = "Border Color"))
	FLinearColor Color = FLinearColor::Blue;
};

USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationICVFX_CameraCustomFrustum
{
	GENERATED_BODY()

	/** Expected ICVFX camera resolution when both 'Adapt Resolution' and 'Enable Inner Frustum Overscan' are enabled. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = NDisplay)
	FIntPoint EstimatedOverscanResolution = { 2560, 1440 };
	
	/** Real ICVFX camera resolution for current settings. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = NDisplay)
	FIntPoint InnerFrustumResolution = { 2560, 1440 };

	/** This value shows the ratio of "Overscan Estimated Resolution" pixels to "Frustum Internal Resolution" pixels. */
	UPROPERTY(BlueprintReadOnly, Category = NDisplay, meta = (HideInDetailPanel))
	float OverscanPixelsIncrease = 0.f;

	/** Enable Custom Frustum. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (DisplayName = "Enable Inner Frustum Overscan"))
	bool bEnable = false;

	/** Enable adaptive resolution. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (DisplayName = "Adapt Resolution"))
	bool bAdaptResolution = false;

	/** Multiply the field of view for the ICVFX camera by this value.  This can increase the overall size of the inner frustum to help provide a buffer against latency when moving the camera. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "In Camera VFX", meta = (DisplayName = "Overscan Multiplier", ClampMin = "0.05", UIMin = "0.05", ClampMax = "5.0", UIMax = "5.0"))
	float FieldOfViewMultiplier = 1.0f;

	/** Enable/disable inner camera custom frustum and specify units as percent or pixel values. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (DisplayName = "Overscan Units"))
	EDisplayClusterConfigurationViewportCustomFrustumMode Mode = EDisplayClusterConfigurationViewportCustomFrustumMode::Percent;

	/** Pixel/Percent value to alter the frustum to the left side */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (DisplayName = "Left", ClampMin = "-500.0", UIMin = "-500.0", ClampMax = "500.0", UIMax = "500.0"))
	float Left = 0;

	/** Pixel/Percent value to alter the frustum to the right side */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (DisplayName = "Right", ClampMin = "-500.0", UIMin = "-500.0", ClampMax = "500.0", UIMax = "500.0"))
	float Right = 0;

	/** Pixel/Percent value to alter the frustum to the top*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (DisplayName = "Top", ClampMin = "-500.0", UIMin = "-500.0", ClampMax = "500.0", UIMax = "500.0"))
	float Top = 0;

	/** Pixel/Percent value to alter the frustum to the bottom */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (DisplayName = "Bottom", ClampMin = "-500.0", UIMin = "-500.0", ClampMax = "500.0", UIMax = "500.0"))
	float Bottom = 0;

public:
	/** Propagates Custom Frustum related settings to the view info. */
	void SetupViewInfo(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings, const FDisplayClusterConfigurationICVFX_CameraSettings& InCameraSettings, FMinimalViewInfo& InOutViewInfo) const;

	/** Get camera FOV multiplier. */
	float GetCameraFieldOfViewMultiplier(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings) const;

	/** Get camera adapt resolution ratio. */
	float GetCameraAdaptResolutionRatio(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings) const;

};

USTRUCT(BlueprintType)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationICVFX_CameraSettings
{
	GENERATED_BODY()

public:
	/**
	* Returns true if this camera is active
	* @param InConfigurationData - cluster configuration data
	* @param InClusterNodeId     - current cluster node name
	* 
	* @return - true if this camera can be used
	**/
	bool IsICVFXEnabled(const class UDisplayClusterConfigurationData& InConfigurationData, const FString& InClusterNodeId) const;

	/** Returns the CineCameraActor from the 'ExternalCameraActor' parameter. */
	ACineCameraActor* GetExternalCineCameraActor() const;

	/** Returns the CineCameraComponent from the 'ExternalCameraActor' parameter. */
	UCineCameraComponent* GetExternalCineCameraComponent() const;

public:
	FDisplayClusterConfigurationICVFX_CameraSettings();

	/** Return InCamera OCIO configuration for the specified cluster node. Return nullptr if no OCIO. */
	const FOpenColorIOColorConversionSettings* FindInnerFrustumOCIOConfiguration(const FString& InClusterNodeId) const;
	
	/** Return Chromakey OCIO configuration for the specified cluster node. Return nullptr if no OCIO. */
	const FOpenColorIOColorConversionSettings* FindChromakeyOCIOConfiguration(const FString& InClusterNodeId) const;

	bool IsInnerFrustumViewportSettingsEqual(const FString& InClusterNodeId1, const FString& InClusterNodeId2) const;
	bool IsChromakeyViewportSettingsEqual(const FString& InClusterNodeId1, const FString& InClusterNodeId2) const;

	/** Return calculated soft edges values. */
	UE_DEPRECATED(5.5, "Use the UDisplayClusterICVFXCameraComponent::GetICVFXCameraShaderParameters() instead")
	FVector4 GetCameraSoftEdge(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings, const UCineCameraComponent& InCineCameraComponent) const { return FVector4(); }

	/** Get camera buffer ratio. */
	float GetCameraBufferRatio(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings) const;

	/** Get camera upscale settings. */
	void GetCameraUpscalerSettings(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings, FDisplayClusterUpscalerSettings& OutUpscalerSettings) const;

	/** Get camera frame resolution. */
	UE_DEPRECATED(5.5, "Use the UDisplayClusterICVFXCameraComponent::GetICVFXCameraFrameSize() instead")
	FIntPoint GetCameraFrameSize(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings, const UCineCameraComponent& InCineCameraComponent) const { return FIntPoint(0, 0); }

	/** Get camera frame aspect ratio. */
	UE_DEPRECATED(5.5, "Use the UDisplayClusterICVFXCameraComponent::GetICVFXCameraFrameSize() instead")
	float GetCameraFrameAspectRatio(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings, const UCineCameraComponent& InCineCameraComponent) const { return 0.f; }

	/** Get camera border settings. */
	UE_DEPRECATED(5.5, "Use the UDisplayClusterICVFXCameraComponent::GetICVFXCameraShaderParameters() instead")
	bool GetCameraBorder(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings, FLinearColor& OutBorderColor, float& OutBorderThickness) const { return false; }

	/** Sets up view info for each relevant setting such as render, custom frustrum and motion blur settings. */
	void SetupViewInfo(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings, FMinimalViewInfo& InOutViewInfo);

public:
	/** Render the inner frustum for this ICVFX camera. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "In Camera VFX", meta = (DisplayName = "Enable Inner Frustum"))
	bool bEnable = true;

	/** Specify a Cine Camera Actor for this ICVFX camera to use instead of the default nDisplay camera. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "In Camera VFX", meta = (DisplayName = "Cine Camera Actor"))
	TSoftObjectPtr<ACineCameraActor> ExternalCameraActor;

	/** Adjust resolution scaling for the inner frustum. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "In Camera VFX", meta = (DisplayName = "Inner Frustum Screen Percentage", ClampMin = "0.05", UIMin = "0.05", ClampMax = "10.0", UIMax = "1.0"))
	float BufferRatio = 1;

	/** Upscaler settings for the inner frustum. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "In Camera VFX", meta = (DisplayName = "Upscaler Settings"))
	FDisplayClusterConfigurationUpscalerSettings UpscalerSettings;

	/** Render a larger or smaller inner frame */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "In Camera VFX", meta = (DisplayName = "Inner Frustum Overscan"))
	FDisplayClusterConfigurationICVFX_CameraCustomFrustum CustomFrustum;

	/** Soften the edges of the inner frustum to help avoid hard lines in reflections seen by the live-action camera. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "In Camera VFX")
	FDisplayClusterConfigurationICVFX_CameraSoftEdge SoftEdge;

	/** Rotate the inner frustum. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "In Camera VFX", meta = (DisplayName = "Inner Frustum Rotation"))
	FRotator  FrustumRotation = FRotator::ZeroRotator;

	/** Specify an offset on the inner frustum. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "In Camera VFX", meta = (DisplayName = "Inner Frustum Offset"))
	FVector FrustumOffset = FVector::ZeroVector;

	/** Off-axis / off-center projection offset as proportion of screen dimensions. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "In Camera VFX", meta = (DisplayName = "Inner Frustum Projection Offset"))
	FVector2D OffCenterProjectionOffset = FVector2D::ZeroVector;

	/**Border for the inner frustum. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "In Camera VFX", meta = (DisplayName = "Inner Frustum Border"))
	FDisplayClusterConfigurationICVFX_CameraBorder Border;

	/** Render motion blur more accurately by subtracting blur from camera motion and avoiding amplification of blur by the physical camera. */
	UPROPERTY(BlueprintReadWrite, BlueprintReadWrite, EditAnywhere, Category = "In Camera VFX")
	FDisplayClusterConfigurationICVFX_CameraMotionBlur CameraMotionBlur;

	/** Settings that control the depth of field blur applied to the ICVFX image */
	UPROPERTY(BlueprintReadWrite, BlueprintReadWrite, EditAnywhere, Category = "In Camera VFX")
	FDisplayClusterConfigurationICVFX_CameraDepthOfField CameraDepthOfField;

	/** Configure global render settings for this viewport */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay)
	FDisplayClusterConfigurationICVFX_CameraRenderSettings RenderSettings;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Chromakey)
	FDisplayClusterConfigurationICVFX_ChromakeySettings Chromakey;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = OCIO)
	FDisplayClusterConfigurationICVFX_CameraOCIO CameraOCIO;

	/** Entire Cluster Color Grading */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Inner Frustum Color Grading", meta = (DisplayName = "Enable Inner Frustum Color Grading"))
	bool EnableInnerFrustumColorGrading = true;

	/** All Nodes Color Grading */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Inner Frustum Color Grading", meta = (DisplayName = "All Nodes Color Grading"))
	FDisplayClusterConfigurationViewport_AllNodesColorGrading AllNodesColorGrading;

	/** Perform advanced color grading operations for the inner frustum on a per-node or group-of-nodes basis. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Inner Frustum Color Grading", meta = (DisplayName = "Per-Node Color Grading", ConfigurationMode = "ClusterNodes"))
	TArray<FDisplayClusterConfigurationViewport_PerNodeColorGrading> PerNodeColorGrading;

	/** Content specified here will not appear in the inner frustum, but can appear in the nDisplay viewports. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "In Camera VFX", meta = (DisplayName = "Content Hidden from Inner Frustum", Substitutions = "LayersTooltip = Layers hidden from the inner frustum, ActorsTooltip = Actors hidden from the inner frustum"))
	FDisplayClusterConfigurationICVFX_VisibilityList CameraHideList;

	/** A list of viewports that the inner frustum is not rendered to. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (ClusterItemType = Viewports, DisplayName = "Inner Frustum Hidden in Viewports"))
	FDisplayClusterConfigurationClusterItemReferenceList HiddenICVFXViewports;
};

USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationICVFX_StageSettings
{
	GENERATED_BODY()

public:
	/** Return OCIO configuration for the specified viewport. Return nullptr if no OCIO. */
	const FOpenColorIOColorConversionSettings* FindViewportOCIOConfiguration(const FString& InViewportId) const;

	/** Return LightCard OCIO configuration for the specified viewport. Return nullptr if no OCIO. */
	const FOpenColorIOColorConversionSettings* FindLightcardOCIOConfiguration(const FString& InViewportId) const;

	/** Get camera overlapping rendering mode.*/
	EDisplayClusterShaderParametersICVFX_CameraOverlappingRenderMode GetCameraOverlappingRenderMode() const;

public:
	/** Enable/disable the inner frustum on all ICVFX cameras. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "In Camera VFX", meta = (DisplayName = "Enable Inner Frustum"))
	bool bEnableInnerFrustums = true;

	/** Render the chromakey where the inner frustum overlaps. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "In Camera VFX", meta = (DisplayName = "Enable Inner Frustum Chromakey Overlap"))
	bool bEnableInnerFrustumChromakeyOverlap = false;

	/** Global upscaler settings that all ICVFX camera components can opt to use. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "In Camera VFX", meta = (DisplayName = "Default Upscaler Settings"))
	FDisplayClusterConfigurationUpscalerSettings GlobalInnerFrustumUpscalerSettings;

	/** Default In-Cameras texture resolution. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Configuration, meta = (DisplayName = "Default Frame Resolution"))
	FDisplayClusterConfigurationICVFX_Size DefaultFrameSize;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Light Cards", meta = (ExpandProperties))
	FDisplayClusterConfigurationICVFX_LightcardSettings Lightcard;

	/** Freeze rendering for viewports. This improves performance. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Viewports, meta = (DisplayName = "Freeze Viewports"))
	bool bFreezeRenderOuterViewports = false;

	/** Global upscaler settings for Outer viewports. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Viewports, meta = (DisplayName = "Default Upscaler Settings"))
	FDisplayClusterConfigurationUpscalerSettings OuterViewportUpscalerSettings;

	// Hide list for all icvfx viewports (outer, inner, cameras, etc)
	// (This allow to hide all actors from layers for icvfx render logic)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Viewports, meta = (DisplayName = "Content Hidden from Entire Cluster", ToolTip = "Content specified here will not appear anywhere in the nDisplay cluster", Substitutions = "LayersTooltip = Layers hidden from the entire nDisplay cluster, ActorsTooltip = Actors hidden from the entire nDisplay cluster"))
	FDisplayClusterConfigurationICVFX_VisibilityList HideList;

	/** Special hide list for Outer viewports */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Viewports, meta = (DisplayName = "Content Hidden from All Viewports", ToolTip = "Content specified here will not appear in the nDisplay viewports, but can appear in the inner frustum.", Substitutions = "LayersTooltip = Layers hidden from the nDisplay viewports, ActorsTooltip = Actors hidden from the nDisplay viewports"))
	FDisplayClusterConfigurationICVFX_VisibilityList OuterViewportHideList;

	/** Viewport Color Grading */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Viewport Color Grading", meta = (DisplayName = "Enable Color Grading"))
	bool EnableColorGrading = true;

	/** Entire Cluster Color Grading */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Viewport Color Grading", meta = (DisplayName = "Entire Cluster"))
	FDisplayClusterConfigurationViewport_EntireClusterColorGrading EntireClusterColorGrading;

	/** Perform advanced color grading operations on a per-viewport or group-of-viewports basis. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Viewport Color Grading", meta = (DisplayName = "Per-Viewport Color Grading", ConfigurationMode = "Viewports"))
	TArray<FDisplayClusterConfigurationViewport_PerViewportColorGrading> PerViewportColorGrading;

	/** OpenColorIO configuration for the Outer viewports. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OCIO", meta = (DisplayName = "Outer Viewports OCIO"))
	FDisplayClusterConfigurationICVFX_ViewportOCIO ViewportOCIO;

	/** Entire cluster Auto Exposure settings. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Exposure", meta = (DisplayName = "Auto Exposure Settings"))
	FDisplayClusterConfiguration_AutoExposureSettings AutoExposureSettings;

	/** Global chromakey settings that all ICVFX camera components can opt to use */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Chromakey")
	FDisplayClusterConfigurationICVFX_GlobalChromakeySettings GlobalChromakey;
};
