// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DisplayClusterConfigurationTypes_Base.h"
#include "DisplayClusterConfigurationTypes_MediaSync.h"
#include "DisplayClusterConfigurationTypes_Tile.h"

#include "MediaPlayer.h"
#include "MediaSource.h"
#include "MediaTexture.h"
#include "MediaOutput.h"

#include "Render/Viewport/Containers/DisplayClusterViewport_TileSettings.h"

#include "DisplayClusterConfigurationTypes_Media.generated.h"


/*
 * Media frame split types
 */
UENUM(BlueprintType)
enum class EDisplayClusterConfigurationMediaSplitType : uint8
{
	FullFrame     UMETA(DisplayName = "Full Frame"),
	UniformTiles  UMETA(DisplayName = "Tiled"),
};


/*
 * Media input item
 */
USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationMediaInput
{
	GENERATED_BODY()

public:
	/** Media source to use */
	UPROPERTY(Instanced, EditAnywhere, BlueprintReadWrite, Category = "Media")
	TObjectPtr<UMediaSource> MediaSource;
};


/*
 * Media output item
 */
USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationMediaOutput
{
	GENERATED_BODY()

public:
	/** Media output to use */
	UPROPERTY(Instanced, EditAnywhere, BlueprintReadWrite, Category = "Media")
	TObjectPtr<UMediaOutput> MediaOutput;

	/** Media output synchronization policy */
	UPROPERTY(Instanced, EditAnywhere, BlueprintReadWrite, Category = "Media", meta = (DisplayName = "Capture Synchronization"))
	TObjectPtr<UDisplayClusterMediaOutputSynchronizationPolicy> OutputSyncPolicy;
};


/*
 * Media input group (ICVFX, Full frame)
 */
USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationMediaInputGroup
	: public FDisplayClusterConfigurationMediaInput
{
	GENERATED_BODY()

public:
	/** Cluster nodes that use media source below */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Media, meta = (ClusterItemType = ClusterNodes))
	FDisplayClusterConfigurationClusterItemReferenceList ClusterNodes;
};


/*
 * Media output group (ICVFX, Full Frame)
 */
USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationMediaOutputGroup
	: public FDisplayClusterConfigurationMediaOutput
{
	GENERATED_BODY()

public:
	/** Cluster nodes that export media via MediaOutput below */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Media, meta = (ClusterItemType = ClusterNodes))
	FDisplayClusterConfigurationClusterItemReferenceList ClusterNodes;
};


/**
 * Uniform tile media input item. Maps a tile to a media source.
 */
USTRUCT(BlueprintType)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationMediaUniformTileInput
{
	GENERATED_BODY()

public:
	/** Tile position */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Media", meta = (ClampMin = 0, ClampMax = 3))
	FIntPoint Position = FIntPoint::ZeroValue;

	/** Media source to use */
	UPROPERTY(Instanced, EditAnywhere, BlueprintReadWrite, Category = "Media")
	TObjectPtr<UMediaSource> MediaSource;
};


/**
 * Uniform tile media output item. Maps a tile to a media output.
 */
USTRUCT(BlueprintType)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationMediaUniformTileOutput
{
	GENERATED_BODY()

public:
	/** Tile position */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Media", meta = (ClampMin = 0, ClampMax = 3))
	FIntPoint Position = FIntPoint::ZeroValue;

	/** Media output to use */
	UPROPERTY(Instanced, EditAnywhere, BlueprintReadWrite, Category = "Media")
	TObjectPtr<UMediaOutput> MediaOutput;

	/** Media output synchronization policy */
	UPROPERTY(Instanced, EditAnywhere, BlueprintReadWrite, Category = "Media", meta = (DisplayName = "Capture Synchronization"))
	TObjectPtr<UDisplayClusterMediaOutputSynchronizationPolicy> OutputSyncPolicy;
};


/*
 * Media input group (ICVFX, Tiled)
 */
USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationMediaTiledInputGroup
{
	GENERATED_BODY()

public:
	/** Cluster nodes that use media source below */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Media", meta = (ClusterItemType = ClusterNodes))
	FDisplayClusterConfigurationClusterItemReferenceList ClusterNodes;

	/** Tile mapping. Maps tiles to media sources. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Media")
	TArray<FDisplayClusterConfigurationMediaUniformTileInput> Tiles;
};


PRAGMA_DISABLE_DEPRECATION_WARNINGS

/*
 * Media output group (ICVFX, Tiled)
 */
USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationMediaTiledOutputGroup
{
	GENERATED_BODY()

public:
	/** Cluster nodes that export media via MediaOutput below */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Media", meta = (ClusterItemType = ClusterNodes))
	FDisplayClusterConfigurationClusterItemReferenceList ClusterNodes;

	/** Tile mapping. Maps tiles to media outputs. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Media")
	TArray<FDisplayClusterConfigurationMediaUniformTileOutput> Tiles;

#if WITH_EDITORONLY_DATA
public:
	/** Media output synchronization policy */
	UE_DEPRECATED(5.6, "This property has been deprecated. Please refer new MediaOutputs property.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "This property has been deprecated. Synchronization policy is bound to the actual media outputs."))
	TObjectPtr<UDisplayClusterMediaOutputSynchronizationPolicy> OutputSyncPolicy;
#endif // WITH_EDITORONLY_DATA
};

PRAGMA_ENABLE_DEPRECATION_WARNINGS


/*
 * Media settings for node backbuffer
 */
USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationMediaNodeBackbuffer
{
	GENERATED_BODY()

public:
	/** Enable/Disable Media */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Media", meta = (DisplayName = "Enable Media"))
	bool bEnable = false;

	/** Full-Frame Media Outputs */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Media")
	TArray<FDisplayClusterConfigurationMediaOutput> MediaOutputs;

	/** Split layout */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Media", meta = (ClampMin = 1, ClampMax = 4))
	FIntPoint TiledSplitLayout = { 1, 1 };

	/** Tile mapping. Maps tiles to media outputs. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Media")
	TArray<FDisplayClusterConfigurationMediaUniformTileOutput> TiledMediaOutputs;

public:

	/** Returns true if at least one media output assigned */
	bool IsMediaOutputAssigned() const;
};


PRAGMA_DISABLE_DEPRECATION_WARNINGS

/*
 * Media settings for viewports
 */
USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationMediaViewport
{
	GENERATED_BODY()

public:
	/** Enable/disable media */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Media", meta = (DisplayName = "Enable Media"))
	bool bEnable = false;

	/** Media source to use */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Media")
	FDisplayClusterConfigurationMediaInput MediaInput;

	/** Media outputs to use */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Media")
	TArray<FDisplayClusterConfigurationMediaOutput> MediaOutputs;

public:
	/** Returns true if a media source assigned */
	bool IsMediaInputAssigned() const;

	/** Returns true if at least one media output assigned */
	bool IsMediaOutputAssigned() const;

#if WITH_EDITORONLY_DATA

protected:
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "This property has been deprecated"))
	FString MediaSharingNode_DEPRECATED;

public:
	UE_DEPRECATED(5.3, "This property has been deprecated. Please refer new MediaInput property.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "This property has been deprecated. Please refer new MediaInput property."))
	TObjectPtr<UMediaSource> MediaSource;

	UE_DEPRECATED(5.3, "This property has been deprecated. Please refer new MediaOutputs property.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "This property has been deprecated. Please refer new MediaOutputs property."))
	TObjectPtr<UMediaOutput> MediaOutput;

	UE_DEPRECATED(5.3, "This property has been deprecated.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "This property has been deprecated."))
	TObjectPtr<UDisplayClusterMediaOutputSynchronizationPolicy> OutputSyncPolicy;

#endif // WITH_EDITORONLY_DATA
};

PRAGMA_ENABLE_DEPRECATION_WARNINGS


/*
 * Media settings for ICVFX cameras
 */
USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationMediaICVFX
{
	GENERATED_BODY()

public:

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FDisplayClusterConfigurationMediaICVFX() = default;
	~FDisplayClusterConfigurationMediaICVFX() = default;
	FDisplayClusterConfigurationMediaICVFX(const FDisplayClusterConfigurationMediaICVFX&) = default;
	FDisplayClusterConfigurationMediaICVFX(FDisplayClusterConfigurationMediaICVFX&&) = default;
	FDisplayClusterConfigurationMediaICVFX& operator=(const FDisplayClusterConfigurationMediaICVFX&) = default;
	FDisplayClusterConfigurationMediaICVFX& operator=(FDisplayClusterConfigurationMediaICVFX&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

public:
	/** Enable/disable media */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Media", meta = (DisplayName = "Enable Media"))
	bool bEnable = false;

	/** Media frame split type */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Media", meta = (DisplayName = "Inner Frustum Type"))
	EDisplayClusterConfigurationMediaSplitType SplitType = EDisplayClusterConfigurationMediaSplitType::FullFrame;

	/// Full-frame

	/** Media input mapping (Full frame) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Media", meta = (DisplayName = "Media Input Groups"))
	TArray<FDisplayClusterConfigurationMediaInputGroup> MediaInputGroups;

	/** Media output mapping (Full frame) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Media", meta = (DisplayName = "Media Output Groups"))
	TArray<FDisplayClusterConfigurationMediaOutputGroup> MediaOutputGroups;

	/// Uniform tiles

	/** Split layout */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Media", meta = (DisplayName = "Tiled Split Layout", ClampMin = 1, ClampMax = 4))
	FIntPoint TiledSplitLayout = { 1, 1 };

	/** Overscan settings for tile. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Media")
	FDisplayClusterConfigurationTile_Overscan TileOverscan;

	/** Cluster nodes that should render unbound tiles. Unbound tiles are the tiles that don't have any media assigned */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Media", meta = (ClusterItemType = ClusterNodes, DisplayName = "Nodes To Render Unbound Tiles", ToolTip = "Choose nodes that should render camera tiles that don't have any media assigned"))
	FDisplayClusterConfigurationClusterItemReferenceList ClusterNodesToRenderUnboundTiles;

	/** Media input mapping (Tiled) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Media", meta = (DisplayName = "Media Input Groups"))
	TArray<FDisplayClusterConfigurationMediaTiledInputGroup> TiledMediaInputGroups;

	/** Media output mapping (Tiled) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Media", meta = (DisplayName = "Media Output Groups"))
	TArray<FDisplayClusterConfigurationMediaTiledOutputGroup> TiledMediaOutputGroups;

	UE_DEPRECATED(5.8, "This property has been depreacted. Late OCIO is automatically detected in runtime.")
	UPROPERTY(meta = (DeprecatedProperty))
	bool bLateOCIOPass_DEPRECATED = false;

	UE_DEPRECATED(5.8, "This property has been depreacted. PQ is allways used now.")
	UPROPERTY(meta = (DeprecatedProperty))
	bool bTransferPQ_DEPRECATED = true;

public:
	/** Returns true if these media settings can be used for tiling. */
	bool ShouldMediaICVFXSplitIntoTiles() const;

	/** Returns additional tile settings for the cluster node. */
	EDisplayClusterViewportTileFlags GetMediaICVFXTileFlags(const FString& NodeId) const;

	/** Returns true if any media source of specified split-type is bound */
	bool HasAnyMediaInputAssigned(const FString& NodeId, EDisplayClusterConfigurationMediaSplitType SplitType) const;

	/** Returns true if any media output of specified split-type is bound */
	bool HasAnyMediaOutputAssigned(const FString& NodeId, EDisplayClusterConfigurationMediaSplitType SplitType) const;

	/** [FullFrame] Returns media source bound to a specific cluster node if configured for full-frame media */
	UMediaSource* GetMediaSource(const FString& NodeId) const;

	/** [FullFrame] Returns media outputs bound to a specific cluster node if configured for full-frame media */
	TArray<FDisplayClusterConfigurationMediaOutputGroup> GetMediaOutputGroups(const FString& NodeId) const;

	/** [UniformTiles] Returns all the tiles bound to a specific cluster node. */
	bool GetMediaInputTiles(const FString& NodeId, TArray<FDisplayClusterConfigurationMediaUniformTileInput>& OutTiles) const;

	/** [UniformTiles] Returns all the tiles bound to a specific cluster node. */
	bool GetMediaOutputTiles(const FString& NodeId, TArray<FDisplayClusterConfigurationMediaUniformTileOutput>& OutTiles) const;

public:
	UE_DEPRECATED(5.3, "This function has been deprecated. Please use GetMediaOutputGroups.")
	TArray<UMediaOutput*> GetMediaOutputs(const FString& NodeId) const
	{
		return {};
	}

	UE_DEPRECATED(5.3, "This function has been deprecated. Please use GetMediaOutputGroups.")
	UDisplayClusterMediaOutputSynchronizationPolicy* GetOutputSyncPolicy(const FString& NodeId) const
	{
		return nullptr;
	}

	UE_DEPRECATED(5.3, "This function has been deprecated. Please use GetMediaOutputGroups.")
	UMediaOutput* GetMediaOutput(const FString& NodeId) const
	{
		return nullptr;
	}

	UE_DEPRECATED(5.2, "This function has been deprecated.")
	bool IsMediaSharingNode(const FString& InNodeId) const
	{
		return false;
	}

	UE_DEPRECATED(5.4, "This function has been deprecated. Please use `HasAnyMediaInputAssigned`.")
	bool IsMediaInputAssigned(const FString& NodeId) const
	{
		return HasAnyMediaInputAssigned(NodeId, EDisplayClusterConfigurationMediaSplitType::FullFrame);
	}

	UE_DEPRECATED(5.4, "This function has been deprecated. Please use `HasAnyMediaOutputAssigned`.")
	bool IsMediaOutputAssigned(const FString& NodeId) const
	{
		return HasAnyMediaOutputAssigned(NodeId, EDisplayClusterConfigurationMediaSplitType::FullFrame);
	}
};


/*
 * Global media settings
 */
USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationGlobalMediaSettings
{
	GENERATED_BODY()

public:
	/** Media latency */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Media", meta = (ClampMin = "0", ClampMax = "9", UIMin = "0", UIMax = "9"))
	int32 Latency = 0;
};
