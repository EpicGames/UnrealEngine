// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"
#include "UObject/ObjectMacros.h"

#include "DisplayClusterConfigurationTypes_MediaSync.generated.h"

class UMediaCapture;

/** Media output sync policy handler interface. All sync logic is handled in a derived native class to avoid UObject interaction from other threads. */
class DISPLAYCLUSTERCONFIGURATION_API IDisplayClusterMediaOutputSynchronizationPolicyHandler : public TSharedFromThis<IDisplayClusterMediaOutputSynchronizationPolicyHandler>
{
public:
	
	/** Destructor */
	virtual ~IDisplayClusterMediaOutputSynchronizationPolicyHandler() = default;

	/** Returns true if specified media capture type can be synchonized by the policy implementation */
	virtual bool IsCaptureTypeSupported(UMediaCapture* MediaCapture) const = 0;

	/** Starts synchronization of specific output stream (capture device). Returns false if failed. */
	virtual bool StartSynchronization(UMediaCapture* MediaCapture, const FString& MediaId) = 0;

	/** Stops synchronization of specific output stream (capture device). */
	virtual void StopSynchronization() = 0;

	/** Returns true if currently synchronizing a media output. */
	virtual bool IsRunning() = 0;

	/** Returns the policy config class used to create this policy handler. */
	virtual TSubclassOf<UDisplayClusterMediaOutputSynchronizationPolicy> GetPolicyClass() const = 0;
};

/*
 * Base media output synchronization policy class settings
 */
UCLASS(Abstract, editinlinenew, BlueprintType, hidecategories = (Object))
class DISPLAYCLUSTERCONFIGURATION_API UDisplayClusterMediaOutputSynchronizationPolicy
	: public UObject
{
	GENERATED_BODY()

public:

	/** Returns the handler associated with this policy config object */
	virtual TSharedPtr<IDisplayClusterMediaOutputSynchronizationPolicyHandler> GetHandler() PURE_VIRTUAL(UDisplayClusterMediaOutputSynchronizationPolicy::GetHandler, return nullptr;);
};
