// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "UObject/ObjectMacros.h"
#include "OpenColorIOColorSpace.h"

#include "DisplayClusterConfigurationTypes_OCIO.generated.h"

/*
 * OCIO configuration structure.
 */
USTRUCT(Blueprintable)
struct FDisplayClusterConfigurationOCIOConfiguration
{
	GENERATED_BODY()

	/**  Enable the application of an OpenColorIO configuration to all viewports. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OCIO")
	bool bIsEnabled = false;

	/** "This property has been deprecated. */
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use the ColorConfiguration property instead"))
	FOpenColorIODisplayConfiguration OCIOConfiguration_DEPRECATED;

	/** Conversion to apply when this display is enabled */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OCIO")
	FOpenColorIOColorConversionSettings ColorConfiguration;

public:

	bool Serialize(FArchive& Ar);
	void PostSerialize(const FArchive& Ar);
};

template<> struct TStructOpsTypeTraits<FDisplayClusterConfigurationOCIOConfiguration> : public TStructOpsTypeTraitsBase2<FDisplayClusterConfigurationOCIOConfiguration>
{
	enum
	{
		WithSerializer = true,
		WithPostSerialize = true,
	};
};

/*
 * OCIO profile structure. Can be configured for viewports or cluster nodes.
 * To enable viewport configuration when using as a UPROPERTY set meta = (ConfigurationMode = "Viewports")
 * To enable cluster node configuration when using as a UPROPERTY set meta = (ConfigurationMode = "ClusterNodes")
 */
USTRUCT(Blueprintable)
struct FDisplayClusterConfigurationOCIOProfile
{
	GENERATED_BODY()

	/** Enable the application of an OpenColorIO configuration for the viewport(s) specified. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OCIO")
	bool bIsEnabled = false;

	/** "This property has been deprecated. */
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use the ColorConfiguration property instead"))
	FOpenColorIODisplayConfiguration OCIOConfiguration_DEPRECATED;

	/** Specify the viewports to apply this OpenColorIO configuration. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "OCIO")
	TArray<FString> ApplyOCIOToObjects;

	/** Conversion to apply when this display is enabled */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OCIO")
	FOpenColorIOColorConversionSettings ColorConfiguration;

public:
	
	bool Serialize(FArchive& Ar);
	void PostSerialize(const FArchive& Ar);

	/** Return true if the configuration is valid for the input object */
	bool IsEnabledForObject(const FString& InObjectId) const;
};

template<> struct TStructOpsTypeTraits<FDisplayClusterConfigurationOCIOProfile> : public TStructOpsTypeTraitsBase2<FDisplayClusterConfigurationOCIOProfile>
{
	enum
	{
		WithSerializer = true,
		WithPostSerialize = true,
	};
};