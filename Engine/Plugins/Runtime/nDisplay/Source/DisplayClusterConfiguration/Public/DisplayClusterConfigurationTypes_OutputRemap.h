// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"

#include "Engine/StaticMesh.h"

#include "DisplayClusterConfigurationTypes_OutputRemap.generated.h"

/* Source types of the output remapping */
UENUM()
enum class EDisplayClusterConfigurationFramePostProcess_OutputRemapSource : uint8
{
	/** Use a Static Mesh reference for output remapping when the Data Source is set to Mesh */
	StaticMesh     UMETA(DisplayName = "Static Mesh"),

	/** Use an external .obj file for output remapping when the Data Source is set to File */
	ExternalFile   UMETA(DisplayName = "External File"),

	/** Use a Mesh component reference for output remapping (ProceduralMeshComponent or StaticMeshComponent) */
	MeshComponent UMETA(DisplayName = "Mesh Component"),
};

/* Screen space remapping of the final backbuffer output. Applied at the whole window */
USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationFramePostProcess_OutputRemap
{
	GENERATED_BODY()

public:
	/** Enables or disables output remapping */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "NDisplay OutputRemap", meta = (DisplayName = "Enable Output Remapping"))
	bool bEnable = false;

	/** Selects either the Static Mesh or External File setting as the source for output remapping */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "NDisplay OutputRemap")
	EDisplayClusterConfigurationFramePostProcess_OutputRemapSource DataSource = EDisplayClusterConfigurationFramePostProcess_OutputRemapSource::StaticMesh;

	/** The Static Mesh reference to use for output remapping when the Data Source is set to Static Mesh */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "NDisplay OutputRemap", meta = (EditCondition = "DataSource == EDisplayClusterConfigurationFramePostProcess_OutputRemapSource::StaticMesh"))
	TObjectPtr<class UStaticMesh> StaticMesh = nullptr;

	/** The MeshComponent reference (ProceduralMeshComponent or StaticMeshComponent) to use for output remapping when the Data Source is set to Mesh Component */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "NDisplay OutputRemap", meta = (EditCondition = "DataSource == EDisplayClusterConfigurationFramePostProcess_OutputRemapSource::MeshComponent"))
	FString MeshComponentName;

	/** The external .obj file to use for output remapping when the Data Source is set to File */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "NDisplay OutputRemap", meta = (EditCondition = "DataSource == EDisplayClusterConfigurationFramePostProcess_OutputRemapSource::ExternalFile"))
	FString ExternalFile;
};
