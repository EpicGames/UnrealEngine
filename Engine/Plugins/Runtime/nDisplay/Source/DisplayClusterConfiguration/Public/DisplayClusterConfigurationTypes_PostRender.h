// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"

#include "DisplayClusterConfigurationTypes_Enums.h"
#include "DisplayClusterConfigurationTypes_Base.h"

#include "Engine/Texture.h"
#include "Engine/Texture2D.h"

#include "DisplayClusterConfigurationTypes_PostRender.generated.h"

USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationPostRender_Override
{
	GENERATED_BODY()

public:
	/** Disable default render, and resolve SourceTexture to viewport */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "NDisplay Render", meta = (DisplayName = "Enable Viewport Texture Replacement"))
	bool bAllowReplace = false;

	/** Texture to use in place of the inner frustum. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "NDisplay Render")
	TObjectPtr<UTexture> SourceTexture = nullptr;

	/** Set to True to crop the texture for the inner frustum as specified below. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "NDisplay Render", meta = (DisplayName = "Use Texture Crop"))
	bool bShouldUseTextureRegion = false;

	/** Texture Crop */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "NDisplay Render", meta = (DisplayName = "Texture Crop"))
	FDisplayClusterReplaceTextureCropRectangle TextureRegion;
};

USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationPostRender_BlurPostprocess
{
	GENERATED_BODY()

public:
	/** Enable/disable Post Process Blur and specify method. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "NDisplay Render")
	EDisplayClusterConfiguration_PostRenderBlur Mode = EDisplayClusterConfiguration_PostRenderBlur::None;

	/** Kernel Radius */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "NDisplay Render", meta = (EditCondition = "Mode != EDisplayClusterConfiguration_PostRenderBlur::None"))
	int   KernelRadius = 5;

	/** Kernel Scale */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "NDisplay Render", meta = (EditCondition = "Mode != EDisplayClusterConfiguration_PostRenderBlur::None"))
	float KernelScale = 20;
};

USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationPostRender_GenerateMips
{
	GENERATED_BODY()

	/** Generate and use mipmaps for the inner frustum.  Disabling this can improve performance but result in visual artifacts on the inner frustum. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "NDisplay Render")
	bool bAutoGenerateMips = false;

	/** Mips Sampler Filter */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "NDisplay Render")
	TEnumAsByte<enum TextureFilter> MipsSamplerFilter = TF_Trilinear;

	/** AutoGenerateMips sampler address mode for U channel. Defaults to clamp. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "NDisplay Render")
	TEnumAsByte<enum TextureAddress> MipsAddressU = TA_Clamp;

	/** AutoGenerateMips sampler address mode for V channel. Defaults to clamp. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "NDisplay Render")
	TEnumAsByte<enum TextureAddress> MipsAddressV = TA_Clamp;

	/** Performance: Allows a limited number of MIPs for high resolution. */
	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Category = "NDisplay Render", meta = (DisplayName = "Enable Maximum Number of Mips"))
	bool bEnabledMaxNumMips = false;

	/** Performance: Use this value as the maximum number of MIPs for high resolution.  */
	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Category = "NDisplay Render", meta = (DisplayName = "Maximum Number of Mips"))
	int MaxNumMips = 0;
};
