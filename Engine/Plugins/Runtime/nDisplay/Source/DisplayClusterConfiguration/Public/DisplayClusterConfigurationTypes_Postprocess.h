// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"

#include "DisplayClusterConfigurationStrings.h"

#include "Engine/Scene.h"

#include "DisplayClusterConfigurationTypes_Postprocess.generated.h"

USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationViewport_CustomPostprocessSettings
{
	GENERATED_BODY()

public:
	// Enable custom postprocess
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "NDisplay Viewport")
	bool bIsEnabled = false;

	// Apply postprocess for one frame
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "NDisplay Viewport")
	bool bIsOneFrame = false;

	// Custom postprocess settings
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "NDisplay Viewport")
	FPostProcessSettings PostProcessSettings;

	// Override blend weight
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "NDisplay Viewport")
	float BlendWeight = 1;
};

USTRUCT()
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationViewport_CustomPostprocess
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FDisplayClusterConfigurationViewport_CustomPostprocessSettings Start;

	UPROPERTY()
	FDisplayClusterConfigurationViewport_CustomPostprocessSettings Override;

	UPROPERTY()
	FDisplayClusterConfigurationViewport_CustomPostprocessSettings Final;
};

USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationViewport_ColorGradingSettings
{
	GENERATED_BODY();

	FDisplayClusterConfigurationViewport_ColorGradingSettings()
		: bOverride_Saturation(false)
		, bOverride_Contrast(false)
		, bOverride_Gamma(false)
		, bOverride_Gain(false)
		, bOverride_Offset(false)
	{ }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Overrides", meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_Saturation : 1;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Overrides", meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_Contrast : 1;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Overrides", meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_Gamma : 1;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Overrides", meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_Gain : 1;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Overrides", meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_Offset : 1;

	// Saturation
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Color Grading", meta = (UIMin = "0.0", UIMax = "2.0", Delta = "0.01", EditCondition = "bOverride_Saturation", ColorGradingMode = "saturation"))
	FVector4 Saturation = FVector4(1.0f, 1.0f, 1.0f, 1.0f);

	// Contrast
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Color Grading", meta = (UIMin = "0.0", UIMax = "2.0", Delta = "0.01", EditCondition = "bOverride_Contrast", ColorGradingMode = "contrast"))
	FVector4 Contrast = FVector4(1.0f, 1.0f, 1.0f, 1.0f);

	// Gamma
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Color Grading", meta = (UIMin = "0.0", UIMax = "2.0", Delta = "0.01", EditCondition = "bOverride_Gamma", ColorGradingMode = "gamma"))
	FVector4 Gamma = FVector4(1.0f, 1.0f, 1.0f, 1.0f);

	// Gain
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Color Grading", meta = (UIMin = "0.0", UIMax = "2.0", Delta = "0.01", EditCondition = "bOverride_Gain", ColorGradingMode = "gain"))
	FVector4 Gain = FVector4(1.0f, 1.0f, 1.0f, 1.0f);

	// Offset
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Color Grading", meta = (UIMin = "-1.0", UIMax = "1.0", Delta = "0.001", EditCondition = "bOverride_Offset", ColorGradingMode = "offset"))
	FVector4 Offset = FVector4(0.0f, 0.0f, 0.0f, 0.0f);
};

USTRUCT(BlueprintType)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationViewport_ColorGradingWhiteBalanceSettings
{
	GENERATED_BODY()

		FDisplayClusterConfigurationViewport_ColorGradingWhiteBalanceSettings()
		: bOverride_TemperatureType(0)
		, bOverride_WhiteTemp(0)
		, bOverride_WhiteTint(0)
	{};

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Overrides", meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_TemperatureType:1;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Overrides", meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_WhiteTemp:1;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Overrides", meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_WhiteTint:1;

	/**
	* Selects the type of temperature calculation.
	* White Balance uses the Temperature value to control the virtual camera's White Balance. This is the default selection.
	* Color Temperature uses the Temperature value to adjust the color temperature of the scene, which is the inverse of the White Balance operation.
	*/
	UPROPERTY(interp, BlueprintReadWrite, Category = "Viewport Settings", meta = (DisplayName = "Temperature Type", EditCondition = "bOverride_TemperatureType"))
	TEnumAsByte<enum ETemperatureMethod> TemperatureType = ETemperatureMethod::TEMP_WhiteBalance;

	// White temperature
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "White Balance", meta = (UIMin = "1500.0", UIMax = "15000.0", EditCondition = "bOverride_WhiteTemp", DisplayName = "Temp"))
	float WhiteTemp = 6500.0f;

	// White tint
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "White Balance", meta = (UIMin = "-1.0", UIMax = "1.0", EditCondition = "bOverride_WhiteTint", DisplayName = "Tint"))
	float WhiteTint = 0.0f;
};

USTRUCT(BlueprintType)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationViewport_ColorGradingMiscSettings
{
	GENERATED_BODY()

		FDisplayClusterConfigurationViewport_ColorGradingMiscSettings()
		: bOverride_BlueCorrection(0)
		, bOverride_ExpandGamut(0)
		, bOverride_SceneColorTint(0)
		, BlueCorrection(0)
		, ExpandGamut(0)
		, SceneColorTint(ForceInitToZero)
	{
	}

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Overrides", meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_BlueCorrection:1;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Overrides", meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_ExpandGamut:1;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Overrides", meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_SceneColorTint:1;

	// Correct for artifacts with "electric" blues due to the ACEScg color space. Bright blue desaturates instead of going to violet.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Misc", meta = (ClampMin = "0.0", ClampMax = "1.0", EditCondition = "bOverride_BlueCorrection"))
	float BlueCorrection;

	// Expand bright saturated colors outside the sRGB gamut to fake wide gamut rendering.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Misc", meta = (ClampMin = "0.0", UIMax = "1.0", EditCondition = "bOverride_ExpandGamut"))
	float ExpandGamut;

	// Scene tint color
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Misc", meta = (EditCondition = "bOverride_SceneColorTint", HideAlphaChannel))
	FLinearColor SceneColorTint;
};

USTRUCT(BlueprintType)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationViewport_ColorGradingRenderingSettings
{
	GENERATED_BODY()

		FDisplayClusterConfigurationViewport_ColorGradingRenderingSettings()
		: bOverride_AutoExposureBias(0)
		, bOverride_ColorCorrectionHighlightsMin(0)
		, bOverride_ColorCorrectionHighlightsMax(0)
		, bOverride_ColorCorrectionShadowsMax(0)
		, Global()
		, Shadows()
		, ColorCorrectionShadowsMax(0)
		, Midtones()
		, Highlights()
		, ColorCorrectionHighlightsMin(0)
		, ColorCorrectionHighlightsMax(1)
	{
	}
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Overrides", meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_AutoExposureBias:1;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Overrides", meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_ColorCorrectionHighlightsMin:1;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Overrides", meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_ColorCorrectionHighlightsMax:1;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Overrides", meta = (PinHiddenByDefault, InlineEditConditionToggle))
	uint8 bOverride_ColorCorrectionShadowsMax:1;

	// Not Implemented: Blend weight
	UPROPERTY()
	float BlendWeight = 1.0f;

	// Exposure compensation
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Viewport Settings", meta = (UIMin = "-15.0", UIMax = "15.0", EditCondition = "bOverride_AutoExposureBias", DisplayName = "Exposure Compensation"))
	float AutoExposureBias = 0.0f;

	// White balance
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Viewport Settings")
	FDisplayClusterConfigurationViewport_ColorGradingWhiteBalanceSettings WhiteBalance;

	// Global color grading
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Viewport Settings")
	FDisplayClusterConfigurationViewport_ColorGradingSettings Global;

	// Shadows color grading
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Viewport Settings")
	FDisplayClusterConfigurationViewport_ColorGradingSettings Shadows;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Viewport Settings", meta = (UIMin = "-1.0", UIMax = "1.0", EditCondition = "bOverride_ColorCorrectionShadowsMax", DisplayName = "ShadowsMax"))
	float ColorCorrectionShadowsMax;

	// Midtones color grading
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Viewport Settings")
	FDisplayClusterConfigurationViewport_ColorGradingSettings Midtones;

	// Highlights color grading
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Viewport Settings")
	FDisplayClusterConfigurationViewport_ColorGradingSettings Highlights;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Viewport Settings", meta = (UIMin = "-1.0", UIMax = "1.0", EditCondition = "bOverride_ColorCorrectionHighlightsMin", DisplayName = "HighlightsMin"))
	float ColorCorrectionHighlightsMin;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Viewport Settings", meta = (UIMin = "1.0", UIMax = "10.0", EditCondition = "bOverride_ColorCorrectionHighlightsMax", DisplayName = "HighlightsMax"))
	float ColorCorrectionHighlightsMax;

	// Highlights color grading misc settings
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Viewport Settings")
	FDisplayClusterConfigurationViewport_ColorGradingMiscSettings Misc;
};


USTRUCT(Blueprintable)
struct FDisplayClusterConfigurationViewport_EntireClusterColorGrading
{
	GENERATED_BODY()

	/** Enable the color grading settings for the entire cluster and add them to nDisplay's color grading stack.  This will affect both the viewports and inner frustum. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Viewport Color Grading", meta = (DisplayName = "Enable Entire Cluster Color Grading"))
	bool bEnableEntireClusterColorGrading = true;

	/** Entire Cluster Color Grading */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Viewport Settings", meta = (DisplayName = "Color Grading"))
	FDisplayClusterConfigurationViewport_ColorGradingRenderingSettings ColorGradingSettings;
};

USTRUCT(Blueprintable)
struct FDisplayClusterConfigurationViewport_PerViewportColorGrading
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = "Viewport Settings")
	FText Name = FText::GetEmpty();
#endif

	/** Enable the color grading settings for the viewport(s) specified and add them to nDisplay's color grading stack.  This will not affect the inner frustum. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Viewport Settings", meta = (DisplayName = "Enable Per-Viewport Color Grading"))
	bool bIsEnabled = true;

	/** Optionally include the Entire Cluster Color Grading settings specified above in nDisplay's color grading stack for these viewports. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Viewport Settings", meta = (DisplayName = "Include Entire Cluster Color Grading"))
	bool bIsEntireClusterEnabled = true;

	/** Color Grading */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Viewport Settings", meta = (DisplayName = "Color Grading"))
	FDisplayClusterConfigurationViewport_ColorGradingRenderingSettings ColorGradingSettings;

	/** Specify the viewports to apply these color grading settings. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Viewport Settings", meta = (DisplayName = "Apply Color Grading to Viewports"))
	TArray<FString> ApplyPostProcessToObjects;
};

USTRUCT(Blueprintable)
struct FDisplayClusterConfigurationViewport_AllNodesColorGrading
{
	GENERATED_BODY()

	/** Enable the color grading settings on the inner frustum for the all nodes and add them to nDisplay's color grading stack. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Viewport Color Grading", meta = (DisplayName = "Enable All Nodes Color Grading"))
	bool bEnableInnerFrustumAllNodesColorGrading = true;

	/** Optionally include Entire Cluster Color Grading settings specified on the root actor in nDisplay's color grading stack for the inner frustum. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Viewport Color Grading", meta = (DisplayName = "Include Entire Cluster Color Grading"))
	bool bEnableEntireClusterColorGrading = true;

	/** Color Grading */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Viewport Settings", meta = (DisplayName = "Color Grading"))
	FDisplayClusterConfigurationViewport_ColorGradingRenderingSettings ColorGradingSettings;
};

USTRUCT(Blueprintable)
struct FDisplayClusterConfigurationViewport_PerNodeColorGrading
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = "Viewport Settings")
	FText Name = FText::GetEmpty();
#endif

	/** Enable the color grading settings for the node(s) specified and add them to nDisplay's color grading stack. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Viewport Settings", meta = (DisplayName = "Enable Per-Node Color Grading"))
	bool bIsEnabled = true;

	/** Optionally include Entire Cluster Color Grading settings specified on the root actor in nDisplay's color grading stack for these nodes. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Viewport Settings", meta = (DisplayName = "Include Entire Cluster Color Grading"))
	bool bEntireClusterColorGrading = true;

	/** Optionally include the All Nodes Color Grading settings specified above in nDisplay's color grading stack for these nodes. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Viewport Settings", meta = (DisplayName = "Include All Nodes Color Grading"))
	bool bAllNodesColorGrading = true;

	/** Color Grading */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Viewport Settings", meta = (DisplayName = "Color Grading"))
	FDisplayClusterConfigurationViewport_ColorGradingRenderingSettings ColorGradingSettings;

	/** Specify the nodes to apply these color grading settings. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Viewport Settings", meta = (DisplayName = "Apply Color Grading to Nodes"))
	TArray<FString> ApplyPostProcessToObjects;
};