// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DisplayClusterConfigurationTypes_Base.h"
#include "DisplayClusterConfigurationTypes_Enums.h"

#include "DisplayClusterConfigurationTypes_Tile.generated.h"

/*
 * Tile rendering
 */
USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationTile_Settings
{
	GENERATED_BODY()

public:
	/** Return true if tile rendering can be used. */
	bool IsEnabled(const struct FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings) const;

	/** Layout validation. */
	static bool IsValid(const FIntPoint& InTilesLayout);

public:
	/** Enable tile rendering. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Tile Rendering")
	bool bEnabled = false;

	/** Tiling layout (X by Y tiles amount). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Tile Rendering", meta = (ClampMin = "1", ClampMax = "4"))
	FIntPoint Layout = { 1, 1 };
};

USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationTile_Overscan
{
	GENERATED_BODY()

public:
	/** Enable/disable Viewport Overscan and specify units as percent or pixel values. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Tile Overscan", meta = (DisplayName = "Enable Tile Overscan"))
	bool bEnabled = false;

	/** Set to True to render at the overscan resolution, set to false to render at the resolution in the configuration and scale for overscan. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Tile Overscan", meta = (DisplayName = "Adapt Resolution", EditCondition = "bEnabled == true"))
	bool bOversize = true;

	/** Optimize overscan values on boundary tiles.
	* When enabled, tile sides not in contact with other tiles will use zero overscan. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Tile Overscan", meta = (DisplayName = "No Overscan For Edges", EditCondition = "bEnabled == true"))
	bool bOptimizeTileOverscan = true;

	/** Enable/disable Viewport Overscan and specify units as percent or pixel values. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Tile Overscan", meta = (DisplayName = "Overscan Units", EditCondition = "bEnabled == true"))
	EDisplayClusterConfigurationViewportOverscanMode Mode = EDisplayClusterConfigurationViewportOverscanMode::Percent;

	/** Overscan value for all sides. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Tile Overscan", meta = (DisplayName = "Overscan Value", EditCondition = "bEnabled == true"))
	float AllSides = 10;

	 /**
	  * Defines how the overscan blend width is interpreted.
	  * Allows blending between tiles to be specified in
	  * a percentage of the overscan size, or disabled entirely.
	  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Tile Overscan", meta = (DisplayName = "Blend Mode", EditCondition = "bEnabled == true"))
	EDisplayClusterConfigurationViewportOverscanBlendMode OverscanBlendMode = EDisplayClusterConfigurationViewportOverscanBlendMode::Percent50;
};
