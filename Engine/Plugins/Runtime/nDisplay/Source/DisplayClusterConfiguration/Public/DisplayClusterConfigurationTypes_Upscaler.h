// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DisplayClusterConfigurationTypes_Enums.h"
#include "Render/Upscaler/DisplayClusterUpscalerSettings.h"

#include "StructUtils/PropertyBag.h"

#include "DisplayClusterConfigurationTypes_Upscaler.generated.h"

/**
* Upscaler settings.
* 
* MethodName  - refers to one of the available methods
* EditingData - Editable data for selected Method.
*               FInstancedPropertyBag is an abstract container and is not handled on the nDisplay side.
* 
* MethodName is customized as a drop-down list where items are taken from:
* - EDisplayClusterConfigurationUpscalingMethod enum elements
* - Upscalers exposed in the IUpscalerModularFeature interface.
* 
* Data logic are implemented in the FDisplayClusterConfigurationUpscalerSettingsDetailCustomization.
* The initial structure FInstancedPropertyBag is derived by the IUpscalerModularFeature::GetSettings().
*/
USTRUCT(BlueprintType)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationUpscalerSettings
{
	GENERATED_BODY()

public:
	/** Get upscaler settings. */
	bool GetUpscalerSettings(
		const FDisplayClusterConfigurationUpscalerSettings* InDefaultUpscalerSettings,
		FDisplayClusterUpscalerSettings& OutUpscalerSettings) const;

public:
	/** Override the Upscaling method. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Upscaler", meta = (DisplayName = "Upscaling Method"))
	FName MethodName;

	/** Upscaling method settings. */
	UPROPERTY(EditAnywhere, Category = "Upscaler", meta = (DisplayName = "Upscaling Settings"))
	FInstancedPropertyBag EditingData;

	/**
	 * In cases where these upscaler settings are overriding global upscaler settings, this list stores the IDs of the properties
	 * that will override the global settings. These are exposed and utilized when the owner of the UpscalerSettings struct uses
	 * the WithOverrides metadata specifier
	 */
	UPROPERTY()
	TArray<FGuid> ParameterOverrideGuids;
};
