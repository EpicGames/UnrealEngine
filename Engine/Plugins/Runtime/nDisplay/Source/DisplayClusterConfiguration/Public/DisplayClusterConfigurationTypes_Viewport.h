// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"

#include "DisplayClusterConfigurationStrings.h"
#include "DisplayClusterConfigurationTypes_Base.h"
#include "DisplayClusterConfigurationTypes_ICVFX.h"
#include "DisplayClusterConfigurationTypes_Upscaler.h"
#include "DisplayClusterConfigurationTypes_Media.h"
#include "DisplayClusterConfigurationTypes_PostRender.h"
#include "DisplayClusterConfigurationTypes_Postprocess.h"
#include "DisplayClusterConfigurationTypes_ViewportRemap.h"
#include "DisplayClusterConfigurationTypes_ViewportOverscan.h"

#include "Containers/DisplayClusterShader_Enums.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_EnumsICVFX.h"

#include "OpenColorIOColorSpace.h"
#include "Engine/Scene.h"

#include "DisplayClusterConfigurationTypes_Viewport.generated.h"

struct FDisplayClusterConfigurationICVFX_StageSettings;

/**
* Unique ICVFX customisation for each viewport.
* 
* Must be processed in UDisplayClusterConfigurationViewport::GetViewportICVFXFlags().
* This will result in some EDisplayClusterViewportICVFXFlags being raised.
*/
USTRUCT(BlueprintType)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationViewport_ICVFX
{
	GENERATED_BODY()

public:
	/** Get lightcard render mode for this viewport. */
	UE_DEPRECATED(5.5, "This function has been moved to FDisplayClusterConfigurationICVFX_LightcardSettings.")
	EDisplayClusterShaderParametersICVFX_LightCardRenderMode GetLightCardRenderMode(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings) const
	{
		return EDisplayClusterShaderParametersICVFX_LightCardRenderMode::None;
	}

	/** Get ICVFX settings flags for viewport*/
	UE_DEPRECATED(5.5, "This function has been moved to UDisplayClusterConfigurationViewport.")
	EDisplayClusterViewportICVFXFlags GetViewportICVFXFlags(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings) const
	{
		return EDisplayClusterViewportICVFXFlags::None;
	}

	EDisplayClusterConfigurationICVFX_OverrideChromakeyType GetOverrideChromakeyType(const FString& CameraId) const;

public:
	/** Enable in-camera VFX for this Viewport (works only with supported Projection Policies) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "In Camera VFX")
	bool bAllowICVFX = true;

	/** Allow the inner frustum to appear on this Viewport */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "In Camera VFX")
	bool bAllowInnerFrustum = true;

	/** Disable incamera render to this viewport */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "In Camera VFX")
	EDisplayClusterConfigurationICVFX_OverrideCameraRenderMode CameraRenderMode = EDisplayClusterConfigurationICVFX_OverrideCameraRenderMode::Default;

	/** Use unique lightcard mode for this viewport */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "In Camera VFX")
	EDisplayClusterConfigurationICVFX_OverrideLightcardRenderMode LightcardRenderMode = EDisplayClusterConfigurationICVFX_OverrideLightcardRenderMode::Default;

	/** Use unique chromakey type for this viewport */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "In Camera VFX")
	EDisplayClusterConfigurationICVFX_OverrideChromakeyType OverrideChromakeyType = EDisplayClusterConfigurationICVFX_OverrideChromakeyType::Default;

	/** Determines the chromakey override per-camera in this viewport.  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "In Camera VFX")
	TMap<FString, EDisplayClusterConfigurationICVFX_OverrideChromakeyType> PerCameraOverrideChromakeyType;

	/** Enforced rendering order for ICVFX cameras on this viewport.
	 * Cameras listed here will be rendered in this order (index 0 is composed on top).
	 * Cameras not in this list will be composed below, in their default order based on RenderOrder property.
	 * If empty, all cameras use default render order. 
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "In Camera VFX")
	TArray<FString> EnforcedCameraOrder;

	/** The order in which the ICVFX cameras are composited over is reversed. Useful for time-multiplexed displays. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "In Camera VFX")
	bool bReverseCameraPriority = false;
};

USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationViewport_RenderSettings
{
	GENERATED_BODY()

public:
	/* Enable cross-GPU transfer for this viewport.
	 * It may be disabled in some configurations. For example, when using offscreen rendering with TextureShare,
	 * cross-gpu transfer can be disabled for this viewport to improve performance, because when transfer is called, 
	 * it freezes the GPUs until synchronization is reached.
	 * (TextureShare uses its own implementation of the crossGPU transfer for the shared textures.) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering", meta = (DisplayName = "Enable Cross-GPU Transfer"))
	bool bEnableCrossGPUTransfer = true;

	/**
	* Specifies the GPU index for the nDisplay viewport in stereo rendering for the second eye.
	* A value of '-1' means to use the value from the GPU Index parameter. (the same value is used for both eyes).
	* Used to improve rendering performance by spreading the load across multiple GPUs.
	*/
	UPROPERTY(EditAnywhere, Category = "Stereo", meta = (DisplayName = "Stereo GPU Index", ClampMin = "-1", UIMin = "-1", ClampMax = "8", UIMax = "8"))
	int StereoGPUIndex = INDEX_NONE;

	/** Enables and sets Stereo mode */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Stereo")
	EDisplayClusterConfigurationViewport_StereoMode StereoMode = EDisplayClusterConfigurationViewport_StereoMode::Default;

	/** Upscaler settings for the viewport. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering", meta = (DisplayName = "Upscaler Settings", NoResetToDefault, WithOverrides))
	FDisplayClusterConfigurationUpscalerSettings UpscalerSettings;

	/** Adjust resolution scaling for an individual viewport.  Viewport Screen Percentage Multiplier is applied to this value. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering", meta = (DisplayName = "Screen Percentage", ClampMin = "0.05", UIMin = "0.05", ClampMax = "10.0", UIMax = "1.0"))
	float BufferRatio = 1;

	/** Adjust resolution scaling for an individual viewport.  Viewport Screen Percentage Multiplier is applied to this value. */
	UPROPERTY()
	float RenderTargetRatio = 1.f;

	UPROPERTY()
	FDisplayClusterConfigurationViewport_CustomPostprocess CustomPostprocess;

	/** Override viewport render from source texture */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Texture Replacement")
	FDisplayClusterConfigurationPostRender_Override Replace;

	/** Add postprocess blur to viewport */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Configuration", AdvancedDisplay)
	FDisplayClusterConfigurationPostRender_BlurPostprocess PostprocessBlur;

	/** Generate Mips texture for this viewport (used, only if projection policy supports this feature) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Configuration", AdvancedDisplay)
	FDisplayClusterConfigurationPostRender_GenerateMips GenerateMips;

	/** Render a larger frame than specified in the configuration to achieve continuity across displays when using post-processing effects. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Configuration")
	FDisplayClusterConfigurationViewport_Overscan Overscan;

	/** Override actor visibility for this viewport. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Configuration")
	FDisplayClusterConfigurationICVFX_VisibilityList HiddenContent;
	
	// Media settings
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Media", meta = (DisplayName = "Media", ShowOnlyInnerProperties))
	FDisplayClusterConfigurationMediaViewport Media;

	// Experimental: Support special frame builder mode - merge viewports to single viewfamily by group num
	// [not implemented yet]
	UPROPERTY()
	int RenderFamilyGroup = -1;

};


UCLASS(Blueprintable)
class DISPLAYCLUSTERCONFIGURATION_API UDisplayClusterConfigurationViewport
	: public UDisplayClusterConfigurationData_Base
{
	GENERATED_BODY()
public:

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPostEditChangeChainProperty, const FPropertyChangedChainEvent&);

	FOnPostEditChangeChainProperty OnPostEditChangeChainProperty;

public:
	UDisplayClusterConfigurationViewport();

public:
	//~ Begin UObject interface
	virtual void PostLoad() override;

#if WITH_EDITOR
	virtual void PreEditChange(FEditPropertyChain& PropertyAboutToChange) override;
	virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject interfacef

public:
	void GetReferencedMeshNames(TArray<FString>& OutMeshNames) const;

	/** The viewport object will only be created or updated if this function returns true.*/
	bool IsViewportEnabled() const
	{
		return bAllowRendering;
	}

	/** Get ICVFX settings flags for viewport*/
	EDisplayClusterViewportICVFXFlags GetViewportICVFXFlags(const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings) const;

	/** Returns the chromakey type for this viewport used by the specified ICVFX camera. */
	EDisplayClusterShaderParametersICVFX_ChromakeySource GetViewportChromakeyType(
		const FDisplayClusterConfigurationICVFX_StageSettings& InStageSettings,
		const FString& InCameraId,
		const struct FDisplayClusterConfigurationICVFX_CameraSettings& InCameraSettings) const;

#if WITH_EDITOR
	/** Enable the preview texture. Only should be called by the object managing the preview texture state. */
	void EnablePreviewTexture();
	
	/**
	 * Signal that the preview texture should be disabled.
	 * @return True if the preview texture was disabled. False if it was already disabled.
	 */
	bool DisablePreviewTexture();

	/** If this viewport is allowed to render a preview texture. Used with resizing viewports. */
	bool IsPreviewTextureAllowed() const { return bAllowPreviewTexture; }
	
protected:
	virtual void OnPreCompile(class FCompilerResultsLog& MessageLog) override;
	
private:
	/** If this viewport is allowed to render a preview texture. */
	bool bAllowPreviewTexture = true;
	
	/** If this object is managing the preview texture state. */
	bool bIsManagingPreviewTexture = false;
#endif

public:
	/** Enables or disables rendering of this specific Viewport */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Configuration", meta = (DisplayName = "Enable Viewport"))
	bool bAllowRendering = true;

	/** Reference to the nDisplay View Point*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Configuration", meta = (DisplayName = "View Point"))
	FString Camera;

	/** Specify your Projection Policy Settings */
	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Category = "Configuration")
	FDisplayClusterConfigurationProjection ProjectionPolicy;

	/** Select a display device to use during preview. An empty string will use the default display device */
	UPROPERTY(EditAnywhere, Category = "Preview", meta = (DisplayName = "Display Device"))
	FString DisplayDeviceName;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Preview", meta = (DisplayName = "Preview Frustum"))
	bool bAllowPreviewFrustumRendering = false;
	
	/** Define the Viewport 2D coordinates */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Configuration", meta = (DisplayMode = "Compound", FixedAspectRatioProperty = "bFixedAspectRatio"))
	FDisplayClusterConfigurationRectangle Region;

	/** Define the Viewport Remap settings */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Configuration", meta = (DisplayName = "Remapping"))
	FDisplayClusterConfigurationViewport_Remap ViewportRemap;

	/** Allows Viewports to overlap and sets Viewport overlapping order priority */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering")
	int OverlapOrder = 0;

	/**
	* Specifies the GPU index for the nDisplay viewport.
	* Value '-1' means do not use multi-GPU
	* Used to improve rendering performance by spreading the load across multiple GPUs.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering", meta = (DisplayName = "GPU Index", ClampMin = "-1", UIMin = "-1", ClampMax = "8", UIMax = "8"))
	int GPUIndex = -1;

	// Configure render for this viewport
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Configuration", meta = (ShowOnlyInnerProperties))
	FDisplayClusterConfigurationViewport_RenderSettings RenderSettings;

	// Configure ICVFX for this viewport
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "In Camera VFX", meta = (ShowOnlyInnerProperties))
	FDisplayClusterConfigurationViewport_ICVFX ICVFX;

#if WITH_EDITORONLY_DATA
	/** Locks the Viewport aspect ratio for easier resizing */
	UPROPERTY(EditAnywhere, Category = "Configuration", meta = (HideProperty))
	bool bFixedAspectRatio;

	UPROPERTY(EditDefaultsOnly, Category = "Configuration", meta = (HideProperty))
	bool bIsUnlocked = true;

	UPROPERTY(EditDefaultsOnly, Category = "Configuration", meta = (HideProperty))
	bool bIsVisible = true;
#endif

public:
	static const float ViewportMinimumSize;
	static const float ViewportMaximumSize;
};

// This struct now stored in UDisplayClusterConfigurationData, and replicated with MultiUser
USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationRenderFrame
{
	GENERATED_BODY()

public:
	// Performance: Allow merge multiple viewports on single RTT with atlasing (required for bAllowViewFamilyMergeOptimization)
	// [not implemented yet] Experimental
	UPROPERTY()
	bool bAllowRenderTargetAtlasing = false;

	// Performance: Allow viewfamily merge optimization (render multiple viewports contexts within single family)
	// [not implemented yet] Experimental
	UPROPERTY()
	EDisplayClusterConfigurationRenderFamilyMode ViewFamilyMode = EDisplayClusterConfigurationRenderFamilyMode::None;

	// Performance: Allow to use parent ViewFamily from parent viewport 
	// (icvfx has child viewports: lightcard and chromakey with prj_view matrices copied from parent viewport. May sense to use same viewfamily?)
	// [not implemented yet] Experimental
	UPROPERTY()
	bool bShouldUseParentViewportRenderFamily = false;

	// Multiplies the RTT size of all viewports within nDisplay by this value.
	UPROPERTY(BlueprintReadWrite, Category = NDisplay, meta = (DisplayName = "Global RTT Size Multiplier", ClampMin = "0.05", UIMin = "0.05", ClampMax = "10", UIMax = "10"))
	float ClusterRenderTargetRatioMult = 1.f;

	// Multiplies the RTT size of the ICVFX Inner Frustum viewports by this value.
	UPROPERTY(BlueprintReadWrite, Category = NDisplay, meta = (DisplayName = "Inner Frustum RTT Size Multiplier", ClampMin = "0.05", UIMin = "0.05", ClampMax = "10", UIMax = "10"))
	float ClusterICVFXInnerViewportRenderTargetRatioMult = 1.f;

	// Multiplies the RTT size of the viewports by this value.
	// (Excluding ICVFX internal viewports such as Inner frustum, LightCards, Chromakey, etc.)
	UPROPERTY(BlueprintReadWrite, Category = NDisplay, meta = (DisplayName = "Viewports RTT Size Multiplier", ClampMin = "0.05", UIMin = "0.05", ClampMax = "10", UIMax = "10"))
	float ClusterICVFXOuterViewportRenderTargetRatioMult = 1.f;

	// Multiplies all screen percentages within nDisplay by this value.
	UPROPERTY(BlueprintReadWrite, Category = NDisplay, meta = (DisplayName = "Global Screen Percentage Multiplier", ClampMin = "0.05", UIMin = "0.05", ClampMax = "10", UIMax = "10"))
	float ClusterBufferRatioMult = 1.f;

	// Multiplies the screen percentage for all ICVFX Inner Frustum viewports by this value.
	UPROPERTY(BlueprintReadWrite, Category = NDisplay, meta = (DisplayName = "Inner Frustum Screen Percentage Multiplier", ClampMin = "0.05", UIMin = "0.05", ClampMax = "10", UIMax = "10"))
	float ClusterICVFXInnerFrustumBufferRatioMult = 1.f;

	// Multiplies the screen percentage for viewports by this value.
	// (Excluding ICVFX internal viewports such as Inner Frustum, LightCards and Chromakey.)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NDisplay, meta = (DisplayName = "Viewports Screen Percentage Multiplier", ClampMin = "0.05", UIMin = "0.05", ClampMax = "10", UIMax = "1"))
	float ClusterICVFXOuterViewportBufferRatioMult = 1.f;

	// Allow warpblend render
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Configuration")
	bool bAllowWarpBlend = true;
};
