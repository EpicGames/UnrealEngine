// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"

#include "DisplayClusterConfigurationTypes_Enums.h"

#include "DisplayClusterConfigurationTypes_ViewportOverscan.generated.h"

USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationViewport_Overscan
{
	GENERATED_BODY()

public:
	/** Enable/disable Viewport Overscan and specify units as percent or pixel values. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Overscan", meta = (DisplayName = "Enable"))
	bool bEnabled = false;

	/** Enable/disable Viewport Overscan and specify units as percent or pixel values. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Overscan")
	EDisplayClusterConfigurationViewportOverscanMode Mode = EDisplayClusterConfigurationViewportOverscanMode::Percent;

	/** Left */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Overscan")
	float Left = 0;

	/** Right */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Overscan")
	float Right = 0;

	/** Top */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Overscan")
	float Top  = 0;

	/** Bottom */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Overscan")
	float Bottom = 0;

	/** Set to True to render at the overscan resolution, set to false to render at the resolution in the configuration and scale for overscan. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Overscan", meta = (DisplayName = "Adapt Resolution", DisplayAfter = "Mode"))
	bool bOversize = true;
};
