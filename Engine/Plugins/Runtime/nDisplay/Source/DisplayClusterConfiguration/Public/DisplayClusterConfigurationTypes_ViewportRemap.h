// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"

#include "DisplayClusterConfigurationTypes_Base.h"

#include "DisplayClusterConfigurationTypes_ViewportRemap.generated.h"

/**
 * Remapping configuration for a single remapped region, which can be any subregion of a viewport, and can be remapped to any
 * part of the screen, and can be rotated or flipped
 */
USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationViewport_RemapData
{
	GENERATED_BODY()

public:
	/** The subregion of the viewport to remap; (0,0) x (W, H) will remap the entire viewport */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Configuration")
	FDisplayClusterConfigurationRectangle ViewportRegion;

	/** The region in screen space to output the remapped region to */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Configuration")
	FDisplayClusterConfigurationRectangle OutputRegion;

	/** The angle in degrees to rotate the remapped region by; rotation is performed around the center of the output region */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Configuration")
	float Angle = 0;

	/** Flips the remapped region horizontally */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Configuration")
	bool bFlipH = false;

	/** Flips the remapped region vertically */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Configuration")
	bool bFlipV = false;


	/** Determines if the remap has non-trivial transformations to apply to a viewport */
	bool IsValid() const;

	/** Determines if the remap has a non-trivial rotation to apply to the viewport */
	bool IsRotating() const;

	/** Determines if the remap is applying a flip to the viewport */
	bool IsFlipping() const;
};

/** Configuration for all remapping to apply to a single viewport */
USTRUCT(Blueprintable)
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterConfigurationViewport_Remap
{
	GENERATED_BODY()

public:
	/** Enables or disables viweport output remapping */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Configuration", meta = (DisplayName = "Enable Output Remapping"))
	bool bEnable = true;

	/** The base remap to apply to the entire viewport, used to apply flipping and rotation to the viewport */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Configuration", meta = (NoHeader, Simplified, AngleInterval = "90"))
	FDisplayClusterConfigurationViewport_RemapData BaseRemap;

	/** Additional remaps to apply to the viewport, can be used to subdivide the viewport into multiple outputs. Experimental, not exposed to users at this time */
	UPROPERTY()
	TArray<FDisplayClusterConfigurationViewport_RemapData> RemapRegions;
};
