// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

enum class EDisplayClusterConfigurationVersion : uint8
{
	Unknown,     // Unknown version or not a config file
	Version_426, // 4.26 JSON based config format
	Version_427, // 4.27 JSON based config format
	Version_500, // 5.00 JSON based config format
};

namespace DisplayClusterConfiguration
{
	static constexpr const TCHAR* GetCurrentConfigurationSchemeMarker()
	{
		return TEXT("5.00");
	}
}
