// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DisplayClusterEditorPropertyReference.generated.h"

/**
 * A dummy structure used to reference properties of subobjects to be displayed at the root level in a details panel.
 * 
 * When placed within an Unreal class or struct, the property reference is replaced with the referenced property in the
 * type's details panel. The display name, tooltip, category, and accessiblity of the property reference will be applied 
 * to the referenced property when provided, overwriting the property's own specifiers. If the property path contains lists
 * (arrays, maps, or sets), then each element of the list will be iterated over, and all properties within that list will 
 * be displayed together in a group.
 * 
 * Use the PropertyPath metadata specifier to specify the path to the referenced property, relative to the object that owns
 * the property reference. 
 * 
 * Additionally, this type supports using property paths within its EditCondition specifier, allowing the edit condition
 * of the referenced property within a details panel to depend on other referenced properties. The && operator is supported,
 * allowing multiple property paths to be used to construct the edit condition.
 */
USTRUCT()
struct DISPLAYCLUSTERCONFIGURATION_API FDisplayClusterEditorPropertyReference
{
	GENERATED_BODY()
};