// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#include "DisplayClusterConfigurationVersion.h"

class UDisplayClusterConfigurationData;


/**
 * Display Cluster configuration module interface
 */
class IDisplayClusterConfiguration : public IModuleInterface
{
public:
	static constexpr const TCHAR* ModuleName = TEXT("DisplayClusterConfiguration");

public:
	virtual ~IDisplayClusterConfiguration() = default;

public:
	/**
	* Singleton-like access to this module's interface.  This is just for convenience!
	* Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	*
	* @return Returns singleton instance, loading the module on demand if needed
	*/
	static inline IDisplayClusterConfiguration& Get()
	{
		return FModuleManager::GetModuleChecked<IDisplayClusterConfiguration>(IDisplayClusterConfiguration::ModuleName);
	}

	/**
	* Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	*
	* @return True if the module is loaded and ready to use
	*/
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded(IDisplayClusterConfiguration::ModuleName);
	}

public:

	/**
	 * Sets the transaction flags so that we don't necessarily re-run the construction scripts all the time.
	 */
	virtual void SetIsSnapshotTransacting(bool bIsSnapshot) = 0;

	/**
	 * Gets the current transaction snapshot status.
	 */
	virtual bool IsTransactingSnapshot() const = 0;

	/**
	* Returns version of a specified config file
	*
	* @param FilePath   - Config file path
	*
	* @return Config version if succeeded, otherwise nulltpr
	*/
	virtual EDisplayClusterConfigurationVersion GetConfigVersion(const FString& FilePath) = 0;

	/**
	* Load configuration data from specified file
	*
	* @param FilePath   - Config file path
	* @param Owner      - UObject owner for the configuration data object
	*
	* @return Configuration data if succeeded, otherwise nulltpr
	*/
	virtual UDisplayClusterConfigurationData* LoadConfig(const FString& FilePath, UObject* Owner = nullptr) = 0;

	/**
	* Save configuration data to a specified file
	*
	* @param Config - Configuration data
	* @param FilePath - File to save
	*
	* @return true if succeeded
	*/
	virtual bool SaveConfig(const UDisplayClusterConfigurationData* Config, const FString& FilePath) = 0;

	/**
	* Converts configuration data to string
	*
	* @param Config - Configuration data
	* @param OutString - String to contain the configuration data
	*
	* @return true if succeeded
	*/
	virtual bool ConfigAsString(const UDisplayClusterConfigurationData* Config, FString& OutString) const = 0;
};
