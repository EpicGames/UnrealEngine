// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class DisplayClusterConfigurator : ModuleRules
{
	public DisplayClusterConfigurator(ReadOnlyTargetRules ROTargetRules) : base(ROTargetRules)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"DisplayCluster",
				"DisplayClusterConfiguration",
				"DisplayClusterModularFeaturesEditor",
				"DisplayClusterProjection",
				"DisplayClusterWarp",

				"AdvancedPreviewScene",
				"ApplicationCore",
				"AppFramework",
				"AssetDefinition",
				"AssetTools",
				"CinematicCamera",
				"ClassViewer",
				"ColorGradingEditor",
				"Core",
				"CoreUObject",
				"DesktopPlatform",
				"BlueprintGraph",
				"GraphEditor",
				"EditorFramework",
				
				"EditorSubsystem",
				"EditorWidgets",
				"Engine",
				"EngineAssetDefinitions",
				"ImageWrapper",
				"InputCore",
				"Kismet",
				"KismetCompiler",
				"MainFrame",
				"MediaAssets",
				"MediaIOCore",
				"MessageLog",
				"Networking",
				"PinnedCommandList",
				"Projects",
				"PropertyEditor",
				"Serialization",
				"Settings",
				"Slate",
				"SlateCore",
				"StructUtilsEditor",
				"ToolMenus",
				"UnrealEd",
				"SubobjectEditor",
				"SubobjectDataInterface",
				"ToolWidgets",
				"ProceduralMeshComponent",
			});
	}
}
