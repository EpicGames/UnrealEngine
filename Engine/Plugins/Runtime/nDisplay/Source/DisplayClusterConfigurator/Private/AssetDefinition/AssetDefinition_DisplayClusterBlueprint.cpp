// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_DisplayClusterBlueprint.h"

#include "DisplayClusterConfiguratorBlueprintEditor.h"
#include "DisplayClusterConfiguratorFactory.h"
#include "Blueprints/DisplayClusterBlueprint.h"

#define LOCTEXT_NAMESPACE "AssetDefinition_DisplayClusterBlueprint"

FText UAssetDefinition_DisplayClusterBlueprint::GetAssetDisplayName() const
{
	return LOCTEXT("DisplayClusterBlueprint_AssetName", "nDisplay Config");
}

FLinearColor UAssetDefinition_DisplayClusterBlueprint::GetAssetColor() const
{
	return FColor(0, 188, 212);
}

TSoftClassPtr<UObject> UAssetDefinition_DisplayClusterBlueprint::GetAssetClass() const
{
	return UDisplayClusterBlueprint::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_DisplayClusterBlueprint::GetAssetCategories() const
{
	static const auto Categories =
	{
		FAssetCategoryPath(FAssetCategoryPath(LOCTEXT("DisplayClusterBlueprint_AssetCategory", "Virtual Production")), LOCTEXT("DisplayClusterBlueprint_CategorySection", "nDisplay"), ECategoryMenuType::Section)
	};

	return Categories;
}

EAssetCommandResult UAssetDefinition_DisplayClusterBlueprint::GetSourceFiles(const FAssetSourceFilesArgs& InArgs, TFunctionRef<bool(const FAssetSourceFilesResult& InSourceFile)> SourceFileFunc) const
{
	for (const FAssetData& AssetData : InArgs.Assets)
	{
		if (const UDisplayClusterBlueprint* Asset = Cast<UDisplayClusterBlueprint>(AssetData.GetAsset()))
		{
			if (UDisplayClusterConfigurationData* ConfigData = Asset->GetConfig())
			{
				const FString& Path = ConfigData->ImportedPath;
				if (!Path.IsEmpty())
				{
					FAssetSourceFilesResult Result;
					Result.FilePath = Path;
					Result.DisplayLabel = Path;
					
					if (!SourceFileFunc(Result))
					{
						break;
					}
				}
			}
		}
	}

	return EAssetCommandResult::Handled;
}

EAssetCommandResult UAssetDefinition_DisplayClusterBlueprint::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	const EToolkitMode::Type Mode = OpenArgs.ToolkitHost.IsValid() ? EToolkitMode::WorldCentric : EToolkitMode::Standalone;

	TArray<UObject*> Objects = OpenArgs.LoadObjects<UObject>();
	for (TArray<UObject*>::TConstIterator ObjIt = Objects.CreateConstIterator(); ObjIt; ++ObjIt)
	{
		if (UDisplayClusterBlueprint* BP = Cast<UDisplayClusterBlueprint>(*ObjIt))
		{
			if (BP->bIsNewlyCreated && BP->GetConfig() == nullptr)
			{
				// This path can be hit if the BP was created by a default factory.
				UDisplayClusterConfiguratorFactory::SetupNewBlueprint(BP);
			}
			
			TSharedRef<FDisplayClusterConfiguratorBlueprintEditor> BlueprintEditor(new FDisplayClusterConfiguratorBlueprintEditor());
			BlueprintEditor->InitDisplayClusterBlueprintEditor(Mode, OpenArgs.ToolkitHost, BP);
		}
	}
	
	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE
