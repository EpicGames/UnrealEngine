// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Script/AssetDefinition_Blueprint.h"
#include "AssetDefinition_DisplayClusterBlueprint.generated.h"

UCLASS()
class UAssetDefinition_DisplayClusterBlueprint : public UAssetDefinition_Blueprint
{
	GENERATED_BODY()
	
public:
	//~ Begin UAssetDefinition
	virtual FText GetAssetDisplayName() const override;
	virtual FLinearColor GetAssetColor() const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	virtual bool CanImport() const override { return true; }
	virtual EAssetCommandResult GetSourceFiles(const FAssetSourceFilesArgs& InArgs, TFunctionRef<bool(const FAssetSourceFilesResult& InSourceFile)> SourceFileFunc) const override;
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	//~ End UAssetDefinition
};
