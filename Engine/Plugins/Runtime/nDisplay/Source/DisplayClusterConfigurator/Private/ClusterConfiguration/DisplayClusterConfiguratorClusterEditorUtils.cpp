// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfiguratorClusterEditorUtils.h"

#include "ClusterConfiguration/DisplayClusterConfiguratorClusterUtils.h"
#include "DisplayClusterConfiguratorBlueprintEditor.h"
#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterConfiguratorPropertyUtils.h"
#include "DisplayClusterConfiguratorStyle.h"
#include "ClusterConfiguration/SDisplayClusterConfiguratorNewClusterItemDialog.h"
#include "ClusterConfiguration/ViewModels/DisplayClusterConfiguratorClusterNodeViewModel.h"
#include "Views/DragDrop/DisplayClusterConfiguratorClusterNodeDragDropOp.h"
#include "Views/DragDrop/DisplayClusterConfiguratorViewportDragDropOp.h"

#include "Factories.h"
#include "ScopedTransaction.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Misc/StringOutputDevice.h"
#include "UnrealExporter.h"
#include "Editor/Transactor.h"
#include "Editor/TransBuffer.h"
#include "Exporters/Exporter.h"
#include "Input/DragAndDrop.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "FDisplayClusterConfiguratorClusterEditorUtils"

namespace UE::DisplayClusterConfiguratorClusterEditorUtils
{
	// All transactions performed within this scope can be considered temporary and will be cleared when out of scope.
	// Transactions prior to the scope cannot be undone while in scope.
	struct FScopedTemporaryTransactions
	{
		// The index of the transaction queue when entering the scope
		int32 StartTransactionIdx = 0;
		
		FScopedTemporaryTransactions()
		{
			if (GEditor)
			{
				// Prevent undoing any actions before the scope started
				GEditor->Trans->SetUndoBarrier();

				// Record the transaction idx before we start our scoped transactions
				StartTransactionIdx = GEditor->Trans->GetCurrentUndoBarrier();

				// Broadcasting changes here can take a big performance hit and isn't necessary since we only use this in a modal window
				GEditor->bSuspendBroadcastPostUndoRedo = true;
			}
		}

		~FScopedTemporaryTransactions()
		{
			if (GEditor)
			{
				GEditor->bSuspendBroadcastPostUndoRedo = false;

				// Clear all transactions that occurred while in scope, accounting for transactions that were
				// undone. None of these transactions should be allowed to be undone or redone from this point forward
				{
					UTransBuffer* TransBuffer = CastChecked<UTransBuffer>(GEditor->Trans);
					const int32 ScopedTransactionsPerformed = TransBuffer->UndoBuffer.Num() - StartTransactionIdx;
				
					if (ScopedTransactionsPerformed > 0
						&& StartTransactionIdx >= 0
						&& StartTransactionIdx < TransBuffer->UndoBuffer.Num()
						&& (StartTransactionIdx + ScopedTransactionsPerformed - 1) < TransBuffer->UndoBuffer.Num())
					{
						// Remove only the transactions performed within scope
						TransBuffer->UndoBuffer.RemoveAt(StartTransactionIdx, ScopedTransactionsPerformed);

						// We can't restore redo items prior to the scope since they would have been cleared while
						// performing transactions within the scope
						TransBuffer->UndoCount = 0;

						// Need to broadcast for other systems since we are changing the undo buffer manually
						TransBuffer->OnUndoBufferChanged().Broadcast();
					}
				}

				// Allow undos again from before the scope started
				GEditor->Trans->RemoveUndoBarrier();
			}
		}
	};

	const FVector2D NewClusterItemDialogSize = FVector2D(410, 512);
	const FString DefaultNewClusterNodeName = TEXT("Node");
	const FString DefaultNewViewportName = TEXT("VP");
}

using namespace UE::DisplayClusterConfiguratorClusterUtils;

UDisplayClusterConfigurationClusterNode* UE::DisplayClusterConfiguratorClusterEditorUtils::CreateNewClusterNodeFromDialog(
	const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& Toolkit,
	UDisplayClusterConfigurationCluster* Cluster,
	const FDisplayClusterConfigurationRectangle& PresetRect,
	TSharedPtr<FScopedTransaction>& OutTransaction,
	FString PresetHost)
{
	UDisplayClusterConfigurationClusterNode* NodeTemplate = NewObject<UDisplayClusterConfigurationClusterNode>(Toolkit->GetBlueprintObj(), NAME_None, RF_Transactional | RF_ArchetypeObject | RF_Public);
	NodeTemplate->WindowRect = FDisplayClusterConfigurationRectangle(PresetRect);
	NodeTemplate->Host = PresetHost;

	TArray<FString> ParentItems;
	ParentItems.Add("Cluster");

	const FString InitialName = GetUniqueNameForClusterNode(DefaultNewClusterNodeName, Cluster, true);
	
	bool bAutoposition = true;
	bool bAddViewport = true;
	const TSharedRef<SWidget> ClusterNodeFooter = SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2.0f)
		[
			SNew(SCheckBox)
			.IsChecked_Lambda([&bAutoposition]() { return bAutoposition ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
			.OnCheckStateChanged_Lambda([&bAutoposition](ECheckBoxState NewState) { bAutoposition = NewState == ECheckBoxState::Checked; })
			.ToolTipText(LOCTEXT("AddNewClusterNode_AutoPositionToolTip", "When checked, auto-positions the new cluster node to prevent overlap with other cluster nodes in the same host"))
			[
				SNew(STextBlock)
				.Text(LOCTEXT("AddNewClusterNode_AutoPositionLabel", "Adjust Cluster Node Position to Prevent Overlap"))
			]
		]

		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2.0f)
		[
			SNew(SCheckBox)
			.IsChecked_Lambda([&bAddViewport]() { return bAddViewport ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
			.OnCheckStateChanged_Lambda([&bAddViewport](ECheckBoxState NewState) { bAddViewport = NewState == ECheckBoxState::Checked; })
			.ToolTipText(LOCTEXT("AddNewClusterNode_AddViewportToolTip", "When checked, adds a new viewport to the cluster node when created that matches the size of the Node application window"))
			[
				SNew(STextBlock)
				.Text(LOCTEXT("AddNewClusterNode_AddViewportLabel", "Add Viewport to New Cluster Node"))
			]
		];

	const TSharedRef<SDisplayClusterConfiguratorNewClusterItemDialog> DialogContent = SNew(SDisplayClusterConfiguratorNewClusterItemDialog, NodeTemplate)
		.ParentItemOptions(ParentItems)
		.InitiallySelectedParentItem("Cluster")
		.PresetItemOptions(FDisplayClusterConfiguratorPresetSize::CommonPresets)
		.InitiallySelectedPreset(FDisplayClusterConfiguratorPresetSize::CommonPresets[FDisplayClusterConfiguratorPresetSize::DefaultPreset])
		.InitialName(InitialName)
		.MaxWindowWidth(NewClusterItemDialogSize.X)
		.MaxWindowHeight(NewClusterItemDialogSize.Y)
		.FooterContent(ClusterNodeFooter)
		.OnPresetChanged_Lambda([=](FVector2D Size) { NodeTemplate->WindowRect.W = Size.X; NodeTemplate->WindowRect.H = Size.Y; });

	{
		UE::DisplayClusterConfiguratorClusterEditorUtils::FScopedTemporaryTransactions ScopedTransactionsForModal;
		ShowNewClusterItemDialogWindow(DialogContent, nullptr, LOCTEXT("AddNewClusterNode_DialogTitle", "Add New Cluster Node"), NewClusterItemDialogSize);
	}
	
	UDisplayClusterConfigurationClusterNode* NewNode = nullptr;

	if (DialogContent->WasAccepted())
	{
		OutTransaction = MakeShared<FScopedTransaction>(LOCTEXT("AddClusterNode", "Add Cluster Node"));
		
		const FString ItemName = DialogContent->GetItemName();
		NodeTemplate->SetFlags(RF_Transactional);

		if (bAutoposition)
		{
			const FVector2D DesiredPosition = FVector2D(NodeTemplate->WindowRect.X, NodeTemplate->WindowRect.Y);
			const FVector2D DesiredSize = FVector2D(NodeTemplate->WindowRect.W, NodeTemplate->WindowRect.H);
			const FVector2D NewPosition = FindNextAvailablePositionForClusterNode(Cluster, NodeTemplate->Host, DesiredPosition, DesiredSize);

			NodeTemplate->WindowRect.X = NewPosition.X;
			NodeTemplate->WindowRect.Y = NewPosition.Y;
		}

		NewNode = AddClusterNodeToCluster(NodeTemplate, Cluster, ItemName);

		// If the newly added cluster node is the only node in the cluster, it should be the primary node, so set it as the primary
		if (Cluster->Nodes.Num() == 1)
		{
			SetClusterNodeAsPrimary(NewNode);
		}

		if (bAddViewport)
		{
			const FString ViewportName = GetUniqueNameForViewport(DefaultNewViewportName, NewNode, true);
			UDisplayClusterConfigurationViewport* NewViewport = NewObject<UDisplayClusterConfigurationViewport>(NewNode, NAME_None, RF_Transactional | RF_ArchetypeObject | RF_Public);
			NewViewport->Region.W = NewNode->WindowRect.W;
			NewViewport->Region.H = NewNode->WindowRect.H;

			NewViewport = AddViewportToClusterNode(NewViewport, NewNode, ViewportName);
		}
	}

	// Clean up the template object so that it gets GCed once this schema action has been destroyed
	NodeTemplate->Rename(nullptr, GetTransientPackage(), REN_DoNotDirty | REN_DontCreateRedirectors);
	NodeTemplate->SetFlags(RF_Transient);

	return NewNode;
}

UDisplayClusterConfigurationViewport* UE::DisplayClusterConfiguratorClusterEditorUtils::CreateNewViewportFromDialog(
	const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& Toolkit,
	UDisplayClusterConfigurationClusterNode* ClusterNode,
	const FDisplayClusterConfigurationRectangle& PresetRect,
	TSharedPtr<FScopedTransaction>& OutTransaction)
{
	UDisplayClusterConfigurationViewport* ViewportTemplate = NewObject<UDisplayClusterConfigurationViewport>(Toolkit->GetBlueprintObj(), NAME_None, RF_Transactional | RF_ArchetypeObject | RF_Public);
	ViewportTemplate->Region = FDisplayClusterConfigurationRectangle(PresetRect);

	UDisplayClusterConfigurationCluster* Cluster = Toolkit->GetEditorData()->Cluster;

	TArray<FString> ParentItems;
	for (const TPair<FString, TObjectPtr<UDisplayClusterConfigurationClusterNode>>& Node : Cluster->Nodes)
	{
		ParentItems.Add(Node.Key);
	}

	FString ClusterNodeName = "";
	if (const FString* Key = Cluster->Nodes.FindKey(ClusterNode))
	{
		ClusterNodeName = *Key;
	}

	const FString InitialName = ClusterNode ? 
		GetUniqueNameForViewport(DefaultNewViewportName, ClusterNode, true) : 
		FString::Printf(TEXT("%s_%d"), *DefaultNewViewportName, 0);

	bool bAutoposition = true;
	bool bExpandClusterNode = true;
	TSharedRef<SWidget> ViewportFooter = SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2.0f)
		[
			SNew(SCheckBox)
			.IsChecked_Lambda([&bAutoposition]() { return bAutoposition ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
			.OnCheckStateChanged_Lambda([&bAutoposition](ECheckBoxState NewState) { bAutoposition = NewState == ECheckBoxState::Checked; })
			.ToolTipText(LOCTEXT("AddNewViewport_AutoPositionToolTip", "When checked, auto-positions the new Viewport to prevent overlap with other Viewports in the same cluster Node"))
			[
				SNew(STextBlock)
				.Text(LOCTEXT("AddNewViewport_AutoPositionLabel", "Adjust Viewport Position to Prevent Overlap"))
			]
		]
	
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2.0f)
		[
			SNew(SCheckBox)
			.IsChecked_Lambda([&bExpandClusterNode]() { return bExpandClusterNode ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
			.OnCheckStateChanged_Lambda([&bExpandClusterNode](ECheckBoxState NewState) { bExpandClusterNode = NewState == ECheckBoxState::Checked; })
			.ToolTipText(LOCTEXT("AddNewViewport_ExpandClusterNodeToolTip", "When checked, expands the parent cluster Node to contain the new Viewport"))
			[
				SNew(STextBlock)
				.Text(LOCTEXT("AddNewViewport_ExpandClusterNodeLabel", "Expand Cluster Node to Fit New Viewport"))
			]
		];

	TSharedRef<SDisplayClusterConfiguratorNewClusterItemDialog> DialogContent = SNew(SDisplayClusterConfiguratorNewClusterItemDialog, ViewportTemplate)
		.ParentItemOptions(ParentItems)
		.InitiallySelectedParentItem(ClusterNodeName)
		.PresetItemOptions(FDisplayClusterConfiguratorPresetSize::CommonPresets)
		.InitiallySelectedPreset(FDisplayClusterConfiguratorPresetSize::CommonPresets[FDisplayClusterConfiguratorPresetSize::DefaultPreset])
		.InitialName(InitialName)
		.MaxWindowWidth(NewClusterItemDialogSize.X)
		.MaxWindowHeight(NewClusterItemDialogSize.Y)
		.FooterContent(ViewportFooter)
		.OnPresetChanged_Lambda([=](FVector2D Size) { ViewportTemplate->Region.W = Size.X; ViewportTemplate->Region.H = Size.Y; });

	{
		UE::DisplayClusterConfiguratorClusterEditorUtils::FScopedTemporaryTransactions ScopedTransactionsForModal;
		ShowNewClusterItemDialogWindow(DialogContent, nullptr, LOCTEXT("AddNewViewport_DialogTitle", "Add New Viewport"), NewClusterItemDialogSize);
	}
	
	UDisplayClusterConfigurationViewport* NewViewport = nullptr;

	if (DialogContent->WasAccepted())
	{
		OutTransaction = MakeShared<FScopedTransaction>(LOCTEXT("AddViewport", "Add Viewport"));
		
		const FString ParentItem = DialogContent->GetSelectedParentItem();
		const FString ItemName = DialogContent->GetItemName();

		check(Cluster->Nodes.Contains(ParentItem));

		UDisplayClusterConfigurationClusterNode* ParentNode = Cluster->Nodes[ParentItem];

		if (bAutoposition)
		{
			FVector2D DesiredPosition = FVector2D(ViewportTemplate->Region.X, ViewportTemplate->Region.Y);
			FVector2D DesiredSize = FVector2D(ViewportTemplate->Region.W, ViewportTemplate->Region.H);
			FVector2D NewPosition = FindNextAvailablePositionForViewport(ParentNode, DesiredPosition, DesiredSize);

			ViewportTemplate->Region.X = NewPosition.X;
			ViewportTemplate->Region.Y = NewPosition.Y;
		}

		NewViewport = DuplicateObject(ViewportTemplate, ParentNode);
		NewViewport->SetFlags(RF_Transactional);
		NewViewport = AddViewportToClusterNode(NewViewport, ParentNode, ItemName);

		if (bExpandClusterNode)
		{
			FIntRect ViewportRegion = NewViewport->Region.ToRect();
			FIntRect ClusterWindowRect = FIntRect(0, 0, ParentNode->WindowRect.W, ParentNode->WindowRect.H);

			ClusterWindowRect.Union(ViewportRegion);
			FIntPoint WindowSize = ClusterWindowRect.Max;
			FDisplayClusterConfigurationRectangle NewWindowRect(ParentNode->WindowRect.X, ParentNode->WindowRect.Y, WindowSize.X, WindowSize.Y);

			FDisplayClusterConfiguratorClusterNodeViewModel ClusterNodeVM(ParentNode);
			ClusterNodeVM.SetWindowRect(NewWindowRect);
		}
	}

	// Clean up the template object so that it gets GCed once this schema action has been destroyed
	ViewportTemplate->Rename(nullptr, GetTransientPackage(), REN_DoNotDirty | REN_DontCreateRedirectors);
	ViewportTemplate->SetFlags(RF_Transient);
	
	return NewViewport;
}

bool UE::DisplayClusterConfiguratorClusterEditorUtils::CanAddNewViewportToToolkit(const TWeakPtr<FDisplayClusterConfiguratorBlueprintEditor>& InToolkit)
{
	check(InToolkit.IsValid());
	const UDisplayClusterConfigurationData* EditorData = InToolkit.Pin()->GetEditorData();
	check(EditorData);
	return EditorData->Cluster && EditorData->Cluster->Nodes.Num() > 0;
}

void UE::DisplayClusterConfiguratorClusterEditorUtils::ShowNewClusterItemDialogWindow(TSharedRef<SDisplayClusterConfiguratorNewClusterItemDialog> DialogContent, TSharedPtr<SWidget> ParentElement, FText WindowTitle, FVector2D WindowSize)
{
	TSharedPtr<SWidget> ParentWindow = FSlateApplication::Get().FindBestParentWindowForDialogs(ParentElement);

	const FSlateRect WorkAreaRect = FSlateApplicationBase::Get().GetPreferredWorkArea();
	const FVector2D DisplayTopLeft(WorkAreaRect.Left, WorkAreaRect.Top);
	const FVector2D DisplaySize(WorkAreaRect.Right - WorkAreaRect.Left, WorkAreaRect.Bottom - WorkAreaRect.Top);

	const float ScaleFactor = FPlatformApplicationMisc::GetDPIScaleFactorAtPoint(DisplayTopLeft.X, DisplayTopLeft.Y);
	const FVector2D DialogWindowSize = WindowSize * ScaleFactor;

	const FVector2D WindowPosition = (DisplayTopLeft + 0.5f * (DisplaySize - DialogWindowSize)) / ScaleFactor;

	// UE-378282
	// There appears to be an issue with modal dialog behavior on Linux.
	// The dialog may appear in the background and not behave as expected.
	// As a workaround, use a simpler window initialization path that does
	// not explicitly request window position or size.
#if 1
	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(WindowTitle)
		.SizingRule(ESizingRule::Autosized);
#else
	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(WindowTitle)
		.SizingRule(ESizingRule::Autosized)
		.AutoCenter(EAutoCenter::None)
		.ClientSize(DialogWindowSize)
		.ScreenPosition(WindowPosition);
#endif

	DialogContent->SetParentWindow(Window);
	Window->SetContent(DialogContent);

	FSlateApplication::Get().AddModalWindow(Window, ParentWindow, false);
}

TSharedPtr<FDragDropOperation> UE::DisplayClusterConfiguratorClusterEditorUtils::MakeDragDropOperation(const TArray<UObject*>& SelectedObjects)
{
	TArray<UDisplayClusterConfigurationViewport*> Viewports;
	TArray<UDisplayClusterConfigurationClusterNode*> ClusterNodes;

	for (UObject* SelectedObject : SelectedObjects)
	{
		if (UDisplayClusterConfigurationViewport* Viewport = Cast<UDisplayClusterConfigurationViewport>(SelectedObject))
		{
			Viewports.Add(Viewport);
		}
		else if (UDisplayClusterConfigurationClusterNode* ClusterNode = Cast<UDisplayClusterConfigurationClusterNode>(SelectedObject))
		{
			ClusterNodes.Add(ClusterNode);
		}
		else
		{
			// Only cluster nodes and viewports can be dragged and dropped. If anything else is selected, abort the drag/drop operation
			return TSharedPtr<FDragDropOperation>();
		}
	}

	if (Viewports.Num() && !ClusterNodes.Num())
	{
		// If only viewports are selected, return a viewport drag/drop operation
		return FDisplayClusterConfiguratorViewportDragDropOp::New(Viewports);
	}
	else if (ClusterNodes.Num() && !Viewports.Num())
	{
		// If only cluster nodes are selected, return a cluster node drag/drop operation
		return FDisplayClusterConfiguratorClusterNodeDragDropOp::New(ClusterNodes);
	}
	else
	{
		// If both cluster nodes and viewports are selected, only allow a cluster node drag/drop operation if all selected viewports are children of the selected cluster nodes
		bool bOnlyChildViewports = true;
		for (UDisplayClusterConfigurationViewport* Viewport : Viewports)
		{
			bool bIsChild = ClusterNodes.ContainsByPredicate([=](UDisplayClusterConfigurationClusterNode* ClusterNode)
			{
				return ClusterNode == Viewport->GetOuter();
			});

			if (!bIsChild)
			{
				bOnlyChildViewports = false;
				break;
			}
		}

		if (bOnlyChildViewports)
		{
			return FDisplayClusterConfiguratorClusterNodeDragDropOp::New(ClusterNodes);
		}
		else
		{
			return TSharedPtr<FDragDropOperation>();
		}
	}
}

namespace
{
	FVector2D FindAvailableSpace(const TArray<FBox2D>& OtherBounds, const FBox2D& DesiredBounds)
	{
		FBox2D CurrentBounds(DesiredBounds);

		// Similar to FBox2D::Intersects, but ignores the case where the box edges are touching.
		auto IntrudesFunc = [](const FBox2D& BoxA, const FBox2D& BoxB)
		{

			// Special case if both boxes are directly on top of each other, which is considered an intrusion.
			if (BoxA == BoxB)
			{
				return true;
			}

			if ((BoxA.Min.X >= BoxB.Max.X) || (BoxB.Min.X >= BoxA.Max.X))
			{
				return false;
			}

			if ((BoxA.Min.Y >= BoxB.Max.Y) || (BoxB.Min.Y >= BoxA.Max.Y))
			{
				return false;
			}

			return true;
		};

		bool bIntersectsBounds;
		do
		{
			bIntersectsBounds = false;

			for (const FBox2D& Bounds : OtherBounds)
			{
				if (IntrudesFunc(CurrentBounds, Bounds))
				{
					bIntersectsBounds = true;

					float Shift = Bounds.GetSize().X;
					CurrentBounds.Min.X += Shift;
					CurrentBounds.Max.X += Shift;

					break;
				}
			}

		} while (bIntersectsBounds);

		return CurrentBounds.Min;
	}
}

FVector2D UE::DisplayClusterConfiguratorClusterEditorUtils::FindNextAvailablePositionForClusterNode(UDisplayClusterConfigurationCluster* Cluster, const FString& DesiredHost, const FVector2D& DesiredPosition, const FVector2D& DesiredSize)
{
	TMap<FString, TMap<FString, UDisplayClusterConfigurationClusterNode*>> ClusterNodesByHost;
	SortClusterNodesByHost(Cluster->Nodes, ClusterNodesByHost);

	// If the desired host doesn't exist yet, simply return the node's desired position
	if (!ClusterNodesByHost.Contains(DesiredHost))
	{
		return DesiredPosition;
	}

	const TMap<FString, UDisplayClusterConfigurationClusterNode*>& ExistingClusterNodes = ClusterNodesByHost[DesiredHost];
	if (!ExistingClusterNodes.Num())
	{
		return DesiredPosition;
	}

	FBox2D DesiredBounds = FBox2D(DesiredPosition, DesiredPosition + DesiredSize);
	TArray<FBox2D> ClusterNodeBounds;
	for (const TPair<FString, UDisplayClusterConfigurationClusterNode*>& ClusterNodeKeyPair : ExistingClusterNodes)
	{
		const FDisplayClusterConfigurationRectangle& WindowRect = ClusterNodeKeyPair.Value->WindowRect;
		ClusterNodeBounds.Add(FBox2D(FVector2D(WindowRect.X, WindowRect.Y), FVector2D(WindowRect.X + WindowRect.W, WindowRect.Y + WindowRect.H)));
	}

	return FindAvailableSpace(ClusterNodeBounds, DesiredBounds);
}

FVector2D UE::DisplayClusterConfiguratorClusterEditorUtils::FindNextAvailablePositionForViewport(UDisplayClusterConfigurationClusterNode* ClusterNode, const FVector2D& DesiredPosition, const FVector2D& DesiredSize)
{
	if (!ClusterNode)
	{
		return DesiredPosition;
	}

	FBox2D DesiredBounds = FBox2D(DesiredPosition, DesiredPosition + DesiredSize);
	TArray<FBox2D> ViewportBounds;
	for (const TPair<FString, TObjectPtr<UDisplayClusterConfigurationViewport>>& ViewportKeyPair : ClusterNode->Viewports)
	{
		const FDisplayClusterConfigurationRectangle& Region = ViewportKeyPair.Value->Region;
		ViewportBounds.Add(FBox2D(FVector2D(Region.X, Region.Y), FVector2D(Region.X + Region.W, Region.Y + Region.H)));
	}

	return FindAvailableSpace(ViewportBounds, DesiredBounds);
}

void UE::DisplayClusterConfiguratorClusterEditorUtils::CopyClusterItemsToClipboard(const TArray<UObject*>& ClusterItemsToCopy)
{
	TSet<UObject*> UniqueItemsToCopy;

	for (UObject* ClusterItem : ClusterItemsToCopy)
	{
		// Only allow viewports and cluster nodes to be copied at this point.
		if (ClusterItem->IsA<UDisplayClusterConfigurationViewport>())
		{
			// If a viewport is being copied, make sure its parent cluster node is not also being copied; if its parent is being copied,
			// the viewport doesn't need to be copied directly as it will be copied with its parent.
			if (!ClusterItemsToCopy.Contains(ClusterItem->GetOuter()))
			{
				UniqueItemsToCopy.Add(ClusterItem);
			}
		}
		else if (ClusterItem->IsA<UDisplayClusterConfigurationClusterNode>())
		{
			UniqueItemsToCopy.Add(ClusterItem);
		}
	}

	UnMarkAllObjects(EObjectMark(OBJECTMARK_TagExp | OBJECTMARK_TagImp));

	FStringOutputDevice Archive;
	const FExportObjectInnerContext Context;

	for (UObject* ClusterItem : UniqueItemsToCopy)
	{
		UExporter::ExportToOutputDevice(&Context, ClusterItem, nullptr, Archive, TEXT("copy"), 0, PPF_ExportsNotFullyQualified | PPF_Copy | PPF_Delimited, false, nullptr);
	}

	FString CopiedText = Archive;
	FPlatformApplicationMisc::ClipboardCopy(*CopiedText);
}

class FClusterNodeTextFactory : public FCustomizableTextObjectFactory
{
public:
	FClusterNodeTextFactory()
		: FCustomizableTextObjectFactory(GWarn)
	{
	}

	// FCustomizableTextObjectFactory implementation

	virtual bool CanCreateClass(UClass* ObjectClass, bool& bOmitSubObjs) const override
	{
		const bool isClusterNode = ObjectClass->IsChildOf(UDisplayClusterConfigurationClusterNode::StaticClass());
		const bool isViewportNode = ObjectClass->IsChildOf(UDisplayClusterConfigurationViewport::StaticClass());

		return isClusterNode || isViewportNode;
	}

	virtual void ProcessConstructedObject(UObject* NewObject) override
	{
		check(NewObject);

		if (UDisplayClusterConfigurationClusterNode* ClusterNode = Cast<UDisplayClusterConfigurationClusterNode>(NewObject))
		{
			ClusterNodes.Add(ClusterNode);
		}
		else if (UDisplayClusterConfigurationViewport* ViewportNode = Cast<UDisplayClusterConfigurationViewport>(NewObject))
		{
			Viewports.Add(ViewportNode);
		}
	}

	// FCustomizableTextObjectFactory (end)

public:
	TArray<UDisplayClusterConfigurationClusterNode*> ClusterNodes;
	TArray<UDisplayClusterConfigurationViewport*> Viewports;
};

bool UE::DisplayClusterConfiguratorClusterEditorUtils::CanPasteClusterItemsFromClipboard(const TArray<UObject*>& TargetClusterItems, int32& OutNumItems)
{
	// Can't paste unless the clipboard has a string in it
	FString ClipboardContent;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardContent);
	if (!ClipboardContent.IsEmpty())
	{
		FClusterNodeTextFactory Factory;
		Factory.ProcessBuffer(GetTransientPackage(), RF_Transactional, ClipboardContent);

		OutNumItems = Factory.ClusterNodes.Num() + Factory.Viewports.Num();

		bool bCanPaste = false;
		if (Factory.ClusterNodes.Num() > 0)
		{
			// If we are trying to paste cluster nodes, the target should be a cluster root, a host display data, or a cluster node.
			bCanPaste = TargetClusterItems.ContainsByPredicate([=](const UObject* Item)
			{
				return
					Item->IsA<UDisplayClusterConfigurationCluster>() ||
					Item->IsA<UDisplayClusterConfigurationClusterNode>() ||
					Item->IsA<UDisplayClusterConfigurationHostDisplayData>();
			});
		}

		if (Factory.Viewports.Num() > 0)
		{
			// If we are trying to paste viewports, the target should either be a cluster node or a viewport.
			bCanPaste = TargetClusterItems.ContainsByPredicate([=](const UObject* Item)
			{
				return Item->IsA<UDisplayClusterConfigurationClusterNode>() || Item->IsA<UDisplayClusterConfigurationViewport>();
			});
		}

		return bCanPaste;
	}

	return false;
}

TArray<UObject*> UE::DisplayClusterConfiguratorClusterEditorUtils::PasteClusterItemsFromClipboard(const TArray<UObject*>& TargetClusterItems, TOptional<FVector2D> PasteLocation)
{
	FString ClipboardContent;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardContent);

	FClusterNodeTextFactory Factory;
	Factory.ProcessBuffer(GetTransientPackage(), RF_Transactional, ClipboardContent);

	// Filter out the target items and find all unique cluster roots and cluster nodes. For each target cluster node, also add its parent root,
	// and for each target viewport, add its parent cluster node. This allows users to paste cluster nodes and viewports by selecting their sibling nodes,
	// which will then add the pasted node to the parent of the target node.

	// Pair up the cluster roots with a host string, which indicates which host the cluster nodes should be pasted into (the cluster nodes' Host strings will be set when duplicated).
	// If the paired host string is empty, the pasted cluster nodes' original host values will be preserved.
	TSet<TTuple<UDisplayClusterConfigurationCluster*, FString>> SelectedRootHosts;
	TSet<UDisplayClusterConfigurationClusterNode*> SelectedNodes;

	for (UObject* TargetItem : TargetClusterItems)
	{
		if (UDisplayClusterConfigurationCluster* ClusterRoot = Cast<UDisplayClusterConfigurationCluster>(TargetItem))
		{
			SelectedRootHosts.Add(TTuple<UDisplayClusterConfigurationCluster*, FString>(ClusterRoot, ""));
		}
		else if (UDisplayClusterConfigurationHostDisplayData* Host = Cast<UDisplayClusterConfigurationHostDisplayData>(TargetItem))
		{
			// If the target is a host display data, that indicates the user wants to paste the cluster node into the host the display data represents,
			// so add an entry to the cluster root list with the parent cluster object and the host string of the host display data.
			if (UDisplayClusterConfigurationCluster* ParentRoot = Cast<UDisplayClusterConfigurationCluster>(TargetItem->GetOuter()))
			{
				FString HostStr = FString();
				if (const FString* Key = ParentRoot->HostDisplayData.FindKey(Host))
				{
					HostStr = *Key;
				}

				SelectedRootHosts.Add(TTuple<UDisplayClusterConfigurationCluster*, FString>(ParentRoot, HostStr));
			}
		}
		else if (UDisplayClusterConfigurationClusterNode* ClusterNode = Cast<UDisplayClusterConfigurationClusterNode>(TargetItem))
		{
			SelectedNodes.Add(ClusterNode);

			if (UDisplayClusterConfigurationCluster* ParentRoot = Cast<UDisplayClusterConfigurationCluster>(TargetItem->GetOuter()))
			{
				// If pasting as a sibling of a target cluster node, set the pasted node's host to be the same as the sibling node's host.
				SelectedRootHosts.Add(TTuple<UDisplayClusterConfigurationCluster*, FString>(ParentRoot, ClusterNode->Host));
			}
		}
		else if (UDisplayClusterConfigurationViewport* Viewport = Cast<UDisplayClusterConfigurationViewport>(TargetItem))
		{
			if (UDisplayClusterConfigurationClusterNode* ParentNode = Cast<UDisplayClusterConfigurationClusterNode>(TargetItem->GetOuter()))
			{
				SelectedNodes.Add(ParentNode);
			}
		}
	}

	TArray<UObject*> CopiedObjects;

	// Keep track of a couple reference positions for pasting the items to a specfic location to correctly offset pasted location in the case where
	// more than one item is being pasted or an item is being pasted into more than one parent. The ReferencePosition variable keeps track of the 
	// relative position of the item to be pasted, while the ReferenceTargetPosition keeps track of the relative position of the target item being pasted
	// into. The way these work is that, if they aren't set when in the pasting loop below, then they are set to the current iteration item's position. Every
	// other iteration will then subtract the reference position from the pasted position. This allows each item being pasted to be correctly offset to maintain
	// both its relative position in the group of items being pasted and its relative position to the item it is being pasted into.
	TOptional<FVector2D> ReferencePosition = TOptional<FVector2D>();
	TOptional<FVector2D> ReferenceTargetPosition = TOptional<FVector2D>();

	for (const TTuple<UDisplayClusterConfigurationCluster*, FString>& ClusterRootHost : SelectedRootHosts)
	{
		UDisplayClusterConfigurationCluster* ClusterRoot = ClusterRootHost.Key;
		FString HostStr = ClusterRootHost.Value;

		for (UDisplayClusterConfigurationClusterNode* ClusterNode : Factory.ClusterNodes)
		{
			UDisplayClusterConfigurationClusterNode* ClusterNodeCopy = DuplicateObject(ClusterNode, ClusterRoot);
			ClusterNodeCopy->SetFlags(RF_Transactional);
			ClusterNodeCopy = AddClusterNodeToCluster(ClusterNodeCopy, ClusterRoot, ClusterNode->GetName());

			// If the host string isn't empty, it means the user is pasting the cluster node into a particular host, so set the cluster node's host string.
			if (!HostStr.IsEmpty())
			{
				ClusterNodeCopy->Host = HostStr;
			}

			if (PasteLocation.IsSet())
			{
				FVector2D NewLocation = *PasteLocation;

				// If we are pasting into an existing host, offset the reference target position so that the position of the node inside the host
				// is consistent for every host we paste into.
				if (UDisplayClusterConfigurationHostDisplayData* HostDisplayData = GetHostDisplayDataForClusterNode(ClusterNodeCopy))
				{
					// If the reference target position isn't set, set it to the current host's position.
					if (!ReferenceTargetPosition.IsSet())
					{
						ReferenceTargetPosition = HostDisplayData->Position;
					}

					// Offset the location to keep this cluster node's position relative to its parent host consistent.
					NewLocation -= *ReferenceTargetPosition;
				}

				// If the reference position isn't set, set it to the current pasted cluster node's position.
				if (!ReferencePosition.IsSet())
				{
					ReferencePosition = FVector2D(ClusterNode->WindowRect.X, ClusterNode->WindowRect.Y);
				}

				// Offset the location to keep this cluster node's relative position with the group of pasted nodes consistent.
				NewLocation -= FVector2D(ClusterNode->WindowRect.X, ClusterNode->WindowRect.Y) - *ReferencePosition;

				ClusterNodeCopy->WindowRect.X = NewLocation.X;
				ClusterNodeCopy->WindowRect.Y = NewLocation.Y;
			}

			// Rename viewports in the new cluster node to ensure they're unique. Copy the viewport pointers into an array
			// first since renaming would cause the dictionary to change while we're iterating it.
			TArray<typename decltype(ClusterNodeCopy->Viewports)::ValueType> CopiedViewports;
			ClusterNodeCopy->Viewports.GenerateValueArray(CopiedViewports);
			
			for (UDisplayClusterConfigurationViewport* Viewport : CopiedViewports)
			{
				RenameViewport(Viewport, *GetUniqueNameForViewport(*Viewport->GetName(), ClusterNodeCopy));
			}

			CopiedObjects.Add(ClusterNodeCopy);
		}
	}

	// Reset the reference positions for the viewports pasting loop
	ReferencePosition = TOptional<FVector2D>();
	ReferenceTargetPosition = TOptional<FVector2D>();

	for (UDisplayClusterConfigurationClusterNode* ClusterNode : SelectedNodes)
	{
		for (UDisplayClusterConfigurationViewport* Viewport : Factory.Viewports)
		{
			UDisplayClusterConfigurationViewport* ViewportCopy = DuplicateObject(Viewport, ClusterNode);
			ViewportCopy->SetFlags(RF_Transactional);
			ViewportCopy = AddViewportToClusterNode(ViewportCopy, ClusterNode, Viewport->GetName());

			if (PasteLocation.IsSet())
			{
				// If the reference target position isn't set, set it to the current cluster node's position.
				if (!ReferenceTargetPosition.IsSet())
				{
					FVector2D ClusterNodeLocation = FVector2D(ClusterNode->WindowRect.X, ClusterNode->WindowRect.Y);

					// Cluster node window rectangles are local to the node's host, so factor in its host position, if it has one.
					if (UDisplayClusterConfigurationHostDisplayData* HostDisplayData = GetHostDisplayDataForClusterNode(ClusterNode))
					{
						ClusterNodeLocation += HostDisplayData->Position;
					}

					ReferenceTargetPosition = ClusterNodeLocation;
				}

				// Offset the location to keep this viewport's position relative to its parent cluster node consistent.
				FVector2D NewLocation = *PasteLocation - *ReferenceTargetPosition;

				// If the reference position isn't set, set it to the current pasted viewport's position.
				if (!ReferencePosition.IsSet())
				{
					ReferencePosition = FVector2D(Viewport->Region.X, Viewport->Region.Y);
				}

				// Offset the location to keep this viewport's relative position with the group of pasted nodes consistent.
				NewLocation -= FVector2D(Viewport->Region.X, Viewport->Region.Y) - *ReferencePosition;

				// Viewports cannot have negative position
				ViewportCopy->Region.X = FMath::Max(NewLocation.X, 0.0f);
				ViewportCopy->Region.Y = FMath::Max(NewLocation.Y, 0.0f);
			}

			CopiedObjects.Add(ViewportCopy);
		}
	}

	return CopiedObjects;
}

#undef LOCTEXT_NAMESPACE