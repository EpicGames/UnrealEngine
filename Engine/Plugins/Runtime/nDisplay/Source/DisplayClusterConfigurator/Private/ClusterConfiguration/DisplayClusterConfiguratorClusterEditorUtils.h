// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectPtr.h"

class SWidget;
class FDragDropOperation;
class FDisplayClusterConfiguratorBlueprintEditor;
class FScopedTransaction;
class UDisplayClusterConfigurationData_Base;
class UDisplayClusterConfigurationCluster;
class UDisplayClusterConfigurationHostDisplayData;
class UDisplayClusterConfigurationClusterNode;
class UDisplayClusterConfigurationViewport;
class SDisplayClusterConfiguratorNewClusterItemDialog;
struct FDisplayClusterConfigurationRectangle;

#define NDISPLAY_DEFAULT_CLUSTER_HOST "127.0.0.1"

namespace UE::DisplayClusterConfiguratorClusterEditorUtils
{
	/**
	 * Creates a new cluster node, presenting a dialog box to allow the user to customize its properties, and adds it to the cluster.
	 * @param Toolkit - The blueprint editor toolkit being used.
	 * @param Cluster - The initial cluster the user wants to add the cluster node to.
	 * @param PresetRect - The initial rectangle to configure the new cluster node with in the dialog box.
	 * @param OutTransaction - The transaction created when adding a cluster node. May be null if the user canceled the operation.
	 * @param PresetHost - Optional initial host string to configure the new cluster node with in the dialog box.
	 * @return The newly created cluster node, or null if the user cancelled out of the dialog box.
	 */
	UDisplayClusterConfigurationClusterNode* CreateNewClusterNodeFromDialog(const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& Toolkit,
		UDisplayClusterConfigurationCluster* Cluster, const FDisplayClusterConfigurationRectangle& PresetRect,
		TSharedPtr<FScopedTransaction>& OutTransaction,
		FString PresetHost = NDISPLAY_DEFAULT_CLUSTER_HOST);

	/**
	 * Creates a new viewport, presenting a dialog box to allow the user to customize its properties, and adds it to the user chosen cluster node.
	 * @param Toolkit - The blueprint editor toolkit being used.
	 * @param ClusterNode - The initial cluster node the user wants to add the viewport to, or null to use the first cluster node in the cluster.
	 * @param PresetRect - The initial rectangle to configure the new viewport with in the dialog box.
	 * @param OutTransaction - The transaction created when adding a viewport. May be null if the user canceled the operation.
	 * @return The newly created viewport, or null if the user cancelled out of the dialog box.
	 */
	UDisplayClusterConfigurationViewport* CreateNewViewportFromDialog(const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& Toolkit,
		UDisplayClusterConfigurationClusterNode* ClusterNode, const FDisplayClusterConfigurationRectangle& PresetRect,
		TSharedPtr<FScopedTransaction>& OutTransaction);

	/**
	 * Checks if a new viewport is allowed to be added by making sure there is a cluster node available.
	 * @param InToolkit The toolkit hosting the viewport.
	 */
	bool CanAddNewViewportToToolkit(const TWeakPtr<FDisplayClusterConfiguratorBlueprintEditor>& InToolkit);
	
	/**
	 * Displays a dialog box used to configure new cluster item properties before adding the new item to the cluster configuration.
	 * @param DialogContent - The widget to display in the dialog box's main section.
	 * @param ParentElement - The UI element to display the dialog box from. If null, will use the active window as a parent.
	 * @param WindowTitle - The title displayed at the top of the dialog box.
	 * @param WindowSize - The size of the dialog box.
	 */
	void ShowNewClusterItemDialogWindow(TSharedRef<SDisplayClusterConfiguratorNewClusterItemDialog> DialogContent, TSharedPtr<SWidget> ParentElement, FText WindowTitle, FVector2D WindowSize);

	/**
	 * Attempts to create an appropriate drag and drop operation for the selected objects.
	 * @param SelectedObjects - The objects to create the drag and drop operation for.
	 * @return A valid pointer to the drag and drop operation if an operation could be made, and invalid pointer if not.
	 */
	TSharedPtr<FDragDropOperation> MakeDragDropOperation(const TArray<UObject*>& SelectedObjects);

	/**
	 * Finds the next available position for a cluster node within a host so that the cluster node doesn't overlap any existing nodes.
	 * @param Cluster - The cluster to find the position within.
	 * @param DesiredHost - The host the cluster node wants to be positioned in.
	 * @param DesiredPosition - The desired position of the cluster node.
	 * @param DesiredSize - The desired size of the cluster node.
	 */
	FVector2D FindNextAvailablePositionForClusterNode(UDisplayClusterConfigurationCluster* Cluster, const FString& DesiredHost, const FVector2D& DesiredPosition, const FVector2D& DesiredSize);

	/**
	 * Finds the next available position for a viewport within a cluster node so that the viewport doesn't overlap any existing viewports.
	 * @param ClusterNode - The cluster node to find the position within.
	 * @param DesiredPosition - The desired position of the viewport.
	 * @param DesiredSize - The desired size of the viewport.
	 */
	FVector2D FindNextAvailablePositionForViewport(UDisplayClusterConfigurationClusterNode* ClusterNode, const FVector2D& DesiredPosition, const FVector2D& DesiredSize);

	/**
	 * Copies the specified list of cluster items to the clipboard.
	 * @param ClusterItemsToCopy - The list of cluster items to copy.
	 */
	void CopyClusterItemsToClipboard(const TArray<UObject*>& ClusterItemsToCopy);

	/**
	 * Checks to see if the clipboard contains cluster items that can be pasted into the target cluster items.
	 * @param TargetClusterItems - The cluster items to paste the copied cluster items into.
	 * @param OutNumItems - The number of cluster items that can be pasted from the clipboard.
	 */
	bool CanPasteClusterItemsFromClipboard(const TArray<UObject*>& TargetClusterItems, int32& OutNumItems);

	/**
	 * Pastes any cluster items in the clipboard into the specified target items, if the target items can accept the pasted items.
	 * @param TargetClusterItems - The cluster items to paste the copied cluster items into, if compatible.
	 * @param PasteLocation - Optional location to paste the cluster items at.
	 * @return The pasted cluster items.
	 */
	TArray<UObject*> PasteClusterItemsFromClipboard(const TArray<UObject*>& TargetClusterItems, TOptional<FVector2D> PasteLocation = TOptional<FVector2D>());
};