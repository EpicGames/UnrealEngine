// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClusterConfiguration/DisplayClusterConfiguratorClusterUtils.h"

#include "DisplayClusterConfiguratorBlueprintEditor.h"
#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterConfiguratorPropertyUtils.h"
#include "DisplayClusterConfiguratorStyle.h"
#include "DisplayClusterConfiguratorUtils.h"

#include "Factories.h"
#include "ObjectTools.h"
#include "Editor/Transactor.h"

#define LOCTEXT_NAMESPACE "FDisplayClusterConfiguratorClusterUtils"

namespace UE::DisplayClusterConfiguratorClusterUtils
{
	/**
	 * Gets a unique name for something given a list of existing names.
	 * @param InitialName - The initial name the object wants
	 * @param UsedNames - The list of names that have already been used
	 * @param Class - The class to generate a name for
	 * @param Parent - The parent to check for in-memory objects that collide with the name
	 * @param bAddZero - Whether to add an "_0" to the initial name if it is unique
	 * @returns A unique name for the object
	 */
	FString GetUniqueName(const FString& InitialName, const TArray<FString>& UsedNames, const UClass* Class, UObject* Parent, bool bAddZero = false)
	{
		FString NewName;
		if (!bAddZero && !UsedNames.Contains(InitialName))
		{
			// Name doesn't need to be modified
			NewName = InitialName;
		}
		else
		{
			int32 Counter = bAddZero ? 0 : 1;

			// Find the start of the existing numeric suffix
			int32 Index = InitialName.Len();
			while (Index > 0 && InitialName[Index-1] >= '0' && InitialName[Index-1] <= '9')
			{
				--Index;
			}

			FString BaseName = InitialName;
			if (Index < BaseName.Len())
			{
				// Strip away the suffix and store the value in the counter so we can count up from there
				FString NumericSuffix = BaseName.RightChop(Index);
				Counter = FCString::Atoi(*NumericSuffix);
				NumericSuffix = FString::FromInt(Counter); // Restringify the counter to account for leading 0s that we don't want to remove
				BaseName.RemoveAt(BaseName.Len() - NumericSuffix.Len(), NumericSuffix.Len(), EAllowShrinking::No);
			}
			else
			{
				// No existing suffix, so add our underscore separator
				BaseName += "_";
			}

			do
			{
				NewName = FString::Printf(TEXT("%s%d"), *BaseName, Counter);
				++Counter;
			}
			while (UsedNames.Contains(NewName));
		}
	
		// If there is already an in-memory object connected to the parent cluster with our generated name, we need to use a globally unique object name.
		if (StaticFindObject(nullptr, Parent, *NewName, EFindObjectFlags::ExactClass))
		{
			NewName = MakeUniqueObjectName(Parent, Class, *NewName).ToString();
		}

		return NewName;
	}
	
	const FString DefaultNewHostName = TEXT("Host");
}

using namespace UE::DisplayClusterConfiguratorPropertyUtils;

UDisplayClusterConfigurationHostDisplayData* UE::DisplayClusterConfiguratorClusterUtils::FindOrCreateHostDisplayData(UDisplayClusterConfigurationCluster* Cluster, FString HostIPAddress)
{
	// In some cases, existing host display data may be pending kill, such as if the user recently performed an undo to a state
	// prior to the data's existence. In this case, simply remove existing host data that is pending kill and create a new one to use.
	TArray<FString> PendingKillHostData;
	for (TPair<FString, TObjectPtr<UDisplayClusterConfigurationHostDisplayData>>& HostPair : Cluster->HostDisplayData)
	{
		if (!IsValid(HostPair.Value))
		{
			PendingKillHostData.Add(HostPair.Key);
		}
	}

	for (const FString& Host : PendingKillHostData)
	{
		Cluster->HostDisplayData.Remove(Host);
	}

	if (!Cluster->HostDisplayData.Contains(HostIPAddress))
	{
		const FString HostName = GetUniqueNameForHost(DefaultNewHostName, Cluster, true);
		UDisplayClusterConfigurationHostDisplayData* NewData = NewObject<UDisplayClusterConfigurationHostDisplayData>(Cluster, NAME_None, RF_Transactional);
		NewData->HostName = FText::FromString(HostName);
		NewData->Color = FDisplayClusterConfiguratorStyle::Get().GetDefaultColor(Cluster->HostDisplayData.Num()).CopyWithNewOpacity(1.0f);

		Cluster->HostDisplayData.Add(HostIPAddress, NewData);
	}

	return Cluster->HostDisplayData[HostIPAddress];
}

bool UE::DisplayClusterConfiguratorClusterUtils::RemoveUnusedHostDisplayData(UDisplayClusterConfigurationCluster* Cluster)
{
	bool bHostDataRemoved = false;

	TArray<FString> UnusedHosts;
	Cluster->HostDisplayData.GetKeys(UnusedHosts);

	for (const TPair<FString, TObjectPtr<UDisplayClusterConfigurationClusterNode>>& NodePair : Cluster->Nodes)
	{
		UnusedHosts.Remove(NodePair.Value->Host);
	}

	if (UnusedHosts.Num() > 0)
	{
		Cluster->Modify();

		for (const FString& UnusedHost : UnusedHosts)
		{
			Cluster->HostDisplayData[UnusedHost]->MarkAsGarbage();
			Cluster->HostDisplayData.Remove(UnusedHost);
		}

		bHostDataRemoved = true;
	}

	return bHostDataRemoved;
}

FString UE::DisplayClusterConfiguratorClusterUtils::GetUniqueNameForHost(FString InitialName, UDisplayClusterConfigurationCluster* ParentCluster, bool bAddZero)
{
	TArray<FString> UsedNames;
	for (TPair<FString, TObjectPtr<UDisplayClusterConfigurationHostDisplayData>>& HostPair : ParentCluster->HostDisplayData)
	{
		UsedNames.Add(HostPair.Value->HostName.ToString());
	}

	return GetUniqueName(InitialName, UsedNames, UDisplayClusterConfigurationClusterNode::StaticClass(), ParentCluster, bAddZero);
}

FString UE::DisplayClusterConfiguratorClusterUtils::GetAddressForHost(UDisplayClusterConfigurationHostDisplayData* HostDisplayData)
{
	FString HostAddress;
	if (UDisplayClusterConfigurationCluster* HostParent = Cast<UDisplayClusterConfigurationCluster>(HostDisplayData->GetOuter()))
	{
		if (const FString* HostKey = HostParent->HostDisplayData.FindKey(HostDisplayData))
		{
			HostAddress = *HostKey;
		}
	}

	return HostAddress;
}

bool UE::DisplayClusterConfiguratorClusterUtils::RemoveHost(UDisplayClusterConfigurationCluster* Cluster, FString Host)
{
	bool bDataRemoved = false;

	if (Cluster->HostDisplayData.Contains(Host))
	{
		Cluster->Modify();
		Cluster->HostDisplayData[Host]->Modify();

		Cluster->HostDisplayData.Remove(Host);

		bDataRemoved = true;
	}

	TArray<FString> ClusterNodesToRemove;
	for (const TPair<FString, TObjectPtr<UDisplayClusterConfigurationClusterNode>>& Pair : Cluster->Nodes)
	{
		if (Pair.Value->Host == Host)
		{
			ClusterNodesToRemove.Add(Pair.Key);
		}
	}

	for (FString ClusterNodeToRemove : ClusterNodesToRemove)
	{
		RemoveClusterNodeFromCluster(Cluster->Nodes[ClusterNodeToRemove]);
	}

	FDisplayClusterConfiguratorUtils::MarkDisplayClusterBlueprintAsModified(Cluster, true);
	
	return bDataRemoved;
}

UDisplayClusterConfigurationHostDisplayData* UE::DisplayClusterConfiguratorClusterUtils::GetHostDisplayDataForClusterNode(UDisplayClusterConfigurationClusterNode* ClusterNode)
{
	UDisplayClusterConfigurationHostDisplayData* HostDisplayData = nullptr;
	if (UDisplayClusterConfigurationCluster* ClusterNodeParent = Cast<UDisplayClusterConfigurationCluster>(ClusterNode->GetOuter()))
	{
		if (ClusterNodeParent->HostDisplayData.Contains(ClusterNode->Host))
		{
			HostDisplayData = ClusterNodeParent->HostDisplayData[ClusterNode->Host];
		}
	}

	return HostDisplayData;
}

FString UE::DisplayClusterConfiguratorClusterUtils::GetClusterNodeName(UDisplayClusterConfigurationClusterNode* ClusterNode)
{
	if (UDisplayClusterConfigurationCluster* ClusterNodeParent = Cast<UDisplayClusterConfigurationCluster>(ClusterNode->GetOuter()))
	{
		if (const FString* KeyPtr = ClusterNodeParent->Nodes.FindKey(ClusterNode))
		{
			return *KeyPtr;
		}
	}

	return "";
}

bool UE::DisplayClusterConfiguratorClusterUtils::IsClusterNodePrimary(UDisplayClusterConfigurationClusterNode* ClusterNode)
{
	if (const UDisplayClusterConfigurationCluster* Cluster = Cast<UDisplayClusterConfigurationCluster>(ClusterNode->GetOuter()))
	{
		const FString PrimaryNodeId = Cluster->PrimaryNode.Id;
		if (Cluster->Nodes.Contains(PrimaryNodeId) && Cluster->Nodes[PrimaryNodeId] == ClusterNode)
		{
			return true;
		}
	}

	return false;
}

FString UE::DisplayClusterConfiguratorClusterUtils::GetUniqueNameForClusterNode(FString InitialName, UDisplayClusterConfigurationCluster* ParentCluster, bool bAddZero)
{
	InitialName = ObjectTools::SanitizeObjectName(InitialName);
	
	TArray<FString> UsedNames;
	ParentCluster->Nodes.GenerateKeyArray(UsedNames);

	return GetUniqueName(InitialName, UsedNames, UDisplayClusterConfigurationClusterNode::StaticClass(), ParentCluster, bAddZero);
}

UDisplayClusterConfigurationClusterNode* UE::DisplayClusterConfiguratorClusterUtils::AddClusterNodeToCluster(UDisplayClusterConfigurationClusterNode* ClusterNode, UDisplayClusterConfigurationCluster* Cluster, FString NewClusterNodeName)
{
	FString ClusterNodeName = "";

	// First, remove the viewport from its current parent cluster node, if it has one.
	if (UDisplayClusterConfigurationCluster* ClusterNodeParent = Cast<UDisplayClusterConfigurationCluster>(ClusterNode->GetOuter()))
	{
		if (const FString* OldKeyPtr = ClusterNodeParent->Nodes.FindKey(ClusterNode))
		{
			ClusterNodeName = *OldKeyPtr;

			ClusterNodeParent->Modify();

			const FName FieldName = GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationCluster, Nodes);
			RemoveKeyFromMap(ClusterNodeParent, FieldName, *OldKeyPtr);
		}
	}

	if (!NewClusterNodeName.IsEmpty())
	{
		ClusterNodeName = NewClusterNodeName;
	}

	if (ClusterNodeName.IsEmpty())
	{
		ClusterNodeName = TEXT("ClusterNode");
	}

	ClusterNodeName = GetUniqueNameForClusterNode(ClusterNodeName, Cluster);

	Cluster->Modify();
	ClusterNode->Modify();
	ClusterNode->Rename(*ClusterNodeName, Cluster, REN_DontCreateRedirectors);

	const FName FieldName = GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationCluster, Nodes);
	UDisplayClusterConfigurationClusterNode* NewClusterNode = CastChecked<UDisplayClusterConfigurationClusterNode>(AddKeyWithInstancedValueToMap(Cluster, FieldName, ClusterNodeName, ClusterNode));

	check(ClusterNode != NewClusterNode);
	ClusterNode->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors | REN_DoNotDirty);
	ClusterNode->SetFlags(RF_Transient);
	
	return NewClusterNode;
}

bool UE::DisplayClusterConfiguratorClusterUtils::RemoveClusterNodeFromCluster(UDisplayClusterConfigurationClusterNode* ClusterNode)
{
	if (UDisplayClusterConfigurationCluster* ClusterNodeParent = Cast<UDisplayClusterConfigurationCluster>(ClusterNode->GetOuter()))
	{
		if (const FString* KeyPtr = ClusterNodeParent->Nodes.FindKey(ClusterNode))
		{
			ClusterNode->Modify();

			ClusterNodeParent->Modify();

			const bool bRemovingPrimaryNode = IsClusterNodePrimary(ClusterNode);
			if (bRemovingPrimaryNode)
			{
				ClusterNodeParent->PrimaryNode.Id.Empty();
			}
			
			RemoveKeyFromMap(ClusterNodeParent, GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationCluster, Nodes), *KeyPtr);
			ensure (ClusterNode->GetPackage() == GetTransientPackage());
			
			ClusterNode->SetFlags(RF_Transient);

			if (bRemovingPrimaryNode)
			{
				if (const TTuple<FString, TObjectPtr<UDisplayClusterConfigurationClusterNode>>* NodesKeyVal = ClusterNodeParent->Nodes.FindArbitraryElement())
				{
					SetClusterNodeAsPrimary(NodesKeyVal->Value);
				}
			}
			
			FDisplayClusterConfiguratorUtils::MarkDisplayClusterBlueprintAsModified(ClusterNodeParent, true);
			
			return true;
		}
	}

	return false;
}

bool UE::DisplayClusterConfiguratorClusterUtils::RenameClusterNode(UDisplayClusterConfigurationClusterNode* ClusterNode, FString NewClusterNodeName)
{
	if (UDisplayClusterConfigurationCluster* ClusterNodeParent = Cast<UDisplayClusterConfigurationCluster>(ClusterNode->GetOuter()))
	{
		if (const FString* KeyPtr = ClusterNodeParent->Nodes.FindKey(ClusterNode))
		{
			if (KeyPtr->Equals(NewClusterNodeName))
			{
				return false;
			}

			const bool bIsPrimary = IsClusterNodePrimary(ClusterNode);

			ClusterNode->Modify();
			ClusterNodeParent->Modify();

			const FString UniqueName = GetUniqueNameForClusterNode(NewClusterNodeName, ClusterNodeParent);

			const FName FieldName = GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationCluster, Nodes);
			RemoveKeyFromMap(ClusterNodeParent, FieldName, *KeyPtr);

			// Rename after remove, before add.
			ClusterNode->Rename(*UniqueName, ClusterNodeParent, REN_DontCreateRedirectors);
			
			UDisplayClusterConfigurationClusterNode* NewClusterNode = CastChecked<UDisplayClusterConfigurationClusterNode>(AddKeyWithInstancedValueToMap(ClusterNodeParent, FieldName, UniqueName, ClusterNode));

			check(ClusterNode != NewClusterNode);
			ClusterNode->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors | REN_DoNotDirty);
			ClusterNode->SetFlags(RF_Transient);
			
			// If the cluster node was a primary node before the rename, we need to update the primary reference in the cluster with the new name
			if (bIsPrimary)
			{
				SetClusterNodeAsPrimary(NewClusterNode);
			}

			return true;
		}
	}

	return false;
}

bool UE::DisplayClusterConfiguratorClusterUtils::SetClusterNodeAsPrimary(UDisplayClusterConfigurationClusterNode* ClusterNode)
{
	if (UDisplayClusterConfigurationCluster* ClusterNodeParent = Cast<UDisplayClusterConfigurationCluster>(ClusterNode->GetOuter()))
	{
		const FString* NodeIdPtr = ClusterNodeParent->Nodes.FindKey(ClusterNode);

		if (NodeIdPtr != nullptr)
		{
			ClusterNodeParent->Modify();
			ClusterNodeParent->PrimaryNode.Id = *NodeIdPtr;
			return true;
		}
	}

	return false;
}

FString UE::DisplayClusterConfiguratorClusterUtils::GetViewportName(UDisplayClusterConfigurationViewport* Viewport)
{
	if (UDisplayClusterConfigurationClusterNode* ViewportParent = Cast<UDisplayClusterConfigurationClusterNode>(Viewport->GetOuter()))
	{
		if (const FString* KeyPtr = ViewportParent->Viewports.FindKey(Viewport))
		{
			return *KeyPtr;
		}
	}

	return "";
}

FString UE::DisplayClusterConfiguratorClusterUtils::GetUniqueNameForViewport(FString InitialName, UDisplayClusterConfigurationClusterNode* ParentClusterNode, bool bAddZero)
{
	InitialName = ObjectTools::SanitizeObjectName(InitialName);
	
	// Viewport names must be unique across the entire cluster, not just within its parent cluster nodes. Gather all of the viewport names
	// in the cluster to check for uniqueness. Add the parent cluster node's viewports first, in case we can't get to the root cluster through
	// the cluster node's Outer (i.e. the cluster node has not been added to the cluster yet)
	TSet<FString> UsedNames;
	for (const TPair<FString, TObjectPtr<UDisplayClusterConfigurationViewport>>& ViewportKeyPair : ParentClusterNode->Viewports)
	{
		UsedNames.Add(ViewportKeyPair.Key);
	}

	if (UDisplayClusterConfigurationCluster* Cluster = Cast<UDisplayClusterConfigurationCluster>(ParentClusterNode->GetOuter()))
	{
		for (const TPair<FString, TObjectPtr<UDisplayClusterConfigurationClusterNode>>& ClusterNodeKeyPair : Cluster->Nodes)
		{
			UDisplayClusterConfigurationClusterNode* ClusterNode = ClusterNodeKeyPair.Value;
			if (ClusterNode != ParentClusterNode)
			{
				for (const TPair<FString, TObjectPtr<UDisplayClusterConfigurationViewport>>& ViewportKeyPair : ClusterNode->Viewports)
				{
					UsedNames.Add(ViewportKeyPair.Key);
				}
			}
		}
	}

	TArray<FString> UsedNamesArray = UsedNames.Array();
	
	return GetUniqueName(InitialName, UsedNamesArray, UDisplayClusterConfigurationViewport::StaticClass(), ParentClusterNode, bAddZero);
}

UDisplayClusterConfigurationViewport* UE::DisplayClusterConfiguratorClusterUtils::AddViewportToClusterNode(UDisplayClusterConfigurationViewport* Viewport, UDisplayClusterConfigurationClusterNode* ClusterNode, FString NewViewportName)
{
	FString ViewportName = "";

	// First, remove the viewport from its current parent cluster node, if it has one.
	if (UDisplayClusterConfigurationClusterNode* ViewportParent = Cast<UDisplayClusterConfigurationClusterNode>(Viewport->GetOuter()))
	{
		if (const FString* OldKeyPtr = ViewportParent->Viewports.FindKey(Viewport))
		{
			ViewportName = *OldKeyPtr;

			ViewportParent->Modify();
			
			RemoveKeyFromMap(ViewportParent, GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationClusterNode, Viewports), *OldKeyPtr);
		}
	}

	if (!NewViewportName.IsEmpty())
	{
		ViewportName = NewViewportName;
	}

	if (ViewportName.IsEmpty())
	{
		ViewportName = TEXT("Viewport");
	}

	ViewportName = GetUniqueNameForViewport(ViewportName, ClusterNode);

	ClusterNode->Modify();
	Viewport->Modify();
	Viewport->Rename(*ViewportName, ClusterNode, REN_DontCreateRedirectors | REN_DoNotDirty | REN_AllowPackageLinkerMismatch);

	const FName FieldName = GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationClusterNode, Viewports);
	UDisplayClusterConfigurationViewport* NewViewport = CastChecked<UDisplayClusterConfigurationViewport>(AddKeyWithInstancedValueToMap(ClusterNode, FieldName, ViewportName, Viewport));

	check(Viewport != NewViewport);
	Viewport->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors | REN_DoNotDirty);
	Viewport->SetFlags(RF_Transient);
	
	FDisplayClusterConfiguratorUtils::MarkDisplayClusterBlueprintAsModified(NewViewport, true);

	return NewViewport;
}

bool UE::DisplayClusterConfiguratorClusterUtils::RemoveViewportFromClusterNode(UDisplayClusterConfigurationViewport* Viewport)
{
	if (UDisplayClusterConfigurationClusterNode* ViewportParent = Cast<UDisplayClusterConfigurationClusterNode>(Viewport->GetOuter()))
	{
		if (const FString* KeyPtr = ViewportParent->Viewports.FindKey(Viewport))
		{
			Viewport->Modify();

			ViewportParent->Modify();

			RemoveKeyFromMap(ViewportParent, GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationClusterNode, Viewports), *KeyPtr);
			ensure (Viewport->GetPackage() == GetTransientPackage());
			
			Viewport->SetFlags(RF_Transient);
			
			return true;
		}
	}

	return false;
}

bool UE::DisplayClusterConfiguratorClusterUtils::RenameViewport(UDisplayClusterConfigurationViewport* Viewport, FString NewViewportName)
{
	if (UDisplayClusterConfigurationClusterNode* ViewportParent = Cast<UDisplayClusterConfigurationClusterNode>(Viewport->GetOuter()))
	{
		if (const FString* KeyPtr = ViewportParent->Viewports.FindKey(Viewport))
		{
			if (KeyPtr->Equals(NewViewportName))
			{
				return false;
			}

			const FString UniqueName = GetUniqueNameForViewport(NewViewportName, ViewportParent);
			
			Viewport->Modify();
			ViewportParent->Modify();
			
			const FName FieldName = GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationClusterNode, Viewports);

			// Remove and re-add. We can't just call RenameKeyInMap because when we rename the viewport after
			// it will lose sync with instances.
			
			RemoveKeyFromMap(ViewportParent, FieldName, *KeyPtr);

			// Rename after removing. If this is done after adding instances will lose sync with the CDO.
			Viewport->Rename(*UniqueName, ViewportParent, REN_DontCreateRedirectors);
			
			UDisplayClusterConfigurationViewport* NewViewport =
				CastChecked<UDisplayClusterConfigurationViewport>(AddKeyWithInstancedValueToMap(ViewportParent, FieldName, UniqueName, Viewport));
			check(Viewport != NewViewport);
			Viewport->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors | REN_DoNotDirty);
			Viewport->SetFlags(RF_Transient);
			
			FDisplayClusterConfiguratorUtils::MarkDisplayClusterBlueprintAsModified(NewViewport, true);
			
			return true;
		}
	}

	return false;
}

void UE::DisplayClusterConfiguratorClusterUtils::SortClusterNodesByHost(const TMap<FString, UDisplayClusterConfigurationClusterNode*>& InClusterNodes, TMap<FString, TMap<FString, UDisplayClusterConfigurationClusterNode*>>& OutSortedNodes)
{
	for (const TPair<FString, UDisplayClusterConfigurationClusterNode*>& ClusterNodePair : InClusterNodes)
	{
		check(ClusterNodePair.Value)

		FString Host = ClusterNodePair.Value->Host;
		if (!OutSortedNodes.Contains(Host))
		{
			OutSortedNodes.Add(Host, TMap<FString, UDisplayClusterConfigurationClusterNode*>());
		}

		OutSortedNodes[Host].Add(ClusterNodePair);
	}

	// Sort the hosts by the host address
	OutSortedNodes.KeySort(TLess<FString>());
}

void UE::DisplayClusterConfiguratorClusterUtils::SortClusterNodesByHost(const TMap<FString, TObjectPtr<UDisplayClusterConfigurationClusterNode>>& InClusterNodes, TMap<FString, TMap<FString, UDisplayClusterConfigurationClusterNode*>>& OutSortedNodes)
{
	for (const TPair<FString, TObjectPtr<UDisplayClusterConfigurationClusterNode>>& ClusterNodePair : InClusterNodes)
	{
		check(ClusterNodePair.Value)

			FString Host = ClusterNodePair.Value->Host;
		if (!OutSortedNodes.Contains(Host))
		{
			OutSortedNodes.Add(Host, TMap<FString, UDisplayClusterConfigurationClusterNode*>());
		}

		OutSortedNodes[Host].Add(ClusterNodePair);
	}

	// Sort the hosts by the host address
	OutSortedNodes.KeySort(TLess<FString>());
}

#undef LOCTEXT_NAMESPACE