// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDisplayClusterConfiguratorNewClusterItemDialog.h"

#include "IDetailsView.h"
#include "PropertyEditorModule.h"
#include "Modules/ModuleManager.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SUniformGridPanel.h"

#define LOCTEXT_NAMESPACE "SDisplayClusterConfiguratorNewClusterItemDialog"

const TArray<FDisplayClusterConfiguratorPresetSize> FDisplayClusterConfiguratorPresetSize::CommonPresets =
{
	FDisplayClusterConfiguratorPresetSize(LOCTEXT("SDDisplayName", "SD"), FVector2D(720, 480)),
	FDisplayClusterConfiguratorPresetSize(LOCTEXT("HDDisplayName", "HD"), FVector2D(1280, 720)),
	FDisplayClusterConfiguratorPresetSize(LOCTEXT("FullHDDisplayName", "Full HD"), FVector2D(1920, 1080)),
	FDisplayClusterConfiguratorPresetSize(LOCTEXT("2KDisplayName", "2K"), FVector2D(2048, 1080)),
	FDisplayClusterConfiguratorPresetSize(LOCTEXT("QuadHDDisplayName", "Quad HD"), FVector2D(2560, 1440)),
	FDisplayClusterConfiguratorPresetSize(LOCTEXT("UltraHDDisplayName", "Ultra HD"), FVector2D(3840, 2160)),
	FDisplayClusterConfiguratorPresetSize(LOCTEXT("4KDisplayName", "4K"), FVector2D(4096, 2160)),
	FDisplayClusterConfiguratorPresetSize(LOCTEXT("UltraHD2DisplayName", "Ultra HD-2"), FVector2D(7680, 4320)),
	FDisplayClusterConfiguratorPresetSize(LOCTEXT("8KDisplayName", "8K"), FVector2D(8192, 4320))
};

const int32 FDisplayClusterConfiguratorPresetSize::DefaultPreset = 2;

void SDisplayClusterConfiguratorNewClusterItemDialog::Construct(const FArguments& InArgs, UObject* InClusterItem)
{
	ParentWindow = InArgs._ParentWindow;
	OnPresetChanged = InArgs._OnPresetChanged;
	FooterContent = InArgs._FooterContent;
	MaxWindowWidth = InArgs._MaxWindowWidth;
	MaxWindowHeight = InArgs._MaxWindowHeight;

	if (!FooterContent.IsValid())
	{
		FooterContent = SNullWidget::NullWidget;
	}

	TSharedPtr<FString> InitiallySelectedParentItem;
	for (const FString& ParentItem : InArgs._ParentItemOptions)
	{
		TSharedPtr<FString> ParentItemPtr = MakeShared<FString>(ParentItem);

		// There should always be a selected item, so if the selected item hasn't been set yet, set it.
		if (!InitiallySelectedParentItem.IsValid() || ParentItem == InArgs._InitiallySelectedParentItem)
		{
			InitiallySelectedParentItem = ParentItemPtr;
		}

		ParentItems.Add(ParentItemPtr);
	}

	TSharedPtr<FDisplayClusterConfiguratorPresetSize> InitiallySelectedPresetItem;
	for (const FDisplayClusterConfiguratorPresetSize& PresetItem : InArgs._PresetItemOptions)
	{
		TSharedPtr<FDisplayClusterConfiguratorPresetSize> PresetItemPtr = MakeShared<FDisplayClusterConfiguratorPresetSize>(PresetItem);

		// There should always be a selected item, so if the selected item hasn't been set yet, set it.
		if (!InitiallySelectedPresetItem.IsValid() || PresetItem == InArgs._InitiallySelectedPreset)
		{
			InitiallySelectedPresetItem = PresetItemPtr;
		}

		PresetItems.Add(PresetItemPtr);
	}

	TSharedPtr<SBox> DetailsBox;
	ChildSlot
	[
		SNew(SBox)
		.MaxDesiredHeight(InArgs._MaxWindowHeight)
		.MaxDesiredWidth(InArgs._MaxWindowWidth)
		[
			SNew(SVerticalBox)

			+SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Fill)
			.Padding(2)
			[
				SNew(SBorder)
				.Padding(FMargin(3))
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					SNew(SVerticalBox)

					+SVerticalBox::Slot()
					.AutoHeight()
					.HAlign(HAlign_Fill)
					.Padding(2)
					[
						SNew(SHorizontalBox)

						+SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.AutoWidth()
						.Padding(2)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("NameTextLabel", "Name"))
						]

						+ SHorizontalBox::Slot()
						[
							SNew(SSpacer)
						]
		
						+SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.AutoWidth()
						.Padding(2)
						[
							SNew(SBox)
							.MinDesiredWidth(128)
							[
								SAssignNew(NameTextBox, SEditableTextBox)
								.Text(FText::FromString(InArgs._InitialName))
							]
						]
					]

					+SVerticalBox::Slot()
					.AutoHeight()
					.HAlign(HAlign_Fill)
					.Padding(2)
					[
						SNew(SHorizontalBox)

						+SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.AutoWidth()
						.Padding(2)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("ParentTextLabel", "Parent"))
						]

						+ SHorizontalBox::Slot()
						[
							SNew(SSpacer)
						]
		
						+SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.AutoWidth()
						.Padding(2)
						[
							SNew(SBox)
							.MinDesiredWidth(128)
							[
								SAssignNew(ParentComboBox, SComboBox<TSharedPtr<FString>>)
								.OptionsSource(&ParentItems)
								.InitiallySelectedItem(InitiallySelectedParentItem)
								.OnGenerateWidget_Lambda([](TSharedPtr<FString> Item)
								{
									return SNew(STextBlock).Text(Item.IsValid() ? FText::FromString(*Item) : FText::GetEmpty());
								})
								.Content()
								[
										SNew(STextBlock)
										.Text(this, &SDisplayClusterConfiguratorNewClusterItemDialog::GetParentComboBoxSelectedText)
								]
							]
						]
					]

					+SVerticalBox::Slot()
					.AutoHeight()
					.HAlign(HAlign_Fill)
					.Padding(2)
					[
						SNew(SHorizontalBox)

						+SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.AutoWidth()
						.Padding(2)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("PresetsTextLabel", "Preset"))
						]

						+ SHorizontalBox::Slot()
						[
							SNew(SSpacer)
						]
		
						+SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.AutoWidth()
						.Padding(2)
						[
							SNew(SBox)
							.MinDesiredWidth(128)
							[
								SAssignNew(PresetsComboBox, SComboBox<TSharedPtr<FDisplayClusterConfiguratorPresetSize>>)
								.OptionsSource(&PresetItems)
								.InitiallySelectedItem(InitiallySelectedPresetItem)
								.OnSelectionChanged(this, &SDisplayClusterConfiguratorNewClusterItemDialog::OnSelectedPresetChanged)
								.OnGenerateWidget_Lambda([this](TSharedPtr<FDisplayClusterConfiguratorPresetSize> Item)
								{
									return SNew(STextBlock).Text(GetPresetDisplayText(Item));
								})
								.Content()
								[
										SNew(STextBlock)
										.Text(this, &SDisplayClusterConfiguratorNewClusterItemDialog::GetPresetsComboBoxSelectedText)
								]
							]
						]
					]

					+SVerticalBox::Slot()
					.AutoHeight()
					.HAlign(HAlign_Fill)
					.Padding(2)
					[
						SNew(SVerticalBox)
						.Visibility_Lambda([this]() { return FooterContent == SNullWidget::NullWidget ? EVisibility::Collapsed : EVisibility::Visible; })

						+SVerticalBox::Slot()
						.AutoHeight()
						.HAlign(HAlign_Fill)
						.Padding(0.0f, 4.0f, 0.0f, 2.0f)
						[
							SNew(SSeparator)
							.SeparatorImage(FAppStyle::Get().GetBrush("Menu.Separator"))
							.Thickness(1.0f)
						]

						+SVerticalBox::Slot()
						.AutoHeight()
						.HAlign(HAlign_Fill)
						[
							FooterContent.ToSharedRef()
						]
					]
				]
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2)
			[
				SAssignNew(DetailsBox, SBox)
				.MaxDesiredHeight(this, &SDisplayClusterConfiguratorNewClusterItemDialog::GetDetailsMaxDesiredSize)
				.WidthOverride(InArgs._MaxWindowWidth - 10)
			]

			+SVerticalBox::Slot()
			.HAlign(HAlign_Right)
			.AutoHeight()
			.Padding(2)
			[
				SAssignNew(AddCancelButtons, SUniformGridPanel)
				.MinDesiredSlotHeight(FAppStyle::Get().GetFloat("StandardDialog.MinDesiredSlotHeight"))
				.MinDesiredSlotWidth(FAppStyle::Get().GetFloat("StandardDialog.MinDesiredSlotWidth"))
				.SlotPadding(FAppStyle::Get().GetMargin("StandardDialog.SlotPadding"))

				+SUniformGridPanel::Slot(0, 0)
				[
					SNew(SButton)
					.ContentPadding(FAppStyle::Get().GetMargin("StandardDialog.ContentPadding"))
					.HAlign(HAlign_Center)
					.Text(LOCTEXT("AddItemLabel", "Add"))
					.OnClicked(this, &SDisplayClusterConfiguratorNewClusterItemDialog::OnAddButtonClicked)
				]

				+SUniformGridPanel::Slot(1, 0)
				[
					SNew(SButton)
					.ContentPadding(FAppStyle::Get().GetMargin("StandardDialog.ContentPadding"))
					.HAlign(HAlign_Center)
					.Text(LOCTEXT("CancelLabel", "Cancel"))
					.OnClicked(this, &SDisplayClusterConfiguratorNewClusterItemDialog::OnCancelButtonClicked)
				]
			]
		]
	];

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	DetailsBox->SetContent(DetailsView->AsShared());
	DetailsView->SetObject(InClusterItem);
}

void SDisplayClusterConfiguratorNewClusterItemDialog::SetParentWindow(TSharedPtr<SWindow> InParentWindow)
{
	ParentWindow = InParentWindow;
}

FString SDisplayClusterConfiguratorNewClusterItemDialog::GetSelectedParentItem() const
{
	TSharedPtr<FString> SelectedItem = ParentComboBox->GetSelectedItem();
	if (SelectedItem.IsValid())
	{
		return *SelectedItem;
	}

	return "";
}

FString SDisplayClusterConfiguratorNewClusterItemDialog::GetItemName() const
{
	return NameTextBox->GetText().ToString();
}

bool SDisplayClusterConfiguratorNewClusterItemDialog::WasAccepted() const
{
	return bWasAccepted;
}

FText SDisplayClusterConfiguratorNewClusterItemDialog::GetParentComboBoxSelectedText() const
{
	TSharedPtr<FString> SelectedItem = ParentComboBox->GetSelectedItem();
	if (SelectedItem.IsValid())
	{
		return FText::FromString(*SelectedItem);
	}

	return FText::GetEmpty();
}

FText SDisplayClusterConfiguratorNewClusterItemDialog::GetPresetsComboBoxSelectedText() const
{
	return GetPresetDisplayText(PresetsComboBox->GetSelectedItem());
}

FReply SDisplayClusterConfiguratorNewClusterItemDialog::OnAddButtonClicked()
{
	bWasAccepted = true;

	if (ParentWindow.IsValid())
	{
		ParentWindow.Pin()->RequestDestroyWindow();
	}

	return FReply::Handled();
}

FReply SDisplayClusterConfiguratorNewClusterItemDialog::OnCancelButtonClicked()
{
	bWasAccepted = false;

	if (ParentWindow.IsValid())
	{
		ParentWindow.Pin()->RequestDestroyWindow();
	}

	return FReply::Handled();
}

FText SDisplayClusterConfiguratorNewClusterItemDialog::GetPresetDisplayText(const TSharedPtr<FDisplayClusterConfiguratorPresetSize>& Preset) const
{
	FText DisplayText = FText::GetEmpty();

	if (Preset.IsValid())
	{
		DisplayText = FText::Format(LOCTEXT("PresetDisplayText", "{0} ({1} x {2})"), Preset->DisplayName, Preset->Size.X, Preset->Size.Y);
	}

	return DisplayText;
}

void SDisplayClusterConfiguratorNewClusterItemDialog::OnSelectedPresetChanged(TSharedPtr<FDisplayClusterConfiguratorPresetSize> SelectedPreset, ESelectInfo::Type SelectionType)
{
	if (SelectionType != ESelectInfo::Type::Direct && SelectedPreset.IsValid())
	{
		OnPresetChanged.ExecuteIfBound(SelectedPreset->Size);
	}
}

FOptionalSize SDisplayClusterConfiguratorNewClusterItemDialog::GetDetailsMaxDesiredSize() const
{
	// The maximum size of the details panel box is dynamically computed as the max window size minus the size of all
	// other elements on the window, including any footer content.
	return MaxWindowHeight - 128 - FooterContent->GetTickSpaceGeometry().GetLocalSize().Y - AddCancelButtons->GetTickSpaceGeometry().GetLocalSize().Y;
}

#undef LOCTEXT_NAMESPACE