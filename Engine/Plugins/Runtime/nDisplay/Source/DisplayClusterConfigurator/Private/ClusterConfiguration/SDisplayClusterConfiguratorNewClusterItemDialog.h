// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Types/SlateStructs.h"

class IDetailsView;
class SWindow;
class SEditableTextBox;

template<class T>
class SComboBox;

struct FDisplayClusterConfiguratorPresetSize
{
public:
	FText DisplayName;
	FVector2D Size;

	FDisplayClusterConfiguratorPresetSize() :
		DisplayName(FText::GetEmpty()),
		Size(FVector2D::ZeroVector)
	{ }

	FDisplayClusterConfiguratorPresetSize(FText InDisplayName, FVector2D InSize) :
		DisplayName(InDisplayName),
		Size(InSize)
	{ }

	bool operator==(const FDisplayClusterConfiguratorPresetSize& Other) const
	{
		return (DisplayName.EqualTo(Other.DisplayName)) && (Size == Other.Size);
	}

public:
	static const TArray<FDisplayClusterConfiguratorPresetSize> CommonPresets;
	static const int32 DefaultPreset;
};

DECLARE_DELEGATE_OneParam(FOnPresetChanged, FVector2D);

class SDisplayClusterConfiguratorNewClusterItemDialog : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDisplayClusterConfiguratorNewClusterItemDialog)
		: _ParentWindow()
		, _ParentItemOptions()
		, _InitiallySelectedParentItem("")
		, _PresetItemOptions()
		, _InitiallySelectedPreset()
		, _InitialName("")
		, _MaxWindowHeight(0.0f)
		, _MaxWindowWidth(0.0f)
		, _FooterContent()
	{ }
		SLATE_ARGUMENT(TSharedPtr<SWindow>, ParentWindow)
		SLATE_ARGUMENT(TArray<FString>, ParentItemOptions)
		SLATE_ARGUMENT(FString, InitiallySelectedParentItem)
		SLATE_ARGUMENT(TArray<FDisplayClusterConfiguratorPresetSize>, PresetItemOptions)
		SLATE_ARGUMENT(FDisplayClusterConfiguratorPresetSize, InitiallySelectedPreset)
		SLATE_ARGUMENT(FString, InitialName)
		SLATE_ARGUMENT(float, MaxWindowHeight)
		SLATE_ARGUMENT(float, MaxWindowWidth)
		SLATE_ARGUMENT(TSharedPtr<SWidget>, FooterContent)
		SLATE_EVENT(FOnPresetChanged, OnPresetChanged)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UObject* InClusterItem);

	void SetParentWindow(TSharedPtr<SWindow> InParentWindow);

	FString GetSelectedParentItem() const;
	FString GetItemName() const;
	bool WasAccepted() const;

private:
	FText GetParentComboBoxSelectedText() const;
	FText GetPresetsComboBoxSelectedText() const;

	FReply OnAddButtonClicked();
	FReply OnCancelButtonClicked();

	FText GetPresetDisplayText(const TSharedPtr<FDisplayClusterConfiguratorPresetSize>& Preset) const;

	void OnSelectedPresetChanged(TSharedPtr<FDisplayClusterConfiguratorPresetSize> SelectedPreset, ESelectInfo::Type SelectionType);

	FOptionalSize GetDetailsMaxDesiredSize() const;

private:
	TWeakPtr<SWindow> ParentWindow;
	TSharedPtr<SWidget> FooterContent;
	/** The container for the Add and Cancel buttons. */
	TSharedPtr<SWidget> AddCancelButtons;
	TSharedPtr<IDetailsView> DetailsView;
	TSharedPtr<SComboBox<TSharedPtr<FString>>> ParentComboBox;
	TSharedPtr<SComboBox<TSharedPtr<FDisplayClusterConfiguratorPresetSize>>> PresetsComboBox;
	TSharedPtr<SEditableTextBox> NameTextBox;

	TArray<TSharedPtr<FString>> ParentItems;
	TArray<TSharedPtr<FDisplayClusterConfiguratorPresetSize>> PresetItems;

	FOnPresetChanged OnPresetChanged;

	float MaxWindowWidth;
	float MaxWindowHeight;
	bool bWasAccepted;
};