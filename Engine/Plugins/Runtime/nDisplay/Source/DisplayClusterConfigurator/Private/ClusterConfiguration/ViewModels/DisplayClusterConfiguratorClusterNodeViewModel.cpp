// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfiguratorClusterNodeViewModel.h"

#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterConfiguratorPropertyUtils.h"

FDisplayClusterConfiguratorClusterNodeViewModel::FDisplayClusterConfiguratorClusterNodeViewModel(UDisplayClusterConfigurationClusterNode* ClusterNode)
{
	ClusterNodePtr = ClusterNode;

	INIT_PROPERTY_HANDLE(UDisplayClusterConfigurationClusterNode, ClusterNode, WindowRect);
	INIT_PROPERTY_HANDLE(UDisplayClusterConfigurationClusterNode, ClusterNode, Host);
}

void FDisplayClusterConfiguratorClusterNodeViewModel::SetWindowRect(const FDisplayClusterConfigurationRectangle& NewWindowRect)
{
	UDisplayClusterConfigurationClusterNode* ClusterNode = ClusterNodePtr.Get();
	check(ClusterNode);
	ClusterNode->Modify();

	const FDisplayClusterConfigurationRectangle& CurrentWindowRect = ClusterNode->WindowRect;
	bool bClusterNodeChanged = false;

	if (CurrentWindowRect.X != NewWindowRect.X)
	{
		bClusterNodeChanged = true;

		TSharedPtr<IPropertyHandle> XHandle = WindowRectHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationRectangle, X));
		check(XHandle);

		XHandle->SetValue(NewWindowRect.X, EPropertyValueSetFlags::NotTransactable);
	}

	if (CurrentWindowRect.Y != NewWindowRect.Y)
	{
		bClusterNodeChanged = true;

		TSharedPtr<IPropertyHandle> YHandle = WindowRectHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationRectangle, Y));
		check(YHandle);

		YHandle->SetValue(NewWindowRect.Y, EPropertyValueSetFlags::NotTransactable);
	}

	if (CurrentWindowRect.W != NewWindowRect.W)
	{
		bClusterNodeChanged = true;

		TSharedPtr<IPropertyHandle> WHandle = WindowRectHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationRectangle, W));
		check(WHandle);

		WHandle->SetValue(NewWindowRect.W, EPropertyValueSetFlags::NotTransactable);
	}

	if (CurrentWindowRect.H != NewWindowRect.H)
	{
		bClusterNodeChanged = true;

		TSharedPtr<IPropertyHandle> HHandle = WindowRectHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationRectangle, H));
		check(HHandle);

		HHandle->SetValue(NewWindowRect.H, EPropertyValueSetFlags::NotTransactable);
	}

	if (bClusterNodeChanged)
	{
		ClusterNode->MarkPackageDirty();
	}
}

void FDisplayClusterConfiguratorClusterNodeViewModel::SetHost(const FString& NewHost)
{
	UDisplayClusterConfigurationClusterNode* ClusterNode = ClusterNodePtr.Get();
	check(ClusterNode);
	ClusterNode->Modify();

	bool bClusterNodeChanged = false;

	if (ClusterNode->Host != NewHost)
	{
		bClusterNodeChanged = true;

		HostHandle->SetValue(NewHost);
	}

	if (bClusterNodeChanged)
	{
		ClusterNode->MarkPackageDirty();
	}
}
