// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/WeakObjectPtr.h"

#include "DisplayClusterConfiguratorViewModelMacros.h"

class UDisplayClusterConfigurationClusterNode;
struct FDisplayClusterConfigurationRectangle;

class FDisplayClusterConfiguratorClusterNodeViewModel
{
public:
	FDisplayClusterConfiguratorClusterNodeViewModel(UDisplayClusterConfigurationClusterNode* ClusterNode);

	void SetWindowRect(const FDisplayClusterConfigurationRectangle& NewWindowRect);
	void SetHost(const FString& NewHost);

private:
	TWeakObjectPtr<UDisplayClusterConfigurationClusterNode> ClusterNodePtr;

	PROPERTY_HANDLE(WindowRect);
	PROPERTY_HANDLE(Host);
};