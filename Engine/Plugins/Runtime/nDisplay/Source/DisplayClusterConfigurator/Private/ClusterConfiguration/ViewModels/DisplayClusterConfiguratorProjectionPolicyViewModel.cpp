// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfiguratorProjectionPolicyViewModel.h"

#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterConfiguratorBlueprintEditor.h"
#include "DisplayClusterConfiguratorUtils.h"
#include "DisplayClusterConfiguratorPropertyUtils.h"

#include "ISinglePropertyView.h"
#include "PropertyHandle.h"

FDisplayClusterConfiguratorProjectionPolicyViewModel::FDisplayClusterConfiguratorProjectionPolicyViewModel(UDisplayClusterConfigurationViewport* InViewport)
{
	ViewportPtr = InViewport;

	ProjectionPolicyView = UE::DisplayClusterConfiguratorPropertyUtils::GetPropertyView(InViewport, GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationViewport, ProjectionPolicy));
	ProjectionPolicyHandle = ProjectionPolicyView->GetPropertyHandle();
}

void FDisplayClusterConfiguratorProjectionPolicyViewModel::SetPolicyType(const FString& PolicyType)
{
	UDisplayClusterConfigurationViewport* Viewport = ViewportPtr.Get();
	check(Viewport);
	Viewport->Modify();

	FStructProperty* StructProperty = FindFProperty<FStructProperty>(Viewport->GetClass(), GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationViewport, ProjectionPolicy));
	check(StructProperty);

	TSharedPtr<IPropertyHandle> TypeHandle = ProjectionPolicyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationProjection, Type));
	check(TypeHandle);

	TypeHandle->SetValue(PolicyType);

	Viewport->MarkPackageDirty();
}

void FDisplayClusterConfiguratorProjectionPolicyViewModel::SetIsCustom(bool bIsCustom)
{
	UDisplayClusterConfigurationViewport* Viewport = ViewportPtr.Get();
	check(Viewport);
	Viewport->Modify();

	FStructProperty* StructProperty = FindFProperty<FStructProperty>(Viewport->GetClass(), GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationViewport, ProjectionPolicy));
	check(StructProperty);

	TSharedPtr<IPropertyHandle> IsCustomHandle = ProjectionPolicyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationProjection, bIsCustom));
	check(IsCustomHandle);

	IsCustomHandle->SetValue(bIsCustom);

	Viewport->MarkPackageDirty();
}

void FDisplayClusterConfiguratorProjectionPolicyViewModel::SetParameterValue(const FString& ParameterKey, const FString& ParameterValue)
{
	UDisplayClusterConfigurationViewport* Viewport = ViewportPtr.Get();
	check(Viewport);
	Viewport->Modify();

	FStructProperty* StructProperty = FindFProperty<FStructProperty>(Viewport->GetClass(), GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationViewport, ProjectionPolicy));
	check(StructProperty);

	TSharedPtr<IPropertyHandle> ParametersHandle = ProjectionPolicyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationProjection, Parameters));
	check(ParametersHandle);

	uint8* MapContainer = StructProperty->ContainerPtrToValuePtr<uint8>(Viewport);
	UE::DisplayClusterConfiguratorPropertyUtils::AddKeyValueToMap(MapContainer, ParametersHandle, ParameterKey, ParameterValue);

	Viewport->MarkPackageDirty();
}

void FDisplayClusterConfiguratorProjectionPolicyViewModel::ClearParameters()
{
	UDisplayClusterConfigurationViewport* Viewport = ViewportPtr.Get();
	check(Viewport);
	Viewport->Modify();

	FStructProperty* StructProperty = FindFProperty<FStructProperty>(Viewport->GetClass(), GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationViewport, ProjectionPolicy));
	check(StructProperty);

	TSharedPtr<IPropertyHandle> ParametersHandle = ProjectionPolicyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationProjection, Parameters));
	check(ParametersHandle);

	uint8* MapContainer = StructProperty->ContainerPtrToValuePtr<uint8>(Viewport);
	UE::DisplayClusterConfiguratorPropertyUtils::EmptyMap(MapContainer, ParametersHandle);

	Viewport->MarkPackageDirty();
}