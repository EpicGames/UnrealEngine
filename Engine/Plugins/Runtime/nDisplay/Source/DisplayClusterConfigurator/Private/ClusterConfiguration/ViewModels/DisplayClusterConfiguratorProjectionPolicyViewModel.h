// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/WeakObjectPtr.h"

class UDisplayClusterConfigurationViewport;
class ISinglePropertyView;
class IPropertyHandle;

class FDisplayClusterConfiguratorProjectionPolicyViewModel
{
public:
	FDisplayClusterConfiguratorProjectionPolicyViewModel(UDisplayClusterConfigurationViewport* InViewport);

	void SetPolicyType(const FString& PolicyType);
	void SetIsCustom(bool bIsCustom);
	void SetParameterValue(const FString& ParameterKey, const FString& ParameterValue);
	void ClearParameters();

private:
	TSharedPtr<ISinglePropertyView> ProjectionPolicyView;
	TSharedPtr<IPropertyHandle> ProjectionPolicyHandle;
	TWeakObjectPtr<UDisplayClusterConfigurationViewport> ViewportPtr;
};