// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define PROPERTY_HANDLE(PropertyName) \
TSharedPtr<class ISinglePropertyView> PropertyName##View; \
TSharedPtr<class IPropertyHandle> PropertyName##Handle;

#define INIT_PROPERTY_HANDLE(ObjectClass, Object, Property) \
Property##View = UE::DisplayClusterConfiguratorPropertyUtils::GetPropertyView(Object, GET_MEMBER_NAME_CHECKED(ObjectClass, Property)); \
Property##Handle = Property##View->GetPropertyHandle();