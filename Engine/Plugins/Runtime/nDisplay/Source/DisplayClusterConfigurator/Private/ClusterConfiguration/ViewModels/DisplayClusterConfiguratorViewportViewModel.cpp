// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfiguratorViewportViewModel.h"

#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterConfiguratorPropertyUtils.h"

FDisplayClusterConfiguratorViewportViewModel::FDisplayClusterConfiguratorViewportViewModel(UDisplayClusterConfigurationViewport* Viewport)
{
	ViewportPtr = Viewport;

	INIT_PROPERTY_HANDLE(UDisplayClusterConfigurationViewport, Viewport, Camera);
	INIT_PROPERTY_HANDLE(UDisplayClusterConfigurationViewport, Viewport, Region);
	INIT_PROPERTY_HANDLE(UDisplayClusterConfigurationViewport, Viewport, ViewportRemap);
}

void FDisplayClusterConfiguratorViewportViewModel::SetCamera(const FString& NewCamera)
{
	UDisplayClusterConfigurationViewport* Viewport = ViewportPtr.Get();
	check(Viewport);
	Viewport->Modify();

	if (NewCamera != Viewport->Camera)
	{
		CameraHandle->SetValue(NewCamera);
		Viewport->MarkPackageDirty();
	}
}

void FDisplayClusterConfiguratorViewportViewModel::SetRegion(const FDisplayClusterConfigurationRectangle& NewRegion)
{
	UDisplayClusterConfigurationViewport* Viewport = ViewportPtr.Get();
	check(Viewport);
	Viewport->Modify();

	const FDisplayClusterConfigurationRectangle& CurrentRegion = Viewport->Region;
	bool bViewportChanged = false;

	if (CurrentRegion.X != NewRegion.X)
	{
		bViewportChanged = true;

		TSharedPtr<IPropertyHandle> XHandle = RegionHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationRectangle, X));
		check(XHandle);

		XHandle->SetValue(NewRegion.X, EPropertyValueSetFlags::NotTransactable);
	}

	if (CurrentRegion.Y != NewRegion.Y)
	{
		bViewportChanged = true;

		TSharedPtr<IPropertyHandle> YHandle = RegionHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationRectangle, Y));
		check(YHandle);

		YHandle->SetValue(NewRegion.Y, EPropertyValueSetFlags::NotTransactable);
	}

	if (CurrentRegion.W != NewRegion.W)
	{
		bViewportChanged = true;

		TSharedPtr<IPropertyHandle> WHandle = RegionHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationRectangle, W));
		check(WHandle);

		WHandle->SetValue(NewRegion.W, EPropertyValueSetFlags::NotTransactable);
	}

	if (CurrentRegion.H != NewRegion.H)
	{
		bViewportChanged = true;

		TSharedPtr<IPropertyHandle> HHandle = RegionHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationRectangle, H));
		check(HHandle);

		HHandle->SetValue(NewRegion.H, EPropertyValueSetFlags::NotTransactable);
	}

	if (bViewportChanged)
	{
		Viewport->MarkPackageDirty();
	}
}

void FDisplayClusterConfiguratorViewportViewModel::SetRemap(const FDisplayClusterConfigurationViewport_RemapData& NewRemap)
{
	UDisplayClusterConfigurationViewport* Viewport = ViewportPtr.Get();
	check(Viewport);
	Viewport->Modify();

	TSharedPtr<IPropertyHandle> BaseRemapHandle = ViewportRemapHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_Remap, BaseRemap));
	check(BaseRemapHandle);

	const FDisplayClusterConfigurationViewport_RemapData& CurrentBaseRemap = Viewport->ViewportRemap.BaseRemap;
	bool bViewportChanged = false;

	if (CurrentBaseRemap.Angle != NewRemap.Angle)
	{
		bViewportChanged = true;

		TSharedPtr<IPropertyHandle> AngleHandle = BaseRemapHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_RemapData, Angle));
		check(AngleHandle);

		AngleHandle->SetValue(NewRemap.Angle, EPropertyValueSetFlags::NotTransactable);
	}

	if (CurrentBaseRemap.bFlipH != NewRemap.bFlipH)
	{
		bViewportChanged = true;

		TSharedPtr<IPropertyHandle> FlipHHandle = BaseRemapHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_RemapData, bFlipH));
		check(FlipHHandle);

		FlipHHandle->SetValue(NewRemap.bFlipH, EPropertyValueSetFlags::NotTransactable);
	}

	if (CurrentBaseRemap.bFlipV != NewRemap.bFlipV)
	{
		bViewportChanged = true;

		TSharedPtr<IPropertyHandle> FlipVHandle = BaseRemapHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_RemapData, bFlipV));
		check(FlipVHandle);

		FlipVHandle->SetValue(NewRemap.bFlipV, EPropertyValueSetFlags::NotTransactable);
	}

	if (bViewportChanged)
	{
		Viewport->MarkPackageDirty();
	}
}