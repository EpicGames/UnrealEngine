// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/WeakObjectPtr.h"

#include "DisplayClusterConfiguratorViewModelMacros.h"

class UDisplayClusterConfigurationViewport;
struct FDisplayClusterConfigurationRectangle;
struct FDisplayClusterConfigurationViewport_RemapData;

/** Stores property handles for a viewport configuration object that can be used to propagate property changed events through the engine when changing those properties */
class FDisplayClusterConfiguratorViewportViewModel
{
public:
	/** Creates a new view model for the specified viewport configuration object */
	FDisplayClusterConfiguratorViewportViewModel(UDisplayClusterConfigurationViewport* Viewport);

	/** Sets the camera property of the viewport, propagating the change through property handles and marking the viewport's package as dirty  */
	void SetCamera(const FString& NewCamera);

	/** Sets the region of the viewport,  propagating the change through property handles and marking the viewport's package as dirty */
	void SetRegion(const FDisplayClusterConfigurationRectangle& NewRegion);

	/** Sets the base remap configuration of the viewport,  propagating the change through property handles and marking the viewport's package as dirty */
	void SetRemap(const FDisplayClusterConfigurationViewport_RemapData& NewRemap);

private:
	/** A pointer to the viewport configuration object the view model encapsulates */
	TWeakObjectPtr<UDisplayClusterConfigurationViewport> ViewportPtr;

	PROPERTY_HANDLE(Camera);
	PROPERTY_HANDLE(Region);
	PROPERTY_HANDLE(ViewportRemap);
};