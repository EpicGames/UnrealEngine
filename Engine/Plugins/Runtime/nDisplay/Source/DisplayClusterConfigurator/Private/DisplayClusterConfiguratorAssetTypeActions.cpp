// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfiguratorAssetTypeActions.h"

#include "DisplayClusterConfiguratorUtils.h"
#include "DisplayClusterRootActor.h"

UClass* FDisplayClusterConfiguratorActorAssetTypeActions::GetSupportedClass() const
{
	return ADisplayClusterRootActor::StaticClass();
}
