// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AssetTypeActions/AssetTypeActions_Blueprint.h"

class FDisplayClusterConfiguratorAssetTypeActions_Base : public FAssetTypeActions_Blueprint
{
public:
	FDisplayClusterConfiguratorAssetTypeActions_Base(uint32 Categories) : MyAssetCategory(Categories) {}
	virtual uint32 GetCategories() override { return MyAssetCategory; }

protected:
	uint32 MyAssetCategory;
};

/** Wrapper just to hide base actor class from being created in the asset browser. */
class FDisplayClusterConfiguratorActorAssetTypeActions : public FDisplayClusterConfiguratorAssetTypeActions_Base
{
public:
	FDisplayClusterConfiguratorActorAssetTypeActions(EAssetTypeCategories::Type InAssetCategory) : FDisplayClusterConfiguratorAssetTypeActions_Base(InAssetCategory) {}

	// FAssetTypeActions_Base
	virtual FText GetName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_DisplayClusterConfiguratorActorConfig", "nDisplay Config"); }
	virtual FColor GetTypeColor() const override { return FColor(0, 188, 212); }
	virtual UClass* GetSupportedClass() const override;
	// ~FAssetTypeActions_Base
};