// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfiguratorBlueprintEditor.h"

#include "Blueprints/DisplayClusterBlueprint.h"
#include "Blueprints/DisplayClusterBlueprintGeneratedClass.h"
#include "ClusterConfiguration/DisplayClusterConfiguratorClusterUtils.h"
#include "Components/DisplayClusterCameraComponent.h"
#include "Components/DisplayClusterOriginComponent.h"
#include "Components/DisplayClusterScreenComponent.h"
#include "Components/DisplayClusterXformComponent.h"
#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterConfiguratorCommands.h"
#include "DisplayClusterConfiguratorEditorMode.h"
#include "DisplayClusterConfiguratorEditorSubsystem.h"
#include "DisplayClusterConfiguratorLog.h"
#include "DisplayClusterConfiguratorModule.h"
#include "DisplayClusterConfiguratorPropertyUtils.h"
#include "DisplayClusterConfiguratorStyle.h"
#include "DisplayClusterConfiguratorUtils.h"
#include "DisplayClusterConfiguratorVersionUtils.h"
#include "DisplayClusterProjectionStrings.h"
#include "DisplayClusterRootActor.h"
#include "IDisplayClusterConfigurator.h"
#include "Settings/DisplayClusterConfiguratorSettings.h"
#include "Views/DisplayClusterConfiguratorToolbar.h"
#include "Views/OutputMapping/DisplayClusterConfiguratorViewOutputMapping.h"
#include "Views/SCSEditor/SDisplayClusterConfiguratorComponentCombo.h"
#include "Views/TreeViews/Cluster/DisplayClusterConfiguratorViewCluster.h"
#include "Views/Viewport/DisplayClusterConfiguratorSCSEditorViewport.h"
#include "Views/Viewport/DisplayClusterConfiguratorSCSEditorViewportClient.h"

#include "Camera/CameraComponent.h"
#include "ComponentAssetBroker.h"
#include "Components/ActorComponent.h"
#include "DesktopPlatformModule.h"
#include "EditorDirectories.h"
#include "EditorModeManager.h"
#include "EditorSupportDelegates.h"
#include "EditorViewportTabContent.h"
#include "Engine/Selection.h"
#include "Engine/SimpleConstructionScript.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Notifications/NotificationManager.h"
#include "GameFramework/Actor.h"
#include "HAL/ConsoleManager.h"
#include "HAL/PlatformApplicationMisc.h"
#include "ISCSEditorUICustomization.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/DebuggerCommands.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Misc/DisplayClusterHelpers.h"
#include "MPCDI/DisplayClusterConfiguratorMPCDIImporter.h"
#include "SBlueprintEditorToolbar.h"
#include "ScopedTransaction.h"
#include "SKismetInspector.h"
#include "SKismetInspector.h"
#include "SSCSEditor.h"
#include "SSubobjectEditor.h"
#include "SubobjectDataSubsystem.h"
#include "SubobjectEditorExtensionContext.h"
#include "Subsystems/PanelExtensionSubsystem.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "Editor/Kismet/Private/SSCSEditorViewport.h"
#include "Tools/BaseAssetToolkit.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SOverlay.h"



#define LOCTEXT_NAMESPACE "DisplayClusterConfiguratorBlueprintEditor"

namespace UE::DisplayClusterConfiguratorBlueprintEditorConstants
{
	const static FName SCSEditorExtensionIdentifier = TEXT("DisplayClusterSCSEditorExtension");
}

/*
 * Extend the SCS Editor. We hide the Add Component button and replace it with our own.
 */
class FDisplayClusterBlueprintEditorSCSEditorUICustomization : public ISCSEditorUICustomization
{
public:
	static TSharedRef<ISCSEditorUICustomization> GetInstance()
	{
		if (!Instance)
		{
			Instance = MakeShareable(new FDisplayClusterBlueprintEditorSCSEditorUICustomization());
		}
		return Instance.ToSharedRef();
	}

	virtual bool HideAddComponentButton(TArrayView<UObject*> Context) const override { return true; }
	virtual bool HideBlueprintButtons(TArrayView<UObject*> Context) const override { return true; }
	virtual EChildActorComponentTreeViewVisualizationMode GetChildActorVisualizationMode() const override { return EChildActorComponentTreeViewVisualizationMode::UseDefault; }

private:
	static TSharedPtr<FDisplayClusterBlueprintEditorSCSEditorUICustomization> Instance;

	FDisplayClusterBlueprintEditorSCSEditorUICustomization() {}
};

TSharedPtr<FDisplayClusterBlueprintEditorSCSEditorUICustomization> FDisplayClusterBlueprintEditorSCSEditorUICustomization::Instance;
TArray<TWeakPtr<FDisplayClusterConfiguratorBlueprintEditor>> FDisplayClusterConfiguratorBlueprintEditor::EditorsRegisteredForPanelExtension;

void FDisplayClusterConfiguratorBlueprintEditor::RegisterPanelExtensionFactory()
{
	if (UPanelExtensionSubsystem* PanelExtensionSubsystem = GEditor->GetEditorSubsystem<UPanelExtensionSubsystem>())
	{
		if (!PanelExtensionSubsystem->IsPanelFactoryRegistered(UE::DisplayClusterConfiguratorBlueprintEditorConstants::SCSEditorExtensionIdentifier))
		{
			FPanelExtensionFactory SCSEditorExtensionWidget;
			SCSEditorExtensionWidget.CreateExtensionWidget = FPanelExtensionFactory::FCreateExtensionWidget::CreateStatic(&FDisplayClusterConfiguratorBlueprintEditor::OnExtendAddComponentButton);
			SCSEditorExtensionWidget.Identifier = UE::DisplayClusterConfiguratorBlueprintEditorConstants::SCSEditorExtensionIdentifier;

			PanelExtensionSubsystem->RegisterPanelFactory(TEXT("SCSEditor.NextToAddComponentButton"), SCSEditorExtensionWidget);
		}
	}
}

void FDisplayClusterConfiguratorBlueprintEditor::UnregisterPanelExtensionFactory()
{
	if (UPanelExtensionSubsystem* PanelExtensionSubsystem = GEditor->GetEditorSubsystem<UPanelExtensionSubsystem>())
	{
		// Unregister all panel factories associated with the DisplayCluster configurator
		PanelExtensionSubsystem->UnregisterPanelFactory(UE::DisplayClusterConfiguratorBlueprintEditorConstants::SCSEditorExtensionIdentifier);
	}
}

TSharedRef<SWidget> FDisplayClusterConfiguratorBlueprintEditor::OnExtendAddComponentButton(FWeakObjectPtr ExtensionContext)
{
	if (const USubobjectEditorExtensionContext* SubobjectEditorExtensionContext = Cast<USubobjectEditorExtensionContext>(ExtensionContext.Get()))
	{
		if (const TSharedPtr<SSubobjectEditor> SubobjectEditor = SubobjectEditorExtensionContext->GetSubobjectEditor().Pin())
		{
			for (const TWeakPtr<FDisplayClusterConfiguratorBlueprintEditor>& BlueprintEditor : EditorsRegisteredForPanelExtension)
			{
				if (BlueprintEditor.IsValid() && BlueprintEditor.Pin()->SubobjectEditor == SubobjectEditor)
				{
					return BlueprintEditor.Pin()->CreateAddComponentButton();
				}
			}
		}
	}

	return SNullWidget::NullWidget;
}

FDisplayClusterConfiguratorBlueprintEditor::FDisplayClusterConfiguratorBlueprintEditor()
	: Super()
	, SharedModeTools(MakeShared<FEditorModeTools>())
{
	SharedModeTools->SetSupportsViewportITF(false);
}

FDisplayClusterConfiguratorBlueprintEditor::~FDisplayClusterConfiguratorBlueprintEditor()
{
	if (ViewOutputMapping.IsValid())
	{
		ViewOutputMapping->Cleanup();

		if (UpdateOutputMappingHandle.IsValid())
		{
			ViewOutputMapping->UnregisterOnOutputMappingBuilt(UpdateOutputMappingHandle);
		}
	}

	ShutdownDCSCSEditors();

	FBlueprintEditorUtils::OnRenameVariableReferencesEvent.Remove(RenameVariableHandle);
	FSlateApplication::Get().OnFocusChanging().Remove(FocusChangedHandle);
}

void FDisplayClusterConfiguratorBlueprintEditor::InitDisplayClusterBlueprintEditor(const EToolkitMode::Type Mode,
                                                                                   const TSharedPtr<IToolkitHost>& InitToolkitHost, UDisplayClusterBlueprint* Blueprint)
{
	LoadedBlueprint = Blueprint;
	FDisplayClusterConfiguratorVersionUtils::SetToLatestVersion(Blueprint);
	
	TSharedPtr<FDisplayClusterConfiguratorBlueprintEditor> Editor = SharedThis(this);
	
	if (!Toolbar.IsValid())
	{
		Toolbar = MakeShareable(new FBlueprintEditorToolbar(Editor));
	}

	if (!ConfiguratorToolbar.IsValid())
	{
		ConfiguratorToolbar = MakeShareable(new FDisplayClusterConfiguratorToolbar(Editor));
	}
	
	GetToolkitCommands()->Append(FPlayWorldCommands::GlobalPlayWorldActions.ToSharedRef());

	LoadedBlueprint->OnCompiled().AddSP(this, &FDisplayClusterConfiguratorBlueprintEditor::OnPostCompiled);
	
	CreateDefaultCommands();

	BindCommands();
	
	RegisterMenus();
	
	{
		const bool bCreateDefaultStandaloneMenu = true;
		const bool bCreateDefaultToolbar = true;

		InitAssetEditor(
			Mode,
			InitToolkitHost,
			FDisplayClusterEditorModes::DisplayClusterEditorName,
			FTabManager::FLayout::NullLayout,
			bCreateDefaultStandaloneMenu,
			bCreateDefaultToolbar,
			Blueprint
		);
	}

	TSharedRef<FApplicationMode> ConfigurationMode = MakeShared<FDisplayClusterConfiguratorEditorConfigurationMode>(Editor);
	AddApplicationMode(ConfigurationMode->GetModeName(), ConfigurationMode);

	// Register this blueprint editor instance to the list of blueprint editors that can receive panel expansion callbacks
	EditorsRegisteredForPanelExtension.Add(StaticCastWeakPtr<FDisplayClusterConfiguratorBlueprintEditor>(AsWeak()));

	const TArray<UBlueprint*> Blueprints{ Blueprint };
	CommonInitialization(Blueprints, false);

	SetSubobjectEditorUICustomization(FDisplayClusterBlueprintEditorSCSEditorUICustomization::GetInstance());
	
	CreateSCSEditorWrapper();

	ExtendMenu();
	ExtendToolbar();
	
	RegenerateMenusAndToolbars();
	
	CreateWidgets();
	
	// This does the actual layout generation.
	SetCurrentMode(ConfigurationMode->GetModeName());

	PostLayoutBlueprintEditorInitialization();

	RenameVariableHandle = FBlueprintEditorUtils::OnRenameVariableReferencesEvent.AddSP(this, &FDisplayClusterConfiguratorBlueprintEditor::OnRenameVariable);
	FocusChangedHandle = FSlateApplication::Get().OnFocusChanging().AddSP(this, &FDisplayClusterConfiguratorBlueprintEditor::OnFocusChanged);
}

void FDisplayClusterConfiguratorBlueprintEditor::PostUndo(bool bSuccess)
{
	FBlueprintEditor::PostUndo(bSuccess);

	if (UBlueprint* Blueprint = GetBlueprintObj())
	{
		// Helps when dealing with removal / addition of nodes and viewports.
		FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
	}

	// Make sure to force any property window displaying this actor class to refresh in case 
	// the cluster hierarchy was changed in the undo transaction
	if (ADisplayClusterRootActor* Actor = Cast<ADisplayClusterRootActor>(GetPreviewActor()))
	{
		FEditorSupportDelegates::ForcePropertyWindowRebuild.Broadcast(Actor->GetClass());
	}
}

void FDisplayClusterConfiguratorBlueprintEditor::PostRedo(bool bSuccess)
{
	FBlueprintEditor::PostRedo(bSuccess);

	if (UBlueprint* Blueprint = GetBlueprintObj())
	{
		// Helps when dealing with removal / addition of nodes and viewports.
		FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
	}

	// Make sure to force any property window displaying this actor class to refresh in case 
	// the cluster hierarchy was changed in the redo transaction
	if (ADisplayClusterRootActor* Actor = Cast<ADisplayClusterRootActor>(GetPreviewActor()))
	{
		FEditorSupportDelegates::ForcePropertyWindowRebuild.Broadcast(Actor->GetClass());
	}
}

void FDisplayClusterConfiguratorBlueprintEditor::RefreshEditors(ERefreshBlueprintEditorReason::Type Reason)
{
	FBlueprintEditor::RefreshEditors(Reason);
	
	if (ViewportTabContent.IsValid())
	{
		TFunction<void(FName, TSharedPtr<IEditorViewportLayoutEntity>)> RefreshFunc =
			[this](FName Name, TSharedPtr<IEditorViewportLayoutEntity> Entity)
			{
				if (Entity.IsValid())
				{
					TSharedRef<SDisplayClusterConfiguratorSCSEditorViewport> Viewport = StaticCastSharedRef<SDisplayClusterConfiguratorSCSEditorViewport>(Entity->AsWidget());
					Viewport->RequestRefresh();
				}
			};

		ViewportTabContent->PerformActionOnViewports(RefreshFunc);
	}
}

UDisplayClusterConfigurationData* FDisplayClusterConfiguratorBlueprintEditor::GetEditorData() const
{
	if (!LoadedBlueprint.IsValid() || !LoadedBlueprint->GeneratedClass)
	{
		return nullptr;
	}

	return LoadedBlueprint->GetOrLoadConfig();
}

FDelegateHandle FDisplayClusterConfiguratorBlueprintEditor::RegisterOnConfigReloaded(const FOnConfigReloadedDelegate& Delegate)
{
	return OnConfigReloaded.Add(Delegate);
}

void FDisplayClusterConfiguratorBlueprintEditor::UnregisterOnConfigReloaded(FDelegateHandle DelegateHandle)
{
	OnConfigReloaded.Remove(DelegateHandle);
}

FDelegateHandle FDisplayClusterConfiguratorBlueprintEditor::RegisterOnObjectSelected(const FOnObjectSelectedDelegate& Delegate)
{
	return OnObjectSelected.Add(Delegate);
}

void FDisplayClusterConfiguratorBlueprintEditor::UnregisterOnObjectSelected(FDelegateHandle DelegateHandle)
{
	OnObjectSelected.Remove(DelegateHandle);
}

FDelegateHandle FDisplayClusterConfiguratorBlueprintEditor::RegisterOnInvalidateViews(const FOnInvalidateViewsDelegate& Delegate)
{
	return OnInvalidateViews.Add(Delegate);
}

void FDisplayClusterConfiguratorBlueprintEditor::UnregisterOnInvalidateViews(FDelegateHandle DelegateHandle)
{
	OnInvalidateViews.Remove(DelegateHandle);
}


FDelegateHandle FDisplayClusterConfiguratorBlueprintEditor::RegisterOnClusterChanged(const FOnClusterChangedDelegate& Delegate)
{
	return OnClusterChanged.Add(Delegate);
}

void FDisplayClusterConfiguratorBlueprintEditor::UnregisterOnClusterChanged(FDelegateHandle DelegateHandle)
{
	OnClusterChanged.Remove(DelegateHandle);
}

TArray<UObject*> FDisplayClusterConfiguratorBlueprintEditor::GetSelectedObjects() const
{
	TArray<UObject*> OutSelectedObjects;
	OutSelectedObjects.Reserve(SelectedObjects.Num());

	for (const TWeakObjectPtr<UObject>& SelectedObject : SelectedObjects)
	{
		if (SelectedObject.IsValid())
		{
			OutSelectedObjects.Add(SelectedObject.Get());
		}
	}

	return OutSelectedObjects;
}

bool FDisplayClusterConfiguratorBlueprintEditor::IsObjectSelected(UObject* Obj) const
{
	return SelectedObjects.Contains(Obj);
}

void FDisplayClusterConfiguratorBlueprintEditor::SelectObjects(TArray<UObject*>& InSelectedObjects, bool bFullRefresh)
{
	SelectedObjects.Empty();
	for (UObject* SelectedObject : InSelectedObjects)
	{
		SelectedObjects.Add(TWeakObjectPtr<UObject>(SelectedObject));
	}

	// Clear old tree view selections and update the details panel.
	
	if (CurrentSelectionSource == ESelectionSource::Internal)
	{
		// SCS Selection is handled from OnSelectionUpdated.

		if (ViewCluster.IsValid())
		{
			ViewCluster->ClearSelection();
		}
	}
	else
	{
		TSharedPtr<SSubobjectEditor> SubobjectEditorTreeView = GetSubobjectEditor();
		if (SubobjectEditorTreeView.IsValid())
		{
			const FSelectionScope SelectionScope(this, ESelectionSource::Ancillary);
			SubobjectEditorTreeView->ClearSelection();
		}
		
		SKismetInspector::FShowDetailsOptions Options;
		Options.bForceRefresh = bFullRefresh;
		Inspector->ShowDetailsForObjects(InSelectedObjects, Options);
	}
	
	OnObjectSelected.Broadcast();
}

void FDisplayClusterConfiguratorBlueprintEditor::SelectAncillaryComponents(const TArray<FString>& ComponentNames)
{
	const FSelectionScope SelectionScope(this, ESelectionSource::Ancillary);
	if (ADisplayClusterRootActor* RootActor = Cast<ADisplayClusterRootActor>(GetPreviewActor()))
	{
		TArray<UActorComponent*> ComponentsToSelect;
		for (const FString& ComponentName : ComponentNames)
		{
			TArray<UActorComponent*> RootActorComponents;
			RootActor->GetComponents(RootActorComponents);

			UActorComponent** FoundComponentPtr =  RootActorComponents.FindByPredicate([&ComponentName](UActorComponent* Component)
			{
				return Component->GetName() == ComponentName;
			});

			if (FoundComponentPtr)
			{
				FindAndSelectSubobjectEditorTreeNode(*FoundComponentPtr, true);
			}
		}
	}
}

void FDisplayClusterConfiguratorBlueprintEditor::SelectAncillaryViewports(const TArray<FString>& ComponentNames)
{
	if (UDisplayClusterConfigurationData* Config = GetConfig())
	{
		TArray<UObject*> ViewportsToSelect;
		for (TPair<FString, UDisplayClusterConfigurationClusterNode*> ClusterNodePair : Config->Cluster->Nodes)
		{
			UDisplayClusterConfigurationClusterNode* ClusterNode = ClusterNodePair.Value;
			for (TPair<FString, UDisplayClusterConfigurationViewport*> ViewportPair : ClusterNode->Viewports)
			{
				UDisplayClusterConfigurationViewport* Viewport = ViewportPair.Value;

				FString ComponentName;
				if (Viewport->ProjectionPolicy.Type.Equals(DisplayClusterProjectionStrings::projection::Mesh, ESearchCase::IgnoreCase)
					&& Viewport->ProjectionPolicy.Parameters.Contains(DisplayClusterProjectionStrings::cfg::mesh::Component))
				{
					ComponentName = Viewport->ProjectionPolicy.Parameters[DisplayClusterProjectionStrings::cfg::mesh::Component];
				}
				else if (Viewport->ProjectionPolicy.Type.Equals(DisplayClusterProjectionStrings::projection::Camera, ESearchCase::IgnoreCase)
					&& Viewport->ProjectionPolicy.Parameters.Contains(DisplayClusterProjectionStrings::cfg::camera::Component))
				{
					ComponentName = Viewport->ProjectionPolicy.Parameters[DisplayClusterProjectionStrings::cfg::camera::Component];
				}

				if (ComponentNames.Contains(ComponentName))
				{
					ViewportsToSelect.Add(Viewport);
				}
			}
		}

		if (ViewCluster.IsValid())
		{
			ViewCluster->FindAndSelectObjects(ViewportsToSelect);
		}

		if (ViewOutputMapping.IsValid())
		{
			ViewOutputMapping->FindAndSelectObjects(ViewportsToSelect);
		}
	}
}

void FDisplayClusterConfiguratorBlueprintEditor::InvalidateViews()
{
	OnInvalidateViews.Broadcast();
}

void FDisplayClusterConfiguratorBlueprintEditor::ClusterChanged(bool bStructureChange)
{
	if (LoadedBlueprint.IsValid())
	{
		const FSelectionScope SelectionScope(this, ESelectionSource::Refresh);
		FDisplayClusterConfiguratorUtils::MarkDisplayClusterBlueprintAsModified(LoadedBlueprint.Get(), bStructureChange);
	}

	if (ADisplayClusterRootActor* Actor = Cast<ADisplayClusterRootActor>(GetPreviewActor()))
	{
		FEditorSupportDelegates::ForcePropertyWindowRebuild.Broadcast(Actor->GetClass());
	}

	OnClusterChanged.Broadcast();
}

void FDisplayClusterConfiguratorBlueprintEditor::ClearViewportSelection()
{
	OnClearViewportSelection.Broadcast();
}

UDisplayClusterConfigurationData* FDisplayClusterConfiguratorBlueprintEditor::GetConfig() const
{
	// TODO: This is hacky to keep it consistent with previous usage.
	// GetEditorData and GetConfig are basically the same.
	if (UDisplayClusterConfigurationData* Data = GetEditorData())
	{
		return Data;
	}
	
	return nullptr;
}

ADisplayClusterRootActor* FDisplayClusterConfiguratorBlueprintEditor::GetDefaultRootActor() const
{
	if (UBlueprint* Blueprint = GetBlueprintObj())
	{
		if (Blueprint->GeneratedClass)
		{
			return Cast<ADisplayClusterRootActor>(Blueprint->GeneratedClass->GetDefaultObject(false));
		}
	}

	return nullptr;
}

TSharedRef<IDisplayClusterConfiguratorViewOutputMapping> FDisplayClusterConfiguratorBlueprintEditor::GetViewOutputMapping() const
{
	return ViewOutputMapping.ToSharedRef();
}

TSharedRef<IDisplayClusterConfiguratorViewTree> FDisplayClusterConfiguratorBlueprintEditor::GetViewCluster() const
{
	return ViewCluster.ToSharedRef();
}

TSharedRef<FEditorModeTools> FDisplayClusterConfiguratorBlueprintEditor::GetModeTools() const
{
	return SharedModeTools;
}

void FDisplayClusterConfiguratorBlueprintEditor::SyncViewports()
{
	if (ViewportTabContent.IsValid())
	{
		TFunction<void(FName, TSharedPtr<IEditorViewportLayoutEntity>)> ViewportFunc =
			[this](FName Name, TSharedPtr<IEditorViewportLayoutEntity> Entity)
		{
			const TSharedRef<SDisplayClusterConfiguratorSCSEditorViewport> Viewport = StaticCastSharedRef<SDisplayClusterConfiguratorSCSEditorViewport>(Entity->AsWidget());
			Viewport->GetDisplayClusterViewportClient()->SyncEditorSettings();
		};

		ViewportTabContent->PerformActionOnViewports(ViewportFunc);
	}
}

void FDisplayClusterConfiguratorBlueprintEditor::RefreshDisplayClusterPreviewActor()
{
	AActor* OldPreviewActor = GetPreviewActor();
	const bool bCreatePreviewActor = OldPreviewActor == nullptr;

	UpdatePreviewActor(GetBlueprintObj(), bCreatePreviewActor);

	AActor* NewPreviewActor = GetPreviewActor();
	
	// For the new preview actor, update the state of its Xform gizmos to match the
	// project settings for the display cluster editor
	UpdateXformGizmos();

	CurrentPreviewActor = NewPreviewActor;
}

void FDisplayClusterConfiguratorBlueprintEditor::RestoreLastEditedState()
{
	check(IsEditingSingleBlueprint());

	UBlueprint* Blueprint = GetBlueprintObj();
	for (const FEditedDocumentInfo& Document : Blueprint->LastEditedDocuments)
	{
		if (UObject* Obj = Document.EditedObjectPath.ResolveObject())
		{
			TSharedPtr<SDockTab> TabWithGraph = OpenDocument(Obj, FDocumentTracker::RestorePreviousDocument);
		}
	}

	// Small hack to make sure our viewport is focused by default. Restoring documents seem to override
	// the focused tab from blueprint editor modes.
	if (TabManager->HasTabSpawner(FDisplayClusterConfiguratorBlueprintModeBase::TabID_Viewport) &&
		TabManager->FindExistingLiveTab(FDisplayClusterConfiguratorBlueprintModeBase::TabID_Viewport))
	{
		TabManager->TryInvokeTab(FDisplayClusterConfiguratorBlueprintModeBase::TabID_Viewport);
	}
}

void FDisplayClusterConfiguratorBlueprintEditor::UpdateXformGizmos()
{
	if (ADisplayClusterRootActor* RootActor = Cast<ADisplayClusterRootActor>(GetPreviewActor()))
	{
		const UDisplayClusterConfiguratorEditorSettings* Settings = GetDefault<UDisplayClusterConfiguratorEditorSettings>();

		// Get all Xform components
		TInlineComponentArray<UDisplayClusterXformComponent*> Xforms;
		RootActor->GetComponents(Xforms);

		// And apply new gizmo settings
		for (UDisplayClusterXformComponent* Xform : Xforms)
		{
			Xform->SetVisualizationScale(Settings->VisXformScale);
			Xform->SetVisualizationEnabled(Settings->bShowVisXforms);
		}
	}
}

bool FDisplayClusterConfiguratorBlueprintEditor::LoadWithOpenFileDialog()
{
	const FString NDisplayFileDescription = LOCTEXT("NDisplayFileDescription", "nDisplay Config").ToString();
	const FString NDisplayFileExtension = TEXT("*.ndisplay");
	const FString FileTypes = FString::Printf(TEXT("%s (%s)|%s"), *NDisplayFileDescription, *NDisplayFileExtension, *NDisplayFileExtension);

	// Prompt the user for the filenames
	TArray<FString> OpenFilenames;
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	bool bFileSelected = false;
	int32 FilterIndex = -1;

	// Open file dialog
	if (DesktopPlatform)
	{
		const void* ParentWindowWindowHandle = FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr);

		bFileSelected = DesktopPlatform->OpenFileDialog(
			ParentWindowWindowHandle,
			LOCTEXT("ImportDialogTitle", "Import").ToString(),
			FEditorDirectories::Get().GetLastDirectory(ELastDirectory::GENERIC_IMPORT),
			TEXT(""),
			FileTypes,
			EFileDialogFlags::None,
			OpenFilenames,
			FilterIndex
		);
	}

	// Load file
	if (bFileSelected)
	{
		if (OpenFilenames.Num() > 0)
		{
			FEditorDirectories::Get().SetLastDirectory(ELastDirectory::GENERIC_IMPORT, OpenFilenames[0]);

			// The system relative path is not associated with the project directory.
			// We must convert it to the relative path of the UE project
			const FString RelativeFilePath = DisplayClusterHelpers::filesystem::GetRelativePathForConfigResource(FPaths::ConvertRelativePathToFull(OpenFilenames[0]));

			return LoadFromFile(RelativeFilePath);
		}
	}

	return false;
}

bool FDisplayClusterConfiguratorBlueprintEditor::LoadFromFile(const FString& FilePath)
{
	check(LoadedBlueprint.IsValid());
	if (ADisplayClusterRootActor* NewRootActor = FDisplayClusterConfiguratorUtils::GenerateRootActorFromConfigFile(FilePath))
	{
		UDisplayClusterBlueprint* const LoadedBlueprintPtr = LoadedBlueprint.Get();
		if (!IsValid(LoadedBlueprintPtr))
		{
			return false;
		}

		LoadedBlueprintPtr->SetConfigData(NewRootActor->GetConfigData(), true);
		UDisplayClusterConfigurationData* ConfigData = LoadedBlueprintPtr->GetConfig();
		check(ConfigData);
		ConfigData->ImportedPath = FilePath;

		FDisplayClusterConfiguratorUtils::AddRootActorComponentsToBlueprint(LoadedBlueprintPtr, NewRootActor, false);
		ClusterChanged();
		FKismetEditorUtilities::CompileBlueprint(LoadedBlueprintPtr);
		UpdatePreviewActor(LoadedBlueprintPtr, true);
		UpdateSubobjectPreview(true);
		SubobjectEditor->UpdateTree();
		LoadedBlueprintPtr->MarkPackageDirty();

		return true;
	}
	
	return false;
}

bool FDisplayClusterConfiguratorBlueprintEditor::ExportConfig()
{
	UDisplayClusterConfigurationData* EditingObject = GetEditorData();
	check(EditingObject);
	if (EditingObject->PathToConfig.IsEmpty())
	{
		return false;
	}
	
	const bool bResult = SaveToFile(EditingObject->PathToConfig);
	
	if (bResult)
	{
		// Add log
		UDisplayClusterConfigurationData* EditorData = GetEditorData();
		check(EditorData != nullptr);
		UE_LOGF(DisplayClusterConfiguratorLog, Log, "Successfully exported config with the path: %ls", *EditorData->PathToConfig);
	}
	
	return bResult;
}

bool FDisplayClusterConfiguratorBlueprintEditor::CanExportConfig() const
{
	UDisplayClusterConfigurationData* EditingObject = GetEditorData();
	return FDisplayClusterConfiguratorUtils::IsPrimaryNodeInConfig(EditingObject);
}

bool FDisplayClusterConfiguratorBlueprintEditor::SaveToFile(const FString& InFilePath)
{
	UDisplayClusterConfiguratorEditorSubsystem* EditorSubsystem = GEditor->GetEditorSubsystem<UDisplayClusterConfiguratorEditorSubsystem>();
	if (EditorSubsystem && LoadedBlueprint.IsValid())
	{
		LoadedBlueprint->PrepareConfigForExport();

		if (UDisplayClusterConfigurationData* Data = GetEditorData())
		{
			// Finally, store data to the file
			if (EditorSubsystem->SaveConfig(Data, InFilePath))
			{
				LoadedBlueprint->SetConfigPath(Data->PathToConfig);
				return true;
			}
		}
	}

	return false;
}

bool FDisplayClusterConfiguratorBlueprintEditor::SaveWithOpenFileDialog()
{
	const FString NDisplayFileDescription = LOCTEXT("NDisplayFileDescription", "nDisplay Config").ToString();
	const FString NDisplayFileExtension = TEXT("*.ndisplay");
	const FString FileTypes = FString::Printf(TEXT("%s (%s)|%s"), *NDisplayFileDescription, *NDisplayFileExtension, *NDisplayFileExtension);

	UDisplayClusterConfigurationData* EditingObject = GetEditorData();

	if (EditingObject && LoadedBlueprint.IsValid())
	{
		// Prompt the user for the filenames
		TArray<FString> SaveFilenames;
		IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
		bool bFileSelected = false;
		int32 FilterIndex = -1;

		// Open file dialog
		if (DesktopPlatform)
		{
			const void* ParentWindowWindowHandle = FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr);

			bFileSelected = DesktopPlatform->SaveFileDialog(
				ParentWindowWindowHandle,
				LOCTEXT("ExportDialogTitle", "Export").ToString(),
				FEditorDirectories::Get().GetLastDirectory(ELastDirectory::GENERIC_EXPORT),
				LoadedBlueprint->GetName(),
				FileTypes,
				EFileDialogFlags::None,
				SaveFilenames
			);
		}

		if (bFileSelected && SaveFilenames.Num() > 0)
		{
			FEditorDirectories::Get().SetLastDirectory(ELastDirectory::GENERIC_EXPORT, FPaths::GetPath(SaveFilenames[0]));
			
			UDisplayClusterConfiguratorEditorSubsystem* EditorSubsystem = GEditor->GetEditorSubsystem<UDisplayClusterConfiguratorEditorSubsystem>();
			if (EditorSubsystem)
			{
				return SaveToFile(SaveFilenames[0]);
			}
		}
	}

	return false;
}

void FDisplayClusterConfiguratorBlueprintEditor::OnRenameVariable(UBlueprint* Blueprint, UClass* VariableClass, const FName& OldVariableName, const FName& NewVariableName)
{
	if (Blueprint != LoadedBlueprint)
	{
		return;
	}

	// Check the configuration's viewports for any matching references to the component variable being renamed, and update those references
	// to the new variable name
	if (const UDisplayClusterConfigurationData* Config = GetConfig())
	{
		for (const TPair<FString, TObjectPtr<UDisplayClusterConfigurationClusterNode>>& ClusterNodePair : Config->Cluster->Nodes)
		{
			UDisplayClusterConfigurationClusterNode* ClusterNode = ClusterNodePair.Value;
			for (const TPair<FString, TObjectPtr<UDisplayClusterConfigurationViewport>>& ViewportPair : ClusterNode->Viewports)
			{
				UDisplayClusterConfigurationViewport* Viewport = ViewportPair.Value;

				// Don't attempt to replace variable names in custom projection policies as users should be in complete control
				// of the parameters in the custom case
				if (!Viewport->ProjectionPolicy.bIsCustom)
				{
					TMap<FString, FString> PolicyCopy = Viewport->ProjectionPolicy.Parameters;

					const TSharedPtr<ISinglePropertyView> ProjectPolicyView =
						UE::DisplayClusterConfiguratorPropertyUtils::GetPropertyView(
							Viewport, GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationViewport, ProjectionPolicy));
					check(ProjectPolicyView);
					
					const TSharedPtr<IPropertyHandle> ParametersHandle = ProjectPolicyView->GetPropertyHandle()->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationProjection, Parameters));
					check(ParametersHandle);

					FStructProperty* StructProperty = FindFProperty<FStructProperty>(Viewport->GetClass(), GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationViewport, ProjectionPolicy));
					check(StructProperty);
					uint8* MapContainer = StructProperty->ContainerPtrToValuePtr<uint8>(Viewport);
					
					for (TPair<FString, FString>& PolicyParameters : PolicyCopy)
					{
						if (PolicyParameters.Value == OldVariableName.ToString())
						{
							UE::DisplayClusterConfiguratorPropertyUtils::RemoveKeyFromMap(MapContainer, ParametersHandle, PolicyParameters.Key);
							UE::DisplayClusterConfiguratorPropertyUtils::AddKeyValueToMap(MapContainer, ParametersHandle, PolicyParameters.Key, NewVariableName.ToString());
						}
					}
				}

				if (Viewport->Camera == OldVariableName.ToString())
				{
					UE::DisplayClusterConfiguratorPropertyUtils::SetPropertyHandleValue(Viewport, GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationViewport, Camera), NewVariableName.ToString());
				}

				if (Viewport->DisplayDeviceName == OldVariableName.ToString())
				{
					UE::DisplayClusterConfiguratorPropertyUtils::SetPropertyHandleValue(Viewport, GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationViewport, DisplayDeviceName), NewVariableName.ToString());
				}
			}
		}
	}
}

void FDisplayClusterConfiguratorBlueprintEditor::OnFocusChanged(const FFocusEvent& FocusEvent, const FWeakWidgetPath& OldFocusedWidgetPath, const TSharedPtr<SWidget>& OldFocusedWidget, const FWidgetPath& NewFocusedWidgetPath, const TSharedPtr<SWidget>& NewFocusedWidget)
{
	if (NewFocusedWidget.IsValid())
	{
		// If the SCSEditor or the Cluster View are being focused, update the property inspector to show the items currently selected in the respective tree view.
		// This ensures that if any item as selected as part of an ancillary selection, the user can still "select" it and have its properties show up in the
		// inspector, since already selected items don't fire a OnSelectionChanged event where we would normally update the property inspector
		if (NewFocusedWidgetPath.ContainsWidget(SubobjectEditor.Get()))
		{
			TArray<FSubobjectEditorTreeNodePtrType> SelectedNodes = SubobjectEditor->GetSelectedNodes();
			if (Inspector.IsValid())
			{
				// Convert the selection set to an array of UObject* pointers
				FText InspectorTitle = FText::GetEmpty();
				TArray<UObject*> InspectorObjects;
				bool bShowComponents = true;
				InspectorObjects.Empty(SelectedNodes.Num());
				for (FSubobjectEditorTreeNodePtrType NodePtr : SelectedNodes)
				{
					if (NodePtr.IsValid())
					{
						FSubobjectData* Data = NodePtr->GetDataSource();
						if (Data->IsActor())
						{
							if (const AActor* DefaultActor = Data->GetObjectForBlueprint<AActor>(GetBlueprintObj()))
							{
								InspectorObjects.Add(const_cast<AActor*>(DefaultActor));

								FString Title;
								DefaultActor->GetName(Title);
								InspectorTitle = FText::FromString(Title);
								bShowComponents = false;

								TryInvokingDetailsTab();
							}
						}
						else
						{
							const UActorComponent* EditableComponent = Data->GetObjectForBlueprint<UActorComponent>(GetBlueprintObj());
							if (EditableComponent)
							{
								InspectorTitle = FText::FromString(NodePtr->GetDisplayString());
								InspectorObjects.Add(const_cast<UActorComponent*>(EditableComponent));
							}
						}
					}
				}

				// Update the details panel
				SKismetInspector::FShowDetailsOptions Options(InspectorTitle, true);
				Options.bShowComponents = bShowComponents;
				Inspector->ShowDetailsForObjects(InspectorObjects, Options);
			}
		}
		else if (NewFocusedWidgetPath.ContainsWidget(&ViewCluster->GetWidget().Get()))
		{
			TArray<UObject*> SelectedClusterObjects;
			ViewCluster->GetSelectedObjects(SelectedClusterObjects);

			SKismetInspector::FShowDetailsOptions Options;
			Inspector->ShowDetailsForObjects(SelectedClusterObjects, Options);
		}
	}
}

void FDisplayClusterConfiguratorBlueprintEditor::BindCommands()
{
	const FDisplayClusterConfiguratorCommands& Commands = IDisplayClusterConfigurator::Get().GetCommands();

	ToolkitCommands->MapAction(Commands.Import, FExecuteAction::CreateSP(this, &FDisplayClusterConfiguratorBlueprintEditor::ImportConfig_Clicked));
	ToolkitCommands->MapAction(Commands.Export, FExecuteAction::CreateSP(this, &FDisplayClusterConfiguratorBlueprintEditor::ExportToFile_Clicked),
		FCanExecuteAction::CreateSP(this, &FDisplayClusterConfiguratorBlueprintEditor::CanExportConfig));
	ToolkitCommands->MapAction(Commands.EditConfig, FExecuteAction::CreateSP(this, &FDisplayClusterConfiguratorBlueprintEditor::EditConfig_Clicked));

	ToolkitCommands->MapAction(
		Commands.ExportConfigOnSave,
		FExecuteAction::CreateSP(this, & FDisplayClusterConfiguratorBlueprintEditor::ToggleExportOnSaveSetting),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FDisplayClusterConfiguratorBlueprintEditor::IsExportOnSaveSet)
	);

	ToolkitCommands->MapAction(Commands.ImportMPCDI, FExecuteAction::CreateSP(this, &FDisplayClusterConfiguratorBlueprintEditor::ImportMPCDI_Clicked));
}

void FDisplayClusterConfiguratorBlueprintEditor::CreateWidgets()
{
	TSharedRef<FDisplayClusterConfiguratorBlueprintEditor> ThisRef(SharedThis(this));

	ViewOutputMapping	= MakeShared<FDisplayClusterConfiguratorViewOutputMapping>(ThisRef);
	ViewCluster			= MakeShared<FDisplayClusterConfiguratorViewCluster>(ThisRef);

	ViewOutputMapping->CreateWidget();
	ViewCluster->CreateWidget();

	// Set the visibility
	{
		bool bReadOnly = false;// IConsoleManager::Get().FindConsoleVariable(TEXT("nDisplay.configurator.ReadOnly"))->GetBool();

		ViewOutputMapping->SetEnabled(!bReadOnly);
		ViewCluster->SetEnabled(!bReadOnly);
	}
	
	OnConfigReloaded.Broadcast();
}

void FDisplayClusterConfiguratorBlueprintEditor::ExtendMenu()
{
	if (MenuExtender.IsValid())
	{
		RemoveMenuExtender(MenuExtender);
		MenuExtender.Reset();
	}

	MenuExtender = MakeShareable(new FExtender);
	AddMenuExtender(MenuExtender);

	// add extensible menu if exists
	IDisplayClusterConfigurator& DisplayClusterEditorModule = FModuleManager::LoadModuleChecked<IDisplayClusterConfigurator>("DisplayClusterConfigurator");
	AddMenuExtender(DisplayClusterEditorModule.GetMenuExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));

}

void FDisplayClusterConfiguratorBlueprintEditor::ExtendToolbar()
{
	// If the ToolbarExtender is valid, remove it before rebuilding it
	if (ToolbarExtender.IsValid())
	{
		RemoveToolbarExtender(ToolbarExtender);
		ToolbarExtender.Reset();
	}

	ToolbarExtender = MakeShareable(new FExtender);
	AddToolbarExtender(ToolbarExtender);

	IDisplayClusterConfigurator& DisplayClusterEditorModule = FModuleManager::LoadModuleChecked<IDisplayClusterConfigurator>("DisplayClusterConfigurator");
	AddToolbarExtender(DisplayClusterEditorModule.GetToolBarExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));
}

void FDisplayClusterConfiguratorBlueprintEditor::OnPostCompiled(UBlueprint* InBlueprint)
{
	TArray<TWeakObjectPtr<UObject>> PrevSelectedObjects = SelectedObjects;
	OnConfigReloaded.Broadcast();

	UDisplayClusterConfigurationData* CurrentConfigData = GetConfig();
	check(CurrentConfigData);
	
	TArray<UObject*> NewObjects;
	for (const TWeakObjectPtr<UObject>& Object : SelectedObjects)
	{
		if (!Object.IsValid())
		{
			continue;
		}

		if (Object->GetPackage() != GetTransientPackage())
		{
			// Object wasn't trashed, okay to re-add.
			NewObjects.Add(Object.Get());
			continue;
		}
		
		// The object is trashed, first find the original config data.
		if (UDisplayClusterConfigurationData* OuterStop = Object->GetTypedOuter<UDisplayClusterConfigurationData>())
		{
			// Build out the full name based on the original config data.
			FString FullPathName = Object->GetPathName(OuterStop);
			if (UObject* ReInstancedObject = StaticFindObject(Object->GetClass(), CurrentConfigData, *FullPathName))
			{
				// Find the current one from the new config data.
				NewObjects.Add(ReInstancedObject);
			}
		}
	}

	SelectObjects(NewObjects);
}

void FDisplayClusterConfiguratorBlueprintEditor::RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	FBlueprintEditor::RegisterTabSpawners(InTabManager);
}

void FDisplayClusterConfiguratorBlueprintEditor::SaveAsset_Execute()
{
	if (CanExportConfig() && IsExportOnSaveSet())
	{
		ExportConfig();
	}
	
	FBlueprintEditor::SaveAsset_Execute();
}

void FDisplayClusterConfiguratorBlueprintEditor::Tick(float DeltaTime)
{
	AActor* PreviewActor = GetPreviewActor();
	if (CurrentPreviewActor != PreviewActor || CurrentPreviewActor.IsStale() || PreviewActor == nullptr)
	{
		// Create or update preview actor. Parent tick does this but we want to also update the config data and refresh output mapping.
		// The preview actor can also be out of date after being reinstanced such as from a compile.
		RefreshDisplayClusterPreviewActor();
	}

	FBlueprintEditor::Tick(DeltaTime);
}

void FDisplayClusterConfiguratorBlueprintEditor::ImportConfig_Clicked()
{
	if (LoadWithOpenFileDialog())
	{
		OnConfigReloaded.Broadcast();

		// Add log
		UDisplayClusterConfigurationData* EditorData = GetEditorData();
		check(EditorData != nullptr);
		UE_LOGF(DisplayClusterConfiguratorLog, Log, "Successfully imported config with the path: %ls", *EditorData->PathToConfig);
	}
}

void FDisplayClusterConfiguratorBlueprintEditor::ExportToFile_Clicked()
{
	if (SaveWithOpenFileDialog())
	{
		// Add log
		UDisplayClusterConfigurationData* EditorData = GetEditorData();
		check(EditorData != nullptr);
		UE_LOGF(DisplayClusterConfiguratorLog, Log, "Successfully exported config with the path: %ls", *EditorData->PathToConfig);
	}
}

void FDisplayClusterConfiguratorBlueprintEditor::EditConfig_Clicked()
{
	UDisplayClusterConfigurationData* EditorData = GetEditorData();
	FPlatformProcess::LaunchFileInDefaultExternalApplication(*EditorData->PathToConfig, nullptr, ELaunchVerb::Edit);
}

bool FDisplayClusterConfiguratorBlueprintEditor::IsExportOnSaveSet() const
{
	const UDisplayClusterConfiguratorEditorSettings* Settings = GetDefault<UDisplayClusterConfiguratorEditorSettings>();
	return Settings->bExportOnSave;
}

void FDisplayClusterConfiguratorBlueprintEditor::ToggleExportOnSaveSetting()
{
	UDisplayClusterConfiguratorEditorSettings* Settings = GetMutableDefault<UDisplayClusterConfiguratorEditorSettings>();
	Settings->bExportOnSave = !Settings->bExportOnSave;
	Settings->SaveConfig();
}

void FDisplayClusterConfiguratorBlueprintEditor::ImportMPCDI_Clicked()
{
	const FString MPCDIFileDescription = LOCTEXT("MPCDIFileDescription", "MPCDI file").ToString();
	const FString MPCDIFileExtension = TEXT("*.mpcdi");
	const FString FileTypes = FString::Printf(TEXT("%s (%s)|%s"), *MPCDIFileDescription, *MPCDIFileExtension, *MPCDIFileExtension);

	TArray<FString> OpenFileNames;
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	bool bFileSelected = false;
	int32 FilterIndex = -1;

	// Open file dialog
	if (DesktopPlatform)
	{
		const void* ParentWindowHandle = FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr);

		bFileSelected = DesktopPlatform->OpenFileDialog(
			ParentWindowHandle,
			LOCTEXT("ImportMPCDIDialogTitle", "Import MPCDI").ToString(),
			FEditorDirectories::Get().GetLastDirectory(ELastDirectory::GENERIC_IMPORT),
			TEXT(""),
			FileTypes,
			EFileDialogFlags::None,
			OpenFileNames,
			FilterIndex
		);
	}

	// Load file
	if (bFileSelected)
	{
		if (OpenFileNames.Num() > 0)
		{
			const FString& FileName = OpenFileNames[0];
			FEditorDirectories::Get().SetLastDirectory(ELastDirectory::GENERIC_IMPORT, FileName);

			// The system relative path is not associated with the project directory.
			// We must convert it to the relative path of the UE project
			const FString RelativeFilePath = DisplayClusterHelpers::filesystem::GetRelativePathForConfigResource(FPaths::ConvertRelativePathToFull(FileName));

			UDisplayClusterBlueprint* Blueprint = LoadedBlueprint.Get();

			FDisplayClusterConfiguratorMPCDIImporterParams ImportParams {};
			if (FDisplayClusterConfiguratorMPCDIImporter::ImportMPCDIIntoBlueprint(RelativeFilePath, Blueprint, ImportParams))
			{
				GetEditorData()->MarkPackageDirty();
				ClusterChanged();

				// Since the blueprint component transforms are edited directly by the importer, we must
				// fully destroy and recreate the preview actor, as otherwise the engine will not propagate the
				// new values to the preview since there is now a difference between the preview and blueprint values
				DestroyPreview();
				UpdateSubobjectPreview(true);
				SubobjectEditor->UpdateTree();
			}
		}
	}
}

TStatId FDisplayClusterConfiguratorBlueprintEditor::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FDisplayClusterConfiguratorBlueprintEditor, STATGROUP_Tickables);
}

bool FDisplayClusterConfiguratorBlueprintEditor::OnRequestClose(EAssetEditorCloseReason InCloseReason)
{
	return FBlueprintEditor::OnRequestClose(InCloseReason);
}

void FDisplayClusterConfiguratorBlueprintEditor::OnClose()
{
	// When closing the blueprint editor, take the time to remove any unused host display data objects.
	if (UDisplayClusterConfigurationData* Config = GetConfig())
	{
		bool bIsDirty = LoadedBlueprint->GetOutermost()->IsDirty();
		bool bHostDataRemoved = UE::DisplayClusterConfiguratorClusterUtils::RemoveUnusedHostDisplayData(Config->Cluster);

		// If the blueprint wasn't dirty before, removing the unused host display data will have make it dirty, which is confusing to the user.
		// In this case, immediately save the host display data removal to the blueprint.
		if (!bIsDirty && CanSaveAsset() && bHostDataRemoved)
		{
			SaveAsset_Execute();
		}
	}
}

void FDisplayClusterConfiguratorBlueprintEditor::Compile()
{
	const FSelectionScope SelectionScope(this, ESelectionSource::Refresh);
	FBlueprintEditor::Compile();
}

void FDisplayClusterConfiguratorBlueprintEditor::CreateDefaultTabContents(const TArray<UBlueprint*>& InBlueprints)
{
	Super::CreateDefaultTabContents(InBlueprints);

	Inspector =
		SNew(SKismetInspector)
		.HideNameArea(true)
		.ViewIdentifier(FName("BlueprintInspector"))
		.Kismet2(SharedThis(this))
		.OnFinishedChangingProperties(FOnFinishedChangingProperties::FDelegate::CreateSP(this, &FDisplayClusterConfiguratorBlueprintEditor::OnFinishedChangingProperties))
		.ShowSectionSelector(true);
}

void FDisplayClusterConfiguratorBlueprintEditor::OnFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::OnFinishedChangingProperties(PropertyChangedEvent);
}

void FDisplayClusterConfiguratorBlueprintEditor::OnSelectionUpdated(const TArray<TSharedPtr<class FSubobjectEditorTreeNode>>& SelectedNodes)
{
	if (CurrentSelectionSource == ESelectionSource::Refresh)
	{
		return;
	}

	if (ViewportTabContent.IsValid())
	{
		TFunction<void(FName, TSharedPtr<IEditorViewportLayoutEntity>)> OnCompSelectionChangeFunc =
			[this](FName Name, TSharedPtr<IEditorViewportLayoutEntity> Entity)
		{
			TSharedRef<SDisplayClusterConfiguratorSCSEditorViewport> Viewport = StaticCastSharedRef<SDisplayClusterConfiguratorSCSEditorViewport>(Entity->AsWidget());
			Viewport->OnComponentSelectionChanged();
		};

		ViewportTabContent->PerformActionOnViewports(OnCompSelectionChangeFunc);
	}

	UBlueprint* Blueprint = GetBlueprintObj();
	check(Blueprint != nullptr && Blueprint->SimpleConstructionScript != nullptr);

	// Update the selection visualization
	if (ADisplayClusterRootActor* EditorActorInstance = Cast<ADisplayClusterRootActor>(Blueprint->SimpleConstructionScript->GetComponentEditorActorInstance()))
	{
		auto IsComponentSelected = [SelectedNodes, this](UActorComponent* InComponent) -> bool
		{
			for (const TSharedPtr<FSubobjectEditorTreeNode>& TreeNode : SelectedNodes)
			{
				const FSubobjectData* NodeData = TreeNode ? TreeNode->GetDataSource() : nullptr;
				if (NodeData)
				{
					const UActorComponent* ActorComponent = NodeData->GetObjectForBlueprint<UActorComponent>(GetBlueprintObj());
					if (TreeNode.IsValid() && ActorComponent == InComponent)
					{
						return true;
					}
				}
			}
			
			return false;
		};

		
		for (UActorComponent* Component : EditorActorInstance->GetComponents())
		{
			if (UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(Component))
			{
				if (PrimitiveComponent->GetName().EndsWith(FDisplayClusterConfiguratorUtils::GetImplSuffix()))
				{
					/*
					 *  Check for display cluster visualization components which should be highlighted on selection.
					 */
					
					if (UActorComponent* OwningComp = Cast<UActorComponent>(PrimitiveComponent->GetOuter()))
					{
						if (IsComponentSelected(OwningComp))
						{
							EditorActorInstance->SetIsSelectedInEditor(true);
							PrimitiveComponent->PushSelectionToProxy();
							EditorActorInstance->SetIsSelectedInEditor(false);
							
							continue;
						}
					}
				}

				// Always call otherwise, leaving it same as parent method.
				PrimitiveComponent->PushSelectionToProxy();
			}
		}
	}

	if (Inspector.IsValid() && CurrentSelectionSource != ESelectionSource::Ancillary)
	{
		// Clear the my blueprints selection
		if (SelectedNodes.Num() > 0)
		{
			SetUISelectionState(FBlueprintEditor::SelectionState_Components);
		}

		// Convert the selection set to an array of UObject* pointers
		FText InspectorTitle = FText::GetEmpty();
		TArray<UObject*> InspectorObjects;
		TArray<FString> SelectedComponentNames;
		bool bShowComponents = true;
		InspectorObjects.Empty(SelectedNodes.Num());
		for (FSubobjectEditorTreeNodePtrType NodePtr : SelectedNodes)
		{
			const FSubobjectData* NodeData = NodePtr ? NodePtr->GetDataSource() : nullptr;
			if (NodeData)
			{
				if (const AActor* Actor = NodeData->GetObject<AActor>())
				{
					if (const AActor* DefaultActor = NodeData->GetObjectForBlueprint<AActor>(GetBlueprintObj()))
					{
						InspectorObjects.Add(const_cast<AActor*>(DefaultActor));
					
						FString Title;
						DefaultActor->GetName(Title);
						InspectorTitle = FText::FromString(Title);
						bShowComponents = false;
					
						TryInvokingDetailsTab();
					}
				}
				else
				{
					const UActorComponent* EditableComponent = NodeData->GetObjectForBlueprint<UActorComponent>(GetBlueprintObj());
					if (EditableComponent)
					{
						InspectorTitle = FText::FromString(NodePtr->GetDisplayString());
						InspectorObjects.Add(const_cast<UActorComponent*>(EditableComponent));

						FString ComponentName = EditableComponent->GetName();
						ComponentName.RemoveFromEnd(UActorComponent::ComponentTemplateNameSuffix);
						SelectedComponentNames.Add(ComponentName);
					}

					if (ViewportTabContent.IsValid())
					{
						TSharedPtr<SEditorViewport> FirstViewport = ViewportTabContent->GetFirstViewport();
						check(FirstViewport.IsValid());
						const TSharedRef<SDisplayClusterConfiguratorSCSEditorViewport> DCViewport = StaticCastSharedRef<
							SDisplayClusterConfiguratorSCSEditorViewport>(FirstViewport->AsShared());
						TSharedPtr<SDockTab> OwnerTab = DCViewport->GetOwnerTab();
						if (OwnerTab.IsValid())
						{
							OwnerTab->FlashTab();
						}
					}
				}
			}
		}
		
		{
			// Small hack to notify Display Cluster windows of a new selection.
			// Only do this if items are being selected, not if the selection is being cleared
			if (SelectedNodes.Num())
			{
				const FSelectionScope SelectionScope(this, ESelectionSource::Internal);
				SelectObjects(InspectorObjects);
			}
		}

		// Update the details panel
		SKismetInspector::FShowDetailsOptions Options(InspectorTitle);
		Options.bShowComponents = bShowComponents;
		Inspector->ShowDetailsForObjects(InspectorObjects, Options);

		if (SelectedComponentNames.Num())
		{
			SelectAncillaryViewports(SelectedComponentNames);
		}
	}
}

void FDisplayClusterConfiguratorBlueprintEditor::OnComponentDoubleClicked(TSharedPtr<class FSubobjectEditorTreeNode> Node)
{
	TSharedPtr<SDockTab> OwnerTab = Inspector->GetOwnerTab();
	if (OwnerTab.IsValid())
	{
		GetTabManager()->TryInvokeTab(FDisplayClusterConfiguratorEditorConfigurationMode::TabID_Viewport);

		AActor* PreviewActor = GetPreviewActor();
		if (ViewportTabContent.IsValid() && Node.IsValid() && PreviewActor)
		{
			if (const USceneComponent* ComponentTemplate = Cast<USceneComponent>(Node->GetComponentTemplate()))
			{
				const bool bIsCamera = ComponentTemplate->IsA<UDisplayClusterCameraComponent>() || ComponentTemplate->IsA<UCameraComponent>();
				if (bIsCamera)
				{
					if (const FSubobjectData* NodeData = Node->GetDataSource())
					{
						if (const USceneComponent* Component = Cast<USceneComponent>(NodeData->FindComponentInstanceInActor(PreviewActor)))
						{
							TFunction<void(FName, TSharedPtr<IEditorViewportLayoutEntity>)> ViewportFunc =
								[this, Component](FName Name, TSharedPtr<IEditorViewportLayoutEntity> Entity)
								{
									const TSharedRef<SDisplayClusterConfiguratorSCSEditorViewport> Viewport =
										StaticCastSharedRef<SDisplayClusterConfiguratorSCSEditorViewport>(Entity->AsWidget());
									Viewport->GetDisplayClusterViewportClient()->SetCameraToComponent(const_cast<USceneComponent*>(Component));
								};

							ViewportTabContent->PerformActionOnViewports(ViewportFunc);
						}
					}
				}
			}
		}
	}
}

void FDisplayClusterConfiguratorBlueprintEditor::CreateDCSCSEditors()
{
	SAssignNew(ViewportTab, SDockTab);
	ViewportTabContent = MakeShared<FEditorViewportTabContent>();

	const FString LayoutId = "nDisplayViewport";

	ViewportTabContent->Initialize([this](const FAssetEditorViewportConstructionArgs InArgs)
	{
		return SNew(SDisplayClusterConfiguratorSCSEditorViewport, InArgs)
			.BlueprintEditor(SharedThis(this))
			.OwningTab(ViewportTab);
	}, ViewportTab.ToSharedRef(), LayoutId);
}

void FDisplayClusterConfiguratorBlueprintEditor::ShutdownDCSCSEditors()
{
	ViewportTabContent.Reset();
	ViewportTab.Reset();

	// Remove this blueprint editor and any registered blueprint editors that are no longer valid (as is the case when in a destructor)
	// from the list of registered blueprint editors
	EditorsRegisteredForPanelExtension.RemoveAll([this](const TWeakPtr<FDisplayClusterConfiguratorBlueprintEditor>& BlueprintEditor)
	{
		return !BlueprintEditor.IsValid() || BlueprintEditor.Pin().Get() == this;
	});
}

TSharedRef<SWidget> FDisplayClusterConfiguratorBlueprintEditor::CreateAddComponentButton()
{
	auto PerformComboAddClass = [this](TSubclassOf<UActorComponent> ComponentClass, EComponentCreateAction::Type ComponentCreateAction,
		UObject* AssetOverride) -> FSubobjectDataHandle
	{
		FSubobjectDataHandle NewComponentHandle = FSubobjectDataHandle::InvalidHandle;

		UClass* NewClass = ComponentClass;
		
		if (NewClass != nullptr)
		{
			TUniquePtr<FScopedTransaction> AddTransaction = MakeUnique<FScopedTransaction>(LOCTEXT("AddComponent", "Add Component"));
			
			FEditorDelegates::LoadSelectedAssetsIfNeeded.Broadcast();
			USelection* Selection = GEditor->GetSelectedObjects();

			bool bAddedComponent = false;

			// Rename the asset to our display names by creating a template.
			// Otherwise the DisplayCluster class name is used, not the custom nDisplay display name.
			if (AssetOverride == nullptr && NewClass->IsClassGroupName(TEXT("DisplayCluster")))
			{
				const FString DisplayName = FDisplayClusterConfiguratorUtils::FormatNDisplayComponentName(NewClass);
				AssetOverride = NewObject<UActorComponent>(GetTransientPackage(), NewClass, *DisplayName);
			}
			
			USubobjectDataSubsystem* SubobjectSystem = USubobjectDataSubsystem::Get();
			FAddNewSubobjectParams AddSubobjectParams;
			AddSubobjectParams.BlueprintContext = SubobjectEditor->GetBlueprint();
			AddSubobjectParams.NewClass = NewClass;
								
			// This adds components according to the type selected in the drop down. If the user
			// has the appropriate objects selected in the content browser then those are added,
			// else we go down the previous route of adding components by type.
			//
			// Furthermore don't try to match up assets for USceneComponent it will match lots of things and doesn't have any nice behavior for asset adds 

			FText OutFailReason;
			
			if (Selection->Num() > 0 && !AssetOverride && NewClass != USceneComponent::StaticClass())
			{
				for (FSelectionIterator ObjectIter(*Selection); ObjectIter; ++ObjectIter)
				{
					UObject* Object = *ObjectIter;
					TArray<TSubclassOf<UActorComponent>> ComponentClasses = FComponentAssetBrokerage::GetComponentsForAsset(Object);

					// if the selected asset supports the selected component type then go ahead and add it
					for (int32 ComponentIndex = 0; ComponentIndex < ComponentClasses.Num(); ComponentIndex++)
					{
						if (ComponentClasses[ComponentIndex]->IsChildOf(NewClass))
						{
							const FSubobjectDataHandle ParentHandle = SubobjectSystem->FindHandleForObject(SubobjectEditor->GetSceneRootNode()->GetDataHandle(), Object);
							if(ParentHandle.IsValid())
							{
								AddSubobjectParams.AssetOverride = Object;
								AddSubobjectParams.ParentHandle = ParentHandle;
								NewComponentHandle = SubobjectSystem->AddNewSubobject(AddSubobjectParams, OutFailReason);
								bAddedComponent = true;
								break;
							}
						}
					}
				}
			}

			if (!bAddedComponent)
			{
				// Attach this component to the override asset first, but if none is given then use the actor context			
				FSubobjectDataHandle ParentHandle = SubobjectSystem->FindHandleForObject(SubobjectEditor->GetSceneRootNode()->GetDataHandle(), AssetOverride);
	
				if(!ParentHandle.IsValid())
				{
					// If we have something selected, then we should attach it to that
					TArray<FSubobjectEditorTreeNodePtrType> SelectedTreeNodes = SubobjectEditor->GetSelectedNodes();
					if(SelectedTreeNodes.Num() > 0)
					{
						ParentHandle = SelectedTreeNodes[0]->GetDataHandle();
					}
					// Otherwise fall back to the root node
					else
					{
						const TArray<FSubobjectEditorTreeNodePtrType>& RootNodes = SubobjectEditor->GetRootNodes();
						ParentHandle = RootNodes.Num() > 0
										? RootNodes[0]->GetDataHandle()
										: FSubobjectDataHandle::InvalidHandle;					
					}
				}
	
				if(ParentHandle.IsValid())
				{
					AddSubobjectParams.AssetOverride = AssetOverride;
					AddSubobjectParams.ParentHandle = ParentHandle;
					NewComponentHandle = SubobjectSystem->AddNewSubobject(AddSubobjectParams, OutFailReason);
				}
			}
			
			if(!NewComponentHandle.IsValid())
			{
				if (OutFailReason.IsEmpty())
				{
					OutFailReason = LOCTEXT("AddComponentFailed_Generic", "Failed to add component!");
				}
				FNotificationInfo Info(OutFailReason);
				Info.Image = FAppStyle::GetBrush(TEXT("Icons.Error"));
				Info.bFireAndForget = true;
				Info.bUseSuccessFailIcons = false;
				Info.ExpireDuration = 5.0f;

				FSlateNotificationManager::Get().AddNotification(Info);
			}
			else
			{
				SubobjectEditor->UpdateTree();
				// Set focus to the newly created subobject
				if(const FSubobjectEditorTreeNodePtrType NewNode = SubobjectEditor->FindSlateNodeForHandle(NewComponentHandle))
				{
					SubobjectEditor->GetDragDropTree()->SetSelection(NewNode);
					SubobjectEditor->GetCommandList()->ExecuteAction(FGenericCommands::Get().Rename.ToSharedRef());
				}
			}
		}

		return NewComponentHandle;
	};
	
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(3.0f, 3.0f)
		.AutoWidth()
		.HAlign(HAlign_Left)
		[
			SNew(SDisplayClusterConfiguratorComponentClassCombo)
			.OnSubobjectClassSelected(FSubobjectClassSelected::CreateLambda(PerformComboAddClass))
			.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("Actor.AddComponent")))
			.ToolTipText(LOCTEXT("AddComponent_Tooltip", "Adds a new component to this actor"))
		];
}

void FDisplayClusterConfiguratorBlueprintEditor::CreateSCSEditorWrapper()
{
	SCSEditorWrapper = SAssignNew(SCSEditorWrapper, SOverlay)
		+SOverlay::Slot()
		[
			SubobjectEditor.ToSharedRef()
		]

		+SOverlay::Slot()
		.Padding(10)
		.VAlign(VAlign_Bottom)
		.HAlign(HAlign_Right)
		[
			SNew(STextBlock)
			.Visibility(EVisibility::HitTestInvisible)
			.TextStyle(FAppStyle::Get(), "Graph.CornerText")
			.Text(LOCTEXT("CornerText","STEP 1"))
		];
}

#undef LOCTEXT_NAMESPACE
