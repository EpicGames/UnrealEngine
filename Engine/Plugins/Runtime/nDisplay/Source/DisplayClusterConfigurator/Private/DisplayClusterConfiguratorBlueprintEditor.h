// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintEditorModes.h"
#include "IDisplayClusterConfiguratorBlueprintEditor.h"

class ADisplayClusterRootActor;
class UActorComponent;
class UDisplayClusterBlueprint;
class FEditorModeTools;
class FEditorViewportTabContent;
class UDisplayClusterConfigurationData;
class IDisplayClusterConfiguratorView;
class IDisplayClusterConfiguratorViewTree;
class FDisplayClusterConfiguratorViewGeneral;
class FDisplayClusterConfiguratorViewDetails;
class IDisplayClusterConfiguratorViewOutputMapping;
class FDisplayClusterConfiguratorViewOutputMapping;
class FDisplayClusterConfiguratorViewCluster;
class FDisplayClusterConfiguratorViewScene;
class FDisplayClusterConfiguratorToolbar;
class SDisplayClusterConfiguratorSCSEditorViewport;
class FSubobjectEditorTreeNode;

/**
 * nDisplay editor UI (should call functions on the subsystem or UNDisplayAssetEditor)
 */
class FDisplayClusterConfiguratorBlueprintEditor
	: public IDisplayClusterConfiguratorBlueprintEditor
{
	using Super = IDisplayClusterConfiguratorBlueprintEditor;

public:
	/** Registers any panel extension factories needed for the blueprint editor */
	static void RegisterPanelExtensionFactory();

	/** Unregisters any panel extension factories that were registered for the blueprint editor */
	static void UnregisterPanelExtensionFactory();

private:
	/** Raised when the blueprint editor's subobject editor is creating extensions next to the Add Component button */
	static TSharedRef<SWidget> OnExtendAddComponentButton(FWeakObjectPtr ExtensionContext);

	/** A list of instantiated blueprint editors that are queried when the AddComponentButton panel extension is creating widgets */
	static TArray<TWeakPtr<FDisplayClusterConfiguratorBlueprintEditor>> EditorsRegisteredForPanelExtension;

public:
	FDisplayClusterConfiguratorBlueprintEditor();
	~FDisplayClusterConfiguratorBlueprintEditor();

public:
	
	virtual void InitDisplayClusterBlueprintEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UDisplayClusterBlueprint* Blueprint);

	//~ Begin IDisplayClusterConfiguratorBlueprintEditor Interface
	virtual TArray<UObject*> GetSelectedObjects() const override;
	virtual bool IsObjectSelected(UObject* Obj) const override;
	//~ End IDisplayClusterConfiguratorBlueprintEditor Interface

	//~ Begin FEditorUndoClient interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
	virtual void RefreshEditors(ERefreshBlueprintEditorReason::Type Reason) override;
	//~ End FEditorUndoClient interface

	virtual void SelectObjects(TArray<UObject*>& InSelectedObjects, bool bFullRefresh = false);
	virtual void SelectAncillaryComponents(const TArray<FString>& ComponentNames);
	virtual void SelectAncillaryViewports(const TArray<FString>& ComponentNames);

	virtual UDisplayClusterConfigurationData* GetEditorData() const;
	virtual FDelegateHandle RegisterOnConfigReloaded(const FOnConfigReloadedDelegate& Delegate);
	virtual void UnregisterOnConfigReloaded(FDelegateHandle DelegateHandle);
	virtual FDelegateHandle RegisterOnObjectSelected(const FOnObjectSelectedDelegate& Delegate);
	virtual void UnregisterOnObjectSelected(FDelegateHandle DelegateHandle);
	virtual FDelegateHandle RegisterOnInvalidateViews(const FOnInvalidateViewsDelegate& Delegate);
	virtual void UnregisterOnInvalidateViews(FDelegateHandle DelegateHandle);
	virtual FDelegateHandle RegisterOnClusterChanged(const FOnClusterChangedDelegate& Delegate);
	virtual void UnregisterOnClusterChanged(FDelegateHandle DelegateHandle);
	
	virtual void InvalidateViews();
	virtual void ClusterChanged(bool bStructureChange = false);
	virtual void ClearViewportSelection();

	/** Retrieve the config data from the CDO. */
	virtual UDisplayClusterConfigurationData* GetConfig() const;

	/** Retrieve the CDO being edited. */
	ADisplayClusterRootActor* GetDefaultRootActor() const;
	
	virtual TSharedRef<IDisplayClusterConfiguratorViewOutputMapping> GetViewOutputMapping() const;
	virtual TSharedRef<IDisplayClusterConfiguratorViewTree> GetViewCluster() const;

	TSharedPtr<SDockTab> GetViewportTab() const { return ViewportTab; }
	TSharedPtr<FEditorViewportTabContent> GetViewportTabContent() const { return ViewportTabContent; }
	
	TSharedPtr<SWidget> GetSCSEditorWrapper() const { return SCSEditorWrapper; }

	TSharedPtr<FDisplayClusterConfiguratorToolbar> GetConfiguratorToolbar() const { return ConfiguratorToolbar; }

	/** Returns mode tools used by this editor instance */
	TSharedRef<FEditorModeTools> GetModeTools() const;

	/** Syncs shared properties between viewports. */
	void SyncViewports();

	/** Make sure the blueprint preview actor is up to date. */
	void RefreshDisplayClusterPreviewActor();

	/** Restores previously open documents. */
	void RestoreLastEditedState();

	/** Updates the root actor's xform components to match the editor settings. */
	void UpdateXformGizmos();

private:
	TWeakObjectPtr<AActor> CurrentPreviewActor;
	
	FDelegateHandle UpdateOutputMappingHandle;

public:
	// Load with OpenFileDialog
	bool LoadWithOpenFileDialog();

	// Load from specified file
	bool LoadFromFile(const FString& FilePath);

	// Save to the same file the config data was read from
	bool ExportConfig();

	// Verifies config can be saved.
	bool CanExportConfig() const;
	
	// Save to a specified file
	bool SaveToFile(const FString& FilePath);

	// Save with SaveFileDialog
	bool SaveWithOpenFileDialog();

protected:

	//~ Begin IAssetEditorInstance Interface
	virtual FName GetEditorName() const override { return TEXT("DisplayClusterConfigurator"); }
	//~ End IAssetEditorInstance Interface

	//~ Begin FBlueprintEditor Interface
	virtual void RegisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;
	virtual void SaveAsset_Execute() override;
	virtual void Tick(float DeltaTime) override;
	virtual bool IsTickable() const override { return true; }
	virtual TStatId GetStatId() const override;
	virtual bool OnRequestClose(EAssetEditorCloseReason InCloseReason) override;
	virtual void OnClose() override;
	virtual void Compile() override;
	virtual void CreateDefaultTabContents(const TArray<UBlueprint*>& InBlueprints) override;
	virtual void OnFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End FBlueprintEditor Interface

	// SSCS Implementation
	/** Delegate invoked when the selection is changed in the SCS editor widget */
	virtual void OnSelectionUpdated(const TArray<TSharedPtr<class FSubobjectEditorTreeNode>>& SelectedNodes) override;

	/** Delegate invoked when an item is double clicked in the SCS editor widget */
	virtual void OnComponentDoubleClicked(TSharedPtr<class FSubobjectEditorTreeNode> Node) override;

	friend struct FDisplayClusterSCSViewportSummoner;
	void CreateDCSCSEditors();
	void ShutdownDCSCSEditors();

	TSharedRef<SWidget> CreateAddComponentButton();

	void CreateSCSEditorWrapper();

public:
	enum class ESelectionSource
	{
		External,
		Internal,
		Ancillary,
		Refresh
	};

	struct FSelectionScope
	{
		FSelectionScope(FDisplayClusterConfiguratorBlueprintEditor* Editor, ESelectionSource ScopedSelectionSource) :
			SourceProperty(Editor->CurrentSelectionSource),
			PreviousValue(Editor->CurrentSelectionSource)
		{
			SourceProperty = ScopedSelectionSource;
		}

		~FSelectionScope()
		{
			SourceProperty = PreviousValue;
		}

		ESelectionSource& SourceProperty;
		ESelectionSource PreviousValue;
	};
	friend FSelectionScope;

private:
	/** Keeps track of the current source of the selection changes that caused OnSelectionUpdated to be invoked. */
	ESelectionSource CurrentSelectionSource = ESelectionSource::External;
	// ~End of SSCS Implementation

protected:
	void CreateWidgets();

	void ExtendMenu();

	void ExtendToolbar();

	void OnPostCompiled(UBlueprint* InBlueprint);

private:
	//~ Begin UI command handlers
	void ImportConfig_Clicked();

	void ExportToFile_Clicked();

	void EditConfig_Clicked();

	bool IsExportOnSaveSet() const;
	void ToggleExportOnSaveSetting();

	void ImportMPCDI_Clicked();
	//~ End UI command handlers

	void OnReadOnlyChanged(bool bReadOnly);
	void OnRenameVariable(UBlueprint* Blueprint, UClass* VariableClass, const FName& OldVariableName, const FName& NewVariableName);
	void OnFocusChanged(const FFocusEvent& FocusEvent, const FWeakWidgetPath& OldFocusedWidgetPath, const TSharedPtr<SWidget>& OldFocusedWidget, const FWidgetPath& NewFocusedWidgetPath, const TSharedPtr<SWidget>& NewFocusedWidget);

	void BindCommands();

private:
	TSharedPtr<FDisplayClusterConfiguratorViewOutputMapping> ViewOutputMapping;
	TSharedPtr<FDisplayClusterConfiguratorViewCluster> ViewCluster;

	/** Owner of the viewport. */
	TSharedPtr<SDockTab> ViewportTab;
	/* Tracking the active viewports in this editor. */
	TSharedPtr<FEditorViewportTabContent> ViewportTabContent;

	TSharedPtr<SWidget> SCSEditorWrapper;

	TSharedPtr<FExtender> MenuExtender;
	TSharedPtr<FExtender> ToolbarExtender;

	/** A mode tools instance shared among viewports */
	TSharedRef<FEditorModeTools> SharedModeTools;

	TSharedPtr<FDisplayClusterConfiguratorToolbar> ConfiguratorToolbar;
	
	FOnConfigReloaded OnConfigReloaded;

	/** Delegate for when an item is selected */
	FOnObjectSelected OnObjectSelected;

	/** View invalidation delegate */
	FOnInvalidateViews OnInvalidateViews;

	FOnClearViewportSelection OnClearViewportSelection;

	/** Delegate which is raised when the cluster configuration is changed. */
	FOnClusterChanged OnClusterChanged;

	TArray<TWeakObjectPtr<UObject>> SelectedObjects;

	/** The currently loaded blueprint. */
	TWeakObjectPtr<UDisplayClusterBlueprint> LoadedBlueprint;

	FDelegateHandle RenameVariableHandle;
	FDelegateHandle FocusChangedHandle;
};