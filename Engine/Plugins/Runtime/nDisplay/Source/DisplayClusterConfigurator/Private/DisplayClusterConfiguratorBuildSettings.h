// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// It's a viewer in 4.26
#define DISPLAY_CLUSTER_CONFIGURATOR_BUILD_VIEWER