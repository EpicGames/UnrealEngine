// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfiguratorCommands.h"

#define LOCTEXT_NAMESPACE "DisplayClusterConfiguratorCommands"


void FDisplayClusterConfiguratorCommands::RegisterCommands()
{
	UI_COMMAND(Import, "Import", "Import an nDisplay config. This may overwrite components. If you need to reimport it is recommended to use the reimport options of the asset context menu", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(Export, "Export", "Export to an nDisplay config. Requires a primary cluster node set. This does not export all components and is meant for launching the cluster from the command line", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(EditConfig, "EditConfig", "Edit config file with text editor", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ExportConfigOnSave, "Export on Save", "Export to nDisplay config automatically on save. Requires a config previously exported", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ImportMPCDI, "Import MPCDI", "Import an MPCDI configuration", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(AddNewClusterNode, "Add New Cluster Node", "Adds a new cluster node to the cluster config", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(AddNewViewport, "Add New Viewport", "Adds a new viewport to the cluster node", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(ResetCamera, "Reset Camera", "Resets the camera to focus on the mesh", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ShowFloor, "Show Floor", "Toggles a ground mesh for collision", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ShowGrid, "Show Grid", "Toggles the grid", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ShowOrigin, "Show World Origin", "Display the exact world origin for nDisplay", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(EnableAA, "Enable AA", "Enable anti aliasing in the preview window", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ShowPreview, "Show Projection Preview", "Show a projection preview when applicable", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(Show3DViewportNames, "Show Viewport Names", "Shows the viewport names in 3d space", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleShowXformGizmos, "Show Xform Gizmos", "Shows the Xform component gizmos", EUserInterfaceActionType::ToggleButton, FInputChord());
}

#undef LOCTEXT_NAMESPACE
