// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"
#include "Styling/AppStyle.h"


class FDisplayClusterConfiguratorCommands 
	: public TCommands<FDisplayClusterConfiguratorCommands>
{
public:
	FDisplayClusterConfiguratorCommands()
		: TCommands<FDisplayClusterConfiguratorCommands>(TEXT("DisplayClusterConfigurator"), 
			NSLOCTEXT("Contexts", "DisplayClusterConfigurator", "Display Cluster Configurator"), NAME_None, FAppStyle::GetAppStyleSetName())
	{ }

	virtual void RegisterCommands() override;

public:
	TSharedPtr<FUICommandInfo> Import;
	TSharedPtr<FUICommandInfo> Export;
	TSharedPtr<FUICommandInfo> EditConfig;
	TSharedPtr<FUICommandInfo> ExportConfigOnSave;
	TSharedPtr<FUICommandInfo> ImportMPCDI;

	// Cluster Configuration commands
	TSharedPtr<FUICommandInfo> AddNewClusterNode;
	TSharedPtr<FUICommandInfo> AddNewViewport;

	// Viewport Preview commands
	TSharedPtr<FUICommandInfo> ResetCamera;
	TSharedPtr<FUICommandInfo> ShowFloor;
	TSharedPtr<FUICommandInfo> ShowGrid;
	TSharedPtr<FUICommandInfo> ShowOrigin;
	TSharedPtr<FUICommandInfo> EnableAA;
	TSharedPtr<FUICommandInfo> ShowPreview;
	TSharedPtr<FUICommandInfo> Show3DViewportNames;
	TSharedPtr<FUICommandInfo> ToggleShowXformGizmos;
};
