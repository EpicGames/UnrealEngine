// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfiguratorCompiler.h"
#include "DisplayClusterConfiguratorUtils.h"

#include "DisplayClusterConfigurationTypes.h"

#include "DisplayClusterRootActor.h"
#include "Blueprints/DisplayClusterBlueprint.h"
#include "Blueprints/DisplayClusterBlueprintGeneratedClass.h"
#include "Components/DisplayClusterCameraComponent.h"
#include "Components/DisplayClusterICVFXCameraComponent.h"
#include "Render/DisplayDevice/Components/DisplayClusterDisplayDeviceBaseComponent.h"
#include "Camera/CameraComponent.h"

#include "Engine/SCS_Node.h"
#include "Engine/SimpleConstructionScript.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Kismet2/KismetReinstanceUtilities.h"
#include "ObjectTools.h"

#define LOCTEXT_NAMESPACE "DisplayClusterConfiguratorCompiler"


bool FDisplayClusterConfiguratorKismetCompiler::CanCompile(const UBlueprint* Blueprint)
{
	return Blueprint->IsA<UDisplayClusterBlueprint>();
}

void FDisplayClusterConfiguratorKismetCompiler::Compile(UBlueprint* Blueprint,
	const FKismetCompilerOptions& CompileOptions, FCompilerResultsLog& Results)
{
	FDisplayClusterConfiguratorKismetCompilerContext Compiler(Blueprint, Results, CompileOptions);
	Compiler.Compile();
}

bool FDisplayClusterConfiguratorKismetCompiler::GetBlueprintTypesForClass(UClass* ParentClass,
	UClass*& OutBlueprintClass, UClass*& OutBlueprintGeneratedClass) const
{
	if (ParentClass && ParentClass->IsChildOf<ADisplayClusterRootActor>())
	{
		OutBlueprintClass = UDisplayClusterBlueprint::StaticClass();
		OutBlueprintGeneratedClass = UDisplayClusterBlueprintGeneratedClass::StaticClass();
		return true;
	}

	return false;
}

FDisplayClusterConfiguratorKismetCompilerContext::FDisplayClusterConfiguratorKismetCompilerContext(UBlueprint* InBlueprint, FCompilerResultsLog& InMessageLog,
	const FKismetCompilerOptions& InCompilerOptions) : FKismetCompilerContext(InBlueprint, InMessageLog,
	                                                                          InCompilerOptions), DCGeneratedBP(nullptr)
{
}

void FDisplayClusterConfiguratorKismetCompilerContext::SpawnNewClass(const FString& NewClassName)
{
	DCGeneratedBP = FindObject<UDisplayClusterBlueprintGeneratedClass>(Blueprint->GetOutermost(), *NewClassName);

	if (DCGeneratedBP == nullptr)
	{
		DCGeneratedBP = NewObject<UDisplayClusterBlueprintGeneratedClass>(Blueprint->GetOutermost(), *NewClassName, RF_Public | RF_Transactional);
	}
	else
	{
		// Already existed, but wasn't linked in the Blueprint yet due to load ordering issues
		FBlueprintCompileReinstancer::Create(DCGeneratedBP);
	}
	NewClass = DCGeneratedBP;
}

void FDisplayClusterConfiguratorKismetCompilerContext::OnNewClassSet(UBlueprintGeneratedClass* ClassToUse)
{
	DCGeneratedBP = CastChecked<UDisplayClusterBlueprintGeneratedClass>(ClassToUse);
}

void FDisplayClusterConfiguratorKismetCompilerContext::PreCompile()
{
	Super::PreCompile();
	
	const UDisplayClusterBlueprint* DCBlueprint = CastChecked<UDisplayClusterBlueprint>(Blueprint);
	if (const UDisplayClusterConfigurationData* ConfigData = DCBlueprint->GetConfig())
	{
		ForEachObjectWithOuter(ConfigData, [this](UObject* Child)
		{
			if (UDisplayClusterConfigurationData_Base* BaseNode = Cast<UDisplayClusterConfigurationData_Base>(Child))
			{
				BaseNode->OnPreCompile(MessageLog);
			}
		});
	}

	ValidateConfiguration();
}

void FDisplayClusterConfiguratorKismetCompilerContext::PostCompile()
{
	Super::PostCompile();

	UDisplayClusterBlueprint* DCBlueprint = CastChecked<UDisplayClusterBlueprint>(Blueprint);
	check(DCBlueprint->GeneratedClass);

	// The parent class (DCRA) may have NotPlaceable flag set. This prevents the child blueprints from
	// placing into a world. To avoid this limitation, we explicitly clear it because we know it's safe.
	DCBlueprint->GeneratedClass->ClassFlags &= ~CLASS_NotPlaceable;
}

void FDisplayClusterConfiguratorKismetCompilerContext::SaveSubObjectsFromCleanAndSanitizeClass(
	FSubobjectCollection& SubObjectsToSave, UBlueprintGeneratedClass* ClassToClean)
{
	OldSubObjects.Empty();
	FKismetCompilerContext::SaveSubObjectsFromCleanAndSanitizeClass(SubObjectsToSave, ClassToClean);

	const UDisplayClusterBlueprint* DCBlueprint = CastChecked<UDisplayClusterBlueprint>(Blueprint);
	if (UDisplayClusterConfigurationData* ConfigData = DCBlueprint->GetConfig())
	{
		SubObjectsToSave.AddObject(ConfigData);

		// Find all subobjects owned by this object. Same as how `AddObject` above works.
		{
			OldSubObjects.AddUnique(ConfigData);
			ForEachObjectWithOuter(ConfigData, [this](UObject* Child)
			{
				OldSubObjects.AddUnique(Child);
			});
		}
	}
}

void FDisplayClusterConfiguratorKismetCompilerContext::CopyTermDefaultsToDefaultObject(UObject* DefaultObject)
{
	Super::CopyTermDefaultsToDefaultObject(DefaultObject);

	UDisplayClusterBlueprint* DCBlueprint = CastChecked<UDisplayClusterBlueprint>(Blueprint);
	if (DCBlueprint->HasAnyFlags(RF_NeedPostLoad | RF_NeedPostLoadSubobjects | RF_NeedInitialization))
	{
		return;
	}

	ADisplayClusterRootActor* RootActor = CastChecked<ADisplayClusterRootActor>(DefaultObject);
	if (Blueprint->bIsNewlyCreated)
	{
		RootActor->PreviewNodeId = DisplayClusterConfigurationStrings::gui::preview::PreviewNodeNone;
	}
	
	// Restore the old config data back to the CDO to preserve transaction history.
	// Ideally transaction history would persist through sub-object re-instancing, but that requires
	// the sub-objects are marked as default sub objects which is incompatible for our use.
	if (OldSubObjects.Num() > 0)
	{
		UDisplayClusterConfigurationData* OldConfigData = CastChecked<UDisplayClusterConfigurationData>(OldSubObjects[0]);
		UDisplayClusterConfigurationData* NewConfigData = RootActor->CurrentConfigData;

		TMap<UObject*, UObject*> NewToOldSubObjects;
		{
			NewToOldSubObjects.Reserve(OldSubObjects.Num());
			NewToOldSubObjects.Add(NewConfigData, OldConfigData);
			
			ForEachObjectWithOuter(NewConfigData, [&](UObject* NewSubObject)
			{
				if (UObject** MatchingOldSubObject = OldSubObjects.FindByPredicate([NewSubObject](const UObject* OldSubObject)
				{
					return OldSubObject->GetName() == NewSubObject->GetName() &&
						OldSubObject->GetClass() == NewSubObject->GetClass();
				}))
				{
					NewToOldSubObjects.Add(NewSubObject, *MatchingOldSubObject);
					NewSubObject->SetFlags(RF_Transient);
					NewSubObject->ClearFlags(RF_Transactional);
					FLinkerLoad::InvalidateExport(NewSubObject);
				}
			});
		}
		
		const ERenameFlags RenFlags = REN_DontCreateRedirectors | REN_NonTransactional | REN_DoNotDirty | REN_AllowPackageLinkerMismatch;

		// Rename our new config data (along with all new sub-objects) to the transient package.
		{
			NewConfigData->SetFlags(RF_Transient);
			NewConfigData->ClearFlags(RF_Transactional);

            // Rename will remove the renamed object's linker when moving to a new package so invalidate the export beforehand
			FLinkerLoad::InvalidateExport(NewConfigData);
			NewConfigData->Rename(nullptr, GetTransientPackage(), RenFlags);
		}

		// Rename our old config data (along with all sub-objects) to our new CDO.
		{
			OldConfigData->Rename(nullptr, DefaultObject, RenFlags);

			// Set the property value directly to avoid going through our custom load logic.
			const FObjectProperty* CurrentConfigProperty =
				CastFieldChecked<FObjectProperty>(RootActor->GetClass()->FindPropertyByName(RootActor->GetCurrentConfigDataMemberName()));

			void* PropertyAddress = CurrentConfigProperty->ContainerPtrToValuePtr<UObject>(DefaultObject);
			CurrentConfigProperty->SetObjectPropertyValue(PropertyAddress, OldConfigData);
		}
		
		if (GEditor)
		{
			GEditor->NotifyToolsOfObjectReplacement(NewToOldSubObjects);
		}

		// Validate
		for (UObject* OldSubObject : OldSubObjects)
		{
			if (OldSubObject->GetTypedOuter(ADisplayClusterRootActor::StaticClass()) != DefaultObject)
			{
				MessageLog.Error(TEXT("Could not preserve original sub-object @@. Transaction history may be lost."), OldSubObject);
			}
		}

		// Fix/hack for UE-136629: Containers (specifically hash based) may need to be reset.
		// The internal index of the CDO container may mismatch the instanced index because we aren't re-instantiating our
		// sub-objects so we can maintain transaction history. Once in this state it takes an asset reload to correct.
		// 
		// If the index is off and a user adds a new element, it may inadvertently overwrite the wrong index on the instance.
		// This occurs when we delete a cluster node, compile, then add a new cluster node in. The map is modified via handle,
		// AddItem() is called, then the keypair value is set. AddItem() will add the value to the CDO, but in the wrong index.
		// When UE propagates this to the instances, it will add it there in the correct index. When the value is set later the wrong
		// index of the instance is used since it propagates based on the CDO index.
		//
		// It's not clear if this issue would be better solved in the engine's internal map/set propagation or if there
		// is a better approach for switching out sub-objects on the CDO before the compile process finishes.

		for (UObject* OldSubObject : OldSubObjects)
		{
			for (const FProperty* Property : TFieldRange<FProperty>(OldSubObject->GetClass()))
			{
				// Is container
				if (Property->IsA<FArrayProperty>() ||
					Property->IsA<FSetProperty>() ||
					Property->IsA<FMapProperty>())
				{
					FString Value;
					Property->ExportTextItem_InContainer(Value, OldSubObject, nullptr, nullptr, 0, nullptr);
					Property->ImportText_InContainer(*Value, OldSubObject, OldSubObject, 0);
				}
			}
		}
	}
}

void FDisplayClusterConfiguratorKismetCompilerContext::ValidateConfiguration()
{
	UDisplayClusterBlueprint* DCBlueprint = CastChecked<UDisplayClusterBlueprint>(Blueprint);
	if (Blueprint->bIsNewlyCreated)
	{
		return;
	}
	
	UDisplayClusterConfigurationData* BlueprintData = DCBlueprint->GetOrLoadConfig();
	if (!BlueprintData)
	{
		MessageLog.Error(*LOCTEXT("NoConfigError", "Critical Error: Configuration data not found!").ToString());
		return;
	}

	if (!BlueprintData->Cluster)
	{
		MessageLog.Error(*LOCTEXT("NoClusterError", "No cluster information found!").ToString());
		return;
	}

	if (BlueprintData->Cluster->Nodes.Num() == 0)
	{
		MessageLog.Warning(*LOCTEXT("NoClusterNodesWarning", "No cluster nodes found. Please add a cluster node.").ToString());
		return;
	}
	
	if (!FDisplayClusterConfiguratorUtils::IsPrimaryNodeInConfig(BlueprintData))
	{
		MessageLog.Error(*LOCTEXT("NoPrimaryNodeError", "Primary cluster node not set. Please set a primary node.").ToString());
	}

	TMap<FString, UDisplayClusterConfigurationViewport*> ViewportsByName;
	
	for (const TPair<FString, TObjectPtr<UDisplayClusterConfigurationClusterNode>>& ClusterNode : BlueprintData->Cluster->Nodes)
	{
		// This is a temporary workaround to prevent crashes while dealing with DCRA BP hierarchies. It looks like
		// we don't have a solution to propagate data changes to the assets that have not been instantiated.
		if (ClusterNode.Key.IsEmpty() || !ClusterNode.Value)
		{
			continue;
		}

		// Validate viewports
		if (ClusterNode.Value->Viewports.Num() > 0)
		{
			for (const TPair<FString, TObjectPtr<UDisplayClusterConfigurationViewport>>& Viewport : ClusterNode.Value->Viewports)
			{
				// This is a temporary workaround to prevent crashes while dealing with DCRA BP hierarchies. It looks like
				// we don't have a solution to propagate data changes to the assets that have not been instantiated.
				if (Viewport.Key.IsEmpty() || !Viewport.Value)
				{
					continue;
				}

				const FString ViewportName = Viewport.Value->GetName();

				// Check that no two viewports have the same name
				if (UDisplayClusterConfigurationViewport** ExistingViewport = ViewportsByName.Find(ViewportName))
				{
					MessageLog.Error(
						*LOCTEXT("DuplicateViewportNameError", "Viewport @@ uses the same name as viewport @@.").ToString(),
						Viewport.Value,
						*ExistingViewport
					);
				}
				else
				{
					ViewportsByName.Add(ViewportName, Viewport.Value);

					// Check display device
					if (!Viewport.Value->DisplayDeviceName.IsEmpty())
					{
						// Only check if bp class has been generated and component owner instantiated. Otherwise native
						// components can't be easily found since the CDO has already been cleared at this point.
						if (UBlueprintGeneratedClass* GenClass = DCBlueprint->GetGeneratedClass())
						{
							if (AActor* ActorOwner = GenClass->SimpleConstructionScript->GetComponentEditorActorInstance())
							{
								bool bDisplayDeviceFound = false;
								// blueprint component
								if (const USCS_Node* SCSNode = GenClass->SimpleConstructionScript->FindSCSNode(*Viewport.Value->DisplayDeviceName))
								{
									if (Cast<UDisplayClusterDisplayDeviceBaseComponent>(SCSNode->GetActualComponentTemplate(GenClass)))
									{
										bDisplayDeviceFound = true;
									}
								}

								// native component
								if (!bDisplayDeviceFound
									&& FindObjectFast<UDisplayClusterDisplayDeviceBaseComponent>(ActorOwner, *Viewport.Value->DisplayDeviceName))
								{
									bDisplayDeviceFound = true;
								}

								if (!bDisplayDeviceFound)
								{
									MessageLog.Error(
										*LOCTEXT("DisplayDeviceNotFound", "Could not find display device component @@ for viewport @@.").ToString(),
										*Viewport.Value->DisplayDeviceName,
										Viewport.Value
									);
								}
							}
						}
					}
				}
				
				if (Viewport.Value->ProjectionPolicy.Type.IsEmpty())
				{
					MessageLog.Warning(*LOCTEXT("NoPolicyError", "No projection policy assigned to viewport @@.").ToString(), Viewport.Value);
				}
			}
		}
		else
		{
			// It's allowed not to have any viewports when rendering offscreen. For example, we're going to render
			// ICVFX inner views only which don't require any viewports. In other cases continue throwing a warning.
			if (!ClusterNode.Value->bRenderHeadless)
			{
				MessageLog.Warning(*LOCTEXT("NoViewportsError", "Node @@ has no viewports. Please add a viewport.").ToString(), *ClusterNode.Key);
			}
		}

		// Warn user about fullscreen + headless
		if (ClusterNode.Value->bRenderHeadless && ClusterNode.Value->bIsFullscreen)
		{
			MessageLog.Warning(*LOCTEXT("HeadlessFullscreen", "Node @@ has both 'headless' and 'fullscreen' flags set. It's not forbidden but make sure it's intentional as the backbuffer will have the desktop's size.").ToString(), *ClusterNode.Key);
		}
	}

	if (DCBlueprint && DCBlueprint->SimpleConstructionScript)
	{
		const TArray<USCS_Node*>& SCSNodes = Blueprint->SimpleConstructionScript->GetAllNodes();

		// Iterate over all SCS nodes
		for (const USCS_Node* const SCSNode : SCSNodes)
		{
			if (SCSNode && SCSNode->ComponentTemplate)
			{
				// Filter ICVFX camera components
				const UActorComponent* const ComponentTemplate = SCSNode->ComponentTemplate;
				if (const UDisplayClusterICVFXCameraComponent* const ICVFXCameraComponent = Cast<UDisplayClusterICVFXCameraComponent>(ComponentTemplate))
				{
					// Validate media settings
					const FDisplayClusterConfigurationMediaICVFX& MediaSettings = ICVFXCameraComponent->CameraSettings.RenderSettings.Media;
					if (MediaSettings.bEnable && MediaSettings.SplitType == EDisplayClusterConfigurationMediaSplitType::UniformTiles)
					{
						const int32 TilesCount = MediaSettings.TiledSplitLayout.X * MediaSettings.TiledSplitLayout.Y;
						if (TilesCount < 2)
						{
							MessageLog.Error(*LOCTEXT("MediaInvalidTilesLayout", "Camera @@ has invalid split layout.").ToString(), ICVFXCameraComponent);
						}
						else
						{
							for (int32 GroupIdx = 0; GroupIdx < MediaSettings.TiledMediaInputGroups.Num(); ++GroupIdx)
							{
								const FDisplayClusterConfigurationMediaTiledInputGroup& TiledInputGroup = MediaSettings.TiledMediaInputGroups[GroupIdx];
								if (TiledInputGroup.Tiles.Num() > TilesCount)
								{
									MessageLog.Error(*FText::Format(LOCTEXT("MediaInvalidTilesNum", "Camera @@ has invalid amount of tiles in the input group {0}."), FText::AsNumber(GroupIdx)).ToString(), ICVFXCameraComponent);
								}
							}
						}
					}
				}

			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
