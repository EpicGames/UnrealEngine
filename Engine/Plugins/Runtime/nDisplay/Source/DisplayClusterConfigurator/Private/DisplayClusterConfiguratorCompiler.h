// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "KismetCompiler.h"
#include "KismetCompilerModule.h"

class UDisplayClusterBlueprint;

class FDisplayClusterConfiguratorKismetCompiler : public IBlueprintCompiler
{
public:
	/** IBlueprintCompiler interface */
	virtual bool CanCompile(const UBlueprint* Blueprint) override;
	virtual void Compile(UBlueprint* Blueprint, const FKismetCompilerOptions& CompileOptions, FCompilerResultsLog& Results) override;
	virtual bool GetBlueprintTypesForClass(UClass* ParentClass, UClass*& OutBlueprintClass, UClass*& OutBlueprintGeneratedClass) const override;
	/** ~IBlueprintCompiler interface */
};

class FDisplayClusterConfiguratorKismetCompilerContext : public FKismetCompilerContext
{
	using Super = FKismetCompilerContext;

public:
	FDisplayClusterConfiguratorKismetCompilerContext(UBlueprint* InBlueprint,
		FCompilerResultsLog& InMessageLog, const FKismetCompilerOptions& InCompilerOptions);

protected:
	/** FKismetCompilerContext interface */
	virtual void SpawnNewClass(const FString& NewClassName) override;
	virtual void OnNewClassSet(UBlueprintGeneratedClass* ClassToUse) override;
	virtual void PreCompile() override;
	virtual void PostCompile() override;
	virtual void SaveSubObjectsFromCleanAndSanitizeClass(FSubobjectCollection& SubObjectsToSave, UBlueprintGeneratedClass* ClassToClean) override;
	virtual void CopyTermDefaultsToDefaultObject(UObject* DefaultObject) override;
	/** ~FKismetCompilerContext interface */

	/**
	 * Preform pre compile time validation on Display Cluster data.
	 */
	void ValidateConfiguration();

private:
	class UDisplayClusterBlueprintGeneratedClass* DCGeneratedBP;
	/** Sub-objects which should survive a compile. */
	TArray<UObject*> OldSubObjects;
};