// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfiguratorEditorMode.h"

#include "BlueprintEditorTabs.h"
#include "DisplayClusterConfiguratorTabSpawners.h"
#include "DisplayClusterConfiguratorBlueprintEditor.h"
#include "SBlueprintEditorToolbar.h"
#include "Tools/BaseAssetToolkit.h"

#include "ToolMenu.h"
#include "HAL/ConsoleManager.h"
#include "Views/DisplayClusterConfiguratorToolbar.h"

#define LOCTEXT_NAMESPACE "DisplayClusterConfiguratorEditorMode"


const FName FDisplayClusterEditorModes::DisplayClusterEditorName("DisplayClusterEditor");
const FName FDisplayClusterEditorModes::DisplayClusterEditorConfigurationMode("Configuration");
const FName FDisplayClusterEditorModes::DisplayClusterEditorGraphMode("Graph");

const FName FDisplayClusterConfiguratorBlueprintModeBase::TabID_Log(TEXT("DisplayClusterConfiguratorTab_Log"));
const FName FDisplayClusterConfiguratorBlueprintModeBase::TabID_OutputMapping(TEXT("DisplayClusterConfiguratorTab_OutputMapping"));
const FName FDisplayClusterConfiguratorBlueprintModeBase::TabID_Scene(TEXT("DisplayClusterConfiguratorTab_Scene"));
const FName FDisplayClusterConfiguratorBlueprintModeBase::TabID_Cluster(TEXT("DisplayClusterConfiguratorTab_Cluster"));
const FName FDisplayClusterConfiguratorBlueprintModeBase::TabID_Viewport(TEXT("DisplayClusterConfiguratorTab_Viewport"));


FDisplayClusterConfiguratorBlueprintModeBase::FDisplayClusterConfiguratorBlueprintModeBase(
	TSharedPtr<FDisplayClusterConfiguratorBlueprintEditor> EditorIn, FName EditorModeIn) : FBlueprintEditorApplicationMode(EditorIn,
		EditorModeIn, FDisplayClusterEditorModes::GetLocalizedMode, false)
{
	Editor = EditorIn;
}

FDisplayClusterConfiguratorEditorConfigurationMode::FDisplayClusterConfiguratorEditorConfigurationMode(
	TSharedPtr<FDisplayClusterConfiguratorBlueprintEditor> EditorIn) : FDisplayClusterConfiguratorBlueprintModeBase(EditorIn,
		FDisplayClusterEditorModes::DisplayClusterEditorConfigurationMode)
{
	TabLayout = BuildDefaultLayout(FString(TEXT("DisplayClusterConfigurator_v1.1")));

	EditorTabFactories.RegisterFactory(MakeShared<FDisplayClusterViewClusterSummoner>(EditorIn));
	EditorTabFactories.RegisterFactory(MakeShared<FDisplayClusterViewOutputMappingSummoner>(EditorIn));
	EditorTabFactories.RegisterFactory(MakeShared<FDisplayClusterSCSViewportSummoner>(EditorIn));
	EditorTabFactories.RegisterFactory(MakeShared<FDisplayClusterSCSSummoner>(EditorIn));
	
	ToolbarExtender = MakeShareable(new FExtender);

	if (UToolMenu* Toolbar = EditorIn->RegisterModeToolbarIfUnregistered(GetModeName()))
	{
		EditorIn->GetToolbarBuilder()->AddCompileToolbar(Toolbar);
		EditorIn->GetToolbarBuilder()->AddBlueprintGlobalOptionsToolbar(Toolbar);
		EditorIn->GetToolbarBuilder()->AddDebuggingToolbar(Toolbar);
	}

	EditorIn->GetConfiguratorToolbar()->AddModesToolbar(ToolbarExtender);
}

void FDisplayClusterConfiguratorEditorConfigurationMode::RegisterTabFactories(TSharedPtr<FTabManager> InTabManager)
{
	TSharedPtr<FBlueprintEditor> BP = Editor.Pin();

	BP->RegisterToolbarTab(InTabManager.ToSharedRef());

	// Mode-specific setup
	BP->PushTabFactories(CoreTabFactories);
	BP->PushTabFactories(EditorTabFactories);
	BP->PushTabFactories(BlueprintEditorOnlyTabFactories);
	BP->PushTabFactories(BlueprintEditorTabFactories);
}

void FDisplayClusterConfiguratorEditorConfigurationMode::PostActivateMode()
{
	// Reopen any documents that were open when the blueprint was last saved
	TSharedPtr<FDisplayClusterConfiguratorBlueprintEditor> BP = StaticCastSharedPtr<FDisplayClusterConfiguratorBlueprintEditor>(MyBlueprintEditor.Pin());
	BP->RestoreLastEditedState();
	BP->SetupViewForBlueprintEditingMode();
	
	FApplicationMode::PostActivateMode();
}

TSharedPtr<FTabManager::FLayout> FDisplayClusterConfiguratorEditorConfigurationMode::BuildDefaultLayout(const FString& LayoutName)
{
	return FTabManager::NewLayout(FName(LayoutName))
		->AddArea
		(
			FTabManager::NewPrimaryArea()->SetOrientation(Orient_Horizontal)
			// Tree
			->Split
			(
				FTabManager::NewSplitter()
				->SetSizeCoefficient(.2f)
				->SetOrientation(Orient_Vertical)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(.5f)
					->AddTab(TabID_Scene, ETabState::OpenedTab)
					->AddTab(FBlueprintEditorTabs::MyBlueprintID, ETabState::OpenedTab)
					->SetForegroundTab(TabID_Scene)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(.5f)
					->AddTab(TabID_Cluster, ETabState::OpenedTab)
					->SetHideTabWell(false)
				)
			)
			// Viewport and OutputMapping
			->Split
			(
				FTabManager::NewSplitter()
				->SetOrientation(Orient_Vertical)
				->SetSizeCoefficient(.6f)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.5f)
					->AddTab(TabID_Viewport, ETabState::OpenedTab)
					->AddTab("Document", ETabState::ClosedTab)
					->SetForegroundTab(TabID_Viewport)
				)
				->Split
				(
					FTabManager::NewStack()
					->AddTab(TabID_OutputMapping, ETabState::OpenedTab)
					->SetSizeCoefficient(0.5f)
					->SetHideTabWell(false)
				)
			)
			// Details and Log
			->Split
			(
				FTabManager::NewSplitter()
				->SetOrientation(Orient_Vertical)
				->SetSizeCoefficient(.2f)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(.7f)
					->AddTab(FBlueprintEditorTabs::DetailsID, ETabState::OpenedTab)
					->SetForegroundTab(FBlueprintEditorTabs::DetailsID)
					->SetHideTabWell(false)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(.3f)
					->AddTab(FBlueprintEditorTabs::CompilerResultsID, ETabState::OpenedTab)
				)
			)
		);
}

#undef LOCTEXT_NAMESPACE
