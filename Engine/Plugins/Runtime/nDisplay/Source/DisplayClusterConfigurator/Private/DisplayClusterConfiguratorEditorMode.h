// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintEditorModes.h"

// This is the list of IDs for Display Cluster Editor modes
struct FDisplayClusterEditorModes
{
	// App Name
	static const FName DisplayClusterEditorName;

	// Mode constants
	static const FName DisplayClusterEditorConfigurationMode;
	static const FName DisplayClusterEditorGraphMode;
	
	static FText GetLocalizedMode(const FName InMode)
	{
		static TMap< FName, FText > LocModes;

		if (LocModes.Num() == 0)
		{
			LocModes.Add(DisplayClusterEditorConfigurationMode, NSLOCTEXT("DisplayClusterEditorModes", "DisplayClusterEditorConfigurationMode", "Configuration"));
			LocModes.Add(DisplayClusterEditorGraphMode, NSLOCTEXT("DisplayClusterEditorModes", "DisplayClusterBlueprintMode", "Graph"));
		}

		check(InMode != NAME_None);
		const FText* OutDesc = LocModes.Find(InMode);
		check(OutDesc);
		return *OutDesc;
	}
private:
	FDisplayClusterEditorModes() {}
};

class FDisplayClusterConfiguratorBlueprintModeBase : public FBlueprintEditorApplicationMode
{
public:
	static const FName TabID_Log;
	static const FName TabID_OutputMapping;
	static const FName TabID_Scene;
	static const FName TabID_Cluster;
	static const FName TabID_Viewport;

public:
	FDisplayClusterConfiguratorBlueprintModeBase(TSharedPtr<class FDisplayClusterConfiguratorBlueprintEditor> EditorIn, FName EditorModeIn	);

protected:
	TWeakPtr<class FDisplayClusterConfiguratorBlueprintEditor> Editor;
	FWorkflowAllowedTabSet EditorTabFactories;
};

class FDisplayClusterConfiguratorEditorConfigurationMode : public FDisplayClusterConfiguratorBlueprintModeBase
{
public:
	FDisplayClusterConfiguratorEditorConfigurationMode(TSharedPtr<class FDisplayClusterConfiguratorBlueprintEditor> EditorIn);

	// FApplicationMode interface
	virtual void RegisterTabFactories(TSharedPtr<FTabManager> InTabManager) override;
	virtual void PostActivateMode() override;
	// End of FApplicationMode interface

protected:
	TSharedPtr<FTabManager::FLayout> BuildDefaultLayout(const FString& LayoutName);
};
