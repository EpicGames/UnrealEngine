// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfiguratorEditorSubsystem.h"
#include "DisplayClusterConfiguratorUtils.h"
#include "DisplayClusterConfiguratorVersionUtils.h"
#include "DisplayClusterConfiguratorBlueprintEditor.h"

#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterConfigurationStrings.h"
#include "IDisplayClusterConfiguration.h"

#include "Blueprints/DisplayClusterBlueprint.h"
#include "DisplayClusterRootActor.h"

#include "AssetToolsModule.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Modules/ModuleManager.h"
#include "Misc/Paths.h"

UDisplayClusterBlueprint* UDisplayClusterConfiguratorEditorSubsystem::ImportAsset(UObject* InParent,
	const FName& InName, const FString& InFilename)
{
	ADisplayClusterRootActor* RootActor = FDisplayClusterConfiguratorUtils::GenerateRootActorFromConfigFile(InFilename);
	if (!RootActor)
	{
		return nullptr;
	}
	RootActor->PreviewNodeId = DisplayClusterConfigurationStrings::gui::preview::PreviewNodeNone;

	UDisplayClusterBlueprint* NewBlueprint = FDisplayClusterConfiguratorUtils::CreateBlueprintFromRootActor(RootActor, InName, InParent);
	if (NewBlueprint == nullptr)
	{
		return nullptr;
	}
	NewBlueprint->LastEditedDocuments.Reset();

	const UDisplayClusterConfigurationData* OriginalConfigData = RootActor->GetConfigData();
	check(OriginalConfigData);

	UDisplayClusterConfigurationData* ConfigData = CastChecked<UDisplayClusterConfigurationData>(StaticDuplicateObject(OriginalConfigData, NewBlueprint));
	ConfigData->ImportedPath = InFilename;
	
	NewBlueprint->SetConfigData(ConfigData, true);

	FDisplayClusterConfiguratorVersionUtils::SetToLatestVersion(NewBlueprint);
	FKismetEditorUtilities::CompileBlueprint(NewBlueprint);

	return NewBlueprint;
}

bool UDisplayClusterConfiguratorEditorSubsystem::ReimportAsset(UDisplayClusterBlueprint* InBlueprint)
{
	if (InBlueprint != nullptr)
	{
		UDisplayClusterConfigurationData* ConfigData = InBlueprint->GetConfig();
		check(ConfigData);
		return ReloadConfig(InBlueprint, ConfigData->ImportedPath) != nullptr;
	}
	
	return false;
}

UDisplayClusterConfigurationData* UDisplayClusterConfiguratorEditorSubsystem::ReloadConfig(UDisplayClusterBlueprint* InBlueprint, const FString& InConfigPath)
{
	if (InBlueprint != nullptr)
	{
		FString ReimportPath = InConfigPath;
		
		UDisplayClusterConfigurationData* NewConfig = IDisplayClusterConfiguration::Get().LoadConfig(InConfigPath);
		if (NewConfig)
		{
			bool bIsAssetBeingEdited = false;
			if (FDisplayClusterConfiguratorUtils::GetBlueprintEditorForObject(InBlueprint))
			{
				// Close open editor.
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseAllEditorsForAsset(InBlueprint);
				bIsAssetBeingEdited = true;
			}
			
			NewConfig->PathToConfig = ReimportPath;
			NewConfig->ImportedPath = ReimportPath;
			UDisplayClusterConfigurationData* NewConfigData = Cast<UDisplayClusterConfigurationData>(StaticDuplicateObject(NewConfig, InBlueprint));
			InBlueprint->SetConfigData(NewConfigData, true);

			// Compile is necessary to update the CDO with the new config data. Otherwise manipulating the data in the editor won't work correctly.
			FKismetEditorUtilities::CompileBlueprint(InBlueprint);
			
			if (bIsAssetBeingEdited)
			{
				// Open editor if it was previously open.
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(InBlueprint);
			}
			
			return NewConfig;
		}
	}

	return nullptr;
}

bool UDisplayClusterConfiguratorEditorSubsystem::SaveConfig(UDisplayClusterConfigurationData* InConfiguratorEditorData, const FString& InConfigPath)
{
	if (InConfiguratorEditorData)
	{
		InConfiguratorEditorData->PathToConfig = InConfigPath;
		return IDisplayClusterConfiguration::Get().SaveConfig(InConfiguratorEditorData, InConfigPath);
	}

	return false;
}

bool UDisplayClusterConfiguratorEditorSubsystem::ConfigAsString(UDisplayClusterConfigurationData* InConfiguratorEditorData, FString& OutString) const
{
	return IDisplayClusterConfiguration::Get().ConfigAsString(InConfiguratorEditorData, OutString);
}

bool UDisplayClusterConfiguratorEditorSubsystem::RenameAssets(const TWeakObjectPtr<UObject>& InAsset, const FString& InNewPackagePath, const FString& InNewName)
{
	FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>(TEXT("AssetTools"));

	TArray<FAssetRenameData> RenameData;

	RenameData.Add(FAssetRenameData(InAsset, InNewPackagePath, InNewName));

	AssetToolsModule.Get().RenameAssetsWithDialog(RenameData);

	return true;
}


