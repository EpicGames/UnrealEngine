// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorSubsystem.h"
#include "UObject/WeakObjectPtr.h"

#include "DisplayClusterConfiguratorEditorSubsystem.generated.h"

class UDisplayClusterBlueprint;
class UDisplayClusterConfigurationData;

UCLASS()
class UDisplayClusterConfiguratorEditorSubsystem 
	: public UEditorSubsystem
{
	GENERATED_BODY()

public:
	UDisplayClusterBlueprint* ImportAsset(UObject* InParent, const FName& InName, const FString& InFilename);
	
	bool ReimportAsset(UDisplayClusterBlueprint* InBlueprint);

	UDisplayClusterConfigurationData* ReloadConfig(UDisplayClusterBlueprint* InBlueprint, const FString& InConfigPath);

	bool RenameAssets(const TWeakObjectPtr<UObject>& InAsset, const FString& InNewPackagePath, const FString& InNewName);

	bool SaveConfig(UDisplayClusterConfigurationData* InConfiguratorEditorData, const FString& InConfigPath);

	// Convert configuration to string
	bool ConfigAsString(UDisplayClusterConfigurationData* InConfiguratorEditorData, FString& OutString) const;
};
