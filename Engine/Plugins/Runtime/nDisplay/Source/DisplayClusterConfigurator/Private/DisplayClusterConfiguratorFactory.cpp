// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfiguratorFactory.h"

#include "AssetToolsModule.h"
#include "DisplayClusterConfiguratorEditorSubsystem.h"
#include "DisplayClusterConfigurationStrings.h"
#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterConfiguratorUtils.h"
#include "DisplayClusterConfiguratorVersionUtils.h"
#include "Framework/Application/SlateApplication.h"
#include "Views/NewAsset/SDisplayClusterConfiguratorNewBlueprintDialog.h"

#include "DisplayClusterRootActor.h"
#include "Blueprints/DisplayClusterBlueprint.h"
#include "Blueprints/DisplayClusterBlueprintGeneratedClass.h"
#include "Components/DisplayClusterCameraComponent.h"
#include "Components/DisplayClusterScreenComponent.h"

#include "Editor.h"
#include "Engine/SCS_Node.h"
#include "Engine/SimpleConstructionScript.h"
#include "Interfaces/IMainFrameModule.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Misc/MessageDialog.h"
#include "EdGraph/EdGraph.h"
#include "EdGraphSchema_K2.h"
#include "IAssetTools.h"


#define LOCTEXT_NAMESPACE "DisplayClusterConfiguratorFactory"

UDisplayClusterConfiguratorFactory::UDisplayClusterConfiguratorFactory()
{
	SupportedClass = UDisplayClusterBlueprint::StaticClass();
	ParentClass = ADisplayClusterRootActor::StaticClass();

	Formats.Add(FString(DisplayClusterConfigurationStrings::file::FileExtJson) + TEXT(";Config"));

	bCreateNew = true;
	bEditorImport = true;
	bEditAfterNew = true;

	// This is the only way for ConfigureProperties to tell if we are importing vs creating new.
	{
		bIsConfigureNewAssetRequest = false;
		OnConfigureNewAssetRequestHandle = FEditorDelegates::OnConfigureNewAssetProperties.AddUObject(this, &UDisplayClusterConfiguratorFactory::OnConfigureNewAssetRequest);
	}
}

UDisplayClusterConfiguratorFactory::~UDisplayClusterConfiguratorFactory()
{
	if (OnConfigureNewAssetRequestHandle.IsValid())
	{
		FEditorDelegates::OnConfigureNewAssetProperties.Remove(OnConfigureNewAssetRequestHandle);
	}
}

bool UDisplayClusterConfiguratorFactory::ConfigureProperties()
{
	if (!bIsConfigureNewAssetRequest)
	{
		// We are probably importing a file.
		return UFactory::ConfigureProperties();
	}
	
	BlueprintToCopy = nullptr;
	
	IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
	TSharedPtr<SWindow>	ParentWindow = MainFrame.GetParentWindow();

	TSharedRef<SDisplayClusterConfiguratorNewBlueprintDialog> NewSystemDialog = SNew(SDisplayClusterConfiguratorNewBlueprintDialog);
	FSlateApplication::Get().AddModalWindow(NewSystemDialog, ParentWindow);

	if (NewSystemDialog->GetUserConfirmedSelection() == false)
	{
		// User cancelled or closed the dialog so abort asset creation.
		return false;
	}

	const TOptional<FAssetData> SelectedAsset = NewSystemDialog->GetSelectedSystemAsset();
	if (SelectedAsset.IsSet())
	{
		BlueprintToCopy = Cast<UDisplayClusterBlueprint>(SelectedAsset->GetAsset());
	}

	return true;
}

UObject* UDisplayClusterConfiguratorFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name,
                                                              EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn, FName CallingContext)
{
	check(Class->IsChildOf(UDisplayClusterBlueprint::StaticClass()));

	if (!ParentClass || !FKismetEditorUtilities::CanCreateBlueprintOfClass(ParentClass) || !ParentClass->IsChildOf(ADisplayClusterRootActor::StaticClass()))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("ClassName"), (ParentClass != nullptr) ? FText::FromString(ParentClass->GetName()) : LOCTEXT("Null", "(null)"));
		FMessageDialog::Open(EAppMsgType::Ok, FText::Format(LOCTEXT("CannotCreateDisplayClusterBlueprint", "Cannot create a Display Cluster Blueprint based on the class '{ClassName}'."), Args));
		return nullptr;
	}

	UDisplayClusterBlueprint* NewBP = nullptr;

	if (BlueprintToCopy)
	{
		NewBP = CastChecked<UDisplayClusterBlueprint>(StaticDuplicateObject(BlueprintToCopy, InParent, Name));
	}
	else
	{
		NewBP = CastChecked<UDisplayClusterBlueprint>(
			FKismetEditorUtilities::CreateBlueprint(ParentClass,
				InParent,
				Name,
				EBlueprintType::BPTYPE_Normal,
				UDisplayClusterBlueprint::StaticClass(),
				UDisplayClusterBlueprintGeneratedClass::StaticClass(),
				CallingContext));

		SetupNewBlueprint(NewBP);
	}
	
	FKismetEditorUtilities::CompileBlueprint(NewBP);
	
	return NewBP;
}

UObject* UDisplayClusterConfiguratorFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name,
	EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return FactoryCreateNew(Class, InParent, Name, Flags, Context, Warn, NAME_None);
}

bool UDisplayClusterConfiguratorFactory::DoesSupportClass(UClass* Class)
{
	return Class == UDisplayClusterBlueprint::StaticClass();
}

UClass* UDisplayClusterConfiguratorFactory::ResolveSupportedClass()
{
	return ADisplayClusterRootActor::StaticClass();
}

UObject* UDisplayClusterConfiguratorFactory::FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, const FString& InFilename, const TCHAR* Parms, FFeedbackContext* Warn, bool& bOutOperationCanceled)
{
	UDisplayClusterConfiguratorEditorSubsystem* EditorSubsystem = GEditor->GetEditorSubsystem<UDisplayClusterConfiguratorEditorSubsystem>();
	check(EditorSubsystem);
	
	UDisplayClusterBlueprint* NewBlueprint = EditorSubsystem->ImportAsset(InParent, InName, InFilename);
	return NewBlueprint;
}

FString UDisplayClusterConfiguratorFactory::GetDefaultNewAssetName() const
{
	if (BlueprintToCopy)
	{
		const FName UniqueName = FDisplayClusterConfiguratorUtils::CreateUniqueName(BlueprintToCopy->GetFName(), BlueprintToCopy->GetClass(), BlueprintToCopy->GetPackage());
		return UniqueName.ToString();
	}

	return FString(TEXT("nDisplayConfig"));
}

uint32 UDisplayClusterConfiguratorFactory::GetMenuCategories() const
{
	IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
	return AssetTools.FindAdvancedAssetCategory("VirtualProduction");
}

void UDisplayClusterConfiguratorFactory::SetupNewBlueprint(UDisplayClusterBlueprint* NewBlueprint)
{
	check(NewBlueprint);
	
	FDisplayClusterConfiguratorVersionUtils::SetToLatestVersion(NewBlueprint);

	UDisplayClusterConfigurationData* NewConfigData = UDisplayClusterConfigurationData::CreateNewConfigData();

	if (ensure(NewConfigData && NewConfigData->Cluster))
	{
#if PLATFORM_WINDOWS
		// In Windows, the default sync policy should be Nvidia. Linux/other can be Ethernet by default, since Nvidia sync might not be supported.
		NewConfigData->Cluster->Sync.RenderSyncPolicy.Type = DisplayClusterConfigurationStrings::config::cluster::render_sync::Nvidia;
#endif // PLATFORM_WINDOWS

		// Decrease outer viewports screen percentage by default for new configurations
		NewConfigData->RenderFrameSettings.ClusterICVFXOuterViewportBufferRatioMult = 0.66f;
	}

	NewBlueprint->SetConfigData(NewConfigData);
	
	check(NewBlueprint->GeneratedClass);
	UClass* SuperClass = NewBlueprint->GeneratedClass->GetSuperClass();
	check(SuperClass);

	// Setup default components only on newly created blueprints that aren't child blueprints.
	if (SuperClass == ADisplayClusterRootActor::StaticClass())
	{
		// We add a screen component only. The DCRA already has a default camera (view point).
		{
			USCS_Node* NewNode = NewBlueprint->SimpleConstructionScript->CreateNode(UDisplayClusterScreenComponent::StaticClass(),
				*FDisplayClusterConfiguratorUtils::FormatNDisplayComponentName(UDisplayClusterScreenComponent::StaticClass()));
			UDisplayClusterScreenComponent* ComponentTemplate = CastChecked<UDisplayClusterScreenComponent>(NewNode->GetActualComponentTemplate(NewBlueprint->GetGeneratedClass()));
			ComponentTemplate->SetRelativeLocation(FVector(100.f, 0.f, 50.f));
			NewBlueprint->SimpleConstructionScript->AddNode(NewNode);
		}
	}

	SetupInitialBlueprintDocuments(NewBlueprint);
}

void UDisplayClusterConfiguratorFactory::SetupInitialBlueprintDocuments(UDisplayClusterBlueprint* NewBlueprint)
{
	NewBlueprint->LastEditedDocuments.Reset();
	
	// Display default event graph.
	if (UEdGraph* EventGraph = FindObject<UEdGraph>(NewBlueprint, *UEdGraphSchema_K2::GN_EventGraph.ToString()))
	{
		NewBlueprint->LastEditedDocuments.Add(EventGraph);
	}
}

void UDisplayClusterConfiguratorFactory::OnConfigureNewAssetRequest(UFactory* InFactory)
{
	if (InFactory == this)
	{
		bIsConfigureNewAssetRequest = true;
	}
}


UDisplayClusterConfiguratorReimportFactory::UDisplayClusterConfiguratorReimportFactory()
{
	SupportedClass = UDisplayClusterBlueprint::StaticClass();

	bCreateNew = false;
}

bool UDisplayClusterConfiguratorReimportFactory::CanReimport(UObject* Obj, TArray<FString>& OutFilenames)
{
	if (UDisplayClusterBlueprint* Blueprint = Cast<UDisplayClusterBlueprint>(Obj))
	{
		if(UDisplayClusterConfigurationData* ConfigData = Blueprint->GetConfig())
		{
			const FString& Path = ConfigData->ImportedPath;
			if (!Path.IsEmpty())
			{
				OutFilenames.Add(Path);
				return true;
			}
		}
	}
	return false;
}

void UDisplayClusterConfiguratorReimportFactory::SetReimportPaths(UObject* Obj, const TArray<FString>& NewReimportPaths)
{
	if (UDisplayClusterBlueprint* Blueprint = Cast<UDisplayClusterBlueprint>(Obj))
	{
		check(NewReimportPaths.Num() > 0);
		Blueprint->SetConfigPath(NewReimportPaths[0]);
		UDisplayClusterConfigurationData* ConfigData = Blueprint->GetConfig();
		check(ConfigData);
		ConfigData->ImportedPath = NewReimportPaths[0];
	}
}

EReimportResult::Type UDisplayClusterConfiguratorReimportFactory::Reimport(UObject* Obj)
{
	if (!Obj || !Obj->IsA(UDisplayClusterBlueprint::StaticClass()))
	{
		return EReimportResult::Failed;
	}

	UDisplayClusterConfiguratorEditorSubsystem* EditorSubsystem = GEditor->GetEditorSubsystem<UDisplayClusterConfiguratorEditorSubsystem>();
	if (EditorSubsystem != nullptr && EditorSubsystem->ReimportAsset(Cast<UDisplayClusterBlueprint>(Obj)))
	{
		return EReimportResult::Succeeded;
	}

	return EReimportResult::Failed;
}

int32 UDisplayClusterConfiguratorReimportFactory::GetPriority() const
{
	return ImportPriority;
}


#undef LOCTEXT_NAMESPACE
