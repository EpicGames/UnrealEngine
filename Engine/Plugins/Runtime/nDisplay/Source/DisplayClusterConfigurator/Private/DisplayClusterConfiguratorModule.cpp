// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfiguratorModule.h"

#include "DisplayClusterConfiguratorAssetTypeActions.h"
#include "DisplayClusterConfiguratorCommands.h"
#include "DisplayClusterConfiguratorLog.h"
#include "DisplayClusterConfiguratorStyle.h"
#include "DisplayClusterConfiguratorVersionUtils.h"
#include "Framework/Application/SlateApplication.h"
#include "PropertyEditorModule.h"
#include "Settings/DisplayClusterConfiguratorSettings.h"
#include "Views/Details/DisplayClusterRootActorDetailsCustomization.h"
#include "Views/OutputMapping/DisplayClusterConfiguratorOutputMappingCommands.h"

#include "Views/Details/Cluster/DisplayClusterConfiguratorCameraOrderCustomization.h"
#include "Views/Details/Cluster/DisplayClusterConfiguratorClusterDetailsCustomization.h"
#include "Views/Details/Cluster/DisplayClusterConfiguratorClusterReferenceListCustomization.h"
#include "Views/Details/Cluster/DisplayClusterConfiguratorDataDetailsCustomization.h"
#include "Views/Details/Cluster/DisplayClusterConfiguratorExternalImageTypeCustomization.h"
#include "Views/Details/Cluster/DisplayClusterConfiguratorGenerateMipsCustomization.h"
#include "Views/Details/Cluster/DisplayClusterConfiguratorNodeSelectionCustomization.h"
#include "Views/Details/Cluster/DisplayClusterConfiguratorRectangleCustomization.h"
#include "Views/Details/Cluster/DisplayClusterConfiguratorViewportDetailsCustomization.h"
#include "Views/Details/Cluster/DisplayClusterConfiguratorViewportRemapCustomization.h"
#include "Views/Details/Components/DisplayClusterCameraComponentDetailsCustomization.h"
#include "Views/Details/Components/DisplayClusterConfiguratorScreenComponentDetailsCustomization.h"
#include "Views/Details/Components/DisplayClusterICVFXCameraComponentDetailsCustomization.h"
#include "Views/Details/DisplayClusterConfiguratorBaseDetailCustomization.h"
#include "Views/Details/DisplayClusterConfiguratorBaseTypeCustomization.h"
#include "Views/Details/DisplayClusterEditorPropertyReferenceTypeCustomization.h"
#include "Views/Details/Media/DCConfiguratorClusterNodeMediaCustomization.h"
#include "Views/Details/Media/DCConfiguratorICVFXMediaCustomization.h"
#include "Views/Details/Media/DCConfiguratorViewportMediaCustomization.h"
#include "Views/Details/Media/DisplayClusterConfiguratorMediaFullFrameCustomization.h"
#include "Views/Details/Media/DisplayClusterConfiguratorMediaTileCustomization.h"
#include "Views/Details/Policies/DisplayClusterConfiguratorPolicyDetailCustomization.h"
#include "Views/Details/Upscaler/DisplayClusterConfigurationUpscalerSettingsDetailCustomization.h"

#include "Blueprints/DisplayClusterBlueprint.h"
#include "Components/DisplayClusterCameraComponent.h"
#include "Components/DisplayClusterICVFXCameraComponent.h"
#include "Components/DisplayClusterInFrustumFitCameraComponent.h"
#include "Components/DisplayClusterScreenComponent.h"
#include "DisplayClusterRootActor.h"
#include "Misc/DisplayClusterObjectRef.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "AssetTypeCategories.h"
#include "IAssetTools.h"
#include "ISettingsModule.h"

#include "ActorFactories/ActorFactoryBlueprint.h"
#include "DisplayClusterConfiguratorBlueprintEditor.h"
#include "DisplayClusterConfiguratorToolbarExtensions.h"
#include "HAL/IConsoleManager.h"
#include "Modules/ModuleManager.h"
#include "ScopedTransaction.h"
#include "SEditorViewportToolBarButton.h"
#include "Subsystems/PanelExtensionSubsystem.h"
#include "ViewportToolbar/UnrealEdViewportToolbar.h"
#include "Views/Details/Cluster/DCConfiguratorColorGradingSettingsCustomization.h"
#include "Views/Details/Cluster/DCConfiguratorWhiteBalanceCustomization.h"

#define REGISTER_PROPERTY_LAYOUT(PropertyType, CustomizationType) { \
	const FName LayoutName = PropertyType::StaticStruct()->GetFName(); \
	RegisteredPropertyLayoutNames.Add(LayoutName); \
	PropertyModule.RegisterCustomPropertyTypeLayout(LayoutName, \
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&CustomizationType::MakeInstance)); \
}

#define REGISTER_OBJECT_LAYOUT(ObjectType, CustomizationType) { \
	const FName LayoutName = ObjectType::StaticClass()->GetFName(); \
	RegisteredClassLayoutNames.Add(LayoutName); \
	PropertyModule.RegisterCustomClassLayout(LayoutName, \
		FOnGetDetailCustomizationInstance::CreateStatic(&CustomizationType::MakeInstance)); \
}

#define LOCTEXT_NAMESPACE "DisplayClusterConfigurator"

void FDisplayClusterConfiguratorModule::StartupModule()
{
	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FDisplayClusterConfiguratorModule::OnPostEngineInit);

	IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();

	/*
	 * Hack for instanced property sync.
	 *
	 * We must clear CPF_EditConst for these properties. They are VisibleInstanceOnly but we are modifying them through their handles
	 * programmatically. If CPF_EditConst is present that operation will fail. We do not want them to be editable on the details panel either.
	 */
	{
		FProperty* Property = FindFProperty<FProperty>(UDisplayClusterConfigurationCluster::StaticClass(), GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationCluster, Nodes));
		Property->ClearPropertyFlags(CPF_EditConst);
	}
	
	{
		FProperty* Property = FindFProperty<FProperty>(UDisplayClusterConfigurationClusterNode::StaticClass(), GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationClusterNode, Viewports));
		Property->ClearPropertyFlags(CPF_EditConst);
	}

	AssetTools.RegisterAdvancedAssetCategory("VirtualProduction", LOCTEXT("VirtualProductionCategory", "Virtual Production"));
	RegisterAssetTypeAction(AssetTools, MakeShareable(new FDisplayClusterConfiguratorActorAssetTypeActions(EAssetTypeCategories::None)));

	RegisterCustomLayouts();
	RegisterSettings();
	RegisterSectionMappings();
	
	FDisplayClusterConfiguratorStyle::Get();

	FDisplayClusterConfiguratorCommands::Register();
	FDisplayClusterConfiguratorOutputMappingCommands::Register();

	ToolbarExtensions = MakeShared<FDisplayClusterConfiguratorToolbarExtensions>();
	MenuExtensibilityManager = MakeShareable(new FExtensibilityManager);
	ToolBarExtensibilityManager = MakeShareable(new FExtensibilityManager);

	// Register blueprint compiler -- primarily seems to be used when creating a new BP.
	IKismetCompilerInterface& KismetCompilerModule = FModuleManager::LoadModuleChecked<IKismetCompilerInterface>(KISMET_COMPILER_MODULENAME);
	KismetCompilerModule.GetCompilers().Add(&BlueprintCompiler);

	// This is needed for actually pressing compile on the BP.
	FKismetCompilerContext::RegisterCompilerForBP(UDisplayClusterBlueprint::StaticClass(), &FDisplayClusterConfiguratorModule::GetCompilerForDisplayClusterBP);

	const UDisplayClusterConfiguratorEditorSettings* Settings = GetDefault<UDisplayClusterConfiguratorEditorSettings>();
	if (Settings->bUpdateAssetsOnStartup)
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		FilesLoadedHandle = AssetRegistryModule.Get().OnFilesLoaded().AddStatic(&FDisplayClusterConfiguratorVersionUtils::UpdateBlueprintsToNewVersion);
	}
}

void FDisplayClusterConfiguratorModule::ShutdownModule()
{
	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);

	if (FAssetToolsModule* AssetTools = FModuleManager::GetModulePtr<FAssetToolsModule>("AssetTools"))
	{
		for (int32 IndexAction = 0; IndexAction < CreatedAssetTypeActions.Num(); ++IndexAction)
		{
			AssetTools->Get().UnregisterAssetTypeActions(CreatedAssetTypeActions[IndexAction].ToSharedRef());
		}
	}

	UnregisterSettings();
	UnregisterCustomLayouts();
	UnregisterSectionMappings();

	ToolbarExtensions->UnregisterToolbarExtensions();
	ToolbarExtensions.Reset();
	
	MenuExtensibilityManager.Reset();
	ToolBarExtensibilityManager.Reset();

	if (GEditor)
	{
		FDisplayClusterConfiguratorBlueprintEditor::UnregisterPanelExtensionFactory();
	}

	IKismetCompilerInterface& KismetCompilerModule = FModuleManager::GetModuleChecked<IKismetCompilerInterface>("KismetCompiler");
	KismetCompilerModule.GetCompilers().Remove(&BlueprintCompiler);

	if (FilesLoadedHandle.IsValid() && FModuleManager::Get().IsModuleLoaded("AssetRegistry"))
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		AssetRegistryModule.Get().OnFilesLoaded().Remove(FilesLoadedHandle);
	}
}

void FDisplayClusterConfiguratorModule::OnPostEngineInit()
{
	if (GEditor)
	{
		FDisplayClusterConfiguratorBlueprintEditor::RegisterPanelExtensionFactory();
	}

	ToolbarExtensions->RegisterToolbarExtensions();
}

const FDisplayClusterConfiguratorCommands& FDisplayClusterConfiguratorModule::GetCommands() const
{
	return FDisplayClusterConfiguratorCommands::Get();
}

void FDisplayClusterConfiguratorModule::RegisterAssetTypeAction(IAssetTools& AssetTools,
	TSharedRef<IAssetTypeActions> Action)
{
	AssetTools.RegisterAssetTypeActions(Action);
	CreatedAssetTypeActions.Add(Action);
}

void FDisplayClusterConfiguratorModule::RegisterSettings()
{
	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
	{
		SettingsModule->RegisterSettings("Editor", "Plugins", "nDisplayEditor",
			LOCTEXT("nDisplayEditorName", "nDisplay Editor"),
			LOCTEXT("nDisplayEditorDescription", "Configure settings for the nDisplay Editor."),
			GetMutableDefault<UDisplayClusterConfiguratorEditorSettings>());
	}
}

void FDisplayClusterConfiguratorModule::UnregisterSettings()
{
	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
	{
		SettingsModule->UnregisterSettings("Editor", "Plugins", "nDisplayEditor");
	}
}

void FDisplayClusterConfiguratorModule::RegisterCustomLayouts()
{
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	/**
	 * CLASSES
	 */
	REGISTER_OBJECT_LAYOUT(ADisplayClusterRootActor, FDisplayClusterRootActorDetailsCustomization);
	REGISTER_OBJECT_LAYOUT(UDisplayClusterConfigurationData, FDisplayClusterConfiguratorDataDetailsCustomization);
	REGISTER_OBJECT_LAYOUT(UDisplayClusterConfigurationCluster, FDisplayClusterConfiguratorClusterDetailsCustomization);
	REGISTER_OBJECT_LAYOUT(UDisplayClusterConfigurationClusterNode, FDisplayClusterConfiguratorBaseDetailCustomization);
	REGISTER_OBJECT_LAYOUT(UDisplayClusterConfigurationViewport, FDisplayClusterConfiguratorViewportDetailsCustomization);
	REGISTER_OBJECT_LAYOUT(UDisplayClusterScreenComponent, FDisplayClusterConfiguratorScreenDetailsCustomization);
	REGISTER_OBJECT_LAYOUT(UDisplayClusterCameraComponent, FDisplayClusterCameraComponentDetailsCustomization);
	REGISTER_OBJECT_LAYOUT(UDisplayClusterICVFXCameraComponent, FDisplayClusterICVFXCameraComponentDetailsCustomization);
	
	/**
	 * STRUCTS
	 */
	REGISTER_PROPERTY_LAYOUT(FDisplayClusterConfigurationICVFX_VisibilityList, FDisplayClusterConfiguratorBaseTypeCustomization);
	REGISTER_PROPERTY_LAYOUT(FDisplayClusterEditorPropertyReference, FDisplayClusterEditorPropertyReferenceTypeCustomization);
	REGISTER_PROPERTY_LAYOUT(FDisplayClusterConfigurationProjection, FDisplayClusterConfiguratorProjectionCustomization);
	REGISTER_PROPERTY_LAYOUT(FDisplayClusterConfigurationRenderSyncPolicy, FDisplayClusterConfiguratorRenderSyncPolicyCustomization);
	REGISTER_PROPERTY_LAYOUT(FDisplayClusterConfigurationInputSyncPolicy, FDisplayClusterConfiguratorInputSyncPolicyCustomization);
	REGISTER_PROPERTY_LAYOUT(FDisplayClusterConfigurationExternalImage, FDisplayClusterConfiguratorExternalImageTypeCustomization);
	REGISTER_PROPERTY_LAYOUT(FDisplayClusterComponentRef, FDisplayClusterConfiguratorBaseTypeCustomization);
	REGISTER_PROPERTY_LAYOUT(FDisplayClusterConfigurationOCIOProfile, FDisplayClusterConfiguratorOCIOProfileCustomization);
	REGISTER_PROPERTY_LAYOUT(FDisplayClusterConfigurationViewport_PerViewportColorGrading, FDisplayClusterConfiguratorPerViewportColorGradingCustomization);
	REGISTER_PROPERTY_LAYOUT(FDisplayClusterConfigurationViewport_PerNodeColorGrading, FDisplayClusterConfiguratorPerNodeColorGradingCustomization);
	REGISTER_PROPERTY_LAYOUT(FDisplayClusterConfigurationPostRender_GenerateMips, FDisplayClusterConfiguratorGenerateMipsCustomization);
	REGISTER_PROPERTY_LAYOUT(FDisplayClusterConfigurationClusterItemReferenceList, FDisplayClusterConfiguratorClusterReferenceListCustomization);
	REGISTER_PROPERTY_LAYOUT(FDisplayClusterConfigurationViewport_RemapData, FDisplayClusterConfiguratorViewportRemapCustomization);
	REGISTER_PROPERTY_LAYOUT(FDisplayClusterConfigurationRectangle, FDisplayClusterConfiguratorRectangleCustomization);
	REGISTER_PROPERTY_LAYOUT(FDisplayClusterConfigurationMediaICVFX, FDCConfiguratorICVFXMediaCustomization);
	REGISTER_PROPERTY_LAYOUT(FDisplayClusterConfigurationMediaNodeBackbuffer, FDCConfiguratorClusterNodeMediaCustomization);
	REGISTER_PROPERTY_LAYOUT(FDisplayClusterConfigurationMediaViewport, FDCConfiguratorViewportMediaCustomization);
	REGISTER_PROPERTY_LAYOUT(FDisplayClusterConfigurationMediaUniformTileInput,  FDisplayClusterConfiguratorMediaInputTileCustomization);
	REGISTER_PROPERTY_LAYOUT(FDisplayClusterConfigurationMediaUniformTileOutput, FDisplayClusterConfiguratorMediaOutputTileCustomization);
	REGISTER_PROPERTY_LAYOUT(FDisplayClusterConfigurationMediaInput,  FDisplayClusterConfiguratorMediaFullFrameInputCustomization);
	REGISTER_PROPERTY_LAYOUT(FDisplayClusterConfigurationMediaOutput, FDisplayClusterConfiguratorMediaFullFrameOutputCustomization);
	REGISTER_PROPERTY_LAYOUT(FDisplayClusterConfigurationMediaInputGroup,  FDisplayClusterConfiguratorMediaFullFrameInputCustomization);
	REGISTER_PROPERTY_LAYOUT(FDisplayClusterConfigurationMediaOutputGroup, FDisplayClusterConfiguratorMediaFullFrameOutputCustomization);
	REGISTER_PROPERTY_LAYOUT(FDisplayClusterConfigurationViewport_ColorGradingSettings, FDCConfiguratorColorGradingSettingsCustomization);
	REGISTER_PROPERTY_LAYOUT(FDisplayClusterConfigurationViewport_ColorGradingWhiteBalanceSettings, FDCConfiguratorWhiteBalanceCustomization);
	REGISTER_PROPERTY_LAYOUT(FDisplayClusterConfigurationUpscalerSettings, FDisplayClusterConfigurationUpscalerSettingsDetailCustomization);
}

void FDisplayClusterConfiguratorModule::UnregisterCustomLayouts()
{
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	for (const FName& LayoutName : RegisteredClassLayoutNames)
	{
		PropertyModule.UnregisterCustomClassLayout(LayoutName);
	}
	
	for (const FName& LayoutName : RegisteredPropertyLayoutNames)
	{
		PropertyModule.UnregisterCustomPropertyTypeLayout(LayoutName);
	}

	RegisteredPropertyLayoutNames.Empty();
}

void FDisplayClusterConfiguratorModule::RegisterSectionMappings()
{
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	
	// Root Actor
	{
		const TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection(ADisplayClusterRootActor::StaticClass()->GetFName(),
			DisplayClusterConfigurationStrings::categories::ViewportsCategory, LOCTEXT("Viewports", "Viewports"));
		Section->AddCategory(DisplayClusterConfigurationStrings::categories::ViewportsCategory);
	}
	{
		const TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection(ADisplayClusterRootActor::StaticClass()->GetFName(),
			DisplayClusterConfigurationStrings::categories::InCameraVFXCategory, LOCTEXT("InCameraVFXCategoryLabel", "In-Camera VFX"));
		Section->AddCategory(DisplayClusterConfigurationStrings::categories::InCameraVFXCategory);
	}
	{
		const TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection(ADisplayClusterRootActor::StaticClass()->GetFName(),
			DisplayClusterConfigurationStrings::categories::ColorGradingCategory, LOCTEXT("Color Grading", "Color Grading"));
		Section->AddCategory(DisplayClusterConfigurationStrings::categories::ColorGradingCategory);
	}
	{
		const TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection(ADisplayClusterRootActor::StaticClass()->GetFName(),
			DisplayClusterConfigurationStrings::categories::OCIOCategory, LOCTEXT("OCIO", "OCIO"));
		Section->AddCategory(DisplayClusterConfigurationStrings::categories::OCIOCategory);
	}
	{
		const TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection(ADisplayClusterRootActor::StaticClass()->GetFName(),
			DisplayClusterConfigurationStrings::categories::ExposureCategory, LOCTEXT("Exposure", "Exposure"));
		Section->AddCategory(DisplayClusterConfigurationStrings::categories::ExposureCategory);
	}
	{
		const TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection(ADisplayClusterRootActor::StaticClass()->GetFName(),
			DisplayClusterConfigurationStrings::categories::LightcardCategory, LOCTEXT("Light Cards", "Light Cards"));
		Section->AddCategory(DisplayClusterConfigurationStrings::categories::LightcardCategory);
	}
	{
		const TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection(ADisplayClusterRootActor::StaticClass()->GetFName(),
			DisplayClusterConfigurationStrings::categories::PreviewCategory, LOCTEXT("Preview", "Preview"));
		Section->AddCategory(DisplayClusterConfigurationStrings::categories::PreviewCategory);
	}	

	// ICVFX Component
	{
		const TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection(UDisplayClusterICVFXCameraComponent::StaticClass()->GetFName(),
		DisplayClusterConfigurationStrings::categories::InnerFrustumCategory, LOCTEXT("InnerFrustumCategoryLabel", "Inner Frustum"));
		Section->AddCategory(DisplayClusterConfigurationStrings::categories::InnerFrustumCategory);
	}
	{
		const TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection(UDisplayClusterICVFXCameraComponent::StaticClass()->GetFName(),
			DisplayClusterConfigurationStrings::categories::ICVFXCameraCategory, LOCTEXT("InnerFrustumCameraSectionLabel", "Camera"));
		Section->AddCategory(DisplayClusterConfigurationStrings::categories::ICVFXCameraCategory);
		Section->AddCategory(DisplayClusterConfigurationStrings::categories::ICVFXCameraCategoryOrig);
	}
	{
		const TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection(UDisplayClusterICVFXCameraComponent::StaticClass()->GetFName(),
		DisplayClusterConfigurationStrings::categories::CameraColorGradingCategory, LOCTEXT("InnerFrustumColorGradingLabel", "Color Grading"));
		Section->AddCategory(DisplayClusterConfigurationStrings::categories::CameraColorGradingCategory);
		Section->AddCategory(DisplayClusterConfigurationStrings::categories::CameraColorGradingCategoryOrig);
	}
	{
		const TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection(UDisplayClusterICVFXCameraComponent::StaticClass()->GetFName(),
			DisplayClusterConfigurationStrings::categories::OCIOCategory, LOCTEXT("OCIO", "OCIO"));
		Section->AddCategory(DisplayClusterConfigurationStrings::categories::OCIOCategory);
	}
	{
		const TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection(UDisplayClusterICVFXCameraComponent::StaticClass()->GetFName(),
			DisplayClusterConfigurationStrings::categories::MediaCategory, LOCTEXT("Media", "Media"));
		Section->AddCategory(DisplayClusterConfigurationStrings::categories::MediaCategory);
	}
	{
		const TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection(UDisplayClusterICVFXCameraComponent::StaticClass()->GetFName(),
			DisplayClusterConfigurationStrings::categories::ChromaKeyCategory, LOCTEXT("Chromakey", "Chromakey"));
		Section->AddCategory(DisplayClusterConfigurationStrings::categories::ChromaKeyCategory);
	}

	// ViewPoint
	{
		const TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection(UDisplayClusterCameraComponent::StaticClass()->GetFName(),
			DisplayClusterConfigurationStrings::categories::ViewPointStereoCategory, LOCTEXT("ViewPointStereoSectionLabel", "Stereo"));
		Section->AddCategory(DisplayClusterConfigurationStrings::categories::ViewPointStereoCategory);
	}
	{
		const TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection(UDisplayClusterCameraComponent::StaticClass()->GetFName(),
			DisplayClusterConfigurationStrings::categories::ViewPointCameraPostProcessCategory, LOCTEXT("ViewPointCameraPostProcessSectionLabel", "Camera Settings"));
		Section->AddCategory(DisplayClusterConfigurationStrings::categories::ViewPointCameraPostProcessCategory);
	}

	// InFrustum ViewPoint
	{
		const TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection(UDisplayClusterInFrustumFitCameraComponent::StaticClass()->GetFName(),
			DisplayClusterConfigurationStrings::categories::ViewPointInFrustumProjectionCategory, LOCTEXT("ViewPointInFrustumProjectionSectionLabel", "Frustum Fit"));
		Section->AddCategory(DisplayClusterConfigurationStrings::categories::ViewPointInFrustumProjectionCategory);
	}

	// Cluster node
	{
		const TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection(UDisplayClusterConfigurationClusterNode::StaticClass()->GetFName(),
			DisplayClusterConfigurationStrings::categories::NetworkCategory, LOCTEXT("ClusterNodeNetworkSectionLabel", "Network"));
		Section->AddCategory(DisplayClusterConfigurationStrings::categories::NetworkCategory);
	}
	{
		const TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection(UDisplayClusterConfigurationClusterNode::StaticClass()->GetFName(),
			DisplayClusterConfigurationStrings::categories::ConfigurationCategory, LOCTEXT("ClusterNodeConfigurationSectionLabel", "Configuration"));
		Section->AddCategory(DisplayClusterConfigurationStrings::categories::ConfigurationCategory);
	}
	{
		const TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection(UDisplayClusterConfigurationClusterNode::StaticClass()->GetFName(),
			DisplayClusterConfigurationStrings::categories::MediaCategory, LOCTEXT("ClusterNodeMediaSectionLabel", "Media"));
		Section->AddCategory(DisplayClusterConfigurationStrings::categories::MediaCategory);
	}

	// Viewport
	{
		const TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection(UDisplayClusterConfigurationViewport::StaticClass()->GetFName(),
			DisplayClusterConfigurationStrings::categories::ConfigurationCategory, LOCTEXT("ViewportConfigurationSectionLabel", "Configuration"));
		Section->AddCategory(DisplayClusterConfigurationStrings::categories::ConfigurationCategory);
	}
	{
		const TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection(UDisplayClusterConfigurationViewport::StaticClass()->GetFName(),
			DisplayClusterConfigurationStrings::categories::MediaCategory, LOCTEXT("ViewportMediaSectionLabel", "Media"));
		Section->AddCategory(DisplayClusterConfigurationStrings::categories::MediaCategory);
	}
	{
		const TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection(UDisplayClusterConfigurationViewport::StaticClass()->GetFName(),
			DisplayClusterConfigurationStrings::categories::PreviewCategory, LOCTEXT("ViewportPreviewSectionLabel", "Preview"));
		Section->AddCategory(DisplayClusterConfigurationStrings::categories::PreviewCategory);
	}
	{
		const TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection(UDisplayClusterConfigurationViewport::StaticClass()->GetFName(),
			DisplayClusterConfigurationStrings::categories::RenderingCategory, LOCTEXT("ViewportRenderingSectionLabel", "Rendering"));
		Section->AddCategory(DisplayClusterConfigurationStrings::categories::RenderingCategory);
		Section->AddCategory(DisplayClusterConfigurationStrings::categories::StereoCategory);
	}
}

void FDisplayClusterConfiguratorModule::UnregisterSectionMappings()
{
	if (FModuleManager::Get().IsModuleLoaded("PropertyEditor") && FSlateApplication::IsInitialized())
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

		// DCRA
		PropertyModule.RemoveSection(ADisplayClusterRootActor::StaticClass()->GetFName(), DisplayClusterConfigurationStrings::categories::ViewportsCategory);
		PropertyModule.RemoveSection(ADisplayClusterRootActor::StaticClass()->GetFName(), DisplayClusterConfigurationStrings::categories::InCameraVFXCategory);
		PropertyModule.RemoveSection(ADisplayClusterRootActor::StaticClass()->GetFName(), DisplayClusterConfigurationStrings::categories::ColorGradingCategory);
		PropertyModule.RemoveSection(ADisplayClusterRootActor::StaticClass()->GetFName(), DisplayClusterConfigurationStrings::categories::OCIOCategory);
		PropertyModule.RemoveSection(ADisplayClusterRootActor::StaticClass()->GetFName(), DisplayClusterConfigurationStrings::categories::ExposureCategory);
		PropertyModule.RemoveSection(ADisplayClusterRootActor::StaticClass()->GetFName(), DisplayClusterConfigurationStrings::categories::LightcardCategory);
		PropertyModule.RemoveSection(ADisplayClusterRootActor::StaticClass()->GetFName(), DisplayClusterConfigurationStrings::categories::PreviewCategory);

		// ICVFX camera component
		PropertyModule.RemoveSection(UDisplayClusterICVFXCameraComponent::StaticClass()->GetFName(), DisplayClusterConfigurationStrings::categories::InnerFrustumCategory);
		PropertyModule.RemoveSection(UDisplayClusterICVFXCameraComponent::StaticClass()->GetFName(), DisplayClusterConfigurationStrings::categories::ICVFXCameraCategory);
		PropertyModule.RemoveSection(UDisplayClusterICVFXCameraComponent::StaticClass()->GetFName(), DisplayClusterConfigurationStrings::categories::CameraColorGradingCategory);
		PropertyModule.RemoveSection(UDisplayClusterICVFXCameraComponent::StaticClass()->GetFName(), DisplayClusterConfigurationStrings::categories::OCIOCategory);
		PropertyModule.RemoveSection(UDisplayClusterICVFXCameraComponent::StaticClass()->GetFName(), DisplayClusterConfigurationStrings::categories::MediaCategory);
		PropertyModule.RemoveSection(UDisplayClusterICVFXCameraComponent::StaticClass()->GetFName(), DisplayClusterConfigurationStrings::categories::ChromaKeyCategory);

		// ViewPoint component
		PropertyModule.RemoveSection(UDisplayClusterCameraComponent::StaticClass()->GetFName(), DisplayClusterConfigurationStrings::categories::ViewPointStereoCategory);
		PropertyModule.RemoveSection(UDisplayClusterCameraComponent::StaticClass()->GetFName(), DisplayClusterConfigurationStrings::categories::ViewPointCameraPostProcessCategory);

		// InFrustum ViewPoint
		PropertyModule.RemoveSection(UDisplayClusterInFrustumFitCameraComponent::StaticClass()->GetFName(), DisplayClusterConfigurationStrings::categories::ViewPointInFrustumProjectionCategory);

		// Cluster node
		PropertyModule.RemoveSection(UDisplayClusterConfigurationClusterNode::StaticClass()->GetFName(), DisplayClusterConfigurationStrings::categories::NetworkCategory);
		PropertyModule.RemoveSection(UDisplayClusterConfigurationClusterNode::StaticClass()->GetFName(), DisplayClusterConfigurationStrings::categories::ConfigurationCategory);
		PropertyModule.RemoveSection(UDisplayClusterConfigurationClusterNode::StaticClass()->GetFName(), DisplayClusterConfigurationStrings::categories::MediaCategory);

		// Viewport
		PropertyModule.RemoveSection(UDisplayClusterConfigurationViewport::StaticClass()->GetFName(), DisplayClusterConfigurationStrings::categories::ConfigurationCategory);
		PropertyModule.RemoveSection(UDisplayClusterConfigurationViewport::StaticClass()->GetFName(), DisplayClusterConfigurationStrings::categories::MediaCategory);
		PropertyModule.RemoveSection(UDisplayClusterConfigurationViewport::StaticClass()->GetFName(), DisplayClusterConfigurationStrings::categories::PreviewCategory);
		PropertyModule.RemoveSection(UDisplayClusterConfigurationViewport::StaticClass()->GetFName(), DisplayClusterConfigurationStrings::categories::RenderingCategory);
	}
}

TSharedPtr<FKismetCompilerContext> FDisplayClusterConfiguratorModule::GetCompilerForDisplayClusterBP(UBlueprint* BP,
                                                                                                     FCompilerResultsLog& InMessageLog, const FKismetCompilerOptions& InCompileOptions)
{
	return TSharedPtr<FKismetCompilerContext>(new FDisplayClusterConfiguratorKismetCompilerContext(CastChecked<UDisplayClusterBlueprint>(BP), InMessageLog, InCompileOptions));
}

IMPLEMENT_MODULE(FDisplayClusterConfiguratorModule, DisplayClusterConfigurator);

#undef LOCTEXT_NAMESPACE
