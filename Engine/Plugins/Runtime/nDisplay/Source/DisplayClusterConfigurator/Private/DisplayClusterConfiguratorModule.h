// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DisplayClusterConfiguratorCompiler.h"
#include "IAssetTypeActions.h"
#include "IDisplayClusterConfigurator.h"

class FDisplayClusterConfiguratorToolbarExtensions;
class FPlacementModeID;
class IAssetTools;
class FDisplayClusterConfiguratorAssetTypeActions;
class FExtensibilityManager;

/**
 * Display Cluster Configurator editor module
 */
class FDisplayClusterConfiguratorModule :
	public IDisplayClusterConfigurator
{
public:
	//~ IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	void OnPostEngineInit();

	/** Gets the extensibility managers for outside entities to extend this editor's menus and toolbars */
	virtual TSharedPtr<FExtensibilityManager> GetMenuExtensibilityManager() const override { return MenuExtensibilityManager; }
	virtual TSharedPtr<FExtensibilityManager> GetToolBarExtensibilityManager() const override { return ToolBarExtensibilityManager; }
	
public:
	//~ Begin IDisplayClusterConfigurator Interface
	virtual const FDisplayClusterConfiguratorCommands& GetCommands() const override;
	//~ End IDisplayClusterConfigurator Interface

private:
	void RegisterAssetTypeAction(IAssetTools& AssetTools, TSharedRef<IAssetTypeActions> Action);
	void RegisterSettings();
	void UnregisterSettings();
	void RegisterCustomLayouts();
	void UnregisterCustomLayouts();
	void RegisterSectionMappings();
	void UnregisterSectionMappings();

	static TSharedPtr<FKismetCompilerContext> GetCompilerForDisplayClusterBP(UBlueprint* BP, FCompilerResultsLog& InMessageLog, const FKismetCompilerOptions& InCompileOptions);

private:
	TArray<TSharedPtr<IAssetTypeActions>> CreatedAssetTypeActions;
	TSharedPtr<FExtensibilityManager> MenuExtensibilityManager;
	TSharedPtr<FExtensibilityManager> ToolBarExtensibilityManager;
	TSharedPtr<FDisplayClusterConfiguratorToolbarExtensions> ToolbarExtensions;
	FDisplayClusterConfiguratorKismetCompiler  BlueprintCompiler;
	TArray<FName> RegisteredClassLayoutNames;
	TArray<FName> RegisteredPropertyLayoutNames;
	FDelegateHandle FilesLoadedHandle;
	FDelegateHandle PostEngineInitHandle;

private:
	static FOnDisplayClusterConfiguratorReadOnlyChanged OnDisplayClusterConfiguratorReadOnlyChanged;
};
