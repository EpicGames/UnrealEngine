// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfiguratorPropertyUtils.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"

#define LOCTEXT_NAMESPACE "DisplayClusterConfiguratorPropertyUtils"

TSharedPtr<ISinglePropertyView> UE::DisplayClusterConfiguratorPropertyUtils::GetPropertyView(UObject* Owner,
	const FName& FieldName)
{
	check(Owner);
	
	const FSinglePropertyParams InitParams;
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	const TSharedPtr<ISinglePropertyView> PropView = PropertyEditorModule.CreateSingleProperty(
		Owner,
		FieldName,
		InitParams);

	return PropView;
}

UObject* UE::DisplayClusterConfiguratorPropertyUtils::AddKeyWithInstancedValueToMap(UObject* MapOwner, const FName& FieldName,
                                                                                const FString& Key, UObject* Value)
{
	check(MapOwner);

	const TSharedPtr<ISinglePropertyView> PropertyView = GetPropertyView(MapOwner, FieldName);
	const TSharedPtr<IPropertyHandle> PropHandle = PropertyView->GetPropertyHandle();
	check(PropHandle.IsValid());

	const TSharedPtr<IPropertyHandle> PairHandle = AddKeyValueToMap((uint8*)MapOwner, PropHandle, Key, Value ? Value->GetPathName() : TEXT("None"), EPropertyValueSetFlags::InstanceObjects);
	check(PairHandle.IsValid());
	
	// The object was copied in so retrieve the new instance.
	UObject* NewValue = nullptr;
	PairHandle->GetValue(NewValue);

	return NewValue;
}

TSharedPtr<IPropertyHandle> UE::DisplayClusterConfiguratorPropertyUtils::AddKeyValueToMap(uint8* MapOwner, TSharedPtr<IPropertyHandle> PropertyHandle, const FString& Key, const FString& Value, EPropertyValueSetFlags::Type SetFlags)
{
	check(MapOwner);
	check(PropertyHandle.IsValid());
	FMapProperty* MapProperty = CastFieldChecked<FMapProperty>(PropertyHandle->GetProperty());
	
	TSharedPtr<IPropertyHandleMap> MapHandle = PropertyHandle->AsMap();
	check(MapHandle.IsValid());

	// Original approach — currently crashes consistently during RemoveKeyFromMap().
	// The root cause is most likely within the compact hashmap implementation,
	// but that is for another discussion.
	//
	// The algorithm was structured as follows:
	// 1. Remove the existing 'Key' entry (it will be recreated later anyway).
	// 2. Remove any existing empty key, since a new empty key will be introduced.
	// 3. Add a new key–value pair, which initially has an empty key
	//    (hence the need for step [2]).
	// 4. Locate the newly created entry from step [3].
	// ... set up key/value data for the entry
#if 0
	// Remove existing key if it exists.
	RemoveKeyFromMap(MapOwner, PropertyHandle, Key);
	
	// Remove empty key before adding or it will fail.
	RemoveKeyFromMap(MapOwner, PropertyHandle, TEXT(""));
	{
		const FPropertyAccess::Result Result = MapHandle->AddItem();
		check(Result == FPropertyAccess::Success);
	}
	
	void* MapContainer = MapProperty->ContainerPtrToValuePtr<void*>(MapOwner);
	FScriptMapHelper MapHelper(MapProperty, MapContainer);

	const int32 IndexToModify = FindMapHandleIndexFromKey(MapHandle, MapHelper, TEXT(""));
	check(IndexToModify >= 0);
#else
	// Revised approach - avoids calling RemoveKeyFromMap(), eliminating the crash.
	//
	// This version achieves the same result via a different flow:
	// 1. Look up the existing 'Key' entry.
	// 2. If it does not exist, create a new entry with an empty key.
	// 3. Locate the resulting entry (either the existing one or the newly created one)
	// ... set up key/value data for the entry
	void* MapContainer = MapProperty->ContainerPtrToValuePtr<void*>(MapOwner);
	FScriptMapHelper MapHelper(MapProperty, MapContainer);
	
	int32 IndexToModify = FindMapHandleIndexFromKey(MapHandle, MapHelper, Key);
	if (IndexToModify == INDEX_NONE)
	{
		const FString DefaultKey = TEXT("");
		IndexToModify = FindMapHandleIndexFromKey(MapHandle, MapHelper, DefaultKey);
		if (IndexToModify == INDEX_NONE)
		{
			const FPropertyAccess::Result Result = MapHandle->AddItem();
			check(Result == FPropertyAccess::Success);
			IndexToModify = FindMapHandleIndexFromKey(MapHandle, MapHelper, DefaultKey);
		}
	}
	check(IndexToModify >= 0);
#endif

	TSharedPtr<IPropertyHandle> PairHandle = PropertyHandle->GetChildHandle(IndexToModify);
	check(PairHandle.IsValid());

	TSharedPtr<IPropertyHandle> KeyPropertyHandle = PairHandle->GetKeyHandle();
	check(KeyPropertyHandle.IsValid());

	{
		const FPropertyAccess::Result Result = KeyPropertyHandle->SetValue(Key);
		check(Result == FPropertyAccess::Success);
	}
	{
		const FPropertyAccess::Result Result = PairHandle->SetValueFromFormattedString(Value, SetFlags);
		check(Result == FPropertyAccess::Success);
	}
	
	MapHelper.Rehash();

	// Search for handle again since rehash may have invalidated it.
	return PropertyHandle->GetChildHandle(IndexToModify);
}

bool UE::DisplayClusterConfiguratorPropertyUtils::RemoveKeyFromMap(UObject* MapOwner, const FName& FieldName,
                                                               const FString& Key)
{
	check(MapOwner);

	const TSharedPtr<ISinglePropertyView> PropertyView = GetPropertyView(MapOwner, FieldName);
	const TSharedPtr<IPropertyHandle> PropHandle = PropertyView->GetPropertyHandle();
	check(PropHandle.IsValid());

	return RemoveKeyFromMap((uint8*)MapOwner, PropHandle, Key);
}

bool UE::DisplayClusterConfiguratorPropertyUtils::RemoveKeyFromMap(uint8* MapOwner,
	TSharedPtr<IPropertyHandle> PropertyHandle, const FString& Key)
{
	check(MapOwner);
	check(PropertyHandle.IsValid());
	
	TSharedPtr<IPropertyHandleMap> MapHandle = PropertyHandle->AsMap();
	check(MapHandle.IsValid());

	FMapProperty* MapProperty = CastFieldChecked<FMapProperty>(PropertyHandle->GetProperty());
	
	void* MapContainer = MapProperty->ContainerPtrToValuePtr<void*>(MapOwner);
	FScriptMapHelper MapHelper(MapProperty, MapContainer);

	const int32 IndexToDelete = FindMapHandleIndexFromKey(MapHandle, MapHelper, Key);
	if (IndexToDelete == INDEX_NONE)
	{
		return false;
	}

	{
		const FPropertyAccess::Result Result = MapHandle->DeleteItem(IndexToDelete);
		check(Result == FPropertyAccess::Success);
	}
	
	MapHelper.Rehash();

	return true;
}

bool UE::DisplayClusterConfiguratorPropertyUtils::EmptyMap(uint8* MapOwner, TSharedPtr<IPropertyHandle> PropertyHandle)
{
	check(MapOwner);
	check(PropertyHandle.IsValid());

	TSharedPtr<IPropertyHandleMap> MapHandle = PropertyHandle->AsMap();
	check(MapHandle.IsValid());

	FMapProperty* MapProperty = CastFieldChecked<FMapProperty>(PropertyHandle->GetProperty());

	void* MapContainer = MapProperty->ContainerPtrToValuePtr<void*>(MapOwner);
	FScriptMapHelper MapHelper(MapProperty, MapContainer);

	{
		const FPropertyAccess::Result Result = MapHandle->Empty();
		check(Result == FPropertyAccess::Success);
	}

	MapHelper.Rehash();

	return true;
}

int32 UE::DisplayClusterConfiguratorPropertyUtils::FindMapHandleIndexFromKey(TSharedPtr<IPropertyHandleMap> MapHandle, FScriptMapHelper& MapHelper,
	const FString& Key)
{
	check(MapHandle.IsValid());
	
	uint32 NumElements = 0;
	{
		const FPropertyAccess::Result Result = MapHandle->GetNumElements(NumElements);
		check(Result == FPropertyAccess::Success);
	}
	
	int32 Index = INDEX_NONE;
	for (uint32 Idx = 0; Idx < NumElements; ++Idx)
	{
		// We have to iterate to find the true index.
		// There is a discrepency between MapHelper and MapHandle indices.
		const int32 InternalIndex = MapHelper.FindInternalIndex(Idx);
		if (InternalIndex != INDEX_NONE)
		{
			FString* LocalKey = (FString*)MapHelper.GetKeyPtr(InternalIndex);
			if (LocalKey && *LocalKey == Key)
			{
				Index = Idx;
				break;
			}
		}
	}

	return Index;
}

#undef LOCTEXT_NAMESPACE
