// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/UnrealType.h"
#include "Templates/SharedPointer.h"
#include "PropertyHandle.h"
#include "ISinglePropertyView.h"

/**
 * Utilities to assist with modifying properties through their handle so values are propagated across instances.
 */
namespace UE::DisplayClusterConfiguratorPropertyUtils
{
	/**
	 * Create a temporary property handle for a given property. Call GetPropertyHandle() from here.
	 * The property handle will be valid as long as the property view is valid.
	 *
	 * @param Owner UObject owning the property.
	 * @param FieldName The field name of the property.
	 *
	 * @return A property handle created for this property.
	 */
	DISPLAYCLUSTERCONFIGURATOR_API TSharedPtr<ISinglePropertyView> GetPropertyView(UObject* Owner, const FName& FieldName);

	/**
	 * Set the value of a property handle for an object by the field name.
	 *
	 * @param Owner UObject owning the property.
	 * @param FieldName The field name of the property.
	 * @param Value The value to the set the property.
	 */
	template<typename T>
	void SetPropertyHandleValue(UObject* Owner, const FName& FieldName, T Value)
	{
		const TSharedPtr<ISinglePropertyView> PropertyView = GetPropertyView(
					Owner, FieldName);
		check(PropertyView.IsValid());
		PropertyView->GetPropertyHandle()->SetValue(Value);
	}

	/**
	 * Add an instanced object value to a map when the property handle is not available.
	 * A temporary handle will be created.
	 * This will deep copy the object into the map.
	 *
	 * @param MapOwner UObject owning the map.
	 * @param FieldName The field name of the map property.
	 * @param Key The Key to add to the map.
	 * @param Value An object instance to add to the map.
	 *
	 * @return An updated object instance that was added to the map. This will not be the same
	 * as the original passed in value.
	 */
	UObject* AddKeyWithInstancedValueToMap(UObject* MapOwner, const FName& FieldName, const FString& Key, UObject* Value);
	
	/**
	 * Add a formatted string value to map when the property handle is already available.
	 *
	 * @param MapOwner The address of the object owning the map.
	 * @param PropertyHandle The property handle of the map.
	 * @param Key The key value to add.
	 * @param Value The formatted string value to add.
	 * @param SetFlags How the formatted string value should be set.
	 * 
	 * @return The KeyPair handle added.
	 */
	TSharedPtr<IPropertyHandle> AddKeyValueToMap(uint8* MapOwner, TSharedPtr<IPropertyHandle> PropertyHandle, const FString& Key, const FString& Value, EPropertyValueSetFlags::Type SetFlags = EPropertyValueSetFlags::DefaultFlags);

	/**
	 * Remove a key from a map without the property handle.
	 * A temporary handle will be created.
	 *
	 * @param MapOwner UObject owning the map.
	 * @param FieldName The field name of the map property.
	 * @param Key The key to remove from the map.
	 *
	 * @return True if successfully removed. False otherwise.
	 */
	bool RemoveKeyFromMap(UObject* MapOwner, const FName& FieldName, const FString& Key);

	/**
	 * Remove a key from a map.
	 *
	 * @param MapOwner Address owning the map.
	 * @param PropertyHandle The map property handle.
	 * @param Key The key to remove from the map.
	 *
	 * @return True if successfully removed. False otherwise.
	 */
	bool RemoveKeyFromMap(uint8* MapOwner, TSharedPtr<IPropertyHandle> PropertyHandle, const FString& Key);

	/**
	 * Empties a map.
	 * @param MapOwner Address owning the map.
	 * @param PropertyHandle The map property handle.
	 * @return true if successfully emptied; false otherwise.
	 */
	bool EmptyMap(uint8* MapOwner, TSharedPtr<IPropertyHandle> PropertyHandle);

	/**
	 * Find the index to use with the handle or INDEX_NONE. O(n)
	 * This is necessary because the ArrayIndex of the handle's internal PropertyNode may not match the index or local index of the map.
	 *
	 * @param MapHandle A handle to the map.
	 * @param MapHelper An FScriptMapHelper initialized to the map property for a given instance.
	 * @param Key The key to find the index for.
	 *
	 * @return The index as defined by the MapHandle.
	 */
	int32 FindMapHandleIndexFromKey(TSharedPtr<IPropertyHandleMap> MapHandle, FScriptMapHelper& MapHelper, const FString& Key);
}
