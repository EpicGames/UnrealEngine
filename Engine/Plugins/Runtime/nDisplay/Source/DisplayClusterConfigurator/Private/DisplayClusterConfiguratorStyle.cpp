// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfiguratorStyle.h"

#include "Styling/AppStyle.h"
#include "Framework/Application/SlateApplication.h"
#include "Interfaces/IPluginManager.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateStyleMacros.h"

FDisplayClusterConfiguratorStyle::FDisplayClusterConfiguratorStyle()
	: FSlateStyleSet(TEXT("DisplayClusterConfiguratorStyle"))
{
	Initialize();

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FDisplayClusterConfiguratorStyle::~FDisplayClusterConfiguratorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}

void FDisplayClusterConfiguratorStyle::ReloadTextures()
{
	FSlateApplication::Get().GetRenderer()->ReloadTextureResources();
}

const FLinearColor& FDisplayClusterConfiguratorStyle::GetDefaultColor(uint32 Index)
{
	return DefaultColors[Index % DefaultColors.Num()].Color;
}

#define EDITOR_IMAGE_BRUSH_SVG( RelativePath, ... ) FSlateVectorImageBrush(FPaths::EngineContentDir() / "Editor/Slate" / RelativePath + TEXT(".svg"), __VA_ARGS__ )
#define EDITOR_BOX_BRUSH( RelativePath, ... ) FSlateBoxBrush(FPaths::EngineContentDir() / "Editor/Slate" / RelativePath + TEXT(".png"), __VA_ARGS__ )

const FVector2D Icon128x128(128.0f, 128.0f);
const FVector2D Icon64x64(64.0f, 64.0f);
const FVector2D Icon40x40(40.0f, 40.0f);
const FVector2D Icon20x20(20.0f, 20.0f);
const FVector2D Icon16x16(16.0f, 16.0f);
const FVector2D Icon12x12(12.0f, 12.0f);

void FDisplayClusterConfiguratorStyle::Initialize()
{
	TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(TEXT("nDisplay"));
	check(Plugin.IsValid());
	if (Plugin.IsValid())
	{
		SetContentRoot(FPaths::Combine(Plugin->GetBaseDir(), TEXT("Content")));
		SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));
	}

	// Class Icons
	{
		Set("ClassIcon.DisplayClusterRootActor", new IMAGE_BRUSH_SVG("Icons/RootActor/nDisplay", Icon16x16));
		Set("ClassThumbnail.DisplayClusterRootActor", new IMAGE_BRUSH_SVG("Icons/RootActor/nDisplay_64", Icon64x64));

		Set("ClassIcon.DisplayClusterXformComponent", new IMAGE_BRUSH_SVG("Icons/Components/nDisplayXform", Icon16x16));
		Set("ClassThumbnail.DisplayClusterXformComponent", new IMAGE_BRUSH_SVG("Icons/Components/nDisplayXform", Icon64x64));

		Set("ClassIcon.DisplayClusterScreenComponent", new IMAGE_BRUSH_SVG("Icons/Components/nDisplayScreen", Icon16x16));
		Set("ClassThumbnail.DisplayClusterScreenComponent", new IMAGE_BRUSH_SVG("Icons/Components/nDisplayScreen", Icon64x64));

		Set("ClassIcon.DisplayClusterCameraComponent", new IMAGE_BRUSH_SVG("Icons/Components/nDisplayViewOrigin", Icon16x16));
		Set("ClassThumbnail.DisplayClusterCameraComponent", new IMAGE_BRUSH_SVG("Icons/Components/nDisplayViewOrigin_64", Icon64x64));

		Set("ClassIcon.DisplayClusterICVFXCameraComponent", new IMAGE_BRUSH_SVG("Icons/Components/nDisplayCamera", Icon16x16));
		Set("ClassThumbnail.DisplayClusterICVFXCameraComponent", new IMAGE_BRUSH_SVG("Icons/Components/nDisplayCamera", Icon64x64));

		Set("ClassIcon.DisplayClusterOriginComponent", new IMAGE_BRUSH_SVG("Icons/Components/nDisplayRootComponent", Icon16x16));
		Set("ClassThumbnail.DisplayClusterOriginComponent", new IMAGE_BRUSH_SVG("Icons/Components/nDisplayRootComponent", Icon64x64));

		Set("ClassIcon.DisplayClusterLightCardActor", new IMAGE_BRUSH_SVG("Icons/LightCard/LightCard", Icon16x16));
		Set("ClassThumbnail.DisplayClusterLightCardActor", new IMAGE_BRUSH_SVG("Icons/LightCard/LightCard", Icon64x64));

		Set("ClassIcon.DisplayClusterLightCardActor.UVLightCard", new IMAGE_BRUSH_SVG("Icons/LightCard/LightCardUV", Icon16x16));
		Set("ClassThumbnail.DisplayClusterLightCardActor.UVLightCard", new IMAGE_BRUSH_SVG("Icons/UVLightCard/LightCardUV", Icon64x64));

		Set("ClassIcon.DisplayClusterLightCardActor.Flag", new IMAGE_BRUSH_SVG("Icons/LightCard/LightCardFlag", Icon16x16));
		Set("ClassThumbnail.DisplayClusterLightCardActor.Flag", new IMAGE_BRUSH_SVG("Icons/UVLightCard/LightCardFlag", Icon64x64));

		Set("ClassIcon.DisplayClusterChromakeyCardActor", new IMAGE_BRUSH_SVG("Icons/LightCard/LightCardChromakey", Icon16x16));
		Set("ClassThumbnail.DisplayClusterChromakeyCardActor", new IMAGE_BRUSH_SVG("Icons/UVLightCard/LightCardChromakey", Icon64x64));
	}

	// Config Editor Tabs
	{
		Set("DisplayClusterConfigurator.Tabs.OutputMapping", new IMAGE_BRUSH_SVG("Icons/Tabs/OutputMapping", Icon16x16));
		Set("DisplayClusterConfigurator.Tabs.Cluster", new IMAGE_BRUSH_SVG("Icons/Tabs/Cluster", Icon16x16));
	}


	// Config Editor Toolbar
	{
		Set("DisplayClusterConfigurator.Toolbar.Import", new CORE_IMAGE_BRUSH_SVG("Starship/Common/import", Icon20x20));
		Set("DisplayClusterConfigurator.Toolbar.SaveToFile", new EDITOR_IMAGE_BRUSH_SVG("Starship/Common/Export", Icon20x20));
	}

	// Level Editor Toolbar
	{
		Set("DisplayClusterConfigurator.LevelEditor.ViewportsFrozen", new IMAGE_BRUSH_SVG("Icons/Viewport/nDisplayFrozen_16", Icon16x16));
	}

	// Cluster
	{
		Set("DisplayClusterConfigurator.TreeItems.Cluster", new IMAGE_BRUSH_SVG("Icons/Cluster/Cluster", Icon16x16));
		Set("DisplayClusterConfigurator.TreeItems.Host", new IMAGE_BRUSH_SVG("Icons/Cluster/Host", Icon16x16));
		Set("DisplayClusterConfigurator.TreeItems.ClusterNode", new IMAGE_BRUSH_SVG("Icons/Cluster/ClusterNode", Icon16x16));
		Set("DisplayClusterConfigurator.TreeItems.Viewport", new IMAGE_BRUSH_SVG("Icons/Cluster/Viewport", Icon16x16));
	}

	// Output Mapping Commands
	{
		Set("DisplayClusterConfigurator.OutputMapping.WindowDisplay", new IMAGE_BRUSH_SVG("Icons/OutputMapping/ToggleWindowInfo", Icon16x16));
		Set("DisplayClusterConfigurator.OutputMapping.ToggleOutsideViewports", new IMAGE_BRUSH_SVG("Icons/OutputMapping/ToggleOutsideViewports", Icon16x16));
		Set("DisplayClusterConfigurator.OutputMapping.ZoomToFit", new IMAGE_BRUSH_SVG("Icons/OutputMapping/ZoomToFit", Icon16x16));
		Set("DisplayClusterConfigurator.OutputMapping.ViewScale", new IMAGE_BRUSH_SVG("Icons/OutputMapping/ViewScale", Icon16x16));
		Set("DisplayClusterConfigurator.OutputMapping.Transform", new IMAGE_BRUSH_SVG("Icons/OutputMapping/Transform", Icon16x16));
		Set("DisplayClusterConfigurator.OutputMapping.Snapping", new IMAGE_BRUSH_SVG("Icons/OutputMapping/Snapping", Icon16x16));
		Set("DisplayClusterConfigurator.OutputMapping.ResizeAreaHandle", new IMAGE_BRUSH("Icons/OutputMapping/ResizeAreaHandle_20x", Icon20x20));

		Set("DisplayClusterConfigurator.OutputMapping.RotateViewport90CW", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Rotate90Clockwise", Icon16x16));
		Set("DisplayClusterConfigurator.OutputMapping.RotateViewport90CCW", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Rotate90CounterClockwise", Icon16x16));
		Set("DisplayClusterConfigurator.OutputMapping.RotateViewport180", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Rotate180", Icon16x16));
		Set("DisplayClusterConfigurator.OutputMapping.FlipViewportHorizontal", new CORE_IMAGE_BRUSH_SVG("Starship/Common/FlipHorizontal", Icon16x16));
		Set("DisplayClusterConfigurator.OutputMapping.FlipViewportVertical", new CORE_IMAGE_BRUSH_SVG("Starship/Common/FlipVertical", Icon16x16));
	}

	// Output Mapping Node Colors
	{
		Set("DisplayClusterConfigurator.Node.Color.Regular", FLinearColor(FColor(97, 97, 97)));
		Set("DisplayClusterConfigurator.Node.Color.Selected", FLinearColor(FColor(249, 165, 1)));
		Set("DisplayClusterConfigurator.Node.Color.Locked", FLinearColor(FColor(92, 92, 92, 255)));
		Set("DisplayClusterConfigurator.Node.Color.Regular.Opacity_50", FLinearColor(FColor(255, 255, 255, 255 * 0.5f)));
		Set("DisplayClusterConfigurator.Node.Color.Selected.Opacity_50", FLinearColor(FColor(249, 165, 1, 255 * 0.5f)));

		Set("DisplayClusterConfigurator.Node.Text.Color.Regular", FLinearColor(FColor(255, 255, 255)));
		Set("DisplayClusterConfigurator.Node.Text.Color.Selected", FLinearColor(FColor(0, 0, 0)));

		Set("DisplayClusterConfigurator.Node.Host.Inner.Background", FLinearColor(FColor(38, 38, 38)));

		Set("DisplayClusterConfigurator.Node.Window.Inner.Background", FLinearColor(FColor(53, 53, 53)));
		Set("DisplayClusterConfigurator.Node.Window.Corner.Color", FLinearColor(FColor(47, 47, 47, 255)));

		Set("DisplayClusterConfigurator.Node.Viewport.Border.Color.Regular", FLinearColor(FColor(164, 164, 164, 255)));
		Set("DisplayClusterConfigurator.Node.Viewport.Border.Color.Outside", FLinearColor(FColor(247, 129, 91, 255)));
		Set("DisplayClusterConfigurator.Node.Viewport.Text.Background", FLinearColor(FColor(73, 73, 73, 255 * 0.65f)));
		Set("DisplayClusterConfigurator.Node.Viewport.Text.Background.Locked", FLinearColor(FColor(92, 92, 92, 255)));

		Set("DisplayClusterConfigurator.Node.Viewport.BackgroundColor.Regular", FLinearColor(FColor(155, 155, 155, 255 * 0.85f)));
		Set("DisplayClusterConfigurator.Node.Viewport.BackgroundColor.Selected", FLinearColor(FColor(254, 178, 27, 255)));
		Set("DisplayClusterConfigurator.Node.Viewport.BackgroundImage.Selected", FLinearColor(FColor(255, 204, 102)));
		Set("DisplayClusterConfigurator.Node.Viewport.BackgroundImage.Locked", FLinearColor(FColor(128, 128, 128)));

		Set("DisplayClusterConfigurator.Node.Viewport.OutsideBackgroundColor.Regular", FLinearColor(FColor(255, 87, 34, 255 * 0.85f)));
		Set("DisplayClusterConfigurator.Node.Viewport.OutsideBackgroundColor.Selected", FLinearColor(FColor(255, 87, 34, 255)));
	}

	// Node Text
	{
		FTextBlockStyle TilteTextBlockStyle = FAppStyle::GetWidgetStyle< FTextBlockStyle >("NormalText");
		TilteTextBlockStyle.SetFont(FCoreStyle::GetDefaultFontStyle("Regular", 28));
		Set("DisplayClusterConfigurator.Node.Text.Regular", TilteTextBlockStyle);
	}

	{
		FTextBlockStyle TilteTextBlockStyle = FAppStyle::GetWidgetStyle< FTextBlockStyle >("NormalText");
		TilteTextBlockStyle.SetFont(FCoreStyle::GetDefaultFontStyle("Bold", 28));
		Set("DisplayClusterConfigurator.Node.Text.Bold", TilteTextBlockStyle);
	}

	{
		FTextBlockStyle TilteTextBlockStyle = FAppStyle::GetWidgetStyle<FTextBlockStyle>("Graph.StateNode.NodeTitle");
		TilteTextBlockStyle.SetFont(FCoreStyle::GetDefaultFontStyle("Bold", 36));
		TilteTextBlockStyle.SetShadowColorAndOpacity(FLinearColor::Transparent);
		Set("DisplayClusterConfigurator.Host.Text.Title", TilteTextBlockStyle);
	}

	{
		FTextBlockStyle TilteTextBlockStyle = FAppStyle::GetWidgetStyle< FTextBlockStyle >("NormalText");
		TilteTextBlockStyle.SetFont(FCoreStyle::GetDefaultFontStyle("Italic", 18));
		Set("DisplayClusterConfigurator.Node.Text.Small", TilteTextBlockStyle);
	}

	// Canvas borders
	{
		Set("DisplayClusterConfigurator.Selected.Canvas.Brush", new BORDER_BRUSH(TEXT("Icons/Slate/NonMarchingAnts"), FMargin(0.25f), GetColor("DisplayClusterConfigurator.Node.Color.Selected.Opacity_50")));
		Set("DisplayClusterConfigurator.Regular.Canvas.Brush", new BORDER_BRUSH(TEXT("Icons/Slate/NonMarchingAnts"), FMargin(0.25f), GetColor("DisplayClusterConfigurator.Node.Color.Regular.Opacity_50")));
	}

	// Window borders
	{
		Set("DisplayClusterConfigurator.Node.Brush.Corner", new BOX_BRUSH_SVG("Icons/OutputMapping/CornerImage", 4.0f / 16.0f));

		FSlateBrush* SelectedBrush = new FSlateBrush();
		SelectedBrush->Margin = FMargin(6.f);
		SelectedBrush->DrawAs = ESlateBrushDrawType::Border;
		SelectedBrush->TintColor = GetColor("DisplayClusterConfigurator.Node.Color.Selected");

		Set("DisplayClusterConfigurator.Node.Window.Border.Brush.Selected", SelectedBrush);

		FSlateBrush* RegularBrush = new FSlateBrush();
		RegularBrush->Margin = FMargin(1.f);
		RegularBrush->DrawAs = ESlateBrushDrawType::Border;
		RegularBrush->TintColor = GetColor("DisplayClusterConfigurator.Node.Color.Regular");
		Set("DisplayClusterConfigurator.Node.Window.Border.Brush.Regular", RegularBrush);
	}

	// Viewport borders
	{
		FSlateBrush* SelectedBrush = new FSlateBrush();
		SelectedBrush->Margin = FMargin(2.f);
		SelectedBrush->DrawAs = ESlateBrushDrawType::Border;
		SelectedBrush->TintColor = GetColor("DisplayClusterConfigurator.Node.Color.Selected");

		Set("DisplayClusterConfigurator.Node.Viewport.Border.Brush.Selected", SelectedBrush);

		FSlateBrush* RegularBrush = new FSlateBrush();
		RegularBrush->Margin = FMargin(1.f);
		RegularBrush->DrawAs = ESlateBrushDrawType::Border;
		RegularBrush->TintColor = GetColor("DisplayClusterConfigurator.Node.Viewport.Border.Color.Regular");
		Set("DisplayClusterConfigurator.Node.Viewport.Border.Brush.Regular", RegularBrush);

		FSlateBrush* OutsideBrush = new FSlateBrush();
		OutsideBrush->Margin = FMargin(1.f);
		OutsideBrush->DrawAs = ESlateBrushDrawType::Border;
		OutsideBrush->TintColor = GetColor("DisplayClusterConfigurator.Node.Viewport.Border.Color.Outside");
		Set("DisplayClusterConfigurator.Node.Viewport.Border.Brush.Outside", OutsideBrush);
	}

	// Corner Colors array
	{
		DefaultColors.Add(FCornerColor("DisplayClusterConfigurator.Node.Corner.Color.0", FLinearColor(FColor(244, 67, 54, 255 * 0.8f))));
		DefaultColors.Add(FCornerColor("DisplayClusterConfigurator.Node.Corner.Color.1", FLinearColor(FColor(156, 39, 176, 255 * 0.8f))));
		DefaultColors.Add(FCornerColor("DisplayClusterConfigurator.Node.Corner.Color.2", FLinearColor(FColor(0, 188, 212, 255 * 0.8f))));
		DefaultColors.Add(FCornerColor("DisplayClusterConfigurator.Node.Corner.Color.3", FLinearColor(FColor(139, 195, 74, 255 * 0.8f))));
		DefaultColors.Add(FCornerColor("DisplayClusterConfigurator.Node.Corner.Color.4", FLinearColor(FColor(255, 235, 59, 255 * 0.8f))));
		DefaultColors.Add(FCornerColor("DisplayClusterConfigurator.Node.Corner.Color.5", FLinearColor(FColor(96, 125, 139, 255 * 0.8f))));

		for (const FCornerColor& Color : DefaultColors)
		{
			Set(Color.Name, Color.Color);
		}
	}

	// New Asset Dialog
	{
		const FTextBlockStyle DefaultTextStyle = FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText");

		Set("DisplayClusterConfigurator.NewAssetDialog.OptionText", FTextBlockStyle(DefaultTextStyle).SetFont(DEFAULT_FONT("Bold", 11)));
		Set("DisplayClusterConfigurator.NewAssetDialog.SubBorder", new EDITOR_BOX_BRUSH("Common/GroupBorderLight", FMargin(4.0f / 16.0f)));
		Set("DisplayClusterConfigurator.NewAssetDialog.ActiveOptionBorderColor", FLinearColor(FColor(96, 96, 96)));
	}
}

#undef EDITOR_IMAGE_BRUSH_SVG
#undef EDITOR_BOX_BRUSH
