// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/ISlateStyle.h"
#include "Styling/SlateStyle.h"

class FDisplayClusterConfiguratorStyle : public FSlateStyleSet
{
public:
	FDisplayClusterConfiguratorStyle();
	~FDisplayClusterConfiguratorStyle();

	static FDisplayClusterConfiguratorStyle& Get()
	{
		static FDisplayClusterConfiguratorStyle Inst;
		return Inst;
	}

	static void ReloadTextures();

	const FLinearColor& GetDefaultColor(uint32 Index);

private:
	void Initialize();

	struct FCornerColor
	{
		FCornerColor() 
		{}
		
		FCornerColor(const FName& InName, const FLinearColor& InColor)
			: Name(InName)
			, Color(InColor)
		{}

		FName Name;
		FLinearColor Color;
	};

	TArray<FCornerColor> DefaultColors;
};
