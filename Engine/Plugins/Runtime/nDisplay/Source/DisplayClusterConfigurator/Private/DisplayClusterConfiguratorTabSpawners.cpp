// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfiguratorTabSpawners.h"
#include "DisplayClusterConfiguratorEditorMode.h"
#include "DisplayClusterConfiguratorStyle.h"
#include "DisplayClusterConfiguratorBlueprintEditor.h"
#include "Views/Viewport/DisplayClusterConfiguratorSCSEditorViewport.h"
#include "Views/OutputMapping/IDisplayClusterConfiguratorViewOutputMapping.h"
#include "Views/TreeViews/IDisplayClusterConfiguratorViewTree.h"

#include "BlueprintEditorTabs.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "DisplayClusterConfiguratorTabSpawners"


FDisplayClusterViewSummoner::FDisplayClusterViewSummoner(const FName& InIdentifier, TSharedPtr<FDisplayClusterConfiguratorBlueprintEditor> InHostingApp) : FWorkflowTabFactory(InIdentifier, InHostingApp)
	, BlueprintEditor(InHostingApp)
{
}

FTabSpawnerEntry& FDisplayClusterViewSummoner::RegisterTabSpawner(TSharedRef<FTabManager> TabManager,
	const FApplicationMode* CurrentApplicationMode) const
{
	FTabSpawnerEntry& SpawnerEntry = FWorkflowTabFactory::RegisterTabSpawner(TabManager, nullptr);
	return SpawnerEntry;
}

FDisplayClusterViewOutputMappingSummoner::FDisplayClusterViewOutputMappingSummoner(
	TSharedPtr<FDisplayClusterConfiguratorBlueprintEditor> InHostingApp) : FDisplayClusterViewSummoner(FDisplayClusterConfiguratorEditorConfigurationMode::TabID_OutputMapping, InHostingApp)
{
	TabLabel = LOCTEXT("OutputMappingTabTitle", "OutputMapping");
	TabIcon = FSlateIcon(FDisplayClusterConfiguratorStyle::Get().GetStyleSetName(), "DisplayClusterConfigurator.Tabs.OutputMapping");
	
	ViewMenuDescription = LOCTEXT("OutputMappingTabDescription", "OutputMapping");
	ViewMenuTooltip = LOCTEXT("OutputMapping_ToolTip", "Shows output mappings");
}

TSharedRef<SWidget> FDisplayClusterViewOutputMappingSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	check(BlueprintEditor.IsValid());
	return BlueprintEditor.Pin()->GetViewOutputMapping()->GetWidget();
}

FDisplayClusterViewClusterSummoner::FDisplayClusterViewClusterSummoner(
	TSharedPtr<FDisplayClusterConfiguratorBlueprintEditor> InHostingApp) : FDisplayClusterViewSummoner(FDisplayClusterConfiguratorEditorConfigurationMode::TabID_Cluster, InHostingApp)
{
	TabLabel = LOCTEXT("ClusterTabTitle", "Cluster");
	TabIcon = FSlateIcon(FDisplayClusterConfiguratorStyle::Get().GetStyleSetName(), "DisplayClusterConfigurator.Tabs.Cluster");
	
	ViewMenuDescription = LOCTEXT("ClusterTabDescription", "Cluster view");
	ViewMenuTooltip = LOCTEXT("Cluster_ToolTip", "Shows the cluster view");
}

TSharedRef<SWidget> FDisplayClusterViewClusterSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	check(BlueprintEditor.IsValid());
	return BlueprintEditor.Pin()->GetViewCluster()->GetWidget();
}

FDisplayClusterSCSSummoner::FDisplayClusterSCSSummoner(TSharedPtr<FDisplayClusterConfiguratorBlueprintEditor> InHostingApp) : FDisplayClusterViewSummoner(FDisplayClusterConfiguratorEditorConfigurationMode::TabID_Scene, InHostingApp)
{
	TabLabel = LOCTEXT("ComponentsTabLabel", "Components");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.Tabs.Components");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("ComponentsView", "Components");
	ViewMenuTooltip = LOCTEXT("ComponentsView_ToolTip", "Show the components view");
}

TSharedRef<SWidget> FDisplayClusterSCSSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	check(BlueprintEditor.IsValid());
	return BlueprintEditor.Pin()->GetSCSEditorWrapper().ToSharedRef();
}

FDisplayClusterSCSViewportSummoner::FDisplayClusterSCSViewportSummoner(
	TSharedPtr<FDisplayClusterConfiguratorBlueprintEditor> InHostingApp) : FDisplayClusterViewSummoner(FDisplayClusterConfiguratorEditorConfigurationMode::TabID_Viewport, InHostingApp)
{
	TabLabel = LOCTEXT("ViewportTabTitle", "Preview");
	TabIcon = FSlateIcon(FSlateIcon(FAppStyle::Get().GetStyleSetName(), "LevelEditor.Tabs.Viewports"));

	ViewMenuDescription = LOCTEXT("ViewportTabDescription", "Viewport");
	ViewMenuTooltip = LOCTEXT("Viewport_ToolTip", "Shows the viewport");
}

TSharedRef<SDockTab> FDisplayClusterSCSViewportSummoner::SpawnTab(const FWorkflowTabSpawnInfo& Info) const
{
	check(BlueprintEditor.IsValid());
	BlueprintEditor.Pin()->CreateDCSCSEditors();

	auto OnClosed = [this](TSharedRef<SDockTab>)
	{
		if (BlueprintEditor.IsValid())
		{
			BlueprintEditor.Pin()->ShutdownDCSCSEditors();
		}
	};

	TSharedRef<SDockTab> Tab = BlueprintEditor.Pin()->GetViewportTab().ToSharedRef();
	Tab->SetTag(GetIdentifier());
	Tab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateLambda(OnClosed));
	return Tab;
}

#undef LOCTEXT_NAMESPACE
