// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintEditorModes.h"

class FDisplayClusterConfiguratorBlueprintEditor;

struct FDisplayClusterViewSummoner : public FWorkflowTabFactory
{
	FDisplayClusterViewSummoner(const FName& InIdentifier, TSharedPtr<FDisplayClusterConfiguratorBlueprintEditor> InHostingApp);

	virtual FTabSpawnerEntry& RegisterTabSpawner(TSharedRef<FTabManager> TabManager, const FApplicationMode* CurrentApplicationMode) const override;

	TWeakPtr<FDisplayClusterConfiguratorBlueprintEditor> BlueprintEditor;
};

struct FDisplayClusterViewOutputMappingSummoner : public FDisplayClusterViewSummoner
{
	FDisplayClusterViewOutputMappingSummoner(TSharedPtr<FDisplayClusterConfiguratorBlueprintEditor> InHostingApp);

	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
};

struct FDisplayClusterViewClusterSummoner : public FDisplayClusterViewSummoner
{
	FDisplayClusterViewClusterSummoner(TSharedPtr<FDisplayClusterConfiguratorBlueprintEditor> InHostingApp);

	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
};

struct FDisplayClusterSCSSummoner : public FDisplayClusterViewSummoner
{
	FDisplayClusterSCSSummoner(TSharedPtr<FDisplayClusterConfiguratorBlueprintEditor> InHostingApp);

	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
};

struct FDisplayClusterSCSViewportSummoner : public FDisplayClusterViewSummoner
{
	FDisplayClusterSCSViewportSummoner(TSharedPtr<FDisplayClusterConfiguratorBlueprintEditor> InHostingApp);

	virtual TSharedRef<SDockTab> SpawnTab(const FWorkflowTabSpawnInfo& Info) const override;
};
