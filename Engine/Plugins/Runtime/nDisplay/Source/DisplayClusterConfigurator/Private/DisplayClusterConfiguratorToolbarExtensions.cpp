// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfiguratorToolbarExtensions.h"

#include "DisplayClusterConfiguratorStyle.h"
#include "DisplayClusterRootActor.h"
#include "ScopedTransaction.h"
#include "ToolMenus.h"
#include "Styling/SlateIconFinder.h"

#define LOCTEXT_NAMESPACE "FDisplayClusterConfiguratorToolbarExtensions"

void FDisplayClusterConfiguratorToolbarExtensions::RegisterToolbarExtensions()
{
	FToolMenuOwnerScoped ToolMenuOwnerScoped(this);

	UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("LevelEditor.ViewportToolBar.PerformanceAndScalability");
	FToolMenuSection& Section = Menu->FindOrAddSection("FreezeNDisplayViewports", LOCTEXT("FreezeNDisplayViewportsSectionLabel", "Freeze nDisplay Viewports"));
	Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateSP(this, &FDisplayClusterConfiguratorToolbarExtensions::CreateFreezeViewportsMenu));
	Section.Visibility = TAttribute<bool>::CreateLambda([this]()
	{
		// Check if there are any nDisplay stages in the level, as the menu section should only show up if there are nDisplay stages
		if (GEditor)
		{
			if (const UWorld* World = GEditor->GetEditorWorldContext().World())
			{
				if (TActorIterator<ADisplayClusterRootActor> It(World); It)
				{
					return true;
				}
			}
		}

		return false;
	});
}

void FDisplayClusterConfiguratorToolbarExtensions::UnregisterToolbarExtensions()
{
}

void FDisplayClusterConfiguratorToolbarExtensions::CreateFreezeViewportsMenu(FToolMenuSection& InSection)
{
	if (GEditor)
	{
		FToolMenuEntry UnfreezeAllEntry = FToolMenuEntry::InitMenuEntry(
			FName("UnfreezeAllViewports"),
			LOCTEXT("UnfreezeAllViewportsLabel", "Unfreeze All Viewports"),
			LOCTEXT("ViewportsFrozenWarningToolTip", "nDisplay viewports are frozen. Click to unfreeze all frozen viewports."),
			FSlateIcon(FDisplayClusterConfiguratorStyle::Get().GetStyleSetName(), "DisplayClusterConfigurator.LevelEditor.ViewportsFrozen"),
			FUIAction(
				FExecuteAction::CreateSP(this, &FDisplayClusterConfiguratorToolbarExtensions::UnfreezeAllViewports),
				FCanExecuteAction::CreateSP(this, &FDisplayClusterConfiguratorToolbarExtensions::AreAnyViewportsFrozen)),
			EUserInterfaceActionType::Button);

		UnfreezeAllEntry.SetShowInToolbarTopLevel(TAttribute<bool>::CreateLambda(
			[this]() -> bool
			{
				return AreAnyViewportsFrozen();
			}
		));

		UnfreezeAllEntry.StyleNameOverride = "ViewportToolbarWarning";
		InSection.AddEntry(UnfreezeAllEntry);

		if (const UWorld* World = GEditor->GetEditorWorldContext().World())
		{
			for (TActorIterator<ADisplayClusterRootActor> It(World); It; ++It)
			{
				TWeakObjectPtr<ADisplayClusterRootActor> WeakPtr = *It;
				FString RootActorLabel = It->GetActorNameOrLabel();
				InSection.AddEntry(FToolMenuEntry::InitMenuEntry(
					FName("ToggleFreezeViewports_" + RootActorLabel),
					FText::FromString(RootActorLabel),
					LOCTEXT("ToggleFreezeViewportsTooltip", "Toggles whether this stage's viewports are frozen or not"),
					FSlateIconFinder::FindIconForClass(ADisplayClusterRootActor::StaticClass()),
					FUIAction(
						FExecuteAction::CreateSP(this, &FDisplayClusterConfiguratorToolbarExtensions::ToggleFreezeViewports, WeakPtr),
						FCanExecuteAction(),
						FIsActionChecked::CreateSP(this, &FDisplayClusterConfiguratorToolbarExtensions::AreViewportsFrozen, WeakPtr)),
					EUserInterfaceActionType::ToggleButton
				));
			}
		}
	}
}

void FDisplayClusterConfiguratorToolbarExtensions::UnfreezeAllViewports()
{
	if (GEditor)
	{
		FScopedTransaction Transaction(LOCTEXT("UnfreezeViewports", "Unfreeze viewports"));
		
		for (TActorIterator<ADisplayClusterRootActor> It(GEditor->GetEditorWorldContext().World()); It; ++It)
		{
			It->SetFreezeOuterViewports(false);
		}
	}
}

bool FDisplayClusterConfiguratorToolbarExtensions::AreAnyViewportsFrozen() const
{
	if (GEditor)
	{
		if (const UWorld* World = GEditor->GetEditorWorldContext().World())
		{
			for (TActorIterator<ADisplayClusterRootActor> It(World); It; ++It)
			{
				const UDisplayClusterConfigurationData* ConfigData = It->GetConfigData();
				if (ConfigData && ConfigData->StageSettings.bFreezeRenderOuterViewports)
				{
					return true;
				}
			}
		}
	}

	return false;
}

void FDisplayClusterConfiguratorToolbarExtensions::ToggleFreezeViewports(TWeakObjectPtr<ADisplayClusterRootActor> InRootActor)
{
	if (!InRootActor.IsValid())
	{
		return;
	}

	const UDisplayClusterConfigurationData* ConfigData = InRootActor->GetConfigData();
	if (!ConfigData)
	{
		return;
	}

	FScopedTransaction Transaction(FText::Format(LOCTEXT("ToggleFreezeViewportsTransaction", "Toggle Freeze Viewports for stage '{0}'"), FText::FromString(InRootActor->GetActorNameOrLabel())));
	InRootActor->SetFreezeOuterViewports(!ConfigData->StageSettings.bFreezeRenderOuterViewports);
}

bool FDisplayClusterConfiguratorToolbarExtensions::AreViewportsFrozen(TWeakObjectPtr<ADisplayClusterRootActor> InRootActor) const
{
	if (!InRootActor.IsValid())
	{
		return false;
	}

	const UDisplayClusterConfigurationData* ConfigData = InRootActor->GetConfigData();
	if (!ConfigData)
	{
		return false;
	}

	return ConfigData->StageSettings.bFreezeRenderOuterViewports;
}

#undef LOCTEXT_NAMESPACE
