// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtrTemplates.h"

struct FToolMenuSection;
class ADisplayClusterRootActor;

/** Manages any toolbar extensions the nDisplay plugin needs to register with the editor */
class FDisplayClusterConfiguratorToolbarExtensions : public TSharedFromThis<FDisplayClusterConfiguratorToolbarExtensions>
{
public:
	/** Registers any toolbar extensions with the editor */
	void RegisterToolbarExtensions();

	/** Unregisters all registered toolbar extensions */
	void UnregisterToolbarExtensions();

private:
	void CreateFreezeViewportsMenu(FToolMenuSection& InSection);

	void UnfreezeAllViewports();
	bool AreAnyViewportsFrozen() const;
	
	void ToggleFreezeViewports(TWeakObjectPtr<ADisplayClusterRootActor> InRootActor);
	bool AreViewportsFrozen(TWeakObjectPtr<ADisplayClusterRootActor> InRootActor) const;
};
