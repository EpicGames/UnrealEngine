// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfiguratorUtils.h"
#include "DisplayClusterConfiguratorBlueprintEditor.h"
#include "DisplayClusterConfiguratorFactory.h"
#include "DisplayClusterConfiguratorLog.h"

#include "DisplayClusterRootActor.h"
#include "Blueprints/DisplayClusterBlueprint.h"
#include "Blueprints/DisplayClusterBlueprintGeneratedClass.h"

#include "IDisplayClusterConfiguration.h"
#include "DisplayClusterConfigurationTypes.h"

#include "Components/DisplayClusterCameraComponent.h"
#include "Components/DisplayClusterSyncTickComponent.h"

#include "Engine/SCS_Node.h"
#include "Engine/SimpleConstructionScript.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Widgets/Notifications/SNotificationList.h"


#define LOCTEXT_NAMESPACE "DisplayClusterConfiguratorUtils"


ADisplayClusterRootActor* FDisplayClusterConfiguratorUtils::GenerateRootActorFromConfigFile(const FString& InFilename)
{
	UDisplayClusterConfigurationData* NewConfig = IDisplayClusterConfiguration::Get().LoadConfig(InFilename);
	if (!NewConfig)
	{
		return nullptr;
	}
	NewConfig->PathToConfig = InFilename;
	return GenerateRootActorFromConfigData(NewConfig);
}

ADisplayClusterRootActor* FDisplayClusterConfiguratorUtils::GenerateRootActorFromConfigData(
	UDisplayClusterConfigurationData* ConfigData)
{
	ADisplayClusterRootActor* DefaultObject = NewObject<ADisplayClusterRootActor>();
	DefaultObject->InitializeFromConfig(ConfigData);
	
	return DefaultObject;
}

UDisplayClusterBlueprint* FDisplayClusterConfiguratorUtils::CreateBlueprintFromRootActor(
	ADisplayClusterRootActor* RootActor, const FName& BlueprintName, UObject* Package)
{
	check(RootActor);
	check(Package);

	const FString BlueprintNameStr = BlueprintName.ToString();
	if (UObject* ExistingObject = FindObject<UObject>(Package, *BlueprintNameStr))
	{
		FNotificationInfo Info(FText::Format(LOCTEXT("FileExists", "Cannot create a new nDisplay blueprint with the name '{0}'. The file already exists."),
			FText::FromName(BlueprintName)));

		UE_LOGF(DisplayClusterConfiguratorLog, Error, "Cannot create a new nDisplay blueprint with the name '%ls'. The file already exists. To re-import right click on the file and use the context option instead.",
			*BlueprintNameStr);
		
		Info.bUseLargeFont = false;
		Info.ExpireDuration = 5.0f;

		const TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info);
		if (Notification.IsValid())
		{
			Notification->SetCompletionState(SNotificationItem::CS_Fail);
		}

		return nullptr;
	}

	UDisplayClusterBlueprint* Blueprint = CastChecked<UDisplayClusterBlueprint>(FKismetEditorUtilities::CreateBlueprint(ADisplayClusterRootActor::StaticClass(),
		Package,
		BlueprintName,
		EBlueprintType::BPTYPE_Normal,
		UDisplayClusterBlueprint::StaticClass(),
		UDisplayClusterBlueprintGeneratedClass::StaticClass(),
		FName("CreateBlueprintFromRootActor")));

	AddRootActorComponentsToBlueprint(Blueprint, RootActor, false);
	UDisplayClusterConfiguratorFactory::SetupInitialBlueprintDocuments(Blueprint);
	
	return Cast<UDisplayClusterBlueprint>(Blueprint);
}

void FDisplayClusterConfiguratorUtils::AddRootActorComponentsToBlueprint(UDisplayClusterBlueprint* Blueprint,
                                                                         ADisplayClusterRootActor* RootActor, const bool bCompile, USCS_Node* NewRootNode)
{
	auto RemoveImplComponent = [&](UActorComponent* InComponent, TArray<UActorComponent*>& CompArray)
	{
		const FString VisCompName = InComponent->GetName() + GetImplSuffix();
		if (UActorComponent* VisComponent = FindObjectFast<UActorComponent>(InComponent, *VisCompName))
		{
			// The visualization component is already a default subobject of the root component.
			CompArray.Remove(VisComponent);
		}
	};
	
	TArray<UActorComponent*> AllComponents = RootActor->GetComponents().Array();
	TArray<UActorComponent*> AllComponentsCopy(AllComponents);
	
	for (UActorComponent* Comp : AllComponentsCopy)
	{
		RemoveImplComponent(Comp, AllComponents);
	}
	
	// Already added as default subobjects.
	AllComponents.Remove(RootActor->GetRootComponent());
	AllComponents.Remove(RootActor->GetSyncTickComponent());
	AllComponents.Remove(RootActor->GetDefaultCamera());
	
	if (NewRootNode == nullptr)
	{
		// If this is being re-imported we should clear out existing components.
		RemoveAllComponentsFromBlueprint(Blueprint);
		NewRootNode = Blueprint->SimpleConstructionScript->GetDefaultSceneRootNode();
	}

	FKismetEditorUtilities::FAddComponentsToBlueprintParams AddCompParams;
	AddCompParams.bKeepMobility = true;
	AddCompParams.HarvestMode = FKismetEditorUtilities::EAddComponentToBPHarvestMode::Harvest_UseComponentName;
	AddCompParams.OptionalNewRootNode = NewRootNode;

	FKismetEditorUtilities::AddComponentsToBlueprint(Blueprint, AllComponents, AddCompParams);

	if (bCompile)
	{
		FKismetEditorUtilities::CompileBlueprint(Blueprint);
	}
}

void FDisplayClusterConfiguratorUtils::RemoveAllComponentsFromBlueprint(UBlueprint* Blueprint)
{
	TArray<USCS_Node*> AllNodes = Blueprint->SimpleConstructionScript->GetAllNodes();
	for (USCS_Node* Node : AllNodes)
	{
		RemoveAllChildrenNodes(Node);
		Blueprint->SimpleConstructionScript->RemoveNode(Node);
	}
}

void FDisplayClusterConfiguratorUtils::RemoveAllChildrenNodes(USCS_Node* InNode)
{
	TArray<USCS_Node*> ChildrenNodes = InNode->GetChildNodes();
	for (USCS_Node* Node : ChildrenNodes)
	{
		RemoveAllChildrenNodes(Node);
		InNode->RemoveChildNode(Node);
	}
}

UDisplayClusterBlueprint* FDisplayClusterConfiguratorUtils::FindBlueprintFromObject(UObject* InObject)
{
	for (UObject* Owner = InObject; Owner; Owner = Owner->GetOuter())
	{
		if (UDisplayClusterBlueprint* Blueprint = Cast<UDisplayClusterBlueprint>(Owner))
		{
			return Blueprint;
		}
		if (UDisplayClusterBlueprintGeneratedClass* GeneratedClass = Cast<UDisplayClusterBlueprintGeneratedClass>(Owner))
		{
			if (UBlueprint* Blueprint = UBlueprint::GetBlueprintFromClass(GeneratedClass))
			{
				return Cast<UDisplayClusterBlueprint>(Blueprint);
			}
		}
		else if (UBlueprint* Blueprint = UBlueprint::GetBlueprintFromClass(Owner->GetClass()))
		{
			return Cast<UDisplayClusterBlueprint>(Blueprint);
		}
	}

	return nullptr;
}

FDisplayClusterConfiguratorBlueprintEditor* FDisplayClusterConfiguratorUtils::GetBlueprintEditorForObject(
	UObject* InObject)
{
	if (UDisplayClusterBlueprint* Blueprint = FindBlueprintFromObject(InObject))
	{
		return static_cast<FDisplayClusterConfiguratorBlueprintEditor*>(GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->
			FindEditorForAsset(Blueprint, false));
	}

	return nullptr;
}

bool FDisplayClusterConfiguratorUtils::IsPrimaryNodeInConfig(UDisplayClusterConfigurationData* ConfigData)
{
	return ConfigData && ConfigData->Cluster && ConfigData->Cluster->Nodes.Num() > 0 && !ConfigData->Cluster->PrimaryNode.Id.IsEmpty();
}

void FDisplayClusterConfiguratorUtils::MarkDisplayClusterBlueprintAsModified(UObject* InObject, const bool bIsStructuralChange)
{
	if (UDisplayClusterBlueprint* Blueprint = FindBlueprintFromObject(InObject))
	{
		if (bIsStructuralChange)
		{
			FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
		}
		else
		{
			FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
		}
	}
}

FName FDisplayClusterConfiguratorUtils::CreateUniqueName(const FName& TargetName, UClass* ObjectClass, UObject* ObjectOuter)
{
	FString CandidateName = TargetName.ToString();
	
	if (StaticFindObject(ObjectClass, ObjectOuter, *CandidateName) == nullptr)
	{
		return TargetName;
	}
	
	FString BaseCandidateName;
	int32 LastUnderscoreIndex;
	int32 NameIndex = 0;
	if (CandidateName.FindLastChar('_', LastUnderscoreIndex) && LexTryParseString(NameIndex, *CandidateName.RightChop(LastUnderscoreIndex + 1)))
	{
		BaseCandidateName = CandidateName.Left(LastUnderscoreIndex);
		NameIndex++;
	}
	else
	{
		BaseCandidateName = CandidateName;
		NameIndex = 1;
	}

	FString UniqueCandidateName = FString::Printf(TEXT("%s_%02i"), *BaseCandidateName, NameIndex);
	while (StaticFindObject(ObjectClass, ObjectOuter, *UniqueCandidateName) != nullptr)
	{
		NameIndex++;
		UniqueCandidateName = FString::Printf(TEXT("%s_%02i"), *BaseCandidateName, NameIndex);
	}
	return *UniqueCandidateName;
}

FString FDisplayClusterConfiguratorUtils::FormatNDisplayComponentName(UClass* ComponentClass)
{
	check(ComponentClass);
	return ComponentClass->GetDisplayNameText().ToString().Replace(TEXT(" "), TEXT("")).Replace(TEXT("NDisplay"), TEXT("nDisplay"));
}

#undef LOCTEXT_NAMESPACE
