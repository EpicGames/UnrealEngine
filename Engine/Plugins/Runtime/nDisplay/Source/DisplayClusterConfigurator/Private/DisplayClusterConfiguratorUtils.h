// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class USCS_Node;
class ADisplayClusterRootActor;
class UDisplayClusterBlueprint;
class UDisplayClusterConfigurationData;
class UDisplayClusterConfigurationViewport;
class UBlueprint;
class FDisplayClusterConfiguratorBlueprintEditor;

class FDisplayClusterConfiguratorUtils
{

public:
	/** Loads a config file into memory and generates a root actor from it. */
	static ADisplayClusterRootActor* GenerateRootActorFromConfigFile(const FString& InFilename);
	
	/** Assemble a root actor with components generated. */
	static ADisplayClusterRootActor* GenerateRootActorFromConfigData(UDisplayClusterConfigurationData* ConfigData);
	
	/** Construct a blueprint with components from an assembled root actor. */
	static UDisplayClusterBlueprint* CreateBlueprintFromRootActor(ADisplayClusterRootActor* RootActor, const FName& BlueprintName, UObject* Package);

	/** Harvest the root actor's dynamic components into an existing blueprint. */
	static void AddRootActorComponentsToBlueprint(UDisplayClusterBlueprint* Blueprint, ADisplayClusterRootActor* RootActor, const bool bCompile = true, USCS_Node* NewRootNode = nullptr);

	/** Recursively remove all components from a blueprint. */
	static void RemoveAllComponentsFromBlueprint(UBlueprint* Blueprint);

	/** Recursively remove all nodes from a node. */
	static void RemoveAllChildrenNodes(USCS_Node* InNode);

	/** Search up the outer objects for a DisplayCluster Blueprint. */
	static UDisplayClusterBlueprint* FindBlueprintFromObject(UObject* InObject);

	/** Locate an open Blueprint Editor for the object. */
	static FDisplayClusterConfiguratorBlueprintEditor* GetBlueprintEditorForObject(UObject* InObject);

	/** Checks if a primary node is present in a config. */
	static bool IsPrimaryNodeInConfig(UDisplayClusterConfigurationData* ConfigData);
	
	/**
	 * Find the blueprint associated with the object and signal it should be recompiled.
	 *
	 * @param InObject An object that is the blueprint or owned by the blueprint.
	 * @param bIsStructuralChange Perform a skeleton recompile of the blueprint.
	 */
	static void MarkDisplayClusterBlueprintAsModified(UObject* InObject, const bool bIsStructuralChange);
	
	/** Calculate and return a unique name for a given object outer. */
	static FName CreateUniqueName(const FName& TargetName, UClass* ObjectClass, UObject* ObjectOuter);

	/** Return the "_impl" suffix used for visualization components. */
	static FString GetImplSuffix() { return TEXT("_impl"); }

	/** Format an nDisplay component name correctly. */
	static FString FormatNDisplayComponentName(UClass* ComponentClass);
};
