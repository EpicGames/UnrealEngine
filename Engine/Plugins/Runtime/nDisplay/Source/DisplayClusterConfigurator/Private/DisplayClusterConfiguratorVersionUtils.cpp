// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfiguratorVersionUtils.h"
#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterConfiguratorBlueprintEditor.h"
#include "DisplayClusterConfiguratorEditorSubsystem.h"
#include "DisplayClusterConfiguratorLog.h"

#include "IDisplayClusterConfiguration.h"

#include "DisplayClusterRootActor.h"
#include "Blueprints/DisplayClusterBlueprint.h"

#include "ObjectTools.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Editor.h"
#include "Editor/UnrealEdEngine.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Misc/ScopedSlowTask.h"
#include "Settings/DisplayClusterConfiguratorSettings.h"
#include "Widgets/Notifications/SNotificationList.h"


#define LOCTEXT_NAMESPACE "DisplayClusterConfiguratorVersionUtils"

static TWeakPtr<SNotificationItem> WrongVersionNotification;
static TWeakPtr<SNotificationItem> ErrorUpdatingAssetsNotification;

void FDisplayClusterConfiguratorVersionUtils::UpdateBlueprintsToNewVersion()
{
	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(FName("AssetRegistry")).Get();

	TArray<FAssetData> OutAssets;
	AssetRegistry.GetAssetsByClass(UDisplayClusterBlueprint::StaticClass()->GetClassPathName(), OutAssets, true);
	AssetRegistry.GetAssetsByClass(UDisplayClusterConfiguratorEditorData::StaticClass()->GetClassPathName(), OutAssets, true);
	
	bool bIsAssetVersionNotSupported = false;

	TArray<FAssetData> AssetsToUpdate;
	for (const FAssetData& Asset : OutAssets)
	{
		int32 Version;

		const bool bVersionFound = Asset.GetTagValue(GET_MEMBER_NAME_CHECKED(UDisplayClusterBlueprint, AssetVersion), Version);
		if (!bVersionFound || !IsBlueprintUpToDate(Version))
		{
			AssetsToUpdate.Add(Asset);
			continue;
		}

		if (IsBlueprintFromNewerPluginVersion(Version))
		{
			// This will display a warning and assets will be 'downgraded'. May not be supported at all in future versions.
			bIsAssetVersionNotSupported = true;
		}
	}

	if (bIsAssetVersionNotSupported)
	{
		// There are assets from a newer version of the plugin.
		FNotificationInfo Info(LOCTEXT("DisplayClusterAssetsFromNewerVersion", "nDisplay assets are from a newer version of the plugin!\nPlease update nDisplay."));
		Info.bFireAndForget = false;
		Info.bUseLargeFont = false;
		Info.bUseThrobber = false;
		Info.FadeOutDuration = 0.25f;
		Info.ButtonDetails.Add(FNotificationButtonInfo(LOCTEXT("DisplayClusterWrongVersionDismiss", "Dismiss"), LOCTEXT("DisplayClusterWrongVersionDismissTT", "Dismiss this notification"),
			FSimpleDelegate::CreateStatic(&FDisplayClusterConfiguratorVersionUtils::DismissWrongVersionNotification)));

		WrongVersionNotification = FSlateNotificationManager::Get().AddNotification(Info);
		WrongVersionNotification.Pin()->SetCompletionState(SNotificationItem::CS_Pending);
	}

	if (AssetsToUpdate.Num() > 0)
	{
		FScopedSlowTask Feedback(AssetsToUpdate.Num(), NSLOCTEXT("DisplayCluster", "DisplayClusterAssetUpdate", "Updating nDisplay assets to the current version..."));
		bool bHasErrors = false;
		
		const UDisplayClusterConfiguratorEditorSettings* Settings = GetDefault<UDisplayClusterConfiguratorEditorSettings>();
		if (Settings->bDisplayAssetUpdateProgress)
		{
			Feedback.MakeDialog(true);
		}
		
		for (FAssetData& Asset : AssetsToUpdate)
		{
			UDisplayClusterBlueprint* Blueprint = Cast<UDisplayClusterBlueprint>(Asset.GetAsset());
			if (Blueprint)
			{
				// Nothing yet, this is if we are updating a 4.27 blueprint to a future version.
				
			}
			else if (UDisplayClusterConfiguratorEditorData* Data = Cast<UDisplayClusterConfiguratorEditorData>(Asset.GetAsset()))
			{
				// 4.26 config assets. These are an old type that need to be recreated as a blueprint.
				if (!Data->bConvertedToBlueprint)
				{
					UDisplayClusterConfiguratorEditorSubsystem* EditorSubsystem = GEditor->GetEditorSubsystem<UDisplayClusterConfiguratorEditorSubsystem>();
					check(EditorSubsystem);

					UPackage* Package = Data->GetPackage();
					FString PackageName = Package->GetName();
					FName ObjectName = Data->GetFName();
					
					{
						// Create a new blueprint asset first.
						// If this fails we won't delete the original asset.
						FString NewBaseName = PackageName + "_BlueprintConverted";
						FName NewObjectName = *NewBaseName;
						FString NewPackageName = NewBaseName;

						int32 Count = 0;
						while (FindPackage(nullptr, *NewPackageName) != nullptr)
						{
							NewPackageName = NewBaseName + "_" + FString::FromInt(Count++);
							NewObjectName = *NewPackageName;
						}
						
						Package = CreatePackage(*NewPackageName);
						Package->MarkAsFullyLoaded();

						const FString ConfigPath = Data->PathToConfig;
						Blueprint = EditorSubsystem->ImportAsset(Package, NewObjectName, ConfigPath);
					}

					if (Blueprint)
					{
						FAssetRegistryModule::AssetCreated(Blueprint);

						// New blueprint created, try deleting the original asset.
						const bool bObjectDeleted = ObjectTools::DeleteSingleObject(Data, true);
						if (bObjectDeleted)
						{
							// Cleans up the package.
							CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
							
							Package->Rename(*PackageName, nullptr, REN_DontCreateRedirectors);
							Blueprint->Rename(*ObjectName.ToString(), nullptr, REN_DontCreateRedirectors);
						}
						else
						{
							// Can't delete old package, mark this asset as already imported.
							Data->bConvertedToBlueprint = true;
							Data->MarkPackageDirty();
						}
					}
				}
			}

			if (Blueprint)
			{
				SetToLatestVersion(Blueprint);
				Blueprint->MarkPackageDirty();
			}
			else
			{
				UE_LOGF(DisplayClusterConfiguratorLog, Error, "Could not update asset: %ls", *Asset.AssetName.ToString());
				bHasErrors = true;
			}
			
			Feedback.CompletedWork += 1;
		}

		if (bHasErrors)
		{
			FNotificationInfo Info(LOCTEXT("DisplayClusterFailedToUpdateAssets", "Failed to update one or more nDisplay assets. See the output log for errors."));
			Info.bFireAndForget = false;
			Info.bUseLargeFont = false;
			Info.bUseThrobber = false;
			Info.FadeOutDuration = 0.25f;
			Info.ButtonDetails.Add(FNotificationButtonInfo(LOCTEXT("DisplayClusterUpdateFailedDismiss", "Dismiss"), LOCTEXT("DisplayClusterUpdateFailedTT", "Dismiss this notification"),
				FSimpleDelegate::CreateStatic(&FDisplayClusterConfiguratorVersionUtils::DismissFailedUpdateNotification)));

			ErrorUpdatingAssetsNotification = FSlateNotificationManager::Get().AddNotification(Info);
			ErrorUpdatingAssetsNotification.Pin()->SetCompletionState(SNotificationItem::CS_Pending);
		}
	}
}

bool FDisplayClusterConfiguratorVersionUtils::IsBlueprintUpToDate(int32 CompareVersion)
{
	return CompareVersion >= GetCurrentBlueprintVersion();
}

bool FDisplayClusterConfiguratorVersionUtils::IsBlueprintUpToDate(UDisplayClusterBlueprint* Blueprint)
{
	const int32 Version = Blueprint->AssetVersion;
	return IsBlueprintUpToDate(Version);
}

bool FDisplayClusterConfiguratorVersionUtils::IsBlueprintFromNewerPluginVersion(int32 CompareVersion)
{
	return CompareVersion > GetCurrentBlueprintVersion();
}

void FDisplayClusterConfiguratorVersionUtils::SetToLatestVersion(UBlueprint* Blueprint)
{
	if (UDisplayClusterBlueprint* DCBlueprint = Cast<UDisplayClusterBlueprint>(Blueprint))
	{
		DCBlueprint->AssetVersion = GetCurrentBlueprintVersion();
	}
}

void FDisplayClusterConfiguratorVersionUtils::DismissWrongVersionNotification()
{
	WrongVersionNotification.Pin()->ExpireAndFadeout();
}

void FDisplayClusterConfiguratorVersionUtils::DismissFailedUpdateNotification()
{
	ErrorUpdatingAssetsNotification.Pin()->ExpireAndFadeout();
}

#undef LOCTEXT_NAMESPACE
