// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DisplayClusterConfiguratorVersionUtils.generated.h"

class ADisplayClusterRootActor;
class UDisplayClusterBlueprint;
class UDisplayClusterConfigurationData;
class UDisplayClusterConfigurationViewport;
class UBlueprint;
class FDisplayClusterConfiguratorBlueprintEditor;

#define DISPLAYCLUSTER_BP_VERSION 2

/**
 * The original format of the DisplayCluster config data UAsset, used only for importing 4.26 assets.
 */
UCLASS(NotBlueprintable)
class UDisplayClusterConfiguratorEditorData final
	: public UObject
{
	GENERATED_BODY()

public:
	UDisplayClusterConfiguratorEditorData() {}

public:
	UPROPERTY(Transient)
	TObjectPtr<class UDisplayClusterConfigurationData> nDisplayConfig;

	UPROPERTY()
	FString PathToConfig;

	/**
	 * True if the original asset is imported but could not be deleted.
	 */
	UPROPERTY()
	bool bConvertedToBlueprint;
};

class FDisplayClusterConfiguratorVersionUtils
{
public:
	/**
	 * Display cluster blueprints are saved with this version number.
	 * On plugin load this version is checked against the asset version.
	 */
	static int32 GetCurrentBlueprintVersion() { return DISPLAYCLUSTER_BP_VERSION; }
	
	/** Check all display cluster blueprints and update to a new version if necessary. */
	static void UpdateBlueprintsToNewVersion();

	/** Checks if the blueprint needs an update. */
	static bool IsBlueprintUpToDate(int32 CompareVersion);

	/** Checks if a blueprint is up to date. */
	static bool IsBlueprintUpToDate(UDisplayClusterBlueprint* Blueprint);

	/** Checks if the blueprint is from a new plugin version than installed. */
	static bool IsBlueprintFromNewerPluginVersion(int32 CompareVersion);
	
	/** Sets the version tag of the asset. */
	static void SetToLatestVersion(UBlueprint* Blueprint);
	
private:
	static void DismissWrongVersionNotification();
	static void DismissFailedUpdateNotification();
};
