// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfiguratorMPCDIImporter.h"
#include "DisplayClusterConfiguratorLog.h"

#include "DisplayClusterProjectionStrings.h"
#include "IDisplayClusterWarp.h"
#include "Blueprints/DisplayClusterBlueprint.h"
#include "Blueprints/DisplayClusterBlueprintGeneratedClass.h"
#include "ClusterConfiguration/DisplayClusterConfiguratorClusterUtils.h"
#include "ClusterConfiguration/ViewModels/DisplayClusterConfiguratorClusterNodeViewModel.h"
#include "ClusterConfiguration/ViewModels/DisplayClusterConfiguratorProjectionPolicyViewModel.h"
#include "ClusterConfiguration/ViewModels/DisplayClusterConfiguratorViewportViewModel.h"
#include "Components/DisplayClusterCameraComponent.h"
#include "Components/DisplayClusterScreenComponent.h"
#include "Engine/SCS_Node.h"
#include "Engine/SimpleConstructionScript.h"
#include "Misc/DisplayClusterHelpers.h"
#include "Misc/DisplayClusterProjectionHelpers.h"

bool FDisplayClusterConfiguratorMPCDIImporter::ImportMPCDIIntoBlueprint(const FString& InFilePath, UDisplayClusterBlueprint* InBlueprint, const FDisplayClusterConfiguratorMPCDIImporterParams& InParams)
{
	const FString MPCDIFileFullPath = DisplayClusterHelpers::filesystem::GetFullPathForConfigResource(InFilePath);
	if (MPCDIFileFullPath.IsEmpty())
	{
		UE_LOGF(DisplayClusterConfiguratorLog, Error, "Could not find the MPCDI file '%ls'.", *InFilePath);
		return false;
	}

	TMap<FString, TMap<FString, FDisplayClusterWarpMPCDIAttributes>> MPCDIFile;
	if (!IDisplayClusterWarp::Get().ReadMPCDFileStructure(InFilePath, MPCDIFile))
	{
		UE_LOGF(DisplayClusterConfiguratorLog, Error, "Could not read the data from the MPCDI file '%ls'.", *InFilePath);
		return false;
	}

	// The ViewPoint component cannot be used as a ViewOrigin. They have different purposes.
	// The default ViewOrigin is the RootComponent.
	const FName OriginComponentName = !InParams.OriginComponentName.IsNone() ? InParams.OriginComponentName : TEXT("RootComponent");

	USCS_Node* OriginNode = nullptr;
	UDisplayClusterCameraComponent* OriginComponent = Cast<UDisplayClusterCameraComponent>(FindBlueprintSceneComponent(InBlueprint, OriginComponentName, &OriginNode));

	FIPv4Address CurrentIPAddress = InParams.HostStartingIPAddress;
	for (const TPair<FString, TMap<FString, FDisplayClusterWarpMPCDIAttributes>>& Buffer : MPCDIFile)
	{
		for (const TPair<FString, FDisplayClusterWarpMPCDIAttributes>& Region : Buffer.Value)
		{
			if (InParams.bCreateStageGeometryComponents)
			{
				// For MPCDI 2D, we create Display Cluster screen components to match the MPCDI buffer and region configuration
				if (Region.Value.ProfileType == EDisplayClusterWarpProfileType::warp_2D)
				{
					bool bFoundExistingScreen = false;
					USCS_Node* ScreenNode = FindOrCreateScreenNodeForRegion(InBlueprint, Region.Key, bFoundExistingScreen);
					if (ScreenNode)
					{
						UDisplayClusterScreenComponent* ScreenComponent = CastChecked<UDisplayClusterScreenComponent>(ScreenNode->GetActualComponentTemplate(InBlueprint->GetGeneratedClass()));
						ConfigureScreenComponentFrom2DProfileRegion(ScreenComponent, OriginComponent, Region.Value, InParams);

						// If a new screen had to be created, it needs to be parented to a valid parent component
						if (!bFoundExistingScreen)
						{
							USCS_Node* ParentNode = nullptr;
							USceneComponent* ParentComponent = nullptr;
							if (InParams.ParentComponentName.IsNone())
							{
								ParentComponent = InBlueprint->SimpleConstructionScript->GetSceneRootComponentTemplate(false, &ParentNode);
							}
							else
							{
								ParentComponent = FindBlueprintSceneComponent(InBlueprint, InParams.ParentComponentName, &ParentNode);
							}

							if (ParentNode && ParentNode->GetSCS() == InBlueprint->SimpleConstructionScript)
							{
								ParentNode->AddChildNode(ScreenNode);
							}
							else
							{
								ScreenNode->SetParent(ParentComponent);
								InBlueprint->SimpleConstructionScript->AddNode(ScreenNode);
							}
						}
					}
				}
			}

			bool bFoundExistingViewport = false;
			UDisplayClusterConfigurationViewport* Viewport = FindOrCreateViewportForRegion(InBlueprint, Region.Key, bFoundExistingViewport);
			if (Viewport)
			{
				ConfigureViewportFromRegion(Viewport, Region.Value, InParams);

				if (!bFoundExistingViewport)
				{
					FDisplayClusterConfiguratorProjectionPolicyViewModel ProjectionPolicyViewModel(Viewport);

					ProjectionPolicyViewModel.SetPolicyType(DisplayClusterProjectionStrings::projection::MPCDI);

					ProjectionPolicyViewModel.SetParameterValue(DisplayClusterProjectionStrings::cfg::mpcdi::MPCDITypeKey, DisplayClusterProjectionStrings::cfg::mpcdi::TypeMPCDI);
					ProjectionPolicyViewModel.SetParameterValue(DisplayClusterProjectionStrings::cfg::mpcdi::File, MPCDIFileFullPath);
					ProjectionPolicyViewModel.SetParameterValue(DisplayClusterProjectionStrings::cfg::mpcdi::Buffer, Buffer.Key);
					ProjectionPolicyViewModel.SetParameterValue(DisplayClusterProjectionStrings::cfg::mpcdi::Region, Region.Key);
					ProjectionPolicyViewModel.SetParameterValue(DisplayClusterProjectionStrings::cfg::mpcdi::EnablePreview, TEXT("true"));

					// Always pass the mpcdi profile type through the projection policy parameter.
					const FString ProfileName = UE::DisplayClusterProjectionHelpers::MPCDI::ProfileTypeToString(Region.Value.ProfileType);
					ProjectionPolicyViewModel.SetParameterValue(DisplayClusterProjectionStrings::cfg::mpcdi::MPCDIType, ProfileName);

					if (Region.Value.ProfileType == EDisplayClusterWarpProfileType::warp_2D && InParams.bCreateStageGeometryComponents)
					{
						ProjectionPolicyViewModel.SetParameterValue(DisplayClusterProjectionStrings::cfg::mpcdi::Component, GetScreenNameForRegion(Region.Key));
					}

					if (!InParams.ParentComponentName.IsNone())
					{
						ProjectionPolicyViewModel.SetParameterValue(DisplayClusterProjectionStrings::cfg::mpcdi::Origin, InParams.ParentComponentName.ToString());
					}

					if (UDisplayClusterConfigurationClusterNode* ClusterNode = Cast<UDisplayClusterConfigurationClusterNode>(Viewport->GetOuter()))
					{
						FDisplayClusterConfiguratorClusterNodeViewModel ClusterNodeViewModel(ClusterNode);
						ClusterNodeViewModel.SetHost(CurrentIPAddress.ToString());
					}
				}

				UDisplayClusterConfigurationCluster* Cluster = InBlueprint->GetOrLoadConfig()->Cluster;
				const bool bHasValidPrimaryNode = Cluster->Nodes.Contains(Cluster->PrimaryNode.Id);
				if (!bHasValidPrimaryNode)
				{
					Cluster->Modify();
					Cluster->PrimaryNode.Id = GetClusterNodeNameForRegion(Region.Key);
				}

				if (InParams.bIncrementHostIPAddress)
				{
					++CurrentIPAddress.Value;
				}
			}
		}
	}

	return true;
}

USCS_Node* FDisplayClusterConfiguratorMPCDIImporter::FindOrCreateScreenNodeForRegion(UDisplayClusterBlueprint* InBlueprint, const FString& RegionId, bool& bOutFoundExistingScreen)
{
	bOutFoundExistingScreen = false;

	const FString ScreenName = GetScreenNameForRegion(RegionId);
	if (USCS_Node* ExistingScreenNode = InBlueprint->SimpleConstructionScript->FindSCSNode(*ScreenName))
	{
		bOutFoundExistingScreen = true;
		return ExistingScreenNode;
	}

	return InBlueprint->SimpleConstructionScript->CreateNode(UDisplayClusterScreenComponent::StaticClass(), *ScreenName);
}

void FDisplayClusterConfiguratorMPCDIImporter::ConfigureScreenComponentFrom2DProfileRegion(
	UDisplayClusterScreenComponent* InScreenComponent,
	USceneComponent* InOriginComponent,
	const FDisplayClusterWarpMPCDIAttributes& InAttributes,
	const FDisplayClusterConfiguratorMPCDIImporterParams& InParams)
{
	FVector ScreenPosition;
	FVector2D ScreenSize;
	if (!InAttributes.CalcProfile2DScreen(ScreenPosition, ScreenSize))
	{
		return;
	}

	if (InOriginComponent)
	{
		// We want to position the screen component to be in front of the view point it is assigned to, so we must compute
		// the transforms from view point to root and from root to screen component to correct position

		// Blueprint components don't have world transforms, so compute the view point to root transform manually
		FTransform ViewPointTransform = FTransform::Identity;
		for (const USceneComponent* Comp = InOriginComponent; Comp != nullptr; Comp = Comp->GetAttachParent())
		{
			ViewPointTransform *= Comp->GetRelativeTransform();
		}

		// The screen may also be parented to a non-root component, so also compute its root transform
		FTransform ScreenTransform = FTransform::Identity;
		for (const USceneComponent* Comp = InScreenComponent->GetAttachParent(); Comp != nullptr; Comp = Comp->GetAttachParent())
		{
			ScreenTransform *= Comp->GetRelativeTransform();
		}

		FTransform ViewPointToScreen = ViewPointTransform * ScreenTransform.Inverse();
		ScreenPosition = ViewPointToScreen.TransformPositionNoScale(ScreenPosition);
	}

	InScreenComponent->SetRelativeLocation(ScreenPosition);
	InScreenComponent->SetScreenSize(ScreenSize);
}

UDisplayClusterConfigurationViewport* FDisplayClusterConfiguratorMPCDIImporter::FindOrCreateViewportForRegion(UDisplayClusterBlueprint* InBlueprint, const FString& RegionId, bool& bOutFoundExistingViewport)
{
	UDisplayClusterConfigurationViewport* Viewport = nullptr;
	bOutFoundExistingViewport = false;

	const FString ViewportName = GetViewportNameForRegion(RegionId);
	UDisplayClusterConfigurationCluster* Cluster = InBlueprint->GetOrLoadConfig()->Cluster;
	for (const TPair<FString, TObjectPtr<UDisplayClusterConfigurationClusterNode>>& NodePair : Cluster->Nodes)
	{
		if (NodePair.Value->Viewports.Contains(ViewportName))
		{
			Viewport = NodePair.Value->Viewports[ViewportName];
			bOutFoundExistingViewport = true;
			break;
		}
	}

	if (!Viewport)
	{
		const FString NodeName = GetClusterNodeNameForRegion(RegionId);
		UDisplayClusterConfigurationClusterNode* ClusterNodeTemplate = NewObject<UDisplayClusterConfigurationClusterNode>(InBlueprint, NAME_None, RF_Transactional | RF_ArchetypeObject | RF_Public);
		UDisplayClusterConfigurationClusterNode* NewClusterNode = UE::DisplayClusterConfiguratorClusterUtils::AddClusterNodeToCluster(ClusterNodeTemplate, Cluster, NodeName);

		UDisplayClusterConfigurationViewport* ViewportTemplate = NewObject<UDisplayClusterConfigurationViewport>(InBlueprint, NAME_None, RF_Transactional | RF_ArchetypeObject | RF_Public);
		Viewport = UE::DisplayClusterConfiguratorClusterUtils::AddViewportToClusterNode(ViewportTemplate, NewClusterNode, ViewportName);
	}

	return Viewport;
}

void FDisplayClusterConfiguratorMPCDIImporter::ConfigureViewportFromRegion(UDisplayClusterConfigurationViewport* InViewport, const FDisplayClusterWarpMPCDIAttributes& InAttributes, const FDisplayClusterConfiguratorMPCDIImporterParams& InParams)
{
	if (UDisplayClusterConfigurationClusterNode* ClusterNode = Cast<UDisplayClusterConfigurationClusterNode>(InViewport->GetOuter()))
	{
		FDisplayClusterConfiguratorClusterNodeViewModel ClusterNodeViewModel(ClusterNode);
		ClusterNodeViewModel.SetWindowRect(FDisplayClusterConfigurationRectangle(0, 0, InAttributes.Region.Resolution.X, InAttributes.Region.Resolution.Y));
	}

	FDisplayClusterConfiguratorViewportViewModel ViewportViewModel(InViewport);
	ViewportViewModel.SetRegion(FDisplayClusterConfigurationRectangle(0, 0, InAttributes.Region.Resolution.X, InAttributes.Region.Resolution.Y));

	if (InParams.ViewPointComponentName != NAME_None)
	{
		ViewportViewModel.SetCamera(InParams.ViewPointComponentName.ToString());
	}
}

USceneComponent* FDisplayClusterConfiguratorMPCDIImporter::FindBlueprintSceneComponent(UDisplayClusterBlueprint* InBlueprint, const FName& ComponentName, USCS_Node** OutComponentNode)
{
	*OutComponentNode = nullptr;

	UClass* GeneratedClass = InBlueprint->GetGeneratedClass();
	UClass* ParentClass = InBlueprint->ParentClass;

	AActor* CDO = Cast<AActor>(GeneratedClass->GetDefaultObject(false));
	if (CDO == nullptr && ParentClass != nullptr)
	{
		CDO = Cast<AActor>(ParentClass->GetDefaultObject(false));
	}

	// First, check to see if there exists a native component on the CDO that matches the specified name
	USceneComponent* FoundComponent = nullptr;
	if (CDO)
	{
		if (ComponentName.IsNone())
		{
			return FoundComponent = CDO->GetRootComponent();
		}

		for (UActorComponent* Component : CDO->GetComponents())
		{
			if (USceneComponent* SceneComponent = Cast<USceneComponent>(Component))
			{
				if (SceneComponent->GetFName() == ComponentName)
				{
					FoundComponent = SceneComponent;
					break;
				}
			}
		}
	}

	// If a native component was not found, check the SCS to see if one exists there
	if (!FoundComponent)
	{
		if (ComponentName.IsNone())
		{
			return nullptr;
		}

		if (USCS_Node* FoundNode = InBlueprint->SimpleConstructionScript->FindSCSNode(ComponentName))
		{
			if (FoundNode->ComponentTemplate && FoundNode->ComponentTemplate->IsA<USceneComponent>())
			{
				*OutComponentNode = FoundNode;
				FoundComponent = Cast<USceneComponent>(FoundNode->ComponentTemplate);
			}
		}
	}

	return FoundComponent;
}

FString FDisplayClusterConfiguratorMPCDIImporter::GetScreenNameForRegion(const FString& RegionId)
{
	return FString::Printf(TEXT("%s_Screen"), *RegionId);
}

FString FDisplayClusterConfiguratorMPCDIImporter::GetClusterNodeNameForRegion(const FString& RegionId)
{
	return FString::Printf(TEXT("%s_Node"), *RegionId);
}

FString FDisplayClusterConfiguratorMPCDIImporter::GetViewportNameForRegion(const FString& RegionId)
{
	return FString::Printf(TEXT("%s_Viewport"), *RegionId);
}
