// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Interfaces/IPv4/IPv4Address.h"

class UDisplayClusterCameraComponent;
struct FDisplayClusterWarpMPCDIAttributes;
class USceneComponent;
class USCS_Node;
class UDisplayClusterConfigurationViewport;
class UDisplayClusterBlueprint;
class UDisplayClusterScreenComponent;

/** Used to pass parameters into the MPCDI importer */
struct FDisplayClusterConfiguratorMPCDIImporterParams
{
	/** The name of the component to make the parent of the MPCDI geometry */
	FName ParentComponentName = NAME_None;

	/** The name of the view point component to link to the viewports. If none is supplied, the default view point is used */
	FName ViewPointComponentName = NAME_None;

	/** The name of the origin component to link to the MPCDI geometry. If none is supplied, the root component is used */
	FName OriginComponentName = NAME_None;

	/** The IP address to configure the first cluster node with */
	FIPv4Address HostStartingIPAddress = FIPv4Address::InternalLoopback;

	/** Indicates that the IP address will be incremented for each MPCDI region a cluster node is created for */
	bool bIncrementHostIPAddress = true;

	/** Indicates if stage geometry components are created to match the MPCDI geometry */
	bool bCreateStageGeometryComponents = true;
};

/** Helper class that can import an MPCDI configuration into an nDisplay config blueprint */
class FDisplayClusterConfiguratorMPCDIImporter
{
public:
	/**
	 * Imports the specified MPCDI file into the specified nDisplay config blueprint, configurating the blueprint to match the MPCDI file configuration
	 * @param InFilePath - The file path to the MPCDI file to import
	 * @param InBlueprint - The blueprint to write the MPCDI configuration to
	 * @param InParams - The parameters to use when importing the MPCDI config
	 **/
	static bool ImportMPCDIIntoBlueprint(const FString& InFilePath, UDisplayClusterBlueprint* InBlueprint, const FDisplayClusterConfiguratorMPCDIImporterParams& InParams);

private:
	/**
	 * Attempts to find an existing SCS screen node for the specified region, and if one is not found, creates a new node
	 * 
	 * @param InBlueprint - The blueprint to find the node on
	 * @param RegionId - The ID of the region to find an existing node for
	 * @param bOutFoundExistingScreen - An out param indicating if an existing node was found or not
	 **/
	static USCS_Node* FindOrCreateScreenNodeForRegion(UDisplayClusterBlueprint* InBlueprint, const FString& RegionId, bool& bOutFoundExistingScreen);

	/** 
	 * Configures a screen component to match the configuration of an MPCDI region
	 * 
	 * @param InScreenComponent - The component to configure
	 * @param InOriginComponent - The Origin component to configure the screens from
	 * @param InAttributes      - The attributes of the region to configure
	 * @param InParams          - The import parameters
	 **/
	static void ConfigureScreenComponentFrom2DProfileRegion(UDisplayClusterScreenComponent* InScreenComponent, USceneComponent* InOriginComponent, const FDisplayClusterWarpMPCDIAttributes& InAttributes, const FDisplayClusterConfiguratorMPCDIImporterParams& InParams);

	/**
	 * Attempts to find an existing viewport configuration for the specified region, and if one is not found, creates a new viewport configuration
	 * 
	 * @param InBlueprint - The blueprint to find the viewport on
	 * @param RegionId - The ID of the region to find an existing viewport for
	 * @param bOutFoundExistingViewport - An out param indicating if an existing viewport was found or not
	 **/
	static UDisplayClusterConfigurationViewport* FindOrCreateViewportForRegion(UDisplayClusterBlueprint* InBlueprint, const FString& RegionId, bool& bOutFoundExistingViewport);

	/** 
	 * Configures a viewport configuration to match the configuration of an MPCDI region
	 * 
	 * @param InViewport - The viewport configuration to configure
	 * @param InAttributes - The attributes of the region to configure
	 * @param InParams - The import parameters 
	 **/
	static void ConfigureViewportFromRegion(UDisplayClusterConfigurationViewport* InViewport, const FDisplayClusterWarpMPCDIAttributes& InAttributes, const FDisplayClusterConfiguratorMPCDIImporterParams& InParams);

	/**
	 * Attempts to find a specified scene component in the blueprint's component hierarchy
	 * 
	 * @param InBlueprint - The blueprint to search
	 * @param ComponentName - The name of the component to search for
	 * @param OutComponentNode - The corresponding SCS node of the found component if the component was a non-native component
	 * @return The found scene component, or nullptr if none was found
	 **/
	static USceneComponent* FindBlueprintSceneComponent(UDisplayClusterBlueprint* InBlueprint, const FName& ComponentName, USCS_Node** OutComponentNode);

	/** Creates a screen component name for a region using its ID */
	static FString GetScreenNameForRegion(const FString& RegionId);

	/** Creates a cluster node name for a region using its ID */
	static FString GetClusterNodeNameForRegion(const FString& RegionId);

	/** Creates a viewport name for a region using its ID */
	static FString GetViewportNameForRegion(const FString& RegionId);
};