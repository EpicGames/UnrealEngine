// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfiguratorSettings.h"

UDisplayClusterConfiguratorEditorSettings::UDisplayClusterConfiguratorEditorSettings()
{
	EditorDefaultCameraLocation = FVector(0.f, -1000.f, 0.f);
	EditorDefaultCameraRotation = FRotator(0.f, 180.f, 0.f);
	
	bEditorShowFloor = true;
	bEditorShowGrid = true;
	bEditorShowWorldOrigin = false;
	bEditorShowPreview = true;
	bEditorShow3DViewportNames = true;
	bExportOnSave = false;
	bUpdateAssetsOnStartup = true;
	bDisplayAssetUpdateProgress = true;

	VisXformScale = 1.0f;
	bShowVisXforms = true;

	bEditorEnableAA = true;

	NewAssetIndex = 0;
}
