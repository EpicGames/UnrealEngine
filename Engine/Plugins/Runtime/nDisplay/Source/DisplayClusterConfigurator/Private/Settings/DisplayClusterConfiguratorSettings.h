// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DisplayClusterConfiguratorSettings.generated.h"


UCLASS(config = EditorPerProjectUserSettings)
class UDisplayClusterConfiguratorEditorSettings : public UObject
{
	GENERATED_BODY()

public:
	UDisplayClusterConfiguratorEditorSettings();

	/** Camera view location when resetting the camera. */
	UPROPERTY(config, EditAnywhere, Category = "Viewport")
	FVector EditorDefaultCameraLocation;

	/** Camera view rotation when resetting the camera. */
	UPROPERTY(config, EditAnywhere, Category = "Viewport")
	FRotator EditorDefaultCameraRotation;
	
	/** Shows the floor in the 3d editor viewport. */
	UPROPERTY(config)
	bool bEditorShowFloor;

	/** Shows the grid in the 3d editor viewport. */
	UPROPERTY(config)
	bool bEditorShowGrid;
	
	/** Displays the world origin at 0, 0, 0 */
	UPROPERTY(config)
	bool bEditorShowWorldOrigin;

	/** Shows the preview in editor */
	UPROPERTY(config)
	bool bEditorShowPreview;

	/** Shows names of the viewport in 3d space. */
	UPROPERTY(config)
	bool bEditorShow3DViewportNames;

	/** Export a config automatically on save. Requires a config initially exported. */
	UPROPERTY(config, EditAnywhere, Category = "Config File")
	bool bExportOnSave;

	/**
	 * Automatically update assets saved by older versions to the most current version. It is strongly recommended to leave this on.
	 */
	UPROPERTY(config, EditAnywhere, Category = "Version Updates")
	bool bUpdateAssetsOnStartup;

	/**
	 * Display a progress bar when updating assets to a new version.
	 */
	UPROPERTY(config, EditAnywhere, Category = "Version Updates", meta = (EditCondition = "bUpdateAssetsOnStartup"))
	bool bDisplayAssetUpdateProgress;

	/** The visual scale of the Xform static mesh */
	UPROPERTY(config)
	float VisXformScale;

	/** Whether to show the Xform static mesh */
	UPROPERTY(config)
	bool bShowVisXforms;

	/** Anti aliasing in 3d viewport. */
	UPROPERTY(config)
	bool bEditorEnableAA;

	/** The last position on the new asset dialog box. */
	UPROPERTY(config)
	int32 NewAssetIndex;
};
