// Copyright Epic Games, Inc. All Rights Reserved.

#include "DCConfiguratorColorGradingSettingsCustomization.h"

#include "DisplayClusterConfigurationTypes_Postprocess.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"

void FDCConfiguratorColorGradingSettingsCustomization::Initialize(const TSharedRef<IPropertyHandle>& InPropertyHandle, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	FDisplayClusterConfiguratorBaseTypeCustomization::Initialize(InPropertyHandle, CustomizationUtils);

	// Check if this color grading struct is an element of an array. If so, we will override its "reset to default" behavior because
	// the default value of a struct in an array that is empty in the CDO is entirely zeroes instead of the default values
	// of the struct, which is undesirable for some color grading properties
	TSharedPtr<IPropertyHandle> CurrentPropertyHandle = InPropertyHandle->GetParentHandle();
	while (CurrentPropertyHandle)
	{
		if (CurrentPropertyHandle->AsArray().IsValid())
		{
			bIsArrayMember = true;
			break;
		}
		
		CurrentPropertyHandle = CurrentPropertyHandle->GetParentHandle();
	}
}

void FDCConfiguratorColorGradingSettingsCustomization::SetChildren(const TSharedRef<IPropertyHandle>& InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	if (!bIsArrayMember)
	{
		FDisplayClusterConfiguratorBaseTypeCustomization::SetChildren(InPropertyHandle, InChildBuilder, CustomizationUtils);
		return;
	}

	static const TMap<FName, FVector4> ColorPropertyDefaultValues =
	{
	{ GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_ColorGradingSettings, Saturation), FVector4::One() },
	{ GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_ColorGradingSettings, Contrast), FVector4::One() },
	{ GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_ColorGradingSettings, Gamma), FVector4::One() },
	{ GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_ColorGradingSettings, Gain), FVector4::One() },
	{ GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_ColorGradingSettings, Offset), FVector4::Zero() },
	};

	auto ResetPropertyColorValue = [](TSharedPtr<IPropertyHandle> InPropertyHandle, FVector4 InDefaultValue)
	{
		// Set the value with an interactive, non-transactable change first to avoid invoking any post edit change events on each component
		// (Vector properties use property handles to set each component's value), in case the object owning the property becomes invalid
		// on post edit change events (such as construction script created objects)
		InPropertyHandle->SetValue(InDefaultValue, EPropertyValueSetFlags::InteractiveChange | EPropertyValueSetFlags::NotTransactable);

		// Now update the value normally to invoke the usual notification/transaction pipeline
		InPropertyHandle->SetValue(InDefaultValue, EPropertyValueSetFlags::ResetToDefault);
	};
	
	uint32 NumChildren = 0;
	InPropertyHandle->GetNumChildren(NumChildren);
	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		TSharedPtr<IPropertyHandle> ChildHandle = InPropertyHandle->GetChildHandle(ChildIndex);
		if (ChildHandle && ChildHandle->IsValidHandle() && !ChildHandle->IsCustomized())
		{
			FText ChildTooltip = ApplySubstitutions(ChildHandle->GetToolTipText());
			ChildHandle->SetToolTipText(ChildTooltip);

			IDetailPropertyRow& PropertyRow = InChildBuilder.AddProperty(ChildHandle.ToSharedRef());
			if (ColorPropertyDefaultValues.Contains(ChildHandle->GetProperty()->GetFName()))
			{
				const FVector4& DefaultValue = ColorPropertyDefaultValues[ChildHandle->GetProperty()->GetFName()];
				PropertyRow.OverrideResetToDefault(FResetToDefaultOverride::Create(FResetToDefaultHandler::CreateLambda(ResetPropertyColorValue, DefaultValue)));
			}
		}
	}
}
