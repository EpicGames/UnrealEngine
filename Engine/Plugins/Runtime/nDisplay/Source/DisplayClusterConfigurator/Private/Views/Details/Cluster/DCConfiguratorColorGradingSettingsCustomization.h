// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Views/Details/DisplayClusterConfiguratorBaseTypeCustomization.h"

/** Customization that ensures proper reset to default values for the color properties in the nDisplay color grading struct when it is an element of an array */
class FDCConfiguratorColorGradingSettingsCustomization : public FDisplayClusterConfiguratorBaseTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FDCConfiguratorColorGradingSettingsCustomization>();
	}

protected:
	virtual void Initialize(const TSharedRef<IPropertyHandle>& InPropertyHandle, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void SetChildren(const TSharedRef<IPropertyHandle>& InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

private:
	/** Indicates if the struct being customized by this customization is a member of an array */
	bool bIsArrayMember = false;
};
