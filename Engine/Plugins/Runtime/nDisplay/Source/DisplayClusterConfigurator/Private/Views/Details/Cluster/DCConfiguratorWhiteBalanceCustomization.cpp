// Copyright Epic Games, Inc. All Rights Reserved.

#include "DCConfiguratorWhiteBalanceCustomization.h"

#include "DisplayClusterConfigurationTypes_Postprocess.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"

void FDCConfiguratorWhiteBalanceCustomization::Initialize(const TSharedRef<IPropertyHandle>& InPropertyHandle, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	FDisplayClusterConfiguratorBaseTypeCustomization::Initialize(InPropertyHandle, CustomizationUtils);

	// Check if this white balance struct is an element of an array. If so, we will override its "reset to default" behavior because
	// the default value of a struct in an array that is empty in the CDO is entirely zeroes instead of the default values
	// of the struct, which is undesirable for some white balance properties
	TSharedPtr<IPropertyHandle> CurrentPropertyHandle = InPropertyHandle->GetParentHandle();
	while (CurrentPropertyHandle)
	{
		if (CurrentPropertyHandle->AsArray().IsValid())
		{
			bIsArrayMember = true;
			break;
		}
		
		CurrentPropertyHandle = CurrentPropertyHandle->GetParentHandle();
	}
}

void FDCConfiguratorWhiteBalanceCustomization::SetChildren(const TSharedRef<IPropertyHandle>& InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	if (!bIsArrayMember)
	{
		FDisplayClusterConfiguratorBaseTypeCustomization::SetChildren(InPropertyHandle, InChildBuilder, CustomizationUtils);
		return;
	}

	static const TMap<FName, float> FloatPropertyDefaultValues =
	{
	{ GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_ColorGradingWhiteBalanceSettings, WhiteTemp), 6500.0f }
	};

	auto ResetPropertyFloatValue = [](TSharedPtr<IPropertyHandle> InPropertyHandle, float InDefaultValue)
	{
		InPropertyHandle->SetValue(InDefaultValue, EPropertyValueSetFlags::ResetToDefault);
	};
	
	uint32 NumChildren = 0;
	InPropertyHandle->GetNumChildren(NumChildren);
	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		TSharedPtr<IPropertyHandle> ChildHandle = InPropertyHandle->GetChildHandle(ChildIndex);
		if (ChildHandle && ChildHandle->IsValidHandle() && !ChildHandle->IsCustomized())
		{
			FText ChildTooltip = ApplySubstitutions(ChildHandle->GetToolTipText());
			ChildHandle->SetToolTipText(ChildTooltip);

			IDetailPropertyRow& PropertyRow = InChildBuilder.AddProperty(ChildHandle.ToSharedRef());
			if (FloatPropertyDefaultValues.Contains(ChildHandle->GetProperty()->GetFName()))
			{
				const float DefaultValue = FloatPropertyDefaultValues[ChildHandle->GetProperty()->GetFName()];
				PropertyRow.OverrideResetToDefault(FResetToDefaultOverride::Create(FResetToDefaultHandler::CreateLambda(ResetPropertyFloatValue, DefaultValue)));
			}
		}
	}
}
