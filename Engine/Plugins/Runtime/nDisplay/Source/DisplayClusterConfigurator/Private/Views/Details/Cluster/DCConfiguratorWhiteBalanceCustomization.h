// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Views/Details/DisplayClusterConfiguratorBaseTypeCustomization.h"

/** Customization that ensures proper reset to default values for the temperature properties in the nDisplay white balance struct when it is an element of an array */
class FDCConfiguratorWhiteBalanceCustomization : public FDisplayClusterConfiguratorBaseTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FDCConfiguratorWhiteBalanceCustomization>();
	}

protected:
	virtual void Initialize(const TSharedRef<IPropertyHandle>& InPropertyHandle, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void SetChildren(const TSharedRef<IPropertyHandle>& InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

private:
	/** Indicates if the struct being customized by this customization is a member of an array */
	bool bIsArrayMember = false;
};
