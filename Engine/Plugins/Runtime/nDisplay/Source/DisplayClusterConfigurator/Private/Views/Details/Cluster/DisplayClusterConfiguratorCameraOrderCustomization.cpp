// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfiguratorCameraOrderCustomization.h"

#include "DisplayClusterConfigurationTypes_Viewport.h"
#include "DisplayClusterConfiguratorBlueprintEditor.h"
#include "DisplayClusterConfiguratorNodeSelectionCustomization.h"
#include "DisplayClusterRootActor.h"

#include "DetailLayoutBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyHandle.h"

void FDisplayClusterConfiguratorCameraOrderCustomization::Initialize(const TSharedRef<IPropertyHandle>& InPropertyHandle, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	FDisplayClusterConfiguratorBaseTypeCustomization::Initialize(InPropertyHandle, CustomizationUtils);

	// Get the EnforcedCameraOrder array property handle
	CameraOrderArrayHandle = InPropertyHandle;

	FDisplayClusterConfiguratorBlueprintEditor* BlueprintEditor = FindBlueprintEditor();
	ADisplayClusterRootActor* RootActor = FindRootActor();

	if (RootActor != nullptr || BlueprintEditor != nullptr)
	{
		// Create a node selection builder configured for ICVFX cameras
		NodeSelectionBuilder = MakeShared<FDisplayClusterConfiguratorNodeSelection>(
			FDisplayClusterConfiguratorNodeSelection::EOperationMode::ICVFXCameras,
			RootActor,
			BlueprintEditor);
		bCanCustomizeDisplay = true;
	}
}

void FDisplayClusterConfiguratorCameraOrderCustomization::SetChildren(const TSharedRef<IPropertyHandle>& InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	if (bCanCustomizeDisplay && CameraOrderArrayHandle && CameraOrderArrayHandle->IsValidHandle())
	{
		CameraOrderArrayHandle->SetPropertyDisplayName(InPropertyHandle->GetPropertyDisplayName());
		CameraOrderArrayHandle->SetToolTipText(InPropertyHandle->GetToolTipText());

		NodeSelectionBuilder->IsEnabled(true);
		NodeSelectionBuilder->CreateArrayBuilder(CameraOrderArrayHandle.ToSharedRef(), InChildBuilder);
	}
	else
	{
		FDisplayClusterConfiguratorBaseTypeCustomization::SetChildren(InPropertyHandle, InChildBuilder, CustomizationUtils);
	}
}
