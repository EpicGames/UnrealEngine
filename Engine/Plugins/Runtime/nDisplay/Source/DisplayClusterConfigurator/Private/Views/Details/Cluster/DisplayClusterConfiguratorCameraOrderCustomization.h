// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Views/Details/DisplayClusterConfiguratorBaseTypeCustomization.h"

class ADisplayClusterRootActor;
class FDisplayClusterConfiguratorBlueprintEditor;
class FDisplayClusterConfiguratorNodeSelection;

/**
 * Type customization for the EnforcedCameraOrder TArray<FString> property in FDisplayClusterConfigurationViewport_ICVFX.
 * Displays a dropdown for each array element listing all available ICVFX cameras.
 */
class FDisplayClusterConfiguratorCameraOrderCustomization final : public FDisplayClusterConfiguratorBaseTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FDisplayClusterConfiguratorCameraOrderCustomization>();
	}

	~FDisplayClusterConfiguratorCameraOrderCustomization()
	{
		NodeSelectionBuilder.Reset();
	}

protected:
	//~ FDisplayClusterConfiguratorBaseTypeCustomization overrides begin
	virtual void Initialize(const TSharedRef<IPropertyHandle>& InPropertyHandle, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void SetChildren(const TSharedRef<IPropertyHandle>& InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual bool ShouldShowHeader(const TSharedRef<IPropertyHandle>& InPropertyHandle) const override { return false; }
	//~ FDisplayClusterConfiguratorBaseTypeCustomization overrides end

private:
	/** The helper object used to generate the custom array widget to display in place of a string list for the EnforcedCameraOrder property */
	TSharedPtr<FDisplayClusterConfiguratorNodeSelection> NodeSelectionBuilder;

	/** The property handle for the EnforcedCameraOrder array */
	TSharedPtr<IPropertyHandle> CameraOrderArrayHandle;

	/** Indicates whether this customizer can customize the array property's display or not */
	bool bCanCustomizeDisplay = false;
};
