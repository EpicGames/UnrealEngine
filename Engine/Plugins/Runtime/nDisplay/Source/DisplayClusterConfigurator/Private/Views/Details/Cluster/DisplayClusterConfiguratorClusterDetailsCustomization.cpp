// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfiguratorClusterDetailsCustomization.h"

#include "DisplayClusterConfigurationTypes.h"

#include "DisplayClusterRootActor.h"

#include "DetailLayoutBuilder.h"

void FDisplayClusterConfiguratorClusterDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& InLayoutBuilder)
{
	FDisplayClusterConfiguratorBaseDetailCustomization::CustomizeDetails(InLayoutBuilder);

	if (ToolkitPtr.IsValid())
	{
		check (RootActorPtr.IsValid());
		if (RootActorPtr->GetConfigData())
		{
			{
				const TSharedPtr<IPropertyHandle> PropertyHandle = InLayoutBuilder.AddObjectPropertyData(
					{ RootActorPtr->GetConfigData() },
					GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationData, Info));
				check(PropertyHandle->IsValidHandle());
				InLayoutBuilder.AddPropertyToCategory(PropertyHandle);
			}

			{
				const TSharedPtr<IPropertyHandle> PropertyHandle = InLayoutBuilder.AddObjectPropertyData(
					{ RootActorPtr->GetConfigData() },
					GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationData, Diagnostics));
				check(PropertyHandle->IsValidHandle());
				InLayoutBuilder.AddPropertyToCategory(PropertyHandle);
			}
		}
	}
}
