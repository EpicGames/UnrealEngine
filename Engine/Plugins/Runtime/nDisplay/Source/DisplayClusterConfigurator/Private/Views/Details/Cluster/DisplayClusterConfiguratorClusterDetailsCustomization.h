// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Views/Details/DisplayClusterConfiguratorBaseDetailCustomization.h"

class FDisplayClusterConfiguratorClusterDetailsCustomization final : public FDisplayClusterConfiguratorBaseDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShared<FDisplayClusterConfiguratorClusterDetailsCustomization>();
	}

	virtual void CustomizeDetails(IDetailLayoutBuilder& InLayoutBuilder) override;
};
