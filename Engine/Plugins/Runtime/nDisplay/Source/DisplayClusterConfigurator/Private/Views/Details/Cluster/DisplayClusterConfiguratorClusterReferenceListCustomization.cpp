// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfiguratorClusterReferenceListCustomization.h"

#include "DisplayClusterRootActor.h"
#include "DisplayClusterConfigurationTypes_Base.h"
#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterConfiguratorBlueprintEditor.h"
#include "DisplayClusterConfiguratorNodeSelectionCustomization.h"
#include "Views/Details/Widgets/SDisplayClusterConfigurationSearchableComboBox.h"

#include "DetailLayoutBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyHandle.h"
#include "PropertyCustomizationHelpers.h"

const FName FDisplayClusterConfiguratorClusterReferenceListCustomization::ClusterItemTypeMetadataKey = TEXT("ClusterItemType");

void FDisplayClusterConfiguratorClusterReferenceListCustomization::Initialize(const TSharedRef<IPropertyHandle>& InPropertyHandle, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	FDisplayClusterConfiguratorBaseTypeCustomization::Initialize(InPropertyHandle, CustomizationUtils);

	ItemNamesArrayHandle = GET_CHILD_HANDLE(FDisplayClusterConfigurationClusterItemReferenceList, ItemNames);

	FDisplayClusterConfiguratorNodeSelection::EOperationMode Mode = FDisplayClusterConfiguratorNodeSelection::EOperationMode::ClusterNodes;
	if (const FString* ClusterItemTypeMetadata = FindMetaData(InPropertyHandle, ClusterItemTypeMetadataKey))
	{
		FString ClusterItemTypeStr = ClusterItemTypeMetadata->ToLower();
		ClusterItemTypeStr.RemoveSpacesInline();
		if (ClusterItemTypeStr == TEXT("viewports"))
		{
			Mode = FDisplayClusterConfiguratorNodeSelection::EOperationMode::Viewports;
		}
		else if (ClusterItemTypeStr == TEXT("clusternodes"))
		{
			Mode = FDisplayClusterConfiguratorNodeSelection::EOperationMode::ClusterNodes;
		}
	}

	FDisplayClusterConfiguratorBlueprintEditor* BlueprintEditor = FindBlueprintEditor();
	ADisplayClusterRootActor * RootActor = FindRootActor();

	if (RootActor != nullptr || BlueprintEditor != nullptr)
	{
		NodeSelectionBuilder = MakeShared<FDisplayClusterConfiguratorNodeSelection>(Mode, RootActor, BlueprintEditor);
		bCanCustomizeDisplay = true;
	}

}

void FDisplayClusterConfiguratorClusterReferenceListCustomization::SetChildren(const TSharedRef<IPropertyHandle>& InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	if (bCanCustomizeDisplay && ItemNamesArrayHandle && ItemNamesArrayHandle->IsValidHandle())
	{
		ItemNamesArrayHandle->SetPropertyDisplayName(InPropertyHandle->GetPropertyDisplayName());
		ItemNamesArrayHandle->SetToolTipText(InPropertyHandle->GetToolTipText());

		NodeSelectionBuilder->IsEnabled(true);
		NodeSelectionBuilder->CreateArrayBuilder(ItemNamesArrayHandle.ToSharedRef(), InChildBuilder);
	}
	else
	{
		FDisplayClusterConfiguratorBaseTypeCustomization::SetChildren(InPropertyHandle, InChildBuilder, CustomizationUtils);
	}
}
