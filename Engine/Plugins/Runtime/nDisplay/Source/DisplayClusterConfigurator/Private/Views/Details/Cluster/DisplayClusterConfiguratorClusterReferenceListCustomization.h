// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Views/Details/DisplayClusterConfiguratorBaseTypeCustomization.h"

class FDisplayClusterConfiguratorBlueprintEditor;
class ADisplayClusterRootActor;
class SDisplayClusterConfigurationSearchableComboBox;
class FDisplayClusterConfiguratorNodeSelection;

/** Type customization for the FDisplayClusterConfigurationClusterItemReferenceList type, which displays a dropdown for each array element listing all selectable cluster items. */
class FDisplayClusterConfiguratorClusterReferenceListCustomization final : public FDisplayClusterConfiguratorBaseTypeCustomization
{
public:
	/** Custom "ClusterItemType" metadata specifier that can be used to specify which cluster item type is displayed in the dropdown menus */
	static const FName ClusterItemTypeMetadataKey;


	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FDisplayClusterConfiguratorClusterReferenceListCustomization>();
	}

	~FDisplayClusterConfiguratorClusterReferenceListCustomization()
	{
		NodeSelectionBuilder.Reset();
	}

protected:
	//~ FDisplayClusterConfiguratorBaseTypeCustomization overrides begin
	virtual void Initialize(const TSharedRef<IPropertyHandle>& InPropertyHandle, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void SetChildren(const TSharedRef<IPropertyHandle>& InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

	virtual bool ShouldShowHeader(const TSharedRef<IPropertyHandle>& InPropertyHandle) const override { return false; }

	//~ FDisplayClusterConfiguratorBaseTypeCustomization overrides end


private:
	/** The helper object used to generate the custom array widget to display in place of a string list for the ApplyPostProcessToObjects property */
	TSharedPtr<FDisplayClusterConfiguratorNodeSelection> NodeSelectionBuilder;

	/** The property handle for the ItemNames array in the FDisplayClusterConfigurationClusterItemReferenceList type */
	TSharedPtr<IPropertyHandle> ItemNamesArrayHandle;

	/** Indicates whether this customizer can customize the array property's display or not */
	bool bCanCustomizeDisplay = false;
};