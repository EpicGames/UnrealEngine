// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfiguratorDataDetailsCustomization.h"

#include "DisplayClusterConfigurationTypes.h"

#include "DisplayClusterRootActor.h"

#include "DetailLayoutBuilder.h"

void FDisplayClusterConfiguratorDataDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& InLayoutBuilder)
{
	FDisplayClusterConfiguratorBaseDetailCustomization::CustomizeDetails(InLayoutBuilder);

	// Displayed under Cluster details panel.
	if (ToolkitPtr.IsValid())
	{
		InLayoutBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationData, Info));
		InLayoutBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationData, Diagnostics));
	}
}
 