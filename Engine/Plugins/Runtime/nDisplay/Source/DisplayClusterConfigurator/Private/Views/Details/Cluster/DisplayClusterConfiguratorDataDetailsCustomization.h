// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Views/Details/DisplayClusterConfiguratorBaseDetailCustomization.h"

class FDisplayClusterConfiguratorDataDetailsCustomization final : public FDisplayClusterConfiguratorBaseDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShared<FDisplayClusterConfiguratorDataDetailsCustomization>();
	}

	virtual void CustomizeDetails(IDetailLayoutBuilder& InLayoutBuilder) override;
};
