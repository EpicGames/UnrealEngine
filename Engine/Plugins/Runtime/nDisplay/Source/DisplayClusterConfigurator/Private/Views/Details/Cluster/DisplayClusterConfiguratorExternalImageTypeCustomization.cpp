// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfiguratorExternalImageTypeCustomization.h"

#include "DisplayClusterConfigurationTypes.h"
#include "Views/OutputMapping/Widgets/SDisplayClusterConfiguratorExternalImagePicker.h"

#include "PropertyHandle.h"
#include "DetailWidgetRow.h"

void FDisplayClusterConfiguratorExternalImageTypeCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	ImagePathHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationExternalImage, ImagePath));
	check(ImagePathHandle->IsValidHandle());

	FString ImagePath;
	ImagePathHandle->GetValue(ImagePath);

	TArray<FString> ImageExtensions = {
		"png",
		"jpeg",
		"jpg",
		"bmp",
		"ico",
		"icns",
		"exr"
	};

	// Create header row
	InHeaderRow.NameContent()
	[
		InPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SDisplayClusterConfiguratorExternalImagePicker)
		.ImagePath(ImagePath)
		.Extensions(ImageExtensions)
		.OnImagePathPicked_Lambda([this](const FString& NewImagePath) { ImagePathHandle->SetValue(NewImagePath); })
	];
}