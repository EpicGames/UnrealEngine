// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPropertyTypeCustomization.h"

/** Type customizer for the FDisplayClusterConfigurationExternalImage type, which replaces the ImagePath text property with a image file picker */
class FDisplayClusterConfiguratorExternalImageTypeCustomization final : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FDisplayClusterConfiguratorExternalImageTypeCustomization>();
	}

	//~ IPropertyTypeCustomization interface begin
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override { }
	//~ IPropertyTypeCustomization interface end

private:
	/** Property handle for the FDisplayClusterConfigurationExternalImage.ImagePath property */
	TSharedPtr<IPropertyHandle> ImagePathHandle;
};