// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfiguratorGenerateMipsCustomization.h"

#include "DisplayClusterConfigurationTypes_PostRender.h"

#include "PropertyHandle.h"

const FName FDisplayClusterConfiguratorGenerateMipsCustomization::SimplifiedMetadataTag = TEXT("Simplified");

void FDisplayClusterConfiguratorGenerateMipsCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	if (InPropertyHandle->HasMetaData(SimplifiedMetadataTag))
	{
		TSharedPtr<IPropertyHandle> MipsAddressUHandle = GET_CHILD_HANDLE(FDisplayClusterConfigurationPostRender_GenerateMips, MipsAddressU);
		MipsAddressUHandle->MarkHiddenByCustomization();

		TSharedPtr<IPropertyHandle> MipsAddressVHandle = GET_CHILD_HANDLE(FDisplayClusterConfigurationPostRender_GenerateMips, MipsAddressV);
		MipsAddressVHandle->MarkHiddenByCustomization();
	}

	FDisplayClusterConfiguratorBaseTypeCustomization::CustomizeChildren(InPropertyHandle, InChildBuilder, CustomizationUtils);
}