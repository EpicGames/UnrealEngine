// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Views/Details/DisplayClusterConfiguratorBaseTypeCustomization.h"

/**
 * Type customization for the FDisplayClusterConfigurationPostRender_GenerateMips type, which supports a custom metadata specifier to display
 * a simplified version of the type that excludes advanced properties.
 */
class FDisplayClusterConfiguratorGenerateMipsCustomization : public FDisplayClusterConfiguratorBaseTypeCustomization
{
public:
	/**
	 * Custom "Simplified" metadata specifier, which hides some of the advanced child properties of a
	 * FDisplayClusterConfigurationPostRender_GenerateMips property
	 */
	static const FName SimplifiedMetadataTag;

	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FDisplayClusterConfiguratorGenerateMipsCustomization>();
	}

	//~ IPropertyTypeCustomization interface begin
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	//~ IPropertyTypeCustomization interface end
};