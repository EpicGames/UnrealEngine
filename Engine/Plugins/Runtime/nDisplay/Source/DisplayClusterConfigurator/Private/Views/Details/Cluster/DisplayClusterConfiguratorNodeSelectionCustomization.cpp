// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfiguratorNodeSelectionCustomization.h"

#include "Components/DisplayClusterICVFXCameraComponent.h"
#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterConfiguratorBlueprintEditor.h"
#include "DisplayClusterConfiguratorUtils.h"
#include "DisplayClusterRootActor.h"
#include "Views/Details/Widgets/SDisplayClusterConfigurationSearchableComboBox.h"

#include "DetailLayoutBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyHandle.h"
#include "Widgets/Text/STextBlock.h"


#define LOCTEXT_NAMESPACE "DisplayClusterConfiguratorNodeSelectionCustomization"

//////////////////////////////////////////////////////////////////////////////////////////////
// Node Selection Customization
//////////////////////////////////////////////////////////////////////////////////////////////

const FName FDisplayClusterConfiguratorNodeSelection::NAME_ElementToolTip = TEXT("ElementToolTip");

FDisplayClusterConfiguratorNodeSelection::FDisplayClusterConfiguratorNodeSelection(EOperationMode InMode, ADisplayClusterRootActor* InRootActor, FDisplayClusterConfiguratorBlueprintEditor* InToolkitPtr)
{
	RootActorPtr = InRootActor;

	if (InToolkitPtr)
	{
		ToolkitPtr = StaticCastSharedRef<FDisplayClusterConfiguratorBlueprintEditor>(InToolkitPtr->AsShared());
	}

	OperationMode = InMode;

	check(RootActorPtr.IsValid() || ToolkitPtr.IsValid());
	ResetOptions();
}

ADisplayClusterRootActor* FDisplayClusterConfiguratorNodeSelection::GetRootActor() const
{
	ADisplayClusterRootActor* RootActor = nullptr;

	if (ToolkitPtr.IsValid())
	{
		RootActor = Cast<ADisplayClusterRootActor>(ToolkitPtr.Pin()->GetPreviewActor());
	}
	else
	{
		RootActor = RootActorPtr.Get();
	}

	ensure(RootActor);

	return RootActor;
}

UDisplayClusterConfigurationData* FDisplayClusterConfiguratorNodeSelection::GetConfigData() const
{
	UDisplayClusterConfigurationData* ConfigData = nullptr;

	if (ToolkitPtr.IsValid())
	{
		ConfigData = ToolkitPtr.Pin()->GetConfig();
	}
	else if (RootActorPtr.IsValid())
	{
		ConfigData = RootActorPtr->GetConfigData();
	}

	return ConfigData;
}

void FDisplayClusterConfiguratorNodeSelection::CreateArrayBuilder(const TSharedRef<IPropertyHandle>& InPropertyHandle,
	IDetailChildrenBuilder& InChildBuilder)
{
	TSharedRef<FDetailArrayBuilder> ArrayBuilder = MakeShareable(new FDetailArrayBuilder(InPropertyHandle));
	ArrayBuilder->OnGenerateArrayElementWidget(FOnGenerateArrayElementWidget::CreateRaw(this,
		&FDisplayClusterConfiguratorNodeSelection::GenerateSelectionWidget));

	InChildBuilder.AddCustomBuilder(ArrayBuilder);
}

FDisplayClusterConfiguratorNodeSelection::EOperationMode FDisplayClusterConfiguratorNodeSelection::GetOperationModeFromProperty(FProperty* Property)
{
	EOperationMode ReturnMode = ClusterNodes;
	if(Property)
	{
		if (const FString* DefinedMode = Property->FindMetaData(TEXT("ConfigurationMode")))
		{
			FString ModeLower = (*DefinedMode).ToLower();
			ModeLower.RemoveSpacesInline();
			if (ModeLower == TEXT("viewports"))
			{
				ReturnMode = FDisplayClusterConfiguratorNodeSelection::EOperationMode::Viewports;
			}
			else if (ModeLower == TEXT("clusternodes"))
			{
				ReturnMode = FDisplayClusterConfiguratorNodeSelection::EOperationMode::ClusterNodes;
			}
			// Define any other modes here.
		}
	}

	return ReturnMode;
}

void FDisplayClusterConfiguratorNodeSelection::GenerateSelectionWidget(
	TSharedRef<IPropertyHandle> PropertyHandle, int32 ArrayIndex, IDetailChildrenBuilder& ChildrenBuilder)
{
	FText ElementTooltip = FText::GetEmpty();

	TSharedPtr<IPropertyHandle> ParentArrayHandle = PropertyHandle->GetParentHandle();
	if (ParentArrayHandle.IsValid() && ParentArrayHandle->IsValidHandle())
	{
		if (const FString* MetaData = ParentArrayHandle->GetInstanceMetaData(NAME_ElementToolTip))
		{
			ElementTooltip = FText::FromString(*MetaData);
		}
		else if (ParentArrayHandle->HasMetaData(NAME_ElementToolTip))
		{
			ElementTooltip = FText::FromString(ParentArrayHandle->GetMetaData(NAME_ElementToolTip));
		}
		else
		{
			ElementTooltip = ParentArrayHandle->GetPropertyDisplayName();
		}
	}

	IDetailPropertyRow& PropertyRow = ChildrenBuilder.AddProperty(PropertyHandle);
	PropertyRow.CustomWidget(false)
		.NameContent()
		[
			PropertyHandle->CreatePropertyNameWidget(FText::GetEmpty(), ElementTooltip)
		]
		.IsEnabled(IsEnabledAttr)
		.ValueContent()
		[
			SAssignNew(OptionsComboBox, SDisplayClusterConfigurationSearchableComboBox)
				.OptionsSource(&Options)
				.OnGenerateWidget(this, &FDisplayClusterConfiguratorNodeSelection::MakeOptionComboWidget)
				.OnSelectionChanged(this, &FDisplayClusterConfiguratorNodeSelection::OnOptionSelected, PropertyHandle)
				.ContentPadding(2)
				.MaxListHeight(200.0f)
				.IsEnabled(IsEnabledAttr)
				.Content()
				[
					SNew(STextBlock)
					.Text(this, &FDisplayClusterConfiguratorNodeSelection::GetSelectedOptionText, PropertyHandle)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
		];
}

void FDisplayClusterConfiguratorNodeSelection::ResetOptions()
{
	Options.Reset();

	if (OperationMode == ICVFXCameras)
	{
		// Get ICVFX camera components from the root actor
		if (ADisplayClusterRootActor* RootActor = GetRootActor())
		{
			TArray<UActorComponent*> CameraComponents;
			RootActor->GetComponents(UDisplayClusterICVFXCameraComponent::StaticClass(), CameraComponents);

			for (UActorComponent* Component : CameraComponents)
			{
				if (Component->IsA<UDisplayClusterICVFXCameraComponent>())
				{
					Options.Add(MakeShared<FString>(Component->GetName()));
				}
			}
		}
	}
	else if (UDisplayClusterConfigurationData* ConfigData = GetConfigData())
	{
		for (const TTuple<FString, TObjectPtr<UDisplayClusterConfigurationClusterNode>>& Node : ConfigData->Cluster->Nodes)
		{
			if (OperationMode == ClusterNodes)
			{
				Options.Add(MakeShared<FString>(Node.Value->GetName()));
				continue;
			}
			for (const TTuple<FString, TObjectPtr<UDisplayClusterConfigurationViewport>>& Viewport : Node.Value->Viewports)
			{
				Options.Add(MakeShared<FString>(Viewport.Value->GetName()));
			}
		}
	}
}

TSharedRef<SWidget> FDisplayClusterConfiguratorNodeSelection::MakeOptionComboWidget(
	TSharedPtr<FString> InItem)
{
	return SNew(STextBlock)
		.Text(FText::FromString(*InItem))
		.Font(IDetailLayoutBuilder::GetDetailFont());
}

void FDisplayClusterConfiguratorNodeSelection::OnOptionSelected(TSharedPtr<FString> InValue,
	ESelectInfo::Type SelectInfo, TSharedRef<IPropertyHandle> InPropertyHandle)
{
	if (InValue.IsValid())
	{
		InPropertyHandle->SetValue(*InValue);
		
		ResetOptions();
		OptionsComboBox->ResetOptionsSource(&Options);
		OptionsComboBox->SetIsOpen(false);
	}
}

FText FDisplayClusterConfiguratorNodeSelection::GetSelectedOptionText(TSharedRef<IPropertyHandle> InPropertyHandle) const
{
	FString Value;
	InPropertyHandle->GetValue(Value);
	return FText::FromString(Value);
}

//////////////////////////////////////////////////////////////////////////////////////////////
// OCIO Profile Customization
//////////////////////////////////////////////////////////////////////////////////////////////

void FDisplayClusterConfiguratorOCIOProfileCustomization::Initialize(const TSharedRef<IPropertyHandle>& InPropertyHandle, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	FDisplayClusterConfiguratorBaseTypeCustomization::Initialize(InPropertyHandle, CustomizationUtils);

	ADisplayClusterRootActor* RootActor = FindRootActor();
	FDisplayClusterConfiguratorBlueprintEditor* BPEditor = FDisplayClusterConfiguratorUtils::GetBlueprintEditorForObject(EditingObject.Get());

	if (RootActor == nullptr && BPEditor == nullptr)
	{
		bIsDefaultDetailsDisplay = true;
		return;
	}
	
	Mode = FDisplayClusterConfiguratorNodeSelection::GetOperationModeFromProperty(InPropertyHandle->GetProperty()->GetOwnerProperty());
	NodeSelection = MakeShared<FDisplayClusterConfiguratorNodeSelection>(Mode, RootActor, BPEditor);
}

void FDisplayClusterConfiguratorOCIOProfileCustomization::SetHeader(const TSharedRef<IPropertyHandle>& InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	if (!bIsDefaultDetailsDisplay)
	{
		FText ElementTooltip = FText::GetEmpty();

		TSharedPtr<IPropertyHandle> ParentArrayHandle = InPropertyHandle->GetParentHandle();
		if (ParentArrayHandle.IsValid() && ParentArrayHandle->IsValidHandle())
		{
			ElementTooltip = ParentArrayHandle->GetPropertyDisplayName();
		}

		InHeaderRow.NameContent()
		[
			InPropertyHandle->CreatePropertyNameWidget(FText::GetEmpty(), ElementTooltip)
		];
	}
	else
	{
		FDisplayClusterConfiguratorBaseTypeCustomization::SetHeader(InPropertyHandle, InHeaderRow, CustomizationUtils);
	}
}

void FDisplayClusterConfiguratorOCIOProfileCustomization::SetChildren(const TSharedRef<IPropertyHandle>& InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	if (!bIsDefaultDetailsDisplay)
	{
		TSharedPtr<IPropertyHandle> EnableOCIOHandle = GET_CHILD_HANDLE(FDisplayClusterConfigurationOCIOProfile, bIsEnabled);
		TSharedPtr<IPropertyHandle> OCIOHandle = GET_CHILD_HANDLE(FDisplayClusterConfigurationOCIOProfile, ColorConfiguration);
		TSharedPtr<IPropertyHandle> ArrayHandle = GET_CHILD_HANDLE(FDisplayClusterConfigurationOCIOProfile, ApplyOCIOToObjects);

		EnableOCIOHandle->SetPropertyDisplayName(Mode == FDisplayClusterConfiguratorNodeSelection::EOperationMode::Viewports ?
			LOCTEXT("EnableOCIOViewportsDisplayName", "Enable Per-Viewport OCIO") : LOCTEXT("EnableOCIOClusterDisplayName", "Enable Per-Node OCIO"));
		EnableOCIOHandle->SetToolTipText(Mode == FDisplayClusterConfiguratorNodeSelection::EOperationMode::Viewports ?
			LOCTEXT("EnableOCIOViewportsTooltip", "Enable the application of an OpenColorIO configuration for the viewport(s) specified.") : LOCTEXT("EnableOCIOClusterNodesTooltip", "Enable the application of an OpenColorIO configuration for the nodes(s) specified."));

		OCIOHandle->SetPropertyDisplayName(Mode == FDisplayClusterConfiguratorNodeSelection::EOperationMode::Viewports ?
			LOCTEXT("OCIOViewportsModeDisplayName", "Viewport OCIO") : LOCTEXT("OCIOClusterModeDisplayName", "Inner Frustum OCIO"));
		OCIOHandle->SetToolTipText(Mode == FDisplayClusterConfiguratorNodeSelection::EOperationMode::Viewports ?
			LOCTEXT("OCIOViewportsModeTooltip", "Viewport OCIO") : LOCTEXT("OCIOClusterModeTooltip", "Inner Frustum OCIO"));

		ArrayHandle->SetPropertyDisplayName(Mode == FDisplayClusterConfiguratorNodeSelection::EOperationMode::Viewports ?
			LOCTEXT("DataViewportsModeDisplayName", "Apply OCIO to Viewports") : LOCTEXT("DataClusterModeDisplayName", "Apply OCIO to Nodes"));
		ArrayHandle->SetToolTipText(Mode == FDisplayClusterConfiguratorNodeSelection::EOperationMode::Viewports ?
			LOCTEXT("DataViewportsModeToolTip", "Specify the viewports to apply this OpenColorIO configuration.") :
			LOCTEXT("DataClusterModeToolTip", "Specify the nodes to apply this OpenColorIO configuration."));
		ArrayHandle->SetInstanceMetaData(FDisplayClusterConfiguratorNodeSelection::NAME_ElementToolTip, ArrayHandle->GetPropertyDisplayName().ToString());

		const TAttribute<bool> OCIOEnabledEditCondition = TAttribute<bool>::Create([this, EnableOCIOHandle]()
		{
			bool bCond1 = false;
			EnableOCIOHandle->GetValue(bCond1);
			return bCond1;
		});

		InChildBuilder.AddProperty(EnableOCIOHandle.ToSharedRef());
		InChildBuilder.AddProperty(OCIOHandle.ToSharedRef()).EditCondition(OCIOEnabledEditCondition, nullptr);

		NodeSelection->IsEnabled(OCIOEnabledEditCondition);
		NodeSelection->CreateArrayBuilder(ArrayHandle.ToSharedRef(), InChildBuilder);
	}
	else
	{
		FDisplayClusterConfiguratorBaseTypeCustomization::SetChildren(InPropertyHandle, InChildBuilder, CustomizationUtils);
	}
}

//////////////////////////////////////////////////////////////////////////////////////////////
// Per viewport color grading customization
//////////////////////////////////////////////////////////////////////////////////////////////
void FDisplayClusterConfiguratorPerViewportColorGradingCustomization::Initialize(const TSharedRef<IPropertyHandle>& InPropertyHandle, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	FDisplayClusterConfiguratorBaseTypeCustomization::Initialize(InPropertyHandle, CustomizationUtils);

	NodeSelection = MakeShared<FDisplayClusterConfiguratorNodeSelection>(FDisplayClusterConfiguratorNodeSelection::EOperationMode::Viewports, FindRootActor(), FDisplayClusterConfiguratorUtils::GetBlueprintEditorForObject(EditingObject.Get()));
}

void FDisplayClusterConfiguratorPerViewportColorGradingCustomization::SetHeader(const TSharedRef<IPropertyHandle>& InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	FText ElementTooltip = FText::GetEmpty();

	TSharedPtr<IPropertyHandle> ParentArrayHandle = InPropertyHandle->GetParentHandle();
	if (ParentArrayHandle.IsValid() && ParentArrayHandle->IsValidHandle())
	{
		ElementTooltip = ParentArrayHandle->GetPropertyDisplayName();
	}

	InHeaderRow.NameContent()
	[
		InPropertyHandle->CreatePropertyNameWidget(FText::GetEmpty(), ElementTooltip)
	];
}

void FDisplayClusterConfiguratorPerViewportColorGradingCustomization::SetChildren(const TSharedRef<IPropertyHandle>& InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	TSharedPtr<IPropertyHandle> NameHandle = GET_CHILD_HANDLE(FDisplayClusterConfigurationViewport_PerViewportColorGrading, Name);
	TSharedPtr<IPropertyHandle> IsEnabledHandle = GET_CHILD_HANDLE(FDisplayClusterConfigurationViewport_PerViewportColorGrading, bIsEnabled);
	TSharedPtr<IPropertyHandle> IsEntireClusterPostProcessHandle = GET_CHILD_HANDLE(FDisplayClusterConfigurationViewport_PerViewportColorGrading, bIsEntireClusterEnabled);
	TSharedPtr<IPropertyHandle> PostProcessSettingsHandle = GET_CHILD_HANDLE(FDisplayClusterConfigurationViewport_PerViewportColorGrading, ColorGradingSettings);
	TSharedPtr<IPropertyHandle> ArrayHandle = GET_CHILD_HANDLE(FDisplayClusterConfigurationViewport_PerViewportColorGrading, ApplyPostProcessToObjects);

	const TAttribute<bool> IsEnabledEditCondition = TAttribute<bool>::Create([this, IsEnabledHandle]()
	{
		bool bCond1 = false;
		IsEnabledHandle->GetValue(bCond1);
		return bCond1;
	});

	InChildBuilder.AddProperty(NameHandle.ToSharedRef());
	InChildBuilder.AddProperty(IsEnabledHandle.ToSharedRef());
	InChildBuilder.AddProperty(IsEntireClusterPostProcessHandle.ToSharedRef());
	InChildBuilder.AddProperty(PostProcessSettingsHandle.ToSharedRef()).EditCondition(IsEnabledEditCondition, nullptr);

	ArrayHandle->SetInstanceMetaData(FDisplayClusterConfiguratorNodeSelection::NAME_ElementToolTip, ArrayHandle->GetPropertyDisplayName().ToString());

	const TAttribute<bool> IsArrayHandleEnabledEditCondition = TAttribute<bool>::Create([this]()
	{
		return true;
	});
	NodeSelection->IsEnabled(IsArrayHandleEnabledEditCondition);
	NodeSelection->CreateArrayBuilder(ArrayHandle.ToSharedRef(), InChildBuilder);
}

//////////////////////////////////////////////////////////////////////////////////////////////
// Per node color grading customization
//////////////////////////////////////////////////////////////////////////////////////////////
void FDisplayClusterConfiguratorPerNodeColorGradingCustomization::Initialize(const TSharedRef<IPropertyHandle>& InPropertyHandle, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	FDisplayClusterConfiguratorBaseTypeCustomization::Initialize(InPropertyHandle, CustomizationUtils);

	NodeSelection = MakeShared<FDisplayClusterConfiguratorNodeSelection>(FDisplayClusterConfiguratorNodeSelection::EOperationMode::ClusterNodes, FindRootActor(), FDisplayClusterConfiguratorUtils::GetBlueprintEditorForObject(EditingObject.Get()));
}

void FDisplayClusterConfiguratorPerNodeColorGradingCustomization::SetHeader(const TSharedRef<IPropertyHandle>& InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	InHeaderRow.NameContent()
	[
		InPropertyHandle->CreatePropertyNameWidget()
	];
}

void FDisplayClusterConfiguratorPerNodeColorGradingCustomization::SetChildren(const TSharedRef<IPropertyHandle>& InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	TSharedPtr<IPropertyHandle> NameHandle = GET_CHILD_HANDLE(FDisplayClusterConfigurationViewport_PerNodeColorGrading, Name);
	TSharedPtr<IPropertyHandle> IsEnabledHandle = GET_CHILD_HANDLE(FDisplayClusterConfigurationViewport_PerNodeColorGrading, bIsEnabled);
	TSharedPtr<IPropertyHandle> IsEntireClusterPostProcessHandle = GET_CHILD_HANDLE(FDisplayClusterConfigurationViewport_PerNodeColorGrading, bEntireClusterColorGrading);
	TSharedPtr<IPropertyHandle> IsAllNodesPostProcessHandle = GET_CHILD_HANDLE(FDisplayClusterConfigurationViewport_PerNodeColorGrading, bAllNodesColorGrading);
	TSharedPtr<IPropertyHandle> PostProcessSettingsHandle = GET_CHILD_HANDLE(FDisplayClusterConfigurationViewport_PerNodeColorGrading, ColorGradingSettings);
	TSharedPtr<IPropertyHandle> ArrayHandle = GET_CHILD_HANDLE(FDisplayClusterConfigurationViewport_PerNodeColorGrading, ApplyPostProcessToObjects);

	const TAttribute<bool> IsEnabledEditCondition = TAttribute<bool>::Create([this, IsEnabledHandle]()
	{
		bool bCond1 = false;
		IsEnabledHandle->GetValue(bCond1);
		return bCond1;
	});

	InChildBuilder.AddProperty(NameHandle.ToSharedRef());
	InChildBuilder.AddProperty(IsEnabledHandle.ToSharedRef());
	InChildBuilder.AddProperty(IsEntireClusterPostProcessHandle.ToSharedRef());
	InChildBuilder.AddProperty(IsAllNodesPostProcessHandle.ToSharedRef());
	InChildBuilder.AddProperty(PostProcessSettingsHandle.ToSharedRef()).EditCondition(IsEnabledEditCondition, nullptr);

	ArrayHandle->SetInstanceMetaData(FDisplayClusterConfiguratorNodeSelection::NAME_ElementToolTip, ArrayHandle->GetPropertyDisplayName().ToString());

	const TAttribute<bool> IsArrayHandleEnabledEditCondition = TAttribute<bool>::Create([this]()
	{
		return true;
	});
	NodeSelection->IsEnabled(IsArrayHandleEnabledEditCondition);
	NodeSelection->CreateArrayBuilder(ArrayHandle.ToSharedRef(), InChildBuilder);
}

#undef LOCTEXT_NAMESPACE