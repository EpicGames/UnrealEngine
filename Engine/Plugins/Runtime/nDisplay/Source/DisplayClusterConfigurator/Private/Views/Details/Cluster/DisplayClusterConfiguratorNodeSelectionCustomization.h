// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Views/Details/DisplayClusterConfiguratorBaseTypeCustomization.h"

class FDisplayClusterConfiguratorBlueprintEditor;
class ADisplayClusterRootActor;
class UDisplayClusterConfigurationData;
class SDisplayClusterConfigurationSearchableComboBox;
class SWidget;

/**
 * Helper class used to construct and manage a custom array property widget that displays for each element a dropdown containing
 * a list of the display cluster's cluster nodes, viewports, or ICVFX cameras to select from.
 */
class FDisplayClusterConfiguratorNodeSelection : public TSharedFromThis<FDisplayClusterConfiguratorNodeSelection>
{
public:
	/** Custom "ElementToolTip" metadata specifier that can be used to specify the tooltip to apply to each element of the displayed array */
	static const FName NAME_ElementToolTip;

	/** Flag to indicate which cluster item is displayed in the dropdown menus of the elements of the array widget  */
	enum EOperationMode
	{
		Viewports,
		ClusterNodes,
		ICVFXCameras
	};

	FDisplayClusterConfiguratorNodeSelection(EOperationMode InMode, ADisplayClusterRootActor* InRootActor, FDisplayClusterConfiguratorBlueprintEditor* InToolkitPtr);

	~FDisplayClusterConfiguratorNodeSelection()
	{
		OptionsComboBox.Reset();
		Options.Reset();
	}

	/** Gets the DisplayClusterRootActor that is being edited by the current details panel */
	ADisplayClusterRootActor* GetRootActor() const;

	/** Gets the cluster configuration data object of the root actor being edited by the current details panel */
	UDisplayClusterConfigurationData* GetConfigData() const;

	/** Creates the child array builder that will be used to construct the cluster item selection array elements for the details panel */
	void CreateArrayBuilder(const TSharedRef<IPropertyHandle>& InPropertyHandle, IDetailChildrenBuilder& InChildBuilder);

	/** Sets the IsEnabled attribute to use on the array property handle */
	FDisplayClusterConfiguratorNodeSelection& IsEnabled(const TAttribute<bool>& InIsEnabled)
	{
		IsEnabledAttr = InIsEnabled;
		return *this;
	}

	/** Inspects the specified property's metadata to determine if the node selection array should show cluster nodes or viewports or icvfx cameras in its dropdown lists */
	static EOperationMode GetOperationModeFromProperty(FProperty* Property);

	/** Generates a widget to display the currently selected cluster item for the specified element in the array widget and adds it to the details panel */
	void GenerateSelectionWidget(TSharedRef<IPropertyHandle> PropertyHandle, int32 ArrayIndex, IDetailChildrenBuilder& ChildrenBuilder);

protected:

	/** Resets the list of cluster items to display in the dropdown lists for each element of the array widget */
	void ResetOptions();

	/** Constructs the text widget to display in the dropdown menu for the specified item */
	TSharedRef<SWidget> MakeOptionComboWidget(TSharedPtr<FString> InItem);

	/** Raised when an item within the dropdown menu is selected */
	void OnOptionSelected(TSharedPtr<FString> InValue, ESelectInfo::Type SelectInfo, TSharedRef<IPropertyHandle> InPropertyHandle);

	/** Gets the display text for the currently selected item of the element in the array widget */
	FText GetSelectedOptionText(TSharedRef<IPropertyHandle> InPropertyHandle) const;

private:
	/** A custom attribute to apply to the array widget to determine if it is enabled or not */
	TAttribute<bool> IsEnabledAttr;

	/** The combo box widget to display for each element in the array widget */
	TSharedPtr<SDisplayClusterConfigurationSearchableComboBox> OptionsComboBox;

	/** The list of options to display in each combo box's dropdown menu */
	TArray<TSharedPtr<FString>> Options;

	/** A weak reference to the current display cluster configuration editor, if the selected root actor is being edited as a CDO in a blueprint editor */
	TWeakPtr<FDisplayClusterConfiguratorBlueprintEditor> ToolkitPtr = nullptr;

	/** A weak reference to the current display cluster root actor that is being edited by the details panel */
	TWeakObjectPtr<ADisplayClusterRootActor> RootActorPtr;

	/** The operation mode which indicates whether to display cluster nodes, viewports, or ICVFX cameras in each element's combo box */
	EOperationMode OperationMode = ClusterNodes;
};

/** Type customization for the FDisplayClusterConfigurationOCIOProfile type, to customize the ApplyOCIOToObjects list property */
class FDisplayClusterConfiguratorOCIOProfileCustomization final
	: public FDisplayClusterConfiguratorBaseTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FDisplayClusterConfiguratorOCIOProfileCustomization>();
	}

	virtual ~FDisplayClusterConfiguratorOCIOProfileCustomization() override
	{
		NodeSelection.Reset();
	}

protected:
	//~ FDisplayClusterConfiguratorBaseTypeCustomization overrides begin
	virtual void Initialize(const TSharedRef<IPropertyHandle>& InPropertyHandle, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void SetHeader(const TSharedRef<IPropertyHandle>& InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void SetChildren(const TSharedRef<IPropertyHandle>& InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	//~ FDisplayClusterConfiguratorBaseTypeCustomization overrides end

private:
	/** The helper object used to generate the custom array widget to display in place of a string list for the ApplyOCIOToObjects property */
	TSharedPtr<FDisplayClusterConfiguratorNodeSelection> NodeSelection;

	/** Flag to indicate if the cluster items being displayed in the node selection widgets are viewports or cluster nodes */
	FDisplayClusterConfiguratorNodeSelection::EOperationMode Mode = FDisplayClusterConfiguratorNodeSelection::EOperationMode::Viewports;

	/** Indicates whether this customizer should display the default details panel instead of customizing the type's properties */
	bool bIsDefaultDetailsDisplay = false;
};


/** Type customization for the FDisplayClusterConfigurationViewport_PerViewportColorGrading type, to customize the ApplyPostProcessToObjects list property */
class FDisplayClusterConfiguratorPerViewportColorGradingCustomization final
	: public FDisplayClusterConfiguratorBaseTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FDisplayClusterConfiguratorPerViewportColorGradingCustomization>();
	}

	virtual ~FDisplayClusterConfiguratorPerViewportColorGradingCustomization() override
	{
		NodeSelection.Reset();
	}

protected:
	//~ FDisplayClusterConfiguratorBaseTypeCustomization overrides begin
	virtual void Initialize(const TSharedRef<IPropertyHandle>& InPropertyHandle, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void SetHeader(const TSharedRef<IPropertyHandle>& InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void SetChildren(const TSharedRef<IPropertyHandle>& InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	//~ FDisplayClusterConfiguratorBaseTypeCustomization overrides end

private:
	/** The helper object used to generate the custom array widget to display in place of a string list for the ApplyPostProcessToObjects property */
	TSharedPtr<FDisplayClusterConfiguratorNodeSelection> NodeSelection;
};


/** Type customization for the FDisplayClusterConfigurationViewport_PerNodeColorGrading type, to customize the ApplyPostProcessToObjects list property */
class FDisplayClusterConfiguratorPerNodeColorGradingCustomization final
	: public FDisplayClusterConfiguratorBaseTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FDisplayClusterConfiguratorPerNodeColorGradingCustomization>();
	}

	virtual ~FDisplayClusterConfiguratorPerNodeColorGradingCustomization() override
	{
		NodeSelection.Reset();
	}

protected:
	//~ FDisplayClusterConfiguratorBaseTypeCustomization overrides begin
	virtual void Initialize(const TSharedRef<IPropertyHandle>& InPropertyHandle, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void SetHeader(const TSharedRef<IPropertyHandle>& InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void SetChildren(const TSharedRef<IPropertyHandle>& InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	//~ FDisplayClusterConfiguratorBaseTypeCustomization overrides end

private:
	/** The helper object used to generate the custom array widget to display in place of a string list for the ApplyPostProcessToObjects property */
	TSharedPtr<FDisplayClusterConfiguratorNodeSelection> NodeSelection;
};