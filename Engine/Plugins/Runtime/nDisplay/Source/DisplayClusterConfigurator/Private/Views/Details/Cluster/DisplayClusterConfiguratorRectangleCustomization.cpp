// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfiguratorRectangleCustomization.h"

#include "DisplayClusterConfigurationTypes_Base.h"

#include "Editor.h"
#include "PropertyHandle.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailCustomNodeBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "FRectanglePositionCustomNodeBuilder"
                 
DECLARE_DELEGATE_OneParam(FOnRatioLockToggled, bool);

/** 
 * A custom node builder that can combine two component properties into a single compound display similar to vectors where the components
 * are shown in the property's header row. Also supports displaying a ratio lock toggle in the header row.
 */
class FRectangleCompoundCustomNodeBuilder : public IDetailCustomNodeBuilder, public TSharedFromThis<FRectangleCompoundCustomNodeBuilder>
{
public:
	FRectangleCompoundCustomNodeBuilder(TSharedPtr<IPropertyHandle> InRectanglePropertyHandle, const FName& XComponentProperty, const FName& YComponentProperty, FText InCompoundLabel)
	{
		RectanglePropertyHandle = InRectanglePropertyHandle;
		XComponentPropertyHandle = RectanglePropertyHandle->GetChildHandle(XComponentProperty);
		YComponentPropertyHandle = RectanglePropertyHandle->GetChildHandle(YComponentProperty);
		CompoundLabel = InCompoundLabel;
	}

	virtual void GenerateHeaderRowContent(FDetailWidgetRow& NodeRow) override
	{
		TWeakPtr<IPropertyHandle> WeakRectHandlePtr = RectanglePropertyHandle;

		FText RatioLockTooltip = FText::GetEmpty();
		if (XComponentPropertyHandle.IsValid() && YComponentPropertyHandle.IsValid())
		{
			RatioLockTooltip = FText::Format(
				LOCTEXT("RatioLockToolTip", "When locked, maintains the ratio between {0} and {1}"),
				XComponentPropertyHandle->GetPropertyDisplayName(),
				YComponentPropertyHandle->GetPropertyDisplayName());
		}

		NodeRow
			.NameContent()
			[
				SNew(STextBlock)
				.Text(CompoundLabel)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.ValueContent()
			.MinDesiredWidth(250.0f)
			.MaxDesiredWidth(250.0f)
			[
				SNew(SHorizontalBox)
				.IsEnabled(this, &FRectangleCompoundCustomNodeBuilder::IsPropertyEnabled, WeakRectHandlePtr)

				+SHorizontalBox::Slot()
				.Padding(FMargin(0.0f, 2.0f, 3.0f, 2.0f))
				[
					CreateNumericEntryBox(XComponentPropertyHandle.ToSharedRef())
				]

				+SHorizontalBox::Slot()
				.Padding(FMargin(0.0f, 2.0f, 0.0f, 2.0f))
				[
					CreateNumericEntryBox(YComponentPropertyHandle.ToSharedRef())
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				.MaxWidth(18.0f)
				.VAlign(VAlign_Center)
				[
					SNew(SCheckBox)
					.Visibility(this, &FRectangleCompoundCustomNodeBuilder::GetRatioLockVisibility)
					.IsChecked(this, &FRectangleCompoundCustomNodeBuilder::GetRatioLockCheckState)
					.OnCheckStateChanged(this, &FRectangleCompoundCustomNodeBuilder::OnRatioLockToggled)
					.Style(FAppStyle::Get(), "TransparentCheckBox")
					.ToolTipText(RatioLockTooltip)
					[
						SNew(SImage)
						.Image(this, &FRectangleCompoundCustomNodeBuilder::GetRatioLockIcon)
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]
			];
	}

	virtual void GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder) override
	{
		ChildrenBuilder.AddProperty(XComponentPropertyHandle.ToSharedRef());
		ChildrenBuilder.AddProperty(YComponentPropertyHandle.ToSharedRef());
	}

	virtual FName GetName() const
	{
		static const FName Name("DisplayClusterRectanglePositionCustomNodeBuilder");
		return Name;
	}

	/** Sets whether the ratio lock toggle is displayed in the header row */
	void SetAllowLockedRatio(bool bInAllowLockedRatio)
	{
		bAllowLockedRatio = bInAllowLockedRatio;
	}

	/** Sets the ratio lock toggle attribute */
	void SetRatioLocked(TAttribute<bool> InRatioLockedAttribute)
	{
		RatioLockedAttribute = InRatioLockedAttribute;
	}

	/** Delegate that is raised when the ratio lock is toggled */
	FOnRatioLockToggled& OnRatioLockToggled()
	{
		return OnRatioLockToggledHandle;
	}

private:
	/** Creates a numeric entry box  for the specified property handle to display in the header row of the compound property */
	TSharedRef<SWidget> CreateNumericEntryBox(TSharedRef<IPropertyHandle> PropertyHandle)
	{
		auto GetMetaData = [&PropertyHandle](const FName& Key) -> const FString&
		{
			const FString* InstanceValue = PropertyHandle->GetInstanceMetaData(Key);
			return (InstanceValue != nullptr) ? *InstanceValue : PropertyHandle->GetMetaData(Key);
		};

		const FString& ClampMinString = GetMetaData(TEXT("ClampMin"));
		const FString& ClampMaxString = GetMetaData(TEXT("ClampMax"));
		const FString& UIMinString = GetMetaData(TEXT("UIMin"));
		const FString& UIMaxString = GetMetaData(TEXT("UIMax"));
		const FString& SliderExponentString = GetMetaData(TEXT("SliderExponent"));
		const FString& DeltaString = GetMetaData(TEXT("Delta"));
		const FString& LinearDeltaSensitivityString = GetMetaData(TEXT("LinearDeltaSensitivity"));
		const FString& ShiftMultiplierString = GetMetaData(TEXT("ShiftMultiplier"));
		const FString& CtrlMultiplierString = GetMetaData(TEXT("CtrlMultiplier"));

		TOptional<int32> ClampMin = !ClampMinString.IsEmpty() ? FCString::Atoi(*ClampMinString) : TOptional<int32>();
		TOptional<int32> ClampMax = !ClampMaxString.IsEmpty() ? FCString::Atoi(*ClampMaxString) : TOptional<int32>();
		TOptional<int32> UIMin = !UIMinString.IsEmpty() ? FMath::Max(ClampMin.Get(TNumericLimits<int32>::Min()), FCString::Atoi(*UIMinString)) : ClampMin;
		TOptional<int32> UIMax = !UIMaxString.IsEmpty() ? FMath::Min(ClampMax.Get(TNumericLimits<int32>::Max()), FCString::Atoi(*UIMaxString)) : ClampMax;
		float SliderExponent = !SliderExponentString.IsEmpty() ? FCString::Atof(*SliderExponentString) : 1.0f;
		int32 Delta = !DeltaString.IsEmpty() ? FCString::Atoi(*DeltaString) : 0;
		int32 LinearDeltaSensitivity = !LinearDeltaSensitivityString.IsEmpty() ? FCString::Atoi(*LinearDeltaSensitivityString) : 0;
		float ShiftMultiplier = !ShiftMultiplierString.IsEmpty() ? FCString::Atof(*ShiftMultiplierString) : 10.f;
		float CtrlMultiplier = !CtrlMultiplierString.IsEmpty() ? FCString::Atof(*CtrlMultiplierString) : 0.1f;

		TWeakPtr<IPropertyHandle> WeakHandlePtr = PropertyHandle;

		return SNew(SNumericEntryBox<int32>)
			.IsEnabled(this, &FRectangleCompoundCustomNodeBuilder::IsPropertyEnabled, WeakHandlePtr)
			.EditableTextBoxStyle(&FCoreStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox"))
			.ToolTipText(this, &FRectangleCompoundCustomNodeBuilder::GetPropertyToolTip, WeakHandlePtr)
			.Value(this, &FRectangleCompoundCustomNodeBuilder::GetPropertyValue, WeakHandlePtr)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.UndeterminedString(NSLOCTEXT("PropertyEditor", "MultipleValues", "Multiple Values"))
			.OnValueCommitted(this, &FRectangleCompoundCustomNodeBuilder::OnValueCommitted, WeakHandlePtr)
			.OnValueChanged(this, &FRectangleCompoundCustomNodeBuilder::OnValueChanged, WeakHandlePtr)
			.OnBeginSliderMovement(this, &FRectangleCompoundCustomNodeBuilder::OnBeginSliderMovement, WeakHandlePtr)
			.OnEndSliderMovement(this, &FRectangleCompoundCustomNodeBuilder::OnEndSliderMovement, WeakHandlePtr)
			.AllowSpin(PropertyHandle->GetNumOuterObjects() < 2)
			.ShiftMultiplier(ShiftMultiplier)
			.CtrlMultiplier(CtrlMultiplier)
			.MinValue(ClampMin)
			.MaxValue(ClampMax)
			.MinSliderValue(UIMin)
			.MaxSliderValue(UIMax)
			.SliderExponent(SliderExponent)
			.Delta(Delta)
			.LinearDeltaSensitivity(LinearDeltaSensitivity != 0 ? LinearDeltaSensitivity : TAttribute<int32>());
	}

	/** Gets whether the specified property handle is enabled or not */
	bool IsPropertyEnabled(TWeakPtr<IPropertyHandle> PropertyHandle) const
	{
		return PropertyHandle.IsValid() && !PropertyHandle.Pin()->IsEditConst();
	}

	/** Gets the tooltip for the specified property handle to display for the entry box in the header row */
	FText GetPropertyToolTip(TWeakPtr<IPropertyHandle> PropertyHandle) const
	{
		if(PropertyHandle.IsValid())
		{
			TOptional<int32> Value = GetPropertyValue(PropertyHandle);
			if (Value.IsSet())
			{
				return FText::Format(LOCTEXT("ValueToolTip", "{0}: {1}"),  PropertyHandle.Pin()->GetPropertyDisplayName(), FText::AsNumber(Value.GetValue()));
			}
		}
	
		return FText::GetEmpty();
	}

	/** Gets the value of the specified property handle */
	TOptional<int32> GetPropertyValue(TWeakPtr<IPropertyHandle> PropertyHandle) const
	{
		int32 Value = 0;
		if (PropertyHandle.IsValid() && PropertyHandle.Pin()->GetValue(Value) == FPropertyAccess::Success)
		{
			return TOptional<int32>(Value);
		}

		return TOptional<int32>();
	}

	/** Sets the value of the specified property handle. If the ratio lock is on, also updates the other component to an appropriately scaled value */
	void SetPropertyValue(int32 NewValue, EPropertyValueSetFlags::Type Flags, TWeakPtr<IPropertyHandle> PropertyHandle)
	{
		if (PropertyHandle.IsValid())
		{
			if (RatioLockedAttribute.Get(false))
			{
				TSharedPtr<IPropertyHandle> DenominatorHandle = PropertyHandle.Pin() == XComponentPropertyHandle ? YComponentPropertyHandle : XComponentPropertyHandle;

				if (DenominatorHandle.IsValid())
				{
					int32 NumValues = DenominatorHandle->GetNumPerObjectValues();
					for (int32 Index = 0; Index < NumValues; ++Index)
					{
						// A zero ratio indicates a proper ratio could not be computed for this denominator value, so just skip updating it
						float Ratio = CurrentRatios[Index];
						if (Ratio != 0)
						{
							int32 NewDenom = FMath::RoundToInt(NewValue / CurrentRatios[Index]);
							DenominatorHandle->SetPerObjectValue(Index, TTypeToString<int32>::ToSanitizedString(NewDenom), Flags);
						}
					}

				}
			}

			PropertyHandle.Pin()->SetValue(NewValue, Flags);
		}
	}

	/** Computes the current ratios between the two specified components per object and stores each ratio in the specified array */
	void GetCurrentAspectRatios(TSharedPtr<IPropertyHandle> NumeratorHandle, TSharedPtr<IPropertyHandle> DenominatorHandle, TArray<float>& OutRatios)
	{
		if (NumeratorHandle.IsValid() && DenominatorHandle.IsValid())
		{
			OutRatios.Init(0.0f, NumeratorHandle->GetNumPerObjectValues());

			TArray<FString> NumStrings;
			TArray<FString> DenomStrings;

			if (NumeratorHandle->GetPerObjectValues(NumStrings) == FPropertyAccess::Success && DenominatorHandle->GetPerObjectValues(DenomStrings) == FPropertyAccess::Success)
			{
				for (int32 Index = 0; Index < NumStrings.Num(); ++Index)
				{
					int32 Num;
					int32 Denom;

					TTypeFromString<int32>::FromString(Num, *NumStrings[Index]);
					TTypeFromString<int32>::FromString(Denom, *DenomStrings[Index]);

					if (Denom != 0.0f)
					{
						OutRatios[Index] = Denom != 0 ? (float)Num / (float)Denom : 0.0;
					}
				}
			}
		}
	}

	/** Raised when a value is committed for the specified property handle through the numeric entry box in the header row */
	void OnValueCommitted(int32 NewValue, ETextCommit::Type CommitType, TWeakPtr<IPropertyHandle> PropertyHandle)
	{
		if (RatioLockedAttribute.Get(false))
		{
			TSharedPtr<IPropertyHandle> NumeratorHandle = PropertyHandle.Pin();
			TSharedPtr<IPropertyHandle> DenominatorHandle = NumeratorHandle == XComponentPropertyHandle ? YComponentPropertyHandle : XComponentPropertyHandle;

			CurrentRatios.Empty();
			GetCurrentAspectRatios(NumeratorHandle, DenominatorHandle, CurrentRatios);
		}

		EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags;
		SetPropertyValue(NewValue, Flags, PropertyHandle);
	}

	/** Raised when a value is changed for the specified property handle through the numeric entry box in the header row */
	void OnValueChanged(int32 NewValue, TWeakPtr<IPropertyHandle> PropertyHandle)
	{
		if (bIsUsingSlider)
		{
			EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::InteractiveChange;
			SetPropertyValue(NewValue, Flags, PropertyHandle);
		}
	}

	/** Raised when the slider is starting to be moved for the numeric entry box of the specified property handle */
	void OnBeginSliderMovement(TWeakPtr<IPropertyHandle> PropertyHandle)
	{
		bIsUsingSlider = true;

		if (RatioLockedAttribute.Get(false))
		{
			// To ensure a stable ratio is maintained during rapid changes in the component values, compute the ratios only once, when the slider movement is beginning
			TSharedPtr<IPropertyHandle> NumeratorHandle = PropertyHandle.Pin();
			TSharedPtr<IPropertyHandle> DenominatorHandle = NumeratorHandle == XComponentPropertyHandle ? YComponentPropertyHandle : XComponentPropertyHandle;

			CurrentRatios.Empty();
			GetCurrentAspectRatios(NumeratorHandle, DenominatorHandle, CurrentRatios);
		}

		GEditor->BeginTransaction(LOCTEXT("SetRectangleProperty", "Set Rectangle Property"));
	}

	/** Raised when the slider movement is ended for the numeric entry box of the specified property handle */
	void OnEndSliderMovement(int32 NewValue, TWeakPtr<IPropertyHandle> PropertyHandle)
	{
		bIsUsingSlider = false;
		GEditor->EndTransaction();
	}

	/** Gets the visibility of the ratio lock toggle button */
	EVisibility GetRatioLockVisibility() const
	{
		return bAllowLockedRatio ? EVisibility::Visible : EVisibility::Collapsed;
	}

	/** Gets the check state of the ratio lock toggle button */
	ECheckBoxState GetRatioLockCheckState() const
	{
		return RatioLockedAttribute.Get(false) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	/** Raised when the ratio lock is toggled */
	void OnRatioLockToggled(ECheckBoxState NewState)
	{
		OnRatioLockToggledHandle.ExecuteIfBound(!RatioLockedAttribute.Get(false));
	}

	/** Gets the icon to use for the ratio lock toggle button */
	const FSlateBrush* GetRatioLockIcon() const
	{
		return RatioLockedAttribute.Get(false) ? FAppStyle::GetBrush(TEXT("Icons.Lock")) : FAppStyle::GetBrush(TEXT("Icons.Unlock"));
	}

private:
	/** The parent struct property handle that contains the components to be combined */
	TSharedPtr<IPropertyHandle> RectanglePropertyHandle;

	/** The x component property handle, which will be displayed first in the compound header row */
	TSharedPtr<IPropertyHandle> XComponentPropertyHandle;

	/** The y component property handle, which will be displayed second in the compound header row */
	TSharedPtr<IPropertyHandle> YComponentPropertyHandle;

	/** The boolean attribute that indicates if the ratio lock is toggled on or off */
	TAttribute<bool> RatioLockedAttribute = TAttribute<bool>();

	/** The label to display in the compound property's header row */
	FText CompoundLabel;

	/** Indicates that the user is currently changing the value of a numeric entry box using the slider */
	bool bIsUsingSlider = false;

	/** Indicates if the ratio lock toggle is displayed for the compound property */
	bool bAllowLockedRatio = false;

	/** Stores the current ratios of the x and y component properties, used when the ratio lock is on to ensure the two components maintain a fixed ratio */
	TArray<float> CurrentRatios;

	/** A delegate that is raised when the ratio lock is toggled */
	FOnRatioLockToggled OnRatioLockToggledHandle;
};

const FName FDisplayClusterConfiguratorRectangleCustomization::DisplayModeMetadataKey = TEXT("DisplayMode");
const FName FDisplayClusterConfiguratorRectangleCustomization::FixedAspectRatioPropertyMetadataKey = TEXT("FixedAspectRatioProperty");

void FDisplayClusterConfiguratorRectangleCustomization::Initialize(const TSharedRef<IPropertyHandle>& InPropertyHandle, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	FDisplayClusterConfiguratorBaseTypeCustomization::Initialize(InPropertyHandle, CustomizationUtils);

	DisplayMode = EDisplayMode::Default;
	if (const FString* DisplayModeStr = FindMetaData(InPropertyHandle, DisplayModeMetadataKey))
	{
		if (DisplayModeStr->ToLower() == TEXT("compound"))
		{
			DisplayMode = EDisplayMode::Compound;
		}
	}

	if (const FString* FixedAspectRatioPropertyStr = FindMetaData(InPropertyHandle, FixedAspectRatioPropertyMetadataKey))
	{
		if (!FixedAspectRatioPropertyStr->IsEmpty())
		{
			TSharedPtr<IPropertyHandle> CurrentPropertyHandle = InPropertyHandle->GetParentHandle();
			while (CurrentPropertyHandle)
			{
				if (TSharedPtr<IPropertyHandle> ChildHandle = CurrentPropertyHandle->GetChildHandle(FName(*FixedAspectRatioPropertyStr)))
				{
					FixedAspectRatioProperty = ChildHandle;
					break;
				}

				CurrentPropertyHandle = CurrentPropertyHandle->GetParentHandle();
			}
		}
	}
}

void FDisplayClusterConfiguratorRectangleCustomization::SetChildren(const TSharedRef<IPropertyHandle>& InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	if (DisplayMode == EDisplayMode::Default)
	{
		AddAllChildren(InPropertyHandle, InChildBuilder);

		if (FixedAspectRatioProperty && FixedAspectRatioProperty->IsValidHandle())
		{
			// Mark the property with the "IsCustomized" flag so that any subsequent layout builders can account for the property
			// being moved and placed here.
			FixedAspectRatioProperty->MarkHiddenByCustomization();
			InChildBuilder.AddProperty(FixedAspectRatioProperty.ToSharedRef());
		}
	}
	else if (DisplayMode == EDisplayMode::Compound)
	{
		TSharedRef<FRectangleCompoundCustomNodeBuilder> CompoundPositionBuilder = MakeShareable(new FRectangleCompoundCustomNodeBuilder(
			InPropertyHandle, 
			GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationRectangle, X), 
			GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationRectangle, Y),
			LOCTEXT("CompoundPosition_Label", "Position")));

		InChildBuilder.AddCustomBuilder(CompoundPositionBuilder);

		TSharedRef<FRectangleCompoundCustomNodeBuilder> CompoundSizeBuilder = MakeShareable(new FRectangleCompoundCustomNodeBuilder(
			InPropertyHandle, 
			GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationRectangle, W), 
			GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationRectangle, H),
			LOCTEXT("CompoundSize_Label", "Size")));

		CompoundSizeBuilder->SetAllowLockedRatio(true);
		CompoundSizeBuilder->SetRatioLocked(TAttribute<bool>(this, &FDisplayClusterConfiguratorRectangleCustomization::GetSizeRatioLocked));
		CompoundSizeBuilder->OnRatioLockToggled().BindSP(this, &FDisplayClusterConfiguratorRectangleCustomization::OnSizeRatioLockToggled);

		InChildBuilder.AddCustomBuilder(CompoundSizeBuilder);

		if (FixedAspectRatioProperty && FixedAspectRatioProperty->IsValidHandle())
		{
			// Mark the property with the "IsCustomized" flag so that any subsequent layout builders can account for the property
			// being moved and placed here.
			FixedAspectRatioProperty->MarkHiddenByCustomization();
		}
	}
}

bool FDisplayClusterConfiguratorRectangleCustomization::GetSizeRatioLocked() const
{
	if (FixedAspectRatioProperty && FixedAspectRatioProperty->IsValidHandle())
	{
		bool bCurrentValue = false;
		if (FixedAspectRatioProperty->GetValue(bCurrentValue) == FPropertyAccess::Success)
		{
			return bCurrentValue;
		}
	}

	return false;
}

void FDisplayClusterConfiguratorRectangleCustomization::OnSizeRatioLockToggled(bool bNewValue)
{
	if (FixedAspectRatioProperty && FixedAspectRatioProperty->IsValidHandle())
	{
		EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::InteractiveChange;
		FixedAspectRatioProperty->SetValue(bNewValue, Flags);
	}
}

#undef LOCTEXT_NAMESPACE