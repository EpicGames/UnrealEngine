// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Views/Details/DisplayClusterConfiguratorBaseTypeCustomization.h"

/**
 * Details panel customization for the FDisplayClusterConfigurationRectangle struct. Allows the rectangle to be displayed in a 
 * compound arrangement similar to how vectors can be displayed with their components editable in the header row of the property.
 * Also supports a fixed aspect ratio toggle for the width and height of the rectangle
 */
class FDisplayClusterConfiguratorRectangleCustomization : public FDisplayClusterConfiguratorBaseTypeCustomization
{
public:
	/** The display mode of the rectangle */
	enum EDisplayMode
	{
		Compound,
		Default
	};

	/** Custom "DisplayMode" meatadata specifier to specify how the FDisplayClusterConfigurationRectangle struct gets displayed */
	static const FName DisplayModeMetadataKey;

	/** Custom "FixedAspectRatioProperty" metadata specifier to specify a property for indicating if the aspect ratio of the rectangle is fixed */
	static const FName FixedAspectRatioPropertyMetadataKey;

	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FDisplayClusterConfiguratorRectangleCustomization>();
	}

protected:
	virtual void Initialize(const TSharedRef<IPropertyHandle>& InPropertyHandle, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void SetChildren(const TSharedRef<IPropertyHandle>& InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

private:
	/** Gets whether the rectangle's aspect ratio is locked */
	bool GetSizeRatioLocked() const;

	/** Raised when the aspect ratio lock is toggled  */
	void OnSizeRatioLockToggled(bool bNewValue);

private:
	/** The property handle of the property to use as the aspect ratio toggle */
	TSharedPtr<IPropertyHandle> FixedAspectRatioProperty;

	/** The display mode of the rectangle being displayed */
	EDisplayMode DisplayMode;
};