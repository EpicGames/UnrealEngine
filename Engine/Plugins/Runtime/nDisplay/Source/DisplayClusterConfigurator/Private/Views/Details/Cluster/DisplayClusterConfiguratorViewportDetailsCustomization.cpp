// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfiguratorViewportDetailsCustomization.h"

#include "Components/DisplayClusterCameraComponent.h"
#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterConfigurationTypes_Viewport.h"
#include "DisplayClusterConfiguratorLog.h"
#include "DisplayClusterProjectionStrings.h"
#include "DisplayClusterRootActor.h"
#include "Render/DisplayDevice/Components/DisplayClusterDisplayDeviceBaseComponent.h"
#include "Views/Details/Cluster/DisplayClusterConfiguratorNodeSelectionCustomization.h"
#include "Views/Details/Widgets/SDisplayClusterConfigurationSearchableComboBox.h"
#include "Views/Details/Widgets/SDisplayClusterConfiguratorComponentPicker.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyHandle.h"

void FDisplayClusterConfiguratorViewportDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& InLayoutBuilder)
{
	Super::CustomizeDetails(InLayoutBuilder);

	ConfigurationViewportPtr = nullptr;
	DefaultViewPointComponentNameOption = MakeShared<FString>("DefaultViewPoint");

	// Set config data pointer
	UDisplayClusterConfigurationData* ConfigurationData = GetConfigData();
	if (ConfigurationData == nullptr)
	{
		UE_LOGF(DisplayClusterConfiguratorLog, Warning, "Details panel config data invalid.");
		return;
	}

	ConfigurationDataPtr = ConfigurationData;

	// Get the Editing object
	const TArray<TWeakObjectPtr<UObject>>& SelectedObjects = InLayoutBuilder.GetSelectedObjects();
	if (SelectedObjects.Num())
	{
		ConfigurationViewportPtr = Cast<UDisplayClusterConfigurationViewport>(SelectedObjects[0]);
	}

	if (ConfigurationViewportPtr == nullptr)
	{
		UE_LOGF(DisplayClusterConfiguratorLog, Warning, "No valid viewport selected in details panel.");
		return;
	}

	if (!ConfigurationViewportPtr->ProjectionPolicy.ShouldDisplayICVFXCategory())
	{
		IDetailCategoryBuilder& Category = InLayoutBuilder.EditCategory(DisplayClusterConfigurationStrings::categories::InCameraVFXCategory);
		Category.SetCategoryVisibility(false);
	}
	else
	{
		// Customize the EnforcedCameraOrder array property with ICVFX camera dropdowns
		CustomizeEnforcedCameraOrder(InLayoutBuilder);
	}

	{
		CameraHandle = InLayoutBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationViewport, Camera));
		check(CameraHandle->IsValidHandle());

		if (ConfigurationViewportPtr->ProjectionPolicy.Type.Equals(DisplayClusterProjectionStrings::projection::Camera, ESearchCase::IgnoreCase))
		{
			CameraHandle->MarkHiddenByCustomization();
		}
		else
		{
			ResetCameraOptions();

			if (IDetailPropertyRow* CameraPropertyRow = InLayoutBuilder.EditDefaultProperty(CameraHandle))
			{
				CameraPropertyRow->CustomWidget()
					.NameContent()
					[
						CameraHandle->CreatePropertyNameWidget()
					]
					.ValueContent()
					[
						CreateCustomCameraWidget()
					];
			}
		}
	}
	
	{
		TSharedPtr<IPropertyHandle> DisplayDeviceHandle = InLayoutBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationViewport, DisplayDeviceName));
		check(DisplayDeviceHandle->IsValidHandle());
		
		if (IDetailPropertyRow* DisplayDevicePropertyRow = InLayoutBuilder.EditDefaultProperty(DisplayDeviceHandle))
		{
			DisplayDevicePropertyRow->CustomWidget()
				.NameContent()
				[
					DisplayDeviceHandle->CreatePropertyNameWidget()
				]
				.ValueContent()
				[
					SNew(SDisplayClusterConfiguratorComponentPicker,
						UDisplayClusterDisplayDeviceBaseComponent::StaticClass(),
						GetRootActor(),
						DisplayDeviceHandle)
						.DefaultOptionText(NSLOCTEXT("nDisplayDevicePicker", "DefaultText", "Default"))
						.DefaultOptionValue(TOptional<FString>(""))
				];
		}
	}
	
	// Update the metadata for the viewport's region. Must set this here instead of in the UPROPERTY specifier because
	// the Region property is a generic FDisplayClusterConfigurationRectangle struct which is used in lots of places, most of
	// which don't make sense to have a minimum or maximum limit
	TSharedRef<IPropertyHandle> RegionPropertyHandle = InLayoutBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationViewport, Region));

	TSharedPtr<IPropertyHandle> XHandle = RegionPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationRectangle, X));
	TSharedPtr<IPropertyHandle> YHandle = RegionPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationRectangle, Y));
	TSharedPtr<IPropertyHandle> WidthHandle = RegionPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationRectangle, W));
	TSharedPtr<IPropertyHandle> HeightHandle = RegionPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationRectangle, H));

	XHandle->SetInstanceMetaData(TEXT("ClampMin"), FString::SanitizeFloat(0.0f));
	XHandle->SetInstanceMetaData(TEXT("UIMin"), FString::SanitizeFloat(0.0f));

	YHandle->SetInstanceMetaData(TEXT("ClampMin"), FString::SanitizeFloat(0.0f));
	YHandle->SetInstanceMetaData(TEXT("UIMin"), FString::SanitizeFloat(0.0f));

	WidthHandle->SetInstanceMetaData(TEXT("ClampMin"), FString::SanitizeFloat(UDisplayClusterConfigurationViewport::ViewportMinimumSize));
	WidthHandle->SetInstanceMetaData(TEXT("UIMin"), FString::SanitizeFloat(UDisplayClusterConfigurationViewport::ViewportMinimumSize));
	WidthHandle->SetInstanceMetaData(TEXT("ClampMax"), FString::SanitizeFloat(UDisplayClusterConfigurationViewport::ViewportMaximumSize));
	WidthHandle->SetInstanceMetaData(TEXT("UIMax"), FString::SanitizeFloat(UDisplayClusterConfigurationViewport::ViewportMaximumSize));

	HeightHandle->SetInstanceMetaData(TEXT("ClampMin"), FString::SanitizeFloat(UDisplayClusterConfigurationViewport::ViewportMinimumSize));
	HeightHandle->SetInstanceMetaData(TEXT("UIMin"), FString::SanitizeFloat(UDisplayClusterConfigurationViewport::ViewportMinimumSize));
	HeightHandle->SetInstanceMetaData(TEXT("ClampMax"), FString::SanitizeFloat(UDisplayClusterConfigurationViewport::ViewportMaximumSize));
	HeightHandle->SetInstanceMetaData(TEXT("UIMax"), FString::SanitizeFloat(UDisplayClusterConfigurationViewport::ViewportMaximumSize));
}

void FDisplayClusterConfiguratorViewportDetailsCustomization::ResetCameraOptions()
{
	CameraOptions.Reset();
	UDisplayClusterConfigurationViewport* ConfigurationViewport = ConfigurationViewportPtr.Get();
	check(ConfigurationViewport != nullptr);

	AActor* RootActor = GetRootActor();

	if (!RootActor)
	{
		return;
	}

	TArray<UActorComponent*> ActorComponents;
	RootActor->GetComponents(UDisplayClusterCameraComponent::StaticClass(), ActorComponents);

	for (UActorComponent* ActorComponent : ActorComponents)
	{
		const FString ComponentName = ActorComponent->GetName();
		CameraOptions.Add(MakeShared<FString>(ComponentName));
	}

	// Component order not guaranteed, sort for consistency.
	CameraOptions.Sort([](const TSharedPtr<FString>& A, const TSharedPtr<FString>& B)
	{
		// Default sort isn't compatible with TSharedPtr<FString>.
		return *A < *B;
	});
}

TSharedRef<SWidget> FDisplayClusterConfiguratorViewportDetailsCustomization::CreateCustomCameraWidget()
{
	if (CameraComboBox.IsValid())
	{
		return CameraComboBox.ToSharedRef();
	}

	return SAssignNew(CameraComboBox, SDisplayClusterConfigurationSearchableComboBox)
		.OptionsSource(&CameraOptions)
		.OnGenerateWidget(this, &FDisplayClusterConfiguratorViewportDetailsCustomization::MakeCameraOptionComboWidget)
		.OnSelectionChanged(this, &FDisplayClusterConfiguratorViewportDetailsCustomization::OnCameraSelected)
		.ContentPadding(2)
		.MaxListHeight(200.0f)
		.Content()
		[
			SNew(STextBlock)
			.Text(this, &FDisplayClusterConfiguratorViewportDetailsCustomization::GetSelectedCameraText)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		];
}

TSharedRef<SWidget> FDisplayClusterConfiguratorViewportDetailsCustomization::MakeCameraOptionComboWidget(TSharedPtr<FString> InItem)
{
	return SNew(STextBlock)
		.Text(FText::FromString(*InItem))
		.Font(IDetailLayoutBuilder::GetDetailFont());
}

void FDisplayClusterConfiguratorViewportDetailsCustomization::OnCameraSelected(TSharedPtr<FString> InCamera, ESelectInfo::Type SelectInfo)
{
	if (InCamera.IsValid())
	{
		UDisplayClusterConfigurationViewport* ConfigurationViewport = ConfigurationViewportPtr.Get();
		check(ConfigurationViewport != nullptr);
		// Handle empty case
		if (InCamera->Equals(*DefaultViewPointComponentNameOption.Get()))
		{
			CameraHandle->SetValue(TEXT(""));
		}
		else
		{
			CameraHandle->SetValue(*InCamera.Get());
		}
		// Reset available options
		ResetCameraOptions();
		CameraComboBox->ResetOptionsSource(&CameraOptions);
		CameraComboBox->SetIsOpen(false);
	}
}

FText FDisplayClusterConfiguratorViewportDetailsCustomization::GetSelectedCameraText() const
{
	FString SelectedOption = ConfigurationViewportPtr.Get()->Camera;
	if (SelectedOption.IsEmpty())
	{
		SelectedOption = *DefaultViewPointComponentNameOption.Get();
	}

	return FText::FromString(SelectedOption);
}

void FDisplayClusterConfiguratorViewportDetailsCustomization::CustomizeEnforcedCameraOrder(IDetailLayoutBuilder& InLayoutBuilder)
{
	// Get the ICVFX property group
	TSharedPtr<IPropertyHandle> ICVFXHandle = InLayoutBuilder.GetProperty(
		GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationViewport, ICVFX));

	if (!ICVFXHandle.IsValid() || !ICVFXHandle->IsValidHandle())
	{
		return;
	}

	// Get the EnforcedCameraOrder array property
	TSharedPtr<IPropertyHandle> EnforcedCameraOrderHandle = ICVFXHandle->GetChildHandle(
		GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_ICVFX, EnforcedCameraOrder));

	if (!EnforcedCameraOrderHandle.IsValid() || !EnforcedCameraOrderHandle->IsValidHandle())
	{
		return;
	}

	ADisplayClusterRootActor* RootActor = GetRootActor();

	if (RootActor == nullptr)
	{
		return;
	}

	// Create a node selection builder configured for ICVFX cameras
	CameraOrderNodeSelection = MakeShared<FDisplayClusterConfiguratorNodeSelection>(
		FDisplayClusterConfiguratorNodeSelection::EOperationMode::ICVFXCameras,
		RootActor,
		nullptr /* FDisplayClusterConfiguratorBlueprintEditor */);

	CameraOrderNodeSelection->IsEnabled(true);

	// Get the property row and hide it since we're replacing with custom builder
	IDetailPropertyRow* PropertyRow = InLayoutBuilder.EditDefaultProperty(EnforcedCameraOrderHandle);
	if (PropertyRow)
	{
		PropertyRow->Visibility(EVisibility::Collapsed);
	}

	// Add the custom array builder to the ICVFX category
	IDetailCategoryBuilder& ICVFXCategory = InLayoutBuilder.EditCategory(
		DisplayClusterConfigurationStrings::categories::InCameraVFXCategory);

	// Create a custom array builder manually since we need to pass IDetailCategoryBuilder
	TSharedRef<FDetailArrayBuilder> ArrayBuilder = MakeShared<FDetailArrayBuilder>(
		EnforcedCameraOrderHandle.ToSharedRef());

	ArrayBuilder->OnGenerateArrayElementWidget(FOnGenerateArrayElementWidget::CreateSP(
		CameraOrderNodeSelection.Get(),
		&FDisplayClusterConfiguratorNodeSelection::GenerateSelectionWidget));

	ICVFXCategory.AddCustomBuilder(ArrayBuilder);
}
