// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Views/Details/DisplayClusterConfiguratorBaseDetailCustomization.h"
#include "Types/SlateEnums.h"

class FDisplayClusterConfiguratorNodeSelection;
class SDisplayClusterConfigurationSearchableComboBox;
class SWidget;
class UDisplayClusterConfigurationData;
class UDisplayClusterConfigurationViewport;

/** Details panel customization for the UDisplayClusterConfigurationViewport object */
class FDisplayClusterConfiguratorViewportDetailsCustomization final : public FDisplayClusterConfiguratorBaseDetailCustomization
{
public:
	using Super = FDisplayClusterConfiguratorBaseDetailCustomization;

	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShared<FDisplayClusterConfiguratorViewportDetailsCustomization>();
	}

	// Begin IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& InLayoutBuilder) override;
	// End IDetailCustomization interface

private:
	/** Rebuilds the list of cameras to show in the dropdown menu of the Camera property widget */
	void ResetCameraOptions();

	/** Creates a combo box widget to replace the default textbox of the Camera property of the UDisplayClusterConfigurationViewport */
	TSharedRef<SWidget> CreateCustomCameraWidget();

	/**
	 * Creates a text block widget to use to display the specified item in the camera dropdown menu
	 * @param InItem - The string item to make the text block for
	 */
	TSharedRef<SWidget> MakeCameraOptionComboWidget(TSharedPtr<FString> InItem);

	/**
	 * Raised when a camera is selected from the camera dropdown menu
	 * @param InCamera - The camera item that was selected
	 * @param SelectionInfo - Flag to indicate through what interface the selection was made
	 */
	void OnCameraSelected(TSharedPtr<FString> InCamera, ESelectInfo::Type SelectInfo);

	/** Gets the text to display for the currently selected camera */
	FText GetSelectedCameraText() const;

	/** Customizes the EnforcedCameraOrder array property with camera name dropdowns */
	void CustomizeEnforcedCameraOrder(IDetailLayoutBuilder& InLayoutBuilder);

private:
	/** The list of camera items to display in the dropdown menu */
	TArray<TSharedPtr<FString>>	CameraOptions;

	/** The property handle for the Camera property of the UDisplayClusterConfigurationViewport object */
	TSharedPtr<IPropertyHandle> CameraHandle;

	/** A cached pointer to the "DefaultViewPoint" option that is added to the list of options in the dropdown menu */
	TSharedPtr<FString> DefaultViewPointComponentNameOption;

	/** A weak reference to the UDisplayClusterConfigurationViewport object being edited by the details panel */
	TWeakObjectPtr<UDisplayClusterConfigurationViewport> ConfigurationViewportPtr;

	/** A weak reference to the configratuion data object that owns the selected UDisplayClusterConfigurationViewport object */
	TWeakObjectPtr<UDisplayClusterConfigurationData> ConfigurationDataPtr;

	/** The combo box that is being displayed in the details panel for the Camera property */
	TSharedPtr<SDisplayClusterConfigurationSearchableComboBox> CameraComboBox;

	/** The helper object used to generate the custom array widget for the EnforcedCameraOrder property */
	TSharedPtr<FDisplayClusterConfiguratorNodeSelection> CameraOrderNodeSelection;
};
