// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfiguratorViewportRemapCustomization.h"

#include "DisplayClusterConfigurationTypes_ViewportRemap.h"

#include "DetailLayoutBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "DetailWidgetRow.h"
#include "Widgets/Input/SComboBox.h"

const FName FDisplayClusterConfiguratorViewportRemapCustomization::AngleIntervalMetadataKey = TEXT("AngleInterval");
const FName FDisplayClusterConfiguratorViewportRemapCustomization::SimplifiedMetadataTag = TEXT("Simplified");

void FDisplayClusterConfiguratorViewportRemapCustomization::Initialize(const TSharedRef<IPropertyHandle>& InPropertyHandle, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	FDisplayClusterConfiguratorBaseTypeCustomization::Initialize(InPropertyHandle, CustomizationUtils);

	AngleInterval = 0;
	if (const FString* AngleIntervalMetadata = FindMetaData(InPropertyHandle, AngleIntervalMetadataKey))
	{
		AngleInterval = FMath::Clamp(FCString::Atof(*(*AngleIntervalMetadata)), 0.0f, 90.0f);
	}

	bSimplifiedDisplay = false;
	if (FindMetaData(InPropertyHandle, SimplifiedMetadataTag))
	{
		bSimplifiedDisplay = true;
	}

	AnglePropertyHandle = GET_CHILD_HANDLE(FDisplayClusterConfigurationViewport_RemapData, Angle);

	GenerateAnglesList();
}

void FDisplayClusterConfiguratorViewportRemapCustomization::SetChildren(const TSharedRef<IPropertyHandle>& InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	if (!bSimplifiedDisplay)
	{
		InChildBuilder.AddProperty(GET_CHILD_HANDLE(FDisplayClusterConfigurationViewport_RemapData, ViewportRegion).ToSharedRef());
		InChildBuilder.AddProperty(GET_CHILD_HANDLE(FDisplayClusterConfigurationViewport_RemapData, OutputRegion).ToSharedRef());
	}

	IDetailPropertyRow& AnglePropertyRow = InChildBuilder.AddProperty(AnglePropertyHandle.ToSharedRef());
	if (AngleInterval > 0.0f)
	{
		AnglePropertyRow.CustomWidget()
			.NameContent()
			[
				AnglePropertyHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			[
				SAssignNew(AnglesComboBox, SComboBox<TSharedPtr<float>>)
					.OptionsSource(&AnglesList)
					.OnSelectionChanged(this, &FDisplayClusterConfiguratorViewportRemapCustomization::SelectionChanged)
					.OnGenerateWidget(this, &FDisplayClusterConfiguratorViewportRemapCustomization::GenerateWidget)
					.Content()
					[
						SNew(STextBlock)
						.Text(this, &FDisplayClusterConfiguratorViewportRemapCustomization::GetSelectedAngleText)
						.Font(IDetailLayoutBuilder::GetDetailFont())
					]
			];
	}

	InChildBuilder.AddProperty(GET_CHILD_HANDLE(FDisplayClusterConfigurationViewport_RemapData, bFlipH).ToSharedRef());
	InChildBuilder.AddProperty(GET_CHILD_HANDLE(FDisplayClusterConfigurationViewport_RemapData, bFlipV).ToSharedRef());
}

void FDisplayClusterConfiguratorViewportRemapCustomization::GenerateAnglesList()
{
	AnglesList.Empty();

	float AnglePropertyValue = 0;
	AnglePropertyHandle->GetValue(AnglePropertyValue);

	if (AngleInterval > 0)
	{
		float CurrentAngle = 0;
		while (CurrentAngle < 360)
		{
			TSharedPtr<float> Angle = MakeShared<float>(CurrentAngle);
			AnglesList.Add(Angle);

			CurrentAngle += AngleInterval;
		}
	}
}

void FDisplayClusterConfiguratorViewportRemapCustomization::SelectionChanged(TSharedPtr<float> InValue, ESelectInfo::Type SelectInfo)
{
	if (InValue.IsValid())
	{
		AnglePropertyHandle->SetValue(*InValue);
	}
}

TSharedRef<SWidget> FDisplayClusterConfiguratorViewportRemapCustomization::GenerateWidget(TSharedPtr<float> InItem)
{
	FString AngleString = FString::SanitizeFloat(*InItem);
	return SNew(STextBlock)
		.Text(FText::FromString(*AngleString))
		.Font(IDetailLayoutBuilder::GetDetailFont());
}

FText FDisplayClusterConfiguratorViewportRemapCustomization::GetSelectedAngleText() const
{
	float CurrentAngle;
	AnglePropertyHandle->GetValue(CurrentAngle);
	return FText::FromString(FString::SanitizeFloat(CurrentAngle));
}