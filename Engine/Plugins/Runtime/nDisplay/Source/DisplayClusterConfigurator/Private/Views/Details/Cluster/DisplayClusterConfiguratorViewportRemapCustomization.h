// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Views/Details/DisplayClusterConfiguratorBaseTypeCustomization.h"

template<class T>
class SComboBox;
class SWidget;

/** Type customization for the FDisplayClusterConfigurationViewport_RemapData struct */
class FDisplayClusterConfiguratorViewportRemapCustomization final : public FDisplayClusterConfiguratorBaseTypeCustomization
{
public:
	/**
	 * Custom "AngleInterval" metadata specifier that can be used to specify if the Angle property of the 
	 * FDisplayClusterConfigurationViewport_RemapData struct is limited to fixed intervals
	 */
	static const FName AngleIntervalMetadataKey;

	/**
	 * Custom "Simplified" metadata specifier, which hides some of the advanced child properties of a
	 * FDisplayClusterConfigurationViewport_RemapData property
	 */
	static const FName SimplifiedMetadataTag;

	/** Creates a new instance of the customization */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FDisplayClusterConfiguratorViewportRemapCustomization>();
	}

protected:
	//~ FDisplayClusterConfiguratorBaseTypeCustomization overrides begin
	virtual void Initialize(const TSharedRef<IPropertyHandle>& InPropertyHandle, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void SetChildren(const TSharedRef<IPropertyHandle>& InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	//~ FDisplayClusterConfiguratorBaseTypeCustomization overrides end

private:
	/** Generates the list of angles to display in the angle dropdown list, based on the angle interval metadata */
	void GenerateAnglesList();

	/** Raised when an item within the dropdown menu is selected */
	void SelectionChanged(TSharedPtr<float> InValue, ESelectInfo::Type SelectInfo);

	/** Constructs the text widget to display in the dropdown menu for the specified item */
	TSharedRef<SWidget> GenerateWidget(TSharedPtr<float> InItem);

	/** Gets the display text for the currently selected item of the element in the array widget */
	FText GetSelectedAngleText() const;

private:
	/** The property handle for the Angle property */
	TSharedPtr<IPropertyHandle> AnglePropertyHandle;

	/** The combo box widget to display for the angle intervals */
	TSharedPtr<SComboBox<TSharedPtr<float>>> AnglesComboBox;

	/** The list of angles the user is allowed to select from */
	TArray<TSharedPtr<float>> AnglesList;

	/** The interval to limit the allowed angle to */
	float AngleInterval;

	/** Indicates that the cusomization should hide the ViewportRegion and OutputRegion properties */
	bool bSimplifiedDisplay;
};