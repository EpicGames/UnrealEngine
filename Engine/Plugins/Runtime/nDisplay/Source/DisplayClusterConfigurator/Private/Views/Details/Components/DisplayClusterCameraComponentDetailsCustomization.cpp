// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterCameraComponentDetailsCustomization.h"

#include "Components/DisplayClusterCameraComponent.h"
#include "DisplayClusterRootActor.h"
#include "Camera/CameraComponent.h"
#include "Components/DisplayClusterICVFXCameraComponent.h"

#include "DisplayClusterProjectionStrings.h"
#include "DisplayClusterConfiguratorLog.h"

#include "Render/DisplayDevice/Components/DisplayClusterDisplayDeviceBaseComponent.h"
#include "Views/Details/Widgets/SDisplayClusterConfiguratorComponentPicker.h"
#include "Views/Details/Widgets/SDisplayClusterConfigurationSearchableComboBox.h"

#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "DetailWidgetRow.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "DisplayClusterCameraComponentDetailsCustomization"

void FDisplayClusterCameraComponentDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& InLayoutBuilder)
{
	Super::CustomizeDetails(InLayoutBuilder);

	DetailLayout = &InLayoutBuilder;

	if (!EditedObject.IsValid())
	{
		TArray<TWeakObjectPtr<UObject>> Objects;
		InLayoutBuilder.GetObjectsBeingCustomized(Objects);

		for (TWeakObjectPtr<UObject> Object : Objects)
		{
			if (Object->IsA<UDisplayClusterCameraComponent>())
			{
				EditedObject = Cast<UDisplayClusterCameraComponent>(Object.Get());
			}
		}
	}

	if (EditedObject.IsValid())
	{
		NoneOption = MakeShared<FString>("None");

		CameraHandle = InLayoutBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UDisplayClusterCameraComponent, ICVFXCameraComponentName));
		check(CameraHandle->IsValidHandle());

		RebuildCameraOptions();

		if (IDetailPropertyRow* CameraPropertyRow = InLayoutBuilder.EditDefaultProperty(CameraHandle))
		{
			CameraPropertyRow->CustomWidget()
				.NameContent()
				[
					CameraHandle->CreatePropertyNameWidget()
				]
				.ValueContent()
				[
					CreateCustomCameraWidget()
				];
		}
	}
}

UClass* FDisplayClusterCameraComponentDetailsCustomization::GetCameraComponentClass() const
{
	return UDisplayClusterICVFXCameraComponent::StaticClass();
}

void FDisplayClusterCameraComponentDetailsCustomization::RebuildCameraOptions()
{
	CameraOptions.Reset();

	// The EditedObject may become invalid when component destoryed (DCRA rebuild).
	if (!EditedObject.IsValid())
	{
		// Just do nothing
		return;
	}

	const UDisplayClusterCameraComponent* DestCameraComponent = EditedObject.Get();
	check(DestCameraComponent != nullptr);

	const AActor* RootActor = GetRootActor();
	if (!IsValid(RootActor))
	{
		return;
	}

	TArray<UActorComponent*> ActorComponents;
	RootActor->GetComponents(GetCameraComponentClass(), ActorComponents);
	for (UActorComponent* ActorComponent : ActorComponents)
	{
		const FString ComponentName = ActorComponent->GetName();
		CameraOptions.Add(MakeShared<FString>(ComponentName));
	}

	// Component order not guaranteed, sort for consistency.
	CameraOptions.Sort([](const TSharedPtr<FString>& A, const TSharedPtr<FString>& B)
	{
		// Default sort isn't compatible with TSharedPtr<FString>.
		return *A < *B;
	});

	// Add None option
	if (!DestCameraComponent->ICVFXCameraComponentName.IsEmpty())
	{
		CameraOptions.Add(NoneOption);
	}
}

TSharedRef<SWidget> FDisplayClusterCameraComponentDetailsCustomization::CreateCustomCameraWidget()
{
	if (CameraComboBoxWidged.IsValid())
	{
		return CameraComboBoxWidged.ToSharedRef();
	}

	CameraComboBoxWidged = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Fill)
		.FillWidth(VAlign_Fill)
		.FillWidth(1.f)
		[
			SAssignNew(CameraComboBox, SDisplayClusterConfigurationSearchableComboBox)
				.OptionsSource(&CameraOptions)
				.OnGenerateWidget(this, &FDisplayClusterCameraComponentDetailsCustomization::MakeCameraOptionComboWidget)
				.OnSelectionChanged(this, &FDisplayClusterCameraComponentDetailsCustomization::OnCameraSelected)
				.ContentPadding(2)
				.MaxListHeight(200.0f)
				.Content()
				[
					SNew(STextBlock)
						.Text(this, &FDisplayClusterCameraComponentDetailsCustomization::GetSelectedCameraText)
						.Font(IDetailLayoutBuilder::GetDetailFont())
				]
		]
		/* Add browse to component button */
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Fill)
		.FillWidth(VAlign_Fill)
		.AutoWidth()
		[
			PropertyCustomizationHelpers::MakeBrowseButton(
				FSimpleDelegate::CreateSP(this, &FDisplayClusterCameraComponentDetailsCustomization::OnSelectComponentButton)
				, TAttribute<FText>(this, &FDisplayClusterCameraComponentDetailsCustomization::GetSelectComponentButtonTooltipText)
				, /** IsEnabled */true
				, /** IsActor */true)
		];

	return CameraComboBoxWidged.ToSharedRef();
}


FText FDisplayClusterCameraComponentDetailsCustomization::GetSelectComponentButtonTooltipText() const
{
	const FText CameraName = GetSelectedCameraText();
	if (!CameraName.ToString().Equals(*NoneOption.Get()))
	{
		return FText::Format(LOCTEXT("SelectCameraComponent", "Select '{0}' camera component in the Root Actor"), CameraName);
	}

	return FText::GetEmpty();
}

void FDisplayClusterCameraComponentDetailsCustomization::OnSelectComponentButton() const
{
	if (!GEditor)
	{
		return;
	}

	const FText CameraName = GetSelectedCameraText();
	if (!CameraName.ToString().Equals(*NoneOption.Get()))
	{
		// The component in the DCRA to be selected.
		UActorComponent* RootActorComponent = nullptr;

		const AActor* RootActor = GetRootActor();
		if (!IsValid(RootActor))
		{
			return;
		}

		TArray<UActorComponent*> ActorComponents;
		RootActor->GetComponents(GetCameraComponentClass(), ActorComponents);
		for (UActorComponent* ActorComponent : ActorComponents)
		{
			const FString ComponentName = ActorComponent->GetName();
			if (ComponentName == CameraName.ToString())
			{
				RootActorComponent = ActorComponent;
				break;
			}
		}

		if (IsValid(RootActorComponent))
		{
			const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "ClickingOnElements", "Clicking on Elements"));

			constexpr bool bNotifySelectionChanged = true;
			GEditor->SelectNone(!bNotifySelectionChanged, /** DeselectBSP */true);
			GEditor->SelectComponent(RootActorComponent, /** IsSelected */true, bNotifySelectionChanged);
		}
	}
}

TSharedRef<SWidget> FDisplayClusterCameraComponentDetailsCustomization::MakeCameraOptionComboWidget(TSharedPtr<FString> InItem)
{
	return SNew(STextBlock)
		.Text(FText::FromString(*InItem))
		.Font(IDetailLayoutBuilder::GetDetailFont());
}

void FDisplayClusterCameraComponentDetailsCustomization::OnCameraSelected(TSharedPtr<FString> InCamera, ESelectInfo::Type SelectInfo)
{
	if (InCamera.IsValid())
	{
		// Handle empty case
		if (InCamera->Equals(*NoneOption.Get()))
		{
			CameraHandle->SetValue(TEXT(""));
		}
		else
		{
			CameraHandle->SetValue(*InCamera.Get());
		}

		// Reset available options
		RebuildCameraOptions();
		CameraComboBox->ResetOptionsSource(&CameraOptions);
		CameraComboBox->SetIsOpen(false);
	}
}

FText FDisplayClusterCameraComponentDetailsCustomization::GetSelectedCameraText() const
{
	// The EditedObject may become invalid when component destoryed (DCRA rebuild).
	if (!EditedObject.IsValid())
	{
		// Just do nothing
		return FText::FromString(*NoneOption.Get());
	}

	const UDisplayClusterCameraComponent* DestCameraComponent = EditedObject.Get();
	check(DestCameraComponent != nullptr);

	FString SelectedOption = DestCameraComponent->ICVFXCameraComponentName;
	if (SelectedOption.IsEmpty())
	{
		SelectedOption = *NoneOption.Get();
	}

	return FText::FromString(SelectedOption);
}

#undef LOCTEXT_NAMESPACE
