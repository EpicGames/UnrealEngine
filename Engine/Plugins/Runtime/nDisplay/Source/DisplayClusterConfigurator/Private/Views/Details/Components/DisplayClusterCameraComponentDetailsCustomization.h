// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Views/Details/DisplayClusterConfiguratorBaseDetailCustomization.h"
#include "Types/SlateEnums.h"

class UDisplayClusterCameraComponent;
class SWidget;
class SDisplayClusterConfigurationSearchableComboBox;

/** Details panel customization for the UDisplayClusterCameraComponent object */
class FDisplayClusterCameraComponentDetailsCustomization final : public FDisplayClusterConfiguratorBaseDetailCustomization
{
public:
	using Super = FDisplayClusterConfiguratorBaseDetailCustomization;

	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShared<FDisplayClusterCameraComponentDetailsCustomization>();
	}

public:
	// Begin IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	// End IDetailCustomization interface

private:
	/** Rebuilds the list of cameras to show in the dropdown menu of the Camera property widget */
	void RebuildCameraOptions();

	/** Creates a combo box widget to replace the default textbox of the Camera property of the UDisplayClusterCameraComponent */
	TSharedRef<SWidget> CreateCustomCameraWidget();

	/**
	 * Creates a text block widget to use to display the specified item in the camera dropdown menu
	 * @param InItem - The string item to make the text block for
	 */
	TSharedRef<SWidget> MakeCameraOptionComboWidget(TSharedPtr<FString> InItem);

	/**
	 * Raised when a camera is selected from the camera dropdown menu
	 * @param InCamera - The camera item that was selected
	 * @param SelectionInfo - Flag to indicate through what interface the selection was made
	 */
	void OnCameraSelected(TSharedPtr<FString> InCamera, ESelectInfo::Type SelectInfo);

	/** Gets the text to display for the currently selected camera */
	FText GetSelectedCameraText() const;

	/** When select component button is pressed. */
	void OnSelectComponentButton() const;

	/** Return select component button tooltip text. */
	FText GetSelectComponentButtonTooltipText() const;

	/** Return camera component class. */
	UClass* GetCameraComponentClass() const;


private:
	/** A weak reference to the UDisplayClusterCameraComponent object being edited by the details panel */
	TWeakObjectPtr<UDisplayClusterCameraComponent> EditedObject;

	/** Reference to the detail layout builder, used to force refresh the layout */
	IDetailLayoutBuilder* DetailLayout = nullptr;

private:
	/** The list of camera items to display in the dropdown menu */
	TArray<TSharedPtr<FString>> CameraOptions;

	/** The property handle for the Camera property of the UDisplayClusterCameraComponent object */
	TSharedPtr<IPropertyHandle> CameraHandle;

	/** A cached pointer to the "None" option that is added to the list of options in the dropdown menu */
	TSharedPtr<FString> NoneOption;

	/** The combo box that is being displayed in the details panel for the Camera property */
	TSharedPtr<SDisplayClusterConfigurationSearchableComboBox> CameraComboBox;

	/** The widged that is being displayed in the details panel for the Camera property */
	TSharedPtr<SWidget> CameraComboBoxWidged;
};
