// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfiguratorScreenComponentDetailsCustomization.h"

#include "Components/DisplayClusterScreenComponent.h"

#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "DetailWidgetRow.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "DisplayClusterConfiguratorDetailCustomization"

const TArray<FDisplayClusterConfiguratorAspectRatioPresetSize> FDisplayClusterConfiguratorAspectRatioPresetSize::CommonPresets =
{
	FDisplayClusterConfiguratorAspectRatioPresetSize(LOCTEXT("3x2", "3:2"), FVector2D(100.f, 66.67f)),
	FDisplayClusterConfiguratorAspectRatioPresetSize(LOCTEXT("4x3", "4:3"), FVector2D(100.f, 75.f)),
	FDisplayClusterConfiguratorAspectRatioPresetSize(LOCTEXT("16x9", "16:9"), FVector2D(100.f, 56.25f)),
	FDisplayClusterConfiguratorAspectRatioPresetSize(LOCTEXT("16x10", "16:10"), FVector2D(100.f, 62.5f)),
	FDisplayClusterConfiguratorAspectRatioPresetSize(LOCTEXT("1.90", "1.90"), FVector2D(100.f, 52.73f))
};

const int32 FDisplayClusterConfiguratorAspectRatioPresetSize::DefaultPreset = 2;

void FDisplayClusterConfiguratorScreenDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& InLayoutBuilder)
{
	// Get the Editing object
	const TArray<TWeakObjectPtr<UObject>>& SelectedObjects = InLayoutBuilder.GetSelectedObjects();
	if (SelectedObjects.Num())
	{
		ScreenComponentPtr = Cast<UDisplayClusterScreenComponent>(SelectedObjects[0]);
	}
	check(ScreenComponentPtr != nullptr);

	if (!ScreenComponentPtr->IsTemplate())
	{
		// Don't allow size property and aspect ratio changes on instances for now.
		return;
	}

	for (const FDisplayClusterConfiguratorAspectRatioPresetSize& PresetItem : FDisplayClusterConfiguratorAspectRatioPresetSize::CommonPresets)
	{
		TSharedPtr<FDisplayClusterConfiguratorAspectRatioPresetSize> PresetItemPtr = MakeShared<FDisplayClusterConfiguratorAspectRatioPresetSize>(PresetItem);
		PresetItems.Add(PresetItemPtr);
	}

	check(FDisplayClusterConfiguratorAspectRatioPresetSize::DefaultPreset >= 0 && FDisplayClusterConfiguratorAspectRatioPresetSize::DefaultPreset < PresetItems.Num());
	const TSharedPtr<FDisplayClusterConfiguratorAspectRatioPresetSize> InitiallySelectedPresetItem = PresetItems[FDisplayClusterConfiguratorAspectRatioPresetSize::DefaultPreset];
	// Make sure default value is set for current preset.
	GetAspectRatioAndSetDefaultValueForPreset(*InitiallySelectedPresetItem.Get());

	const FText RowName = LOCTEXT("DisplayClusterConfiguratorResolution", "Aspect Ratio Preset");

	SizeHandlePtr = InLayoutBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UDisplayClusterScreenComponent, Size));
	SizeHandlePtr->SetOnChildPropertyValueChanged(FSimpleDelegate::CreateRaw(this, &FDisplayClusterConfiguratorScreenDetailsCustomization::OnSizePropertyChanged));
	SizeHandlePtr->SetOnPropertyResetToDefault(FSimpleDelegate::CreateRaw(this, &FDisplayClusterConfiguratorScreenDetailsCustomization::OnSizePropertyChanged));

	// This will detect custom ratios.
	OnSizePropertyChanged();

	IDetailCategoryBuilder& ScreenSizeCategory = InLayoutBuilder.EditCategory(TEXT("Screen Size"));

	ScreenSizeCategory.AddCustomRow(RowName)
		.NameWidget
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(RowName)
		]
		.ValueWidget
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.HAlign(HAlign_Fill)
			[
				SAssignNew(PresetsComboBox, SComboBox<TSharedPtr<FDisplayClusterConfiguratorAspectRatioPresetSize>>)
				.OptionsSource(&PresetItems)
				.InitiallySelectedItem(InitiallySelectedPresetItem)
				.OnSelectionChanged(this, &FDisplayClusterConfiguratorScreenDetailsCustomization::OnSelectedPresetChanged)
				.OnGenerateWidget_Lambda([this](TSharedPtr<FDisplayClusterConfiguratorAspectRatioPresetSize> Item)
				{
					return SNew(STextBlock)
						.Font(IDetailLayoutBuilder::GetDetailFont())
						.Text(GetPresetDisplayText(Item));
				})
				.Content()
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(this, &FDisplayClusterConfiguratorScreenDetailsCustomization::GetPresetsComboBoxSelectedText)
				]
			]
		];

		ScreenSizeCategory.AddProperty(SizeHandlePtr);
}

FText FDisplayClusterConfiguratorScreenDetailsCustomization::GetPresetsComboBoxSelectedText() const
{
	return GetSelectedPresetDisplayText(PresetsComboBox->GetSelectedItem());
}

FText FDisplayClusterConfiguratorScreenDetailsCustomization::GetPresetDisplayText(const TSharedPtr<FDisplayClusterConfiguratorAspectRatioPresetSize>& Preset) const
{
	FText DisplayText = FText::GetEmpty();

	if (Preset.IsValid())
	{
		DisplayText = FText::Format(LOCTEXT("PresetDisplayText", "{0}"), Preset->DisplayName);
	}

	return DisplayText;
}

FText FDisplayClusterConfiguratorScreenDetailsCustomization::GetSelectedPresetDisplayText(const TSharedPtr<FDisplayClusterConfiguratorAspectRatioPresetSize>& Preset) const
{
	FText DisplayText = FText::GetEmpty();

	if (bIsCustomAspectRatio)
	{
		DisplayText = LOCTEXT("PresetDisplayCustomText", "Custom");
	}
	else if (Preset.IsValid())
	{
		DisplayText = FText::Format(LOCTEXT("PresetDisplayText", "{0}"), Preset->DisplayName);
	}

	return DisplayText;
}

void FDisplayClusterConfiguratorScreenDetailsCustomization::OnSelectedPresetChanged(TSharedPtr<FDisplayClusterConfiguratorAspectRatioPresetSize> SelectedPreset, ESelectInfo::Type SelectionType)
{
	if (SelectionType != ESelectInfo::Type::Direct && SelectedPreset.IsValid() && SizeHandlePtr.IsValid())
	{
		FVector2D NewValue;
		GetAspectRatioAndSetDefaultValueForPreset(*SelectedPreset.Get(), &NewValue);

		// Compute size based on new aspect ratio and old value.
		{
			FVector2D OldSize;
			SizeHandlePtr->GetValue(OldSize);
			NewValue.X = OldSize.X;
			NewValue.Y = (double)NewValue.X / SelectedPreset->GetAspectRatio();
		}

		SizeHandlePtr->SetValue(NewValue);
	}
}

void FDisplayClusterConfiguratorScreenDetailsCustomization::GetAspectRatioAndSetDefaultValueForPreset(const FDisplayClusterConfiguratorAspectRatioPresetSize& Preset, FVector2D* OutAspectRatio)
{
	if (UDisplayClusterScreenComponent* Archetype = Cast<UDisplayClusterScreenComponent>(ScreenComponentPtr->GetArchetype()))
	{
		// Set the DEFAULT value here, that way user can always reset to default for the current preset.
		if (Archetype->GetScreenSize() != Preset.Size)
		{
			Archetype->SetScreenSize(Preset.Size);
		}
	}

	if (OutAspectRatio)
	{
		*OutAspectRatio = Preset.Size;
	}
}

void FDisplayClusterConfiguratorScreenDetailsCustomization::OnSizePropertyChanged()
{
	if (SizeHandlePtr.IsValid())
	{
		FVector2D SizeValue;
		SizeHandlePtr->GetValue(SizeValue);

		const double AspectRatio = (double)SizeValue.X / (double)SizeValue.Y;

		const FDisplayClusterConfiguratorAspectRatioPresetSize* FoundPreset = nullptr;
		for (const FDisplayClusterConfiguratorAspectRatioPresetSize& Preset : FDisplayClusterConfiguratorAspectRatioPresetSize::CommonPresets)
		{
			const double PresetAspectRatio = Preset.GetAspectRatio();
			if (FMath::IsNearlyEqual(AspectRatio, PresetAspectRatio, 0.001))
			{
				FoundPreset = &Preset;
				break;
			}
		}

		bIsCustomAspectRatio = FoundPreset == nullptr;
	}
}

#undef LOCTEXT_NAMESPACE