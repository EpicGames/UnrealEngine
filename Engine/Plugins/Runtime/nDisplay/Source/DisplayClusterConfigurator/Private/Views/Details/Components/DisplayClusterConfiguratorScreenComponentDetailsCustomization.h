// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "Types/SlateEnums.h"

class IPropertyHandle;
class UDisplayClusterScreenComponent;

template<typename OptionType>
class SComboBox;

/** Stores a size preset for the screen that cab be displayed and selected by the user. */
struct FDisplayClusterConfiguratorAspectRatioPresetSize
{
public:
	FDisplayClusterConfiguratorAspectRatioPresetSize() :
		DisplayName(FText::GetEmpty()),
		Size(FVector2D::ZeroVector)
	{ }

	FDisplayClusterConfiguratorAspectRatioPresetSize(FText InDisplayName, FVector2D InSize) :
		DisplayName(InDisplayName),
		Size(InSize)
	{ }

	bool operator==(const FDisplayClusterConfiguratorAspectRatioPresetSize& Other) const
	{
		return (DisplayName.EqualTo(Other.DisplayName)) && (Size == Other.Size);
	}

	/** Calculates the aspect ratio of the preset based on its width and height */
	double GetAspectRatio() const { return (double)Size.X / (double)Size.Y; }

public:
	/** A list of common presets of screen sizes */
	static const TArray<FDisplayClusterConfiguratorAspectRatioPresetSize> CommonPresets;

	/** The index of the preset to default to in the CommonPresets list */
	static const int32 DefaultPreset;

	/** The display name of the preset */
	FText DisplayName;

	/** The width and height of the preset */
	FVector2D Size;
};

/** Details panel customization for the UDisplayClusterConfigurationViewport object */
class FDisplayClusterConfiguratorScreenDetailsCustomization final : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShared<FDisplayClusterConfiguratorScreenDetailsCustomization>();
	}

public:
	// Begin IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	// End IDetailCustomization interface

protected:
	/** Gets the text to display for the currently selected size preset of the screen */
	FText GetPresetsComboBoxSelectedText() const;

	/**
	 * Gets the display text of the specified size preset
	 * @param Preset - The preset to get the display text for
	 */
	FText GetPresetDisplayText(const TSharedPtr<FDisplayClusterConfiguratorAspectRatioPresetSize>& Preset) const;

	/**
	 * Gets the text to display for the specified preset when it is the currently selected preset
	 * @param Preset - The preset to get the display text for when it is the selected preset
	 */
	FText GetSelectedPresetDisplayText(const TSharedPtr<FDisplayClusterConfiguratorAspectRatioPresetSize>& Preset) const;

	/**
	 * Raised when the specified preset is selected from the dropdown menu
	 * @param SelectedPreset - The preset that was selected
	 * @param SelectionInfo - Flag to indicate through what interface the selection was made
	 */
	void OnSelectedPresetChanged(TSharedPtr<FDisplayClusterConfiguratorAspectRatioPresetSize> SelectedPreset, ESelectInfo::Type SelectionType);

	/**
	 * Sets the value to default to on the selected object's archetype, and returns the default value's aspect ratio in the form of width and height.
	 * @param Preset - The preset to make the default value on the selected object's archetype
	 * @param OutAspectRatio - The aspect ratio, in the form of width and height, of the preset
	 */
	void GetAspectRatioAndSetDefaultValueForPreset(const FDisplayClusterConfiguratorAspectRatioPresetSize& Preset, FVector2D* OutAspectRatio = nullptr);

	/**
	 * Raised when the Size property of the selected screen component is changed
	 */
	void OnSizePropertyChanged();

private:
	/** A pointer to the current screen component being edited by the details panel. */
	UDisplayClusterScreenComponent* ScreenComponentPtr = nullptr;

	/** A list of presets to display in the size preset combo box */
	TArray<TSharedPtr<FDisplayClusterConfiguratorAspectRatioPresetSize>> PresetItems;

	/** The size presets combo box widget to display in the screen component's details panel */
	TSharedPtr<SComboBox<TSharedPtr<FDisplayClusterConfiguratorAspectRatioPresetSize>>> PresetsComboBox;

	/** The property handle for the Size property of the selected screen component */
	TSharedPtr<IPropertyHandle> SizeHandlePtr;

	/** Indicates whether the user has entered a custom size into the Size property of the selected screen component */
	bool bIsCustomAspectRatio = false;
};