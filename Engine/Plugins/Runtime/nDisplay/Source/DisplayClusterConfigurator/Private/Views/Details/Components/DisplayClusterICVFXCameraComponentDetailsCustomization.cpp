// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterICVFXCameraComponentDetailsCustomization.h"

#include "DisplayClusterConfigurationStrings.h"
#include "Components/DisplayClusterICVFXCameraComponent.h"

#include "CineCameraActor.h"
#include "CineCameraComponent.h"
#include "DisplayClusterRootActor.h"
#include "UObject/SoftObjectPtr.h"

#include "ColorGradingEditorUtil.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "IDetailGroup.h"
#include "DetailWidgetRow.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "DisplayClusterICVFXCameraComponentDetailsCustomization"

namespace DisplayClusterICVFXCameraComponentDetailsCustomizationUtils
{
	void SortCategories(const TMap<FName, IDetailCategoryBuilder*>& AllCategoryMap)
	{
		static const TArray<FName> CategoryOrder =
		{
			TEXT("Variable"),
			TEXT("TransformCommon"),
			DisplayClusterConfigurationStrings::categories::InnerFrustumCategory,
			DisplayClusterConfigurationStrings::categories::ICVFXCameraCategory,
			DisplayClusterConfigurationStrings::categories::CameraColorGradingCategory,
			DisplayClusterConfigurationStrings::categories::OCIOCategory,
			DisplayClusterConfigurationStrings::categories::MediaCategory,
			DisplayClusterConfigurationStrings::categories::ChromaKeyCategory,
			DisplayClusterConfigurationStrings::categories::OverrideCategory,
			DisplayClusterConfigurationStrings::categories::ConfigurationCategory
		};

		for (const TPair<FName, IDetailCategoryBuilder*>& Pair : AllCategoryMap)
		{
			int32 CurrentSortOrder = Pair.Value->GetSortOrder();

			int32 DesiredSortOrder = 0;
			if (CategoryOrder.Find(Pair.Key, DesiredSortOrder))
			{
				CurrentSortOrder = DesiredSortOrder;
			}
			else
			{
				CurrentSortOrder += CategoryOrder.Num();
			}

			Pair.Value->SetSortOrder(CurrentSortOrder);
		}
	}
}

TSharedRef<IDetailCustomization> FDisplayClusterICVFXCameraComponentDetailsCustomization::MakeInstance()
{
	return MakeShared<FDisplayClusterICVFXCameraComponentDetailsCustomization>();
}

void FDisplayClusterICVFXCameraComponentDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& InLayoutBuilder)
{
	DetailLayout = &InLayoutBuilder;

	if (!EditedObject.IsValid())
	{
		TArray<TWeakObjectPtr<UObject>> Objects;
		InLayoutBuilder.GetObjectsBeingCustomized(Objects);

		for (TWeakObjectPtr<UObject> Object : Objects)
		{
			if (Object->IsA<UDisplayClusterICVFXCameraComponent>())
			{
				EditedObject = Cast<UDisplayClusterICVFXCameraComponent>(Object.Get());
			}
		}
	}

	// Hide some groups if an external CineCameraActor is set
	if (EditedObject.IsValid() && EditedObject->CameraSettings.ExternalCameraActor.IsValid())
	{
		InLayoutBuilder.HideCategory(TEXT("TransformCommon"));
		InLayoutBuilder.HideCategory(TEXT("Current Camera Settings"));
		InLayoutBuilder.HideCategory(TEXT("CameraOptions"));
		InLayoutBuilder.HideCategory(TEXT("Camera"));
		InLayoutBuilder.HideCategory(TEXT("PostProcess"));
		InLayoutBuilder.HideCategory(TEXT("Lens"));
		InLayoutBuilder.HideCategory(TEXT("LOD"));
		InLayoutBuilder.HideCategory(TEXT("ColorGrading"));
		InLayoutBuilder.HideCategory(TEXT("RenderingFeatures"));
		InLayoutBuilder.HideCategory(TEXT("Color Grading"));
		InLayoutBuilder.HideCategory(TEXT("Rendering Features"));
	}

	// Sockets category must be hidden manually instead of through the HideCategories metadata specifier
	InLayoutBuilder.HideCategory(TEXT("Sockets"));

	// Rename "Inner Frustum Color Grading" to "Color Grading" for brevity, as the category itself needs to remain distinct from the camera's "Color Grading" category.
	IDetailCategoryBuilder& ColorGradingCategory = InLayoutBuilder.EditCategory(DisplayClusterConfigurationStrings::categories::CameraColorGradingCategory, LOCTEXT("ICVFXColorGradingCategoryLabel", "Color Grading"));

	// Add the Color Grading button at the top of the new category
	ColorGradingCategory.AddCustomRow(NSLOCTEXT("ColorCorrectWindowDetails", "OpenColorGrading", "Open Color Grading"))
		.RowTag("OpenColorGrading")
		[
			ColorGradingEditorUtil::MakeColorGradingLaunchButton()
		];

	IDetailCategoryBuilder& CameraCategory = InLayoutBuilder.EditCategory(DisplayClusterConfigurationStrings::categories::ICVFXCameraCategory, LOCTEXT("ICVFXCameraCategoryLabel", "Camera"));

	// Re-add the external camera to the category to ensure it is always above the camera's fiz properties in the details panel
	CameraCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UDisplayClusterICVFXCameraComponent, ExternalCameraActorRef));

	if (EditedObject.IsValid() && EditedObject->CameraSettings.ExternalCameraActor.IsValid())
	{
		TArray<UObject*> ExternalCameraComponents = { EditedObject->CameraSettings.ExternalCameraActor.Get()->GetCineCameraComponent() };
		CameraCategory.AddExternalObjectProperty(ExternalCameraComponents, GET_MEMBER_NAME_CHECKED(UCineCameraComponent, FocusSettings));
		CameraCategory.AddExternalObjectProperty(ExternalCameraComponents, GET_MEMBER_NAME_CHECKED(UCineCameraComponent, CurrentFocalLength));
		CameraCategory.AddExternalObjectProperty(ExternalCameraComponents, GET_MEMBER_NAME_CHECKED(UCineCameraComponent, CurrentAperture));
	}
	else
	{
		CameraCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UDisplayClusterICVFXCameraComponent, FocusSettings), UCineCameraComponent::StaticClass());
		CameraCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UDisplayClusterICVFXCameraComponent, CurrentFocalLength), UCineCameraComponent::StaticClass());
		CameraCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UDisplayClusterICVFXCameraComponent, CurrentAperture), UCineCameraComponent::StaticClass());
	}

	if (EditedObject.IsValid())
	{
		const FName UpscaleMethodName = EditedObject->CameraSettings.UpscalerSettings.MethodName;
		bool bIsCustomUpscaleMethod = UpscaleMethodName != NAME_None;
	
		if (const UEnum* EnumClass = StaticEnum<EDisplayClusterConfigurationUpscalingMethod>())
		{
			for (int32 EnumElementIndex = 0; EnumElementIndex < EnumClass->NumEnums(); ++EnumElementIndex)
			{
				if (EnumClass->GetNameStringByIndex(EnumElementIndex) == UpscaleMethodName && !EnumClass->HasMetaData(TEXT("Hidden"), EnumElementIndex))
				{
					bIsCustomUpscaleMethod = false;
				}
			}
		}
	
		if (bIsCustomUpscaleMethod)
		{
			// If the upscale method is a custom upscale method, hide the screen percentage
			InLayoutBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UDisplayClusterICVFXCameraComponent, BufferRatioRef))->MarkHiddenByCustomization();
		}
	}

	InLayoutBuilder.SortCategories(DisplayClusterICVFXCameraComponentDetailsCustomizationUtils::SortCategories);

	// Most of the properties in the camera settings are exposed through property references, so hide the camera settings property.
	InLayoutBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UDisplayClusterICVFXCameraComponent, CameraSettings));
}

#undef LOCTEXT_NAMESPACE
