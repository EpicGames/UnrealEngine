// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Types/SlateEnums.h"
#include "UObject/WeakObjectPtr.h"

class UDisplayClusterICVFXCameraComponent;
class ACineCameraActor;
class IDetailLayoutBuilder;

/** Details panel customization for the UDisplayClusterICVFXCameraComponent object */
class FDisplayClusterICVFXCameraComponentDetailsCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

protected:
	// Begin IDetailCustomization interface
	virtual void CustomizeDetails(class IDetailLayoutBuilder& InLayoutBuilder) override;
	// End IDetailCustomization interface

protected:
	/** A weak reference to the UDisplayClusterICVFXCameraComponent object being edited by the details panel */
	TWeakObjectPtr<UDisplayClusterICVFXCameraComponent> EditedObject;

	/** Reference to the detail layout builder, used to force refresh the layout */
	IDetailLayoutBuilder* DetailLayout = nullptr;
};
