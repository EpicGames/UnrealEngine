// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfiguratorBaseDetailCustomization.h"

#include "DisplayClusterRootActor.h"
#include "DisplayClusterConfiguratorBlueprintEditor.h"
#include "DisplayClusterConfiguratorUtils.h"
#include "DisplayClusterConfiguratorLog.h"

#include "PropertyHandle.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "IDetailGroup.h"

const FName FDisplayClusterConfiguratorBaseDetailCustomization::HidePropertyMetaDataKey = TEXT("HideProperty");
const FName FDisplayClusterConfiguratorBaseDetailCustomization::HidePropertyInstanceOnlyMetaDataKey = TEXT("HidePropertyInstanceOnly");

void FDisplayClusterConfiguratorBaseDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& InLayoutBuilder)
{
	UObject* ObjectBeingEdited = nullptr;

	{
		const TArray<TWeakObjectPtr<UObject>>& ObjectsBeingEdited = InLayoutBuilder.GetSelectedObjects();
		check(ObjectsBeingEdited.Num() > 0);
		ObjectBeingEdited = ObjectsBeingEdited[0].Get();

		for (UObject* Owner = ObjectBeingEdited; Owner; Owner = Owner->GetOuter())
		{
			if (ADisplayClusterRootActor* RootActor = Cast<ADisplayClusterRootActor>(Owner))
			{
				RootActorPtr = RootActor;
				break;
			}
		}
	}

	if (!RootActorPtr.IsValid() || (ObjectBeingEdited && ObjectBeingEdited->IsTemplate(RF_ClassDefaultObject)))
	{
		if (FDisplayClusterConfiguratorBlueprintEditor* BPEditor = FDisplayClusterConfiguratorUtils::GetBlueprintEditorForObject(ObjectBeingEdited))
		{
			ToolkitPtr = StaticCastSharedRef<FDisplayClusterConfiguratorBlueprintEditor>(BPEditor->AsShared());
		}
	}

	if (!(RootActorPtr.IsValid() || ToolkitPtr.IsValid()))
	{
		// Possible to hit if details panel selected during undo/redo.
		UE_LOGF(DisplayClusterConfiguratorLog, Warning, "Details panel root actor and toolkit invalid.");
		return;
	}

	// Iterate over all of the properties in the object being edited and process any custom metadata that the properties may have
	{
		TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
		InLayoutBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);
		check(ObjectsBeingCustomized.Num() > 0);

		UObject* ObjectBeingCustomized = ObjectsBeingCustomized[0].Get();

		if (ObjectBeingCustomized)
		{
			UClass* ObjectClass = ObjectBeingCustomized->GetClass();

			for (TFieldIterator<FProperty> It(ObjectClass); It; ++It)
			{
				if (FProperty* Property = *It)
				{
					TSharedRef<IPropertyHandle> PropertyHandle = InLayoutBuilder.GetProperty(Property->GetFName());

					if (PropertyHandle->IsValidHandle())
					{
						ProcessPropertyMetaData(PropertyHandle, InLayoutBuilder);
					}
				}
			}
		}
	}
}

void FDisplayClusterConfiguratorBaseDetailCustomization::CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& InLayoutBuilder)
{
	DetailLayoutBuilder = InLayoutBuilder;
	CustomizeDetails(*InLayoutBuilder);
}

void FDisplayClusterConfiguratorBaseDetailCustomization::ProcessPropertyMetaData(const TSharedRef<IPropertyHandle>& InPropertyHandle, IDetailLayoutBuilder& InLayoutBuilder)
{
	bool bShouldHideProperty = InPropertyHandle->HasMetaData(HidePropertyMetaDataKey) ||
		(InPropertyHandle->HasMetaData(HidePropertyInstanceOnlyMetaDataKey) && !InLayoutBuilder.HasClassDefaultObject());

	if (bShouldHideProperty)
	{
		InLayoutBuilder.HideProperty(InPropertyHandle);
	}
}

IDetailLayoutBuilder* FDisplayClusterConfiguratorBaseDetailCustomization::GetLayoutBuilder() const
{
	IDetailLayoutBuilder* LayoutBuilderPtr = nullptr;
	if (DetailLayoutBuilder.IsValid())
	{
		LayoutBuilderPtr = DetailLayoutBuilder.Pin().Get();
	}

	return LayoutBuilderPtr;
}

FDisplayClusterConfiguratorBlueprintEditor* FDisplayClusterConfiguratorBaseDetailCustomization::GetBlueprintEditor() const
{
	FDisplayClusterConfiguratorBlueprintEditor* BlueprintEditorPtr = nullptr;
	if (ToolkitPtr.IsValid())
	{
		BlueprintEditorPtr = ToolkitPtr.Pin().Get();
	}

	return BlueprintEditorPtr;
}

ADisplayClusterRootActor* FDisplayClusterConfiguratorBaseDetailCustomization::GetRootActor() const
{
	ADisplayClusterRootActor* RootActor = nullptr;

	if (ToolkitPtr.IsValid())
	{
		RootActor = Cast<ADisplayClusterRootActor>(ToolkitPtr.Pin()->GetPreviewActor());
	}
	else
	{
		RootActor = RootActorPtr.Get();
	}

	return RootActor;
}

UDisplayClusterConfigurationData* FDisplayClusterConfiguratorBaseDetailCustomization::GetConfigData() const
{
	UDisplayClusterConfigurationData* ConfigData = nullptr;

	if (ToolkitPtr.IsValid())
	{
		ConfigData = ToolkitPtr.Pin()->GetConfig();
	}
	else if (RootActorPtr.IsValid())
	{
		ConfigData = RootActorPtr->GetConfigData();
	}

	return ConfigData;
}