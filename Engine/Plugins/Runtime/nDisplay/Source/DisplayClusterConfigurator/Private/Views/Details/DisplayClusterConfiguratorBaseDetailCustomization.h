// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "UObject/WeakObjectPtr.h"

class IPropertyHandle;
class IDetailGroup;
class IDetailPropertyRow;
class ADisplayClusterRootActor;
class UDisplayClusterConfigurationData;
class FDisplayClusterConfiguratorBlueprintEditor;

/** 
 * A base class for details customization of UObjects in nDisplay. Contains support for common custom metadata specifiers.
 */
class FDisplayClusterConfiguratorBaseDetailCustomization : public IDetailCustomization
{
public:
	/**
	 * Metadata specifier used to hide a property from an object's details panel. Used in cases where the property needs to have a 
	 * Visible or Edit flag (such as being part of a property reference path), but should not show up in the details panel.
	 */
	static const FName HidePropertyMetaDataKey;

	/** Similar to the HideProperty metadata specifier, but only hides the property when an instance object is being edited. */
	static const FName HidePropertyInstanceOnlyMetaDataKey;


	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShared<FDisplayClusterConfiguratorBaseDetailCustomization>();
	}

	// IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& InLayoutBuilder) override;
	virtual void CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& InLayoutBuilder) override;
	// End IDetailCustomization interface

protected:
	/** 
	 * Processes any custom metadata specifiers for the specified property. Called on each child property of the object
	 * this details customization is applied to. Override to add additional processing for the object's child properties.
	 * 
	 * @param InPropertyHandle - The property handle to process metadata specifiers on.
	 * @param InLayoutBuilder - The detail customization's layout builder.
	 */
	virtual void ProcessPropertyMetaData(const TSharedRef<IPropertyHandle>& InPropertyHandle, IDetailLayoutBuilder& InLayoutBuilder);

	/** Gets a pointer to the details layout builder used by this customization */
	IDetailLayoutBuilder* GetLayoutBuilder() const;

	/** Gets a pointer to the blueprint editor */
	FDisplayClusterConfiguratorBlueprintEditor* GetBlueprintEditor() const;

	/** Gets a pointer to the display cluster root actor being edited by the details panel */
	ADisplayClusterRootActor* GetRootActor() const;

	/** Gets a pointer to the display cluster configuration data of the root actor being edited by the details panel */
	UDisplayClusterConfigurationData* GetConfigData() const;

	/** Gets whether the details panel is being displayed within the cluster configuration blueprint editor */
	bool IsRunningForBlueprintEditor() const { return ToolkitPtr.IsValid(); }

protected:
	/** A weak reference to the display cluster configuration blueprint editor, if the details panel is being displayed in one */
	TWeakPtr<FDisplayClusterConfiguratorBlueprintEditor> ToolkitPtr = nullptr;

	/** A weak reference to the display cluster root actor being edited by the details panel */
	TWeakObjectPtr<ADisplayClusterRootActor> RootActorPtr;

	/** A weak reference to the details layout builder used by this customization */
	TWeakPtr<IDetailLayoutBuilder> DetailLayoutBuilder;
};