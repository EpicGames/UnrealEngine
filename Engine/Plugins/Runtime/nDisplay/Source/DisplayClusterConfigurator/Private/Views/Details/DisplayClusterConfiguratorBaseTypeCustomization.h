// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPropertyTypeCustomization.h"

class ADisplayClusterRootActor;
class FDisplayClusterConfiguratorBlueprintEditor;

#define GET_CHILD_HANDLE(ParentType, ChildName) GetChildHandleChecked(InPropertyHandle, GET_MEMBER_NAME_CHECKED(ParentType, ChildName))

/**
 * A base class for type customization of UStructs in nDisplay. Contains support for common custom metadata specifiers.
 */
class FDisplayClusterConfiguratorBaseTypeCustomization : public IPropertyTypeCustomization
{
public:
	/**
	 * Metadata specifier used to indicate that a type should not have a header to encapsulate its properties in a details panel; similar to 
	 * ShowOnlyInnerProperties, but forces all child properties to be within the category the parent type belongs to, and is applied at any
	 * depth within a details panel.
	 */
	static const FName NoHeaderMetadataKey;

	/** Metadata specifier used to indicate that a type should not display any of its child properties in a details panel. */
	static const FName HideChildrenMetadataKey;

	/** Metadata specifier used specify substitutions to make to the display text of any of this type's child properties. */
	static const FName SubstitutionsMetadataKey;

	/** Metadata specifier used specify the default substitutions to make to the display text of any of this type's child properties when no substitutions are specified. */
	static const FName DefaultSubstitutionsMetadataKey;


	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FDisplayClusterConfiguratorBaseTypeCustomization>();
	}

	//~ IPropertyTypeCustomization interface begin
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	//~ IPropertyTypeCustomization interface end

protected:
	/** Gets the child property handle from the specified property handle, and performs checks to ensure the child handle is valid. */
	TSharedPtr<IPropertyHandle> GetChildHandleChecked(const TSharedRef<IPropertyHandle>& InPropertyHandle, const FName& ChildPropertyName) const;

	/** Initializes the type customization just before specifying the type's header, giving the customization the opportunity to initialize anything before generating the customized UI. */
	virtual void Initialize(const TSharedRef<IPropertyHandle>& InPropertyHandle, IPropertyTypeCustomizationUtils& CustomizationUtils);

	/** Sets the header of the type customization. By default, sets the header to the property's default name and value widget. Override to customize the type's header. */
	virtual void SetHeader(const TSharedRef<IPropertyHandle>& InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils);

	/** Sets the children of the type customization. By default, adds all of the type's children to the customized layout. Override to customize the type's children. */
	virtual void SetChildren(const TSharedRef<IPropertyHandle>& InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils);

	/** Adds all of the type's children to the customization's child builder. */
	void AddAllChildren(const TSharedRef<IPropertyHandle>& InPropertyHandle, IDetailChildrenBuilder& InChildBuilder);

	/** Fills the substitutiom map to use for any text subsitutions using the metadata of the specified property handle */
	void FillSubstitutionMap(const TSharedRef<IPropertyHandle>& InPropertyHandle);

	/** Parses the specified comma-deliminated substitution string into a list of key-value substitution pairs */
	void ParseSubstitutions(FFormatNamedArguments& OutSubstitutionsMap, const FString& InSubstitutionsStr) const;
	FText ApplySubstitutions(const FText& InText) const;

	/** Gets whether the type should display a header when displayed in a details panel. By default, check's the property's metadata for the NoHeader or ShowOnlyInnerProperties specifier. */
	virtual bool ShouldShowHeader(const TSharedRef<IPropertyHandle>& InPropertyHandle) const;

	/** Gets whether the type should display its children when displayed in a details panel. By default, check's the property's metadata for the HideChildren specifier. */
	virtual bool ShouldShowChildren(const TSharedRef<IPropertyHandle>& InPropertyHandle) const;

	/** Forces the currently edited blueprint to recompile and refresh is configuration */
	void RefreshBlueprint();

	/** Marks the currently edited blueprint as modified */
	void ModifyBlueprint();

	/** Attempts to find the display cluster root actor that owns the type being customized. */
	ADisplayClusterRootActor* FindRootActor() const;

	/** Attempts to find the blueprint editor editing the selected object, if the object is being edited in a blueprint editor. */
	FDisplayClusterConfiguratorBlueprintEditor* FindBlueprintEditor() const;

	/** Searches the specified property's instanced and default metadata maps for metadata using the specified key */
	const FString* FindMetaData(TSharedPtr<IPropertyHandle> PropertyHandle, const FName& Key) const;

protected:
	/** A weak reference to the object being edited by the details panel */
	TWeakObjectPtr<UObject> EditingObject;

	/** A list of weak references to the objects being edited by the details panel */
	TArray<TWeakObjectPtr<UObject>> EditingObjects;

	/** A weak reference to the property utilities used by this type customization */
	TWeakPtr<IPropertyUtilities> PropertyUtilities;

	/** A list of substitutions that will be applied to the display text of each child property of the customized type */
	FFormatNamedArguments SubstitutionsMap;
};