// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterEditorPropertyReferenceTypeCustomization.h"

#include "PropertyHandle.h"
#include "DetailWidgetRow.h"
#include "DetailLayoutBuilder.h"
#include "IDetailGroup.h"
#include "IDetailChildrenBuilder.h"
#include "IPropertyTypeCustomization.h"
#include "IPropertyUtilities.h"
#include "Widgets/Text/STextBlock.h"

const FName FDisplayClusterEditorPropertyReferenceTypeCustomization::PropertyPathMetadataKey = TEXT("PropertyPath");
const FName FDisplayClusterEditorPropertyReferenceTypeCustomization::EditConditionPathMetadataKey = TEXT("EditConditionPath");

TSharedRef<IPropertyTypeCustomization> FDisplayClusterEditorPropertyReferenceTypeCustomization::MakeInstance()
{
	return MakeShared<FDisplayClusterEditorPropertyReferenceTypeCustomization>();
}

void FDisplayClusterEditorPropertyReferenceTypeCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	PropertyUtilities = CustomizationUtils.GetPropertyUtilities();

	check(InPropertyHandle->IsValidHandle());
	if (InPropertyHandle->HasMetaData(EditConditionPathMetadataKey))
	{
		// Check to see if the edit condition is using a property path as well.
		FString EditCondition = InPropertyHandle->GetMetaData(EditConditionPathMetadataKey);
		EditCondition.RemoveSpacesInline();
		if (EditCondition.Contains(TEXT(".")) || EditCondition.Contains(TEXT("->")))
		{
			// Support the && operator to allow multiple boolean edit conditions at once. Parse the condition
			// into property paths and add each found property to the list of edit conditions
			TArray<FString> PropertyPaths;
			EditCondition.Replace(TEXT("->"), TEXT(".")).ParseIntoArray(PropertyPaths, TEXT("&&"));

			for (const FString& PropertyPath : PropertyPaths)
			{
				TSharedPtr<IPropertyHandle> EditConditionPropertyHandle = FindPropertyHandle(PropertyPath, InPropertyHandle);
				if (EditConditionPropertyHandle && EditConditionPropertyHandle->IsValidHandle())
				{
					EditConditionPropertyHandles.Add(EditConditionPropertyHandle.ToSharedRef());
				}
			}
		}
	}

	if (InPropertyHandle->HasMetaData(PropertyPathMetadataKey))
	{
		const FString PropertyPath = InPropertyHandle->GetMetaData(PropertyPathMetadataKey).Replace(TEXT("->"), TEXT("."));

		if (!PropertyPath.IsEmpty())
		{
			if (!FindPropertyHandles(PropertyPath, InPropertyHandle))
			{
				InHeaderRow.WholeRowContent()
				[
					SNew(STextBlock)
					.Text(FText::Format(INVTEXT("Could not find property handles at the supplied path: {0}"), FText::FromString(PropertyPath)))
				];
			}
		}
		else
		{
			InHeaderRow.WholeRowContent()
			[
				SNew(STextBlock)
				.Text(INVTEXT("No property path was supplied. Reference a property using the PropertyPath metadata"))
			];
		}
	}
	else
	{
		InHeaderRow.WholeRowContent()
		[
			SNew(STextBlock)
			.Text(INVTEXT("No property path was supplied. Reference a property using the PropertyPath metadata"))
		];
	}
}

void FDisplayClusterEditorPropertyReferenceTypeCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	if (ReferencedPropertyHandles.Num())
	{
		if (bIsIteratedList)
		{
			const FText DisplayName = InPropertyHandle->HasMetaData(TEXT("DisplayName")) ? InPropertyHandle->GetPropertyDisplayName() : ReferencedPropertyHandles[0].Value->GetPropertyDisplayName();
			const FText ToolTip = InPropertyHandle->HasMetaData(TEXT("ToolTip")) ? InPropertyHandle->GetToolTipText() : ReferencedPropertyHandles[0].Value->GetToolTipText();
			IDetailGroup& Group = InChildBuilder.AddGroup(InPropertyHandle->GetProperty()->GetFName(), DisplayName);

			FDetailWidgetRow& GroupHeaderRow = Group.HeaderRow()
				.NameContent()
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(DisplayName)
					.ToolTipText(ToolTip)
				];

			if (EditConditionPropertyHandles.Num())
			{
				GroupHeaderRow.EditCondition(CreateEditConditional(), nullptr);
			}

			for (FPropertyHandlePair& PropertyHandlePair : ReferencedPropertyHandles)
			{
				TSharedPtr<IPropertyHandle> PropertyHandle = PropertyHandlePair.Value;
				if (PropertyHandle && PropertyHandle->IsValidHandle())
				{
					PropertyHandle->SetPropertyDisplayName(PropertyHandlePair.Key);
					PropertyHandle->SetToolTipText(PropertyHandlePair.Key);

					IDetailPropertyRow& PropertyRow = Group.AddPropertyRow(PropertyHandle.ToSharedRef());

					// Mark the property with the "IsCustomized" flag so that any subsequent layout builders can account for the property
					// being moved and placed here.
					PropertyHandle->MarkHiddenByCustomization();

					if (EditConditionPropertyHandles.Num())
					{
						PropertyRow.EditCondition(CreateEditConditional(), nullptr);
					}
				}
			}
		}
		else
		{
			TSharedPtr<IPropertyHandle> ReferencedPropertyHandle = ReferencedPropertyHandles[0].Value;
			if (ReferencedPropertyHandle && ReferencedPropertyHandle->IsValidHandle())
			{
				if (InPropertyHandle->HasMetaData(TEXT("DisplayName")))
				{
					ReferencedPropertyHandle->SetPropertyDisplayName(InPropertyHandle->GetPropertyDisplayName());
				}

				if (InPropertyHandle->HasMetaData(TEXT("ToolTip")))
				{
					ReferencedPropertyHandle->SetToolTipText(InPropertyHandle->GetToolTipText());
				}

				ReferencedPropertyHandle->SetOnPropertyValueChangedWithData(TDelegate<void(const FPropertyChangedEvent&)>::CreateSP(
					this, &FDisplayClusterEditorPropertyReferenceTypeCustomization::OnReferencedPropertyValueChanged));

				const bool bHasShowOnlyInners = InPropertyHandle->HasMetaData(TEXT("ShowOnlyInnerProperties")) || InPropertyHandle->GetInstanceMetaData(TEXT("ShowOnlyInnerProperties"));
				if (bHasShowOnlyInners)
				{
					// If the property referencer has the ShowOnlyInnerProperties metadata, expand the reference property and display the child properties directly
					uint32 NumChildren;
					ReferencedPropertyHandle->GetNumChildren(NumChildren);

					for (uint32 Index = 0; Index < NumChildren; ++Index)
					{
						TSharedPtr<IPropertyHandle> ChildHandle = ReferencedPropertyHandle->GetChildHandle(Index);
						if (!ChildHandle->IsCustomized())
						{
							InChildBuilder.AddProperty(ChildHandle.ToSharedRef());
						}
					}
				}
				else
				{
					IDetailPropertyRow& PropertyRow = InChildBuilder.AddProperty(ReferencedPropertyHandle.ToSharedRef());

					if (EditConditionPropertyHandles.Num())
					{
						PropertyRow.EditCondition(CreateEditConditional(), nullptr);
					}
				}

				// Mark the property with the "IsCustomized" flag so that any subsequent layout builders can account for the property
				// being moved and placed here.
				ReferencedPropertyHandle->MarkHiddenByCustomization();
			}
		}
	}
}

void FDisplayClusterEditorPropertyReferenceTypeCustomization::OnReferencedPropertyValueChanged(const FPropertyChangedEvent& PropertyChangedEvent)
{
	// When a referenced property is changed, we have to trigger layout refresh, as long as the change is not an interactive change
	const bool bIsInteractiveChange = (PropertyChangedEvent.ChangeType & EPropertyChangeType::Interactive) != 0;
	if (PropertyUtilities.IsValid() && !bIsInteractiveChange)
	{
		PropertyUtilities.Pin()->ForceRefresh();
	}
}

bool FDisplayClusterEditorPropertyReferenceTypeCustomization::FindPropertyHandles(const FString& PropertyPath, TSharedRef<IPropertyHandle>& InPropertyHandle)
{
	bIsIteratedList = false;
	TSharedPtr<IPropertyHandle> RootPropertyHandle = FindRootPropertyHandle(PropertyPath, InPropertyHandle);

	if (!(RootPropertyHandle && RootPropertyHandle->IsValidHandle()))
	{
		return false;
	}

	TArray<FString> Properties;
	PropertyPath.ParseIntoArray(Properties, TEXT("."));

	if (Properties.Num())
	{
		int32 CurrentPropertyIndex = 0;
		TArray<FPropertyHandlePair> CurrentPropertyHandles;
		CurrentPropertyHandles.Add(FPropertyHandlePair(FText::GetEmpty(), RootPropertyHandle->GetChildHandle(FName(*Properties[0]))));

		++CurrentPropertyIndex;

		TArray<FPropertyHandlePair> NextPropertyHandles;
		while (CurrentPropertyIndex < Properties.Num() && CurrentPropertyHandles.Num())
		{
			for (FPropertyHandlePair& PropertyHandlePair : CurrentPropertyHandles)
			{
				TSharedPtr<IPropertyHandle>& PropertyHandle = PropertyHandlePair.Value;
				if (IsListType(PropertyHandle))
				{
					bIsIteratedList = true;

					uint32 NumElements = 0;
					if (PropertyHandle->GetNumChildren(NumElements) == FPropertyAccess::Success)
					{
						for (uint32 Index = 0; Index < NumElements; ++Index)
						{
							TSharedPtr<IPropertyHandle> ElementHandle = PropertyHandle->GetChildHandle(Index);
							TSharedPtr<IPropertyHandle> ChildHandle = GetChildPropertyHandle(Properties[CurrentPropertyIndex], ElementHandle);

							if (ChildHandle.IsValid() && ChildHandle->IsValidHandle())
							{
								FText ElementName = FText::GetEmpty();
								if (PropertyHandle->AsMap().IsValid())
								{
									TSharedPtr<IPropertyHandle> KeyHandle = ElementHandle->GetKeyHandle();

									FText KeyValue;
									if (KeyHandle.IsValid() && KeyHandle->GetValueAsDisplayText(KeyValue) == FPropertyAccess::Success)
									{
										ElementName = KeyValue;
									}
								}
								else
								{
									ElementName = FText::Format(INVTEXT("{0}[{1}]"), PropertyHandle->GetPropertyDisplayName(), FText::AsNumber(Index));
								}

								NextPropertyHandles.Add(FPropertyHandlePair(ElementName, ChildHandle));
							}
						}
					}
				}
				else
				{
					TSharedPtr<IPropertyHandle> ChildHandle = GetChildPropertyHandle(Properties[CurrentPropertyIndex], PropertyHandle);

					if (ChildHandle.IsValid() && ChildHandle->IsValidHandle())
					{
						NextPropertyHandles.Add(FPropertyHandlePair(PropertyHandlePair.Key, ChildHandle));
					}
				}
			}

			CurrentPropertyHandles.Empty();
			CurrentPropertyHandles.Append(NextPropertyHandles);
			NextPropertyHandles.Empty();
			++CurrentPropertyIndex;
		}

		if (CurrentPropertyHandles.Num())
		{
			ReferencedPropertyHandles.Empty();
			ReferencedPropertyHandles.Append(CurrentPropertyHandles);

			return true;
		}
		else
		{
			return false;
		}
	}

	return false;
}

TSharedPtr<IPropertyHandle> FDisplayClusterEditorPropertyReferenceTypeCustomization::FindPropertyHandle(const FString& PropertyPath, TSharedRef<IPropertyHandle>& InPropertyHandle)
{
	TSharedPtr<IPropertyHandle> RootPropertyHandle = FindRootPropertyHandle(PropertyPath, InPropertyHandle);
	if (!(RootPropertyHandle && RootPropertyHandle->IsValidHandle()))
	{
		return nullptr;
	}

	TArray<FString> Properties;
	PropertyPath.ParseIntoArray(Properties, TEXT("."));

	if (Properties.Num())
	{
		int32 CurrentPropertyIndex = 0;
		TSharedPtr<IPropertyHandle> CurrentPropertyHandle = RootPropertyHandle->GetChildHandle(FName(*Properties[0]));

		++CurrentPropertyIndex;

		while (CurrentPropertyIndex < Properties.Num() && CurrentPropertyHandle)
		{
			CurrentPropertyHandle = GetChildPropertyHandle(Properties[CurrentPropertyIndex], CurrentPropertyHandle);
			++CurrentPropertyIndex;
		}

		return CurrentPropertyHandle;
	}

	return nullptr;
}

TSharedPtr<IPropertyHandle> FDisplayClusterEditorPropertyReferenceTypeCustomization::FindRootPropertyHandle(const FString& PropertyPath, TSharedRef<IPropertyHandle>& InPropertyHandle)
{
	// Find the property handle from which the property path can be traversed. Do this by searching upwards for the first property handle that contains a child
	// that matches the first property in the property path.
	FName FirstProperty;
	int32 PeriodIndex;
	if (PropertyPath.FindChar('.', PeriodIndex))
	{
		FirstProperty = FName(*PropertyPath.Left(PeriodIndex));
	}
	else
	{
		FirstProperty = FName(*PropertyPath);
	}

	TSharedPtr<IPropertyHandle> CurrentPropertyHandle = InPropertyHandle->GetParentHandle();
	while (CurrentPropertyHandle)
	{
		if (CurrentPropertyHandle->GetChildHandle(FirstProperty))
		{
			return CurrentPropertyHandle;
		}

		CurrentPropertyHandle = CurrentPropertyHandle->GetParentHandle();
	}

	return nullptr;
}

TSharedPtr<IPropertyHandle> FDisplayClusterEditorPropertyReferenceTypeCustomization::GetChildPropertyHandle(const FString& PropertyName, TSharedPtr<IPropertyHandle>& PropertyHandle)
{
	if (PropertyName.Contains(TEXT("[")) && PropertyName.Contains(TEXT("]")))
	{
		FString ListPropertyName;
		FString IndexStr;
		PropertyName.Split(TEXT("["), &ListPropertyName, &IndexStr);

		int32 Index = FCString::Atoi(*IndexStr);

		TSharedPtr<IPropertyHandle> ListPropertyHandle = PropertyHandle->GetChildHandle(FName(*ListPropertyName));
		if (IsListType(ListPropertyHandle))
		{
			uint32 NumElements;
			ListPropertyHandle->GetNumChildren(NumElements);

			if (Index >= 0 && Index < (int32)NumElements)
			{
				if (TSharedPtr<IPropertyHandle> ElementHandle = ListPropertyHandle->GetChildHandle(Index))
				{
					return ElementHandle;
				}
			}
		}

		return nullptr;
	}
	else
	{
		if (TSharedPtr<IPropertyHandle> ChildPropertyHandle = PropertyHandle->GetChildHandle(FName(*PropertyName)))
		{
			return ChildPropertyHandle;
		}

		return nullptr;
	}
}

bool FDisplayClusterEditorPropertyReferenceTypeCustomization::IsListType(const TSharedPtr<IPropertyHandle>& PropertyHandle)
{
	return PropertyHandle->AsArray() || PropertyHandle->AsSet() || PropertyHandle->AsMap();
}

TAttribute<bool> FDisplayClusterEditorPropertyReferenceTypeCustomization::CreateEditConditional() const
{
	return TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &FDisplayClusterEditorPropertyReferenceTypeCustomization::IsEditable));
}

bool FDisplayClusterEditorPropertyReferenceTypeCustomization::IsEditable() const
{
	bool bFinalConditionalValue = true;

	if (EditConditionPropertyHandles.Num())
	{
		for (const TSharedRef<IPropertyHandle>& EditConditionPropertyHandle : EditConditionPropertyHandles)
		{
			if (EditConditionPropertyHandle->IsValidHandle())
			{
				bool bConditionalValue = true;
				EditConditionPropertyHandle->GetValue(bConditionalValue);

				bFinalConditionalValue = bFinalConditionalValue && bConditionalValue;
			}
		}
	}

	return bFinalConditionalValue;
}