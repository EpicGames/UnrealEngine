// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPropertyTypeCustomization.h"

class IPropertyTypeCustomizationUtils;


/** A type customizer that expands a property reference to display the referenced property directly within a details panel. */
class FDisplayClusterEditorPropertyReferenceTypeCustomization : public IPropertyTypeCustomization
{
public:
	typedef TPair<FText, TSharedPtr<IPropertyHandle>> FPropertyHandlePair;

	/** Metadata specifier to provide the path to a property to display. */
	static const FName PropertyPathMetadataKey;

	/** Metadata specifier to provide a path to a property to use as an edit condition. */
	static const FName EditConditionPathMetadataKey;


	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	//~ IPropertyTypeCustomization interface begin
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	//~ IPropertyTypeCustomization interface end

private:
	bool FindPropertyHandles(const FString& PropertyPath, TSharedRef<IPropertyHandle>& InPropertyHandle);
	TSharedPtr<IPropertyHandle> FindPropertyHandle(const FString& PropertyPath, TSharedRef<IPropertyHandle>& InPropertyHandle);

	TSharedPtr<IPropertyHandle> FindRootPropertyHandle(const FString& PropertyPath, TSharedRef<IPropertyHandle>& InPropertyHandle);
	TSharedPtr<IPropertyHandle> GetChildPropertyHandle(const FString& PropertyName, TSharedPtr<IPropertyHandle>& PropertyHandle);

	void OnReferencedPropertyValueChanged(const FPropertyChangedEvent& PropertyChangedEvent);

	bool IsListType(const TSharedPtr<IPropertyHandle>& PropertyHandle);

	TAttribute<bool> CreateEditConditional() const;
	bool IsEditable() const;

	// Customization utils instance to trigger GUI update on referenced value changes
	TWeakPtr<IPropertyUtilities> PropertyUtilities = nullptr;

	TArray<FPropertyHandlePair> ReferencedPropertyHandles;
	TArray<TSharedRef<IPropertyHandle>> EditConditionPropertyHandles;
	bool bIsIteratedList;
};