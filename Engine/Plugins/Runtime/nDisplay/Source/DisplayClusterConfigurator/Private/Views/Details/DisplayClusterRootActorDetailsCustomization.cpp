// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterRootActorDetailsCustomization.h"

#include "Widgets/SDisplayClusterConfiguratorComponentPicker.h"

#include "Render/DisplayDevice/Components/DisplayClusterDisplayDeviceBaseComponent.h"
#include "DisplayClusterRootActor.h"
#include "DisplayClusterConfigurationStrings.h"
#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterConfigurationTypes_Viewport.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailGroup.h"
#include "PropertyHandle.h"

#include "ColorGradingEditorUtil.h"
#include "EditorSupportDelegates.h"
#include "Widgets/Input/SSearchableComboBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Algo/Transform.h"


#define LOCTEXT_NAMESPACE "DisplayClusterRootActorDetailsCustomization"

namespace DisplayClusterRootActorDetailsCustomizationUtils
{
	void SortCategories(const TMap<FName, IDetailCategoryBuilder*>& AllCategoryMap)
	{
		static const TArray<FName> CategoryOrder =
		{
			TEXT("TransformCommon"),
			DisplayClusterConfigurationStrings::categories::ViewportsCategory,
			DisplayClusterConfigurationStrings::categories::InCameraVFXCategory,
			DisplayClusterConfigurationStrings::categories::ColorGradingCategory,
			DisplayClusterConfigurationStrings::categories::OCIOCategory,
			DisplayClusterConfigurationStrings::categories::ExposureCategory,
			DisplayClusterConfigurationStrings::categories::ChromaKeyCategory,
			DisplayClusterConfigurationStrings::categories::LightcardCategory,
			DisplayClusterConfigurationStrings::categories::OverrideCategory,
			TEXT("Rendering"),
			TEXT("WorldPartition"),
			DisplayClusterConfigurationStrings::categories::PreviewCategory,
			DisplayClusterConfigurationStrings::categories::ConfigurationCategory,
			DisplayClusterConfigurationStrings::categories::AdvancedCategory,
		};

		for (const TPair<FName, IDetailCategoryBuilder*>& Pair : AllCategoryMap)
		{
			int32 CurrentSortOrder = Pair.Value->GetSortOrder();

			int32 DesiredSortOrder;
			if (CategoryOrder.Find(Pair.Key, DesiredSortOrder))
			{
				CurrentSortOrder = DesiredSortOrder;
			}
			else
			{
				CurrentSortOrder += CategoryOrder.Num();
			}

			Pair.Value->SetSortOrder(CurrentSortOrder);
		}
	}
}

TSharedRef<IDetailCustomization> FDisplayClusterRootActorDetailsCustomization::MakeInstance()
{
	return MakeShared<FDisplayClusterRootActorDetailsCustomization>();
}

FDisplayClusterRootActorDetailsCustomization::~FDisplayClusterRootActorDetailsCustomization()
{
	FEditorSupportDelegates::ForcePropertyWindowRebuild.Remove(ForcePropertyWindowRebuildHandle);
}

void FDisplayClusterRootActorDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& InLayoutBuilder)
{
	FDisplayClusterConfiguratorBaseDetailCustomization::CustomizeDetails(InLayoutBuilder);

	// Add the Color Grading button at the top of the relevant category
	{
		IDetailCategoryBuilder& ColorGradingCategory = InLayoutBuilder.EditCategory(DisplayClusterConfigurationStrings::categories::ColorGradingCategory, LOCTEXT("ColorGradingDetails", "Color Grading"));
		ColorGradingCategory.AddCustomRow(NSLOCTEXT("ColorCorrectWindowDetails", "OpenColorGrading", "Open Color Grading"))
			.RowTag("OpenColorGrading")
			[
				ColorGradingEditorUtil::MakeColorGradingLaunchButton()
			];
	}

	const TArray<TWeakObjectPtr<UObject>>& SelectedObjects = InLayoutBuilder.GetSelectedObjects();
	bMultipleObjectsSelected = SelectedObjects.Num() > 1;

	ForcePropertyWindowRebuildHandle = FEditorSupportDelegates::ForcePropertyWindowRebuild.AddSP(this, &FDisplayClusterRootActorDetailsCustomization::OnForcePropertyWindowRebuild);

	// Hide Categories:
	InLayoutBuilder.HideCategory(DisplayClusterConfigurationStrings::categories::ViewPointStereoCategory);
	InLayoutBuilder.HideCategory(DisplayClusterConfigurationStrings::categories::ViewPointCameraPostProcessCategory);
	InLayoutBuilder.HideCategory(DisplayClusterConfigurationStrings::categories::ViewPointInFrustumProjectionCategory);

	// For DCRA in a scene, hide more categories:
	if (!IsRunningForBlueprintEditor())
	{
		InLayoutBuilder.HideCategory(DisplayClusterConfigurationStrings::categories::ConfigurationCategory);
		InLayoutBuilder.HideCategory(DisplayClusterConfigurationStrings::categories::DefaultCategory);
	}

	InLayoutBuilder.SortCategories(DisplayClusterRootActorDetailsCustomizationUtils::SortCategories);
	
	/*
	* The code below is commented out because it causes a crash.
	* You cannot change the structure of the DCRootActor inside the customization delegate.
	* This causes a crash inside the LayoutData.ClassToPropertyMap iterator.
	* Customization called from  \\Engine\Source\Editor\PropertyEditor\Private\DetailLayoutHelpers.cpp:356
	*/
	// Manually add the transform properties' data to the layout builder's property in order to generate property handles for them.
	{
		TArray<UObject*> RootComponents;
		Algo::Transform(SelectedObjects, RootComponents, [](TWeakObjectPtr<UObject> Obj)
		{
			USceneComponent* RootComponent = nullptr;
			if (ADisplayClusterRootActor* RootActor = Cast<ADisplayClusterRootActor>(Obj.Get()))
			{
				RootComponent = RootActor->DisplayClusterRootComponent;
			}

			return RootComponent;
		});

		InLayoutBuilder.AddObjectPropertyData(RootComponents, USceneComponent::GetRelativeLocationPropertyName());
		InLayoutBuilder.AddObjectPropertyData(RootComponents, USceneComponent::GetRelativeRotationPropertyName());
		InLayoutBuilder.AddObjectPropertyData(RootComponents, USceneComponent::GetRelativeScale3DPropertyName());
	}

	// Manually label the ICVFX category to properly format it to have the dash in "In-Camera"
	InLayoutBuilder.EditCategory(DisplayClusterConfigurationStrings::categories::InCameraVFXCategory, LOCTEXT("InCameraVFXCategoryLabel", "In-Camera VFX"));

	// Customize the PreviewNodeId property to be a dropdown filled with the nodes configured on the root actor.
	{
		PreviewNodeIdHandle = InLayoutBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ADisplayClusterRootActor, PreviewNodeId));
		check(PreviewNodeIdHandle->IsValidHandle());

		// Doesn't make sense to have this dropdown if multiple root actors are selected, so only display it if a single object is selected.
		if (!bMultipleObjectsSelected)
		{
			if (RebuildNodeIdOptionsList())
			{
				if (IDetailPropertyRow* PropertyRow = InLayoutBuilder.EditDefaultProperty(PreviewNodeIdHandle))
				{
					PropertyRow->CustomWidget()
						.NameContent()[PreviewNodeIdHandle->CreatePropertyNameWidget()]
						.ValueContent()[CreateCustomNodeIdWidget()];
				}
			}
		}
		else
		{
			PreviewNodeIdHandle->MarkHiddenByCustomization();
		}
	}

	// Hide unwanted properties from "Rendering" category
	{
		IDetailCategoryBuilder& RenderingCategory = InLayoutBuilder.EditCategory(TEXT("Rendering"));

		TArray<TSharedRef<IPropertyHandle>> DefaultProperties;
		RenderingCategory.GetDefaultProperties(DefaultProperties);

		for (TSharedRef<IPropertyHandle>& PropertyHandle : DefaultProperties)
		{
			if (const FProperty* Property = PropertyHandle->GetProperty())
			{
				if (Property->GetFName() != TEXT("bHidden")) // "Actor Hidden In Game"
				{
					PropertyHandle->MarkHiddenByCustomization();
				}
			}
		}
	}

	// Update the selected item in the NodeId combo box to match the current value on the root actor
	UpdateNodeIdSelection();

	// Default Display Device component selection
	TSharedPtr<IPropertyHandle> DisplayDeviceHandle = InLayoutBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ADisplayClusterRootActor, DefaultDisplayDeviceName));
	if (DisplayDeviceHandle.IsValid() && DisplayDeviceHandle->IsValidHandle())
	{
		if (IDetailPropertyRow* DisplayDevicePropertyRow = InLayoutBuilder.EditDefaultProperty(DisplayDeviceHandle))
		{
			DisplayDevicePropertyRow->CustomWidget()
				.NameContent()
				[
					DisplayDeviceHandle->CreatePropertyNameWidget()
				]
				.ValueContent()
				[
					SNew(SDisplayClusterConfiguratorComponentPicker,
						UDisplayClusterDisplayDeviceBaseComponent::StaticClass(),
						GetRootActor(),
						DisplayDeviceHandle)
				];
		}
	}

	if (UDisplayClusterConfigurationData* ConfigData = GetConfigData())
	{
		const FName UpscaleMethodName = ConfigData->StageSettings.OuterViewportUpscalerSettings.MethodName;
		bool bIsCustomUpscaleMethod = UpscaleMethodName != NAME_None;

		if (const UEnum* EnumClass = StaticEnum<EDisplayClusterConfigurationUpscalingMethod>())
		{
			for (int32 EnumElementIndex = 0; EnumElementIndex < EnumClass->NumEnums(); ++EnumElementIndex)
			{
				if (EnumClass->GetNameStringByIndex(EnumElementIndex) == UpscaleMethodName && !EnumClass->HasMetaData(TEXT("Hidden"), EnumElementIndex))
				{
					bIsCustomUpscaleMethod = false;
				}
			}
		}

		if (!bIsCustomUpscaleMethod)
		{
			// If the upscale method is not a custom upscale method, hide the per-viewport upscaler settings list
			InLayoutBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ADisplayClusterRootActor, ViewportUpscalerSettingsRef))->MarkHiddenByCustomization();
		}
		else
		{
			// Otherwise, hide the screen percentage properties
			InLayoutBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ADisplayClusterRootActor, ViewportScreenPercentageMultiplierRef))->MarkHiddenByCustomization();
			InLayoutBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ADisplayClusterRootActor, ViewportScreenPercentageRef))->MarkHiddenByCustomization();

			InLayoutBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ADisplayClusterRootActor, ViewportUpscalerSettingsRef))->SetPropertyDisplayName(
				FText::Format(LOCTEXT("PerViewportUpscalerSettingOverrides", "Per-Viewport {0} Setting Overrides"), FText::FromName(UpscaleMethodName)));
		}
	}
}

TSharedRef<SWidget> FDisplayClusterRootActorDetailsCustomization::CreateCustomNodeIdWidget()
{
	if (NodeIdComboBox.IsValid())
	{
		return NodeIdComboBox.ToSharedRef();
	}

	return SAssignNew(NodeIdComboBox, SSearchableComboBox)
		.OptionsSource(&NodeIdOptions)
		.OnGenerateWidget(this, &FDisplayClusterRootActorDetailsCustomization::CreateComboWidget)
		.OnSelectionChanged(this, &FDisplayClusterRootActorDetailsCustomization::OnNodeIdSelected)
		.ContentPadding(2)
		.Content()
		[
			SNew(STextBlock)
			.Text(this, &FDisplayClusterRootActorDetailsCustomization::GetSelectedNodeIdText)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		];
}

bool FDisplayClusterRootActorDetailsCustomization::RebuildNodeIdOptionsList()
{
	// Get current configuration data
	const UDisplayClusterConfigurationData* ConfigurationData = RootActorPtr->GetConfigData();
	if (!ConfigurationData)
	{
		return false;
	}

	// Initialize special options
	NodeIdOptionNone = MakeShared<FString>(DisplayClusterConfigurationStrings::gui::preview::PreviewNodeNone);

	// Fill combobox with the options
	NodeIdOptions.Reset();
	NodeIdOptions.Emplace(NodeIdOptionNone);
	for (const TPair<FString, TObjectPtr<UDisplayClusterConfigurationClusterNode>>& Node : ConfigurationData->Cluster->Nodes)
	{
		if (!Node.Key.IsEmpty())
		{
			NodeIdOptions.Emplace(MakeShared<FString>(Node.Key));
		}
	}

	// Set 'None' each time we update the preview config file
	if (NodeIdComboBox.IsValid())
	{
		NodeIdComboBox->SetSelectedItem(NodeIdOptionNone);
	}

	// Make sure we've got at least one cluster node in the config
	// (None+node or None+All+node1+node2+...)
	return NodeIdOptions.Num() >= 2;
}

void FDisplayClusterRootActorDetailsCustomization::UpdateNodeIdSelection()
{
	if (NodeIdComboBox.IsValid())
	{
		const FString& CurrentPreviewNode = RootActorPtr->PreviewNodeId;
		TSharedPtr<FString>* FoundItem = NodeIdOptions.FindByPredicate([CurrentPreviewNode](const TSharedPtr<FString>& Item)
		{
			return Item->Equals(CurrentPreviewNode, ESearchCase::IgnoreCase);
		});

		if (FoundItem)
		{
			NodeIdComboBox->SetSelectedItem(*FoundItem);
		}
		else
		{
			// Set combobox selected item (options list is not empty here)
			NodeIdComboBox->SetSelectedItem(NodeIdOptionNone);
		}
	}
}

void FDisplayClusterRootActorDetailsCustomization::OnNodeIdSelected(TSharedPtr<FString> PreviewNodeId, ESelectInfo::Type SelectInfo)
{
	const FString NewValue = (PreviewNodeId.IsValid() ? *PreviewNodeId : *NodeIdOptionNone);
	PreviewNodeIdHandle->SetValue(NewValue);
}

FText FDisplayClusterRootActorDetailsCustomization::GetSelectedNodeIdText() const
{
	if (NodeIdComboBox.IsValid())
	{
		TSharedPtr<FString> CurSelection = NodeIdComboBox->GetSelectedItem();
		return FText::FromString(CurSelection.IsValid() ? *CurSelection : *NodeIdOptionNone);
	}

	return FText::FromString(*NodeIdOptionNone);
}

TSharedRef<SWidget> FDisplayClusterRootActorDetailsCustomization::CreateComboWidget(TSharedPtr<FString> InItem)
{
	return SNew(STextBlock)
		.Text(FText::FromString(*InItem))
		.Font(IDetailLayoutBuilder::GetDetailFont());
}

void FDisplayClusterRootActorDetailsCustomization::OnForcePropertyWindowRebuild(UObject* Object)
{
	if (DetailLayoutBuilder.IsValid() && RootActorPtr.IsValid())
	{
		if (RootActorPtr->GetClass() == Object)
		{
			if (IDetailLayoutBuilder* LayoutBuilder = GetLayoutBuilder())
			{
				LayoutBuilder->ForceRefreshDetails();
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
