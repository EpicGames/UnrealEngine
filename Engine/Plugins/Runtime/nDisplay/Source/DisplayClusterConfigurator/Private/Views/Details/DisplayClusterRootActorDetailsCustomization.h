// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Views/Details/DisplayClusterConfiguratorBaseDetailCustomization.h"

#include "CoreMinimal.h"
#include "Types/SlateEnums.h"

class ADisplayClusterRootActor;
class IDetailCategoryBuilder;
class IDetailLayoutBuilder;
class IPropertyHandle;
class SWidget;
class SSearchableComboBox;


class FDisplayClusterRootActorDetailsCustomization final : public FDisplayClusterConfiguratorBaseDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

	~FDisplayClusterRootActorDetailsCustomization();


	// IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& InLayoutBuilder) override;
	// End IDetailCustomization interface

private:
	TSharedRef<SWidget> CreateCustomNodeIdWidget();
	bool RebuildNodeIdOptionsList();
	void UpdateNodeIdSelection();

	void OnNodeIdSelected(TSharedPtr<FString> NodeId, ESelectInfo::Type SelectInfo);
	FText GetSelectedNodeIdText() const;
	TSharedRef<SWidget> CreateComboWidget(TSharedPtr<FString> InItem);

	void OnForcePropertyWindowRebuild(UObject* Object);

private:
	FDelegateHandle ForcePropertyWindowRebuildHandle;

	TSharedPtr<IPropertyHandle> PreviewNodeIdHandle;

	TArray<TSharedPtr<FString>> NodeIdOptions;
	TSharedPtr<FString> NodeIdOptionNone;
	TSharedPtr<SSearchableComboBox> NodeIdComboBox;

	bool bMultipleObjectsSelected;
};
