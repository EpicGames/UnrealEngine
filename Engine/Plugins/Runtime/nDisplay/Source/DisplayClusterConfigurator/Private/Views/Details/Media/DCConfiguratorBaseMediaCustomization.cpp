// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/Details/Media/DCConfiguratorBaseMediaCustomization.h"
#include "Views/Details/Media/DisplayClusterConfiguratorMediaUtils.h"

#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

#include "DetailWidgetRow.h"
#include "DisplayClusterConfiguratorBlueprintEditor.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "FDCConfiguratorBaseMediaCustomization"


void FDCConfiguratorBaseMediaCustomization::AddResetButton(IDetailChildrenBuilder& InChildBuilder, const FText& ButtonText)
{
	InChildBuilder.AddCustomRow(FText::GetEmpty())
		.WholeRowContent()
		[
			SNew(SBox)
			.Padding(5.f)
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.OnClicked(this, &FDCConfiguratorBaseMediaCustomization::OnResetButtonClicked)
				[
					SNew(STextBlock)
					.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
					.Text(ButtonText)
				]
			]
		];
}

FReply FDCConfiguratorBaseMediaCustomization::OnResetButtonClicked()
{
	if (EditingObject.IsValid())
	{
		FScopedTransaction Transaction(LOCTEXT("ResetMediaSettings", "Reset Media Settings"));
		EditingObject->Modify();

		// Notify tile customizers to re-initialize their media objects
		FDisplayClusterConfiguratorMediaUtils::Get().OnMediaResetToDefaults().Broadcast(EditingObject.Get());

		// Set owning package dirty
		MarkDirty();
	}

	return FReply::Handled();
}

void FDCConfiguratorBaseMediaCustomization::MarkDirty()
{
	if (EditingObject.IsValid())
	{
		// Blueprint
		if (EditingObject->IsInBlueprint())
		{
			ModifyBlueprint();
		}
		// Instance
		else
		{
			EditingObject->MarkPackageDirty();
		}
	}
}

#undef LOCTEXT_NAMESPACE
