// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Views/Details/DisplayClusterConfiguratorBaseTypeCustomization.h"

#include "Input/Reply.h"


/**
 * Base customization class. Provides some common functionality for concrete implementations.
 */
class FDCConfiguratorBaseMediaCustomization
	: public FDisplayClusterConfiguratorBaseTypeCustomization
{
public:

	/** Abstract */
	virtual ~FDCConfiguratorBaseMediaCustomization() = default;

protected:

	/** Builds reset button widget. */
	void AddResetButton(IDetailChildrenBuilder& ChildBuilder, const FText& ButtonText);

	/** Handles reset button clicks. */
	FReply OnResetButtonClicked();

	/** Marks package as dirty */
	void MarkDirty();
};
