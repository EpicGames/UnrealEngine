// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/Details/Media/DCConfiguratorClusterNodeMediaCustomization.h"

#include "IDetailChildrenBuilder.h"
#include "IPropertyUtilities.h"
#include "PropertyHandle.h"

#define LOCTEXT_NAMESPACE "FDCConfiguratorClusterNodeMediaCustomization"


void FDCConfiguratorClusterNodeMediaCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	// Build all
	Super::CustomizeChildren(InPropertyHandle, InChildBuilder, InCustomizationUtils);

	// Create 'reset' button at the bottom
	AddResetButton(InChildBuilder, LOCTEXT("ResetToDefaultButtonTitle", "Reset Media Output to Default"));
}

#undef LOCTEXT_NAMESPACE
