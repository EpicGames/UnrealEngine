// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Views/Details/Media/DCConfiguratorBaseMediaCustomization.h"


/**
 * Details panel customization for 'FDisplayClusterConfigurationMediaNodeBackbuffer' struct.
 */
class FDCConfiguratorClusterNodeMediaCustomization
	: public FDCConfiguratorBaseMediaCustomization
{
private:

	using Super = FDCConfiguratorBaseMediaCustomization;

public:

	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FDCConfiguratorClusterNodeMediaCustomization>();
	}

protected:

	//~ Begin IPropertyTypeCustomization
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	//~ End IPropertyTypeCustomization
};
