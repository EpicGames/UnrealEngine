// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/Details/Media/DCConfiguratorICVFXMediaCustomization.h"
#include "Views/Details/Media/DisplayClusterConfiguratorMediaUtils.h"
#include "Views/Details/Media/SMediaTilesConfigurationDialog.h"

#include "DisplayClusterConfigurationTypes_Media.h"
#include "DisplayClusterConfiguratorBlueprintEditor.h"
#include "DisplayClusterConfiguratorUtils.h"
#include "DisplayClusterRootActor.h"

#include "Blueprints/DisplayClusterBlueprint.h"
#include "Components/DisplayClusterICVFXCameraComponent.h"

#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "IPropertyUtilities.h"
#include "PropertyHandle.h"

#include "Framework/Application/SlateApplication.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Interfaces/IMainFrameModule.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"

#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

#include "MediaSource.h"
#include "MediaOutput.h"

#define LOCTEXT_NAMESPACE "FDCConfiguratorICVFXMediaCustomization"


void FDCConfiguratorICVFXMediaCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	// SplitType property
	TSharedPtr<IPropertyHandle> SplitTypeHandle = GET_CHILD_HANDLE(FDisplayClusterConfigurationMediaICVFX, SplitType);
	check(SplitTypeHandle && SplitTypeHandle->IsValidHandle());

	// Layout property
	TSharedPtr<IPropertyHandle> TilesLayoutHandle = GET_CHILD_HANDLE(FDisplayClusterConfigurationMediaICVFX, TiledSplitLayout);
	check(TilesLayoutHandle && TilesLayoutHandle->IsValidHandle());

	// Separate groups specific for every split type available
	TArray<TSharedPtr<IPropertyHandle>> FullFramePropertyHandles;
	TArray<TSharedPtr<IPropertyHandle>> UniformTilePropertyHandles;

	// FullFrame properties
	FullFramePropertyHandles.Add(GET_CHILD_HANDLE(FDisplayClusterConfigurationMediaICVFX, MediaInputGroups));
	FullFramePropertyHandles.Add(GET_CHILD_HANDLE(FDisplayClusterConfigurationMediaICVFX, MediaOutputGroups));

	// UniformTile properties
	UniformTilePropertyHandles.Add(GET_CHILD_HANDLE(FDisplayClusterConfigurationMediaICVFX, TiledSplitLayout));
	UniformTilePropertyHandles.Add(GET_CHILD_HANDLE(FDisplayClusterConfigurationMediaICVFX, TileOverscan));
	UniformTilePropertyHandles.Add(GET_CHILD_HANDLE(FDisplayClusterConfigurationMediaICVFX, ClusterNodesToRenderUnboundTiles));
	UniformTilePropertyHandles.Add(GET_CHILD_HANDLE(FDisplayClusterConfigurationMediaICVFX, TiledMediaInputGroups));
	UniformTilePropertyHandles.Add(GET_CHILD_HANDLE(FDisplayClusterConfigurationMediaICVFX, TiledMediaOutputGroups));

	// Get current split type
	EDisplayClusterConfigurationMediaSplitType SplitTypeValue = EDisplayClusterConfigurationMediaSplitType::UniformTiles;
	SplitTypeHandle->GetValue((uint8&)SplitTypeValue);

	TSharedPtr<IPropertyUtilities> PropertyUtils = InCustomizationUtils.GetPropertyUtilities();
	check(PropertyUtils);

	// Set details update request on frustum type change
	SplitTypeHandle->SetOnPropertyValueChanged(
		FSimpleDelegate::CreateLambda([PropertyUtils]()
		{
			if (PropertyUtils)
			{
				PropertyUtils->RequestForceRefresh();
			}
		}));

	// Filter properties to hide
	TArray<TArray<TSharedPtr<IPropertyHandle>>*> HiddenPropertyHandles;
	switch (SplitTypeValue)
	{
	case EDisplayClusterConfigurationMediaSplitType::FullFrame:
		HiddenPropertyHandles.Add(&UniformTilePropertyHandles);
		break;

	case EDisplayClusterConfigurationMediaSplitType::UniformTiles:
		HiddenPropertyHandles.Add(&FullFramePropertyHandles);
		break;

	default:
		checkNoEntry();
	}

	// Hide unnecessary properties depending on the frustum (split) type currently selected
	for (const TArray<TSharedPtr<IPropertyHandle>>* HiddenGroup : HiddenPropertyHandles)
	{
		for (const TSharedPtr<IPropertyHandle>& PropertyHandle : *HiddenGroup)
		{
			PropertyHandle->MarkHiddenByCustomization();
		}
	}

	// Finally, build the panel
	if (ShouldShowChildren(InPropertyHandle))
	{
		const bool bUsingTiles = (SplitTypeValue == EDisplayClusterConfigurationMediaSplitType::UniformTiles);

		uint32 NumChildren = 0;
		InPropertyHandle->GetNumChildren(NumChildren);

		// For each child property, build its own layout
		for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
		{
			TSharedPtr<IPropertyHandle> ChildHandle = InPropertyHandle->GetChildHandle(ChildIndex);
			if (ChildHandle && ChildHandle->IsValidHandle() && !ChildHandle->IsCustomized())
			{
				FText ChildTooltip = ApplySubstitutions(ChildHandle->GetToolTipText());
				ChildHandle->SetToolTipText(ChildTooltip);

				InChildBuilder.AddProperty(ChildHandle.ToSharedRef());

				// Insert tile configuration button after the split type combobox
				if (bUsingTiles && ChildHandle->IsSamePropertyNode(SplitTypeHandle))
				{
					AddConfigureTilesButton(InChildBuilder);
				}
			}
		}

		// Create 'reset' button at the bottom
		AddResetButton(InChildBuilder, LOCTEXT("ResetToDefaultButtonTitle", "Reset Media Input and Output to Default"));
	}
}

void FDCConfiguratorICVFXMediaCustomization::AddConfigureTilesButton(IDetailChildrenBuilder& InChildBuilder)
{
	InChildBuilder.AddCustomRow(FText::GetEmpty())
		.WholeRowContent()
		[
			SNew(SBox)
			.Padding(5.f)
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.OnClicked(this, &FDCConfiguratorICVFXMediaCustomization::OnConfigureTilesButtonClicked)
				[
					SNew(STextBlock)
					.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
					.Text(LOCTEXT("ConfigureTilesButtonTitle", "Configure Tiles"))
				]
			]
		];
}

FReply FDCConfiguratorICVFXMediaCustomization::OnConfigureTilesButtonClicked()
{
	// We're in camera tiles customization so let's get the camera component
	UDisplayClusterICVFXCameraComponent* ICVFXCamera = Cast<UDisplayClusterICVFXCameraComponent>(EditingObject.Get());
	if (!ICVFXCamera)
	{
		return FReply::Handled();
	}

	// Get config data of the DCRA owning the camera being edited
	const UDisplayClusterConfigurationData* ConfigData = GetConfig();
	if (!ConfigData)
	{
		return FReply::Handled();
	}

	// Nothing to do if no cluster nodes available
	if (ConfigData->Cluster->Nodes.Num() < 1)
	{
		FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("MessageNoClusterNodesAvailable", "There are no cluster nodes assigned. Can't configure media."));
		return FReply::Handled();
	}

	// Camera's media settings
	FDisplayClusterConfigurationMediaICVFX& MediaSettings = ICVFXCamera->CameraSettings.RenderSettings.Media;

	// Prepare everything for the configuration dialog
	FMediaTilesConfigurationDialogParameters Parameters;
	Parameters.Owner        = ICVFXCamera;
	Parameters.ConfigData   = ConfigData;
	Parameters.SplitLayout  = &MediaSettings.TiledSplitLayout;
	Parameters.InputGroups  = &MediaSettings.TiledMediaInputGroups;
	Parameters.OutputGroups = &MediaSettings.TiledMediaOutputGroups;
	Parameters.bAutoPreconfigureOutputMapping = true;

	// Instantiate and show the config dialog
	const TSharedRef<SMediaTilesConfigurationDialog> TilesConfigurationDialog = SNew(SMediaTilesConfigurationDialog, Parameters);
	if (GEditor)
	{
		GEditor->EditorAddModalWindow(TilesConfigurationDialog);
	}

	// Process configuration results
	if (TilesConfigurationDialog->WasConfigurationCompleted())
	{
		// Redraw property views
		if (PropertyUtilities.IsValid())
		{
			PropertyUtilities.Pin()->ForceRefresh();
		}

		// Notify tile customizers to re-initialize all media we just generated
		FDisplayClusterConfiguratorMediaUtils::Get().OnMediaResetToDefaults().Broadcast(EditingObject.Get());

		// Set owning package dirty
		MarkDirty();
	}

	return FReply::Handled();
}

UDisplayClusterConfigurationData* FDCConfiguratorICVFXMediaCustomization::GetConfig() const
{
	if (UDisplayClusterICVFXCameraComponent* ICVFXCameraComponent = Cast<UDisplayClusterICVFXCameraComponent>(EditingObject))
	{
		// For instances
		if (ADisplayClusterRootActor* DCRA = Cast<ADisplayClusterRootActor>(ICVFXCameraComponent->GetOwner()))
		{
			return DCRA->GetConfigData();
		}
		// For DCRA configurator
		else if (FDisplayClusterConfiguratorBlueprintEditor* BlueprintEditor = FDisplayClusterConfiguratorUtils::GetBlueprintEditorForObject(ICVFXCameraComponent))
		{
			return BlueprintEditor->GetConfig();
		}
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
