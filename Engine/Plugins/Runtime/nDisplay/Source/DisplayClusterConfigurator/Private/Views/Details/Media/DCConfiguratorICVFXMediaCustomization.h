// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Views/Details/Media/DCConfiguratorBaseMediaCustomization.h"

#include "Input/Reply.h"

#include "IPropertyTypeCustomization.h"

class UDisplayClusterConfigurationData;


/**
 * Details panel customization for the FDisplayClusterConfigurationMediaICVFX struct.
 */
class FDCConfiguratorICVFXMediaCustomization
	: public FDCConfiguratorBaseMediaCustomization
{
private:

	using Super = FDCConfiguratorBaseMediaCustomization;

public:

	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FDCConfiguratorICVFXMediaCustomization>();
	}

protected:

	//~ Begin IPropertyTypeCustomization
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	//~ End IPropertyTypeCustomization

private:

	/** Builds the button widget for tiles configuration. */
	void AddConfigureTilesButton(IDetailChildrenBuilder& ChildBuilder);

	/** Handles configure tiles button clicks. */
	FReply OnConfigureTilesButtonClicked();

private:

	/** Returns configuration of a DCRA owning the camera being edited. */
	UDisplayClusterConfigurationData* GetConfig() const;
};
