// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/Details/Media/DCConfiguratorViewportMediaCustomization.h"

#include "IDetailChildrenBuilder.h"
#include "IPropertyUtilities.h"
#include "PropertyHandle.h"

#define LOCTEXT_NAMESPACE "FDCConfiguratorViewportMediaCustomization"

void FDCConfiguratorViewportMediaCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	// Build all
	Super::CustomizeChildren(InPropertyHandle, InChildBuilder, InCustomizationUtils);

	// Create 'reset' button at the bottom
	AddResetButton(InChildBuilder, LOCTEXT("ResetToDefaultButtonTitle", "Reset Media Input and Output to Default"));
}

#undef LOCTEXT_NAMESPACE
