// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Views/Details/Media/DCConfiguratorBaseMediaCustomization.h"


/**
 * Details panel customization for 'FDisplayClusterConfigurationMediaViewport' struct.
 */
class FDCConfiguratorViewportMediaCustomization
	: public FDCConfiguratorBaseMediaCustomization
{
private:

	using Super = FDCConfiguratorBaseMediaCustomization;

public:

	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FDCConfiguratorViewportMediaCustomization>();
	}

protected:

	//~ Begin IPropertyTypeCustomization
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	//~ End IPropertyTypeCustomization
};
