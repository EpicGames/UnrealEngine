// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/Details/Media/DisplayClusterConfiguratorMediaCustomizationBase.h"
#include "Views/Details/Media/DisplayClusterConfiguratorMediaCustomizationCVars.h"
#include "Views/Details/Media/DisplayClusterConfiguratorMediaUtils.h"

#include "Components/DisplayClusterICVFXCameraComponent.h"

#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterConfigurationTypes_Media.h"
#include "DisplayClusterConfiguratorBlueprintEditor.h"
#include "DisplayClusterConfiguratorUtils.h"
#include "DisplayClusterRootActor.h"

#include "IDetailChildrenBuilder.h"
#include "IPropertyUtilities.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"


#define LOCTEXT_NAMESPACE "FDisplayClusterConfiguratorMediaFullFrameCustomizationBase"


FDisplayClusterConfiguratorMediaFullFrameCustomizationBase::FDisplayClusterConfiguratorMediaFullFrameCustomizationBase()
{
	// Subscribe for auto-configure event
	FDisplayClusterConfiguratorMediaUtils::Get().OnMediaResetToDefaults().AddRaw(this, &FDisplayClusterConfiguratorMediaFullFrameCustomizationBase::OnAutoConfigureRequested);
}

FDisplayClusterConfiguratorMediaFullFrameCustomizationBase::~FDisplayClusterConfiguratorMediaFullFrameCustomizationBase()
{
	// Unsubscribe from auto-configure event
	FDisplayClusterConfiguratorMediaUtils::Get().OnMediaResetToDefaults().RemoveAll(this);
}

void FDisplayClusterConfiguratorMediaFullFrameCustomizationBase::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	if (CVarMediaAutoInitializationEnabled.GetValueOnGameThread())
	{
		// Subscribe for change callbacks
		if (MediaObjectHandle->IsValidHandle())
		{
			MediaObjectHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FDisplayClusterConfiguratorMediaFullFrameCustomizationBase::OnMediaObjectChanged));
		}
	}

	FDisplayClusterConfiguratorBaseTypeCustomization::CustomizeChildren(InPropertyHandle, InChildBuilder, InCustomizationUtils);
}

void FDisplayClusterConfiguratorMediaFullFrameCustomizationBase::ModifyMediaObjectParameters()
{
	// Nothing to do if disabled by a CVar
	if (!CVarMediaAutoInitializationEnabled.GetValueOnGameThread())
	{
		return;
	}

	// Validate object being edited
	if (!EditingObject.IsValid())
	{
		return;
	}

	UObject* Owner = EditingObject.Get();
	if (!MediaObjectHandle->IsValidHandle() || !Owner)
	{
		return;
	}

	// Get media object
	UObject* NewMediaObject = nullptr;
	if (MediaObjectHandle->GetValue(NewMediaObject) != FPropertyAccess::Success)
	{
		return;
	}

	if (!NewMediaObject)
	{
		return;
	}

	// Find a suitable initializer, and let it process new media data
	const TArray<IDisplayClusterModularFeatureMediaInitializer*> MediaInitializers = FDisplayClusterConfiguratorMediaUtils::Get().GetMediaInitializers();
	for (IDisplayClusterModularFeatureMediaInitializer* Initializer : MediaInitializers)
	{
		if (Initializer && Initializer->IsMediaObjectSupported(NewMediaObject))
		{
			FScopedTransaction Transaction(LOCTEXT("ResetMediaSettings", "Reset Media Settings"));

			NewMediaObject->Modify();

			PerformMediaInitialization(Owner, NewMediaObject, Initializer);

			break;
		}
	}
}

bool FDisplayClusterConfiguratorMediaFullFrameCustomizationBase::PerformMediaInitialization(UObject* Owner, UObject* MediaObject, IDisplayClusterModularFeatureMediaInitializer* Initializer)
{
	FMediaObjectOwnerInfo OwnerInfo;
	const bool bGotOwnerData = GetOwnerData(Owner, OwnerInfo);

	if (bGotOwnerData && Initializer)
	{
		Initializer->InitializeMediaObjectForFullFrame(MediaObject, OwnerInfo);
		return true;
	}

	return false;
}

bool FDisplayClusterConfiguratorMediaFullFrameCustomizationBase::GetOwnerData(const UObject* Owner, FMediaObjectOwnerInfo& OutOwnerInfo) const
{
	if (const UDisplayClusterICVFXCameraComponent* ICVFXCameraComponent = Cast<UDisplayClusterICVFXCameraComponent>(Owner))
	{
		return GetOwnerData(ICVFXCameraComponent, OutOwnerInfo);
	}
	else if (const UDisplayClusterConfigurationViewport* ViewportCfg = Cast<UDisplayClusterConfigurationViewport>(Owner))
	{
		return GetOwnerData(ViewportCfg, OutOwnerInfo);
	}
	else if (const UDisplayClusterConfigurationClusterNode* NodeCfg = Cast<UDisplayClusterConfigurationClusterNode>(Owner))
	{
		return GetOwnerData(NodeCfg, OutOwnerInfo);
	}
	else
	{
		checkNoEntry();
	}

	return false;
}

bool FDisplayClusterConfiguratorMediaFullFrameCustomizationBase::GetOwnerData(const UDisplayClusterICVFXCameraComponent* ICVFXCameraComponent, FMediaObjectOwnerInfo& OutOwnerInfo) const
{
	if (const AActor* const OwningActor = GetOwningActor())
	{
		// Get all camera components
		TArray<UDisplayClusterICVFXCameraComponent*> ICVFXCameras;
		OwningActor->GetComponents(ICVFXCameras);

		const int32 CamerasAmount = ICVFXCameras.Num();
		if (CamerasAmount > 0)
		{
			// Sort by name to always keep the same ABC-order
			ICVFXCameras.Sort([](const UDisplayClusterICVFXCameraComponent& LHS, const UDisplayClusterICVFXCameraComponent& RHS)
				{
					return LHS.GetName().Compare(RHS.GetName(), ESearchCase::IgnoreCase) < 0;
				});

			FString OrigCameraName = ICVFXCameraComponent->GetName();
			OrigCameraName.RemoveFromEnd(TEXT("_GEN_VARIABLE"));

			// Find camera index in the list
			for (uint8 CameraIdx = 0; CameraIdx < CamerasAmount; ++CameraIdx)
			{
				if (ICVFXCameras[CameraIdx]->GetName().Equals(OrigCameraName, ESearchCase::IgnoreCase))
				{
					OutOwnerInfo.OwnerType = FMediaObjectOwnerInfo::EMediaObjectOwnerType::ICVFXCamera;
					OutOwnerInfo.OwnerName = OrigCameraName;
					OutOwnerInfo.OwnerUniqueIdx = CameraIdx;

					return true;
				}
			}
		}
	}

	return false;
}

bool FDisplayClusterConfiguratorMediaFullFrameCustomizationBase::GetOwnerData(const UDisplayClusterConfigurationViewport* ViewportCfg, FMediaObjectOwnerInfo& OutOwnerInfo) const
{
	// Get cluster node that holds the viewport requested
	if (const UDisplayClusterConfigurationClusterNode* const ClusterNodeCfg = ViewportCfg->GetTypedOuter<UDisplayClusterConfigurationClusterNode>())
	{
		// Find the pair that holds the viewport requested
		for (const TPair<FString, TObjectPtr<UDisplayClusterConfigurationViewport>>& ViewportIt : ClusterNodeCfg->Viewports)
		{
			if (ViewportIt.Value == ViewportCfg)
			{
				// Get ABC-sorted list of the viewport IDs
				TArray<FString> ViewportIds;
				ClusterNodeCfg->Viewports.GenerateKeyArray(ViewportIds);
				ViewportIds.Sort();

				// Find viewport index
				for (int32 ViewportIdx = 0; ViewportIdx < ViewportIds.Num(); ++ViewportIdx)
				{
					if (ViewportIds[ViewportIdx].Equals(ViewportIt.Key, ESearchCase::IgnoreCase))
					{
						checkSlow(ViewportIdx < TNumericLimits<uint8>::Max());

						OutOwnerInfo.OwnerType = FMediaObjectOwnerInfo::EMediaObjectOwnerType::Viewport;
						OutOwnerInfo.OwnerName = ViewportIt.Key;
						OutOwnerInfo.OwnerUniqueIdx = static_cast<uint8>(ViewportIdx);

						// Here we leverage cluster node data provider to get unique cluster node index
						{
							FMediaObjectOwnerInfo TempNodeInfo;
							if (GetOwnerData(ClusterNodeCfg, TempNodeInfo))
							{
								OutOwnerInfo.ClusterNodeUniqueIdx = TempNodeInfo.ClusterNodeUniqueIdx;
							}
						}

						return true;
					}
				}
			}
		}
	}

	return false;
}

bool FDisplayClusterConfiguratorMediaFullFrameCustomizationBase::GetOwnerData(const UDisplayClusterConfigurationClusterNode* NodeCfg, FMediaObjectOwnerInfo& OutOwnerInfo) const
{
	// Get cluster object holding the node requested
	if (const UDisplayClusterConfigurationCluster* const ClusterCfg = NodeCfg->GetTypedOuter<UDisplayClusterConfigurationCluster>())
	{
		// Find the pair that holds the node requested
		for (const TPair<FString, TObjectPtr<UDisplayClusterConfigurationClusterNode>>& NodeIt : ClusterCfg->Nodes)
		{
			if (NodeIt.Value == NodeCfg)
			{
				// Get ABC-sorted list of the cluster nodes
				TArray<FString> NodeIds;
				ClusterCfg->Nodes.GenerateKeyArray(NodeIds);
				NodeIds.Sort();

				// Find node index
				for (int32 NodeIdx = 0; NodeIdx < ClusterCfg->Nodes.Num(); ++NodeIdx)
				{
					if (NodeIds[NodeIdx].Equals(NodeIt.Key, ESearchCase::IgnoreCase))
					{
						checkSlow(NodeIdx < TNumericLimits<uint8>::Max());

						OutOwnerInfo.OwnerType = FMediaObjectOwnerInfo::EMediaObjectOwnerType::Backbuffer;
						OutOwnerInfo.OwnerName = NodeIt.Key;
						OutOwnerInfo.OwnerUniqueIdx = static_cast<uint8>(NodeIdx);
						OutOwnerInfo.ClusterNodeUniqueIdx = static_cast<uint8>(NodeIdx);

						return true;
					}
				}
			}
		}
	}

	return false;
}

AActor* FDisplayClusterConfiguratorMediaFullFrameCustomizationBase::GetOwningActor() const
{
	if (UDisplayClusterICVFXCameraComponent* ICVFXCameraComponent = Cast<UDisplayClusterICVFXCameraComponent>(EditingObject))
	{
		// For instances
		if (AActor* Actor = ICVFXCameraComponent->GetOwner())
		{
			return Actor;
		}
		// For DCRA configurator
		else if (FDisplayClusterConfiguratorBlueprintEditor* BlueprintEditor = FDisplayClusterConfiguratorUtils::GetBlueprintEditorForObject(ICVFXCameraComponent))
		{
			return BlueprintEditor->GetPreviewActor();
		}
	}

	return nullptr;
}

void FDisplayClusterConfiguratorMediaFullFrameCustomizationBase::OnMediaObjectChanged()
{
	ModifyMediaObjectParameters();
}

void FDisplayClusterConfiguratorMediaFullFrameCustomizationBase::OnAutoConfigureRequested(UObject* InEditingObject)
{
	ModifyMediaObjectParameters();
}


//
// Common customization
//

void FDisplayClusterConfiguratorMediaTileCustomizationBase::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	if (CVarMediaAutoInitializationEnabled.GetValueOnGameThread())
	{
		// Subscribe for change callbacks
		if (TilePosHandle->IsValidHandle())
		{
			TilePosHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FDisplayClusterConfiguratorMediaTileCustomizationBase::OnTilePositionChanged));
			TilePosHandle->SetOnChildPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FDisplayClusterConfiguratorMediaTileCustomizationBase::OnTilePositionChanged));
		}
	}

	FDisplayClusterConfiguratorMediaFullFrameCustomizationBase::CustomizeChildren(InPropertyHandle, InChildBuilder, InCustomizationUtils);
}

bool FDisplayClusterConfiguratorMediaTileCustomizationBase::PerformMediaInitialization(UObject* Owner, UObject* MediaObject, IDisplayClusterModularFeatureMediaInitializer* Initializer)
{
	FMediaObjectOwnerInfo OwnerInfo;
	const bool bGotOwnerData = GetOwnerData(Owner, OwnerInfo);

	if (bGotOwnerData)
	{
		// Get input tile position
		const FIntPoint TilePos = GetEditedTilePos();
		if (TilePos.X != FIntPoint::NoneValue.X && TilePos.Y != FIntPoint::NoneValue.Y)
		{
			if (Initializer)
			{
				Initializer->InitializeMediaObjectForTile(MediaObject, OwnerInfo, TilePos);
				return true;
			}
		}
	}

	return false;
}

FIntPoint FDisplayClusterConfiguratorMediaTileCustomizationBase::GetEditedTilePos() const
{
	FIntPoint TilePos = FIntPoint::NoneValue;

	// Children 0 and 1 correspond to X and Y subproperties of FIntPoint
	TilePosHandle->GetChildHandle(0)->GetValue(TilePos.X);
	TilePosHandle->GetChildHandle(1)->GetValue(TilePos.Y);

	return TilePos;
}

void FDisplayClusterConfiguratorMediaTileCustomizationBase::OnTilePositionChanged()
{
	ModifyMediaObjectParameters();
}

#undef LOCTEXT_NAMESPACE
