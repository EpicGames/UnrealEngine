// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Views/Details/DisplayClusterConfiguratorBaseTypeCustomization.h"

#include "IDisplayClusterModularFeatureMediaInitializer.h"

class UDisplayClusterConfigurationClusterNode;
class UDisplayClusterConfigurationViewport;
class UDisplayClusterICVFXCameraComponent;


/**
 * Base class for full frame media input & output customization
 */
class FDisplayClusterConfiguratorMediaFullFrameCustomizationBase
	: public FDisplayClusterConfiguratorBaseTypeCustomization
{
public:
	FDisplayClusterConfiguratorMediaFullFrameCustomizationBase();
	~FDisplayClusterConfiguratorMediaFullFrameCustomizationBase();

protected:

	//~ Begin IPropertyTypeCustomization
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	//~ End IPropertyTypeCustomization

protected:

	/** Entry point to modify media objects. */
	void ModifyMediaObjectParameters();

	/** Virtual media initialization depending on media type (full-frame, split, etc.) */
	virtual bool PerformMediaInitialization(UObject* Owner, UObject* MediaObject, IDisplayClusterModularFeatureMediaInitializer* Initializer);

	/** Returns the actor that owns the object being edited. */
	AActor* GetOwningActor() const;

	/** Returns the name of media object's owner. */
	bool GetOwnerData(const UObject* Owner, FMediaObjectOwnerInfo& OutOwnerInfo) const;

private:

	/** Helper data provider for ICVFX cameras. */
	bool GetOwnerData(const UDisplayClusterICVFXCameraComponent* ICVFXCameraComponent, FMediaObjectOwnerInfo& OutOwnerInfo) const;

	/** Helper data provider for viewports. */
	bool GetOwnerData(const UDisplayClusterConfigurationViewport* ViewportCfg, FMediaObjectOwnerInfo& OutOwnerInfo) const;

	/** Helper data provider for cluster nodes. */
	bool GetOwnerData(const UDisplayClusterConfigurationClusterNode* NodeCfg, FMediaObjectOwnerInfo& OutOwnerInfo) const;

private:

	/** Handles media source/output change callbacks */
	void OnMediaObjectChanged();

	/** Auto-configuration requests handler */
	void OnAutoConfigureRequested(UObject* EditingObject);

protected:

	/** MediaSource or MediaOutput property handle, depending on the child implementation. */
	TSharedPtr<IPropertyHandle> MediaObjectHandle;
};



/**
 * Base class for tiled media input & output customization
 */
class FDisplayClusterConfiguratorMediaTileCustomizationBase
	: public FDisplayClusterConfiguratorMediaFullFrameCustomizationBase
{
protected:

	//~ Begin IPropertyTypeCustomization
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	//~ End IPropertyTypeCustomization

protected:

	//~ Begin FDisplayClusterConfiguratorMediaFullFrameCustomizationBase

	/** Virtual media initialization depending on media type (full-frame, split, etc.) */
	virtual bool PerformMediaInitialization(UObject* Owner, UObject* MediaObject, IDisplayClusterModularFeatureMediaInitializer* Initializer) override;

	//~ End FDisplayClusterConfiguratorMediaFullFrameCustomizationBase

private:

	/** Returns tile position currently set. */
	FIntPoint GetEditedTilePos() const;

private:

	/** Handles tile position change callbacks */
	void OnTilePositionChanged();

protected:

	/** Tile position property handle. */
	TSharedPtr<IPropertyHandle> TilePosHandle;
};
