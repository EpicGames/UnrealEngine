// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/Details/Media/DisplayClusterConfiguratorMediaCustomizationCVars.h"


TAutoConsoleVariable<bool> CVarMediaAutoInitializationEnabled(
	TEXT("nDisplay.Media.MediaAutoInitialization"),
	true,
	TEXT("nDisplay tiled media auto-configuration\n")
	TEXT("0 : Disabled\n")
	TEXT("1 : Enabled\n"),
	ECVF_Default
);
