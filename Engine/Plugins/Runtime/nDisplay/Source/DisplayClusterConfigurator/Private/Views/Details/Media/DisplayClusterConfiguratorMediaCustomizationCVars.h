// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/IConsoleManager.h"

// Enable/disable live auto-configuration. Does not affect "auto-configure" button.
extern TAutoConsoleVariable<bool> CVarMediaAutoInitializationEnabled;
