// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/Details/Media/DisplayClusterConfiguratorMediaFullFrameCustomization.h"

#include "DisplayClusterConfigurationTypes_Media.h"

#include "IDetailChildrenBuilder.h"
#include "IPropertyUtilities.h"
#include "PropertyHandle.h"


//
// Input tiles
//
void FDisplayClusterConfiguratorMediaFullFrameInputCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	// MediaSource property
	MediaObjectHandle = GET_CHILD_HANDLE(FDisplayClusterConfigurationMediaInput, MediaSource);
	check(MediaObjectHandle->IsValidHandle());

	FDisplayClusterConfiguratorMediaFullFrameCustomizationBase::CustomizeChildren(InPropertyHandle, InChildBuilder, InCustomizationUtils);
}

//
// Output tiles
//
void FDisplayClusterConfiguratorMediaFullFrameOutputCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	// MediaOutput property
	MediaObjectHandle = GET_CHILD_HANDLE(FDisplayClusterConfigurationMediaOutput, MediaOutput);
	check(MediaObjectHandle->IsValidHandle());

	FDisplayClusterConfiguratorMediaFullFrameCustomizationBase::CustomizeChildren(InPropertyHandle, InChildBuilder, InCustomizationUtils);
}
