// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Views/Details/Media/DisplayClusterConfiguratorMediaCustomizationBase.h"


/**
 * Details panel customization for FDisplayClusterConfigurationMediaUniformTileInput struct (input tiles).
 */
class FDisplayClusterConfiguratorMediaFullFrameInputCustomization
	: public FDisplayClusterConfiguratorMediaFullFrameCustomizationBase
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FDisplayClusterConfiguratorMediaFullFrameInputCustomization>();
	}

protected:
	//~ Begin IPropertyTypeCustomization
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	//~ End IPropertyTypeCustomization
};


/**
 * Details panel customization for FDisplayClusterConfigurationMediaUniformTileOutput struct (output tiles).
 */
class FDisplayClusterConfiguratorMediaFullFrameOutputCustomization
	: public FDisplayClusterConfiguratorMediaFullFrameCustomizationBase
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FDisplayClusterConfiguratorMediaFullFrameOutputCustomization>();
	}

protected:
	//~ Begin IPropertyTypeCustomization
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	//~ End IPropertyTypeCustomization
};
