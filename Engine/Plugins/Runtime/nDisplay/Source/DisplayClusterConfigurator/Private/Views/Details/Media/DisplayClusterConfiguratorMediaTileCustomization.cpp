// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/Details/Media/DisplayClusterConfiguratorMediaTileCustomization.h"

#include "DisplayClusterConfigurationTypes_Media.h"

#include "IDetailChildrenBuilder.h"
#include "IPropertyUtilities.h"
#include "PropertyHandle.h"


//
// Input tiles
//
void FDisplayClusterConfiguratorMediaInputTileCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	// MediaSource property
	MediaObjectHandle = GET_CHILD_HANDLE(FDisplayClusterConfigurationMediaUniformTileInput, MediaSource);
	check(MediaObjectHandle->IsValidHandle());

	// Tile position property
	TilePosHandle = GET_CHILD_HANDLE(FDisplayClusterConfigurationMediaUniformTileInput, Position);
	check(TilePosHandle->IsValidHandle());

	FDisplayClusterConfiguratorMediaTileCustomizationBase::CustomizeChildren(InPropertyHandle, InChildBuilder, InCustomizationUtils);
}


//
// Output tiles
//
void FDisplayClusterConfiguratorMediaOutputTileCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	// MediaOutput property
	MediaObjectHandle = GET_CHILD_HANDLE(FDisplayClusterConfigurationMediaUniformTileOutput, MediaOutput);
	check(MediaObjectHandle->IsValidHandle());

	// Tile position property
	TilePosHandle = GET_CHILD_HANDLE(FDisplayClusterConfigurationMediaUniformTileOutput, Position);
	check(TilePosHandle->IsValidHandle());

	FDisplayClusterConfiguratorMediaTileCustomizationBase::CustomizeChildren(InPropertyHandle, InChildBuilder, InCustomizationUtils);
}
