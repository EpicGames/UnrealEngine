// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Views/Details/Media/DisplayClusterConfiguratorMediaCustomizationBase.h"


/**
 * Details panel customization for FDisplayClusterConfigurationMediaUniformTileInput struct (input tiles).
 */
class FDisplayClusterConfiguratorMediaInputTileCustomization
	: public FDisplayClusterConfiguratorMediaTileCustomizationBase
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FDisplayClusterConfiguratorMediaInputTileCustomization>();
	}

protected:
	//~ Begin IPropertyTypeCustomization
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	//~ End IPropertyTypeCustomization
};


/**
 * Details panel customization for FDisplayClusterConfigurationMediaUniformTileOutput struct (output tiles).
 */
class FDisplayClusterConfiguratorMediaOutputTileCustomization
	: public FDisplayClusterConfiguratorMediaTileCustomizationBase
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FDisplayClusterConfiguratorMediaOutputTileCustomization>();
	}

protected:
	//~ Begin IPropertyTypeCustomization
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	//~ End IPropertyTypeCustomization
};
