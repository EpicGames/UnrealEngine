// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/Details/Media/DisplayClusterConfiguratorMediaUtils.h"

#include "IDisplayClusterModularFeatureMediaInitializer.h"
#include "Features/IModularFeatures.h"


FDisplayClusterConfiguratorMediaUtils& FDisplayClusterConfiguratorMediaUtils::Get()
{
	static FDisplayClusterConfiguratorMediaUtils Instance;
	return Instance;
}

FDisplayClusterConfiguratorMediaUtils::FDisplayClusterConfiguratorMediaUtils()
{
	// Obtain media initializer modular features once. We don't expect any of them to be loaded dynamically in runtime.
	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	ModularFeatures.LockModularFeatureList();
	MediaInitializers = ModularFeatures.GetModularFeatureImplementations<IDisplayClusterModularFeatureMediaInitializer>(IDisplayClusterModularFeatureMediaInitializer::ModularFeatureName);
	ModularFeatures.UnlockModularFeatureList();
}
