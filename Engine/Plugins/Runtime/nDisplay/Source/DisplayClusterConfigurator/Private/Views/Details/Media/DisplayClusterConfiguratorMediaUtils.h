// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class IDisplayClusterModularFeatureMediaInitializer;

/**
 * Media customization utilities
 */
class FDisplayClusterConfiguratorMediaUtils
{
protected:
	FDisplayClusterConfiguratorMediaUtils();

public:

	/** Singleton access */
	static FDisplayClusterConfiguratorMediaUtils& Get();

public:

	/** Returns media initializers */
	const TArray<IDisplayClusterModularFeatureMediaInitializer*>& GetMediaInitializers() const
	{
		return MediaInitializers;
	}

	/** Tiled media auto-configuration event */
	DECLARE_EVENT_OneParam(FDisplayClusterConfiguratorMediaUtils, FResetToDefaultsEvent, UObject*);
	FResetToDefaultsEvent& OnMediaResetToDefaults()
	{
		return MediaResetToDefaultsEvent;
	}

private:

	/** Media initializers (modular features) available. */
	TArray<IDisplayClusterModularFeatureMediaInitializer*> MediaInitializers;

	/** Tiled media reset to defaults event */
	FResetToDefaultsEvent MediaResetToDefaultsEvent;
};
