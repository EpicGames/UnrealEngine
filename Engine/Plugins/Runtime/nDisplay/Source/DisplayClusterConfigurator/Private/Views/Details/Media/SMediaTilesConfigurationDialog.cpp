// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/Details/Media/SMediaTilesConfigurationDialog.h"
#include "Views/Details/Media/DisplayClusterConfiguratorMediaUtils.h"

#include "DisplayClusterConfiguratorLog.h"
#include "DisplayClusterConfiguratorStyle.h"

#include "IDisplayClusterModularFeatureMediaInitializer.h"

#include "ClassViewerFilter.h"
#include "ClassViewerModule.h"
#include "PropertyCustomizationHelpers.h"
#include "SClassViewer.h"

#include "MediaOutput.h"
#include "MediaSource.h"

#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "UObject/UObjectGlobals.h"

#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Workflow/SWizard.h"

#define LOCTEXT_NAMESPACE "SMediaTilesConfigurationDialog"


namespace UE::DisplayClusterConfigurator::MediaTilesConfigurationDialog::Private
{
	const FText TextDialogTitle = LOCTEXT("DialogTitle", "Media Tiles Configuration");

	// Page: Layout
	const FText TextPageLayoutHeader = LOCTEXT("PageTilesLayoutHeader", "Step 1: Choose split layout");

	// Page: Media
	const FText TextPageMediaHeader = LOCTEXT("PageMediaObjectsHeader", "Step 2: Configure template media source & output");
	const FText TextPageMediaSource = LOCTEXT("PageMediaSourceRowName", "Media Source:");
	const FText TextPageMediaOutput = LOCTEXT("PageMediaOutputRowName", "Media Output:");
	const FText TextPageMediaComboboxNone  = LOCTEXT("PageMediaMediaComboboxItemNone", "None");
	const FText TextPageMediaStatusOk      = LOCTEXT("PageMediaStatusOk", "Ok");
	const FText TextPageMediaStatusNotSupp = LOCTEXT("PageMediaStatusNotSupported", "Auto-configuration is not supported");
	const FText TextPageMediaNotCompatOrNotSupp = LOCTEXT("PageMediaStatusNotCompatible", "Not compatible or not supported");

	// Page: Nodes
	const FText TextPageNodesHeader = LOCTEXT("PageNodesHeader", "Step 3: Allot cluster nodes");
	const FText TextPageNodesSectionOutputHeader = LOCTEXT("PageNodesSectionOutputHeader", "Tile Senders");
	const FText TextPageNodesSectionInputHeader  = LOCTEXT("PageNodesSectionInputHeader", "Tile Receivers");
	const FText TextPageNodesMenuSelectAll       = LOCTEXT("PageNodesMenuSelectAll", "Select All");
	const FText TextPageNodesMenuDeselectAll     = LOCTEXT("PageNodesMenuDeselectAll", "Deselect All");
	const FText TextPageNodesMenuSelectAllNoSenders   = LOCTEXT("PageNodesMenuSelectAllNoSenders", "Select All w/o Senders");
	const FText TextPageNodesMenuSelectAllNoReceivers = LOCTEXT("PageNodesMenuSelectAllNoReceivers", "Select All w/o Receivers");
	const FText TextPageNodesMenuDeselectSenders      = LOCTEXT("PageNodesMenuSelectDeselectSenders", "Deselect Senders");
	const FText TextPageNodesMenuDeselectReceivers    = LOCTEXT("PageNodesMenuSelectDeselectReceivers", "Deselect Receivers");

	// Page: Finalization
	const FText TextPageFinalizationHeader = LOCTEXT("PageFinalizationHeader", "Step 4: Output mapping (tile senders)");

	// Feedback message
	const FText TextMsgConfigurationFeedbackBegin = LOCTEXT("MsgConfigurationFeedbackBegin", "Some issues were found during the configuration process:");
	const FText TextMsgConfigurationFeedbackEnd   = LOCTEXT("MsgConfigurationFeedbackEnd",   "Ignore and proceed (Yes) or return and fix media configuration (No)?");
}

void SMediaTilesConfigurationDialog::Construct(const FArguments& InArgs, const FMediaTilesConfigurationDialogParameters& InParameters)
{
	InParameters.Validate();

	// Save input parameters first
	Parameters = InParameters;

	// Initialize internals
	InitializeInternals();

	// And construct the dialog widget
	SWindow::Construct(SWindow::FArguments()
		.Title(UE::DisplayClusterConfigurator::MediaTilesConfigurationDialog::Private::TextDialogTitle)
		.SizingRule(ESizingRule::Autosized)
		.SupportsMaximize(false)
		.SupportsMinimize(false)
		[
			SNew(SVerticalBox)

				// Current page hint
				+SVerticalBox::Slot()
				.Padding(10, 10, 10, 5)
				.AutoHeight()
				[
					SAssignNew(PageHint, STextBlock)
					.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
				]

				// Page body
				+SVerticalBox::Slot()
				.Padding(10, 5, 10, 10)
				.AutoHeight()
				[
					SAssignNew(Wizard, SWizard)
					.OnCanceled(this, &SMediaTilesConfigurationDialog::OnCancelButtonClicked)
					.OnFinished(this, &SMediaTilesConfigurationDialog::OnFinishButtonClicked)
					.CanFinish(this, &SMediaTilesConfigurationDialog::IsFinishButtonEnabled)
					.ShowPageList(false)

						// Page: Layout
						+SWizard::Page()
						.CanShow(true)
						.OnEnter(this, &SMediaTilesConfigurationDialog::PageLayout_OnEnter)
						[
							PageLayout_Build()
						]

						// Page: Media
						+SWizard::Page()
						.CanShow(this, &SMediaTilesConfigurationDialog::PageMedia_OnCanShow)
						.OnEnter(this, &SMediaTilesConfigurationDialog::PageMedia_OnEnter)
						[
							PageMedia_Build()
						]

						// Page: Nodes
						+SWizard::Page()
						.CanShow(this, &SMediaTilesConfigurationDialog::PageNodes_OnCanShow)
						.OnEnter(this, &SMediaTilesConfigurationDialog::PageNodes_OnEnter)
						[
							PageNodes_Build()
						]

						// Page: Finalization
						+SWizard::Page()
						.CanShow(this, &SMediaTilesConfigurationDialog::PageFinalization_OnCanShow)
						.OnEnter(this, &SMediaTilesConfigurationDialog::PageFinalization_OnEnter)
						[
							PageFinalization_Build()
						]
				]
		]);
}

bool SMediaTilesConfigurationDialog::WasConfigurationCompleted() const
{
	return bConfigurationCompleted;
}

void SMediaTilesConfigurationDialog::InitializeInternals()
{
	// Pre-save some data to simplify future use
	for (const TPair<FString, TObjectPtr<UDisplayClusterConfigurationClusterNode>>& Node : Parameters.ConfigData->Cluster->Nodes)
	{
		if (Node.Value)
		{
			// All nodes
			ClusterNodeIds.Add(Node.Key);
			// Node to host
			NodeToHostMap.Emplace(Node.Key, Node.Value->Host);
			// Nodes per host
			HostToNodesMap.FindOrAdd(Node.Value->Host).Add(Node.Key);
			// Nodes allowed to be used for tile rendering
			NodesAllowedForOutput.Add(Node.Key);
			// Nodes allowed to be used for tile receiving and compositing
			NodesAllowedForInput.Add(Node.Key);

			// Is offscreen?
			if (Node.Value->bRenderHeadless)
			{
				OffscreenNodes.Add(Node.Key);
			}
		}
	}

	// Fill output mapping with defaults
	for (int32 TileX = 0; TileX < MaxTilesAmount; ++TileX)
	{
		for (int32 TileY = 0; TileY < MaxTilesAmount; ++TileY)
		{
			FOutputMappingData& TileData = OutputMapping.Emplace({TileX, TileY});
			TileData.ClusterNodes.Reserve(NodesAllowedForOutput.Num());
		}
	}
}

bool SMediaTilesConfigurationDialog::IsFinishButtonEnabled() const
{
	// Check if all settings are good
	const bool bConfigValid = PageLayout_IsConfigurationValid() && PageMedia_IsConfigurationValid() && PageNodes_IsConfigurationValid() && PageFinalization_IsConfigurationValid();
	// Make sure users have seen the final configuration on the last page
	const bool bOnLastPage = Wizard->GetCurrentPageIndex() == Wizard->GetNumPages() - 1;

	return bConfigValid && bOnLastPage;
}

void SMediaTilesConfigurationDialog::OnFinishButtonClicked()
{
	TArray<FString> Errors;
	TArray<FString> Warnings;

	bConfigurationCompleted = ApplyConfiguration(Errors, Warnings);

	// Any warnings/errors/notes?
	if (bConfigurationCompleted && (!Errors.IsEmpty() || !Warnings.IsEmpty()))
	{
		FText Message;
		GenerateIssuesFoundMessage(Errors, Warnings, Message);

		// Notify user
		const EAppReturnType::Type Result = FMessageDialog::Open(EAppMsgType::YesNo, Message);

		// User decided to return to configuration. So we don't close this configuration dialog.
		if (Result == EAppReturnType::No)
		{
			return;
		}
	}

	// Close this window
	RequestDestroyWindow();
}

void SMediaTilesConfigurationDialog::OnCancelButtonClicked()
{
	bConfigurationCompleted = false;
	RequestDestroyWindow();
}

bool SMediaTilesConfigurationDialog::ApplyConfiguration(TArray<FString>& OutErrors, TArray<FString>& OutWarnings)
{
	if (!Parameters.Validate())
	{
		return false;
	}

	// Redirect to a proper handler
	if (EnumHasAnyFlags(MediaPropagationTypes, EMediaStreamPropagationType::Multicast))
	{
		return ApplyConfiguration_Multicast(OutErrors, OutWarnings);
	}
	else if (EnumHasAnyFlags(MediaPropagationTypes, EMediaStreamPropagationType::LocalMulticast))
	{
		return ApplyConfiguration_LocalMulticast(OutErrors, OutWarnings);
	}
	else
	{
		return false;
	}
}

bool SMediaTilesConfigurationDialog::ApplyConfiguration_Multicast(TArray<FString>& OutErrors, TArray<FString>& OutWarnings)
{
	// In multicast, the data is propagated in OneSender-to-MultipleReceivers way. Thus we
	// can have a single input group with all the receivers because they get same data from
	// the same senders. Each receiver gets the full set of tiles, so it's connected to all
	// the receivers. This allows for all of them to share a single MediaSource for each tile.
	// There is no limitation on the receivers amount.
	// 
	// As for the senders, we don't actually need more than total amount of tiles. Each tile
	// is produced by a dedicated sender and propagated to all the receivers. Based on
	// this we're going to have a dedicated output group for every tile/sender.
	//
	// INPUT:
	// [InputGroup]
	//   Nodes: Receiver0, Receiver1, ..., ReceiverM
	//   Tiles: 0x0, 0x1, ..., AxB
	//
	// OUTPUT:
	// [OutputGroup0]
	//   Nodes: Sender0
	//   Tiles: 0x0
	// [OutputGroup1]
	//   Nodes: Sender1
	//   Tiles: 0x1
	// ...
	// [OutputGroupN]
	//   Nodes: SenderN
	//   Tiles: AxB
	//
	// Where
	//  - A - tiles amount horizontally
	//  - B - tiles amount vertically
	//  - N = A * B == amount of senders
	//  - M - any amount of receivers
	//
	// ==================================================================================================
	// HOWEVER!
	// Currently, nDisplay doesn't allow any loopback-like setups. So it's not allowed the same node
	// to output a tile, and consume the same tile. Also it's not allowed to have both output and
	// input assigned to the same tile (passthrough-like). This requires us to remove any input mapping
	// from the tiles that have already output assigned on the same node. To simplify final configuration,
	// we'll have a separate input group for each node. Each group will have the full set of tiles that
	// follow the limitations mentioned above.
	// Hope it's temporary and we can get back to a single input group soon.
	// ==================================================================================================

	// Apply tile layout
	*Parameters.SplitLayout = Accepted + FIntPoint{ 1, 1 };

	// Object flags for new media source/output objects
	const EObjectFlags MediaObjectFlags = Parameters.Owner->IsInBlueprint() ?
		RF_Public | RF_Transactional | RF_ArchetypeObject :
		RF_Public | RF_Transactional;

	const int32 TilesAmount = Parameters.SplitLayout->X * Parameters.SplitLayout->Y;

	//
	// INPUT setup
	//

	if (InputSelection.IsEmpty())
	{
		OutErrors.Add(TEXT("No receivers found"));
	}

	// For each receiver, create a separate input group
	Parameters.InputGroups->Reset(InputSelection.Num());
	for (const FString& ReceiverId : InputSelection)
	{
		FDisplayClusterConfigurationMediaTiledInputGroup& NewInputGroup = Parameters.InputGroups->AddDefaulted_GetRef();

		// One receiver per group
		NewInputGroup.ClusterNodes.ItemNames.Add(ReceiverId);

		// Setup input tiles
		for (int32 TileX = 0; TileX < Parameters.SplitLayout->X; ++TileX)
		{
			for (int32 TileY = 0; TileY < Parameters.SplitLayout->Y; ++TileY)
			{
				const FIntPoint Tile{ TileX, TileY };

				// Don't allow 'loopback'
				const bool bReceiverHasOutputAssignedForThisTile = OutputMapping[Tile].ClusterNodes.Contains(ReceiverId);
				if (!bReceiverHasOutputAssignedForThisTile)
				{
					FDisplayClusterConfigurationMediaUniformTileInput& NewTile = NewInputGroup.Tiles.AddDefaulted_GetRef();

					NewTile.Position = Tile;
					NewTile.MediaSource = NewObject<UMediaSource>(Parameters.Owner, MediaSource->GetClass(), NAME_None, MediaObjectFlags);
				}
			}
		}
	}

	//
	// OUTPUT setup
	//

	// First, remove any existing output mapping
	Parameters.OutputGroups->Reset(TilesAmount);

	// For each tile, we create an output group with a corresponding sender node
	for (int32 TileX = 0; TileX < Parameters.SplitLayout->X; ++TileX)
	{
		for (int32 TileY = 0; TileY < Parameters.SplitLayout->Y; ++TileY)
		{
			// Create new output group
			FDisplayClusterConfigurationMediaTiledOutputGroup& NewOutputGroup = Parameters.OutputGroups->AddDefaulted_GetRef();

			// Add cluster nodes bound to this tile
			const FIntPoint Tile{ TileX, TileY };
			const int32 SenderNodesAmount = OutputMapping[Tile].ClusterNodes.Num();

			if (SenderNodesAmount >= 1)
			{
				const FString& SenderNodeId = *OutputMapping[Tile].ClusterNodes.CreateConstIterator();
				NewOutputGroup.ClusterNodes.ItemNames.Add(SenderNodeId);

				// Normally we should have a single sender
				if (SenderNodesAmount > 1)
				{
					OutWarnings.Add(FString::Printf(TEXT("Too many (%d) senders found for tile '%dx%d'. A single one is set up: %s"),
						SenderNodesAmount, TileX, TileY, *SenderNodeId));
				}
			}
			else
			{
				OutErrors.Add(FString::Printf(TEXT("No senders found for tile '%dx%d'"), TileX, TileY));
			}

			// Create new tile in this group
			FDisplayClusterConfigurationMediaUniformTileOutput& NewTileInGroup = NewOutputGroup.Tiles.AddDefaulted_GetRef();
			NewTileInGroup.Position = Tile;
			NewTileInGroup.MediaOutput = NewObject<UMediaOutput>(Parameters.Owner, MediaOutput->GetClass(), NAME_None, MediaObjectFlags);
		}
	}

	return true;
}

bool SMediaTilesConfigurationDialog::ApplyConfiguration_LocalMulticast(TArray<FString>& OutErrors, TArray<FString>& OutWarnings)
{
	// Local multicast implies every host produces and consumes its own set of tiles. This means we can
	// group inputs and outputs per every host like this:
	//
	// [Host A]
	//   - SenderA1,   SenderA2,   ..., SenderAN    <== separate output groups for Host A
	//   - ReceiverA1, ReceiverA2, ..., ReceiverAN  <== single input group for Host A
	// [Host B]
	//   - SenderB1,   SenderB2,   ..., SenderBN    <== separate output groups for Host A
	//   - ReceiverB1, ReceiverB2, ..., ReceiverBN  <== single input group for Host A
	//
	// In other words. Assuming there are N tiles, each host that has at least one tile
	// receiver must also have N tile senders (or less, but some senders would have to render
	// multiple tiles in this case).
	//
	// ==================================================================================================
	// HOWEVER!
	// Currently, nDisplay doesn't allow any loopback-like setups. So it's not allowed the same node
	// to output a tile, and consume the same tile. Also it's not allowed to have both output and
	// input assigned to the same tile (passthrough-like). This requires us to remove any input mapping
	// from the tiles that have already output assigned on the same node. To simplify final configuration,
	// we'll have a separate input group for each node. Each group will have the full set of tiles that
	// follow the limitations mentioned above.
	// Hope it's temporary and we can get back to a single input group soon.
	// ==================================================================================================

	// Apply tile layout
	*Parameters.SplitLayout = Accepted + FIntPoint{ 1, 1 };

	// All receivers involved
	const TSet<FString>& AllReceivers = InputSelection;

	// All senders involved
	TSet<FString> AllSenders;
	for (const TPair<FIntPoint, FOutputMappingData>& TileData : OutputMapping)
	{
		AllSenders.Append(TileData.Value.ClusterNodes);
	}

	// All nodes (senders and receivers) being used in tiled media
	TSet<FString> HostsWithMedia;
	{
		const TSet<FString> AllNodes = AllSenders.Union(AllReceivers);
		for (const FString& NodeId : AllNodes)
		{
			if (const FString* Host = NodeToHostMap.Find(NodeId))
			{
				HostsWithMedia.Add(*Host);
			}
		}
	}

	// Object flags for new media source/output objects
	const EObjectFlags MediaObjectFlags = Parameters.Owner->IsInBlueprint() ?
		RF_Public | RF_Transactional | RF_ArchetypeObject :
		RF_Public | RF_Transactional;

	const int32 TilesAmount = Parameters.SplitLayout->X * Parameters.SplitLayout->Y;

	// Reset any existing data
	Parameters.InputGroups->Reset(AllReceivers.Num());
	Parameters.OutputGroups->Reset(HostsWithMedia.Num() * TilesAmount);

	// Now, we can generate per-receiver input groups and per-tile output groups for every host.
	for (const FString& Host : HostsWithMedia)
	{
		//
		// INPUT setup
		//

		// Find all receivers on this particular host
		const TSet<FString> ReceiversOnThisHost = HostToNodesMap[Host].Intersect(AllReceivers);

		// We expect at least one to be set
		if (ReceiversOnThisHost.IsEmpty())
		{
			OutErrors.Add(FString::Printf(TEXT("No receivers found on host '%s'"), *Host));
		}

		// For each receiver on this host, add a new input group
		for (const FString& ReceiverId : ReceiversOnThisHost)
		{
			// Add input group
			FDisplayClusterConfigurationMediaTiledInputGroup& NewInputGroup = Parameters.InputGroups->AddDefaulted_GetRef();

			// Assing all the receivers on this host to the group
			NewInputGroup.ClusterNodes.ItemNames.Add(ReceiverId);

			// Setup input tiles (setup all the tiles in the group)
			for (int32 TileX = 0; TileX < Parameters.SplitLayout->X; ++TileX)
			{
				for (int32 TileY = 0; TileY < Parameters.SplitLayout->Y; ++TileY)
				{
					const FIntPoint Tile{ TileX, TileY };

					// Don't allow 'loopback'
					const bool bReceiverHasOutputAssignedForThisTile = OutputMapping[Tile].ClusterNodes.Contains(ReceiverId);
					if (!bReceiverHasOutputAssignedForThisTile)
					{
						FDisplayClusterConfigurationMediaUniformTileInput& NewTile = NewInputGroup.Tiles.AddDefaulted_GetRef();

						NewTile.Position = Tile;
						NewTile.MediaSource = NewObject<UMediaSource>(Parameters.Owner, MediaSource->GetClass(), NAME_None, MediaObjectFlags);
					}
				}
			}
		}

		//
		// OUTPUT setup
		//

		// For each tile, we create an output group with all the nodes bound to this tile
		for (int32 TileX = 0; TileX < Parameters.SplitLayout->X; ++TileX)
		{
			for (int32 TileY = 0; TileY < Parameters.SplitLayout->Y; ++TileY)
			{
				const FIntPoint Tile{ TileX, TileY };

				// Create new output group
				FDisplayClusterConfigurationMediaTiledOutputGroup& NewOutputGroup = Parameters.OutputGroups->AddDefaulted_GetRef();

				// Get all senders of this tile on this particualar host
				const TSet<FString>& AllSendersMappedToThisTile = OutputMapping[Tile].ClusterNodes;
				const TSet<FString>  AllSendersOnThisHost = AllSenders.Intersect(HostToNodesMap[Host]);
				const TSet<FString>  SendersOfThisTile = AllSendersOnThisHost.Intersect(AllSendersMappedToThisTile);
				
				// Find a sender for this tile
				const int32 AmountOfSendersAssignedForThisTile = SendersOfThisTile.Num();
				if (AmountOfSendersAssignedForThisTile >= 1)
				{
					const FString& SenderNodeId = *SendersOfThisTile.CreateConstIterator();
					NewOutputGroup.ClusterNodes.ItemNames.Add(SenderNodeId);

					// Normally we should have a single sender
					if (AmountOfSendersAssignedForThisTile > 1)
					{
						OutWarnings.Add(FString::Printf(TEXT("Too many (%d) senders found for tile '%dx%d' on host '%s'"),
							AmountOfSendersAssignedForThisTile, TileX, TileY, *Host));
					}
				}
				else
				{
					OutErrors.Add(FString::Printf(TEXT("No senders found for tile '%dx%d' on host '%s'"),
						TileX, TileY, *Host));
				}

				// Create new tile in this group
				FDisplayClusterConfigurationMediaUniformTileOutput& NewTileInGroup = NewOutputGroup.Tiles.AddDefaulted_GetRef();
				NewTileInGroup.Position = Tile;
				NewTileInGroup.MediaOutput = NewObject<UMediaOutput>(Parameters.Owner, MediaOutput->GetClass(), NAME_None, MediaObjectFlags);
			}
		}
	}

	return true;
}

void SMediaTilesConfigurationDialog::GenerateIssuesFoundMessage(const TArray<FString>& Errors, const TArray<FString>& Warnings, FText& OutMessage) const
{
	FString Temp;
	Temp.Reserve(512);

	Temp += UE::DisplayClusterConfigurator::MediaTilesConfigurationDialog::Private::TextMsgConfigurationFeedbackBegin.ToString();
	Temp += TEXT("\n\n");

	if (!Errors.IsEmpty())
	{
		Temp += TEXT("Errors:\n");
		for (const FString& Message : Errors)
		{
			Temp += FString::Printf(TEXT("- %s\n"), *Message);
		}
	}

	if (!Warnings.IsEmpty())
	{
		Temp += TEXT("Warnings:\n");
		for (const FString& Message : Warnings)
		{
			Temp += FString::Printf(TEXT("- %s\n"), *Message);
		}
	}

	Temp += TEXT("\n\n");
	Temp += UE::DisplayClusterConfigurator::MediaTilesConfigurationDialog::Private::TextMsgConfigurationFeedbackEnd.ToString();

	OutMessage = FText::FromString(Temp);
}

bool SMediaTilesConfigurationDialog::IsMediaObjectSupported(const UObject* MediaObject) const
{
	const TArray<IDisplayClusterModularFeatureMediaInitializer*> MediaInitializers = FDisplayClusterConfiguratorMediaUtils::Get().GetMediaInitializers();
	const bool bSupported = MediaInitializers.ContainsByPredicate([MediaObject](IDisplayClusterModularFeatureMediaInitializer* Initializer)
		{
			return Initializer ? Initializer->IsMediaObjectSupported(MediaObject) : false;
		});

	return bSupported;
}

bool SMediaTilesConfigurationDialog::AreMediaObjectsCompatible(const UObject* InMediaSource, const UObject* InMediaOutput) const
{
	const TArray<IDisplayClusterModularFeatureMediaInitializer*> MediaInitializers = FDisplayClusterConfiguratorMediaUtils::Get().GetMediaInitializers();
	const bool bCompatible = MediaInitializers.ContainsByPredicate([InMediaSource, InMediaOutput](IDisplayClusterModularFeatureMediaInitializer* Initializer)
		{
			return Initializer ? Initializer->AreMediaObjectsCompatible(InMediaSource, InMediaOutput) : false;
		});

	return bCompatible;
}

bool SMediaTilesConfigurationDialog::GetMediaPropagationTypes(const UObject* InMediaSource, const UObject* InMediaOutput, EMediaStreamPropagationType& OutPropagationFlags) const
{
	// Find appropriate media initializer and get media type
	const TArray<IDisplayClusterModularFeatureMediaInitializer*>& MediaInitializers = FDisplayClusterConfiguratorMediaUtils::Get().GetMediaInitializers();
	for (IDisplayClusterModularFeatureMediaInitializer* MediaInitializer : MediaInitializers)
	{
		// Find corresponding media initializer, and get its media propagation compatibilities
		if (MediaInitializer && MediaInitializer->GetSupportedMediaPropagationTypes(InMediaSource, InMediaOutput, OutPropagationFlags))
		{
			return true;
		}
	}

	return false;
}


/**
 * Layout page
 */
TSharedRef<SWidget> SMediaTilesConfigurationDialog::PageLayout_Build()
{
	TSharedRef<SUniformGridPanel> LayoutGrid = SNew(SUniformGridPanel)
		.SlotPadding(1)
		.MinDesiredSlotWidth(LayoutGridCellSize)
		.MinDesiredSlotHeight(LayoutGridCellSize);

	// Build the grid
	for (int32 TileX = 0; TileX < MaxTilesAmount; ++TileX)
	{
		for (int32 TileY = 0; TileY < MaxTilesAmount; ++TileY)
		{
			// Current tile
			const FIntPoint Tile = { TileX, TileY };

			// Instantiate button at XY
			TSharedRef<SButton> GridButton = SNew(SButton)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Text(FText::FromString(FString::Printf(TEXT("%dx%d"), TileX + 1, TileY + 1)))
				.OnClicked(FOnClicked::CreateLambda([this, Tile]()
					{
						PageLayout_OnGridCellClicked(Tile);
						return FReply::Handled();
					}))
				.OnHovered(FSimpleDelegate::CreateLambda([this, Tile]()
					{
						PageLayout_OnGridCellHovered(Tile);
					}))
				.OnUnhovered(FSimpleDelegate::CreateLambda([this]()
					{
						PageLayout_OnGridCellUnhovered();
					}));

			// Store it internally
			LayoutGridButtons.Emplace(Tile, GridButton);

			// Add the button to layout
			LayoutGrid->AddSlot(TileX, TileY)
			[
				SNew(SBox)
				[
					GridButton
				]
			];
		}
	}

	return LayoutGrid;
}

void SMediaTilesConfigurationDialog::PageLayout_OnEnter()
{
	PageHint->SetText(UE::DisplayClusterConfigurator::MediaTilesConfigurationDialog::Private::TextPageLayoutHeader);
}

bool SMediaTilesConfigurationDialog::PageLayout_IsConfigurationValid() const
{
	// Anything other than 1x1 is valid
	return Accepted.X > 0 || Accepted.Y > 0;
}

void SMediaTilesConfigurationDialog::PageLayout_OnGridCellClicked(const FIntPoint& Tile)
{
	const bool bInvalidChoice = (Tile == FIntPoint::ZeroValue);

	Accepted = bInvalidChoice ? FIntPoint{ -1, -1 } : Tile;
	
	PageLayout_RefreshGridLayout();
}

void SMediaTilesConfigurationDialog::PageLayout_OnGridCellHovered(const FIntPoint& Tile)
{
	Hovered = Tile;
	PageLayout_RefreshGridLayout();
}

void SMediaTilesConfigurationDialog::PageLayout_OnGridCellUnhovered()
{
	Hovered = { -1, -1 };
	PageLayout_RefreshGridLayout();
}

void SMediaTilesConfigurationDialog::PageLayout_RefreshGridLayout()
{
	const FButtonStyle& ButtonStyle = FAppStyle::Get().GetWidgetStyle<FButtonStyle>("Button");

	const FSlateColor ColorAccepted  { COLOR("#00FF00FF") };
	const FSlateColor ColorProposed1 { COLOR("#00FF0080") };
	const FSlateColor ColorProposed2 { FStyleColors::Foreground };
	const FSlateColor ColorDefault   { ButtonStyle.NormalForeground };
	const FSlateColor ColorInvalid   { FLinearColor::Red };

	const bool bMatchesAccepted = (Accepted == Hovered);

	for (int32 XPos = 0; XPos < MaxTilesAmount; ++XPos)
	{
		for (int32 YPos = 0; YPos < MaxTilesAmount; ++YPos)
		{
			const FSlateColor* NewColor = nullptr;

			// 1x1 not allowed
			if (XPos == 0 && YPos == 0 && Hovered == FIntPoint::ZeroValue)
			{
				NewColor = &ColorInvalid;
			}
			// Cells proposed to select again (if doesn't match the currently accepted region)
			else if (XPos <= Hovered.X && YPos <= Hovered.Y && XPos <= Accepted.X && YPos <= Accepted.Y && !bMatchesAccepted)
			{
				NewColor = &ColorProposed1;
			}
			// Cells proposed to select (if doesn't match the currently accepted region)
			else if (XPos <= Hovered.X && YPos <= Hovered.Y && !bMatchesAccepted)
			{
				NewColor = &ColorProposed2;
			}
			// Currently selected
			else if (XPos <= Accepted.X && YPos <= Accepted.Y)
			{
				NewColor = &ColorAccepted;
			}
			// Remaining cells
			else
			{
				NewColor = &ColorDefault;
			}

			LayoutGridButtons[{XPos, YPos}]->SetBorderBackgroundColor(*NewColor);
		}
	}
}


/**
 * Media objects page
 */
TSharedRef<SWidget> SMediaTilesConfigurationDialog::PageMedia_Build()
{
	TSharedRef<SVerticalBox> MediaObjectsPage = SNew(SVerticalBox);

	// Add input and output sections
	for (int32 Idx = 0; Idx < 2; ++Idx)
	{
		MediaObjectsPage->AddSlot()
		.Padding(5, 5)
		[
			SNew(SBorder)
			.Visibility(EVisibility::All)
			.Content()
			[
				SNew(SVerticalBox)

					// Media object row
					+SVerticalBox::Slot()
					.Padding(5, 5)
					[
						SNew(SHorizontalBox)

							// Field name
							+SHorizontalBox::Slot()
							.AutoWidth()
							.HAlign(HAlign_Left)
							.VAlign(VAlign_Center)
							[
								SNew(STextBlock)
								.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
								.Text( Idx == 0 ?
									UE::DisplayClusterConfigurator::MediaTilesConfigurationDialog::Private::TextPageMediaSource :
									UE::DisplayClusterConfigurator::MediaTilesConfigurationDialog::Private::TextPageMediaOutput)
							]

							+SHorizontalBox::Slot()
							.MaxWidth(20)
							[
								SNew(SSpacer)
							]

							// Field value
							+SHorizontalBox::Slot()
							.AutoWidth()
							.HAlign(HAlign_Fill)
							.VAlign(VAlign_Center)
							[
								SNew(SComboButton)
								.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButton")
								.ContentPadding(0)
								.HasDownArrow(true)
								.OnGetMenuContent_Lambda([this, Idx]()
								{
									return PageMedia_BuildClassPicker(Idx == 0 ? UMediaSource::StaticClass() : UMediaOutput::StaticClass());
								})
								.ButtonContent()
								[
									SAssignNew(Idx == 0 ? MediaSourceSelectedName : MediaOutputSelectedName, STextBlock)
									.Text(UE::DisplayClusterConfigurator::MediaTilesConfigurationDialog::Private::TextPageMediaComboboxNone)
									.ColorAndOpacity(FStyleColors::Foreground)
								]
							]
					]

					// Media object status
					+SVerticalBox::Slot()
					.Padding(5, 5)
					[
						SNew(SHorizontalBox)

							+SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							.HAlign(HAlign_Left)
							.Padding(2.f)
							[
								SAssignNew(Idx == 0 ? MediaSourceStatusImage : MediaOutputStatusImage, SImage)
								.Image(nullptr)
							]

							+SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							.HAlign(HAlign_Left)
							[
								SAssignNew(Idx == 0 ? MediaSourceStatusText : MediaOutputStatusText, STextBlock)
							]
					]

			]
		];
	}

	return MediaObjectsPage;
}

TSharedRef<SWidget> SMediaTilesConfigurationDialog::PageMedia_BuildClassPicker(UClass* FilterClass)
{
	// Auxiliary class filter implementation
	class FDisplayClusterConfiguratorMediaTypeFilter final : public IClassViewerFilter
	{
	public:
		explicit FDisplayClusterConfiguratorMediaTypeFilter(UClass* InRequiredBaseClass = nullptr)
			: RequiredBaseClass(InRequiredBaseClass)
		{ }

		virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef<FClassViewerFilterFuncs> InFilterFuncs) override
		{
			static const EClassFlags FilterFlags = EClassFlags::CLASS_Abstract | EClassFlags::CLASS_Deprecated | EClassFlags::CLASS_Hidden | EClassFlags::CLASS_HideDropDown;
			return InClass && !InClass->HasAnyClassFlags(FilterFlags) && (RequiredBaseClass ? InClass->IsChildOf(RequiredBaseClass) : true);
		}

		virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef<const IUnloadedBlueprintData> InUnloadedClassData, TSharedRef<FClassViewerFilterFuncs> InFilterFuncs) override
		{
			return false;
		}

	private:
		/** Classes must have this base class to pass the filter. */
		UClass* RequiredBaseClass = nullptr;
	};

	FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");

	FClassViewerInitializationOptions Options;
	Options.Mode = EClassViewerMode::ClassPicker;
	Options.NameTypeToDisplay = EClassViewerNameTypeToDisplay::DisplayName;
	Options.bShowNoneOption = true;
	Options.bIsActorsOnly = false;
	Options.bShowUnloadedBlueprints = false;
	Options.ClassFilters.Add(MakeShared<FDisplayClusterConfiguratorMediaTypeFilter>(FilterClass));

	const bool bIsMediaSource = (FilterClass == UMediaSource::StaticClass());

	// Instantiate class viewer widget
	const TSharedRef<SWidget> ClassViewer = ClassViewerModule.CreateClassViewer(
		Options,
		FOnClassPicked::CreateSP(this, FilterClass == UMediaSource::StaticClass() ?
			&SMediaTilesConfigurationDialog::PageMedia_OnMediaSourceChanged :
			&SMediaTilesConfigurationDialog::PageMedia_OnMediaOutputChanged)
		);

	return SNew(SBox)
		[
			ClassViewer
		];
}

bool SMediaTilesConfigurationDialog::PageMedia_OnCanShow() const
{
	return PageLayout_IsConfigurationValid();
}

void SMediaTilesConfigurationDialog::PageMedia_OnEnter()
{
	PageHint->SetText(UE::DisplayClusterConfigurator::MediaTilesConfigurationDialog::Private::TextPageMediaHeader);
}

bool SMediaTilesConfigurationDialog::PageMedia_IsConfigurationValid() const
{
	// Media source and output must be valid and compatible
	const bool bMediaObjectsCompatible = AreMediaObjectsCompatible(MediaSource.Get(), MediaOutput.Get());
	return bMediaObjectsCompatible;
}

void SMediaTilesConfigurationDialog::PageMedia_OnMediaSourceChanged(UClass* InNewClass)
{
	// Process new choice
	if (InNewClass && InNewClass->IsChildOf<UMediaSource>())
	{
		MediaSource.Reset(NewObject<UMediaSource>(GetTransientPackage(), InNewClass));
		MediaSourceSelectedName->SetText(InNewClass->GetDisplayNameText());
	}
	else
	{
		MediaSource.Reset();
		MediaSourceSelectedName->SetText(UE::DisplayClusterConfigurator::MediaTilesConfigurationDialog::Private::TextPageMediaComboboxNone);
	}

	// Update feedback
	PageMedia_UpdateMessage();

	FSlateApplication::Get().DismissAllMenus();
}

void SMediaTilesConfigurationDialog::PageMedia_OnMediaOutputChanged(UClass* InNewClass)
{
	// Process new choice
	if (InNewClass && InNewClass->IsChildOf<UMediaOutput>())
	{
		MediaOutput.Reset(NewObject<UMediaOutput>(GetTransientPackage(), InNewClass));
		MediaOutputSelectedName->SetText(InNewClass->GetDisplayNameText());
	}
	else
	{
		MediaOutput.Reset();
		MediaOutputSelectedName->SetText(UE::DisplayClusterConfigurator::MediaTilesConfigurationDialog::Private::TextPageMediaComboboxNone);
	}

	// Update feedback
	PageMedia_UpdateMessage();

	FSlateApplication::Get().DismissAllMenus();
}

void SMediaTilesConfigurationDialog::PageMedia_UpdateMessage()
{
	if (!MediaSource)
	{
		MediaSourceStatusImage->SetImage(nullptr);
		MediaSourceStatusText->SetText(FText::GetEmpty());
	}

	if (!MediaOutput)
	{
		MediaOutputStatusImage->SetImage(nullptr);
		MediaOutputStatusText->SetText(FText::GetEmpty());
	}

	const FSlateBrush* StatusImageOk   = FAppStyle::Get().GetBrush("EditorViewport.LightingOnlyMode");
	const FSlateBrush* StatusImageWarn = FAppStyle::Get().GetBrush("Level.LightingScenarioNotIcon16x");

	// Invalidate media propagation type. It will be set to a proper value later if media objects are good.
	MediaPropagationTypes = EMediaStreamPropagationType::None;

	// Only one object is chosen
	if((MediaSource && !MediaOutput) || (!MediaSource && MediaOutput))
	{
		const UObject* CurrentObject = (MediaSource.IsValid() ? Cast<UObject>(MediaSource.Get()) : Cast<UObject>(MediaOutput.Get()));

		const TSharedPtr<SImage>&     StatusImage = (MediaSource.IsValid() ? MediaSourceStatusImage : MediaOutputStatusImage);
		const TSharedPtr<STextBlock>& StatusText  = (MediaSource.IsValid() ? MediaSourceStatusText : MediaOutputStatusText);

		const bool bIsObjectSupported = IsMediaObjectSupported(CurrentObject);

		StatusImage->SetImage(bIsObjectSupported ? StatusImageOk : StatusImageWarn);

		StatusText->SetColorAndOpacity(bIsObjectSupported ? FLinearColor::Green : FLinearColor::Yellow);

		StatusText->SetText(bIsObjectSupported ?
			UE::DisplayClusterConfigurator::MediaTilesConfigurationDialog::Private::TextPageMediaStatusOk :
			UE::DisplayClusterConfigurator::MediaTilesConfigurationDialog::Private::TextPageMediaStatusNotSupp);
		
	}
	// Both objects are chosen
	else if(MediaSource && MediaOutput)
	{
		const bool bAreObjectsCompatible = AreMediaObjectsCompatible(MediaSource.Get(), MediaOutput.Get());

		TSharedPtr<SImage>*     StatusImage[] = { &MediaSourceStatusImage, &MediaOutputStatusImage };
		TSharedPtr<STextBlock>* StatusText[]  = { &MediaSourceStatusText , &MediaOutputStatusText };

		for (int32 Idx = 0; Idx < 2; ++Idx)
		{
			(*StatusImage[Idx])->SetImage(bAreObjectsCompatible ? StatusImageOk : StatusImageWarn);

			(*StatusText[Idx])->SetColorAndOpacity(bAreObjectsCompatible ? FLinearColor::Green : FLinearColor::Yellow);

			(*StatusText[Idx])->SetText(bAreObjectsCompatible ?
				UE::DisplayClusterConfigurator::MediaTilesConfigurationDialog::Private::TextPageMediaStatusOk :
				UE::DisplayClusterConfigurator::MediaTilesConfigurationDialog::Private::TextPageMediaNotCompatOrNotSupp);
		}

		// Update media propagation types
		if (bAreObjectsCompatible)
		{
			GetMediaPropagationTypes(MediaSource.Get(), MediaOutput.Get(), MediaPropagationTypes);
		}
	}
}


/**
 * Nodes selection page
 */
TSharedRef<SWidget> SMediaTilesConfigurationDialog::PageNodes_Build()
{
	return SNew(SHorizontalBox)

		// Output nodes that produce the tiles
		+SHorizontalBox::Slot()
		.Padding(2)
		[
			SNew(SBorder)
			.OnMouseButtonDown_Lambda([this](const FGeometry& Geometry, const FPointerEvent& MouseEvent)
				{
					return PageNodes_OnMouseButtonDown(Geometry, MouseEvent, true);
				})
			[
				PageNodes_BuildSelSection(
					UE::DisplayClusterConfigurator::MediaTilesConfigurationDialog::Private::TextPageNodesSectionOutputHeader,
					NodesAllowedForOutput,
					OutputSelection)
			]
		]

		// Input nodes that receive all the tiles and compose the full frame
		+SHorizontalBox::Slot()
		.Padding(2)
		[
			SNew(SBorder)
			.OnMouseButtonDown_Lambda([this](const FGeometry& Geometry, const FPointerEvent& MouseEvent)
				{
					return PageNodes_OnMouseButtonDown(Geometry, MouseEvent, false);
				})
			[
				PageNodes_BuildSelSection(
					UE::DisplayClusterConfigurator::MediaTilesConfigurationDialog::Private::TextPageNodesSectionInputHeader,
					NodesAllowedForInput,
					InputSelection)
			]
		];
}

TSharedRef<SWidget> SMediaTilesConfigurationDialog::PageNodes_BuildSelSection(const FText& Header, const TSet<FString>& AllowedItems, TSet<FString>& CurrentSelection)
{
	TSharedRef<SVerticalBox> Section = SNew(SVerticalBox);

	// Output header
	Section->AddSlot()
	.Padding(5)
	.AutoHeight()
	.HAlign(HAlign_Center)
	.VAlign(VAlign_Top)
	[
		SNew(STextBlock)
		.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
		.Text(Header)
	];

	Section->AddSlot()
	.MaxHeight(10)
	[
		SNew(SSpacer)
	];

	// Output body
	Section->AddSlot()
	.Padding(5, 3)
	.MaxHeight(400)
	.AutoHeight()
	[
		SNew(SScrollBox)
		.Orientation(EOrientation::Orient_Vertical)

			+SScrollBox::Slot()
			[
				PageNodes_BuildNodePicker(AllowedItems, CurrentSelection)
			]
	];

	return Section;
}

TSharedRef<SWidget> SMediaTilesConfigurationDialog::PageNodes_BuildNodePicker(const TSet<FString>& AllowedItems, TSet<FString>& CurrentSelection)
{
	TSharedRef<SVerticalBox> NodePicker = SNew(SVerticalBox);

	// Generate output nodes list
	for (const FString& NodeId : AllowedItems)
	{
		NodePicker->AddSlot()
			.AutoHeight()
			.Padding(3)
			[
				SNew(SCheckBox)
				.Padding(5)
				.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
				.CheckBoxContentUsesAutoWidth(true)
				.IsChecked_Lambda([NodeId, &CurrentSelection]()
					{
						return CurrentSelection.Contains(NodeId) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					})
				.OnCheckStateChanged_Lambda([NodeId, &CurrentSelection](ECheckBoxState NewState)
					{
						if (NewState == ECheckBoxState::Checked)
						{
							CurrentSelection.Add(NodeId);
						}
						else
						{
							CurrentSelection.Remove(NodeId);
						}
					})
				.Content()
				[
					SNew(STextBlock)
					.Text(FText::FromString(NodeId))
					.AutoWrapText(true)
				]
			];
	}

	return NodePicker;
}

FReply SMediaTilesConfigurationDialog::PageNodes_OnMouseButtonDown(const FGeometry& Geometry, const FPointerEvent& PointerEvent, bool bOutputSelection)
{
	if (PointerEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		PageNodes_CreateContextMenu(PointerEvent.GetScreenSpacePosition(), bOutputSelection);
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SMediaTilesConfigurationDialog::PageNodes_CreateContextMenu(const FVector2D& CursorPosition, bool bOutputSelection)
{
	constexpr bool bCloseAfterSelection = true;
	FMenuBuilder MenuBuilder(bCloseAfterSelection, nullptr);

	// Section: general commands
	{
		const FName SectionName = TEXT("General");

		MenuBuilder.BeginSection(SectionName);

		MenuBuilder.AddWidget(
			SNew(STextBlock)
			.Text(FText::FromName(SectionName)),
			FText(),
			true
		);

		// Select All
		MenuBuilder.AddMenuEntry(
			UE::DisplayClusterConfigurator::MediaTilesConfigurationDialog::Private::TextPageNodesMenuSelectAll,
			FText(),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SMediaTilesConfigurationDialog::PageNodes_Menu_OnChangeSelectionAll, bOutputSelection, true)
			)
		);

		// Deselect all
		MenuBuilder.AddMenuEntry(
			UE::DisplayClusterConfigurator::MediaTilesConfigurationDialog::Private::TextPageNodesMenuDeselectAll,
			FText(),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SMediaTilesConfigurationDialog::PageNodes_Menu_OnChangeSelectionAll, bOutputSelection, false)
			)
		);

		MenuBuilder.EndSection();
	}

	// Section: onscreen
	{
		const FName SectionName = TEXT("On-screen nodes");

		MenuBuilder.BeginSection(SectionName);

		MenuBuilder.AddWidget(
			SNew(STextBlock)
			.Text(FText::FromName(SectionName)),
			FText(),
			true
		);

		// Select all onscreen
		MenuBuilder.AddMenuEntry(
			UE::DisplayClusterConfigurator::MediaTilesConfigurationDialog::Private::TextPageNodesMenuSelectAll,
			FText(),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SMediaTilesConfigurationDialog::PageNodes_Menu_OnSelectDeselectAllOnOffscreen, bOutputSelection, true, false)
			)
		);

		// Deselect all onscreen
		MenuBuilder.AddMenuEntry(
			UE::DisplayClusterConfigurator::MediaTilesConfigurationDialog::Private::TextPageNodesMenuDeselectAll,
			FText(),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SMediaTilesConfigurationDialog::PageNodes_Menu_OnSelectDeselectAllOnOffscreen, bOutputSelection, false, false)
			)
		);

		MenuBuilder.EndSection();
	}

	// Section: offscreen
	{
		const FName SectionName = TEXT("Off-screen nodes");

		MenuBuilder.BeginSection(SectionName);

		MenuBuilder.AddWidget(
			SNew(STextBlock)
			.Text(FText::FromName(SectionName)),
			FText(),
			true
		);

		// Select all offscreen
		MenuBuilder.AddMenuEntry(
			UE::DisplayClusterConfigurator::MediaTilesConfigurationDialog::Private::TextPageNodesMenuSelectAll,
			FText(),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SMediaTilesConfigurationDialog::PageNodes_Menu_OnSelectDeselectAllOnOffscreen, bOutputSelection, true, true)
			)
		);

		// Deselect all offscreen
		MenuBuilder.AddMenuEntry(
			UE::DisplayClusterConfigurator::MediaTilesConfigurationDialog::Private::TextPageNodesMenuDeselectAll,
			FText(),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SMediaTilesConfigurationDialog::PageNodes_Menu_OnSelectDeselectAllOnOffscreen, bOutputSelection, false, true)
			)
		);

		MenuBuilder.EndSection();
	}

	// Section: extra commands
	{
		const FName SectionName = TEXT("Extra");

		MenuBuilder.BeginSection(SectionName);

		MenuBuilder.AddWidget(
			SNew(STextBlock)
			.Text(FText::FromName(SectionName)),
			FText(),
			true
		);

		// Select All w/o counterpart
		MenuBuilder.AddMenuEntry(
			bOutputSelection ?
				UE::DisplayClusterConfigurator::MediaTilesConfigurationDialog::Private::TextPageNodesMenuSelectAllNoReceivers :
				UE::DisplayClusterConfigurator::MediaTilesConfigurationDialog::Private::TextPageNodesMenuSelectAllNoSenders,
			FText(),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SMediaTilesConfigurationDialog::PageNodes_Menu_OnSelectAllExceptOfCounterpart, bOutputSelection)
			)
		);

		// Deselect all from counterpart
		MenuBuilder.AddMenuEntry(
			bOutputSelection ?
				UE::DisplayClusterConfigurator::MediaTilesConfigurationDialog::Private::TextPageNodesMenuDeselectReceivers :
				UE::DisplayClusterConfigurator::MediaTilesConfigurationDialog::Private::TextPageNodesMenuDeselectSenders,
			FText(),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SMediaTilesConfigurationDialog::PageNodes_Menu_OnDeselectAllFromCounterpart, bOutputSelection)
			)
		);

		MenuBuilder.EndSection();
	}

	// Show the context menu
	FSlateApplication::Get().PushMenu(
		SharedThis(this),
		FWidgetPath(),
		MenuBuilder.MakeWidget(),
		CursorPosition,
		FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));
}

void SMediaTilesConfigurationDialog::PageNodes_Menu_OnChangeSelectionAll(bool bOutputSelection, bool bSelected)
{
	// A set being edited
	TSet<FString>& CurrentSet = bOutputSelection ? OutputSelection : InputSelection;
	// Allowed items for the set being edited
	const TSet<FString>& AllowedSet = bOutputSelection ? NodesAllowedForOutput : NodesAllowedForInput;

	// Clear first either it's 'select all' or 'deselect all'
	CurrentSet.Reset();

	// And 'select all' if it was actually requested
	if (bSelected)
	{
		CurrentSet = AllowedSet;
	}
}

void SMediaTilesConfigurationDialog::PageNodes_Menu_OnSelectAllExceptOfCounterpart(bool bOutputSelection)
{
	// A set being edited
	TSet<FString>& SetA = bOutputSelection ? OutputSelection : InputSelection;
	// A counterpart set
	const TSet<FString>& SetB = bOutputSelection ? InputSelection : OutputSelection;
	// Allowed items for the set being edited
	const TSet<FString>& AllowedSetA = bOutputSelection ? NodesAllowedForOutput : NodesAllowedForInput;

	// Select all in A that aren't currently selected in B
	// Result = AllowedA - SelectedB
	SetA = AllowedSetA.Difference(SetB);
}

void SMediaTilesConfigurationDialog::PageNodes_Menu_OnDeselectAllFromCounterpart(bool bOutputSelection)
{
	// A set being edited
	TSet<FString>& SetA = bOutputSelection ? OutputSelection : InputSelection;
	// A counterpart set
	const TSet<FString>& SetB = bOutputSelection ? InputSelection : OutputSelection;

	// In set A, deselect everything that is in set B
	SetA = SetA.Difference(SetB);
}

void SMediaTilesConfigurationDialog::PageNodes_Menu_OnSelectDeselectAllOnOffscreen(bool bOutputSelection, bool bSelect, bool bOffscreen)
{
	// A set being edited
	TSet<FString>& CurrentSet = bOutputSelection ? OutputSelection : InputSelection;
	// Allowed items for the set being edited
	const TSet<FString>& AllowedSet = bOutputSelection ? NodesAllowedForOutput : NodesAllowedForInput;

	// Find nodes that we're going to select or deselect
	const TSet<FString> DesiredNodes = bOffscreen ?
		// All offscreen nodes allowed for this section (input or output)
		AllowedSet.Intersect(OffscreenNodes) :
		// All onscreen nodes allowed for this section (input or output)
		AllowedSet.Intersect(ClusterNodeIds.Difference(OffscreenNodes));

	if (bSelect)
	{
		// Select all
		CurrentSet.Append(DesiredNodes);
	}
	else
	{
		// Deselect all
		CurrentSet = CurrentSet.Difference(DesiredNodes);
	}
}

bool SMediaTilesConfigurationDialog::PageNodes_OnCanShow() const
{
	return PageMedia_IsConfigurationValid();
}

void SMediaTilesConfigurationDialog::PageNodes_OnEnter()
{
	PageHint->SetText(UE::DisplayClusterConfigurator::MediaTilesConfigurationDialog::Private::TextPageNodesHeader);
}

bool SMediaTilesConfigurationDialog::PageNodes_IsConfigurationValid() const
{
	// We need at least one input node to be selected. There is no requirement for output nodes
	// because those can be set up on the next page manually.
	const bool bHasInputAssigned = !InputSelection.IsEmpty();

	return bHasInputAssigned;
}


/**
 * Finalization page
 */
TSharedRef<SWidget> SMediaTilesConfigurationDialog::PageFinalization_Build()
{
	// Output mapping grid
	return SNew(SBorder)
	[
		SAssignNew(OutputMappingGrid, SUniformGridPanel)
		.SlotPadding(2)
		.MinDesiredSlotWidth(OutputMappingGridCellSize)
		.MinDesiredSlotHeight(OutputMappingGridCellSize)
	];
}

TSharedRef<SWidget> SMediaTilesConfigurationDialog::PageFinalization_BuildOutputTileButtonContent(const FIntPoint& Tile)
{
	return SNew(SBox)
		.MaxDesiredHeight(OutputMappingGridCellSize)
		.Content()
		[
			SNew(SScrollBox)
			.Orientation(EOrientation::Orient_Vertical)

				+SScrollBox::Slot()
				[
					SAssignNew(OutputMapping[Tile].ButtonContentWidget, SVerticalBox)
				]
		];
}

TSharedRef<SWidget> SMediaTilesConfigurationDialog::PageFinalization_BuildOutputTileButtonDropdownContent(const FIntPoint& Tile)
{
	TSharedRef<SVerticalBox> NodePicker = SNew(SVerticalBox);

	// Generate nodes list
	for (const FString& NodeId : ClusterNodeIds)
	{
		NodePicker->AddSlot()
			.AutoHeight()
			.Padding(1)
			[
				SNew(SCheckBox)
				.Padding(1)
				.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
				.CheckBoxContentUsesAutoWidth(true)
				.IsChecked_Lambda([this, Tile, NodeId]()
					{
						const bool bSelected = OutputMapping[Tile].ClusterNodes.Contains(NodeId);
						return bSelected ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					})
				.OnCheckStateChanged_Lambda([this, Tile, NodeId](ECheckBoxState NewState)
					{
						if (NewState == ECheckBoxState::Checked)
						{
							OutputMapping[Tile].ClusterNodes.Add(NodeId);
						}
						else
						{
							OutputMapping[Tile].ClusterNodes.Remove(NodeId);
						}

						// Update grid cell content
						PageFinalization_UpdateTileButtonContent(Tile);
					})
				.Content()
				[
					SNew(SBox)
					.Padding(3)
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Top)
					[
						SNew(STextBlock)
						.Text(FText::FromString(NodeId))
						.TextStyle(&FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("SmallText"))
						.Font(FAppStyle::GetFontStyle("TinyText"))
						.AutoWrapText(true)
					]
				]
			];
	}

	return SNew(SBox)
		.MaxDesiredHeight(600)
		.Content()
		[
			SNew(SScrollBox)
			.Orientation(EOrientation::Orient_Vertical)

				+SScrollBox::Slot()
				[
					NodePicker
				]
		];
}

bool SMediaTilesConfigurationDialog::PageFinalization_OnCanShow() const
{
	return PageNodes_IsConfigurationValid();
}

void SMediaTilesConfigurationDialog::PageFinalization_OnEnter()
{
	PageHint->SetText(UE::DisplayClusterConfigurator::MediaTilesConfigurationDialog::Private::TextPageFinalizationHeader);

	// Clean up any current mapping
	PageFinalization_ResetOutputMapping();

	// Pre-configure output mapping
	if (Parameters.bAutoPreconfigureOutputMapping)
	{
		PageFinalization_PresetupOutputs();
	}

	// Re-generate grid to properly reflect current configuration
	PageFinalization_RegenerateGrid();
}

void SMediaTilesConfigurationDialog::PageFinalization_ResetOutputMapping()
{
	// Clean up current mapping
	for (TPair<FIntPoint, FOutputMappingData>& TileData : OutputMapping)
	{
		TileData.Value.ClusterNodes.Reset();
	}
}

void SMediaTilesConfigurationDialog::PageFinalization_PresetupOutputs()
{
	// Pre-configure output mapping based on media type previously chosen
	if (EnumHasAnyFlags(MediaPropagationTypes, EMediaStreamPropagationType::Multicast))
	{
		PageFinalization_PresetupOutputs_Multicast();
	}
	else if (EnumHasAnyFlags(MediaPropagationTypes, EMediaStreamPropagationType::LocalMulticast))
	{
		PageFinalization_PresetupOutputs_LocalMulticast();
	}
	else
	{
		unimplemented();
	}
}

void SMediaTilesConfigurationDialog::PageFinalization_PresetupOutputs_LocalMulticast()
{
	// With LocalMulticast, we basically have per-host tile propagation. So every host
	// that has at least one tile receiver, must also have the senders that produce
	// full set of tiles.

	// Max amount of senders according to layout (e.g. would be 6 nodes for 3x2 layout)
	const int32 MaxSendersPerHost = (Accepted.X + 1) * (Accepted.Y + 1);

	// Pre-build senders per-host mapping
	TMultiMap<FString, FString> HostToSendersMap;
	for (const FString& NodeId : OutputSelection)
	{
		if (const FString* Host = NodeToHostMap.Find(NodeId))
		{
			HostToSendersMap.Add(*Host, NodeId);
		}
	}

	// Get list of hosts where we have at least one sender
	TSet<FString> SenderHosts;
	HostToSendersMap.GetKeys(SenderHosts);

	// Map tile senders for every host
	for (const FString& SenderHost : SenderHosts)
	{
		TArray<FString> SenderNodes;
		HostToSendersMap.MultiFind(SenderHost, SenderNodes);

		// Sort alphabetically. This may be handy if user has some naming convention which is kind of typical.
		SenderNodes.Sort();

		// Counter of senders already mapped
		int32 AssignedAmount = 0;

		// For each sender at host...
		for (const FString& SenderNodeId : SenderNodes)
		{
			// A tile we're going to assign for this sender
			const int32 TileX = AssignedAmount % (Accepted.X + 1);
			const int32 TileY = AssignedAmount / (Accepted.X + 1);
			const FIntPoint Tile { TileX, TileY };

			// Map this sender to the tile
			OutputMapping[Tile].ClusterNodes.Add(SenderNodeId);

			// Ignore other senders
			if (++AssignedAmount >= MaxSendersPerHost)
			{
				break;
			}
		}
	}
}

void SMediaTilesConfigurationDialog::PageFinalization_PresetupOutputs_Multicast()
{
	// Multicast implies every tile is produced once by a dedicated node then propagated
	// to all the receivers. This means we can simply pre-configure output mapping
	// by binding N tiles to N senders using 1-1 relation, and any amount of receivers.
	// We don't care about host-node mapping in this case as it's true multicast.

	// Max amount of senders according to layout (e.g. would be 6 nodes for 3x2 layout)
	const int32 MaxSendersPerHost = (Accepted.X + 1) * (Accepted.Y + 1);

	// Counter of senders already mapped
	int32 AssignedAmount = 0;

	// Iterate through the current output selection
	for (const FString& SenderNodeId : OutputSelection)
	{
		// A tile we're going to assign for this sender
		const int32 TileX = AssignedAmount % (Accepted.X + 1);
		const int32 TileY = AssignedAmount / (Accepted.X + 1);
		const FIntPoint Tile{ TileX, TileY };

		// Map this sender to the tile
		OutputMapping[Tile].ClusterNodes.Add(SenderNodeId);

		// Ignore other senders
		if (++AssignedAmount >= MaxSendersPerHost)
		{
			break;
		}
	}
}

void SMediaTilesConfigurationDialog::PageFinalization_RegenerateGrid()
{
	// Remove slots if there are any
	OutputMappingGrid->ClearChildren();

	// Create new slots
	for (int32 TileX = 0; TileX <= Accepted.X; ++TileX)
	{
		for (int32 TileY = 0; TileY <= Accepted.Y; ++TileY)
		{
			const FIntPoint Tile{ TileX, TileY };
			FOutputMappingData& TileOutputMapping = OutputMapping[Tile];

			// Instantiate button at XY
			TSharedRef<SComboButton> GridButton = SNew(SComboButton)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Top)
				.ContentPadding(3)
				.HasDownArrow(true)
				.ForegroundColor_Lambda([this, Tile]()
					{
						return PageFinalization_GetButtonColor(Tile);
					})
				.ButtonColorAndOpacity_Lambda([this, Tile]()
					{
						return PageFinalization_GetButtonColor(Tile);
					})
				.OnGetMenuContent_Lambda([this, Tile]()
					{
						return PageFinalization_BuildOutputTileButtonDropdownContent(Tile);
					})
				.ButtonContent()
				[
					PageFinalization_BuildOutputTileButtonContent(Tile)
				];

			// Store it internally
			TileOutputMapping.ButtonWidget = GridButton;

			// Grid cell (button)
			OutputMappingGrid->AddSlot(TileX, TileY)
				[
					SNew(SBox)
					[
						GridButton
					]
				];

			// Force button content update
			PageFinalization_UpdateTileButtonContent(Tile);
		}
	}
}

bool SMediaTilesConfigurationDialog::PageFinalization_IsConfigurationValid() const
{
	// There are no strict rules to validate current configuration at this step. The GUI
	// has pre-configured output mapping, plus user may want to change the configuration
	// in any way. User even may clean everything or partially and configure the rest
	// manually in the property editor. Therefore always return true.
	return true;
}

void SMediaTilesConfigurationDialog::PageFinalization_UpdateTileButtonContent(const FIntPoint& Tile)
{
	FOutputMappingData& TileData = OutputMapping[Tile];

	// Clean all
	TileData.ButtonContentWidget->ClearChildren();

	// Rebuild children
	for (const FString& NodeId : TileData.ClusterNodes)
	{
		TileData.ButtonContentWidget->AddSlot()
			[
				SNew(STextBlock)
				.Text(FText::FromString(NodeId))
				.TextStyle(FAppStyle::Get(), "SmallButtonText")
			];
	}
}

FSlateColor SMediaTilesConfigurationDialog::PageFinalization_GetButtonColor(const FIntPoint& Tile)
{
	// Visualize whether a tile has any output nodes assigned
	const bool bNodesAssigned = !OutputMapping[Tile].ClusterNodes.IsEmpty();
	return bNodesAssigned ? FLinearColor::Green : FLinearColor::Yellow;
}

#undef LOCTEXT_NAMESPACE
