// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SWindow.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/SlateColor.h"

#include "DisplayClusterConfigurationTypes.h"


class SButton;
class SComboButton;
class SImage;
class STextBlock;
class SUniformGridPanel;
class SWidget;
class SWizard;

class UMediaOutput;
class UMediaSource;

enum class EMediaStreamPropagationType : uint8;


/**
 * Tiled media configuration dialog parameters.
 */
struct FMediaTilesConfigurationDialogParameters
{
	/** Configuration data of a DCRA being edited. */
	const UDisplayClusterConfigurationData* ConfigData = nullptr;

	/** An object owning the configuration data being edited. */
	TObjectPtr<UObject> Owner;

	/** Whether output mapping should be pre-configured automatically. */
	bool bAutoPreconfigureOutputMapping = true;

	/** Layout ref of a DCRA's entity being edited. */
	FIntPoint* SplitLayout = nullptr;

	/** Input groups ref of a DCRA's entity being edited. */
	TArray<FDisplayClusterConfigurationMediaTiledInputGroup>* InputGroups = nullptr;

	/** Output groups ref of a DCRA's entity being edited. */
	TArray<FDisplayClusterConfigurationMediaTiledOutputGroup>* OutputGroups = nullptr;

public:

	/** Config parameters validation. */
	bool Validate() const
	{
		const bool bIsValid = ConfigData && Owner && SplitLayout && InputGroups && OutputGroups;
		checkSlow(bIsValid);
		return bIsValid;
	}
};


/**
 * Configuration dialog for tiled media
 */
class SMediaTilesConfigurationDialog
	: public SWindow
{
public:
	SLATE_BEGIN_ARGS(SMediaTilesConfigurationDialog)
	{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const FMediaTilesConfigurationDialogParameters& InParameters);

public:

	/** Returns true if configuration was completed. */
	bool WasConfigurationCompleted() const;

private:

	/** Initializes internal data. */
	void InitializeInternals();

	/** Returns true if 'Finish' button should be active. */
	bool IsFinishButtonEnabled() const;

	/** 'Finish' button handler. */
	void OnFinishButtonClicked();

	/** 'Cancel' button handler. */
	void OnCancelButtonClicked();

private:

	/** Apply configuration entry point. */
	bool ApplyConfiguration(TArray<FString>& OutErrors, TArray<FString>& OutWarnings);

	/** Apply configuration for 'Multicast' media. */
	bool ApplyConfiguration_Multicast(TArray<FString>& OutErrors, TArray<FString>& OutWarnings);

	/** Apply configuration for 'LocalMulticast' media. */
	bool ApplyConfiguration_LocalMulticast(TArray<FString>& OutErrors, TArray<FString>& OutWarnings);

private:

	/** Helper function to generate a message with the issues found. */
	void GenerateIssuesFoundMessage(const TArray<FString>& Errors, const TArray<FString>& Warnings, FText& OutMessage) const;

	/** Helper function to check if specific media object is supported for auto-configuration */
	bool IsMediaObjectSupported(const UObject* MediaObject) const;

	/** Helper function to check if specific media objects are compatible to each other */
	bool AreMediaObjectsCompatible(const UObject* InMediaSource, const UObject* InMediaOutput) const;

	/** Helper function to get media propagation types from a media initializer */
	bool GetMediaPropagationTypes(const UObject* InMediaSource, const UObject* InMediaOutput, EMediaStreamPropagationType& OutPropagationFlags) const;


	/**
	 * Layout page
	 */
private:

	/** Builds layout page. */
	TSharedRef<SWidget> PageLayout_Build();

	/** Callback on layout page enter. */
	void PageLayout_OnEnter();

	/** Returns true if layout configuration is valid. */
	bool PageLayout_IsConfigurationValid() const;

	/** Cell click handler. */
	void PageLayout_OnGridCellClicked(const FIntPoint& Tile);

	/** Cell hover handler. */
	void PageLayout_OnGridCellHovered(const FIntPoint& Tile);

	/** Cell unhover handler. */
	void PageLayout_OnGridCellUnhovered();

	/** Refresh current cell states. */
	void PageLayout_RefreshGridLayout();

	/**
	 * Media objects page
	 */
private:

	/** Builds media objects page. */
	TSharedRef<SWidget> PageMedia_Build();

	/** Builds media class picker widget. */
	TSharedRef<SWidget> PageMedia_BuildClassPicker(UClass* FilterClass);

	/** Callback to check if we can switch to the media objects configuration page. */
	bool PageMedia_OnCanShow() const;

	/** Callback on media objects page enter. */
	void PageMedia_OnEnter();

	/** Returns true if media objects configuration is valid. */
	bool PageMedia_IsConfigurationValid() const;

	/** Media source change handler. */
	void PageMedia_OnMediaSourceChanged(UClass* InNewClass);

	/** Media output change handler. */
	void PageMedia_OnMediaOutputChanged(UClass* InNewClass);

	/** Updates configuration message/feedback. */
	void PageMedia_UpdateMessage();

	/**
	 * Nodes selection page
	 */
private:

	/** Builds nodes selection page. */
	TSharedRef<SWidget> PageNodes_Build();

	/** Builds intput/output section. */
	TSharedRef<SWidget> PageNodes_BuildSelSection(const FText& Header, const TSet<FString>& AllowedItems, TSet<FString>& CurrentSelection);

	/** Builds node picker widget for input/output mapping. */
	TSharedRef<SWidget> PageNodes_BuildNodePicker(const TSet<FString>& AllowedItems, TSet<FString>& CurrentSelection);

	/** Mouse click handler (for context menu). */
	FReply PageNodes_OnMouseButtonDown(const FGeometry& Geometry, const FPointerEvent& PointerEvent, bool bOutputSelection);

	/** Mouse click handler (for context menu). */
	void PageNodes_CreateContextMenu(const FVector2D& CursorPosition, bool bOutputSelection);

	/** Context menu handler: Select/Deselect All. */
	void PageNodes_Menu_OnChangeSelectionAll(bool bOutputSelection, bool bSelected);

	/** Context menu handler: Select all items in A that aren't selected in B (A-Bsel). */
	void PageNodes_Menu_OnSelectAllExceptOfCounterpart(bool bOutputSelection);

	/** Context menu handler: Deselect all items in A that are selected in B (Asel-Bsel). */
	void PageNodes_Menu_OnDeselectAllFromCounterpart(bool bOutputSelection);

	/** Context menu handler: Select all onscreen/offscreen nodes. */
	void PageNodes_Menu_OnSelectDeselectAllOnOffscreen(bool bOutputSelection, bool bSelect, bool bOffscreen);

	/** Callback to check if we can switch to the nodes selection page. */
	bool PageNodes_OnCanShow() const;

	/** Callback on nodes selection page enter. */
	void PageNodes_OnEnter();

	/** Returns true if nodes selection is valid. */
	bool PageNodes_IsConfigurationValid() const;

	/**
	 * Finalization page
	 */
private:

	/** Builds finalization page. */
	TSharedRef<SWidget> PageFinalization_Build();

	/** Builds tile button content. */
	TSharedRef<SWidget> PageFinalization_BuildOutputTileButtonContent(const FIntPoint& Tile);

	/** Builds output mapping dropdown content. */
	TSharedRef<SWidget> PageFinalization_BuildOutputTileButtonDropdownContent(const FIntPoint& Tile);

	/** Callback to check if we can switch to the media groups configuration page. */
	bool PageFinalization_OnCanShow() const;

	/** Callback on media groups page enter. */
	void PageFinalization_OnEnter();

	/** Reset current output mapping. */
	void PageFinalization_ResetOutputMapping();

	/** Pre-configures output mapping. */
	void PageFinalization_PresetupOutputs();

	/** Pre-configures output mapping for local multicast (like SharedMemoryMedia). */
	void PageFinalization_PresetupOutputs_LocalMulticast();

	/** Pre-configures output mapping for multicast (like Rivermax). */
	void PageFinalization_PresetupOutputs_Multicast();

	/** Regenerates grid data to match current layout. */
	void PageFinalization_RegenerateGrid();

	/** Returns true if media groups configuration is valid. */
	bool PageFinalization_IsConfigurationValid() const;

	/** Updates cell content. */
	void PageFinalization_UpdateTileButtonContent(const FIntPoint& Tile);

	/** Returns cell color for its current state. */
	FSlateColor PageFinalization_GetButtonColor(const FIntPoint& Tile);

private:

	/** Tile layout constraints. */
	static constexpr int32 MaxTilesAmount = 4;

	/** External configuration parameters. */
	FMediaTilesConfigurationDialogParameters Parameters;

	/** List of all cluster node IDs. */
	TSet<FString> ClusterNodeIds;

	/** List of all offscreen nodes. */
	TSet<FString> OffscreenNodes;

	/** Node-to-host map. */
	TMap<FString, FString> NodeToHostMap;

	/** Cluster nodes per host. */
	TMap<FString, TSet<FString>> HostToNodesMap;

	/** Whether user completed configuration. */
	bool bConfigurationCompleted = false;

	/** Page hint. */
	TSharedPtr<STextBlock> PageHint;

	/** Wizard widget. */
	TSharedPtr<SWizard> Wizard;


	/** [Layout] Button widgets representing tile layout. */
	TMap<FIntPoint, TSharedPtr<SButton>> LayoutGridButtons;

	/** [Layout] Layout grid cell size. */
	const int32 LayoutGridCellSize = 80;

	/** [Layout] Current 'hover' postion. */
	FIntPoint Hovered = { -1, -1 };

	/** [Layout] Last 'accepted' position. */
	FIntPoint Accepted = { -1, -1 };


	/** [Media] Status image for media source. */
	TSharedPtr<SImage> MediaSourceStatusImage;

	/** [Media] Status image for media output. */
	TSharedPtr<SImage> MediaOutputStatusImage;

	/** [Media] Status text for media source. */
	TSharedPtr<STextBlock> MediaSourceStatusText;

	/** [Media] Status text for media output. */
	TSharedPtr<STextBlock> MediaOutputStatusText;

	/** [Media] Currently selected media source name (button text). */
	TSharedPtr<STextBlock> MediaSourceSelectedName;

	/** [Media] Currently selected media output name (button text). */
	TSharedPtr<STextBlock> MediaOutputSelectedName;

	/** [Media] Media source template choosed by user. */
	TStrongObjectPtr<UMediaSource> MediaSource;

	/** [Media] Media output template choosed by user. */
	TStrongObjectPtr<UMediaOutput> MediaOutput;

	/** [Media] Stream propagation type (unicast, multicast, etc.). */
	EMediaStreamPropagationType MediaPropagationTypes = static_cast<EMediaStreamPropagationType>(0);


	/** [Nodes] Nodes allowed to be picked for input (allowed for receiving). */
	TSet<FString> NodesAllowedForInput;

	/** [Nodes] Nodes allowed to be picked for output (allowed for sending). */
	TSet<FString> NodesAllowedForOutput;

	/** [Nodes] Input nodes selection. */
	TSet<FString> InputSelection;

	/** [Nodes] Output nodes selection. */
	TSet<FString> OutputSelection;


	/** [Finalization] Output mapping widget. */
	TSharedPtr<SUniformGridPanel> OutputMappingGrid;

	/** [Finalization] Output mapping grid cell size. */
	const int32 OutputMappingGridCellSize = 150;

	/** [Finalization] Helper container for output tile mapping. */
	struct FOutputMappingData
	{
		/** A drop-down button widget representing a tile (grid cell) */
		TSharedPtr<SComboButton> ButtonWidget;

		/** A drop-down content widget for the button above */
		TSharedPtr<SVerticalBox> ButtonContentWidget;

		/** Cluster nodes associated with this tile (grid cell) */
		TSet<FString> ClusterNodes;
	};

	/** [Finalization] Tile output mapping. */
	TMap<FIntPoint, FOutputMappingData> OutputMapping;
};
