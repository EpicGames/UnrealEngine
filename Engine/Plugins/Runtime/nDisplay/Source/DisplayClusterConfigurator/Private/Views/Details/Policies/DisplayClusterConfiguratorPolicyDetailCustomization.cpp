// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/Details/Policies/DisplayClusterConfiguratorPolicyDetailCustomization.h"

#include "DisplayClusterConfiguratorBlueprintEditor.h"
#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterConfiguratorUtils.h"
#include "DisplayClusterConfiguratorPropertyUtils.h"
#include "DisplayClusterConfiguratorLog.h"
#include "DisplayClusterConfiguratorPolicyParameterCustomization.h"
#include "Views/Details/Widgets/SDisplayClusterConfigurationSearchableComboBox.h"

#include "DisplayClusterRootActor.h"
#include "Blueprints/DisplayClusterBlueprint.h"
#include "Components/DisplayClusterScreenComponent.h"
#include "Components/DisplayClusterICVFXCameraComponent.h"

#include "ProceduralMeshComponent.h"

#include "Camera/CameraComponent.h"

#include "DisplayClusterProjectionStrings.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailGroup.h"
#include "IPropertyUtilities.h"
#include "PropertyHandle.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Misc/DisplayClusterTypesConverter.h"

#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Input/SButton.h"

#define LOCTEXT_NAMESPACE "FDisplayClusterConfiguratorPolicyDetailCustomization"

//////////////////////////////////////////////////////////////////////////////////////////////
// Projection Type Customization
//////////////////////////////////////////////////////////////////////////////////////////////

void FDisplayClusterConfiguratorProjectionCustomization::Initialize(const TSharedRef<IPropertyHandle>& InPropertyHandle, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	FDisplayClusterConfiguratorPolymorphicEntityCustomization::Initialize(InPropertyHandle, CustomizationUtils);

	CustomOption = MakeShared<FString>("Custom");

	// Get the Editing object
	if (!EditingObject->IsA<UDisplayClusterConfigurationViewport>())
	{
		// The editing object should only be invalid in the case where the customized row was created in a different context than the config editor,
		// ie. when using a property row generator like in the Remote Control Preset.
		return;
	}

	ConfigurationViewportPtr = CastChecked<UDisplayClusterConfigurationViewport>(EditingObject);

	for (const TWeakObjectPtr<UObject>& Object : EditingObjects)
	{
		UDisplayClusterConfigurationViewport* Viewport = CastChecked<UDisplayClusterConfigurationViewport>(Object.Get());
		ConfigurationViewports.Add(Viewport);
	}

	// Store what's currently selected.
	CurrentSelectedPolicy = ConfigurationViewportPtr->ProjectionPolicy.Type;

	const bool bRequireCustomPolicy = true;
	bIsCustomPolicy = IsCustomTypeInConfig() && IsPolicyIdenticalAcrossEditedObjects(bRequireCustomPolicy);
	if (bIsCustomPolicy)
	{
		// Load default config
		CustomPolicy = ConfigurationViewportPtr->ProjectionPolicy.Type;
	}
}

void FDisplayClusterConfiguratorProjectionCustomization::SetChildren(const TSharedRef<IPropertyHandle>& InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	// Hide properties 
	TypeHandle->MarkHiddenByCustomization();
	ParametersHandle->MarkHiddenByCustomization();

	// Add custom rows
	ResetProjectionPolicyOptions();
	AddProjectionPolicyRow(InChildBuilder);
	AddCustomPolicyRow(InChildBuilder);

	BuildParametersForPolicy(GetCurrentPolicy(), InChildBuilder);

	// Add Parameters property with Visibility handler
	InChildBuilder
		.AddProperty(ParametersHandle.ToSharedRef())
		.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FDisplayClusterConfiguratorProjectionCustomization::GetCustomRowsVisibility)))
		.ShouldAutoExpand(true);
}

TSharedRef<SWidget> FDisplayClusterConfiguratorProjectionCustomization::MakeProjectionPolicyOptionComboWidget(TSharedPtr<FString> InItem)
{
	return SNew(STextBlock)
		.Text(FText::FromString(*InItem))
		.Font(IDetailLayoutBuilder::GetDetailFont());
}

EVisibility FDisplayClusterConfiguratorProjectionCustomization::GetCustomRowsVisibility() const
{
	return bIsCustomPolicy && IsPolicyIdenticalAcrossEditedObjects() ? EVisibility::Visible :  EVisibility::Collapsed;
}

void FDisplayClusterConfiguratorProjectionCustomization::ResetProjectionPolicyOptions()
{
	ProjectionPolicyOptions.Reset();

	if (UDisplayClusterConfigurationViewport* ConfigurationViewport = ConfigurationViewportPtr.Get())
	{
		for (const FString& ProjectionPolicy : UDisplayClusterConfigurationData::ProjectionPolicies)
		{
			ProjectionPolicyOptions.Add(MakeShared<FString>(ProjectionPolicy));
		}

		// Add Custom option
		if (!bIsCustomPolicy)
		{
			ProjectionPolicyOptions.Add(CustomOption);
		}

		if (ProjectionPolicyComboBox.IsValid())
		{
			// Refreshes the available options now that the shared array has been updated.
		
			ProjectionPolicyComboBox->ResetOptionsSource();
		}
	}
}

void FDisplayClusterConfiguratorProjectionCustomization::AddProjectionPolicyRow(IDetailChildrenBuilder& InChildBuilder)
{
	if (ProjectionPolicyComboBox.IsValid())
	{
		return;
	}
	
	InChildBuilder.AddCustomRow(TypeHandle->GetPropertyDisplayName())
	.NameContent()
	[
		TypeHandle->CreatePropertyNameWidget(FText::GetEmpty(), LOCTEXT("ProjectionPolicyTypeTooltip", "Type of Projection Policy"))
	]
	.ValueContent()
	[
		SAssignNew(ProjectionPolicyComboBox, SDisplayClusterConfigurationSearchableComboBox)
		.OptionsSource(&ProjectionPolicyOptions)
		.OnGenerateWidget(this, &FDisplayClusterConfiguratorProjectionCustomization::MakeProjectionPolicyOptionComboWidget)
		.OnSelectionChanged(this, &FDisplayClusterConfiguratorProjectionCustomization::OnProjectionPolicySelected)
		.ContentPadding(2)
		.MaxListHeight(200.0f)
		.Content()
		[
			SNew(STextBlock)
			.Text(this, &FDisplayClusterConfiguratorProjectionCustomization::GetSelectedProjectionPolicyText)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
	];
}

void FDisplayClusterConfiguratorProjectionCustomization::AddCustomPolicyRow(IDetailChildrenBuilder& InChildBuilder)
{
	if (CustomPolicyRow.IsValid())
	{
		return;
	}

	FText SyncProjectionName = LOCTEXT("SyncProjectionName", "Name");

	InChildBuilder.AddCustomRow(SyncProjectionName)
	.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FDisplayClusterConfiguratorProjectionCustomization::GetCustomRowsVisibility)))
	.NameContent()
	[
		SNew(STextBlock).Text(SyncProjectionName)
	]
	.ValueContent()
	[
		SAssignNew(CustomPolicyRow, SEditableTextBox)
			.Text(this, &FDisplayClusterConfiguratorProjectionCustomization::GetCustomPolicyText)
			.OnTextCommitted(this, &FDisplayClusterConfiguratorProjectionCustomization::OnTextCommittedInCustomPolicyText)
			.Font(IDetailLayoutBuilder::GetDetailFont())
	];
}

void FDisplayClusterConfiguratorProjectionCustomization::OnProjectionPolicySelected(TSharedPtr<FString> InPolicy, ESelectInfo::Type SelectInfo)
{
	if (InPolicy.IsValid())
	{
		FString SelectedPolicy = *InPolicy.Get();

		if (UDisplayClusterConfigurationViewport* ConfigurationViewport = ConfigurationViewportPtr.Get())
		{
			ConfigurationViewport->Modify();
			ModifyBlueprint();
		
			if (SelectedPolicy.Equals(*CustomOption.Get()) && IsPolicyIdenticalAcrossEditedObjects())
			{
				bIsCustomPolicy = true;
				CustomPolicy = ConfigurationViewport->ProjectionPolicy.Type;
				IsCustomHandle->SetValue(true);
			}
			else
			{
				bIsCustomPolicy = false;
				IsCustomHandle->SetValue(false);
			
				TypeHandle->SetValue(SelectedPolicy);

				if (CurrentSelectedPolicy.ToLower() != SelectedPolicy.ToLower())
				{
					// Reset when going from custom to another policy.
					ensure(ParametersHandle->AsMap()->Empty() == FPropertyAccess::Result::Success);
				}
			}

			CurrentSelectedPolicy = SelectedPolicy;

			RefreshBlueprint();
			PropertyUtilities.Pin()->ForceRefresh();
		}
	}
	else
	{
		CurrentSelectedPolicy.Reset();
	}
}

FText FDisplayClusterConfiguratorProjectionCustomization::GetSelectedProjectionPolicyText() const
{
	if (IsPolicyIdenticalAcrossEditedObjects())
	{
		return FText::FromString(GetCurrentPolicy());
	}

	return LOCTEXT("MultipleValues", "Multiple Values");
}

FText FDisplayClusterConfiguratorProjectionCustomization::GetCustomPolicyText() const
{
	return FText::FromString(CustomPolicy);
}

const FString& FDisplayClusterConfiguratorProjectionCustomization::GetCurrentPolicy() const
{
	if (bIsCustomPolicy)
	{
		return *CustomOption.Get();
	}

	if (UDisplayClusterConfigurationViewport* ConfigurationViewport = ConfigurationViewportPtr.Get())
	{
		return ConfigurationViewport->ProjectionPolicy.Type;
	}

	static FString Empty;
	return Empty;
}

bool FDisplayClusterConfiguratorProjectionCustomization::IsCustomTypeInConfig() const
{
	if (UDisplayClusterConfigurationViewport* ConfigurationViewport = ConfigurationViewportPtr.Get())
	{
		if (ConfigurationViewport->ProjectionPolicy.bIsCustom)
		{
			return true;
		}
	
		for (const FString& ProjectionPolicy : UDisplayClusterConfigurationData::ProjectionPolicies)
		{
			if (ConfigurationViewport->ProjectionPolicy.Type.ToLower().Equals(ProjectionPolicy.ToLower()))
			{
				return false;
			}
		}
	}

	return true;
}

void FDisplayClusterConfiguratorProjectionCustomization::OnTextCommittedInCustomPolicyText(const FText& InValue, ETextCommit::Type CommitType)
{
	if (!bIsCustomPolicy)
	{
		// Can be hit if this textbox was selected but the user switched the policy out of CustomPolicy.
		return;
	}
	
	CustomPolicy = InValue.ToString();
	TypeHandle->SetValue(CustomPolicy);

	// Check if the custom config same as any of the ProjectionPolicies configs
	// Turning this off for now in case users want to customize individual parameters..
	// Uncomment this to auto select a default policy if the user types one in the custom name field.
	/*
	for (const FString& ProjectionPolicy : UDisplayClusterConfigurationData::ProjectionPoliсies)
	{
		if (CustomPolicy.Equals(ProjectionPolicy))
		{
			bIsCustomPolicy = false;

			ProjectionPolicyComboBox->SetSelectedItem(MakeShared<FString>(CustomPolicy));

			break;
		}
	}
	*/
}

bool FDisplayClusterConfiguratorProjectionCustomization::IsPolicyIdenticalAcrossEditedObjects(bool bRequireCustomPolicy) const
{
	if (ConfigurationViewports.Num() <= 1)
	{
		return true;
	}
	for (const TWeakObjectPtr<UDisplayClusterConfigurationViewport>& Viewport : ConfigurationViewports)
	{
		if (Viewport.IsValid() &&
			(Viewport->ProjectionPolicy.Type != CurrentSelectedPolicy ||
			(bRequireCustomPolicy && !Viewport->ProjectionPolicy.bIsCustom) ||
			(!bRequireCustomPolicy && Viewport->ProjectionPolicy.bIsCustom != bIsCustomPolicy)))
		{
			return false;
		}
	}

	return true;
}

void FDisplayClusterConfiguratorProjectionCustomization::BuildParametersForPolicy(const FString& Policy, IDetailChildrenBuilder& InChildBuilder)
{
	CustomPolicyParameters.Reset();

	if (!IsPolicyIdenticalAcrossEditedObjects())
	{
		return;
	}
	
	UDisplayClusterBlueprint* Blueprint = FDisplayClusterConfiguratorUtils::FindBlueprintFromObject(EditingObject.Get());
	if (Blueprint == nullptr)
	{
		UE_LOGF(DisplayClusterConfiguratorLog, Warning, "Details policy selection blueprint invalid.");
		return;
	}

	const FString PolicyLower = Policy.ToLower();
	/*
	 * Camera
	 */
	if (PolicyLower == DisplayClusterProjectionStrings::projection::Camera)
	{
		CreateCameraPolicy(Blueprint);
	}
	/*
	 * Mesh
	 */
	else if (PolicyLower == DisplayClusterProjectionStrings::projection::Mesh)
	{
		CreateMeshPolicy(Blueprint);
	}
	/*
	 * Domeprojection
	 */
	else if (PolicyLower == DisplayClusterProjectionStrings::projection::Domeprojection)
	{
		CreateDomePolicy(Blueprint);
	}
	/*
	 * VIOSO
	 */
	else if (PolicyLower == DisplayClusterProjectionStrings::projection::VIOSO)
	{
		CreateVIOSOPolicy(Blueprint);
	}
	/*
	 * EasyBlend
	 */
	else if (PolicyLower == DisplayClusterProjectionStrings::projection::EasyBlend)
	{
		CreateEasyBlendPolicy(Blueprint);
	}
	/*
	 * Reference
	 */
	else if (PolicyLower == DisplayClusterProjectionStrings::projection::Reference)
	{
		CreateReferencePolicy(Blueprint);
	}
	/*
	 * Manual
	 */
	else if (PolicyLower == DisplayClusterProjectionStrings::projection::Manual)
	{
		CreateManualPolicy(Blueprint);
	}
	/*
	 * MPCDI
	 */
	else if (PolicyLower == DisplayClusterProjectionStrings::projection::MPCDI)
	{
		CreateMPCDIPolicy(Blueprint);
	}
	
	// Create the row widgets.
	for (TSharedPtr<FPolicyParameterInfo>& Param : CustomPolicyParameters)
	{
		Param->CreateCustomRowWidget(InChildBuilder);
	}
}

void FDisplayClusterConfiguratorProjectionCustomization::CreateCameraPolicy(UDisplayClusterBlueprint* Blueprint)
{
	CustomPolicyParameters.Add(MakeShared<FPolicyParameterInfoComponentCombo>(
		"Camera",
		DisplayClusterProjectionStrings::cfg::camera::Component,
		Blueprint,
		ConfigurationViewports,
		TArray<TSubclassOf<UActorComponent>>{ UCameraComponent::StaticClass() } ));

	CustomPolicyParameters.Add(MakeShared<FPolicyParameterInfoBool>(
		"Use nDisplay Renderer",
		DisplayClusterProjectionStrings::cfg::camera::Native,
		Blueprint,
		ConfigurationViewports,
		true));
}

void FDisplayClusterConfiguratorProjectionCustomization::CreateMeshPolicy(UDisplayClusterBlueprint* Blueprint)
{
	CustomPolicyParameters.Add(MakeShared<FPolicyParameterInfoComponentCombo>(
		"Mesh",
		DisplayClusterProjectionStrings::cfg::mesh::Component,
		Blueprint,
		ConfigurationViewports,
		TArray<TSubclassOf<UActorComponent>>{ UStaticMeshComponent::StaticClass(), UProceduralMeshComponent::StaticClass()}));

	TSharedPtr<FPolicyParameterInfo> SectionIndexParameterRef = MakeShared<FPolicyParameterInfoNumber<int32>>(
		"Section Index",
		DisplayClusterProjectionStrings::cfg::mesh::SectionIndex,
		Blueprint,
		ConfigurationViewports, 0, 0);
	SectionIndexParameterRef->SetParameterTooltip(LOCTEXT("MeshPolicySectionIndexTooltip", "Section index with geometry in the ProceduralMesh component"));

	TSharedPtr<FPolicyParameterInfo> BaseUVIndexParameterRef = MakeShared<FPolicyParameterInfoNumber<int32>>(
		"Warp UVs",
		DisplayClusterProjectionStrings::cfg::mesh::BaseUVIndex,
		Blueprint,
		ConfigurationViewports, INDEX_NONE, INDEX_NONE);
	BaseUVIndexParameterRef->SetParameterTooltip(LOCTEXT("MeshPolicyBaseUVIndexTooltip", "Define custom UV channel in the original geometry as UV for warp. The default is -1."));

	TSharedPtr<FPolicyParameterInfo> ChromakeyUVIndexParameterRef = MakeShared<FPolicyParameterInfoNumber<int32>>(
		"Chromakey UVs",
		DisplayClusterProjectionStrings::cfg::mesh::ChromakeyUVIndex,
		Blueprint,
		ConfigurationViewports, INDEX_NONE, INDEX_NONE);
	ChromakeyUVIndexParameterRef->SetParameterTooltip(LOCTEXT("MeshPolicyChromakeyUVIndexTooltip", "Define custom UV channel in the original geometry as UV for ChromakeyMarkers. The default is -1."));

	CustomPolicyParameters.Add(SectionIndexParameterRef);
	CustomPolicyParameters.Add(BaseUVIndexParameterRef);
	CustomPolicyParameters.Add(ChromakeyUVIndexParameterRef);
}

void FDisplayClusterConfiguratorProjectionCustomization::CreateDomePolicy(UDisplayClusterBlueprint* Blueprint)
{
	CustomPolicyParameters.Add(MakeShared<FPolicyParameterInfoFile>(
		"File",
		DisplayClusterProjectionStrings::cfg::domeprojection::File,
		Blueprint,
		ConfigurationViewports,
		TArray<FString>{"xml"}));

	CustomPolicyParameters.Add(MakeShared<FPolicyParameterInfoComponentCombo>(
		"Origin",
		DisplayClusterProjectionStrings::cfg::domeprojection::Origin,
		Blueprint,
		ConfigurationViewports,
		TArray<TSubclassOf<UActorComponent>>{ USceneComponent::StaticClass() }));

	CustomPolicyParameters.Add(MakeShared<FPolicyParameterInfoNumber<int32>>(
		"Channel",
		DisplayClusterProjectionStrings::cfg::domeprojection::Channel,
		Blueprint,
		ConfigurationViewports));
}

void FDisplayClusterConfiguratorProjectionCustomization::CreateVIOSOPolicy(UDisplayClusterBlueprint* Blueprint)
{
	const FString VIOSOTypeKey = "VIOSOType";
	const FString TypeVIOSO    = "VIOSO";
	const FString TypeVWF      = "Explicit VWF";

	auto RefreshPolicy = [this](const FString& SelectedItem)
	{
		PropertyUtilities.Pin()->ForceRefresh();
	};

	const bool bSort = false;

	const TSharedPtr<FPolicyParameterInfoCombo> VIOSOCombo = MakeShared<FPolicyParameterInfoCombo>(
		"VIOSO Type",
		VIOSOTypeKey,
		Blueprint,
		ConfigurationViewports,
		TArray<FString>{TypeVIOSO, TypeVWF},
		&TypeVWF,
		bSort);
	VIOSOCombo->SetOnSelectedDelegate(FPolicyParameterInfoCombo::FOnItemSelected::CreateLambda(RefreshPolicy));
	CustomPolicyParameters.Add(VIOSOCombo);

	const FString Setting = VIOSOCombo->GetOrAddCustomParameterValueText().ToString();
	if (Setting == TypeVIOSO)
	{
		CustomPolicyParameters.Add(MakeShared<FPolicyParameterInfoFile>(
			"INI File",
			DisplayClusterProjectionStrings::cfg::VIOSO::INIFile,
			Blueprint,
			ConfigurationViewports,
			TArray<FString>{"ini"}));
		CustomPolicyParameters.Last()->SetParameterTooltip(LOCTEXT("DisplayClusterProjectionVIOSOPolicyFileINI", "File containing calibration settings."));

		// channel name from ini file, empty by default, which means getting the first matching one.
		CustomPolicyParameters.Add(MakeShared<FPolicyParameterInfoNumber<int32>>(
			"Channel Name",
			DisplayClusterProjectionStrings::cfg::VIOSO::ChannelName,
			Blueprint,
			ConfigurationViewports));
		CustomPolicyParameters.Last()->SetParameterTooltip(LOCTEXT("DisplayClusterProjectionVIOSOPolicyChannelName", "The VIOSO calibration file contains several instances of geometry (for several screens in one file), so we must use this parameter to be able to select the right one."));
	}
	else if (Setting == TypeVWF)
	{
		CustomPolicyParameters.Add(MakeShared<FPolicyParameterInfoFile>(
			"VWF File",
			DisplayClusterProjectionStrings::cfg::VIOSO::File,
			Blueprint,
			ConfigurationViewports,
			TArray<FString>{"vwf"}));
		CustomPolicyParameters.Last()->SetParameterTooltip(LOCTEXT("DisplayClusterProjectionVIOSOPolicyFile", "File containing one or more geometries created in the VIOSO calibration app."));

		const float DefaultUnitsInMeter = 1000.f;
		CustomPolicyParameters.Add(MakeShared<FPolicyParameterInfoNumber<float>>(
			"Units In Meter",
			DisplayClusterProjectionStrings::cfg::VIOSO::UnitsInMeter,
			Blueprint,
			ConfigurationViewports, DefaultUnitsInMeter));
		CustomPolicyParameters.Last()->SetParameterTooltip(LOCTEXT("DisplayClusterProjectionVIOSOPolicyUnitsInMeter", "How many VIOSO units in meter. Specifies the unit of the calibration geometry in meters."));

		// the calibration index in mapping file, defaults to 0,
		// The VIOSO calibration file contains several instances of geometry (for several screens in one file), so we must use this parameter to be able to select the right one.
		CustomPolicyParameters.Add(MakeShared<FPolicyParameterInfoNumber<int32>>(
			"Calib Index",
			DisplayClusterProjectionStrings::cfg::VIOSO::CalibIndex,
			Blueprint,
			ConfigurationViewports));
		CustomPolicyParameters.Last()->SetParameterTooltip(LOCTEXT("DisplayClusterProjectionVIOSOPolicyCalibIndex", "The VIOSO calibration file contains several instances of geometry (for several screens in one file), so we must use this parameter to be able to select the right one."));
	}

	CustomPolicyParameters.Add(MakeShared<FPolicyParameterInfoComponentCombo>(
		"Origin",
		DisplayClusterProjectionStrings::cfg::VIOSO::Origin,
		Blueprint,
		ConfigurationViewports,
		TArray<TSubclassOf<UActorComponent>>{ USceneComponent::StaticClass() }));
	CustomPolicyParameters.Last()->SetParameterTooltip(LOCTEXT("DisplayClusterProjectionVIOSOPolicyOrigin", "The VIOSO calibration app uses a special point in real world space and uses it as zero. The same point must be defined on the UE side inside DCRA."));

	CustomPolicyParameters.Add(MakeShared<FPolicyParameterInfoBool>(
		"Enable Preview",
		DisplayClusterProjectionStrings::cfg::VIOSO::EnablePreview,
		Blueprint,
		ConfigurationViewports));
}

void FDisplayClusterConfiguratorProjectionCustomization::CreateEasyBlendPolicy(UDisplayClusterBlueprint* Blueprint)
{
	CustomPolicyParameters.Add(MakeShared<FPolicyParameterInfoFile>(
		"File",
		DisplayClusterProjectionStrings::cfg::easyblend::File,
		Blueprint,
		ConfigurationViewports,
		TArray<FString>{"pol*", "ol*"}));

	CustomPolicyParameters.Add(MakeShared<FPolicyParameterInfoComponentCombo>(
		"Origin Component",
		DisplayClusterProjectionStrings::cfg::easyblend::Origin,
		Blueprint,
		ConfigurationViewports,
		TArray<TSubclassOf<UActorComponent>>{ USceneComponent::StaticClass() }));

	CustomPolicyParameters.Add(MakeShared<FPolicyParameterInfoVector>(
		"Origin Location",
		DisplayClusterProjectionStrings::cfg::easyblend::OriginOffset,
		Blueprint,
		ConfigurationViewports));

	CustomPolicyParameters.Add(MakeShared<FPolicyParameterInfoNumber<float>>(
		"Scale",
		DisplayClusterProjectionStrings::cfg::easyblend::Scale,
		Blueprint,
		ConfigurationViewports,
		1.f, 0.0001f));

	// EasyBlend D3D12 supports export of geometry
	CustomPolicyParameters.Add(MakeShared<FPolicyParameterInfoBool>(
		"Enable Preview",
		DisplayClusterProjectionStrings::cfg::mpcdi::EnablePreview,
		Blueprint,
		ConfigurationViewports));
}

void FDisplayClusterConfiguratorProjectionCustomization::CreateReferencePolicy(UDisplayClusterBlueprint* Blueprint)
{
	TSharedPtr<FPolicyParameterInfoViewportCombo> ViewportCombo = MakeShared<FPolicyParameterInfoViewportCombo>(
		"Source Viewport",
		DisplayClusterProjectionStrings::cfg::reference::ViewportId,
		Blueprint,
		ConfigurationViewports);

	ViewportCombo->SetParameterTooltip(LOCTEXT("ReferencePolicySourceViewportTooltip",
		"The viewport to reference for this projection. Only viewports within the same cluster node can be referenced. "
		"This option is disabled when editing viewports from different cluster nodes."));

	CustomPolicyParameters.Add(ViewportCombo);
}

void FDisplayClusterConfiguratorProjectionCustomization::CreateManualPolicy(UDisplayClusterBlueprint* Blueprint)
{
	check(Blueprint);
	
	const FString RenderingKey = DisplayClusterProjectionStrings::cfg::manual::Rendering;
	const FString RenderingMono = DisplayClusterProjectionStrings::cfg::manual::RenderingType::Mono;
	const FString RenderingStereo = DisplayClusterProjectionStrings::cfg::manual::RenderingType::Stereo;
	const FString RenderingMonoStereo = DisplayClusterProjectionStrings::cfg::manual::RenderingType::MonoStereo;

	const FString FrustumKey = DisplayClusterProjectionStrings::cfg::manual::Type;
	const FString FrustumMatrix = DisplayClusterProjectionStrings::cfg::manual::FrustumType::Matrix;
	const FString FrustumAngles = DisplayClusterProjectionStrings::cfg::manual::FrustumType::Angles;

	auto RefreshPolicy = [this](const FString& SelectedItem)
	{
		PropertyUtilities.Pin()->ForceRefresh();
	};

	const bool bSort = false;
	
	const TSharedPtr<FPolicyParameterInfoCombo> RenderingCombo = MakeShared<FPolicyParameterInfoCombo>(
		"Rendering",
		RenderingKey,
		Blueprint,
		ConfigurationViewports,
		//TArray<FString>{RenderingMono, RenderingStereo, RenderingMonoStereo}, temporarily disabled MonoStereo, not supported  implementation from projection policy side
		TArray<FString>{RenderingMono, RenderingStereo},
		& RenderingMono,
		bSort);
	RenderingCombo->SetOnSelectedDelegate(FPolicyParameterInfoCombo::FOnItemSelected::CreateLambda(RefreshPolicy));
	CustomPolicyParameters.Add(RenderingCombo);

	const TSharedPtr<FPolicyParameterInfoCombo> FrustumCombo = MakeShared<FPolicyParameterInfoCombo>(
		"Frustum",
		FrustumKey,
		Blueprint,
		ConfigurationViewports,
		TArray<FString>{FrustumMatrix, FrustumAngles},
		& FrustumMatrix,
		bSort);
	FrustumCombo->SetOnSelectedDelegate(FPolicyParameterInfoCombo::FOnItemSelected::CreateLambda(RefreshPolicy));
	CustomPolicyParameters.Add(FrustumCombo);

	/*
	 * Rotation
	 */
	{
		CustomPolicyParameters.Add(MakeShared<FPolicyParameterInfoRotator>(
			"Rotation",
			DisplayClusterProjectionStrings::cfg::manual::Rotation,
			Blueprint,
			ConfigurationViewports));
	}
	
	/*
	 * Matrices
	 */
	{
		auto IsMatrixVisible = [RenderingCombo, RenderingMono, RenderingMonoStereo, FrustumCombo, FrustumMatrix]() -> bool
		{
			const FString RenderSetting = RenderingCombo->GetOrAddCustomParameterValueText().ToString();
			const FString FrustumSetting = FrustumCombo->GetOrAddCustomParameterValueText().ToString();

			const bool bVisible = (*RenderSetting == RenderingMono && *FrustumSetting == FrustumMatrix)
				|| (*RenderSetting == RenderingMonoStereo && *FrustumSetting == FrustumMatrix);

			return bVisible;
		};

		if (IsMatrixVisible())
		{
			const TSharedPtr<FPolicyParameterInfo4x4Matrix> MatrixPolicy = MakeShared<FPolicyParameterInfo4x4Matrix>(
				"Matrix",
				DisplayClusterProjectionStrings::cfg::manual::Matrix,
				Blueprint,
				ConfigurationViewports);
			CustomPolicyParameters.Add(MatrixPolicy);
		}
	}

	{
		auto IsMatrixLeftRightVisible = [RenderingCombo, RenderingStereo, RenderingMonoStereo, FrustumCombo, FrustumMatrix]() -> bool
		{
			const FString RenderSetting = RenderingCombo->GetOrAddCustomParameterValueText().ToString();
			const FString FrustumSetting = FrustumCombo->GetOrAddCustomParameterValueText().ToString();

			const bool bVisible = (*RenderSetting == RenderingStereo && *FrustumSetting == FrustumMatrix)
				|| (*RenderSetting == RenderingMonoStereo && *FrustumSetting == FrustumMatrix);

			return bVisible;
		};

		if (IsMatrixLeftRightVisible())
		{
			const TSharedPtr<FPolicyParameterInfo4x4Matrix> MatrixLeftPolicy = MakeShared<FPolicyParameterInfo4x4Matrix>(
				"MatrixLeft",
				DisplayClusterProjectionStrings::cfg::manual::MatrixLeft,
				Blueprint,
				ConfigurationViewports);
			CustomPolicyParameters.Add(MatrixLeftPolicy);

			const TSharedPtr<FPolicyParameterInfo4x4Matrix> MatrixRightPolicy =
				MakeShared<FPolicyParameterInfo4x4Matrix>(
					"MatrixRight",
					DisplayClusterProjectionStrings::cfg::manual::MatrixRight,
					Blueprint,
					ConfigurationViewports);
			CustomPolicyParameters.Add(MatrixRightPolicy);
		}
	}

	/*
	 * Frustums
	 */
	{
		auto IsFrustumVisible = [RenderingCombo, RenderingMono, RenderingMonoStereo, FrustumCombo, FrustumAngles]() -> bool
		{
			const FString RenderSetting = RenderingCombo->GetOrAddCustomParameterValueText().ToString();
			const FString FrustumSetting = FrustumCombo->GetOrAddCustomParameterValueText().ToString();

			const bool bVisible = ((*RenderSetting == RenderingMono || *RenderSetting == RenderingMonoStereo) && *FrustumSetting == FrustumAngles);
			return bVisible;
		};

		if (IsFrustumVisible())
		{
			CustomPolicyParameters.Add(MakeShared<FPolicyParameterInfoFrustumAngle>(
				"Frustum",
				DisplayClusterProjectionStrings::cfg::manual::Frustum,
				Blueprint,
				ConfigurationViewports));
		}

		auto IsLeftRightFrustumVisible = [RenderingCombo, RenderingStereo, RenderingMonoStereo, FrustumCombo, FrustumAngles]() -> bool
		{
			const FString RenderSetting = RenderingCombo->GetOrAddCustomParameterValueText().ToString();
			const FString FrustumSetting = FrustumCombo->GetOrAddCustomParameterValueText().ToString();

			const bool bVisible = ((*RenderSetting == RenderingStereo || *RenderSetting == RenderingMonoStereo) && *FrustumSetting == FrustumAngles);
			return bVisible;
		};

		if (IsLeftRightFrustumVisible())
		{
			CustomPolicyParameters.Add(MakeShared<FPolicyParameterInfoFrustumAngle>(
				"FrustumLeft",
				DisplayClusterProjectionStrings::cfg::manual::FrustumLeft,
				Blueprint,
				ConfigurationViewports));

			CustomPolicyParameters.Add(MakeShared<FPolicyParameterInfoFrustumAngle>(
				"FrustumRight",
				DisplayClusterProjectionStrings::cfg::manual::FrustumRight,
				Blueprint,
				ConfigurationViewports));
		}
	}
}

void FDisplayClusterConfiguratorProjectionCustomization::CreateMPCDIPolicy(UDisplayClusterBlueprint* Blueprint)
{
	check(Blueprint);

	const FString MPCDITypeKey = "MPCDIType";
	const FString TypeMPCDI = "MPCDI";
	const FString TypePFM = "Explicit PFM";
	
	auto RefreshPolicy = [this](const FString& SelectedItem)
	{
		PropertyUtilities.Pin()->ForceRefresh();
	};

	const bool bSort = false;

	const TSharedPtr<FPolicyParameterInfoCombo> MPCDICombo = MakeShared<FPolicyParameterInfoCombo>(
		"MPCDI Type",
		MPCDITypeKey,
		Blueprint,
		ConfigurationViewports,
		TArray<FString>{TypeMPCDI, TypePFM},
		&TypeMPCDI,
		bSort);
	MPCDICombo->SetOnSelectedDelegate(FPolicyParameterInfoCombo::FOnItemSelected::CreateLambda(RefreshPolicy));
	CustomPolicyParameters.Add(MPCDICombo);

	const FString Setting = MPCDICombo->GetOrAddCustomParameterValueText().ToString();
	if (Setting == TypeMPCDI)
	{
		CustomPolicyParameters.Add(MakeShared<FPolicyParameterInfoFile>(
			"File",
			DisplayClusterProjectionStrings::cfg::mpcdi::File,
			Blueprint,
			ConfigurationViewports,
			TArray<FString>{"mpcdi"}));

		CustomPolicyParameters.Add(MakeShared<FPolicyParameterInfoText>(
			"Buffer",
			DisplayClusterProjectionStrings::cfg::mpcdi::Buffer,
			Blueprint,
			ConfigurationViewports));

		CustomPolicyParameters.Add(MakeShared<FPolicyParameterInfoText>(
			"Region",
			DisplayClusterProjectionStrings::cfg::mpcdi::Region,
			Blueprint,
			ConfigurationViewports));

		CustomPolicyParameters.Add(MakeShared<FPolicyParameterInfoText>(
			"Screen",
			DisplayClusterProjectionStrings::cfg::mpcdi::Component,
			Blueprint,
			ConfigurationViewports));
	}
	else if (Setting == TypePFM)
	{
		const FString MPCDIProfile = DisplayClusterProjectionStrings::cfg::mpcdi::Profiles::mpcdi_a3d;
		const TSharedPtr<FPolicyParameterInfoCombo> MPCDIProfilesCombo = MakeShared<FPolicyParameterInfoCombo>(
			"Profile",
			DisplayClusterProjectionStrings::cfg::mpcdi::MPCDIType,
			Blueprint,
			ConfigurationViewports,
			TArray<FString>{
				// Note: The MPCDI/PFM configuration currently supports only 2 mpcdi profiles.
				DisplayClusterProjectionStrings::cfg::mpcdi::Profiles::mpcdi_2d,
				DisplayClusterProjectionStrings::cfg::mpcdi::Profiles::mpcdi_a3d
			},
			& MPCDIProfile,
			false);

		CustomPolicyParameters.Add(MPCDIProfilesCombo);
		MPCDIProfilesCombo->SetOnSelectedDelegate(FPolicyParameterInfoCombo::FOnItemSelected::CreateLambda(RefreshPolicy));
		const FString CurrentMPCDIProfile = MPCDIProfilesCombo->GetOrAddCustomParameterValueText().ToString();

		CustomPolicyParameters.Add(MakeShared<FPolicyParameterInfoFile>(
			"File",
			DisplayClusterProjectionStrings::cfg::mpcdi::FilePFM,
			Blueprint,
			ConfigurationViewports,
			TArray<FString>{"pfm"}));

		if (CurrentMPCDIProfile == DisplayClusterProjectionStrings::cfg::mpcdi::Profiles::mpcdi_2d)
		{
			// Adds additional parameters for the mpcdi 2d profile.
			// <buffer Xresolution, Yresolution> <region x, y, xsize, ysize>

			CustomPolicyParameters.Add(MakeShared<FPolicyParameterInfoResolution>(
				"Buffer Resolution",
				DisplayClusterProjectionStrings::cfg::mpcdi::Attributes::Buffer::Resolution,
				Blueprint,
				ConfigurationViewports));

			CustomPolicyParameters.Add(MakeShared<FPolicyParameterInfoNormalizedVector2D>(
				"Region Pos",
				DisplayClusterProjectionStrings::cfg::mpcdi::Attributes::Region::Pos,
				Blueprint,
				ConfigurationViewports));

			const FString RegionDefaultSize("X=1, Y=1");
			CustomPolicyParameters.Add(MakeShared<FPolicyParameterInfoNormalizedVector2D>(
				"Region Size",
				DisplayClusterProjectionStrings::cfg::mpcdi::Attributes::Region::Size,
				Blueprint,
				ConfigurationViewports,&RegionDefaultSize));
		}

		CustomPolicyParameters.Add(MakeShared<FPolicyParameterInfoFile>(
			"Alpha Mask",
			DisplayClusterProjectionStrings::cfg::mpcdi::FileAlpha,
			Blueprint,
			ConfigurationViewports,
			TArray<FString>{"png"}));
		
		CustomPolicyParameters.Add(MakeShared<FPolicyParameterInfoNumber<float>>(
			"Alpha Gamma",
			DisplayClusterProjectionStrings::cfg::mpcdi::AlphaGamma,
			Blueprint,
			ConfigurationViewports,
			1.f));

		CustomPolicyParameters.Add(MakeShared<FPolicyParameterInfoFile>(
			"Beta Mask",
			DisplayClusterProjectionStrings::cfg::mpcdi::FileBeta,
			Blueprint,
			ConfigurationViewports,
			TArray<FString>{"png"}));

		CustomPolicyParameters.Add(MakeShared<FPolicyParameterInfoNumber<float>>(
			"Scale",
			DisplayClusterProjectionStrings::cfg::mpcdi::WorldScale,
			Blueprint,
			ConfigurationViewports,
			1.f));
		
		CustomPolicyParameters.Add(MakeShared<FPolicyParameterInfoBool>(
			"Use Unreal Axis",
			DisplayClusterProjectionStrings::cfg::mpcdi::UseUnrealAxis,
			Blueprint,
			ConfigurationViewports));
	}
	
	CustomPolicyParameters.Add(MakeShared<FPolicyParameterInfoComponentCombo>(
			"Origin", 
			DisplayClusterProjectionStrings::cfg::mpcdi::Origin, 
			Blueprint, ConfigurationViewports, 
			TArray<TSubclassOf<UActorComponent>>{ USceneComponent::StaticClass() }));

	CustomPolicyParameters.Add(MakeShared<FPolicyParameterInfoBool>(
		"Enable Preview",
		DisplayClusterProjectionStrings::cfg::mpcdi::EnablePreview,
		Blueprint,
		ConfigurationViewports));
}


//////////////////////////////////////////////////////////////////////////////////////////////
// Render Sync Type Customization
//////////////////////////////////////////////////////////////////////////////////////////////

const FString FDisplayClusterConfiguratorRenderSyncPolicyCustomization::SwapGroupName = TEXT("SwapGroup");
const FString FDisplayClusterConfiguratorRenderSyncPolicyCustomization::SwapBarrierName = TEXT("SwapBarrier");

void FDisplayClusterConfiguratorRenderSyncPolicyCustomization::Initialize(const TSharedRef<IPropertyHandle>& InPropertyHandle, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	FDisplayClusterConfiguratorPolymorphicEntityCustomization::Initialize(InPropertyHandle, CustomizationUtils);

	SwapGroupValue = SwapBarrierValue = 1;
	NvidiaOption = MakeShared<FString>("Nvidia");
	CustomOption = MakeShared<FString>("Custom");

	// Get the Editing object
	TArray<UObject*> OuterObjects;
	InPropertyHandle->GetOuterObjects(OuterObjects);
	if (OuterObjects.Num())
	{
		ConfigurationClusterPtr = Cast<UDisplayClusterConfigurationCluster>(OuterObjects[0]);
	}
	check(ConfigurationClusterPtr != nullptr);

	// Set initial Nvidia option values
	{
		UDisplayClusterConfigurationCluster* ConfigurationCluster = ConfigurationClusterPtr.Get();
		FString* SwapGroupParam = ConfigurationCluster->Sync.RenderSyncPolicy.Parameters.Find(SwapGroupName);
		if (SwapGroupParam != nullptr)
		{
			LexTryParseString<int32>(SwapGroupValue, **SwapGroupParam);
		}

		FString* SwapBarrierParam = ConfigurationCluster->Sync.RenderSyncPolicy.Parameters.Find(SwapBarrierName);
		if (SwapBarrierParam != nullptr)
		{
			LexTryParseString<int32>(SwapBarrierValue, **SwapBarrierParam);
		}
	}

	bIsCustomPolicy = IsCustomTypeInConfig();
	if (bIsCustomPolicy)
	{
		// Load default config
		UDisplayClusterConfigurationCluster* ConfigurationCluster = ConfigurationClusterPtr.Get();
		CustomPolicy = ConfigurationCluster->Sync.RenderSyncPolicy.Type;
	}
}

void FDisplayClusterConfiguratorRenderSyncPolicyCustomization::SetChildren(const TSharedRef<IPropertyHandle>& InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	// Hide properties 
	TypeHandle->MarkHiddenByCustomization();

	// Add custom rows
	ResetRenderSyncPolicyOptions();
	AddRenderSyncPolicyRow(InChildBuilder);
	AddNvidiaPolicyRows(InChildBuilder);
	AddCustomPolicyRow(InChildBuilder);

	// Add Parameters property with Visibility handler
	InChildBuilder
		.AddProperty(ParametersHandle.ToSharedRef())
		.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FDisplayClusterConfiguratorRenderSyncPolicyCustomization::GetCustomRowsVisibility)))
		.ShouldAutoExpand(true);
}

EVisibility FDisplayClusterConfiguratorRenderSyncPolicyCustomization::GetCustomRowsVisibility() const
{
	if (bIsCustomPolicy)
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

EVisibility FDisplayClusterConfiguratorRenderSyncPolicyCustomization::GetNvidiaPolicyRowsVisibility() const
{
	if (UDisplayClusterConfigurationCluster* ConfigurationCluster = ConfigurationClusterPtr.Get())
	{
		if (ConfigurationCluster->Sync.RenderSyncPolicy.Type.Equals(*NvidiaOption.Get()))
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}

void FDisplayClusterConfiguratorRenderSyncPolicyCustomization::ResetRenderSyncPolicyOptions()
{
	UDisplayClusterConfigurationCluster* ConfigurationCluster = ConfigurationClusterPtr.Get();
	check(ConfigurationCluster != nullptr);

	RenderSyncPolicyOptions.Reset();
	for (const FString& RenderSyncPolicy : UDisplayClusterConfigurationData::RenderSyncPolicies)
	{
		RenderSyncPolicyOptions.Add(MakeShared<FString>(RenderSyncPolicy));
	}

	// Add Custom option
	if (!bIsCustomPolicy)
	{
		RenderSyncPolicyOptions.Add(CustomOption);
	}
}

void FDisplayClusterConfiguratorRenderSyncPolicyCustomization::AddRenderSyncPolicyRow(IDetailChildrenBuilder& InChildBuilder)
{
	if (RenderSyncPolicyComboBox.IsValid())
	{
		return;
	}

	InChildBuilder.AddCustomRow(TypeHandle->GetPropertyDisplayName())
	.NameContent()
	[
		TypeHandle->CreatePropertyNameWidget(FText::GetEmpty(), LOCTEXT("RenderSyncPolicyToolTip", "Specify your nDisplay Render Sync Policy"))
	]
	.ValueContent()
	[
		SAssignNew(RenderSyncPolicyComboBox, SDisplayClusterConfigurationSearchableComboBox)
		.OptionsSource(&RenderSyncPolicyOptions)
		.OnGenerateWidget(this, &FDisplayClusterConfiguratorRenderSyncPolicyCustomization::MakeRenderSyncPolicyOptionComboWidget)
		.OnSelectionChanged(this, &FDisplayClusterConfiguratorRenderSyncPolicyCustomization::OnRenderSyncPolicySelected)
		.ContentPadding(2)
		.MaxListHeight(200.0f)
		.Content()
		[
			SNew(STextBlock)
			.Text(this, &FDisplayClusterConfiguratorRenderSyncPolicyCustomization::GetSelectedRenderSyncPolicyText)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
	];
}

void FDisplayClusterConfiguratorRenderSyncPolicyCustomization::AddNvidiaPolicyRows(IDetailChildrenBuilder& InChildBuilder)
{
	InChildBuilder.AddCustomRow(TypeHandle->GetPropertyDisplayName())
	.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FDisplayClusterConfiguratorRenderSyncPolicyCustomization::GetNvidiaPolicyRowsVisibility)))
	.NameContent()
	[
		SNew(STextBlock)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Text(LOCTEXT("SwapGroup", "Swap Group"))
	]
	.ValueContent()
	[
		SAssignNew(SwapGroupSpinBox, SSpinBox<int32>)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.MinValue(1)
		.MaxValue(9)
		.Value_Lambda([this]()
		{
			return SwapGroupValue;
		})
		.OnValueChanged_Lambda([this](int32 InValue)
		{
			SwapGroupValue = InValue;
			AddToParameterMap(SwapGroupName, FString::FromInt(SwapGroupValue));
		})
	];

	InChildBuilder.AddCustomRow(TypeHandle->GetPropertyDisplayName())
	.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FDisplayClusterConfiguratorRenderSyncPolicyCustomization::GetNvidiaPolicyRowsVisibility)))
	.NameContent()
	[
		SNew(STextBlock)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Text(LOCTEXT("SwapBarrier", "Swap Barrier"))
	]
	.ValueContent()
	[
		SAssignNew(SwapBarrierSpinBox, SSpinBox<int32>)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.MinValue(1)
		.MaxValue(9)
		.Value_Lambda([this]()
		{
			return SwapBarrierValue;
		})
		.OnValueChanged_Lambda([this](int32 InValue)
		{
			SwapBarrierValue = InValue;
			AddToParameterMap(SwapBarrierName, FString::FromInt(SwapBarrierValue));
		})
	];
}

void FDisplayClusterConfiguratorRenderSyncPolicyCustomization::AddCustomPolicyRow(IDetailChildrenBuilder& InChildBuilder)
{
	if (CustomPolicyRow.IsValid())
	{
		return;
	}

	FText SyncProjectionName = LOCTEXT("SyncProjectionName", "Name");

	InChildBuilder.AddCustomRow(SyncProjectionName)
	.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FDisplayClusterConfiguratorRenderSyncPolicyCustomization::GetCustomRowsVisibility)))
	.NameContent()
	[
		SNew(STextBlock).Text(SyncProjectionName)
	]
	.ValueContent()
	[
		SAssignNew(CustomPolicyRow, SEditableTextBox)
		.Text(this, &FDisplayClusterConfiguratorRenderSyncPolicyCustomization::GetCustomPolicyText)
		.OnTextCommitted(this, &FDisplayClusterConfiguratorRenderSyncPolicyCustomization::OnTextCommittedInCustomPolicyText)
		.Font(IDetailLayoutBuilder::GetDetailFont())
	];
}

TSharedRef<SWidget> FDisplayClusterConfiguratorRenderSyncPolicyCustomization::MakeRenderSyncPolicyOptionComboWidget(TSharedPtr<FString> InItem)
{
	FString TypeStr = *InItem;
	int32 TypeIndex = GetPolicyTypeIndex(TypeStr);

	FText DisplayText;
	if (TypeIndex > INDEX_NONE)
	{
		DisplayText = FText::Format(LOCTEXT("RenderPolicyTypeDisplayFormat", "{0} ({1})"), FText::FromString(TypeStr), FText::AsNumber(TypeIndex));
	}
	else
	{
		DisplayText = FText::FromString(TypeStr);
	}

	return SNew(STextBlock)
		.Text(DisplayText)
		.Font(IDetailLayoutBuilder::GetDetailFont());
}

void FDisplayClusterConfiguratorRenderSyncPolicyCustomization::OnRenderSyncPolicySelected(TSharedPtr<FString> InPolicy, ESelectInfo::Type SelectInfo)
{
	if (InPolicy.IsValid())
	{
		UDisplayClusterConfigurationCluster* ConfigurationCluster = ConfigurationClusterPtr.Get();
		check(ConfigurationCluster != nullptr);

		const FString SelectedPolicy = *InPolicy.Get();

		ConfigurationCluster->Modify();
		ModifyBlueprint();

		if (SelectedPolicy.Equals(*CustomOption.Get()))
		{
			bIsCustomPolicy = true;
			TypeHandle->SetValue(CustomPolicy);
			IsCustomHandle->SetValue(true);
		}
		else
		{
			bIsCustomPolicy = false;
			TypeHandle->SetValue(SelectedPolicy);
			IsCustomHandle->SetValue(false);
		}

		if (ConfigurationCluster->Sync.RenderSyncPolicy.Type.Equals(*NvidiaOption.Get()))
		{
			AddToParameterMap(SwapGroupName, FString::FromInt(SwapGroupValue));
			AddToParameterMap(SwapBarrierName, FString::FromInt(SwapBarrierValue));
		}
		else
		{
			RemoveFromParameterMap(SwapGroupName);
			RemoveFromParameterMap(SwapBarrierName);
		}

		// Reset available options
		ResetRenderSyncPolicyOptions();
		RenderSyncPolicyComboBox->ResetOptionsSource(&RenderSyncPolicyOptions);
		RenderSyncPolicyComboBox->SetIsOpen(false);
	}
}

FText FDisplayClusterConfiguratorRenderSyncPolicyCustomization::GetSelectedRenderSyncPolicyText() const
{
	UDisplayClusterConfigurationCluster* ConfigurationCluster = ConfigurationClusterPtr.Get();
	if (ConfigurationCluster == nullptr)
	{
		return FText::GetEmpty();
	}

	if (bIsCustomPolicy)
	{
		return FText::FromString(*CustomOption.Get());
	}

	FString TypeStr = ConfigurationCluster->Sync.RenderSyncPolicy.Type;
	int32 TypeIndex = GetPolicyTypeIndex(TypeStr);

	if (TypeIndex > INDEX_NONE)
	{
		return FText::Format(LOCTEXT("RenderPolicyTypeDisplayFormat", "{0} ({1})"), FText::FromString(TypeStr), FText::AsNumber(TypeIndex));
	}
	else
	{
		return FText::FromString(TypeStr);
	}
}

FText FDisplayClusterConfiguratorRenderSyncPolicyCustomization::GetCustomPolicyText() const
{
	return FText::FromString(CustomPolicy);
}

bool FDisplayClusterConfiguratorRenderSyncPolicyCustomization::IsCustomTypeInConfig() const
{
	UDisplayClusterConfigurationCluster* ConfigurationCluster = ConfigurationClusterPtr.Get();
	check(ConfigurationCluster != nullptr);

	if (ConfigurationCluster->Sync.RenderSyncPolicy.bIsCustom)
	{
		return true;
	}

	for (const FString& Policy : UDisplayClusterConfigurationData::RenderSyncPolicies)
	{
		if (ConfigurationCluster->Sync.RenderSyncPolicy.Type.ToLower().Equals(Policy.ToLower()))
		{
			return false;
		}
	}

	return true;
}

int32 FDisplayClusterConfiguratorRenderSyncPolicyCustomization::GetPolicyTypeIndex(const FString& Type) const
{
	int32 TypeIndex = INDEX_NONE;

	if (Type.ToLower().Equals(DisplayClusterConfigurationStrings::config::cluster::render_sync::None))
	{
		TypeIndex = 0;
	}
	else if (Type.ToLower().Equals(DisplayClusterConfigurationStrings::config::cluster::render_sync::Ethernet))
	{
		TypeIndex = 1;
	}
	else if (Type.ToLower().Equals(DisplayClusterConfigurationStrings::config::cluster::render_sync::Nvidia))
	{
		TypeIndex = 2;
	}

	return TypeIndex;
}

void FDisplayClusterConfiguratorRenderSyncPolicyCustomization::OnTextCommittedInCustomPolicyText(const FText& InValue, ETextCommit::Type CommitType)
{
	CustomPolicy = InValue.ToString();

	// Update Config
	UDisplayClusterConfigurationCluster* ConfigurationCluster = ConfigurationClusterPtr.Get();
	check(ConfigurationCluster != nullptr);

	TypeHandle->SetValue(CustomPolicy);

	// Check if the custom config same as any of the ProjectionPoliсies configs 
	bIsCustomPolicy = true;
	for (const FString& ProjectionPolicy : UDisplayClusterConfigurationData::ProjectionPolicies)
	{
		if (CustomPolicy.Equals(ProjectionPolicy))
		{
			bIsCustomPolicy = false;

			RenderSyncPolicyComboBox->SetSelectedItem(MakeShared<FString>(CustomPolicy));

			break;
		}
	}
}

void FDisplayClusterConfiguratorRenderSyncPolicyCustomization::AddToParameterMap(const FString& Key,
	const FString& Value)
{
	UDisplayClusterConfigurationCluster* ConfigurationCluster = ConfigurationClusterPtr.Get();
	check(ConfigurationCluster != nullptr);

	FStructProperty* SyncStructProperty = FindFProperty<FStructProperty>(ConfigurationCluster->GetClass(), GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationCluster, Sync));
	check(SyncStructProperty);

	FStructProperty* RenderStructProperty = FindFProperty<FStructProperty>(SyncStructProperty->Struct, GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationClusterSync, RenderSyncPolicy));
	check(RenderStructProperty);

	uint8* MapContainer = RenderStructProperty->ContainerPtrToValuePtr<uint8>(&ConfigurationCluster->Sync);
	UE::DisplayClusterConfiguratorPropertyUtils::AddKeyValueToMap(MapContainer, ParametersHandle, Key, Value);
}

void FDisplayClusterConfiguratorRenderSyncPolicyCustomization::RemoveFromParameterMap(const FString& Key)
{
	UDisplayClusterConfigurationCluster* ConfigurationCluster = ConfigurationClusterPtr.Get();
	check(ConfigurationCluster != nullptr);

	FStructProperty* SyncStructProperty = FindFProperty<FStructProperty>(ConfigurationCluster->GetClass(), GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationCluster, Sync));
	check(SyncStructProperty);

	FStructProperty* RenderStructProperty = FindFProperty<FStructProperty>(SyncStructProperty->Struct, GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationClusterSync, RenderSyncPolicy));
	check(RenderStructProperty);

	uint8* MapContainer = RenderStructProperty->ContainerPtrToValuePtr<uint8>(&ConfigurationCluster->Sync);
	UE::DisplayClusterConfiguratorPropertyUtils::RemoveKeyFromMap(MapContainer, ParametersHandle, Key);
}

//////////////////////////////////////////////////////////////////////////////////////////////
// Input Sync Type Customization
//////////////////////////////////////////////////////////////////////////////////////////////
void FDisplayClusterConfiguratorInputSyncPolicyCustomization::Initialize(const TSharedRef<IPropertyHandle>& InPropertyHandle, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	FDisplayClusterConfiguratorPolymorphicEntityCustomization::Initialize(InPropertyHandle, CustomizationUtils);

	// Get the Editing object
	TArray<UObject*> OuterObjects;
	InPropertyHandle->GetOuterObjects(OuterObjects);
	if (OuterObjects.Num())
	{
		ConfigurationClusterPtr = Cast<UDisplayClusterConfigurationCluster>(OuterObjects[0]);
	}
	check(ConfigurationClusterPtr != nullptr);
}

void FDisplayClusterConfiguratorInputSyncPolicyCustomization::SetChildren(const TSharedRef<IPropertyHandle>& InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	// Hide properties 
	TypeHandle->MarkHiddenByCustomization();
	ParametersHandle->MarkHiddenByCustomization();

	// Add custom rows
	ResetInputSyncPolicyOptions();
	AddInputSyncPolicyRow(InChildBuilder);
}

void FDisplayClusterConfiguratorInputSyncPolicyCustomization::ResetInputSyncPolicyOptions()
{
	InputSyncPolicyOptions.Reset();

	UDisplayClusterConfigurationCluster* ConfigurationCluster = ConfigurationClusterPtr.Get();
	check(ConfigurationCluster != nullptr);

	for (const FString& InputSyncPolicy : UDisplayClusterConfigurationData::InputSyncPolicies)
	{
		InputSyncPolicyOptions.Add(MakeShared<FString>(InputSyncPolicy));
	}
}

void FDisplayClusterConfiguratorInputSyncPolicyCustomization::AddInputSyncPolicyRow(IDetailChildrenBuilder& InChildBuilder)
{
	if (InputSyncPolicyComboBox.IsValid())
	{
		return;
	}

	InChildBuilder.AddCustomRow(TypeHandle->GetPropertyDisplayName())
	.NameContent()
	[
		TypeHandle->CreatePropertyNameWidget(FText::GetEmpty(), LOCTEXT("InputSyncPolicyTooltip", "Specify your nDisplay Input Sync Policy"))
	]
	.ValueContent()
	[
		SAssignNew(InputSyncPolicyComboBox, SDisplayClusterConfigurationSearchableComboBox)
		.OptionsSource(&InputSyncPolicyOptions)
		.OnGenerateWidget(this, &FDisplayClusterConfiguratorInputSyncPolicyCustomization::MakeInputSyncPolicyOptionComboWidget)
		.OnSelectionChanged(this, &FDisplayClusterConfiguratorInputSyncPolicyCustomization::OnInputSyncPolicySelected)
		.ContentPadding(2)
		.MaxListHeight(200.0f)
		.Content()
		[
			SNew(STextBlock)
			.Text(this, &FDisplayClusterConfiguratorInputSyncPolicyCustomization::GetSelectedInputSyncPolicyText)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
	];
}

TSharedRef<SWidget> FDisplayClusterConfiguratorInputSyncPolicyCustomization::MakeInputSyncPolicyOptionComboWidget(TSharedPtr<FString> InItem)
{
	return SNew(STextBlock)
		.Text(FText::FromString(*InItem))
		.Font(IDetailLayoutBuilder::GetDetailFont());
}

void FDisplayClusterConfiguratorInputSyncPolicyCustomization::OnInputSyncPolicySelected(TSharedPtr<FString> InPolicy, ESelectInfo::Type SelectInfo)
{
	if (InPolicy.IsValid())
	{
		UDisplayClusterConfigurationCluster* ConfigurationCluster = ConfigurationClusterPtr.Get();
		check(ConfigurationCluster != nullptr);

		ConfigurationCluster->Modify();
		ModifyBlueprint();

		TypeHandle->SetValue(*InPolicy.Get());

		// Reset available options
		ResetInputSyncPolicyOptions();
		InputSyncPolicyComboBox->ResetOptionsSource(&InputSyncPolicyOptions);

		InputSyncPolicyComboBox->SetIsOpen(false);
	}
}

FText FDisplayClusterConfiguratorInputSyncPolicyCustomization::GetSelectedInputSyncPolicyText() const
{
	UDisplayClusterConfigurationCluster* ConfigurationCluster = ConfigurationClusterPtr.Get();
	if (ConfigurationCluster == nullptr)
	{
		return FText::GetEmpty();
	}

	return FText::FromString(ConfigurationCluster->Sync.InputSyncPolicy.Type);
}

#undef LOCTEXT_NAMESPACE
