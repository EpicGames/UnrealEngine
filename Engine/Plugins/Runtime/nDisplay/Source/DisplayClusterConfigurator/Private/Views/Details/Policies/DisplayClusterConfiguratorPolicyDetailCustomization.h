// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DisplayClusterConfiguratorPolymorphicEntityCustomization.h"

#include "CoreMinimal.h"
#include "IPropertyTypeCustomization.h"
#include "Layout/Visibility.h"
#include "Types/SlateEnums.h"


class FPolicyParameterInfo;
class UDisplayClusterBlueprint;
class ADisplayClusterRootActor;
class UDisplayClusterConfigurationViewport;
class UDisplayClusterConfigurationCluster;
class SDisplayClusterConfigurationSearchableComboBox;
class SEditableTextBox;
class SWidget;

template<typename NumericType>
class SSpinBox;

/**
 * Projection Type Customization
 */
class FDisplayClusterConfiguratorProjectionCustomization final
	: public FDisplayClusterConfiguratorPolymorphicEntityCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FDisplayClusterConfiguratorProjectionCustomization>();
	}

protected:
	virtual void Initialize(const TSharedRef<IPropertyHandle>& InPropertyHandle, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void SetChildren(const TSharedRef<IPropertyHandle>& InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

	// Projection Policy Selection
protected:
	EVisibility GetCustomRowsVisibility() const;
	TSharedRef<SWidget> MakeProjectionPolicyOptionComboWidget(TSharedPtr<FString> InItem);
	void ResetProjectionPolicyOptions();
	void AddProjectionPolicyRow(IDetailChildrenBuilder& InChildBuilder);
	void AddCustomPolicyRow(IDetailChildrenBuilder& InChildBuilder);
	void OnProjectionPolicySelected(TSharedPtr<FString> InPolicy, ESelectInfo::Type SelectInfo);
	FText GetSelectedProjectionPolicyText() const;
	FText GetCustomPolicyText() const;
	/** Return either custom policy or selected policy. */
	const FString& GetCurrentPolicy() const;
	bool IsCustomTypeInConfig() const;
	void OnTextCommittedInCustomPolicyText(const FText& InValue, ETextCommit::Type CommitType);
	bool IsPolicyIdenticalAcrossEditedObjects(bool bRequireCustomPolicy = false) const;
private:
	TSharedPtr<FString>	CustomOption;
	TArray< TSharedPtr<FString> > ProjectionPolicyOptions;
	TSharedPtr<SDisplayClusterConfigurationSearchableComboBox> ProjectionPolicyComboBox;
	TSharedPtr<SEditableTextBox> CustomPolicyRow;
	FString CurrentSelectedPolicy;
	FString CustomPolicy;
	bool bIsCustomPolicy = false;
	// End Projection Policy Selection

	// Custom Parameters Selection
private:
	/** Given a policy create all parameters. */
	void BuildParametersForPolicy(const FString& Policy, IDetailChildrenBuilder& InChildBuilder);

	void CreateSimplePolicy(UDisplayClusterBlueprint* Blueprint);
	void CreateCameraPolicy(UDisplayClusterBlueprint* Blueprint);
	void CreateMeshPolicy(UDisplayClusterBlueprint* Blueprint);
	void CreateDomePolicy(UDisplayClusterBlueprint* Blueprint);
	void CreateVIOSOPolicy(UDisplayClusterBlueprint* Blueprint);
	void CreateEasyBlendPolicy(UDisplayClusterBlueprint* Blueprint);
	void CreateManualPolicy(UDisplayClusterBlueprint* Blueprint);
	void CreateMPCDIPolicy(UDisplayClusterBlueprint* Blueprint);
	void CreateReferencePolicy(UDisplayClusterBlueprint* Blueprint);

private:
	/** All custom parameters for the selected policy. */
	TArray<TSharedPtr<FPolicyParameterInfo>> CustomPolicyParameters;
	// End Custom Parameters Selection
	
private:
	TArray<TWeakObjectPtr<UDisplayClusterConfigurationViewport>> ConfigurationViewports;
	TWeakObjectPtr<UDisplayClusterConfigurationViewport> ConfigurationViewportPtr;
};


/**
 * Render Sync Type Customization
 */
class FDisplayClusterConfiguratorRenderSyncPolicyCustomization final
	: public FDisplayClusterConfiguratorPolymorphicEntityCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FDisplayClusterConfiguratorRenderSyncPolicyCustomization>();
	}

protected:
	virtual void Initialize(const TSharedRef<IPropertyHandle>& InPropertyHandle, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void SetChildren(const TSharedRef<IPropertyHandle>& InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

private:
	EVisibility GetCustomRowsVisibility() const;
	EVisibility GetNvidiaPolicyRowsVisibility() const;

	void ResetRenderSyncPolicyOptions();
	void AddRenderSyncPolicyRow(IDetailChildrenBuilder& InChildBuilder);
	void AddNvidiaPolicyRows(IDetailChildrenBuilder& InChildBuilder);
	void AddCustomPolicyRow(IDetailChildrenBuilder& InChildBuilder);

	TSharedRef<SWidget> MakeRenderSyncPolicyOptionComboWidget(TSharedPtr<FString> InItem);
	void OnRenderSyncPolicySelected(TSharedPtr<FString> InPolicy, ESelectInfo::Type SelectInfo);

	FText GetSelectedRenderSyncPolicyText() const;
	FText GetCustomPolicyText() const;

	bool IsCustomTypeInConfig() const;

	int32 GetPolicyTypeIndex(const FString& Type) const;

	void OnTextCommittedInCustomPolicyText(const FText& InValue, ETextCommit::Type CommitType);

	void AddToParameterMap(const FString& Key, const FString& Value);
	void RemoveFromParameterMap(const FString& Key);

private:
	TSharedPtr<FString>	NvidiaOption;
	TSharedPtr<FString>	CustomOption;

	TWeakObjectPtr<UDisplayClusterConfigurationCluster> ConfigurationClusterPtr;

	TArray<TSharedPtr<FString>>	RenderSyncPolicyOptions;

	TSharedPtr<SDisplayClusterConfigurationSearchableComboBox> RenderSyncPolicyComboBox;
	TSharedPtr<SSpinBox<int32>> SwapGroupSpinBox;
	TSharedPtr<SSpinBox<int32>> SwapBarrierSpinBox;

	int32 SwapGroupValue = 0;
	int32 SwapBarrierValue = 0;

	TSharedPtr<SEditableTextBox> CustomPolicyRow;
	bool bIsCustomPolicy = false;
	FString CustomPolicy;

private:
	const static FString SwapGroupName;
	const static FString SwapBarrierName;
};

/**
 * Input Sync Type Customization
 */
class FDisplayClusterConfiguratorInputSyncPolicyCustomization final
	: public FDisplayClusterConfiguratorPolymorphicEntityCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FDisplayClusterConfiguratorInputSyncPolicyCustomization>();
	}

protected:
	virtual void Initialize(const TSharedRef<IPropertyHandle>& InPropertyHandle, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void SetChildren(const TSharedRef<IPropertyHandle>& InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

private:
	void ResetInputSyncPolicyOptions();

	void AddInputSyncPolicyRow(IDetailChildrenBuilder& InChildBuilder);

	TSharedRef<SWidget> MakeInputSyncPolicyOptionComboWidget(TSharedPtr<FString> InItem);
	void OnInputSyncPolicySelected(TSharedPtr<FString> InPolicy, ESelectInfo::Type SelectInfo);

	FText GetSelectedInputSyncPolicyText() const;

private:
	TWeakObjectPtr<UDisplayClusterConfigurationCluster> ConfigurationClusterPtr;
	TArray<TSharedPtr<FString>>	InputSyncPolicyOptions;
	TSharedPtr<SDisplayClusterConfigurationSearchableComboBox> InputSyncPolicyComboBox;
};