// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/Details/Policies/DisplayClusterConfiguratorPolicyParameterCustomization.h"

#include "DisplayClusterConfiguratorBlueprintEditor.h"
#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterConfiguratorUtils.h"
#include "DisplayClusterConfiguratorPropertyUtils.h"
#include "Views/Details/Widgets/SDisplayClusterConfigurationSearchableComboBox.h"

#include "DisplayClusterRootActor.h"
#include "DisplayClusterProjectionStrings.h"
#include "Blueprints/DisplayClusterBlueprint.h"
#include "Components/DisplayClusterScreenComponent.h"
#include "Misc/DisplayClusterHelpers.h"
#include "Misc/DisplayClusterTypesConverter.h"

#include "EditorDirectories.h"
#include "IDesktopPlatform.h"
#include "DesktopPlatformModule.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailGroup.h"
#include "IPropertyUtilities.h"
#include "PropertyHandle.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Components/StaticMeshComponent.h"
#include "ScopedTransaction.h"

#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"

#define LOCTEXT_NAMESPACE "FDisplayClusterConfiguratorPolicyParameterCustomization"

FLinearColor OrangeLabelBackgroundColor(0.8f, 0.3f, 0.0f);


int32 GDisplayClusterConfiguratorMinResolution = 0;
static FAutoConsoleVariableRef CVarDisplayClusterConfiguratorMinResolution(
	TEXT("nDisplay.UI.MinResolution"),
	GDisplayClusterConfiguratorMinResolution,
	TEXT("The minimum value of the resolution parameter in the nDisplay UI. (Default = 0)"),
	ECVF_Default
);

int32 GDisplayClusterConfiguratorMaxResolution = 16384;
static FAutoConsoleVariableRef CVarDisplayClusterConfiguratorMaxResolution(
	TEXT("nDisplay.UI.MaxResolution"),
	GDisplayClusterConfiguratorMaxResolution,
	TEXT("The maximum value of the resolution parameter in the nDisplay UI. (Default = 16384)"),
	ECVF_Default
);

//////////////////////////////////////////////////////////////////////////////////////////////
// Policy Parameter Configuration
//////////////////////////////////////////////////////////////////////////////////////////////

FPolicyParameterInfo::FPolicyParameterInfo(
	const FString& InDisplayName,
	const FString& InKey,
	UDisplayClusterBlueprint* InBlueprint,
	const TArray<TWeakObjectPtr<UDisplayClusterConfigurationViewport>>& InConfigurationViewports,
	const FString* InInitialValue)
{
	ParamDisplayName = InDisplayName;
	ParamTooltip = FText::GetEmpty();
	ParamKey = InKey;
	BlueprintOwnerPtr = InBlueprint;
	ConfigurationViewports = InConfigurationViewports;
	
	BlueprintEditorPtrCached = FDisplayClusterConfiguratorUtils::GetBlueprintEditorForObject(InBlueprint);

	if (InInitialValue)
	{
		InitialValue = MakeShared<FString>(*InInitialValue);
	}
}

void FPolicyParameterInfo::SetParameterVisibilityDelegate(FParameterVisible InDelegate)
{
	OnParameterVisibilityCheck = InDelegate;
}

FText FPolicyParameterInfo::GetOrAddCustomParameterValueText() const
{
	if (!DoParametersMatchForAllViewports())
	{
		return LOCTEXT("MultipleValues", "Multiple Values");
	}
	
	FString* ParameterValue = nullptr;
	// Only need values from one viewport since they match.
	if (ConfigurationViewports.Num() > 0)
	{
		UDisplayClusterConfigurationViewport* ConfigurationViewport = ConfigurationViewports[0].Get();
		check(ConfigurationViewport != nullptr);

		ParameterValue = ConfigurationViewport->ProjectionPolicy.Parameters.Find(GetParameterKey());
		if (ParameterValue == nullptr)
		{
			UpdateCustomParameterValueText(InitialValue.IsValid() ? *InitialValue : TEXT(""), false);
			ParameterValue = ConfigurationViewport->ProjectionPolicy.Parameters.Find(GetParameterKey());
		}
	}

	if (ParameterValue == nullptr)
	{
		return FText::GetEmpty();
	}
	return FText::FromString(*ParameterValue);
}

bool FPolicyParameterInfo::IsParameterAlreadyAdded() const
{
	for (const TWeakObjectPtr<UDisplayClusterConfigurationViewport>& Viewport : ConfigurationViewports)
	{
		UDisplayClusterConfigurationViewport* ConfigurationViewport = Viewport.Get();
		check(ConfigurationViewport != nullptr);

		if (ConfigurationViewport->ProjectionPolicy.Parameters.Contains(GetParameterKey()))
		{
			return true;
		}
	}

	return false;
}

bool FPolicyParameterInfo::DoParametersMatchForAllViewports() const
{
	FString LastParameterValue;
	bool bFirstParamFound = false;
	for (const TWeakObjectPtr<UDisplayClusterConfigurationViewport>& Viewport : ConfigurationViewports)
	{
		if (!Viewport.IsValid())
		{
			continue;
		}

		if (FString* ParameterValue = Viewport->ProjectionPolicy.Parameters.Find(GetParameterKey()))
		{
			if (bFirstParamFound && LastParameterValue != *ParameterValue)
			{
				return false;
			}

			LastParameterValue = *ParameterValue;
		}

		bFirstParamFound = true;
	}

	return true;
}

void FPolicyParameterInfo::UpdateCustomParameterValueText(const FString& NewValue, bool bNotify) const
{
	bool bHasChanged = false;
	FScopedTransaction Transaction(LOCTEXT("UpdateParameterText", "Update Parameter"));
	for (const TWeakObjectPtr<UDisplayClusterConfigurationViewport>& Viewport : ConfigurationViewports)
	{
		UDisplayClusterConfigurationViewport* ConfigurationViewport = Viewport.Get();
		check(ConfigurationViewport != nullptr);

		const FString& Key = GetParameterKey();
		
		if (const FString *OldValue = ConfigurationViewport->ProjectionPolicy.Parameters.Find(Key))
		{
			if (*OldValue == NewValue)
			{
				continue;
			}
		}

		bHasChanged = true;
		ConfigurationViewport->Modify();

		FStructProperty* StructProperty = FindFProperty<FStructProperty>(ConfigurationViewport->GetClass(), GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationViewport, ProjectionPolicy));
		check(StructProperty);

		const TSharedPtr<ISinglePropertyView> ProjectionPolicyView = UE::DisplayClusterConfiguratorPropertyUtils::GetPropertyView(
			ConfigurationViewport, GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationViewport, ProjectionPolicy));

		check(ProjectionPolicyView.IsValid());

		TSharedPtr<IPropertyHandle> PropertyHandle = ProjectionPolicyView->GetPropertyHandle()->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationPolymorphicEntity, Parameters));
		check(PropertyHandle.IsValid());
		
		uint8* MapContainer = StructProperty->ContainerPtrToValuePtr<uint8>(ConfigurationViewport);
		UE::DisplayClusterConfiguratorPropertyUtils::AddKeyValueToMap(MapContainer, PropertyHandle, Key, NewValue);
	}

	if (bNotify && bHasChanged)
	{
		if (FDisplayClusterConfiguratorBlueprintEditor* BlueprintEditor = FDisplayClusterConfiguratorUtils::GetBlueprintEditorForObject(BlueprintOwnerPtr.Get()))
		{
			BlueprintEditor->ClusterChanged(true);
			BlueprintEditor->RefreshDisplayClusterPreviewActor();
		}
	}
	if (!bHasChanged)
	{
		Transaction.Cancel();
	}
}

EVisibility FPolicyParameterInfo::IsParameterVisible() const
{
	return OnParameterVisibilityCheck.IsBound() ? OnParameterVisibilityCheck.Execute(GetOrAddCustomParameterValueText()) : EVisibility::Visible;
}

FPolicyParameterInfoCombo::FPolicyParameterInfoCombo(const FString& InDisplayName, const FString& InKey,
													UDisplayClusterBlueprint* InBlueprint,const TArray<TWeakObjectPtr<UDisplayClusterConfigurationViewport>>& InConfigurationViewports,
													const TArray<FString>& InValues, const FString* InInitialItem, bool bSort) : FPolicyParameterInfo(InDisplayName, InKey, InBlueprint, InConfigurationViewports, InInitialItem)
{
	for (const FString& Value : InValues)
	{
		CustomParameterOptions.Add(MakeShared<FString>(Value));
	}

	if (bSort)
	{
		CustomParameterOptions.Sort([](const TSharedPtr<FString>& A, const TSharedPtr<FString>& B)
		{
			// Default sort isn't compatible with TSharedPtr<FString>.
			return *A < *B;
		});
	}
}

void FPolicyParameterInfoCombo::CreateCustomRowWidget(IDetailChildrenBuilder& InDetailWidgetRow)
{
	InDetailWidgetRow.AddCustomRow(GetParameterDisplayName())
	.NameContent()
	[
		SNew(STextBlock)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Text(this, &FPolicyParameterInfo::GetParameterDisplayName)
		.ToolTipText(this, &FPolicyParameterInfo::GetParameterTooltip)
	]
	.ValueContent()
	[
		SAssignNew(GetCustomParameterValueComboBox(), SDisplayClusterConfigurationSearchableComboBox)
		.OptionsSource(&GetCustomParameterOptions())
		.InitiallySelectedItem(InitialValue)
		.OnGenerateWidget(this, &FPolicyParameterInfoCombo::MakeCustomParameterValueComboWidget)
		.OnSelectionChanged(this, &FPolicyParameterInfoCombo::OnCustomParameterValueSelected)
		.Visibility(this, &FPolicyParameterInfoCombo::IsParameterVisible)
		.ContentPadding(2)
		.MaxListHeight(200.0f)
		.Content()
		[
			SNew(STextBlock)
			.Text(this, &FPolicyParameterInfoCombo::GetOrAddCustomParameterValueText)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
	];
}

void FPolicyParameterInfoCombo::SetOnSelectedDelegate(FOnItemSelected InDelegate)
{
	OnItemSelected = InDelegate;
}

TSharedRef<SWidget> FPolicyParameterInfoCombo::MakeCustomParameterValueComboWidget(TSharedPtr<FString> InItem)
{
	return SNew(STextBlock)
		.Text(FText::FromString(*InItem))
		.Font(IDetailLayoutBuilder::GetDetailFont());
}

void FPolicyParameterInfoCombo::OnCustomParameterValueSelected(TSharedPtr<FString> InValue, ESelectInfo::Type SelectInfo)
{
	const FString ValueString = InValue.IsValid() ? *InValue.Get() : "";
	UpdateCustomParameterValueText(ValueString);
	OnItemSelected.ExecuteIfBound(ValueString);
}


FPolicyParameterInfoComponentCombo::FPolicyParameterInfoComponentCombo(
	const FString& InDisplayName,
	const FString& InKey,
	UDisplayClusterBlueprint* InBlueprint,
	const TArray<TWeakObjectPtr<UDisplayClusterConfigurationViewport>>& InConfigurationViewports,
	const TArray<TSubclassOf<UActorComponent>>& InComponentClasses) : FPolicyParameterInfoCombo(InDisplayName, InKey, InBlueprint, InConfigurationViewports, {}, nullptr)
{
	ComponentTypes = InComponentClasses;
	if (BlueprintEditorPtrCached)
	{
		// BlueprintEditorPtrCached can be null when remote control is interfacing with the world instance details panel. At this stage
		// there isn't a blueprint editor available. Normally this customization is hidden here but remote control seems to trigger it.
		
		ADisplayClusterRootActor* RootActor = CastChecked<ADisplayClusterRootActor>(BlueprintEditorPtrCached->GetPreviewActor());
		CreateParameterValues(RootActor);
	}
}

void FPolicyParameterInfoComponentCombo::CreateParameterValues(ADisplayClusterRootActor* RootActor)
{
	for (const TSubclassOf<UActorComponent>& ComponentType : ComponentTypes)
	{
		TArray<UActorComponent*> ActorComponents;
		RootActor->GetComponents(ComponentType, ActorComponents);
		for (UActorComponent* ActorComponent : ActorComponents)
		{
			// Filter out components that should not be listed, including implicit components and visualization components. We must
			// specially check for screen components since they are flagged as visualization components but we want them to show up
			// in the list.
			const bool bIsImplicitComponent = ActorComponent->GetName().EndsWith(FDisplayClusterConfiguratorUtils::GetImplSuffix());
			const bool bIsVisualizationComponent = ActorComponent->IsA<UStaticMeshComponent>() && ActorComponent->IsVisualizationComponent();
			const bool bIsScreenComponent = ActorComponent->IsA<UDisplayClusterScreenComponent>();

			if (bIsImplicitComponent || (bIsVisualizationComponent && !bIsScreenComponent))
			{
				// Ignore the default impl subobjects.
				continue;
			}
			const FString ComponentName = ActorComponent->GetName();
			CustomParameterOptions.Add(MakeShared<FString>(ComponentName));
		}
	}
}

FPolicyParameterInfoViewportCombo::FPolicyParameterInfoViewportCombo(
	const FString& InDisplayName,
	const FString& InKey,
	UDisplayClusterBlueprint* InBlueprint,
	const TArray<TWeakObjectPtr<UDisplayClusterConfigurationViewport>>& InConfigurationViewports)
	: FPolicyParameterInfoCombo(InDisplayName, InKey, InBlueprint, InConfigurationViewports, {}, nullptr)
{
	CreateParameterValues();
}

void FPolicyParameterInfoViewportCombo::CreateParameterValues()
{
	// Reference policy can only reference viewports within the same cluster node.
	// When multiple viewports are selected for editing, we use the first valid one to determine
	// which cluster node's viewports to populate. If the selected viewports belong to different
	// cluster nodes, AreAllViewportsInSameClusterNode() will return false and disable the control.
	UDisplayClusterConfigurationClusterNode* ClusterNode = nullptr;
	TSet<FString> EditedViewportIds;

	// Find the first valid cluster node and build the set of edited viewport IDs
	for (const TWeakObjectPtr<UDisplayClusterConfigurationViewport>& Viewport : ConfigurationViewports)
	{
		if (Viewport.IsValid())
		{
			UDisplayClusterConfigurationClusterNode* CurrentClusterNode = Viewport->GetTypedOuter<UDisplayClusterConfigurationClusterNode>();
			if (CurrentClusterNode)
			{
				// Use the first valid cluster node we find
				if (!ClusterNode)
				{
					ClusterNode = CurrentClusterNode;
				}

				// Build set of currently edited viewport IDs from the first cluster node
				// (AreAllViewportsInSameClusterNode will validate they're all the same)
				if (ClusterNode == CurrentClusterNode)
				{
					for (const auto& ViewportPair : ClusterNode->Viewports)
					{
						if (ViewportPair.Value == Viewport.Get())
						{
							EditedViewportIds.Add(ViewportPair.Key);
							break;
						}
					}
				}
			}
		}
	}

	// If we couldn't find a cluster node, we can't populate the list
	if (!ClusterNode)
	{
		return;
	}

	// Collect all viewports from the cluster node, excluding the currently edited ones
	for (const auto& ViewportPair : ClusterNode->Viewports)
	{
		const FString& ViewportId = ViewportPair.Key;

		// Skip if this is one of the viewports being edited
		if (EditedViewportIds.Contains(ViewportId))
		{
			continue;
		}

		CustomParameterOptions.Add(MakeShared<FString>(ViewportId));
	}

	// Sort the options alphabetically for better UX
	CustomParameterOptions.Sort([](const TSharedPtr<FString>& A, const TSharedPtr<FString>& B)
	{
		return *A < *B;
	});
}

void FPolicyParameterInfoViewportCombo::CreateCustomRowWidget(IDetailChildrenBuilder& InDetailWidgetRow)
{
	InDetailWidgetRow.AddCustomRow(GetParameterDisplayName())
	.NameContent()
	[
		SNew(STextBlock)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Text(this, &FPolicyParameterInfo::GetParameterDisplayName)
		.ToolTipText(this, &FPolicyParameterInfo::GetParameterTooltip)
	]
	.ValueContent()
	[
		SAssignNew(GetCustomParameterValueComboBox(), SDisplayClusterConfigurationSearchableComboBox)
		.OptionsSource(&GetCustomParameterOptions())
		.InitiallySelectedItem(InitialValue)
		.OnGenerateWidget_Lambda([this](TSharedPtr<FString> InItem)
		{
			return MakeCustomParameterValueComboWidget(InItem);
		})
		.OnSelectionChanged_Lambda([this](TSharedPtr<FString> InValue, ESelectInfo::Type SelectInfo)
		{
			OnCustomParameterValueSelected(InValue, SelectInfo);
		})
		.Visibility_Lambda([this]()
		{
			return IsParameterVisible();
		})
		.IsEnabled(this, &FPolicyParameterInfoViewportCombo::AreAllViewportsInSameClusterNode)
		.ContentPadding(2)
		.MaxListHeight(200.0f)
		.Content()
		[
			SNew(STextBlock)
			.Text(this, &FPolicyParameterInfoCombo::GetOrAddCustomParameterValueText)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
	];
}

bool FPolicyParameterInfoViewportCombo::AreAllViewportsInSameClusterNode() const
{
	UDisplayClusterConfigurationClusterNode* FirstClusterNode = nullptr;

	for (const TWeakObjectPtr<UDisplayClusterConfigurationViewport>& Viewport : ConfigurationViewports)
	{
		if (Viewport.IsValid())
		{
			UDisplayClusterConfigurationClusterNode* ClusterNode = Viewport->GetTypedOuter<UDisplayClusterConfigurationClusterNode>();

			if (!ClusterNode)
			{
				// If we can't find a cluster node, disable the control
				return false;
			}

			if (!FirstClusterNode)
			{
				FirstClusterNode = ClusterNode;
			}
			else if (FirstClusterNode != ClusterNode)
			{
				// Different cluster nodes found, disable the control
				return false;
			}
		}
	}

	// If we didn't find any valid cluster node, disable the control
	return FirstClusterNode != nullptr;
}


FPolicyParameterInfoText::FPolicyParameterInfoText(const FString& InDisplayName, const FString& InKey,
	UDisplayClusterBlueprint* InBlueprint,
	const TArray<TWeakObjectPtr<UDisplayClusterConfigurationViewport>>& InConfigurationViewports) :
	FPolicyParameterInfo(InDisplayName, InKey, InBlueprint, InConfigurationViewports)
{
}

void FPolicyParameterInfoText::CreateCustomRowWidget(IDetailChildrenBuilder& InDetailWidgetRow)
{
	InDetailWidgetRow.AddCustomRow(GetParameterDisplayName())
		.NameContent()
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(GetParameterDisplayName())
			.ToolTipText(this, &FPolicyParameterInfo::GetParameterTooltip)
		]
		.ValueContent()
		[
			SNew(SEditableTextBox)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(this, &FPolicyParameterInfoText::GetOrAddCustomParameterValueText)
			.OnTextCommitted_Lambda([this](const FText& InValue, ETextCommit::Type InCommitType)
			{
				UpdateCustomParameterValueText(InValue.ToString());
			})
		];
}

FPolicyParameterInfoBool::FPolicyParameterInfoBool(const FString& InDisplayName, const FString& InKey,
	UDisplayClusterBlueprint* InBlueprint, const TArray<TWeakObjectPtr<UDisplayClusterConfigurationViewport>>& InConfigurationViewports, bool bInvertValue) :
	FPolicyParameterInfo(InDisplayName, InKey, InBlueprint, InConfigurationViewports),
	bInvertValue(bInvertValue)
{
}

void FPolicyParameterInfoBool::CreateCustomRowWidget(IDetailChildrenBuilder& InDetailWidgetRow)
{
	InDetailWidgetRow.AddCustomRow(GetParameterDisplayName())
		.NameContent()
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(GetParameterDisplayName())
			.ToolTipText(this, &FPolicyParameterInfo::GetParameterTooltip)
		]
		.ValueContent()
		[
			SNew(SCheckBox)
			.IsChecked(this, &FPolicyParameterInfoBool::IsChecked)
			.OnCheckStateChanged_Lambda([this](ECheckBoxState InValue)
			{
				const bool bIsChecked = InValue == ECheckBoxState::Checked;
				UpdateCustomParameterValueText(DisplayClusterTypesConverter::template ToString(bInvertValue ? !bIsChecked : bIsChecked));
			})
		];
}

ECheckBoxState FPolicyParameterInfoBool::IsChecked() const
{
	if (!DoParametersMatchForAllViewports())
	{
		return ECheckBoxState::Undetermined;	
	}
	
	const FString StrValue = GetOrAddCustomParameterValueText().ToString().ToLower();
	const bool bValue = DisplayClusterTypesConverter::template FromString<bool>(StrValue);
	const bool bIsChecked = bInvertValue ? !bValue : bValue;

	return bIsChecked ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}


void FPolicyParameterInfoFile::CreateCustomRowWidget(IDetailChildrenBuilder& InDetailWidgetRow)
{
	InDetailWidgetRow.AddCustomRow(GetParameterDisplayName())
		.NameContent()
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(GetParameterDisplayName())
			.ToolTipText(this, &FPolicyParameterInfo::GetParameterTooltip)
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			[
				SNew(SEditableTextBox)
				.IsReadOnly(true)
				.Text(this, &FPolicyParameterInfoFile::GetOrAddCustomParameterValueText)
			]
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SButton)
				.OnClicked(this, &FPolicyParameterInfoFile::OnChangePathClicked)
				.ToolTipText(LOCTEXT("ChangeSourcePath_Tooltip", "Browse for file"))
				[
					SNew(STextBlock)
					.Text(LOCTEXT("...", "..."))
				]
			]
		];
}

void FPolicyParameterInfoFile::OpenSelectFileDialogue()
{
	FString FileTypes;
	FString AllExtensions;

	for (const FString& Format : FileExtensions)
	{
		TArray<FString> FormatComponents;
		Format.ParseIntoArray(FormatComponents, TEXT(";"), false);

		for (int32 ComponentIndex = 0; ComponentIndex < FormatComponents.Num(); ComponentIndex += 2)
		{
			const FString& Extension = FormatComponents[ComponentIndex];

			if (!AllExtensions.IsEmpty())
			{
				AllExtensions.AppendChar(TEXT(';'));
			}
			AllExtensions.Append(TEXT("*."));
			AllExtensions.Append(Extension);

			if (!FileTypes.IsEmpty())
			{
				FileTypes.AppendChar(TEXT('|'));
			}

			FileTypes.Append(FString::Printf(TEXT("(*.%s)|*.%s"), *Extension, *Extension));
		}
	}
	
	if (IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get())
	{
		const FString SupportedExtensions(FString::Printf(TEXT("All Files (%s)|%s|%s"), *AllExtensions, *AllExtensions, *FileTypes));
		const FString DefaultLocation(FEditorDirectories::Get().GetLastDirectory(ELastDirectory::GENERIC_IMPORT));

		TArray<FString> OpenedFiles;
		
		const bool bOpened = DesktopPlatform->OpenFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			LOCTEXT("FileDialogTitle", "Open file").ToString(),
			DefaultLocation,
			TEXT(""),
			SupportedExtensions,
			EFileDialogFlags::None,
			OpenedFiles
		);

		if (bOpened)
		{
			FString NewPathToFile;
			if (!OpenedFiles.IsEmpty())
			{
				const FString& OpenedFile = OpenedFiles[0];
				FEditorDirectories::Get().SetLastDirectory(ELastDirectory::GENERIC_IMPORT, FPaths::GetPath(OpenedFile));

				// Gets the full path to the selected file
				const FString OutFullPath = FPaths::ConvertRelativePathToFull(OpenedFile);

				// Gets the relative path to the selected file if it is possible.
				NewPathToFile = DisplayClusterHelpers::filesystem::GetRelativePathForConfigResource(OutFullPath);
			}

			// The file path is updated only if the OpenSelectFileDialogue() function returns a valid value
			UpdateCustomParameterValueText(NewPathToFile);
		}
		else
		{
			// When we close the file select dialog box using the cancel or close button, we do nothing.
		}
	}
}

FReply FPolicyParameterInfoFile::OnChangePathClicked()
{
	OpenSelectFileDialogue();

	return FReply::Handled();
}

/**
* FPolicyParameterInfoFloatReference
*/
TSharedRef<SWidget> FPolicyParameterInfoFloatReference::MakeFloatInputWidget(TSharedRef<TOptional<float>>& ProxyValue, const FText& Label,
											const bool bRotationInDegrees,
											const FLinearColor& LabelColor, const FLinearColor& LabelBackgroundColor,
											const float* InMinValue, const float* InMaxValue)
{
	TOptional<float> MinValue, MaxValue;

	if (bRotationInDegrees)
	{
		MinValue = 0;
		MaxValue = 360;
	}

	if (InMinValue)
	{
		MinValue = *InMinValue;
	}
	
	if (InMaxValue)
	{
		MaxValue = *InMaxValue;
	}


	return
		SNew(SNumericEntryBox<float>)
			.Value(this, &FPolicyParameterInfoFloatReference::OnGetValue, ProxyValue)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.UndeterminedString(NSLOCTEXT("PropertyEditor", "MultipleValues", "Multiple Values"))
			.OnValueCommitted(this, &FPolicyParameterInfoFloatReference::OnValueCommitted, ProxyValue)
			.LabelVAlign(VAlign_Fill)
			.LabelPadding(0)
			.AllowSpin(bRotationInDegrees)
			.MaxSliderValue(MaxValue)
			.MinSliderValue(MinValue)
			.Label()
		[
			SNumericEntryBox<float>::BuildLabel(Label, LabelColor, LabelBackgroundColor)
		];
}

/**
* FPolicyParameterInfoIntReference
*/
TSharedRef<SWidget> FPolicyParameterInfoIntReference::MakeIntInputWidget(TSharedRef<TOptional<int32>>& ProxyValue, const FText& Label,
										const FLinearColor& LabelColor, const FLinearColor& LabelBackgroundColor,
										const int32* InMinValue, const int32* InMaxValue)
{
	TOptional<int32> MinValue, MaxValue;

	if (InMinValue)
	{
		MinValue = *InMinValue;
	}

	if (InMaxValue)
	{
		MaxValue = *InMaxValue;
	}
	return
		SNew(SNumericEntryBox<int32>)
			.Value(this, &FPolicyParameterInfoIntReference::OnGetValue, ProxyValue)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.UndeterminedString(NSLOCTEXT("PropertyEditor", "MultipleValues", "Multiple Values"))
			.OnValueCommitted(this, &FPolicyParameterInfoIntReference::OnValueCommitted, ProxyValue)
			.LabelVAlign(VAlign_Fill)
			.LabelPadding(0)
			.MaxSliderValue(MaxValue)
			.MinSliderValue(MinValue)
			.Label()
		[
			SNumericEntryBox<float>::BuildLabel(Label, LabelColor, LabelBackgroundColor)
		];
}

/**
* FPolicyParameterInfoMatrix
*/
const FString FPolicyParameterInfoMatrix::BaseMatrixString = DisplayClusterTypesConverter::template ToString<FMatrix>(FMatrix(FPlane(1.0f, 0.0f, 0.0f, 0.0f), FPlane(0.0f, 1.0f, 0.0f, 0.0f), FPlane(0.0f, 0.0f, 0.0f, 1.0f), FPlane(0.0f, 0.0f, 1.0f, 0.0f)));

FPolicyParameterInfoMatrix::FPolicyParameterInfoMatrix(const FString& InDisplayName, const FString& InKey,
                                                       UDisplayClusterBlueprint* InBlueprint,
                                                       const TArray<TWeakObjectPtr<UDisplayClusterConfigurationViewport>>& InConfigurationViewports):
	FPolicyParameterInfoFloatReference(InDisplayName, InKey, InBlueprint, InConfigurationViewports, &BaseMatrixString),
	CachedTranslationX(MakeShared<TOptional<float>>()),
	CachedTranslationY(MakeShared<TOptional<float>>()),
	CachedTranslationZ(MakeShared<TOptional<float>>()),
	CachedRotationYaw(MakeShared<TOptional<float>>()),
	CachedRotationPitch(MakeShared<TOptional<float>>()),
	CachedRotationRoll(MakeShared<TOptional<float>>()),
	CachedScaleX(MakeShared<TOptional<float>>()),
	CachedScaleY(MakeShared<TOptional<float>>()),
	CachedScaleZ(MakeShared<TOptional<float>>())
{
	const FText TextValue = GetOrAddCustomParameterValueText();
	const FMatrix Matrix = DisplayClusterTypesConverter::template FromString<FMatrix>(TextValue.ToString());

	const FVector Translation = Matrix.GetOrigin();
	const FRotator Rotation = Matrix.Rotator();
	const FVector Scale = Matrix.GetScaleVector();

	*CachedTranslationX = Translation.X;
	*CachedTranslationY = Translation.Y;
	*CachedTranslationZ = Translation.Z;

	*CachedRotationYaw = Rotation.Yaw;
	*CachedRotationPitch = Rotation.Pitch;
	*CachedRotationRoll = Rotation.Roll;

	*CachedScaleX = Scale.X;
	*CachedScaleY = Scale.Y;
	*CachedScaleZ = Scale.Z;

	// TODO: Multiple values.
}


void FPolicyParameterInfoMatrix::CreateCustomRowWidget(IDetailChildrenBuilder& InDetailWidgetRow)
{
	IDetailGroup& Group = InDetailWidgetRow.AddGroup(*GetParameterKey(), GetParameterDisplayName());
	Group.HeaderRow()
	[
		SNew(STextBlock)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Text(GetParameterDisplayName())
		.ToolTipText(this, &FPolicyParameterInfo::GetParameterTooltip)
		.Visibility(this, &FPolicyParameterInfoMatrix::IsParameterVisible)
	]
	.Visibility(MakeAttributeRaw(this, &FPolicyParameterInfoMatrix::IsParameterVisible));
	
	CustomizeLocation(Group.AddWidgetRow());
	CustomizeRotation(Group.AddWidgetRow());
	CustomizeScale(Group.AddWidgetRow());
	
	Group.ToggleExpansion(true);
}

void FPolicyParameterInfoMatrix::CustomizeLocation(FDetailWidgetRow& InDetailWidgetRow)
{
	InDetailWidgetRow
	.Visibility(MakeAttributeRaw(this, &FPolicyParameterInfoMatrix::IsParameterVisible))
	.NameContent()
	[
		SNew(STextBlock)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Text(LOCTEXT("LocationLabel", "Location"))
	]
	.ValueContent()
	.MinDesiredWidth(375.0f)
	.MaxDesiredWidth(375.0f)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(FMargin(0.0f, 2.0f, 3.0f, 2.0f))
		[
			MakeFloatInputWidget(CachedTranslationX, LOCTEXT("TranslationX", "X"), false, FLinearColor::White, SNumericEntryBox<float>::RedLabelBackgroundColor)
		]
		+ SHorizontalBox::Slot()
		.Padding(FMargin(0.0f, 2.0f, 3.0f, 2.0f))
		[
			MakeFloatInputWidget(CachedTranslationY, LOCTEXT("TranslationY", "Y"), false, FLinearColor::White, SNumericEntryBox<float>::GreenLabelBackgroundColor)
		]
		+ SHorizontalBox::Slot()
		.Padding(FMargin(0.0f, 2.0f, 3.0f, 2.0f))
		[
			MakeFloatInputWidget(CachedTranslationZ, LOCTEXT("TranslationZ", "Z"), false, FLinearColor::White, SNumericEntryBox<float>::BlueLabelBackgroundColor)
		]
	];
}

void FPolicyParameterInfoMatrix::CustomizeRotation(FDetailWidgetRow& InDetailWidgetRow)
{
	InDetailWidgetRow
	.Visibility(MakeAttributeRaw(this, &FPolicyParameterInfoMatrix::IsParameterVisible))
	.NameContent()
	[
		SNew(STextBlock)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Text(LOCTEXT("RotationLabel", "Rotation"))
	]
	.ValueContent()
	.MinDesiredWidth(375.0f)
	.MaxDesiredWidth(375.0f)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(FMargin(0.0f, 2.0f, 3.0f, 2.0f))
		[
			MakeFloatInputWidget(CachedRotationRoll, LOCTEXT("RotationX", "X"), true, FLinearColor::White, SNumericEntryBox<float>::RedLabelBackgroundColor)
		]
		+ SHorizontalBox::Slot()
		.Padding(FMargin(0.0f, 2.0f, 3.0f, 2.0f))
		[
			MakeFloatInputWidget(CachedRotationPitch, LOCTEXT("RotationY", "Y"), true, FLinearColor::White, SNumericEntryBox<float>::GreenLabelBackgroundColor)
		]
		+ SHorizontalBox::Slot()
		.Padding(FMargin(0.0f, 2.0f, 3.0f, 2.0f))
		[
			MakeFloatInputWidget(CachedRotationYaw, LOCTEXT("RotationZ", "Z"), true, FLinearColor::White, SNumericEntryBox<float>::BlueLabelBackgroundColor)
		]
	];
}

void FPolicyParameterInfoMatrix::CustomizeScale(FDetailWidgetRow& InDetailWidgetRow)
{
	InDetailWidgetRow
	.Visibility(MakeAttributeRaw(this, &FPolicyParameterInfoMatrix::IsParameterVisible))
	.NameContent()
	[
		SNew(STextBlock)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Text(LOCTEXT("ScaleLabel", "Scale"))
	]
	.ValueContent()
	.MinDesiredWidth(375.0f)
	.MaxDesiredWidth(375.0f)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(FMargin(0.0f, 2.0f, 3.0f, 2.0f))
		[
			MakeFloatInputWidget(CachedScaleX, LOCTEXT("ScaleX", "X"), false, FLinearColor::White, SNumericEntryBox<float>::RedLabelBackgroundColor)
		]
		+ SHorizontalBox::Slot()
		.Padding(FMargin(0.0f, 2.0f, 3.0f, 2.0f))
		[
			MakeFloatInputWidget(CachedScaleY, LOCTEXT("ScaleY", "Y"), false, FLinearColor::White, SNumericEntryBox<float>::GreenLabelBackgroundColor)
		]
		+ SHorizontalBox::Slot()
		.Padding(FMargin(0.0f, 2.0f, 3.0f, 2.0f))
		[
			MakeFloatInputWidget(CachedScaleZ, LOCTEXT("ScaleZ", "Z"), false, FLinearColor::White, SNumericEntryBox<float>::BlueLabelBackgroundColor)
		]
	];
}

void FPolicyParameterInfoMatrix::FormatTextAndUpdateParameter()
{
	const bool bAllValuesSet =
		CachedTranslationX->IsSet() && CachedTranslationY->IsSet() && CachedTranslationZ->IsSet() &&
		CachedRotationPitch->IsSet() && CachedRotationYaw->IsSet() && CachedRotationRoll->IsSet() &&
		CachedScaleX->IsSet() && CachedScaleY->IsSet() && CachedScaleZ->IsSet();

	if (!bAllValuesSet)
	{
		// We can't call Get() on optional values that aren't set.
		// This check isn't ideal but we need the complete matrix
		// to process it back to a string.
		return;
	}
	
	const FVector Translation(*CachedTranslationX.Get(), *CachedTranslationY.Get(), *CachedTranslationZ.Get());
	const FRotator Rotation(*CachedRotationPitch.Get(), *CachedRotationYaw.Get(), *CachedRotationRoll.Get());
	const FVector Scale(*CachedScaleX.Get(), *CachedScaleY.Get(), *CachedScaleZ.Get());
	
	const FMatrix Matrix = FScaleRotationTranslationMatrix(Scale, Rotation, Translation);
	
	const FString MatrixString = DisplayClusterTypesConverter::template ToString(Matrix);
	UpdateCustomParameterValueText(MatrixString);
}

const FString FPolicyParameterInfo4x4Matrix::BaseMatrixString = DisplayClusterTypesConverter::template ToString<FMatrix>(FMatrix(FPlane(1.0f, 0.0f, 0.0f, 0.0f), FPlane(0.0f, 1.0f, 0.0f, 0.0f), FPlane(0.0f, 0.0f, 0.0f, 1.0f), FPlane(0.0f, 0.0f, 1.0f, 0.0f)));

FPolicyParameterInfo4x4Matrix::FPolicyParameterInfo4x4Matrix(const FString& InDisplayName, const FString& InKey,
	UDisplayClusterBlueprint* InBlueprint, const TArray<TWeakObjectPtr<UDisplayClusterConfigurationViewport>>& InConfigurationViewports, const FString* DefaultMatrixString)
	: FPolicyParameterInfoFloatReference(InDisplayName, InKey, InBlueprint, InConfigurationViewports, DefaultMatrixString ? DefaultMatrixString  : &BaseMatrixString)
	, A(MakeShared<TOptional<float>>()), B(MakeShared<TOptional<float>>()), C(MakeShared<TOptional<float>>()), D(MakeShared<TOptional<float>>()), E(MakeShared<TOptional<float>>())
	, F(MakeShared<TOptional<float>>()), G(MakeShared<TOptional<float>>()), H(MakeShared<TOptional<float>>()), I(MakeShared<TOptional<float>>()), J(MakeShared<TOptional<float>>())
	, K(MakeShared<TOptional<float>>()), L(MakeShared<TOptional<float>>()), M(MakeShared<TOptional<float>>()), N(MakeShared<TOptional<float>>()), O(MakeShared<TOptional<float>>())
	, P(MakeShared<TOptional<float>>())
{
	if (!DoParametersMatchForAllViewports())
	{
		for (const TWeakObjectPtr<UDisplayClusterConfigurationViewport>& Viewport : ConfigurationViewports)
		{
			UDisplayClusterConfigurationViewport* ConfigurationViewport = Viewport.Get();
			check(ConfigurationViewport != nullptr);

			FString* ParameterValue = ConfigurationViewport->ProjectionPolicy.Parameters.Find(GetParameterKey());
			check(ParameterValue);

			const FText TextValue = FText::FromString(*ParameterValue);
			const FMatrix Matrix = DisplayClusterTypesConverter::template FromString<FMatrix>(TextValue.ToString());

			ResetOrSetCachedValue(A, Matrix.M[0][0]);
			ResetOrSetCachedValue(B, Matrix.M[0][1]);
			ResetOrSetCachedValue(C, Matrix.M[0][2]);
			ResetOrSetCachedValue(D, Matrix.M[0][3]);

			ResetOrSetCachedValue(E, Matrix.M[1][0]);
			ResetOrSetCachedValue(F, Matrix.M[1][1]);
			ResetOrSetCachedValue(G, Matrix.M[1][2]);
			ResetOrSetCachedValue(H, Matrix.M[1][3]);

			ResetOrSetCachedValue(I, Matrix.M[2][0]);
			ResetOrSetCachedValue(J, Matrix.M[2][1]);
			ResetOrSetCachedValue(K, Matrix.M[2][2]);
			ResetOrSetCachedValue(L, Matrix.M[2][3]);

			ResetOrSetCachedValue(M, Matrix.M[3][0]);
			ResetOrSetCachedValue(N, Matrix.M[3][1]);
			ResetOrSetCachedValue(O, Matrix.M[3][2]);
			ResetOrSetCachedValue(P, Matrix.M[3][3]);
		}
	}
	else
	{
		const FText TextValue = GetOrAddCustomParameterValueText();
		const FMatrix Matrix = DisplayClusterTypesConverter::template FromString<FMatrix>(TextValue.ToString());
		*A = Matrix.M[0][0];
		*B = Matrix.M[0][1];
		*C = Matrix.M[0][2];
		*D = Matrix.M[0][3];

		*E = Matrix.M[1][0];
		*F = Matrix.M[1][1];
		*G = Matrix.M[1][2];
		*H = Matrix.M[1][3];

		*I = Matrix.M[2][0];
		*J = Matrix.M[2][1];
		*K = Matrix.M[2][2];
		*L = Matrix.M[2][3];

		*M = Matrix.M[3][0];
		*N = Matrix.M[3][1];
		*O = Matrix.M[3][2];
		*P = Matrix.M[3][3];
	}
}

void FPolicyParameterInfo4x4Matrix::CreateCustomRowWidget(IDetailChildrenBuilder& InDetailWidgetRow)
{
	IDetailGroup& Group = InDetailWidgetRow.AddGroup(*GetParameterKey(), GetParameterDisplayName());
	Group.HeaderRow()
	[
		SNew(STextBlock)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Text(GetParameterDisplayName())
		.ToolTipText(this, &FPolicyParameterInfo::GetParameterTooltip)
		.Visibility(this, &FPolicyParameterInfo4x4Matrix::IsParameterVisible)
	]
	.Visibility(MakeAttributeRaw(this, &FPolicyParameterInfo4x4Matrix::IsParameterVisible));
	
	CustomizeRow(LOCTEXT("RowX", "X"), A, B, C, D, Group.AddWidgetRow());
	CustomizeRow(LOCTEXT("RowY", "Y"), E, F, G, H, Group.AddWidgetRow());
	CustomizeRow(LOCTEXT("RowZ", "Z"), I, J, K, L, Group.AddWidgetRow());
	CustomizeRow(LOCTEXT("RowW", "W"), M, N, O, P, Group.AddWidgetRow());
	
	Group.ToggleExpansion(true);
}

void FPolicyParameterInfo4x4Matrix::CustomizeRow(const FText& InHeaderText, TSharedRef<TOptional<float>>& InX, TSharedRef<TOptional<float>>& InY,
		TSharedRef<TOptional<float>>& InZ, TSharedRef<TOptional<float>>& InW, FDetailWidgetRow& InDetailWidgetRow)
{
	InDetailWidgetRow
	.Visibility(MakeAttributeRaw(this, &FPolicyParameterInfo4x4Matrix::IsParameterVisible))
	.NameContent()
	[
		SNew(STextBlock)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Text(InHeaderText)
	]
	.ValueContent()
	.MinDesiredWidth(375.0f)
	.MaxDesiredWidth(375.0f)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(FMargin(0.0f, 2.0f, 3.0f, 2.0f))
		[
			MakeFloatInputWidget(InX, LOCTEXT("MatrixX", "X"), false, FLinearColor::White, SNumericEntryBox<float>::RedLabelBackgroundColor)
		]
		+ SHorizontalBox::Slot()
		.Padding(FMargin(0.0f, 2.0f, 3.0f, 2.0f))
		[
			MakeFloatInputWidget(InY, LOCTEXT("MatrixY", "Y"), false, FLinearColor::White, SNumericEntryBox<float>::GreenLabelBackgroundColor)
		]
		+ SHorizontalBox::Slot()
		.Padding(FMargin(0.0f, 2.0f, 3.0f, 2.0f))
		[
			MakeFloatInputWidget(InZ, LOCTEXT("MatrixZ", "Z"), false, FLinearColor::White, SNumericEntryBox<float>::BlueLabelBackgroundColor)
		]
		+ SHorizontalBox::Slot()
		.Padding(FMargin(0.0f, 2.0f, 3.0f, 2.0f))
		[
			MakeFloatInputWidget(InW, LOCTEXT("MatrixW", "W"), false, FLinearColor::White, OrangeLabelBackgroundColor)
		]
	];
}

void FPolicyParameterInfo4x4Matrix::FormatTextAndUpdateParameter()
{
	const bool bAllValuesSet =
		A->IsSet() && B->IsSet() && C->IsSet() && D->IsSet() &&
		E->IsSet() && F->IsSet() && G->IsSet() && H->IsSet() &&
		I->IsSet() && J->IsSet() && K->IsSet() && L->IsSet() &&
		M->IsSet() && N->IsSet() && O->IsSet() && P->IsSet();

	if (!bAllValuesSet)
	{
		// We can't call Get() on optional values that aren't set.
		// This check isn't ideal but we need the complete matrix
		// to process it back to a string.
		return;
	}
	
	const FPlane PlaneX(*A.Get(), *B.Get(), *C.Get(), *D.Get());
	const FPlane PlaneY(*E.Get(), *F.Get(), *G.Get(), *H.Get());
	const FPlane PlaneZ(*I.Get(), *J.Get(), *K.Get(), *L.Get());
	const FPlane PlaneW(*M.Get(), *N.Get(), *O.Get(), *P.Get());
	
	const FMatrix Matrix(PlaneX, PlaneY, PlaneZ, PlaneW);

	const FString MatrixString = DisplayClusterTypesConverter::template ToString(Matrix);
	UpdateCustomParameterValueText(MatrixString);
}

const FString FPolicyParameterInfoRotator::BaseRotatorString = DisplayClusterTypesConverter::template ToString<FRotator>(FRotator::ZeroRotator);

FPolicyParameterInfoRotator::FPolicyParameterInfoRotator(const FString& InDisplayName, const FString& InKey,
                                                         UDisplayClusterBlueprint* InBlueprint,
                                                         const TArray<TWeakObjectPtr<UDisplayClusterConfigurationViewport>>& InConfigurationViewports) :
	FPolicyParameterInfoFloatReference(InDisplayName, InKey, InBlueprint, InConfigurationViewports, &BaseRotatorString),
	CachedRotationYaw(MakeShared<TOptional<float>>()),
	CachedRotationPitch(MakeShared<TOptional<float>>()),
	CachedRotationRoll(MakeShared<TOptional<float>>())
{

	if (!DoParametersMatchForAllViewports())
	{
		for (const TWeakObjectPtr<UDisplayClusterConfigurationViewport>& Viewport : ConfigurationViewports)
		{
			UDisplayClusterConfigurationViewport* ConfigurationViewport = Viewport.Get();
			check(ConfigurationViewport != nullptr);

			FString* ParameterValue = ConfigurationViewport->ProjectionPolicy.Parameters.Find(GetParameterKey());
			check(ParameterValue);

			const FText TextValue = FText::FromString(*ParameterValue);
			const FRotator Rotation = DisplayClusterTypesConverter::template FromString<FRotator>(TextValue.ToString());

			ResetOrSetCachedValue(CachedRotationYaw, Rotation.Yaw);
			ResetOrSetCachedValue(CachedRotationPitch, Rotation.Pitch);
			ResetOrSetCachedValue(CachedRotationRoll, Rotation.Roll);
		}
	}
	else
	{
		const FText TextValue = GetOrAddCustomParameterValueText();
		const FRotator Rotation = DisplayClusterTypesConverter::template FromString<FRotator>(TextValue.ToString());

		*CachedRotationYaw = Rotation.Yaw;
		*CachedRotationPitch = Rotation.Pitch;
		*CachedRotationRoll = Rotation.Roll;
	}
}

void FPolicyParameterInfoRotator::CreateCustomRowWidget(IDetailChildrenBuilder& InDetailWidgetRow)
{
	InDetailWidgetRow.AddCustomRow(GetParameterDisplayName())
	.Visibility(MakeAttributeRaw(this, &FPolicyParameterInfoRotator::IsParameterVisible))
	.NameContent()
	[
		SNew(STextBlock)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Text(GetParameterDisplayName())
		.ToolTipText(this, &FPolicyParameterInfo::GetParameterTooltip)
	]
	.ValueContent()
	.MinDesiredWidth(375.0f)
	.MaxDesiredWidth(375.0f)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(FMargin(0.0f, 2.0f, 3.0f, 2.0f))
		[
			MakeFloatInputWidget(CachedRotationRoll, LOCTEXT("RotationX", "X"), true, FLinearColor::White, SNumericEntryBox<float>::RedLabelBackgroundColor)
		]
		+ SHorizontalBox::Slot()
		.Padding(FMargin(0.0f, 2.0f, 3.0f, 2.0f))
		[
			MakeFloatInputWidget(CachedRotationPitch, LOCTEXT("RotationY", "Y"), true, FLinearColor::White, SNumericEntryBox<float>::GreenLabelBackgroundColor)
		]
		+ SHorizontalBox::Slot()
		.Padding(FMargin(0.0f, 2.0f, 3.0f, 2.0f))
		[
			MakeFloatInputWidget(CachedRotationYaw, LOCTEXT("RotationZ", "Z"), true, FLinearColor::White, SNumericEntryBox<float>::BlueLabelBackgroundColor)
		]
	];
}

void FPolicyParameterInfoRotator::FormatTextAndUpdateParameter()
{
	const bool bAllValuesSet = CachedRotationPitch->IsSet() && CachedRotationYaw->IsSet() && CachedRotationRoll->IsSet();
	if (!bAllValuesSet)
	{
		return;
	}
	
	const FRotator Rotation(*CachedRotationPitch.Get(), *CachedRotationYaw.Get(), *CachedRotationRoll.Get());

	const FString RotationString = DisplayClusterTypesConverter::template ToString(Rotation);
	UpdateCustomParameterValueText(RotationString);
}


const FString FPolicyParameterInfoVector::BaseVectorString = DisplayClusterTypesConverter::template ToString<FVector>(FVector::ZeroVector);

FPolicyParameterInfoVector::FPolicyParameterInfoVector(const FString& InDisplayName, const FString& InKey,
	UDisplayClusterBlueprint* InBlueprint,
	const TArray<TWeakObjectPtr<UDisplayClusterConfigurationViewport>>& InConfigurationViewports) :
	FPolicyParameterInfoFloatReference(InDisplayName, InKey, InBlueprint, InConfigurationViewports, &BaseVectorString),
	CachedVectorX(MakeShared<TOptional<float>>()),
	CachedVectorY(MakeShared<TOptional<float>>()),
	CachedVectorZ(MakeShared<TOptional<float>>())
{

	if (!DoParametersMatchForAllViewports())
	{
		for (const TWeakObjectPtr<UDisplayClusterConfigurationViewport>& Viewport : ConfigurationViewports)
		{
			UDisplayClusterConfigurationViewport* ConfigurationViewport = Viewport.Get();
			check(ConfigurationViewport != nullptr);

			FString* ParameterValue = ConfigurationViewport->ProjectionPolicy.Parameters.Find(GetParameterKey());
			check(ParameterValue);

			const FText TextValue = FText::FromString(*ParameterValue);
			const FVector Location = DisplayClusterTypesConverter::template FromString<FVector>(TextValue.ToString());

			ResetOrSetCachedValue(CachedVectorX, Location.X);
			ResetOrSetCachedValue(CachedVectorY, Location.Y);
			ResetOrSetCachedValue(CachedVectorZ, Location.Z);
		}
	}
	else
	{
		const FText TextValue = GetOrAddCustomParameterValueText();
		const FVector Location = DisplayClusterTypesConverter::template FromString<FVector>(TextValue.ToString());

		*CachedVectorX = Location.X;
		*CachedVectorY = Location.Y;
		*CachedVectorZ = Location.Z;
	}
}

void FPolicyParameterInfoVector::CreateCustomRowWidget(IDetailChildrenBuilder& InDetailWidgetRow)
{
	InDetailWidgetRow.AddCustomRow(GetParameterDisplayName())
		.Visibility(MakeAttributeRaw(this, &FPolicyParameterInfoVector::IsParameterVisible))
		.NameContent()
		[
			SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(GetParameterDisplayName())
				.ToolTipText(this, &FPolicyParameterInfo::GetParameterTooltip)
		]
		.ValueContent()
		.MinDesiredWidth(375.0f)
		.MaxDesiredWidth(375.0f)
		[
			SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(FMargin(0.0f, 2.0f, 3.0f, 2.0f))
				[
					MakeFloatInputWidget(CachedVectorX, LOCTEXT("LocationX", "X"), true, FLinearColor::White, SNumericEntryBox<float>::RedLabelBackgroundColor)
				]
			+ SHorizontalBox::Slot()
				.Padding(FMargin(0.0f, 2.0f, 3.0f, 2.0f))
				[
					MakeFloatInputWidget(CachedVectorY, LOCTEXT("LocationY", "Y"), true, FLinearColor::White, SNumericEntryBox<float>::GreenLabelBackgroundColor)
				]
			+ SHorizontalBox::Slot()
				.Padding(FMargin(0.0f, 2.0f, 3.0f, 2.0f))
				[
					MakeFloatInputWidget(CachedVectorZ, LOCTEXT("LocationZ", "Z"), true, FLinearColor::White, SNumericEntryBox<float>::BlueLabelBackgroundColor)
				]
		];
}

void FPolicyParameterInfoVector::FormatTextAndUpdateParameter()
{
	const bool bAllValuesSet = CachedVectorX->IsSet() && CachedVectorY->IsSet() && CachedVectorZ->IsSet();
	if (!bAllValuesSet)
	{
		return;
	}

	const FVector Location(*CachedVectorX.Get(), *CachedVectorY.Get(), *CachedVectorZ.Get());

	const FString LocationString = DisplayClusterTypesConverter::template ToString(Location);
	UpdateCustomParameterValueText(LocationString);
}

const FString FPolicyParameterInfoFrustumAngle::BaseFrustumPlanesString = FString("l=-30, r=30, t=30, b=-30");

FPolicyParameterInfoFrustumAngle::FPolicyParameterInfoFrustumAngle(const FString& InDisplayName, const FString& InKey,
	UDisplayClusterBlueprint* InBlueprint, const TArray<TWeakObjectPtr<UDisplayClusterConfigurationViewport>>& InConfigurationViewports) :
	FPolicyParameterInfoFloatReference(InDisplayName, InKey, InBlueprint, InConfigurationViewports, &BaseFrustumPlanesString),
	CachedAngleL(MakeShared<TOptional<float>>()),
	CachedAngleR(MakeShared<TOptional<float>>()),
	CachedAngleT(MakeShared<TOptional<float>>()),
	CachedAngleB(MakeShared<TOptional<float>>())
{
	if (!DoParametersMatchForAllViewports())
	{
		for (const TWeakObjectPtr<UDisplayClusterConfigurationViewport>& Viewport : ConfigurationViewports)
		{
			UDisplayClusterConfigurationViewport* ConfigurationViewport = Viewport.Get();
			check(ConfigurationViewport != nullptr);

			if (const FString* ParameterValue = ConfigurationViewport->ProjectionPolicy.Parameters.Find(GetParameterKey()))
			{
				const FString TextValue = *ParameterValue;
				float Left;
				if (DisplayClusterHelpers::str::ExtractValue(TextValue, FString(DisplayClusterProjectionStrings::cfg::manual::AngleL), Left))
				{
					ResetOrSetCachedValue(CachedAngleL, Left);
				}

				float Right;
				if (DisplayClusterHelpers::str::ExtractValue(TextValue, FString(DisplayClusterProjectionStrings::cfg::manual::AngleR), Right))
				{
					ResetOrSetCachedValue(CachedAngleR, Right);
				}

				float Top;
				if (DisplayClusterHelpers::str::ExtractValue(TextValue, FString(DisplayClusterProjectionStrings::cfg::manual::AngleT), Top))
				{
					ResetOrSetCachedValue(CachedAngleT, Top);
				}

				float Bottom;
				if (DisplayClusterHelpers::str::ExtractValue(TextValue, FString(DisplayClusterProjectionStrings::cfg::manual::AngleB), Bottom))
				{
					ResetOrSetCachedValue(CachedAngleB, Bottom);
				}
			}
		}
	}
	else
	{
		const FString TextValue = GetOrAddCustomParameterValueText().ToString();

		float Left;
		if (DisplayClusterHelpers::str::ExtractValue(TextValue, FString(DisplayClusterProjectionStrings::cfg::manual::AngleL), Left))
		{
			*CachedAngleL = Left;
		}

		float Right;
		if (DisplayClusterHelpers::str::ExtractValue(TextValue, FString(DisplayClusterProjectionStrings::cfg::manual::AngleR), Right))
		{
			*CachedAngleR = Right;
		}

		float Top;
		if (DisplayClusterHelpers::str::ExtractValue(TextValue, FString(DisplayClusterProjectionStrings::cfg::manual::AngleT), Top))
		{
			*CachedAngleT = Top;
		}

		float Bottom;
		if (DisplayClusterHelpers::str::ExtractValue(TextValue, FString(DisplayClusterProjectionStrings::cfg::manual::AngleB), Bottom))
		{
			*CachedAngleB = Bottom;
		}
	}
}

void FPolicyParameterInfoFrustumAngle::CreateCustomRowWidget(IDetailChildrenBuilder& InDetailWidgetRow)
{
	InDetailWidgetRow.AddCustomRow(GetParameterDisplayName())
	.Visibility(MakeAttributeRaw(this, &FPolicyParameterInfoFrustumAngle::IsParameterVisible))
	.NameContent()
	[
		SNew(STextBlock)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Text(GetParameterDisplayName())
		.ToolTipText(this, &FPolicyParameterInfo::GetParameterTooltip)
	]
	.ValueContent()
	.MinDesiredWidth(375.0f)
	.MaxDesiredWidth(375.0f)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(FMargin(0.0f, 2.0f, 3.0f, 2.0f))
		[
			MakeFloatInputWidget(CachedAngleL, LOCTEXT("AngleL", "L"), true, FLinearColor::White, SNumericEntryBox<float>::RedLabelBackgroundColor)
		]
		+ SHorizontalBox::Slot()
		.Padding(FMargin(0.0f, 2.0f, 3.0f, 2.0f))
		[
			MakeFloatInputWidget(CachedAngleR, LOCTEXT("AngleR", "R"), true, FLinearColor::White, SNumericEntryBox<float>::GreenLabelBackgroundColor)
		]
		+ SHorizontalBox::Slot()
		.Padding(FMargin(0.0f, 2.0f, 3.0f, 2.0f))
		[
			MakeFloatInputWidget(CachedAngleT, LOCTEXT("AngleT", "T"), true, FLinearColor::White, SNumericEntryBox<float>::BlueLabelBackgroundColor)
		]
		+ SHorizontalBox::Slot()
		.Padding(FMargin(0.0f, 2.0f, 3.0f, 2.0f))
		[
			MakeFloatInputWidget(CachedAngleB, LOCTEXT("AngleB", "B"), true, FLinearColor::White, OrangeLabelBackgroundColor)
		]
	];
}

void FPolicyParameterInfoFrustumAngle::FormatTextAndUpdateParameter()
{
	const bool bAllValuesSet = CachedAngleL->IsSet() && CachedAngleR->IsSet() && CachedAngleT->IsSet()  && CachedAngleB->IsSet();
	if (!bAllValuesSet)
	{
		return;
	}
	
	const FString AngleLStr = FString::SanitizeFloat(*CachedAngleL.Get());
	const FString AngleRStr = FString::SanitizeFloat(*CachedAngleR.Get());
	const FString AngleTStr = FString::SanitizeFloat(*CachedAngleT.Get());
	const FString AngleBStr = FString::SanitizeFloat(*CachedAngleB.Get());
	
	const FString AngleString = FString::Printf(TEXT("l=%s, r=%s, t=%s, b=%s"), *AngleLStr, *AngleRStr, *AngleTStr, *AngleBStr);

	UpdateCustomParameterValueText(AngleString);
}

/**
* FPolicyParameterInfoResolution
*/
const FString FPolicyParameterInfoResolution::BaseResolutionString = FString("X=2048, Y=2048");

FPolicyParameterInfoResolution::FPolicyParameterInfoResolution(const FString& InDisplayName, const FString& InKey,
	UDisplayClusterBlueprint* InBlueprint, const TArray<TWeakObjectPtr<UDisplayClusterConfigurationViewport>>& InConfigurationViewports)
	: FPolicyParameterInfoIntReference(InDisplayName, InKey, InBlueprint, InConfigurationViewports, &BaseResolutionString)
	, CachedX(MakeShared<TOptional<int32>>())
	, CachedY(MakeShared<TOptional<int32>>())
{
	if (!DoParametersMatchForAllViewports())
	{
		for (const TWeakObjectPtr<UDisplayClusterConfigurationViewport>& Viewport : ConfigurationViewports)
		{
			UDisplayClusterConfigurationViewport* ConfigurationViewport = Viewport.Get();
			check(ConfigurationViewport != nullptr);

			if (const FString* ParameterValue = ConfigurationViewport->ProjectionPolicy.Parameters.Find(GetParameterKey()))
			{
				const FIntPoint Resolution = DisplayClusterTypesConverter::template FromString<FIntPoint>(*ParameterValue);

				ResetOrSetCachedValue(CachedX, FMath::Max(0, Resolution.X));
				ResetOrSetCachedValue(CachedY, FMath::Max(0, Resolution.Y));
			}
		}
	}
	else
	{
		const FString TextValue = GetOrAddCustomParameterValueText().ToString();
		const FIntPoint Resolution = DisplayClusterTypesConverter::template FromString<FIntPoint>(*TextValue);

		*CachedX = FMath::Max(0, Resolution.X);
		*CachedY = FMath::Max(0, Resolution.Y);
	}
}

void FPolicyParameterInfoResolution::CreateCustomRowWidget(IDetailChildrenBuilder& InDetailWidgetRow)
{
	InDetailWidgetRow.AddCustomRow(GetParameterDisplayName())
		.Visibility(MakeAttributeRaw(this, &FPolicyParameterInfoResolution::IsParameterVisible))
		.NameContent()
		[
			SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(GetParameterDisplayName())
				.ToolTipText(this, &FPolicyParameterInfo::GetParameterTooltip)
		]
		.ValueContent()
		.MinDesiredWidth(375.0f)
		.MaxDesiredWidth(375.0f)
		[
			SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(FMargin(0.0f, 2.0f, 3.0f, 2.0f))
				[
					MakeIntInputWidget(CachedX, LOCTEXT("Width", "W"),
						FLinearColor::White, SNumericEntryBox<float>::RedLabelBackgroundColor,
						&GDisplayClusterConfiguratorMinResolution, &GDisplayClusterConfiguratorMaxResolution)
				]
				+ SHorizontalBox::Slot()
				.Padding(FMargin(0.0f, 2.0f, 3.0f, 2.0f))
				[
					MakeIntInputWidget(CachedY, LOCTEXT("Height", "H"),
						FLinearColor::White, SNumericEntryBox<float>::GreenLabelBackgroundColor,
						&GDisplayClusterConfiguratorMinResolution, &GDisplayClusterConfiguratorMaxResolution)
				]
		];
}

void FPolicyParameterInfoResolution::FormatTextAndUpdateParameter()
{
	const bool bAllValuesSet = CachedX->IsSet() && CachedY->IsSet();
	if (!bAllValuesSet)
	{
		return;
	}

	const FIntPoint Resolution(*CachedX.Get(), *CachedY.Get());
	UpdateCustomParameterValueText(Resolution.ToString());
}

/**
* FPolicyParameterInfoNormalizedVector2D
*/
const FString FPolicyParameterInfoNormalizedVector2D::BaseVectorValueStr = FString("X=0, Y=0");

FPolicyParameterInfoNormalizedVector2D::FPolicyParameterInfoNormalizedVector2D(const FString& InDisplayName, const FString& InKey,
	UDisplayClusterBlueprint* InBlueprint, const TArray<TWeakObjectPtr<UDisplayClusterConfigurationViewport>>& InConfigurationViewports, const FString* InDefaultValue)
	: FPolicyParameterInfoFloatReference(InDisplayName, InKey, InBlueprint, InConfigurationViewports, InDefaultValue ? InDefaultValue : &BaseVectorValueStr)
	, CachedX(MakeShared<TOptional<float>>())
	, CachedY(MakeShared<TOptional<float>>())
{
	if (!DoParametersMatchForAllViewports())
	{
		for (const TWeakObjectPtr<UDisplayClusterConfigurationViewport>& Viewport : ConfigurationViewports)
		{
			UDisplayClusterConfigurationViewport* ConfigurationViewport = Viewport.Get();
			check(ConfigurationViewport != nullptr);

			if (const FString* ParameterValue = ConfigurationViewport->ProjectionPolicy.Parameters.Find(GetParameterKey()))
			{
				const FVector2D Pts = DisplayClusterTypesConverter::template FromString<FVector2D>(*ParameterValue);

				ResetOrSetCachedValue(CachedX, FMath::Max(0.f, FMath::Min(1.f, Pts.X)));
				ResetOrSetCachedValue(CachedY, FMath::Max(0.f, FMath::Min(1.f, Pts.Y)));
			}
		}
	}
	else
	{
		const FString TextValue = GetOrAddCustomParameterValueText().ToString();
		const FVector2D Pts = DisplayClusterTypesConverter::template FromString<FVector2D>(*TextValue);

		*CachedX = FMath::Max(0.f, FMath::Min(1.f, Pts.X));
		*CachedY = FMath::Max(0.f, FMath::Min(1.f, Pts.Y));
	}
}

void FPolicyParameterInfoNormalizedVector2D::CreateCustomRowWidget(IDetailChildrenBuilder& InDetailWidgetRow)
{
	const float MinVectorComponentValue = 0.f;
	const float MaxVectorComponentValue = 1.0f;

	InDetailWidgetRow.AddCustomRow(GetParameterDisplayName())
		.Visibility(MakeAttributeRaw(this, &FPolicyParameterInfoNormalizedVector2D::IsParameterVisible))
		.NameContent()
		[
			SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(GetParameterDisplayName())
				.ToolTipText(this, &FPolicyParameterInfo::GetParameterTooltip)
		]
		.ValueContent()
		.MinDesiredWidth(375.0f)
		.MaxDesiredWidth(375.0f)
		[
			SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(FMargin(0.0f, 2.0f, 3.0f, 2.0f))
				[
					MakeFloatInputWidget(CachedX, LOCTEXT("NormV2D_X", "X"), false, FLinearColor::White, SNumericEntryBox<float>::RedLabelBackgroundColor, &MinVectorComponentValue, &MaxVectorComponentValue)
				]
				+ SHorizontalBox::Slot()
				.Padding(FMargin(0.0f, 2.0f, 3.0f, 2.0f))
				[
					MakeFloatInputWidget(CachedY, LOCTEXT("NormV2D_Y", "Y"), false, FLinearColor::White, SNumericEntryBox<float>::GreenLabelBackgroundColor, &MinVectorComponentValue, &MaxVectorComponentValue)
				]
		];
}

void FPolicyParameterInfoNormalizedVector2D::FormatTextAndUpdateParameter()
{
	const bool bAllValuesSet = CachedX->IsSet() && CachedY->IsSet();
	if (!bAllValuesSet)
	{
		return;
	}

	const FVector2D Pts(*CachedX.Get(), *CachedY.Get());
	UpdateCustomParameterValueText(DisplayClusterTypesConverter::ToString(Pts));
}

#undef LOCTEXT_NAMESPACE
