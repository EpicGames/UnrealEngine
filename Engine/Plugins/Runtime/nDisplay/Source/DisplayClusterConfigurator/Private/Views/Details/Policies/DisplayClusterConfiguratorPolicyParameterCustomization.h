// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Components/ActorComponent.h"

#include "IDetailChildrenBuilder.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Layout/Visibility.h"
#include "Types/SlateEnums.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SNumericEntryBox.h"

class SDisplayClusterConfigurationSearchableComboBox;
class FDisplayClusterConfiguratorBlueprintEditor;
class UDisplayClusterBlueprint;
class ADisplayClusterRootActor;
class UDisplayClusterConfigurationViewport;
class IDetailChildrenBuilder;

/**
 * Individual policy parameter data base class.
 */
class FPolicyParameterInfo : public TSharedFromThis<FPolicyParameterInfo>
{
public:
	DECLARE_DELEGATE_RetVal_OneParam(EVisibility, FParameterVisible, FText);
	
public:
	FPolicyParameterInfo(
		const FString& InDisplayName,
		const FString& InKey,
		UDisplayClusterBlueprint* InBlueprint,
		const TArray<TWeakObjectPtr<UDisplayClusterConfigurationViewport>>& InConfigurationViewports,
		const FString* InInitialValue = nullptr);

	virtual ~FPolicyParameterInfo() {}

	/** The friendly display name for this parameter. */
	FText GetParameterDisplayName() const { return FText::FromString(ParamDisplayName); }

	/** Gets the tooltip for this parameter. */
	FText GetParameterTooltip() const { return ParamTooltip; }

	/** Sets the tooltip for this parameter. */
	void SetParameterTooltip(const FText& NewTooltip) { ParamTooltip = NewTooltip; }

	/** The keu used for the parameter map. */
	const FString& GetParameterKey() const { return ParamKey; }

	/** Create the widget representation of this parameter and add it as a child row. */
	virtual void CreateCustomRowWidget(IDetailChildrenBuilder& InDetailWidgetRow) = 0;

	/** This delegate will be used to determine visibility. */
	void SetParameterVisibilityDelegate(FParameterVisible InDelegate);

	/** Retrieve (or add) the parameter information from the viewport data object. */
	FText GetOrAddCustomParameterValueText() const;
protected:
	
	/** Checks the data model if the parameter exists. */
	bool IsParameterAlreadyAdded() const;

	/** Compares all selected viewport values. */
	bool DoParametersMatchForAllViewports() const;
	
	/** Update the parameter value in the viewport data object. */
	void UpdateCustomParameterValueText(const FString& NewValue, bool bNotify = true) const;

	/** Visibility of this parameter. Uses OnParameterVisibilityCheck to check. */
	virtual EVisibility IsParameterVisible() const;

protected:
	/** Owning blueprint of this policy. */
	TWeakObjectPtr<UDisplayClusterBlueprint> BlueprintOwnerPtr;

	/** Viewport owning policy. */
	TArray<TWeakObjectPtr<UDisplayClusterConfigurationViewport>> ConfigurationViewports;

	/** BP editor found from BlueprintOwner. */
	FDisplayClusterConfiguratorBlueprintEditor* BlueprintEditorPtrCached;

	/** Optional initial selected item. */
	TSharedPtr<FString> InitialValue;

	/** Used to determine parameter visibility. */
	FParameterVisible OnParameterVisibilityCheck;

private:
	/** The display name only for the UI. */
	FString ParamDisplayName;
	
	/** The tooltip to display in the UI */
	FText ParamTooltip;

	/** The proper title (key) of this parameter. */
	FString ParamKey;
};

/**
 * Policy info for combo box picker with custom values.
 */
class FPolicyParameterInfoCombo : public FPolicyParameterInfo
{
public:
	DECLARE_DELEGATE_OneParam(FOnItemSelected, const FString&)
	
public:
	FPolicyParameterInfoCombo(
		const FString& InDisplayName,
		const FString& InKey,
		UDisplayClusterBlueprint* InBlueprint,
		const TArray<TWeakObjectPtr<UDisplayClusterConfigurationViewport>>& InConfigurationViewports,
		const TArray<FString>& InValues,
		const FString* InInitialItem = nullptr,
		bool bSort = true);

	// FPolicyParameterInfo
	virtual void CreateCustomRowWidget(IDetailChildrenBuilder& InDetailWidgetRow) override;
	// ~FPolicyParameterInfo

	/** The combo box containing all of the options.  */
	TSharedPtr<SDisplayClusterConfigurationSearchableComboBox>& GetCustomParameterValueComboBox() { return CustomParameterValueComboBox; }

	/** The options used in the combo box. */
	TArray< TSharedPtr<FString> >& GetCustomParameterOptions() { return CustomParameterOptions; }

	void SetOnSelectedDelegate(FOnItemSelected InDelegate);

protected:
	/** Widget used for each option. */
	TSharedRef<SWidget> MakeCustomParameterValueComboWidget(TSharedPtr<FString> InItem);
	
	/** When a custom parameter value option has been selected. */
	void OnCustomParameterValueSelected(TSharedPtr<FString> InValue, ESelectInfo::Type SelectInfo);

protected:
	/** Current parameters' value combo box. */
	TSharedPtr<SDisplayClusterConfigurationSearchableComboBox> CustomParameterValueComboBox;

	/** List of options for the combo value box. */
	TArray< TSharedPtr<FString> > CustomParameterOptions;

	FOnItemSelected OnItemSelected;
};

/**
 * Policy info for combo box picker for actor components.
 */
class FPolicyParameterInfoComponentCombo final : public FPolicyParameterInfoCombo
{
public:
	FPolicyParameterInfoComponentCombo(
		const FString& InDisplayName,
		const FString& InKey,
		UDisplayClusterBlueprint* InBlueprint,
		const TArray<TWeakObjectPtr<UDisplayClusterConfigurationViewport>>& InConfigurationViewports,
		const TArray<TSubclassOf<UActorComponent>>& InComponentClasses);

	/** Populate CustomParameterOptions based on this parameter. */
	void CreateParameterValues(ADisplayClusterRootActor* RootActor);

	/** The type of component this parameter represents. */
	const TArray<TSubclassOf<UActorComponent>>& GetComponentTypes() const { return ComponentTypes; }

private:
	/** A component class to use for creating parameter options. */
	TArray<TSubclassOf<UActorComponent>> ComponentTypes;
};

/**
 * Policy info for combo box picker for viewports (for Reference projection policy).
 */
class FPolicyParameterInfoViewportCombo final : public FPolicyParameterInfoCombo
{
public:
	FPolicyParameterInfoViewportCombo(
		const FString& InDisplayName,
		const FString& InKey,
		UDisplayClusterBlueprint* InBlueprint,
		const TArray<TWeakObjectPtr<UDisplayClusterConfigurationViewport>>& InConfigurationViewports);

	// FPolicyParameterInfo
	virtual void CreateCustomRowWidget(IDetailChildrenBuilder& InDetailWidgetRow) override;
	// ~FPolicyParameterInfo

	/** Populate CustomParameterOptions with all viewports except the currently edited ones. */
	void CreateParameterValues();

private:
	/** Check if all edited viewports belong to the same cluster node. */
	bool AreAllViewportsInSameClusterNode() const;
};

/**
 * Policy info for number value.
 */
template<typename T>
class FPolicyParameterInfoNumber final : public FPolicyParameterInfo
{
public:
	FPolicyParameterInfoNumber(
		const FString& InDisplayName,
		const FString& InKey,
		UDisplayClusterBlueprint* InBlueprint,
		const TArray<TWeakObjectPtr<UDisplayClusterConfigurationViewport>>& InConfigurationViewports,
		T InDefaultValue = 0, TOptional<T> InMinValue = TOptional<T>(), TOptional<T> InMaxValue = TOptional<T>()) : FPolicyParameterInfo(InDisplayName, InKey, InBlueprint, InConfigurationViewports)
	{
		MinValue = InMinValue;
		MaxValue = InMaxValue;

		if (!DoParametersMatchForAllViewports())
		{
			bIsMultipleValues = true;
		}
		
		if (IsParameterAlreadyAdded())
		{
			const FText TextValue = GetOrAddCustomParameterValueText();
			NumberValue = static_cast<T>(FCString::Atof(*TextValue.ToString()));
		}
		else
		{
			NumberValue = InDefaultValue;
			GetOrAddCustomParameterValueText();
			UpdateCustomParameterValueText(FString::SanitizeFloat(static_cast<float>(NumberValue)),
				/*bNotify*/ false); // Notify only necessary on a change. Calling during construction could potentially crash when details is refreshed.
		}
	}

	// FPolicyParameterInfo
	virtual void CreateCustomRowWidget(IDetailChildrenBuilder& InDetailWidgetRow) override
	{
		InDetailWidgetRow.AddCustomRow(GetParameterDisplayName())
			.NameContent()
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(GetParameterDisplayName())
				.ToolTipText(this, &FPolicyParameterInfo::GetParameterTooltip)
			]
			.ValueContent()
			[
				SNew(SNumericEntryBox<T>)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Value(this, &FPolicyParameterInfoNumber::OnGetValue)
				.AllowSpin(true)
				.MinValue(MinValue)
				.MaxValue(MaxValue)
				.MinSliderValue(MinValue)
				.MaxSliderValue(MaxValue)
				.UndeterminedString(NSLOCTEXT("PropertyEditor", "MultipleValues", "Multiple Values"))
				.OnValueChanged_Lambda([this](T InValue)
				{
					NumberValue = InValue;
					bIsMultipleValues = false;
					UpdateCustomParameterValueText(FString::SanitizeFloat(static_cast<float>(NumberValue)));
				})
			];
	}
	// ~FPolicyParameterInfo
	
	TOptional<T> OnGetValue() const
	{
		TOptional<T> Result(NumberValue);
		if (bIsMultipleValues)
		{
			Result.Reset();
		}
		return Result;
	}

private:
	TOptional<T> MinValue;
	TOptional<T> MaxValue;
	T NumberValue;
	bool bIsMultipleValues = false;
};

/**
 * Policy info for text value.
 */
class FPolicyParameterInfoText final : public FPolicyParameterInfo
{
public:
	FPolicyParameterInfoText(
		const FString& InDisplayName,
		const FString& InKey,
		UDisplayClusterBlueprint* InBlueprint,
		const TArray<TWeakObjectPtr<UDisplayClusterConfigurationViewport>>& InConfigurationViewports);

	// FPolicyParameterInfo
	virtual void CreateCustomRowWidget(IDetailChildrenBuilder& InDetailWidgetRow) override;
	// ~FPolicyParameterInfo
};

/**
 * Policy info for bool value.
 */
class FPolicyParameterInfoBool final : public FPolicyParameterInfo
{
public:
	FPolicyParameterInfoBool(
		const FString& InDisplayName,
		const FString& InKey,
		UDisplayClusterBlueprint* InBlueprint,
		const TArray<TWeakObjectPtr<UDisplayClusterConfigurationViewport>>& InConfigurationViewports,
		bool bInvertValue = false
		);

	// FPolicyParameterInfo
	virtual void CreateCustomRowWidget(IDetailChildrenBuilder& InDetailWidgetRow) override;
	// ~FPolicyParameterInfo

private:
	ECheckBoxState IsChecked() const;

	bool bInvertValue;
};

/**
 * Policy info for file picker.
 */
class FPolicyParameterInfoFile final : public FPolicyParameterInfo
{
public:
	FPolicyParameterInfoFile(
		const FString& InDisplayName,
		const FString& InKey,
		UDisplayClusterBlueprint* InBlueprint,
		const TArray<TWeakObjectPtr<UDisplayClusterConfigurationViewport>>& InConfigurationViewports,
		const TArray<FString>& InFileExtensions) : FPolicyParameterInfo(InDisplayName, InKey, InBlueprint, InConfigurationViewports)
	{
		FileExtensions = InFileExtensions;
	}

	// FPolicyParameterInfo
	virtual void CreateCustomRowWidget(IDetailChildrenBuilder& InDetailWidgetRow) override;
	// ~FPolicyParameterInfo

private:
	/* Prompts user to select a file. */
	void OpenSelectFileDialogue();
	FReply OnChangePathClicked();

private:
	TArray<FString> FileExtensions;
};

/**
 * Policy info for modifying float reference, typically in a vector or matrix.
 */
class FPolicyParameterInfoFloatReference : public FPolicyParameterInfo
{
public:
	FPolicyParameterInfoFloatReference(
		const FString& InDisplayName,
		const FString& InKey,
		UDisplayClusterBlueprint* InBlueprint,
		const TArray<TWeakObjectPtr<UDisplayClusterConfigurationViewport>>& InConfigurationViewports,
		const FString* InInitialValue = nullptr) :
	FPolicyParameterInfo(InDisplayName, InKey, InBlueprint, InConfigurationViewports, InInitialValue)
	{
	}

protected:
	/** Converts values to text. */
	virtual void FormatTextAndUpdateParameter() = 0;
	
	/** Creates SNumericEntryBox<float> widget instance. */
	TSharedRef<SWidget> MakeFloatInputWidget(TSharedRef<TOptional<float>>& ProxyValue, const FText& Label,
											const bool bRotationInDegrees,
											const FLinearColor& LabelColor, const FLinearColor& LabelBackgroundColor,
											const float* InMinValue = nullptr, const float* InMaxValue = nullptr);

	/** Returns an optional float value. */
	TOptional<float> OnGetValue(TSharedRef<TOptional<float>> Value) const
	{
		return Value.Get();
	}

	/** Callback that is called when the value changes. */
	void OnValueCommitted(float NewValue, ETextCommit::Type CommitType, TSharedRef<TOptional<float>> Value)
	{
		*Value = NewValue;
		FormatTextAndUpdateParameter();
	}

	/** Reset or set cached value.
	* This function can be called multiple times when editing a value for a group selection.
	* In this case, when the values in the group differ, a reset is used.
	*/
	static void ResetOrSetCachedValue(TSharedRef<TOptional<float>>& InCachedValue, float InCompareValue)
	{
		if (InCachedValue->IsSet() && InCachedValue.Get() != InCompareValue)
		{
			InCachedValue->Reset();
		}
		else
		{
			*InCachedValue = InCompareValue;
		}
	};
};

/**
 * Policy info for modifying positive int32 reference, typically in a resolution
 */
class FPolicyParameterInfoIntReference : public FPolicyParameterInfo
{
public:
	FPolicyParameterInfoIntReference(
		const FString& InDisplayName,
		const FString& InKey,
		UDisplayClusterBlueprint* InBlueprint,
		const TArray<TWeakObjectPtr<UDisplayClusterConfigurationViewport>>& InConfigurationViewports,
		const FString* InInitialValue = nullptr) :
		FPolicyParameterInfo(InDisplayName, InKey, InBlueprint, InConfigurationViewports, InInitialValue)
	{
	}

protected:
	/** Converts values to text. */
	virtual void FormatTextAndUpdateParameter() = 0;

	/** Creates SNumericEntryBox<int32> widget instance. */
	TSharedRef<SWidget> MakeIntInputWidget(TSharedRef<TOptional<int32>>& ProxyValue, const FText& Label,
										const FLinearColor& LabelColor, const FLinearColor& LabelBackgroundColor,
										const int32* InMinValue = nullptr, const int32* InMaxValue = nullptr);

	/** Returns an optional int32 value. */
	TOptional<int32> OnGetValue(TSharedRef<TOptional<int32>> Value) const
	{
		return Value.Get();
	}

	/** Callback that is called when the value changes. */
	void OnValueCommitted(int32 NewValue, ETextCommit::Type CommitType, TSharedRef<TOptional<int32>> Value)
	{
		*Value = NewValue;
		FormatTextAndUpdateParameter();
	}

	/** Reset or set cached value.
	* This function can be called multiple times when editing a value for a group selection.
	* In this case, when the values in the group differ, a reset is used.
	*/
	static void ResetOrSetCachedValue(TSharedRef<TOptional<int32>>& InCachedValue, int32 InCompareValue)
	{
		if (InCachedValue->IsSet() && InCachedValue.Get() != InCompareValue)
		{
			InCachedValue->Reset();
		}
		else
		{
			*InCachedValue = InCompareValue;
		}
	};
};

/**
 * Policy info for matrix.
 */
class FPolicyParameterInfoMatrix final : public FPolicyParameterInfoFloatReference
{
public:
	FPolicyParameterInfoMatrix(
		const FString& InDisplayName,
		const FString& InKey,
		UDisplayClusterBlueprint* InBlueprint,
		const TArray<TWeakObjectPtr<UDisplayClusterConfigurationViewport>>& InConfigurationViewports);

	// FPolicyParameterInfo
	virtual void CreateCustomRowWidget(IDetailChildrenBuilder& InDetailWidgetRow) override;
	// ~FPolicyParameterInfo

private:
	void CustomizeLocation(FDetailWidgetRow& InDetailWidgetRow);
	void CustomizeRotation(FDetailWidgetRow& InDetailWidgetRow);
	void CustomizeScale(FDetailWidgetRow& InDetailWidgetRow);
	virtual void FormatTextAndUpdateParameter() override;

private:
	mutable TSharedRef<TOptional<float>> CachedTranslationX;
	mutable TSharedRef<TOptional<float>> CachedTranslationY;
	mutable TSharedRef<TOptional<float>> CachedTranslationZ;

	mutable TSharedRef<TOptional<float>> CachedRotationYaw;
	mutable TSharedRef<TOptional<float>> CachedRotationPitch;
	mutable TSharedRef<TOptional<float>> CachedRotationRoll;

	mutable TSharedRef<TOptional<float>> CachedScaleX;
	mutable TSharedRef<TOptional<float>> CachedScaleY;
	mutable TSharedRef<TOptional<float>> CachedScaleZ;

	static const FString BaseMatrixString;
};

/**
* Policy info for 4x4 matrix.
*/
class FPolicyParameterInfo4x4Matrix final : public FPolicyParameterInfoFloatReference
{
public:
	FPolicyParameterInfo4x4Matrix(
		const FString& InDisplayName,
		const FString& InKey,
		UDisplayClusterBlueprint* InBlueprint,
		const TArray<TWeakObjectPtr<UDisplayClusterConfigurationViewport>>& InConfigurationViewports,
		const FString* DefaultMatrixString = nullptr);

	// FPolicyParameterInfo
	virtual void CreateCustomRowWidget(IDetailChildrenBuilder& InDetailWidgetRow) override;
	// ~FPolicyParameterInfo

private:
	void CustomizeRow(const FText& InHeaderText, TSharedRef<TOptional<float>>& InX, TSharedRef<TOptional<float>>& InY,
		TSharedRef<TOptional<float>>& InZ, TSharedRef<TOptional<float>>& InW, FDetailWidgetRow& InDetailWidgetRow);
	virtual void FormatTextAndUpdateParameter() override;

private:
	mutable TSharedRef<TOptional<float>> A;
	mutable TSharedRef<TOptional<float>> B;
	mutable TSharedRef<TOptional<float>> C;
	mutable TSharedRef<TOptional<float>> D;

	mutable TSharedRef<TOptional<float>> E;
	mutable TSharedRef<TOptional<float>> F;
	mutable TSharedRef<TOptional<float>> G;
	mutable TSharedRef<TOptional<float>> H;

	mutable TSharedRef<TOptional<float>> I;
	mutable TSharedRef<TOptional<float>> J;
	mutable TSharedRef<TOptional<float>> K;
	mutable TSharedRef<TOptional<float>> L;

	mutable TSharedRef<TOptional<float>> M;
	mutable TSharedRef<TOptional<float>> N;
	mutable TSharedRef<TOptional<float>> O;
	mutable TSharedRef<TOptional<float>> P;

	static const FString BaseMatrixString;
};

/**
 * Policy info for rotator.
 */
class FPolicyParameterInfoRotator final : public FPolicyParameterInfoFloatReference
{
public:
	FPolicyParameterInfoRotator(
		const FString& InDisplayName,
		const FString& InKey,
		UDisplayClusterBlueprint* InBlueprint,
		const TArray<TWeakObjectPtr<UDisplayClusterConfigurationViewport>>& InConfigurationViewports);

	// FPolicyParameterInfo
	virtual void CreateCustomRowWidget(IDetailChildrenBuilder& InDetailWidgetRow) override;
	// ~FPolicyParameterInfo

private:
	virtual void FormatTextAndUpdateParameter() override;

private:
	mutable TSharedRef<TOptional<float>> CachedRotationYaw;
	mutable TSharedRef<TOptional<float>> CachedRotationPitch;
	mutable TSharedRef<TOptional<float>> CachedRotationRoll;

	static const FString BaseRotatorString;
};

/**
 * Policy info for vector.
 */
class FPolicyParameterInfoVector final : public FPolicyParameterInfoFloatReference
{
public:
	FPolicyParameterInfoVector(
		const FString& InDisplayName,
		const FString& InKey,
		UDisplayClusterBlueprint* InBlueprint,
		const TArray<TWeakObjectPtr<UDisplayClusterConfigurationViewport>>& InConfigurationViewports);

	// FPolicyParameterInfo
	virtual void CreateCustomRowWidget(IDetailChildrenBuilder& InDetailWidgetRow) override;
	// ~FPolicyParameterInfo

private:
	virtual void FormatTextAndUpdateParameter() override;

private:
	mutable TSharedRef<TOptional<float>> CachedVectorX;
	mutable TSharedRef<TOptional<float>> CachedVectorY;
	mutable TSharedRef<TOptional<float>> CachedVectorZ;

	static const FString BaseVectorString;
};

/**
 * Policy info for frustum angle.
 */
class FPolicyParameterInfoFrustumAngle final : public FPolicyParameterInfoFloatReference
{
public:
	FPolicyParameterInfoFrustumAngle(
		const FString& InDisplayName,
		const FString& InKey,
		UDisplayClusterBlueprint* InBlueprint,
		const TArray<TWeakObjectPtr<UDisplayClusterConfigurationViewport>>& InConfigurationViewports);

	// FPolicyParameterInfo
	virtual void CreateCustomRowWidget(IDetailChildrenBuilder& InDetailWidgetRow) override;
	// ~FPolicyParameterInfo

private:
	virtual void FormatTextAndUpdateParameter() override;

private:
	mutable TSharedRef<TOptional<float>> CachedAngleL;
	mutable TSharedRef<TOptional<float>> CachedAngleR;
	mutable TSharedRef<TOptional<float>> CachedAngleT;
	mutable TSharedRef<TOptional<float>> CachedAngleB;

	static const FString BaseFrustumPlanesString;
};

/**
 * Policy info for resolution
 */
class FPolicyParameterInfoResolution final : public FPolicyParameterInfoIntReference
{
public:
	FPolicyParameterInfoResolution(
		const FString& InDisplayName,
		const FString& InKey,
		UDisplayClusterBlueprint* InBlueprint,
		const TArray<TWeakObjectPtr<UDisplayClusterConfigurationViewport>>& InConfigurationViewports);

	// FPolicyParameterInfo
	virtual void CreateCustomRowWidget(IDetailChildrenBuilder& InDetailWidgetRow) override;
	// ~FPolicyParameterInfo

private:
	virtual void FormatTextAndUpdateParameter() override;

private:
	mutable TSharedRef<TOptional<int32>> CachedX;
	mutable TSharedRef<TOptional<int32>> CachedY;

	static const FString BaseResolutionString;
};

/**
 * Policy information for a normalized vector 2d whose component values must be between 0 and 1.
 */
class FPolicyParameterInfoNormalizedVector2D final : public FPolicyParameterInfoFloatReference
{
public:
	FPolicyParameterInfoNormalizedVector2D(
		const FString& InDisplayName,
		const FString& InKey,
		UDisplayClusterBlueprint* InBlueprint,
		const TArray<TWeakObjectPtr<UDisplayClusterConfigurationViewport>>& InConfigurationViewports,
		const FString* InDefaultValue = nullptr
	);

	// FPolicyParameterInfo
	virtual void CreateCustomRowWidget(IDetailChildrenBuilder& InDetailWidgetRow) override;
	// ~FPolicyParameterInfo

private:
	virtual void FormatTextAndUpdateParameter() override;

private:
	mutable TSharedRef<TOptional<float>> CachedX;
	mutable TSharedRef<TOptional<float>> CachedY;

	static const FString BaseVectorValueStr;
};
