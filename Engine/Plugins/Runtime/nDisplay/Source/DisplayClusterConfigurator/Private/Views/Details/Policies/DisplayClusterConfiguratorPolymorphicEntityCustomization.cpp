// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfiguratorPolymorphicEntityCustomization.h"

#include "DisplayClusterConfigurationTypes.h"

#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"

void FDisplayClusterConfiguratorPolymorphicEntityCustomization::Initialize(const TSharedRef<IPropertyHandle>& InPropertyHandle, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	FDisplayClusterConfiguratorBaseTypeCustomization::Initialize(InPropertyHandle, CustomizationUtils);

	TypeHandle = GET_CHILD_HANDLE(FDisplayClusterConfigurationPolymorphicEntity, Type);
	ParametersHandle = GET_CHILD_HANDLE(FDisplayClusterConfigurationPolymorphicEntity, Parameters);
	IsCustomHandle = GET_CHILD_HANDLE(FDisplayClusterConfigurationPolymorphicEntity, bIsCustom);

	IsCustomHandle->MarkHiddenByCustomization();
}