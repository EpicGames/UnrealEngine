// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Views/Details/DisplayClusterConfiguratorBaseTypeCustomization.h"

/** Base type customization for any types derived from FDisplayClusterConfigurationPolymorphicEntity */
class FDisplayClusterConfiguratorPolymorphicEntityCustomization : public FDisplayClusterConfiguratorBaseTypeCustomization
{
protected:
	//~ FDisplayClusterConfiguratorBaseTypeCustomization overrides begin
	virtual void Initialize(const TSharedRef<IPropertyHandle>& InPropertyHandle, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	//~ FDisplayClusterConfiguratorBaseTypeCustomization overrides end

	/** The property handle of the Type property of the FDisplayClusterConfigurationPolymorphicEntity type */
	TSharedPtr<IPropertyHandle> TypeHandle;
	
	/** The property handle of the Parameters property of the FDisplayClusterConfigurationPolymorphicEntity type */
	TSharedPtr<IPropertyHandle> ParametersHandle;

	/** The property handle of the bIsCustom property of the FDisplayClusterConfigurationPolymorphicEntity type */
	TSharedPtr<IPropertyHandle> IsCustomHandle;
};
