// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/Details/Upscaler/DisplayClusterConfigurationUpscalerSettingsDetailCustomization.h"

#include "DetailLayoutBuilder.h"
#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterConfigurationTypes_Enums.h"

#include "IUpscalerModularFeature.h"

#include "UObject/EnumProperty.h"
#include "UObject/UnrealType.h"
#include "UObject/Object.h"

#include "Widgets/Input/SComboBox.h"

#include "DetailWidgetRow.h"
#include "DisplayClusterRootActor.h"

#include "Features/IModularFeatures.h"

#include "PropertyEditorModule.h"
#include "PropertyEditorDelegates.h"
#include "PropertyBagDetails.h"
#include "IDetailChildrenBuilder.h"

#include "IPropertyUtilities.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "DisplayClusterConfigurationUpscalerSettingsDetailCustomization"

using namespace UE::VirtualProduction;

namespace UE::DisplayClusterConfigurator::UpscalerSettingsDetailCustomization
{
	/** Returns true if only the enabled upscaling methods can be seen by the user. */
	static bool ShouldShowOnlyEnabledUpscalingMethods()
	{
		static IConsoleVariable* const ICVar = IConsoleManager::Get().FindConsoleVariable(TEXT("nDisplay.render.upscaling.HideDisabledMethods"));
		if (ICVar && ICVar->GetInt() == 0)
		{
			return false;
		}

		return true;
	}

	/** Returns true if the method is enabled for the current project settings/hardware. */
	static bool IsUpscalingMethodEnabled(const EDisplayClusterConfigurationUpscalingMethod InMethod)
	{
		return true;
	}

	/** Iterate over all default upscaler methods. */
	static void ForeachDefaultUpscalerMethod(TFunction<void(const UEnum&, const int32, const EDisplayClusterConfigurationUpscalingMethod)> IteratorFunc)
	{
		// Collect embedded values from the EDisplayClusterConfigurationUpscalingMethod
		if (const UEnum* EnumSource = StaticEnum<EDisplayClusterConfigurationUpscalingMethod>())
		{
			for (int32 EnumIndex = 0; EnumIndex < EnumSource->NumEnums() - 1; ++EnumIndex)
			{
				const bool bIsHidden = EnumSource->HasMetaData(TEXT("Hidden"), EnumIndex);
				if (!bIsHidden)
				{
					const EDisplayClusterConfigurationUpscalingMethod UpscalingMethod = static_cast<EDisplayClusterConfigurationUpscalingMethod>(EnumIndex);
					if (!ShouldShowOnlyEnabledUpscalingMethods() || IsUpscalingMethodEnabled(UpscalingMethod))
					{
						IteratorFunc(*EnumSource, EnumIndex, UpscalingMethod);
					}
				}
			}
		}
	}

	/** Iterate over all upscaler interfaces. */
	static void ForeachUpscaler(TFunction<void(IUpscalerModularFeature&)> IteratorFunc)
	{
		static IModularFeatures& ModularFeatures = IModularFeatures::Get();

		ModularFeatures.LockModularFeatureList();
		ON_SCOPE_EXIT
		{
			ModularFeatures.UnlockModularFeatureList();
		};

		const TArray<IUpscalerModularFeature*> Extenders = ModularFeatures.GetModularFeatureImplementations<IUpscalerModularFeature>(IUpscalerModularFeature::ModularFeatureName);
		for (IUpscalerModularFeature* Extender : Extenders)
		{
			// Iterate over active upscalers
			if (Extender
				&& (!ShouldShowOnlyEnabledUpscalingMethods() || Extender->IsFeatureEnabled()))
			{
				IteratorFunc(*Extender);
			}
		}
	}

	/** Get upscaling settings for method name. */
	static void GetMethodSettings(const FName& InMethodName, FInstancedPropertyBag& InOutEditableData)
	{
		if (InMethodName.IsNone())
		{
			return;
		}

		// Get settings from the custom upscaler
		ForeachUpscaler([&](IUpscalerModularFeature& Upscaler)
			{
				if (Upscaler.GetName() == InMethodName)
				{
					Upscaler.GetSettings(InOutEditableData);
				}
			});
	}
	
	class FPropertyBagCustomization : public FPropertyBagInstanceDataDetails
	{
	public:
		FPropertyBagCustomization(
			const TArray<TWeakObjectPtr<UObject>>& InOwningObjects,
			TSharedPtr<IPropertyHandle> InParentStructProperty,
			TSharedPtr<IPropertyHandle> InPropertyBagHandle,
			const TSharedPtr<IPropertyUtilities>& InPropUtils,
			bool bInIsOverridable)
			: FPropertyBagInstanceDataDetails(InPropertyBagHandle, InPropUtils, /* bInFixedLayout */ true)
			, OwningObjects(InOwningObjects)
			, ParentStructProperty(InParentStructProperty)
			, bIsOverridable(bInIsOverridable)
		{ }

		virtual void OnChildRowAdded(IDetailPropertyRow& ChildRow) override
		{
			FPropertyBagInstanceDataDetails::OnChildRowAdded(ChildRow);

			if (!bIsOverridable)
			{
				ChildRow.OverrideResetToDefault(FResetToDefaultOverride::Create(
					FIsResetToDefaultVisible::CreateLambda([this](TSharedPtr<IPropertyHandle> PropertyHandle)
					{
						const FName PropertyName = PropertyHandle->GetProperty()->GetFName();
						return !IsPropertyDefaultValue(PropertyName);
					}),
					FResetToDefaultHandler::CreateLambda([this](TSharedPtr<IPropertyHandle> PropertyHandle)
					{
						const FName PropertyName = PropertyHandle->GetProperty()->GetFName();

						FScopedTransaction Transaction(FText::Format(LOCTEXT("ResetToDefault", "Reset {0} to default value"), FText::FromName(PropertyName)));
						PropertyHandle->NotifyPreChange();

						SetPropertyToDefaultValue(PropertyName);

						PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
						PropertyHandle->NotifyFinishedChangingProperties();
					})));
			}
		}
		
		virtual bool HasPropertyOverrides() const override { return bIsOverridable; }
		
		virtual void PreChangeOverrides() override
		{
			ParentStructProperty->NotifyPreChange();
		}

		virtual void PostChangeOverrides() override
		{
			ParentStructProperty->NotifyPostChange(EPropertyChangeType::ValueSet);
			ParentStructProperty->NotifyFinishedChangingProperties();
		}
		
		virtual void EnumeratePropertyBags(TSharedPtr<IPropertyHandle> PropertyBagHandle, const EnumeratePropertyBagFuncRef& Func) const override
		{
			struct FUpscalerSettingsOverrideProvider : public IPropertyBagOverrideProvider
			{
			public:
				FUpscalerSettingsOverrideProvider(FDisplayClusterConfigurationUpscalerSettings& InUpscalerSettings)
					: UpscalerSettings(InUpscalerSettings)
				{
				}
		
				virtual bool IsPropertyOverridden(const FGuid PropertyID) const override
				{
					return UpscalerSettings.ParameterOverrideGuids.Contains(PropertyID);
				}
		
				virtual void SetPropertyOverride(const FGuid PropertyID, const bool bIsOverridden) const override
				{
					if (bIsOverridden)
					{
						UpscalerSettings.ParameterOverrideGuids.AddUnique(PropertyID);
					}
					else
					{
						UpscalerSettings.ParameterOverrideGuids.Remove(PropertyID);
					}
				}

			private:
				FDisplayClusterConfigurationUpscalerSettings& UpscalerSettings;
			};
			
			ParentStructProperty->EnumerateRawData([this, Func](void* RawData, const int32 DataIndex, const int32 NumDatas)
			{
				if (!OwningObjects.IsValidIndex(DataIndex) || !OwningObjects[DataIndex].IsValid())
				{
					return true;
				}

				UObject* OwningObject = OwningObjects[DataIndex].Get();
				if (FDisplayClusterConfigurationUpscalerSettings* UpscalerSettings = static_cast<FDisplayClusterConfigurationUpscalerSettings*>(RawData))
				{
					if (ParentStructProperty->GetProperty()->GetNameCPP() == GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_RenderSettings, UpscalerSettings))
					{
						if (ADisplayClusterRootActor* OwningRootActor = Cast<ADisplayClusterRootActor>(OwningObject))
						{
							const FDisplayClusterConfigurationUpscalerSettings& GlobalUpscalerSettings = OwningRootActor->GetConfigData()->StageSettings.OuterViewportUpscalerSettings;
							const FInstancedPropertyBag& DefaultParameters = GlobalUpscalerSettings.EditingData;
							FInstancedPropertyBag& Parameters = UpscalerSettings->EditingData;
							FUpscalerSettingsOverrideProvider OverrideProvider(*UpscalerSettings);
							if (!Func(DefaultParameters, Parameters, OverrideProvider))
							{
								return false;
							}
						}
					}
				}
				return true;
			});
		}

	private:
		bool IsPropertyDefaultValue(const FName& PropertyName)
		{
			TSharedPtr<IPropertyHandle> MethodNameHandle = ParentStructProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationUpscalerSettings, MethodName));

			FName MethodName;
			MethodNameHandle->GetValue(MethodName);
			
			FInstancedPropertyBag DefaultPropertyBag;
			GetMethodSettings(MethodName, DefaultPropertyBag);
			
			const FPropertyBagPropertyDesc* DefaultPropertyDesc = DefaultPropertyBag.FindPropertyDescByName(PropertyName);
			
			bool bIsDifferent = false;
			ParentStructProperty->EnumerateRawData([this, &PropertyName, &bIsDifferent, &DefaultPropertyBag, &DefaultPropertyDesc](void* RawData, const int32 DataIndex, const int32 NumDatas)
			{
				if (FDisplayClusterConfigurationUpscalerSettings* UpscalerSettings = static_cast<FDisplayClusterConfigurationUpscalerSettings*>(RawData))
				{
					const FPropertyBagPropertyDesc* PropertyDesc = UpscalerSettings->EditingData.FindPropertyDescByName(PropertyName);
					if (!ArePropertiesIdentical(DefaultPropertyDesc, DefaultPropertyBag, PropertyDesc, UpscalerSettings->EditingData))
					{
						bIsDifferent = true;
					}
				}

				return true;
			});

			return !bIsDifferent;
		}

		void SetPropertyToDefaultValue(const FName& PropertyName)
		{
			TSharedPtr<IPropertyHandle> MethodNameHandle = ParentStructProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationUpscalerSettings, MethodName));

			FName MethodName;
			MethodNameHandle->GetValue(MethodName);
			
			FInstancedPropertyBag DefaultPropertyBag;
			GetMethodSettings(MethodName, DefaultPropertyBag);
			
			const FPropertyBagPropertyDesc* DefaultPropertyDesc = DefaultPropertyBag.FindPropertyDescByName(PropertyName);

			
			bool bIsDifferent = false;
			ParentStructProperty->EnumerateRawData([this, &PropertyName, &bIsDifferent, &DefaultPropertyBag, &DefaultPropertyDesc](void* RawData, const int32 DataIndex, const int32 NumDatas)
			{
				if (FDisplayClusterConfigurationUpscalerSettings* UpscalerSettings = static_cast<FDisplayClusterConfigurationUpscalerSettings*>(RawData))
				{
					const FPropertyBagPropertyDesc* PropertyDesc = UpscalerSettings->EditingData.FindPropertyDescByName(PropertyName);
					CopyPropertyValue(DefaultPropertyDesc, DefaultPropertyBag, PropertyDesc, UpscalerSettings->EditingData);
				}

				return true;
			});
		}
		
		bool ArePropertiesIdentical(const FPropertyBagPropertyDesc* InSourcePropertyDesc, const FInstancedPropertyBag& InSourceInstance, const FPropertyBagPropertyDesc* InTargetPropertyDesc, const FInstancedPropertyBag& InTargetInstance)
		{
			if (!InSourceInstance.IsValid() || !InTargetInstance.IsValid() || !InSourcePropertyDesc || !InSourcePropertyDesc->CachedProperty || !InTargetPropertyDesc || !InTargetPropertyDesc->CachedProperty)
			{
				return false;
			}

			if (!InSourcePropertyDesc->CompatibleType(*InTargetPropertyDesc))
			{
				return false;
			}

			const uint8* SourceValueAddress = InSourceInstance.GetValue().GetMemory() + InSourcePropertyDesc->CachedProperty->GetOffset_ForInternal();
			const uint8* TargetValueAddress = InTargetInstance.GetValue().GetMemory() + InTargetPropertyDesc->CachedProperty->GetOffset_ForInternal();

			return InSourcePropertyDesc->CachedProperty->Identical(SourceValueAddress, TargetValueAddress);
		}

		void CopyPropertyValue(const FPropertyBagPropertyDesc* InSourcePropertyDesc, const FInstancedPropertyBag& InSourceInstance, const FPropertyBagPropertyDesc* InTargetPropertyDesc, FInstancedPropertyBag& InTargetInstance)
		{
			if (!InSourceInstance.IsValid() || !InTargetInstance.IsValid() || !InSourcePropertyDesc || !InSourcePropertyDesc->CachedProperty || !InTargetPropertyDesc || !InTargetPropertyDesc->CachedProperty)
			{
				return;
			}

			// Can't copy if they are not compatible.
			if (!InSourcePropertyDesc->CompatibleType(*InTargetPropertyDesc))
			{
				return;
			}

			const uint8* SourceValueAddress = InSourceInstance.GetValue().GetMemory() + InSourcePropertyDesc->CachedProperty->GetOffset_ForInternal();
			uint8* TargetValueAddress = InTargetInstance.GetMutableValue().GetMemory() + InTargetPropertyDesc->CachedProperty->GetOffset_ForInternal();

			InSourcePropertyDesc->CachedProperty->CopyCompleteValue(TargetValueAddress, SourceValueAddress);
		}
		
	private:
		const TArray<TWeakObjectPtr<UObject>>& OwningObjects;
		TSharedPtr<IPropertyHandle> ParentStructProperty;
		bool bIsOverridable;
	};
	
};

FDisplayClusterConfigurationUpscalerSettingsDetailCustomization::FDisplayClusterConfigurationUpscalerSettingsDetailCustomization()
{ }

void FDisplayClusterConfigurationUpscalerSettingsDetailCustomization::CustomizeHeader(
	TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{ }

void FDisplayClusterConfigurationUpscalerSettingsDetailCustomization::CustomizeChildren(
	TSharedRef<IPropertyHandle> InPropertyHandle,
	IDetailChildrenBuilder& InChildBuilder,
	IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	using namespace UE::DisplayClusterConfigurator::UpscalerSettingsDetailCustomization;

	// CUSTOMIZE PROPERTY: UpscalingSettings (with UpscalingSettingsStorage)

	const bool bWithOverrides = InPropertyHandle->HasMetaData(TEXT("WithOverrides"));

	MethodNameHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationUpscalerSettings, MethodName));
	EditingDataHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationUpscalerSettings, EditingData));

	if (!MethodNameHandle.IsValid() || !EditingDataHandle.IsValid())
	{
		MethodNameHandle.Reset();
		EditingDataHandle.Reset();

		return;
	}
	
	/** Find upscaling method from string. */
	auto FindUpscalingMethod = [&](const FName& InMethodName)
		{
			for (const TSharedPtr<FUpscalerMethodEntry>& UpscalerMethod : UpscalerMethods)
			{
				if (UpscalerMethod.IsValid() && UpscalerMethod->Name == InMethodName)
				{
					return UpscalerMethod;
				}
			}

			// Method is not available.
			// When the DCRA configuration is from another project, but the current project/hardware does not have this method.
			
			// return first possible
			return !UpscalerMethods.IsEmpty() ? UpscalerMethods[0] : nullptr;
		};

	auto RebuildUpscalerMethods = [&]()
		{
			// Collect embedded values from the EDisplayClusterConfigurationUpscalingMethod
			ForeachDefaultUpscalerMethod([&](
				const UEnum& StaticEnum,
				const int32 ElementIndex,
				const EDisplayClusterConfigurationUpscalingMethod InUpscalingMethod)
				{
					TSharedPtr<FUpscalerMethodEntry> NewElement = MakeShared<FUpscalerMethodEntry>(
						FName(StaticEnum.GetNameStringByIndex(ElementIndex)),
						StaticEnum.GetDisplayNameTextByIndex(ElementIndex),
						StaticEnum.GetToolTipTextByIndex(ElementIndex));

					UpscalerMethods.Add(NewElement);
				});

			// Collect custom  upscalers
			ForeachUpscaler([&](IUpscalerModularFeature& Upscaler)
				{
					TSharedPtr<FUpscalerMethodEntry> NewElement = MakeShared<FUpscalerMethodEntry>(
						Upscaler.GetName(),
						Upscaler.GetDisplayName(),
						Upscaler.GetTooltipText());

					UpscalerMethods.Add(NewElement);
				});
		};

	FName CurrentMethodName;

	// CUSTOMIZE PROPERTY: UpscalingMethod
	{
		RebuildUpscalerMethods();

		// Update current value
		MethodNameHandle->GetValue(CurrentMethodName);
		CurrentUpscalerMethod = FindUpscalingMethod(CurrentMethodName);

		if (!bWithOverrides)
		{
			// Build dropdown list
			InChildBuilder.AddCustomRow(FText::FromString("Selected Option"))
				.NameContent()
				[
					MethodNameHandle->CreatePropertyNameWidget()
				]
				.ValueContent()
				[
					SNew(SComboBox<TSharedPtr<FUpscalerMethodEntry>>)
						.OptionsSource(&UpscalerMethods)
						.OnGenerateWidget_Lambda([](TSharedPtr<FUpscalerMethodEntry> InMethod)
							{							
								if (InMethod.IsValid())
								{
									return SNew(STextBlock)
										.Text(InMethod->DisplayName.IsEmpty()
											? FText::FromName(InMethod->Name)
											: InMethod->DisplayName)
										.ToolTipText(InMethod->Tooltip);
								}

								//@todo
								return SNew(STextBlock).Text(FText::GetEmpty());
							})
						.OnSelectionChanged_Lambda([&](TSharedPtr<FUpscalerMethodEntry> NewUpscalerMethod, ESelectInfo::Type)
							{
								// Create a transaction around the whole process of setting the method name so that changing the editor data property bag
								// also gets serialized to the transaction buffer
								FScopedTransaction Transaction(LOCTEXT("EditMethodNameTransaction", "Edit Method Name"));
							
								CurrentUpscalerMethod = NewUpscalerMethod;
								FName MethodName = CurrentUpscalerMethod.IsValid() ? CurrentUpscalerMethod->Name : NAME_None;
							
								// When MethodName changes, copy from PerMethodData[MethodName] to EditingData
								const bool bResetEditingData = true;
								UpdateEditingData(bResetEditingData);
							
								MethodNameHandle->SetValue(MethodName);
							})
						[
							SNew(STextBlock)
								.Font(IDetailLayoutBuilder::GetDetailFont())
								.Text_Lambda([&]() -> FText
									{
										return CurrentUpscalerMethod.IsValid() ? CurrentUpscalerMethod->DisplayName : FText::FromString(TEXT("Default"));
									})
								.ToolTipText_Lambda([&]() -> FText
									{
										return CurrentUpscalerMethod.IsValid() ? CurrentUpscalerMethod->Tooltip : FText::GetEmpty();
									})
						]
				]
				.OverrideResetToDefault(FResetToDefaultOverride::Create(
 					TAttribute<bool>::CreateLambda([this] 
					{
 						FName MethodName;
 						MethodNameHandle->GetValue(MethodName);

 						FName DefaultName = !UpscalerMethods.IsEmpty() ? UpscalerMethods[0]->Name : NAME_None;
 						return MethodName != NAME_None && MethodName != DefaultName;
					}),
 					FSimpleDelegate::CreateLambda([this] 
					{
 						CurrentUpscalerMethod = !UpscalerMethods.IsEmpty() ? UpscalerMethods[0] : nullptr;
 						const FName MethodName = CurrentUpscalerMethod ? CurrentUpscalerMethod->Name : NAME_None;
 						
 						const bool bResetEditingData = true;
 						UpdateEditingData(bResetEditingData);
 						
 						MethodNameHandle->SetValue(MethodName, EPropertyValueSetFlags::ResetToDefault);
					})
				));
		}
		else
		{
			MethodNameHandle->MarkHiddenByCustomization();
		}
	}
	
	const bool bResetEditingData = !CurrentUpscalerMethod.IsValid() || CurrentMethodName != CurrentUpscalerMethod->Name;
	
	// Update method name
	if (CurrentUpscalerMethod.IsValid() && CurrentMethodName != CurrentUpscalerMethod->Name)
	{
		// Update method name interactively and without a transaction to prevent script reconstruction from running immediately when the customizer is initialized
		MethodNameHandle->SetValue(CurrentUpscalerMethod->Name, EPropertyValueSetFlags::InteractiveChange | EPropertyValueSetFlags::NotTransactable);
	}
	
	// Initialize EditingData
	UpdateEditingData(bResetEditingData);
	
	uint32 NumFields = 0;
	EditingDataHandle->GetNumChildren(NumFields);
	EditingDataHandle->MarkResetToDefaultCustomized(true);

	// Show customization if we have at least 1 property in the bag
	if (NumFields > 0)
	{
		TSharedRef<FPropertyBagCustomization> EditingDataDetails =
			MakeShared<FPropertyBagCustomization>(
				InCustomizationUtils.GetPropertyUtilities()->GetSelectedObjects(),
				InPropertyHandle
				, EditingDataHandle
				, InCustomizationUtils.GetPropertyUtilities()
				, bWithOverrides);

		InChildBuilder
			.AddCustomBuilder(EditingDataDetails);
	}
}

bool FDisplayClusterConfigurationUpscalerSettingsDetailCustomization::UpdateEditingData(bool bResetEditingData)
{
	if (!EditingDataHandle.IsValid() || !CurrentUpscalerMethod.IsValid())
	{
		return false;
	}

	/** Update EditingData property. */
	using namespace UE::DisplayClusterConfigurator::UpscalerSettingsDetailCustomization;

	// NewSettingsBag->EditingData
	{
		void* StructPointer = nullptr;
		if (EditingDataHandle->GetValueData(StructPointer) == FPropertyAccess::Success && StructPointer)
		{
			bool bEditingDataChanged = false;
			bool bNewUpscalerMethodApplied = false;
			
			FInstancedPropertyBag* EditingDataBag = static_cast<FInstancedPropertyBag*>(StructPointer);
			if (bResetEditingData)
			{
				EditingDataHandle->NotifyPreChange();
				EditingDataBag->Reset();
				bEditingDataChanged = true;
			}

			if(!EditingDataBag->IsValid() || EditingDataBag->GetNumPropertiesInBag()==0)
			{
				// Get new settings
				FInstancedPropertyBag NewSettingsBag;
				GetMethodSettings(CurrentUpscalerMethod->Name, NewSettingsBag);

				const bool bTheSame = NewSettingsBag.Identical(EditingDataBag, PPF_None);
				if (!bTheSame)
				{
					// If we haven't already notified for the change, do it now
					if (!bEditingDataChanged)
					{
						EditingDataHandle->NotifyPreChange();
					}
					
					EditingDataBag->MigrateToNewBagInstance(NewSettingsBag);
					bEditingDataChanged = true;
					bNewUpscalerMethodApplied = true;
				}
			}

			if (bEditingDataChanged)
			{
				// Set the change interactively to prevent script reconstruction from being run before method name can also be set
				EditingDataHandle->NotifyPostChange(EPropertyChangeType::Interactive);
				EditingDataHandle->NotifyFinishedChangingProperties();
			}

			return bNewUpscalerMethodApplied;
		}
	}

	return false;
}

#undef LOCTEXT_NAMESPACE