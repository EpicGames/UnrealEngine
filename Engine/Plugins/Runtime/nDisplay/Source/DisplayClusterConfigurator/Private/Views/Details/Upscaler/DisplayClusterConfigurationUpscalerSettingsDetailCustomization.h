// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPropertyTypeCustomization.h"
#include "DisplayClusterConfigurationTypes_Upscaler.h"

/**
* DisplayCluster UI customization for Upscaler Settings.
*/
class FDisplayClusterConfigurationUpscalerSettingsDetailCustomization : public IPropertyTypeCustomization
{
public:
	FDisplayClusterConfigurationUpscalerSettingsDetailCustomization();

	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FDisplayClusterConfigurationUpscalerSettingsDetailCustomization);
	}

	//~Begin IDetailCustomization
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	//~~End IDetailCustomization

protected:
	// Initialize EditingDataHandle
	bool UpdateEditingData(bool bResetBag);

protected: // CUSTOMIZE PROPERTY: UpscalingMethod

	/** Droplist element with the tooltip text. */
	struct FUpscalerMethodEntry
	{
		FUpscalerMethodEntry() = default;
		FUpscalerMethodEntry(
			const FName& InName,
			const FText& InDisplayName,
			const FText& InTooltip)
			: Name(InName), DisplayName(InDisplayName), Tooltip(InTooltip)
		{ }

	public:
		// Key
		FName Name;

		// (opt) The display name shown in the UI. Use 'Name' is this value is empty.
		FText DisplayName;

		// (opt) Tooltip text
		FText Tooltip;
	};

	TArray<TSharedPtr<FUpscalerMethodEntry>> UpscalerMethods;
	TSharedPtr<FUpscalerMethodEntry> CurrentUpscalerMethod;

	// property handles
	TSharedPtr<IPropertyHandle> MethodNameHandle;
	TSharedPtr<IPropertyHandle> EditingDataHandle;
};
