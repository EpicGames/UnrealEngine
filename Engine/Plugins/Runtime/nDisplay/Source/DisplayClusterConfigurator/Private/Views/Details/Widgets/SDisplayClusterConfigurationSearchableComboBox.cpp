// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/Details/Widgets/SDisplayClusterConfigurationSearchableComboBox.h"

void SDisplayClusterConfigurationSearchableComboBox::Construct(const FArguments& InArgs)
{
	// Set up content
	TSharedPtr<SWidget> ButtonContent = InArgs._Content.Widget;
	if (InArgs._Content.Widget == SNullWidget::NullWidget)
	{
		SAssignNew(ButtonContent, STextBlock)
			.Text(NSLOCTEXT("SDisplayClusterConfigurationSearchableComboBox", "ContentWarning", "No Content Provided"))
			.ColorAndOpacity(FLinearColor::Red);
	}

	DisplayClusterOptionsSource = InArgs._OptionsSource;

	SSearchableComboBox::Construct(SSearchableComboBox::FArguments()
			.ComboBoxStyle(InArgs._ComboBoxStyle)
			.ButtonStyle(InArgs._ButtonStyle)
			.ItemStyle(InArgs._ItemStyle)
			.ContentPadding(InArgs._ContentPadding)
			.ForegroundColor(InArgs._ForegroundColor)
			.OptionsSource(InArgs._OptionsSource)
			.OnSelectionChanged(InArgs._OnSelectionChanged)
			.OnGenerateWidget(InArgs._OnGenerateWidget)
			.OnComboBoxOpening(InArgs._OnComboBoxOpening)
			.CustomScrollbar(InArgs._CustomScrollbar)
			.InitiallySelectedItem(InArgs._InitiallySelectedItem)
			.Method(InArgs._Method)
			.MaxListHeight(InArgs._MaxListHeight)
			.HasDownArrow(InArgs._HasDownArrow)
			.SearchVisibility(InArgs._SearchVisibility)
			.Content()
			[
				ButtonContent.ToSharedRef()
			]
		);
}

void SDisplayClusterConfigurationSearchableComboBox::ResetOptionsSource(const TArray<TSharedPtr<FString>>* NewOptionsSource)
{
	if (NewOptionsSource && NewOptionsSource != DisplayClusterOptionsSource)
	{
		DisplayClusterOptionsSource = NewOptionsSource;
		SetOptionsSource(NewOptionsSource);
	}
	
	RefreshOptions();
	
	{
		// Without this the selection is cleared and dropdown auto closed when clicking on the combo button next. Combo box bug?
		
		const TSharedPtr<FString> CurrentSelectedItem = GetSelectedItem();
		if (CurrentSelectedItem.IsValid() && DisplayClusterOptionsSource && DisplayClusterOptionsSource->Contains(CurrentSelectedItem))
		{
			SetSelectedItem(CurrentSelectedItem);
		}
		else
		{
			ClearSelection();
		}
	}
}
