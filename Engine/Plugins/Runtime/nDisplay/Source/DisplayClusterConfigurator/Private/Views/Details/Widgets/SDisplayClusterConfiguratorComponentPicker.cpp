// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDisplayClusterConfiguratorComponentPicker.h"

#include "SDisplayClusterConfigurationSearchableComboBox.h"

#include "DetailLayoutBuilder.h"
#include "GameFramework/Actor.h"

void SDisplayClusterConfiguratorComponentPicker::Construct(
	const FArguments& InArgs,
	const TSubclassOf<UActorComponent>& InComponentClass,
	const TWeakObjectPtr<AActor>& InOwningActor,
	const TSharedPtr<IPropertyHandle>& InPropertyHandle)
{
	ComponentClass = InComponentClass;
	PropertyHandle = InPropertyHandle;
	OwningActor = InOwningActor;

	DefaultOptionText = InArgs._DefaultOptionText;
	DefaultOptionValue = InArgs._DefaultOptionValue;

	RefreshComponentOptions();

	ChildSlot
	[
		CreateComboBoxWidget()
	];
}

TSharedRef<SWidget> SDisplayClusterConfiguratorComponentPicker::CreateComboBoxWidget()
{
	if (ComponentComboBox.IsValid())
	{
		return ComponentComboBox.ToSharedRef();
	}

	return SAssignNew(ComponentComboBox, SDisplayClusterConfigurationSearchableComboBox)
		.OptionsSource(&AvailableComponentOptions)
		.OnGenerateWidget(this, &SDisplayClusterConfiguratorComponentPicker::MakeOptionComboWidget)
		.OnSelectionChanged(this, &SDisplayClusterConfiguratorComponentPicker::OnComponentSelected)
		.ContentPadding(2)
		.MaxListHeight(200.0f)
		.Content()
		[
			SNew(STextBlock)
			.Text(this, &SDisplayClusterConfiguratorComponentPicker::GetSelectedComponentText)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		];
}

TSharedRef<SWidget> SDisplayClusterConfiguratorComponentPicker::MakeOptionComboWidget(TSharedPtr<FString> InItem)
{
	return SNew(STextBlock)
		.Text(FText::FromString(*InItem))
		.Font(IDetailLayoutBuilder::GetDetailFont());
}

FText SDisplayClusterConfiguratorComponentPicker::GetSelectedComponentText() const
{
	FString SelectedOption;
	if (PropertyHandle.IsValid())
	{
		if (FPropertyAccess::Result::MultipleValues == PropertyHandle->GetValue(SelectedOption))
		{
			SelectedOption = TEXT("Multiple Values");
		}

		if (PropertyHandle->GetPropertyClass()->IsChildOf(FNameProperty::StaticClass()))
		{
			if (FName(SelectedOption) == NAME_None)
			{
				SelectedOption.Empty();
			}
		}
	}

	if (SelectedOption.IsEmpty())
	{
		return DefaultOptionText;
	}

	return FText::FromString(SelectedOption);
}

void SDisplayClusterConfiguratorComponentPicker::OnComponentSelected(TSharedPtr<FString> InComponentName,
                                                                           ESelectInfo::Type SelectInfo)
{
	if (InComponentName.IsValid())
	{
		if (*InComponentName == DefaultOptionText.ToString()
			&& DefaultOptionValue.IsSet())
		{
			const FString& StrValue = DefaultOptionValue.GetValue();
			PropertyHandle->SetValue(*StrValue);
		}
		else
		{
			PropertyHandle->SetValue(*InComponentName.Get());
		}
		
		// Reset available options
		RefreshComponentOptions();
		ComponentComboBox->ResetOptionsSource(&AvailableComponentOptions);
		ComponentComboBox->SetIsOpen(false);
	}
}

void SDisplayClusterConfiguratorComponentPicker::RefreshComponentOptions()
{
	AvailableComponentOptions.Reset();

	FString DefaultOption = DefaultOptionText.ToString();
	if (!DefaultOption.IsEmpty())
	{
		AvailableComponentOptions.Add(MakeShared<FString>(DefaultOption));
	}
	
	if (OwningActor.IsValid())
	{
		TArray<UActorComponent*> AvailableComponents;
		OwningActor->GetComponents(ComponentClass, AvailableComponents);
		for (const UActorComponent* Component : AvailableComponents)
		{
			const FString ComponentName = Component->GetName();
			if (ComponentName != DefaultOption)
			{
				AvailableComponentOptions.Add(MakeShared<FString>(ComponentName));
			}
		}

		// Component order not guaranteed, sort for consistency.
		AvailableComponentOptions.Sort([](const TSharedPtr<FString>& A, const TSharedPtr<FString>& B)
		{
			// Default sort isn't compatible with TSharedPtr<FString>.
			return *A < *B;
		});
	}
}
