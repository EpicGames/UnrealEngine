// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Templates/SubclassOf.h"
#include "Widgets/SCompoundWidget.h"

class IPropertyHandle;
class SDisplayClusterConfigurationSearchableComboBox;
class UActorComponent;

/**
 * A searchable text combo box for components which allows reading/writing a string value to a property handle.
 * Works with CDOs.
 */
class SDisplayClusterConfiguratorComponentPicker
	: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDisplayClusterConfiguratorComponentPicker)
	{}
	SLATE_ARGUMENT(FText, DefaultOptionText)
	SLATE_ARGUMENT(TOptional<FString>, DefaultOptionValue)
	SLATE_END_ARGS()
	
	/**
	 * Construct the widget from a declaration
	 *
	 * @param InArgs   Declaration from which to construct the widget
	 * @param InComponentClass The component class and all children this picker should display
	 * @param InOwningActor The actor containing the components to display
	 * @param InPropertyHandle The property handle to read and write values to
	 */
	void Construct(const FArguments& InArgs, const TSubclassOf<UActorComponent>& InComponentClass, const TWeakObjectPtr<AActor>& InOwningActor,
		const TSharedPtr<IPropertyHandle>& InPropertyHandle);

private:
	/** Creates a combo box widget to represent components */
	TSharedRef<SWidget> CreateComboBoxWidget();

	/**
	 * Creates a text block widget to use to display the specified item in the dropdown menu
	 * @param InItem - The string item to make the text block for
	 */
	TSharedRef<SWidget> MakeOptionComboWidget(TSharedPtr<FString> InItem);

	/** Gets the text to display for the currently selected component */
	FText GetSelectedComponentText() const;
	
	/**
	 * Raised when a component is selected from the dropdown menu
	 * @param InComponentName - The component name that was selected
	 * @param SelectInfo - Flag to indicate through what interface the selection was made
	 */
	void OnComponentSelected(TSharedPtr<FString> InComponentName, ESelectInfo::Type SelectInfo);

	/** Rebuilds the list of components to show in the dropdown menu */
	void RefreshComponentOptions();
	
private:
	/** The list of components to display in the dropdown menu */
	TArray<TSharedPtr<FString>>	AvailableComponentOptions;
	
	/** The property handle to read/set the component value to */
	TSharedPtr<IPropertyHandle> PropertyHandle;
	
	/** The combo box that is being displayed in the details panel for the component property */
	TSharedPtr<SDisplayClusterConfigurationSearchableComboBox> ComponentComboBox;

	/** The actor containing the components */
	TWeakObjectPtr<AActor> OwningActor;

	/** The component type for this widget */
	TSubclassOf<UActorComponent> ComponentClass;

	/** The default option to use */
	FText DefaultOptionText;

	/** The value to apply when selecting the default option */
	TOptional<FString> DefaultOptionValue;
};
