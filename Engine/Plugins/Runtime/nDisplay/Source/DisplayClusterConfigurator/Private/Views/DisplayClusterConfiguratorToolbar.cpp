// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/DisplayClusterConfiguratorToolbar.h"

#include "DisplayClusterConfigurationStrings.h"
#include "DisplayClusterConfiguratorCommands.h"
#include "DisplayClusterConfiguratorEditorMode.h"
#include "DisplayClusterConfiguratorStyle.h"
#include "DisplayClusterConfiguratorBlueprintEditor.h"
#include "Blueprints/DisplayClusterBlueprint.h"
#include "IDisplayClusterConfigurator.h"

#include "WorkflowOrientedApp/SModeWidget.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Layout/SSpacer.h"

#define LOCTEXT_NAMESPACE "DisplayClusterConfiguratorToolbar"

class SDisplayClusterConfiguratorModeSeparator : public SBorder {
public:
	SLATE_BEGIN_ARGS(SDisplayClusterConfiguratorModeSeparator) {}
	SLATE_END_ARGS()

		void Construct(const FArguments& InArg)
	{
		SBorder::Construct(
			SBorder::FArguments()
			.BorderImage(FAppStyle::GetBrush("BlueprintEditor.PipelineSeparator"))
			.Padding(0.0f)
		);
	}

	// SWidget interface
	virtual FVector2D ComputeDesiredSize(float) const override
	{
		const float Height = 20.0f;
		const float Thickness = 16.0f;
		return FVector2D(Thickness, Height);
	}
	// End of SWidget interface
};


void FDisplayClusterConfiguratorToolbar::AddModesToolbar(TSharedPtr<FExtender> Extender)
{
	Extender->AddToolBarExtension(
		"Asset",
		EExtensionHook::After,
		Editor.Pin()->GetToolkitCommands(),
		FToolBarExtensionDelegate::CreateSP(this, &FDisplayClusterConfiguratorToolbar::FillModesToolbar));
}

void FDisplayClusterConfiguratorToolbar::FillModesToolbar(FToolBarBuilder& ToolbarBuilder)
{
	TSharedPtr<FDisplayClusterConfiguratorBlueprintEditor> EditorPtr = Editor.Pin();

	const FDisplayClusterConfiguratorCommands& Commands = IDisplayClusterConfigurator::Get().GetCommands();

	ToolbarBuilder.BeginSection("nDisplay");
	ToolbarBuilder.AddToolBarButton(Commands.Import, 
		NAME_None,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FDisplayClusterConfiguratorStyle::Get().GetStyleSetName(), "DisplayClusterConfigurator.Toolbar.Import"));

	ToolbarBuilder.AddToolBarButton(Commands.Export,
		NAME_None,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FDisplayClusterConfiguratorStyle::Get().GetStyleSetName(), "DisplayClusterConfigurator.Toolbar.SaveToFile"));
	
	ToolbarBuilder.AddComboButton(FUIAction(),
		FOnGetContent::CreateRaw(this, &FDisplayClusterConfiguratorToolbar::GenerateExportMenu),
		TAttribute<FText>(),
		TAttribute<FText>(),
		TAttribute<FSlateIcon>(),
		/*bSimpleComboBox =*/true
	);

	ToolbarBuilder.AddToolBarButton(Commands.ImportMPCDI);
	ToolbarBuilder.EndSection();

	/*
	 * TODO: Either delete or enable if we add back in the seperate graph mode.
	TAttribute<FName> GetActiveMode(EditorPtr.ToSharedRef(), &FDisplayClusterConfiguratorBlueprintEditor::GetCurrentMode);
	FOnModeChangeRequested SetActiveMode = FOnModeChangeRequested::CreateSP(
		EditorPtr.ToSharedRef(), &FDisplayClusterConfiguratorBlueprintEditor::SetCurrentMode);

	// Left side padding
	EditorPtr->AddToolbarWidget(SNew(SSpacer).Size(FVector2D(4.0f, 1.0f)));

	// Configuration Mode
	EditorPtr->AddToolbarWidget(
		SNew(SModeWidget, FDisplayClusterEditorModes::GetLocalizedMode(FDisplayClusterEditorModes::DisplayClusterEditorConfigurationMode),
			FDisplayClusterEditorModes::DisplayClusterEditorConfigurationMode)
		.OnGetActiveMode(GetActiveMode)
		.OnSetActiveMode(SetActiveMode)
		.ToolTipText(LOCTEXT("ConfigurationModeButtonTooltip", "Switch to configuration mode"))
	);

	EditorPtr->AddToolbarWidget(SNew(SDisplayClusterConfiguratorModeSeparator));

	// Blueprint Mode
	EditorPtr->AddToolbarWidget(
		SNew(SModeWidget, FDisplayClusterEditorModes::GetLocalizedMode(FDisplayClusterEditorModes::DisplayClusterEditorGraphMode),
			FDisplayClusterEditorModes::DisplayClusterEditorGraphMode)
		.OnGetActiveMode(GetActiveMode)
		.OnSetActiveMode(SetActiveMode)
		.ToolTipText(LOCTEXT("nDisplayGraphModeButtonTooltip", "Switch to graph mode"))
		.IconImage(FAppStyle::GetBrush("FullBlueprintEditor.SwitchToScriptingMode"))
	);

	// Right side padding
	EditorPtr->AddToolbarWidget(SNew(SSpacer).Size(FVector2D(4.0f, 1.0f)));
	*/

}
 
TSharedRef<SWidget> FDisplayClusterConfiguratorToolbar::GenerateExportMenu()
{
	check(Editor.IsValid());
	const TSharedRef<FUICommandList> CommandList = Editor.Pin()->GetToolkitCommands();
	
	const FDisplayClusterConfiguratorCommands& Commands = IDisplayClusterConfigurator::Get().GetCommands();
	FMenuBuilder MenuBuilder(true, CommandList);
	
	const TSharedRef<SWidget> ExportPathWidget =
		SNew(SBox)
		.Padding(FMargin(16.0f, 3.0f))
		[
			SNew(STextBlock)
			.ColorAndOpacity(FSlateColor::UseSubduedForeground())
			.Text(this, &FDisplayClusterConfiguratorToolbar::GetExportPath)
			.ToolTipText(LOCTEXT("ExportPath_Tooltip", "Export to change the path"))
			.WrapTextAt(300)
		];
	
	MenuBuilder.BeginSection("ExportPath", LOCTEXT("ExportPath_Header", "Path"));
	MenuBuilder.AddWidget(ExportPathWidget, FText::GetEmpty());
	MenuBuilder.EndSection();
	MenuBuilder.AddSeparator();
	MenuBuilder.BeginSection("ExportOptions");
	MenuBuilder.AddMenuEntry(Commands.ExportConfigOnSave);
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

FText FDisplayClusterConfiguratorToolbar::GetExportPath() const
{
	if (Editor.IsValid())
	{
		if (UDisplayClusterBlueprint* Blueprint = Cast<UDisplayClusterBlueprint>(Editor.Pin()->GetBlueprintObj()))
		{
			FString ConfigPath = Blueprint->GetConfigPath();
			if (!ConfigPath.IsEmpty())
			{
				const FString CorrectExtension = DisplayClusterConfigurationStrings::file::FileExtJson;

				if (FPaths::GetExtension(ConfigPath) != CorrectExtension)
				{
					ConfigPath = FPaths::ChangeExtension(ConfigPath, CorrectExtension);
				}
				if (FPaths::IsRelative(ConfigPath))
				{
					ConfigPath = FPaths::ConvertRelativePathToFull(ConfigPath);
				}
				return FText::FromString(ConfigPath);
			}
		}
	}

	return LOCTEXT("NoPath", "No path set - Export to set a path");
}

#undef LOCTEXT_NAMESPACE
