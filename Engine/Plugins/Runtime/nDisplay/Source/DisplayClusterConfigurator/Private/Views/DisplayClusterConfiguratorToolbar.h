// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Templates/SharedPointer.h"

class FDisplayClusterConfiguratorBlueprintEditor;
class FExtender;
class FToolBarBuilder;

class FDisplayClusterConfiguratorToolbar : public TSharedFromThis<FDisplayClusterConfiguratorToolbar> {
public:
	FDisplayClusterConfiguratorToolbar(TSharedPtr<FDisplayClusterConfiguratorBlueprintEditor> InEditor)
		: Editor(InEditor) {
	}

	void AddModesToolbar(TSharedPtr<FExtender> Extender);

protected:
	void FillModesToolbar(FToolBarBuilder& ToolbarBuilder);
	TSharedRef<SWidget> GenerateExportMenu();

	FText GetExportPath() const;

protected:
	TWeakPtr<FDisplayClusterConfiguratorBlueprintEditor> Editor;
};
