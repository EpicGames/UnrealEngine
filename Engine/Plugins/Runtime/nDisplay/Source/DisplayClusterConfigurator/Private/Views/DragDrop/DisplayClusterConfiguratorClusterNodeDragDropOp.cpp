// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfiguratorClusterNodeDragDropOp.h"

#include "DisplayClusterConfigurationTypes.h"

#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "FDisplayClusterConfiguratorClusterNodeDragDropOp"

TSharedRef<FDisplayClusterConfiguratorClusterNodeDragDropOp> FDisplayClusterConfiguratorClusterNodeDragDropOp::New(const TArray<UDisplayClusterConfigurationClusterNode*>& ClusterNodesToDrag)
{
	TSharedRef<FDisplayClusterConfiguratorClusterNodeDragDropOp> NewOperation = MakeShareable(new FDisplayClusterConfiguratorClusterNodeDragDropOp());

	for (UDisplayClusterConfigurationClusterNode* ClusterNode : ClusterNodesToDrag)
	{
		NewOperation->DraggedClusterNodes.Add(ClusterNode);
	}

	NewOperation->Construct();

	return NewOperation;
}

FText FDisplayClusterConfiguratorClusterNodeDragDropOp::GetHoverText() const
{
	if (DraggedClusterNodes.Num() > 1)
	{
		return LOCTEXT("MultipleClusterNodesLabel", "Multiple Cluster Nodes");
	}
	else if (DraggedClusterNodes.Num() == 1 && DraggedClusterNodes[0].IsValid())
	{
		UDisplayClusterConfigurationClusterNode* ClusterNode = DraggedClusterNodes[0].Get();
		FString ClusterNodeId = "Cluster Nodes";

		if (UDisplayClusterConfigurationCluster* ParentCluster = Cast<UDisplayClusterConfigurationCluster>(ClusterNode->GetOuter()))
		{
			if (const FString* KeyPtr = ParentCluster->Nodes.FindKey(ClusterNode))
			{
				ClusterNodeId = *KeyPtr;
			}
		}

		return FText::Format(LOCTEXT("SingleClusterNodeLabel", "{0}"), FText::FromString(ClusterNodeId));
	}
	else
	{
		return FText::GetEmpty();
	}
}

#undef LOCTEXT_NAMESPACE