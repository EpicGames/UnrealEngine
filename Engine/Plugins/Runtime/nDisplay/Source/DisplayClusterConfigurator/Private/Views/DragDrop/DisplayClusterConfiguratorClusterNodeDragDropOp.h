// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DisplayClusterConfiguratorValidatedDragDropOp.h"

#include "UObject/WeakObjectPtrTemplates.h"

class UDisplayClusterConfigurationClusterNode;
class SWidget;

class FDisplayClusterConfiguratorClusterNodeDragDropOp : public FDisplayClusterConfiguratorValidatedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FDisplayClusterConfiguratorClusterNodeDragDropOp, FDisplayClusterConfiguratorValidatedDragDropOp)

	static TSharedRef<FDisplayClusterConfiguratorClusterNodeDragDropOp> New(const TArray<UDisplayClusterConfigurationClusterNode*>& ClusterNodesToDrag);

	const TArray<TWeakObjectPtr<UDisplayClusterConfigurationClusterNode>>& GetDraggedClusterNodes() const { return DraggedClusterNodes; }

protected:
	virtual FText GetHoverText() const override;

private:
	TArray<TWeakObjectPtr<UDisplayClusterConfigurationClusterNode>> DraggedClusterNodes;
};