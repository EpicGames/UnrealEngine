// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfiguratorValidatedDragDropOp.h"

#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "FDisplayClusterConfiguratorValidatedDragDropOp"


void FDisplayClusterConfiguratorValidatedDragDropOp::Construct()
{
	DefaultHoverText = GetHoverText();
	ResetToDefaultToolTip();

	FDecoratedDragDropOp::Construct();
}

void FDisplayClusterConfiguratorValidatedDragDropOp::SetDropAsInvalid(FText Message)
{
	bCanBeDropped = false;
	CurrentHoverText = !Message.IsEmpty() ? FText::Format(LOCTEXT("HoverText_Formatted", "{0}: {1}"), GetHoverText(), Message) : DefaultHoverText;

	CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));
}

void FDisplayClusterConfiguratorValidatedDragDropOp::SetDropAsValid(FText Message)
{
	bCanBeDropped = true;
	CurrentHoverText = !Message.IsEmpty() ? FText::Format(LOCTEXT("HoverText_Formatted", "{0}: {1}"), GetHoverText(), Message) : DefaultHoverText;

	CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK"));
}

#undef LOCTEXT_NAMESPACE