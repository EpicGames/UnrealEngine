// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/DragAndDrop.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "UObject/GCObject.h"

class FDisplayClusterConfiguratorValidatedDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FDisplayClusterConfiguratorValidatedDragDropOp, FDecoratedDragDropOp)

	FDisplayClusterConfiguratorValidatedDragDropOp() :
		bCanBeDropped(true)
	{ }

	// Begin FDragDropOperation
protected:
	virtual void Construct() override;
	// End FDragDropOperation

public:
	void SetDropAsInvalid(FText Message = FText::GetEmpty());
	void SetDropAsValid(FText Message = FText::GetEmpty());
	bool CanBeDropped() const { return bCanBeDropped; }

protected:
	virtual FText GetHoverText() const { return FText::GetEmpty(); }

protected:
	bool bCanBeDropped;
};