// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfiguratorViewportDragDropOp.h"

#include "DisplayClusterConfigurationTypes.h"

#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "FDisplayClusterConfiguratorViewportDragDropOp"

TSharedRef<FDisplayClusterConfiguratorViewportDragDropOp> FDisplayClusterConfiguratorViewportDragDropOp::New(const TArray<UDisplayClusterConfigurationViewport*>& ViewportsToDrag)
{
	TSharedRef<FDisplayClusterConfiguratorViewportDragDropOp> NewOperation = MakeShareable(new FDisplayClusterConfiguratorViewportDragDropOp());

	for (UDisplayClusterConfigurationViewport* Viewport : ViewportsToDrag)
	{
		NewOperation->DraggedViewports.Add(Viewport);
	}

	NewOperation->Construct();

	return NewOperation;
}

FText FDisplayClusterConfiguratorViewportDragDropOp::GetHoverText() const
{
	if (DraggedViewports.Num() > 1)
	{
		return LOCTEXT("MultipleViewportsLabel", "Multiple Viewports");
	}
	else if (DraggedViewports.Num() == 1 && DraggedViewports[0].IsValid())
	{
		UDisplayClusterConfigurationViewport* Viewport = DraggedViewports[0].Get();
		FString ViewportId = "Viewport";

		if (UDisplayClusterConfigurationClusterNode* ParentNode = Cast<UDisplayClusterConfigurationClusterNode>(Viewport->GetOuter()))
		{
			if (const FString* KeyPtr = ParentNode->Viewports.FindKey(Viewport))
			{
				ViewportId = *KeyPtr;
			}
		}

		return FText::Format(LOCTEXT("SingleViewportLabel", "{0}"), FText::FromString(ViewportId));
	}
	else
	{
		return FText::GetEmpty();
	}
}

#undef LOCTEXT_NAMESPACE