// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DisplayClusterConfiguratorValidatedDragDropOp.h"

#include "UObject/WeakObjectPtrTemplates.h"

class UDisplayClusterConfigurationViewport;
class SWidget;

class FDisplayClusterConfiguratorViewportDragDropOp : public FDisplayClusterConfiguratorValidatedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FDisplayClusterConfiguratorViewportDragDropOp, FDisplayClusterConfiguratorValidatedDragDropOp)

	static TSharedRef<FDisplayClusterConfiguratorViewportDragDropOp> New(const TArray<UDisplayClusterConfigurationViewport*>& ViewportsToDrag);

	const TArray<TWeakObjectPtr<UDisplayClusterConfigurationViewport>>& GetDraggedViewports() const { return DraggedViewports; }

private:
	virtual FText GetHoverText() const override;

private:
	TArray<TWeakObjectPtr<UDisplayClusterConfigurationViewport>> DraggedViewports;
};