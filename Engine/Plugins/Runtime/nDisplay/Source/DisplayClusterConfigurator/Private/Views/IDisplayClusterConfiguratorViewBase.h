// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"


class IDisplayClusterConfiguratorViewBase
	: public SCompoundWidget
{
public:
	virtual ~IDisplayClusterConfiguratorViewBase() = default;

public:
	virtual void ConstructChildren(const TAttribute<FMargin>& InPadding, const TSharedRef<SWidget>& InContent) = 0;
};
