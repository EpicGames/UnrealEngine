// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/Log/DisplayClusterConfiguratorViewLog.h"
#include "Views/Log/SDisplayClusterConfiguratorViewLog.h"
#include "DisplayClusterConfiguratorBlueprintEditor.h"

#include "IMessageLogListing.h"
#include "MessageLogModule.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "FDisplayClusterConfiguratorViewLog"

FDisplayClusterConfiguratorViewLog::FDisplayClusterConfiguratorViewLog(const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& InToolkit)
	: ToolkitPtr(InToolkit)
{
}

TSharedRef<SWidget> FDisplayClusterConfiguratorViewLog::CreateWidget()
{
	if (!ViewLog.IsValid())
	{
		FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
		LogListingWidget = MessageLogModule.CreateLogListingWidget(CreateLogListing());

		SAssignNew(ViewLog, SDisplayClusterConfiguratorViewLog, ToolkitPtr.Pin().ToSharedRef(), LogListingWidget.ToSharedRef());
	}

	return ViewLog.ToSharedRef();
}

TSharedRef<SWidget> FDisplayClusterConfiguratorViewLog::GetWidget()
{
	return ViewLog.ToSharedRef();
}

TSharedRef<IMessageLogListing> FDisplayClusterConfiguratorViewLog::GetMessageLogListing() const
{
	check(MessageLogListing.IsValid())

	return MessageLogListing.ToSharedRef();
}

TSharedRef<IMessageLogListing> FDisplayClusterConfiguratorViewLog::CreateLogListing()
{
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	FMessageLogInitializationOptions LogOptions;
	// Show Pages so that user is never allowed to clear log messages
	LogOptions.bShowPages   = false;
	LogOptions.bShowFilters = false;
	LogOptions.bAllowClear  = true;
	LogOptions.MaxPageCount = 1;
	MessageLogListing = MessageLogModule.CreateLogListing("DisplayClusterConfiguratorLog", LogOptions);

	Log(FText(NSLOCTEXT("FDisplayClusterConfiguratorViewLog", "LoggingStarted", "Logging started")));

	return MessageLogListing.ToSharedRef();
}

void FDisplayClusterConfiguratorViewLog::Log(const FText& Message, EVerbosityLevel Verbosity /*= EVerbosityLevel::Log*/)
{
	static TArray<TSharedRef<FTokenizedMessage>> Messages;

	TSharedRef<FTokenizedMessage> Line = FTokenizedMessage::Create(
		Verbosity == EVerbosityLevel::Log ? EMessageSeverity::Info : 
			(Verbosity == EVerbosityLevel::Warning ? EMessageSeverity::Warning : EMessageSeverity::Error));

	Line->AddToken(FTextToken::Create(FText::Format(LOCTEXT("null", "{0}"), FText::FromString(FDateTime::Now().ToString(TEXT("%H:%M:%S - "))))));
	Line->AddToken(FTextToken::Create(Message));
	Messages.Add(Line);

	if (MessageLogListing.IsValid())
	{
		MessageLogListing->AddMessages(Messages);
		Messages.Reset();
	}
}

#undef LOCTEXT_NAMESPACE
