// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Views/Log/IDisplayClusterConfiguratorViewLog.h"

class SDisplayClusterConfiguratorViewLog;
class FDisplayClusterConfiguratorBlueprintEditor;
class SWidget;

/**
 * Custom display cluster logging. TODO: Currently not used anywhere in favor of compiler results logging.
 * Leaving in for now because cluster logging options may be needed in the future.
 */
class FDisplayClusterConfiguratorViewLog
	: public IDisplayClusterConfiguratorViewLog
{
public:
	FDisplayClusterConfiguratorViewLog(const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& InToolkit);

	//~ Begin IDisplayClusterConfiguratorView Interface
	virtual TSharedRef<SWidget> CreateWidget() override;
	virtual TSharedRef<SWidget> GetWidget() override;
	//~ End IDisplayClusterConfiguratorView Interface

	//~ Begin IDisplayClusterConfiguratorViewLog Interface
	virtual TSharedRef<IMessageLogListing> GetMessageLogListing() const override;
	virtual TSharedRef<IMessageLogListing> CreateLogListing() override;
	virtual void Log(const FText& Message, EVerbosityLevel Verbosity = EVerbosityLevel::Log) override;
	//~ End IDisplayClusterConfiguratorViewLog Interface

private:
	TSharedPtr<SDisplayClusterConfiguratorViewLog> ViewLog;

	TWeakPtr<FDisplayClusterConfiguratorBlueprintEditor> ToolkitPtr;

	TSharedPtr<IMessageLogListing> MessageLogListing;

	TSharedPtr<SWidget> LogListingWidget;
};
