// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/Log/SDisplayClusterConfiguratorViewLog.h"
#include "DisplayClusterConfiguratorBlueprintEditor.h"

#include "MessageLogModule.h"
#include "Modules/ModuleManager.h"
#include "Types/ISlateMetaData.h"

#define LOCTEXT_NAMESPACE "SDisplayClusterConfiguratorViewLog"

SDisplayClusterConfiguratorViewLog::~SDisplayClusterConfiguratorViewLog()
{
}

void SDisplayClusterConfiguratorViewLog::Construct(const FArguments& InArgs, const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& InToolkit, const TSharedRef<SWidget>& InListingWidget)
{
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");

	SDisplayClusterConfiguratorViewBase::Construct(
		SDisplayClusterConfiguratorViewBase::FArguments()
		.Padding(0.0f)
		.Content()
		[
			SNew(SBox)
			.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("ModuleStats")))
			[
				InListingWidget
			]
		],
		InToolkit);
}

#undef LOCTEXT_NAMESPACE
