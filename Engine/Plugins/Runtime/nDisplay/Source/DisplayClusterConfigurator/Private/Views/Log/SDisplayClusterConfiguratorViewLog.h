// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Views/SDisplayClusterConfiguratorViewBase.h"

#include "EditorUndoClient.h"

class FDisplayClusterConfiguratorBlueprintEditor;
class SWidget;

class SDisplayClusterConfiguratorViewLog
	: public SDisplayClusterConfiguratorViewBase
	, public FEditorUndoClient
{

public:
	SLATE_BEGIN_ARGS(SDisplayClusterConfiguratorViewLog)
	{}

	SLATE_END_ARGS()

public:
	SDisplayClusterConfiguratorViewLog()
	{}

	~SDisplayClusterConfiguratorViewLog();

	void Construct(const FArguments& InArgs, const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& InToolkit, const TSharedRef<SWidget>& InListingWidget);

private:
	TSharedPtr<SWidget> LogListingWidget;
};