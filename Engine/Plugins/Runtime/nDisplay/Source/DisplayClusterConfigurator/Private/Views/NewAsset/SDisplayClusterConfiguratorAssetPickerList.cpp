// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDisplayClusterConfiguratorAssetPickerList.h"

#include "Blueprints/DisplayClusterBlueprint.h"
#include "DisplayClusterRootActor.h"

#include "ContentBrowserDelegates.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"

#define LOCTEXT_NAMESPACE "SDisplayClusterConfiguratorAssetPickerList"

SDisplayClusterConfiguratorAssetPickerList::~SDisplayClusterConfiguratorAssetPickerList()
{
	OnAssetSelectedEvent.Unbind();
}

void SDisplayClusterConfiguratorAssetPickerList::Construct(const FArguments& InArgs)
{
	OnAssetSelectedEvent = InArgs._OnAssetSelected;
	
	FAssetPickerConfig AssetPickerConfig;
	{
		AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateSP(this, &SDisplayClusterConfiguratorAssetPickerList::OnAssetSelected);
		AssetPickerConfig.OnShouldFilterAsset = FOnShouldFilterAsset::CreateSP(this, &SDisplayClusterConfiguratorAssetPickerList::OnShouldFilterAsset);
		AssetPickerConfig.InitialAssetViewType = EAssetViewType::Tile;
		AssetPickerConfig.SelectionMode = ESelectionMode::Single;
		AssetPickerConfig.bFocusSearchBoxWhenOpened = true;
		AssetPickerConfig.bAllowNullSelection = false;
		AssetPickerConfig.bShowBottomToolbar = true;
		AssetPickerConfig.bAutohideSearchBar = false;
		AssetPickerConfig.bAllowDragging = false;
		AssetPickerConfig.bCanShowClasses = false;
		AssetPickerConfig.ThumbnailLabel = EThumbnailLabel::AssetName;
		
		AssetPickerConfig.Filter.ClassPaths.Add(UDisplayClusterBlueprint::StaticClass()->GetClassPathName());
		AssetPickerConfig.Filter.bRecursiveClasses = true;
	}

	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
	const TSharedRef<SWidget> AssetPickerWidget = ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig);

	ChildSlot
		[
			AssetPickerWidget
		];
}

void SDisplayClusterConfiguratorAssetPickerList::OnAssetSelected(const FAssetData& InAssetData)
{
	SelectedAssets.Add(InAssetData);
	
	if(UDisplayClusterBlueprint* Blueprint = Cast<UDisplayClusterBlueprint>(InAssetData.GetAsset()))
	{
		OnAssetSelectedEvent.ExecuteIfBound(Blueprint);
	}
}

bool SDisplayClusterConfiguratorAssetPickerList::OnShouldFilterAsset(const FAssetData& InAssetData)
{
	if (InAssetData.AssetClassPath == UDisplayClusterBlueprint::StaticClass()->GetClassPathName())
	{
		const FString ParentClassPath = InAssetData.GetTagValueRef<FString>("ParentClass");
		if (!ParentClassPath.IsEmpty())
		{
			UClass* ParentClass = FindObject<UClass>(nullptr, *ParentClassPath);
			if (ParentClass && ParentClass->IsChildOf<ADisplayClusterRootActor>())
			{
				return false;
			}
		}
	}

	return true;
}


#undef LOCTEXT_NAMESPACE