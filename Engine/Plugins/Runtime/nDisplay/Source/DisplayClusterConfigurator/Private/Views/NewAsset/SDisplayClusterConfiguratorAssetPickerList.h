// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "AssetRegistry/AssetData.h"

#include "Blueprints/DisplayClusterBlueprint.h"

DECLARE_DELEGATE_OneParam(FOnDisplayClusterAssetSelected, UDisplayClusterBlueprint*);

class SDisplayClusterConfiguratorAssetPickerList : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDisplayClusterConfiguratorAssetPickerList)
	: _OnAssetSelected()
	{}
		SLATE_EVENT(FOnDisplayClusterAssetSelected, OnAssetSelected)
	SLATE_END_ARGS()

	~SDisplayClusterConfiguratorAssetPickerList();
	void Construct(const FArguments& InArgs);

	const TArray<FAssetData>& GetSelectedAssets() const { return SelectedAssets; }

private:
	void OnAssetSelected(const FAssetData& InAssetData);
	bool OnShouldFilterAsset(const FAssetData& InAssetData);

private:
	FOnDisplayClusterAssetSelected OnAssetSelectedEvent;
	TArray<FAssetData> SelectedAssets;
};
