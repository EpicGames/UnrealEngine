// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/NewAsset/SDisplayClusterConfiguratorNewAssetDialog.h"
#include "Settings/DisplayClusterConfiguratorSettings.h"
#include "DisplayClusterConfiguratorStyle.h"

#include "AssetRegistry/AssetData.h"

#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Styling/AppStyle.h"
#include "Widgets/Workflow/SWizard.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"

#define LOCTEXT_NAMESPACE "SDisplayClusterConfiguratorNewAssetDialog"

void SDisplayClusterConfiguratorNewAssetDialog::Construct(const FArguments& InArgs, FText AssetTypeDisplayName, TArray<FDisplayClusterConfiguratorNewAssetDialogOption> InOptions)
{
	bUserConfirmedSelection = false;

	const UDisplayClusterConfiguratorEditorSettings* Settings = GetDefault<UDisplayClusterConfiguratorEditorSettings>();
	SelectedOptionIndex = Settings->NewAssetIndex;

	Options = InOptions;

	SetOnWindowClosed(FOnWindowClosed::CreateSP(this, &SDisplayClusterConfiguratorNewAssetDialog::OnWindowClosed));
	
	TSharedPtr<SVerticalBox> OptionsBox;
	TSharedPtr<SOverlay> AssetPickerOverlay;

	TSharedRef<SVerticalBox> RootBox =
		SNew(SVerticalBox)

		// Creation mode toggle buttons.
		+ SVerticalBox::Slot()
		.Padding(0, 5, 0, 5)
		.AutoHeight()
		[
			SNew(SBox)
			.Padding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
			[
					
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				.Padding(FMargin(7))
				[
					SAssignNew(OptionsBox, SVerticalBox)
				]
			]
		];


	int32 OptionIndex = 0;
	for (FDisplayClusterConfiguratorNewAssetDialogOption& Option : Options)
	{
		OptionsBox->AddSlot()
			.Padding(0, 0, 0, OptionIndex < Options.Num() - 1 ? 7 : 0)
			.AutoHeight()
			[
				SNew(SBorder)
				.BorderImage(FDisplayClusterConfiguratorStyle::Get().GetBrush("DisplayClusterConfigurator.NewAssetDialog.SubBorder"))
				.BorderBackgroundColor(this, &SDisplayClusterConfiguratorNewAssetDialog::GetOptionBorderColor, OptionIndex)
				[
						SNew(SCheckBox)
						.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
						.CheckBoxContentUsesAutoWidth(false)
						.IsChecked(this, &SDisplayClusterConfiguratorNewAssetDialog::GetOptionCheckBoxState, OptionIndex)
						.OnCheckStateChanged(this, &SDisplayClusterConfiguratorNewAssetDialog::OptionCheckBoxStateChanged, OptionIndex)
						.Content()
						[
							// this border catches the double click before the checkbox can
							SNew(SBorder)
							.BorderImage(FAppStyle::Get().GetBrush("NoBorder"))
							.OnMouseDoubleClick(this, &SDisplayClusterConfiguratorNewAssetDialog::OnOptionDoubleClicked, OptionIndex)
							[
								SNew(SVerticalBox)
								+ SVerticalBox::Slot()
								.AutoHeight()
								.Padding(5, 2)
								[		
									SNew(STextBlock)
									.TextStyle(FDisplayClusterConfiguratorStyle::Get(), "DisplayClusterConfigurator.NewAssetDialog.OptionText")
									.ColorAndOpacity(this, &SDisplayClusterConfiguratorNewAssetDialog::GetOptionTextColor, OptionIndex)
									.Text(Option.OptionText)
									.AutoWrapText(true)
								]
								+ SVerticalBox::Slot()
								.AutoHeight()
								.Padding(5, 2, 5, 7)
								[
									SNew(STextBlock)
									.ColorAndOpacity(this, &SDisplayClusterConfiguratorNewAssetDialog::GetOptionTextColor, OptionIndex)
									.Text(Option.OptionDescription)
									.AutoWrapText(true)
								]
							]
						]
					
				]
			];

		OptionIndex++;
	}
	SWindow::Construct(SWindow::FArguments()
		.Title(FText::Format(LOCTEXT("NewConfigurationDialogTitle", "Pick a starting point for your {0}"), AssetTypeDisplayName))
		.SizingRule(ESizingRule::UserSized)
		.ClientSize(FVector2D(480.f, 600.f))
		.SupportsMaximize(false)
		.SupportsMinimize(false)
		[
			SAssignNew(Wizard, SWizard)
			.OnCanceled(this, &SDisplayClusterConfiguratorNewAssetDialog::OnCancelButtonClicked)
			.OnFinished(this, &SDisplayClusterConfiguratorNewAssetDialog::OnOkButtonClicked)
			.CanFinish(this, &SDisplayClusterConfiguratorNewAssetDialog::IsOkButtonEnabled)
			.ShowPageList(false)

			+SWizard::Page()
			.CanShow(true)
			.OnEnter(this, &SDisplayClusterConfiguratorNewAssetDialog::ResetStage)
			[
				RootBox
			]

			+SWizard::Page()
			.CanShow(this, &SDisplayClusterConfiguratorNewAssetDialog::HasAssetPage)
			.OnEnter(this, &SDisplayClusterConfiguratorNewAssetDialog::GetAssetPicker)
			[
				SAssignNew(AssetSettingsPage, SBox)
			]
		]);
}

void SDisplayClusterConfiguratorNewAssetDialog::GetAssetPicker()
{
	bOnAssetStage = true;
	AssetSettingsPage->SetContent(
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.Padding(0.f, 0.f, 0.f, 2.5f)
		.FillHeight(1.f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(EVerticalAlignment::VAlign_Fill)
			.FillWidth(1.f)
			[
				Options[SelectedOptionIndex].AssetPicker
			]
		]
	);
}

void SDisplayClusterConfiguratorNewAssetDialog::ResetStage()
{
	bOnAssetStage = false;
}

bool SDisplayClusterConfiguratorNewAssetDialog::GetUserConfirmedSelection() const
{
	return bUserConfirmedSelection;
}

const TArray<FAssetData>& SDisplayClusterConfiguratorNewAssetDialog::GetSelectedAssets() const
{
	return SelectedAssets;
}

void SDisplayClusterConfiguratorNewAssetDialog::ConfirmSelection()
{
	const FDisplayClusterConfiguratorNewAssetDialogOption& SelectedOption = Options[SelectedOptionIndex];
	if (SelectedOption.OnGetSelectedAssetsFromPicker.IsBound())
	{
		SelectedOption.OnGetSelectedAssetsFromPicker.Execute(SelectedAssets);
		ensureMsgf(SelectedAssets.Num() > 0, TEXT("No assets selected when dialog was confirmed."));
	}
	SelectedOption.OnSelectionConfirmed.ExecuteIfBound();
	bUserConfirmedSelection = true;

	RequestDestroyWindow();
}

void SDisplayClusterConfiguratorNewAssetDialog::OnWindowClosed(const TSharedRef<SWindow>& Window)
{
}

FSlateColor SDisplayClusterConfiguratorNewAssetDialog::GetOptionBorderColor(int32 OptionIndex) const
{
	return SelectedOptionIndex == OptionIndex
		? FDisplayClusterConfiguratorStyle::Get().GetColor("DisplayClusterConfigurator.NewAssetDialog.ActiveOptionBorderColor")
		: FSlateColor(FLinearColor::Transparent);
}

FSlateColor SDisplayClusterConfiguratorNewAssetDialog::GetOptionTextColor(int32 OptionIndex) const
{
	return SelectedOptionIndex == OptionIndex
		? FSlateColor(FLinearColor::White)
		: FSlateColor::UseForeground();
}

ECheckBoxState SDisplayClusterConfiguratorNewAssetDialog::GetOptionCheckBoxState(int32 OptionIndex) const
{
	return SelectedOptionIndex == OptionIndex ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SDisplayClusterConfiguratorNewAssetDialog::OptionCheckBoxStateChanged(ECheckBoxState InCheckBoxState, int32 OptionIndex)
{
	if (InCheckBoxState == ECheckBoxState::Checked)
	{
		SelectedOptionIndex = OptionIndex;
		
		UDisplayClusterConfiguratorEditorSettings* Settings = GetMutableDefault<UDisplayClusterConfiguratorEditorSettings>();
		Settings->NewAssetIndex = SelectedOptionIndex;
	
	}
}

FReply SDisplayClusterConfiguratorNewAssetDialog::OnOptionDoubleClicked(const FGeometry& Geometry, const FPointerEvent& PointerEvent, int32 OptionIndex)
{
	SelectedOptionIndex = OptionIndex;
	if (Wizard->CanShowPage(Wizard->GetCurrentPageIndex() + 1))
	{
		Wizard->AdvanceToPage(Wizard->GetCurrentPageIndex() + 1);
		return FReply::Handled();
	}
	// if we can't advance, we attempt to finish instead
	else
	{
		if (IsOkButtonEnabled())
		{
			OnOkButtonClicked();
			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

FText SDisplayClusterConfiguratorNewAssetDialog::GetAssetPickersLabelText() const
{
	if (SelectedOptionIndex < 0 || SelectedOptionIndex >= Options.Num())
	{
		return FText::GetEmpty();
	}
	return Options[SelectedOptionIndex].AssetPickerHeader;
}

bool SDisplayClusterConfiguratorNewAssetDialog::IsOkButtonEnabled() const
{
	const FDisplayClusterConfiguratorNewAssetDialogOption& SelectedOption = Options[SelectedOptionIndex];
	if (SelectedOption.OnGetSelectedAssetsFromPicker.IsBound())
	{
		TArray<FAssetData> TempSelectedAssets;
		SelectedOption.OnGetSelectedAssetsFromPicker.Execute(TempSelectedAssets);
		return bOnAssetStage && TempSelectedAssets.Num() != 0;
	}
	else
	{
		return true;
	}
}

void SDisplayClusterConfiguratorNewAssetDialog::OnOkButtonClicked()
{
	ConfirmSelection();
}

void SDisplayClusterConfiguratorNewAssetDialog::OnCancelButtonClicked()
{
	bUserConfirmedSelection = false;
	SelectedAssets.Empty();

	RequestDestroyWindow();
}

bool SDisplayClusterConfiguratorNewAssetDialog::HasAssetPage() const
{
	return !IsOkButtonEnabled();
}

#undef LOCTEXT_NAMESPACE
