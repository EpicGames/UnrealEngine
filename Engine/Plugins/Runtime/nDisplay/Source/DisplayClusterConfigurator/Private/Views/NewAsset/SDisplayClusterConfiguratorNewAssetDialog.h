// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SWindow.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SWidget.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "AssetRegistry/AssetData.h"

class FDisplayClusterConfiguratorBlueprintEditor;
class SWizard;

class SDisplayClusterConfiguratorNewAssetDialog : public SWindow
{
public:
	DECLARE_DELEGATE_OneParam(FOnGetSelectedAssetsFromPicker, TArray<FAssetData>& /* OutSelectedAssets */);
	DECLARE_DELEGATE(FOnSelectionConfirmed);

public:
	class FDisplayClusterConfiguratorNewAssetDialogOption
	{
	public:
		FText OptionText;
		FText OptionDescription;
		FText AssetPickerHeader;
		TSharedRef<SWidget> AssetPicker;
		FOnGetSelectedAssetsFromPicker OnGetSelectedAssetsFromPicker;
		FOnSelectionConfirmed OnSelectionConfirmed;

		FDisplayClusterConfiguratorNewAssetDialogOption(FText InOptionText, FText InOptionDescription, FText InAssetPickerHeader, FOnGetSelectedAssetsFromPicker InOnGetSelectedAssetsFromPicker, FOnSelectionConfirmed InOnSelecitonConfirmed, TSharedRef<SWidget> InAssetPicker)
			: OptionText(InOptionText)
			, OptionDescription(InOptionDescription)
			, AssetPickerHeader(InAssetPickerHeader)
			, AssetPicker(InAssetPicker)
			, OnGetSelectedAssetsFromPicker(InOnGetSelectedAssetsFromPicker)
			, OnSelectionConfirmed(InOnSelecitonConfirmed)
		{
		}
	};

public:
	SLATE_BEGIN_ARGS(SDisplayClusterConfiguratorNewAssetDialog)
	{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, FText AssetTypeDisplayName, TArray<FDisplayClusterConfiguratorNewAssetDialogOption> InOptions);
	void GetAssetPicker();
	void ResetStage();
	bool GetUserConfirmedSelection() const;

protected:
	const TArray<FAssetData>& GetSelectedAssets() const;

	void ConfirmSelection();
	int32 GetSelectedObjectIndex() const { return SelectedOptionIndex; };

protected:
	TSharedPtr<SBox> AssetSettingsPage;

private:

	void OnWindowClosed(const TSharedRef<SWindow>& Window);

	FSlateColor GetOptionBorderColor(int32 OptionIndex) const;

	ECheckBoxState GetOptionCheckBoxState(int32 OptionIndex) const;

	void OptionCheckBoxStateChanged(ECheckBoxState InCheckBoxState, int32 OptionIndex);
	FSlateColor GetOptionTextColor(int32 OptionIndex) const;

	FReply OnOptionDoubleClicked(const FGeometry& Geometry, const FPointerEvent& PointerEvent, int32 OptionIndex);

	FText GetAssetPickersLabelText() const;
	bool IsOkButtonEnabled() const;
	void OnOkButtonClicked();
	void OnCancelButtonClicked();
	bool HasAssetPage() const;

private:
	TSharedPtr<SWizard> Wizard;
	TArray<FDisplayClusterConfiguratorNewAssetDialogOption> Options;
	int32 SelectedOptionIndex;
	bool bUserConfirmedSelection;
	TArray<FAssetData> SelectedAssets;
	bool bOnAssetStage;
};