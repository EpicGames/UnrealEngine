// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDisplayClusterConfiguratorNewBlueprintDialog.h"

#include "Blueprints/DisplayClusterBlueprint.h"

#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Styling/AppStyle.h"
#include "IContentBrowserSingleton.h"

#define LOCTEXT_NAMESPACE "SDisplayClusterConfiguratorNewBlueprintDialog"

void SDisplayClusterConfiguratorNewBlueprintDialog::Construct(const FArguments& InArgs)
{
	SDisplayClusterConfiguratorNewAssetDialog::Construct(SDisplayClusterConfiguratorNewAssetDialog::FArguments(), LOCTEXT("AssetTypeName", "nDisplay Config"),
		{
			SDisplayClusterConfiguratorNewAssetDialog::FDisplayClusterConfiguratorNewAssetDialogOption(
				LOCTEXT("CreateFromOtherSystemLabel", "Copy Existing Configuration"),
				LOCTEXT("CreateFromOtherSystemDescription", "Copies an existing nDisplay configuration."),
				LOCTEXT("ProjectSystemsLabel", "Select a Project System"),
				SDisplayClusterConfiguratorNewAssetDialog::FOnGetSelectedAssetsFromPicker::CreateSP(this, &SDisplayClusterConfiguratorNewBlueprintDialog::GetSelectedProjectSystemAssets),
				SDisplayClusterConfiguratorNewAssetDialog::FOnSelectionConfirmed(),
				SAssignNew(SystemAssetPicker, SDisplayClusterConfiguratorAssetPickerList)),
			SDisplayClusterConfiguratorNewAssetDialog::FDisplayClusterConfiguratorNewAssetDialogOption(
				LOCTEXT("CreateEmptyLabel", "Create New Config"),
				LOCTEXT("CreateEmptyDescription", "Create a new nDisplay configuration."),
				LOCTEXT("EmptyLabel", "New Display Cluster"),
				SDisplayClusterConfiguratorNewAssetDialog::FOnGetSelectedAssetsFromPicker(),
				SDisplayClusterConfiguratorNewAssetDialog::FOnSelectionConfirmed(),
				SNew(SBox)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("NoOptionsLabel", "No Options"))
				])
		});
}

TOptional<FAssetData> SDisplayClusterConfiguratorNewBlueprintDialog::GetSelectedSystemAsset() const
{
	const TArray<FAssetData>& AllSelectedAssets = GetSelectedAssets();
	TArray<FAssetData> SelectedSystemAssets;
	for (const FAssetData& SelectedAsset : AllSelectedAssets)
	{
		if (SelectedAsset.AssetClassPath == UDisplayClusterBlueprint::StaticClass()->GetClassPathName())
		{
			SelectedSystemAssets.Add(SelectedAsset);
		}
	}
	if (SelectedSystemAssets.Num() == 1)
	{
		return TOptional<FAssetData>(SelectedSystemAssets[0]);
	}
	return TOptional<FAssetData>();
}

void SDisplayClusterConfiguratorNewBlueprintDialog::GetSelectedProjectSystemAssets(TArray<FAssetData>& OutSelectedAssets)
{
	OutSelectedAssets.Append(SystemAssetPicker->GetSelectedAssets());
}

#undef LOCTEXT_NAMESPACE