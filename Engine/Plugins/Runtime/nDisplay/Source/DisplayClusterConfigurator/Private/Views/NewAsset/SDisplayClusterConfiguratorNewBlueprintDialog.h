// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SDisplayClusterConfiguratorNewAssetDialog.h"
#include "SDisplayClusterConfiguratorAssetPickerList.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "AssetRegistry/AssetData.h"
#include "IAssetTypeActions.h"


/** A modal dialog to collect information needed to create a new Display Cluster. */
class SDisplayClusterConfiguratorNewBlueprintDialog : public SDisplayClusterConfiguratorNewAssetDialog
{
public:
	SLATE_BEGIN_ARGS(SDisplayClusterConfiguratorNewBlueprintDialog)
	{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	TOptional<FAssetData> GetSelectedSystemAsset() const;

private:
	void GetSelectedProjectSystemAssets(TArray<FAssetData>& OutSelectedAssets);

private:
	TSharedPtr<SDisplayClusterConfiguratorAssetPickerList> SystemAssetPicker;
};