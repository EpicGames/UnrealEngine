// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfiguratorHostNodeArrangementHelper.h"

#include "DisplayClusterConfigurationTypes.h"
#include "Views/OutputMapping/EdNodes/DisplayClusterConfiguratorHostNode.h"
#include "Views/OutputMapping/EdNodes/DisplayClusterConfiguratorWindowNode.h"


class FNodePlacer
{
public:
	FNodePlacer() :
		NextAvailablePosition(FVector2D::ZeroVector)
	{ }
	virtual ~FNodePlacer() { }

	virtual void Initialize(const TArray<UDisplayClusterConfiguratorHostNode*>& ManuallyPlacedNodes, const TArray<UDisplayClusterConfiguratorHostNode*>& AutomaticallyPlacedNodes) { };
	virtual FVector2D GetNextPosition(UDisplayClusterConfiguratorHostNode* Node) { return NextAvailablePosition; }
	virtual void AdvancePosition(UDisplayClusterConfiguratorHostNode* Node) { };
	virtual bool CheckForIntersections() const { return true; }

protected:
	FVector2D NextAvailablePosition;
};

class FNodeHorizontalPlacer : public FNodePlacer
{
public:
	virtual void AdvancePosition(UDisplayClusterConfiguratorHostNode* Node) override
	{
		NextAvailablePosition.X = Node->NodePosX + Node->NodeWidth + UDisplayClusterConfiguratorHostNode::HorizontalSpanBetweenHosts;
	}
};

class FNodeVerticalPlacer : public FNodePlacer
{
public:
	virtual void AdvancePosition(UDisplayClusterConfiguratorHostNode* Node) override
	{
		NextAvailablePosition.Y = Node->NodePosY + Node->NodeHeight + UDisplayClusterConfiguratorHostNode::VerticalSpanBetweenHosts;
	}
};

class FNodeWrapPlacer : public FNodePlacer
{
public:
	FNodeWrapPlacer(float InWrapThreshold) :
		WrapThreshold(InWrapThreshold),
		MaxRowHeight(0)
	{ }

	virtual FVector2D GetNextPosition(UDisplayClusterConfiguratorHostNode* Node) override
	{
		// We can't wrap if the next available position is already at 0, since that indicates the node is the only node in the row.
		if (NextAvailablePosition.X > 0 && NextAvailablePosition.X + Node->NodeWidth > WrapThreshold)
		{
			NextAvailablePosition.X = 0;
			NextAvailablePosition.Y += MaxRowHeight + UDisplayClusterConfiguratorHostNode::VerticalSpanBetweenHosts;
			MaxRowHeight = Node->NodeHeight;
		}

		return NextAvailablePosition;
	}

	virtual void AdvancePosition(UDisplayClusterConfiguratorHostNode* Node) override
	{
		NextAvailablePosition.X = Node->NodePosX + Node->NodeWidth + UDisplayClusterConfiguratorHostNode::HorizontalSpanBetweenHosts;

		if (!Node->CanUserMoveNode())
		{
			if (Node->NodeHeight > MaxRowHeight)
			{
				MaxRowHeight = Node->NodeHeight;
			}
		}
	}

private:
	float WrapThreshold;
	float MaxRowHeight;
};

class FNodeGridPlacer : public FNodePlacer
{
public:
	FNodeGridPlacer(int InGridSize) :
		NextGridIndex(0),
		GridSize(InGridSize)
	{ }

	virtual void Initialize(const TArray<UDisplayClusterConfiguratorHostNode*>& ManuallyPlacedNodes, const TArray<UDisplayClusterConfiguratorHostNode*>& AutomaticallyPlacedNodes) override
	{
		if (!AutomaticallyPlacedNodes.Num())
		{
			return;
		}

		const int32 NumRows = AutomaticallyPlacedNodes.Num() / GridSize + 1;
		ColumnCells.AddZeroed(GridSize);
		RowCells.AddZeroed(NumRows);

		const FMargin& VisualMargin = UDisplayClusterConfiguratorHostNode::VisualMargin;
		const FVector2D SpanBetweenHosts = FVector2D(UDisplayClusterConfiguratorHostNode::HorizontalSpanBetweenHosts, UDisplayClusterConfiguratorHostNode::VerticalSpanBetweenHosts);

		for (int GridIndex = 0; GridIndex < AutomaticallyPlacedNodes.Num(); ++GridIndex)
		{
			UDisplayClusterConfiguratorHostNode* Node = AutomaticallyPlacedNodes[GridIndex];
			int Column = GridIndex % GridSize;
			int Row = GridIndex / GridSize;

			if (Node->IsUserInteractingWithNode(true))
			{
				ColumnCells[Column].Position = Node->NodePosX;
				RowCells[Row].Position = Node->NodePosY;
			}

			// If the current column's position is overlapping with the previous column, we need to update it.
			if (Column > 0 && (ColumnCells[Column - 1].Position + ColumnCells[Column - 1].Size + SpanBetweenHosts.X) > ColumnCells[Column].Position)
			{
				ColumnCells[Column].Position = ColumnCells[Column - 1].Position + ColumnCells[Column - 1].Size + SpanBetweenHosts.X;
			}

			// If the current rows's position is overlapping with the previous row, we need to update it.
			if (Row > 0 && (RowCells[Row - 1].Position + RowCells[Row - 1].Size + SpanBetweenHosts.Y) > RowCells[Row].Position)
			{
				RowCells[Row].Position = RowCells[Row - 1].Position + RowCells[Row - 1].Size + SpanBetweenHosts.Y;
			}

			if (Node->NodeWidth > ColumnCells[Column].Size)
			{
				ColumnCells[Column].Size = Node->NodeWidth;
			}

			if (Node->NodeHeight > RowCells[Row].Size)
			{
				RowCells[Row].Size = Node->NodeHeight;
			}
		}

		// In the case that the automatically placed nodes don't fully fill out the last grid row, we need to iterate over the remaining columns and make sure their
		// position is appropriately placed, in case any of the last row of nodes expanded the previous column sizes.
		for (int GridIndex = AutomaticallyPlacedNodes.Num(); GridIndex < ColumnCells.Num() * RowCells.Num(); ++GridIndex)
		{
			int Column = GridIndex % GridSize;

			// If the current column's position is overlapping with the previous column, we need to update it.
			if (Column > 0 && ColumnCells[Column - 1].Position + ColumnCells[Column - 1].Size > ColumnCells[Column].Position)
			{
				ColumnCells[Column].Position = ColumnCells[Column - 1].Position + ColumnCells[Column - 1].Size + UDisplayClusterConfiguratorHostNode::HorizontalSpanBetweenHosts;
			}
		}

		// After sizing and placing all grid cells, run an intersection test against all manually placed nodes to adjust the columns to avoid intersection
		for (int GridIndex = 0; GridIndex < AutomaticallyPlacedNodes.Num(); ++GridIndex)
		{
			UDisplayClusterConfiguratorHostNode* Node = AutomaticallyPlacedNodes[GridIndex];
			int Column = GridIndex % GridSize;
			int Row = GridIndex / GridSize;

			const FVector2D CellMin = FVector2D(ColumnCells[Column].Position, RowCells[Row].Position) - VisualMargin.GetTopLeft();
			const FVector2D CellMax = CellMin + FVector2D(ColumnCells[Column].Size, RowCells[Row].Size) + VisualMargin.GetDesiredSize();
			const FBox2D CellBounds = FBox2D(CellMin, CellMax);

			if (!Node->IsUserInteractingWithNode(true))
			{
				for (UDisplayClusterConfiguratorHostNode* ManualNode : ManuallyPlacedNodes)
				{
					// If the user is not interacting with the manually placed node (e.g. moving it around), reduce its bounds by 1 so that automatically placed nodes
					// can touch the manually placed nodes without getting immediately moved.
					const FBox2D NodeBounds = ManualNode->IsUserInteractingWithNode() ? ManualNode->GetNodeBounds() : ManualNode->GetNodeBounds().ExpandBy(-1);
					if (CellBounds.Intersect(NodeBounds))
					{
						ColumnCells[Column].Position = ManualNode->NodePosX + ManualNode->NodeWidth + SpanBetweenHosts.X;

						// Recompute all column positions after this one to account for the shift in position
						for (int NextColumn = Column + 1; NextColumn < ColumnCells.Num(); ++NextColumn)
						{
							ColumnCells[NextColumn].Position = ColumnCells[NextColumn - 1].Position + ColumnCells[NextColumn - 1].Size + SpanBetweenHosts.Y;
						}
					}
				}
			}
		}
	}

	virtual FVector2D GetNextPosition(UDisplayClusterConfiguratorHostNode* Node) override
	{
		int Column = NextGridIndex % GridSize;
		int Row = NextGridIndex / GridSize;

		return FVector2D(ColumnCells[Column].Position, RowCells[Row].Position);
	}

	virtual void AdvancePosition(UDisplayClusterConfiguratorHostNode* Node) override
	{
		if (!Node->CanUserMoveNode())
		{
			++NextGridIndex;
		}
	}

	virtual bool CheckForIntersections() const { return false; }

private:
	struct FGridAxisCell
	{
		float Position;
		float Size;
	};

	int NextGridIndex;
	int32 GridSize;
	TArray<FGridAxisCell> RowCells;
	TArray<FGridAxisCell> ColumnCells;
};

void FDisplayClusterConfiguratorHostNodeArrangementHelper::PlaceNodes(const TArray<UDisplayClusterConfiguratorHostNode*>& HostNodes)
{
	TUniquePtr<FNodePlacer> Placer;
	switch (ArrangementSettings.ArrangementType)
	{
	case EHostArrangementType::Horizontal:
		Placer = MakeUnique<FNodeHorizontalPlacer>();
		break;

	case EHostArrangementType::Vertical:
		Placer = MakeUnique<FNodeVerticalPlacer>();
		break;

	case EHostArrangementType::Wrap:
		Placer = MakeUnique<FNodeWrapPlacer>(ArrangementSettings.WrapThreshold);
		break;

	case EHostArrangementType::Grid:
		Placer = MakeUnique<FNodeGridPlacer>(ArrangementSettings.GridSize);
		break;
	}

	for (UDisplayClusterConfiguratorHostNode* HostNode : HostNodes)
	{
		// Size the node now, as we need the bounds later for placing the automatically positioned nodes
		if (!HostNode->CanUserResizeNode())
		{
			FBox2D ChildBounds;
			ChildBounds.Init();

			for (UDisplayClusterConfiguratorBaseNode* ChildNode : HostNode->GetChildren())
			{
				if (UDisplayClusterConfiguratorWindowNode* WindowNode = Cast<UDisplayClusterConfiguratorWindowNode>(ChildNode))
				{
					// Instead of using the child node's global position to compute the host size, use the raw window configuration data.
					// This will give better results for specific edge cases such as undo operations for moving window nodes.
					const FDisplayClusterConfigurationRectangle& WindowRect = WindowNode->GetCfgWindowRect();
					if (WindowRect.W > 0 && WindowRect.H > 0)
					{
						ChildBounds += FBox2D(FVector2D(WindowRect.X, WindowRect.Y), FVector2D(WindowRect.X + WindowRect.W, WindowRect.Y + WindowRect.H));
					}
				}
				else
				{
					if (!ChildNode->GetNodeSize().IsZero())
					{
						ChildBounds += ChildNode->GetNodeBounds();
					}
				}
			}

			FVector2D Size = HostNode->TransformSizeToGlobal(FVector2D::Max(ChildBounds.Max + HostNode->GetHostOrigin(), FVector2D::ZeroVector));

			HostNode->NodeWidth = Size.X;
			HostNode->NodeHeight = Size.Y;
		}

		if (HostNode->CanUserMoveNode())
		{
			ManuallyPlacedNodes.Add(HostNode);
		}
		else
		{
			AutomaticallyPlacedNodes.Add(HostNode);
		}
	}

	Placer->Initialize(ManuallyPlacedNodes, AutomaticallyPlacedNodes);

	for (UDisplayClusterConfiguratorHostNode* Node : AutomaticallyPlacedNodes)
	{
		// If the user is interacting with the node (e.g. changing its size), use its current position as the starting position to prevent it
		// from being moved out from underneath the user.
		FVector2D NodePosition = Node->IsUserInteractingWithNode(true) ? Node->GetNodePosition() : Placer->GetNextPosition(Node);

		// We can only check for intersection if the node isn't actively being manupulated (size being changed, children being moved around), as otherwise
		// the node could inadvertently be moved around while the user is trying to manipulate it.
		if (Placer->CheckForIntersections() && !Node->IsUserInteractingWithNode(true))
		{
			UDisplayClusterConfiguratorHostNode* IntersectingNode = CheckForOverlap(Node, NodePosition);
			while (IntersectingNode)
			{
				Placer->AdvancePosition(IntersectingNode);
				NodePosition = Placer->GetNextPosition(Node);
				IntersectingNode = CheckForOverlap(Node, NodePosition);
			}
		}

		Node->NodePosX = NodePosition.X;
		Node->NodePosY = NodePosition.Y;

		Node->UpdateChildNodes();

		Placer->AdvancePosition(Node);
	}
}

UDisplayClusterConfiguratorHostNode* FDisplayClusterConfiguratorHostNodeArrangementHelper::CheckForOverlap(UDisplayClusterConfiguratorHostNode* HostNode, FVector2D DesiredPosition)
{
	FBox2D HostBounds = HostNode->GetNodeBounds().ShiftBy(DesiredPosition - HostNode->GetNodePosition());

	for (UDisplayClusterConfiguratorHostNode* ManuallyPlacedNode : ManuallyPlacedNodes)
	{
		// If the user is not interacting with the manually placed node (e.g. moving it around), reduce its bounds by 1 so that automatically placed nodes
		// can touch the manually placed nodes without getting immediately moved.
		FBox2D ManuallyPlacedBounds = ManuallyPlacedNode->IsUserInteractingWithNode() ? ManuallyPlacedNode->GetNodeBounds() : ManuallyPlacedNode->GetNodeBounds().ExpandBy(-1);

		if (ManuallyPlacedBounds.Intersect(HostBounds))
		{
			return ManuallyPlacedNode;
		}
	}

	return nullptr;
}