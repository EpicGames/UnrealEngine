// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Views/OutputMapping/IDisplayClusterConfiguratorViewOutputMapping.h"

class UDisplayClusterConfiguratorHostNode;

class FDisplayClusterConfiguratorHostNodeArrangementHelper
{
public:
	FDisplayClusterConfiguratorHostNodeArrangementHelper(const FHostNodeArrangementSettings& InArrangementSettings) :
		ArrangementSettings(InArrangementSettings)
	{ }

	void PlaceNodes(const TArray<UDisplayClusterConfiguratorHostNode*>& HostNodes);

private:
	UDisplayClusterConfiguratorHostNode* CheckForOverlap(UDisplayClusterConfiguratorHostNode* HostNode, FVector2D DesiredPosition);

private:
	FHostNodeArrangementSettings ArrangementSettings;

	TArray<UDisplayClusterConfiguratorHostNode*> ManuallyPlacedNodes;
	TArray<UDisplayClusterConfiguratorHostNode*> AutomaticallyPlacedNodes;
};