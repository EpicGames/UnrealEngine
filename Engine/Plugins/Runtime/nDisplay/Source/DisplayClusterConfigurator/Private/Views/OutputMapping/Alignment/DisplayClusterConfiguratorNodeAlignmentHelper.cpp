// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfiguratorNodeAlignmentHelper.h"

#include "Views/OutputMapping/EdNodes/DisplayClusterConfiguratorBaseNode.h"

namespace
{
	const int32 X_AXIS = 0;
	const int32 Y_AXIS = 1;
}

const FVector2D& FNodeAlignmentAnchors::GetAnchorPosition(EAlignmentAnchor Anchor) const
{
	switch (Anchor)
	{
	case EAlignmentAnchor::Center: return Center;
	case EAlignmentAnchor::Top: return Top;
	case EAlignmentAnchor::Bottom: return Bottom;
	case EAlignmentAnchor::Left: return Left;
	case EAlignmentAnchor::Right: return Right;
	default: return Center;
	}
}

const FVector2D& FNodeAlignmentAnchors::GetAdjacentAnchorPosition(EAlignmentAnchor Anchor) const
{
	switch (Anchor)
	{
	case EAlignmentAnchor::Center: return Center;
	case EAlignmentAnchor::Top: return Bottom;
	case EAlignmentAnchor::Bottom: return Top;
	case EAlignmentAnchor::Left: return Right;
	case EAlignmentAnchor::Right: return Left;
	default: return Center;
	}
}

FNodeAlignmentAnchors FNodeAlignmentAnchors::ShiftBy(const FVector2D& Offset)
{
	FNodeAlignmentAnchors NewAnchors;

	NewAnchors.Center = Center + Offset;
	NewAnchors.Top = Top + Offset;
	NewAnchors.Bottom = Bottom + Offset;
	NewAnchors.Left = Left + Offset;
	NewAnchors.Right = Right + Offset;

	return NewAnchors;
}

FNodeAlignmentAnchors FNodeAlignmentAnchors::ExpandBy(const FVector2D& SizeChange)
{
	FNodeAlignmentAnchors NewAnchors;

	NewAnchors.Center = Center + 0.5f * SizeChange;
	NewAnchors.Top = Top;
	NewAnchors.Bottom = Bottom + FVector2D(0, SizeChange.Y);
	NewAnchors.Left = Left;
	NewAnchors.Right = Right + FVector2D(SizeChange.X, 0);

	return NewAnchors;
}

FVector2D FNodeAlignmentPair::GetOffset() const
{
	return FVector2D(XAlignment.AlignmentOffset, YAlignment.AlignmentOffset);
}

bool FNodeAlignmentPair::HasAlignments() const
{
	return XAlignment.IsValid() || YAlignment.IsValid();
}

FDisplayClusterConfiguratorNodeAlignmentHelper::FDisplayClusterConfiguratorNodeAlignmentHelper(const UDisplayClusterConfiguratorBaseNode* InNodeToAlign, const FNodeAlignmentAnchors& InNodeAnchors, const FNodeAlignmentParams& InAlignmentParams) :
	NodeToAlign(InNodeToAlign),
	NodeAnchors(InNodeAnchors),
	AlignmentParams(InAlignmentParams)
{
	check(NodeToAlign.IsValid());
}

void FDisplayClusterConfiguratorNodeAlignmentHelper::AddAlignmentsToNode(const UDisplayClusterConfiguratorBaseNode* TargetNode)
{
	// The order in which we check anchors here matters in cases where two anchor alignments are equivalent (e.g. the node can align left, center, and right at the same time).
	// The list of alignments is sorted in a way that the last-checked anchor point is put on top of any previously added equivalent anchors. This matters when displaying
	// which anchor is the "active" anchor using the alignment rulers in the GUI, and it makes most sense to prioritize center, then left/top, then right/bottom, hence
	// the order in which we check the anchor points.
	if (AlignmentParams.bCanSnapAdjacentEdges)
	{
		TryAlignAjacentEdge(TargetNode, EAlignmentAnchor::Bottom, false);
		TryAlignAjacentEdge(TargetNode, EAlignmentAnchor::Top, false);
		TryAlignAjacentEdge(TargetNode, EAlignmentAnchor::Right, false);
		TryAlignAjacentEdge(TargetNode, EAlignmentAnchor::Left, false);
	}

	if (AlignmentParams.bCanSnapSameEdges)
	{
		TryAlignSameEdge(TargetNode, EAlignmentAnchor::Bottom, false);
		TryAlignSameEdge(TargetNode, EAlignmentAnchor::Top, false);
		TryAlignSameEdge(TargetNode, EAlignmentAnchor::Right, false);
		TryAlignSameEdge(TargetNode, EAlignmentAnchor::Left, false);
		TryAlignSameEdge(TargetNode, EAlignmentAnchor::Center, false);
	}
}

void FDisplayClusterConfiguratorNodeAlignmentHelper::AddAlignmentsToParent(const UDisplayClusterConfiguratorBaseNode* TargetParent)
{
	if (AlignmentParams.bCanSnapSameEdges)
	{
		TryAlignSameEdge(TargetParent, EAlignmentAnchor::Bottom, true);
		TryAlignSameEdge(TargetParent, EAlignmentAnchor::Top, true);
		TryAlignSameEdge(TargetParent, EAlignmentAnchor::Right, true);
		TryAlignSameEdge(TargetParent, EAlignmentAnchor::Left, true);
		TryAlignSameEdge(TargetParent, EAlignmentAnchor::Center, true);
	}
}

FNodeAlignmentPair FDisplayClusterConfiguratorNodeAlignmentHelper::GetAlignments() const
{
	FNodeAlignment XAlignment = XAxisAlignments.Num() > 0 ? XAxisAlignments[0] : FNodeAlignment();
	FNodeAlignment YAlignment = YAxisAlignments.Num() > 0 ? YAxisAlignments[0] : FNodeAlignment();

	return FNodeAlignmentPair(XAlignment, YAlignment);
}

bool FDisplayClusterConfiguratorNodeAlignmentHelper::HasAlignments() const
{
	return XAxisAlignments.Num() > 0 || YAxisAlignments.Num() > 0;
}

void FDisplayClusterConfiguratorNodeAlignmentHelper::AddAlignment(FNodeAlignment NewAlignment, int32 AlignmentAxis)
{
	TArray<FNodeAlignment>& AlignmentList = AlignmentAxis == X_AXIS ? XAxisAlignments : YAxisAlignments;

	// To make sure the list of alignments is always sorted, find the last alignment that is less than the new offset, and add 
	// a new alignment after the found alignment.
	int32 Index = AlignmentList.FindLastByPredicate([=](const FNodeAlignment& Alignment)
	{
		return FMath::Abs(Alignment.AlignmentOffset) < FMath::Abs(NewAlignment.AlignmentOffset);
	});

	AlignmentList.Insert(NewAlignment, Index + 1);
}

void FDisplayClusterConfiguratorNodeAlignmentHelper::TryAlignAjacentEdge(const UDisplayClusterConfiguratorBaseNode* TargetNode, EAlignmentAnchor Anchor, bool bIsParent)
{
	if (Anchor == EAlignmentAnchor::Center)
	{
		return;
	}

	if (AlignmentParams.AnchorsToIgnore.Contains(Anchor))
	{
		return;
	}

	const FNodeAlignmentAnchors TargetAnchors = TargetNode->GetNodeAlignmentAnchors(bIsParent);

	const FVector2D NodeAnchorPosition = NodeAnchors.GetAnchorPosition(Anchor);
	const FVector2D TargetAnchorPosition = TargetAnchors.GetAdjacentAnchorPosition(Anchor);

	if (FVector2D::Distance(NodeAnchorPosition, TargetAnchorPosition) > AlignmentParams.MaxSnapRadius)
	{
		return;
	}

	// The axis that is being snapped, Y axis for top and bottom, X for left and right. Note that 1 - Axis gives us the other axis.
	int32 Axis = Anchor == EAlignmentAnchor::Top || Anchor == EAlignmentAnchor::Bottom ? Y_AXIS : X_AXIS;

	// Make sure that the edges being snapped are overlapping each other. 
	EAlignmentAnchor MinBoundAnchor = (Anchor == EAlignmentAnchor::Top || Anchor == EAlignmentAnchor::Bottom) ? EAlignmentAnchor::Left : EAlignmentAnchor::Top;
	EAlignmentAnchor MaxBoundAnchor = (Anchor == EAlignmentAnchor::Top || Anchor == EAlignmentAnchor::Bottom) ? EAlignmentAnchor::Right : EAlignmentAnchor::Bottom;

	float MinBoundValue = TargetAnchors.GetAnchorPosition(MinBoundAnchor)[1 - Axis];
	float MaxBoundValue = TargetAnchors.GetAnchorPosition(MaxBoundAnchor)[1 - Axis];
	float CurrentMinValue = NodeAnchors.GetAnchorPosition(MinBoundAnchor)[1 - Axis];
	float CurrentMaxValue = NodeAnchors.GetAnchorPosition(MaxBoundAnchor)[1 - Axis];

	if (CurrentMaxValue < MinBoundValue || CurrentMinValue > MaxBoundValue)
	{
		return;
	}

	float PaddingDirection = Anchor == EAlignmentAnchor::Top || Anchor == EAlignmentAnchor::Left ? 1 : -1;

	float CurrentValue = NodeAnchorPosition[Axis];
	float TargetValue = TargetAnchorPosition[Axis] + AlignmentParams.SnapAdjacentEdgesPadding * PaddingDirection;

	if (FMath::Abs(TargetValue - CurrentValue) <= AlignmentParams.SnapProximity)
	{
		float AlignOffset = TargetValue - CurrentValue;
		AddAlignment(FNodeAlignment(TargetNode, Anchor, true, AlignOffset), Axis);
	}
}

void FDisplayClusterConfiguratorNodeAlignmentHelper::TryAlignSameEdge(const UDisplayClusterConfiguratorBaseNode* TargetNode, EAlignmentAnchor Anchor, bool bIsParent)
{
	if (AlignmentParams.AnchorsToIgnore.Contains(Anchor))
	{
		return;
	}

	const FNodeAlignmentAnchors TargetAnchors = TargetNode->GetNodeAlignmentAnchors(bIsParent);

	const FVector2D NodeAnchorPosition = NodeAnchors.GetAnchorPosition(Anchor);
	const FVector2D TargetAnchorPosition = TargetAnchors.GetAnchorPosition(Anchor);

	if (FVector2D::Distance(NodeAnchorPosition, TargetAnchorPosition) > AlignmentParams.MaxSnapRadius)
	{
		return;
	}

	if (Anchor == EAlignmentAnchor::Center)
	{
		// The center node can be aligned along either axis, or both (e.g. we attempt to align the node with the center of its parent).
		// Perform checks on both axes for alignment.
		if (FMath::Abs(TargetAnchorPosition.X - NodeAnchorPosition.X) <= AlignmentParams.SnapProximity)
		{
			float AlignXOffset = TargetAnchorPosition.X - NodeAnchorPosition.X;
			AddAlignment(FNodeAlignment(TargetNode, Anchor, false, AlignXOffset), X_AXIS);
		}

		if (FMath::Abs(TargetAnchorPosition.Y - NodeAnchorPosition.Y) <= AlignmentParams.SnapProximity)
		{
			float AlignYOffset = TargetAnchorPosition.Y - NodeAnchorPosition.Y;
			AddAlignment(FNodeAlignment(TargetNode, Anchor, false, AlignYOffset), Y_AXIS);
		}
	}
	else
	{
		// The axis that is being snapped, Y axis for top and bottom, X for left and right. Note that 1 - Axis gives us the other axis.
		int32 Axis = X_AXIS;
		Axis = Anchor == EAlignmentAnchor::Top || Anchor == EAlignmentAnchor::Bottom ? Y_AXIS : X_AXIS;

		float CurrentValue = NodeAnchorPosition[Axis];
		float TargetValue = TargetAnchorPosition[Axis];

		if (FMath::Abs(TargetValue - CurrentValue) <= AlignmentParams.SnapProximity)
		{
			float AlignOffset = TargetValue - CurrentValue;
			AddAlignment(FNodeAlignment(TargetNode, Anchor, false, AlignOffset), Axis);
		}
	}
}