// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Views/OutputMapping/EdNodes/DisplayClusterConfiguratorBaseNode.h"

class UDisplayClusterConfiguratorBaseNode;

enum class EAlignmentAnchor : uint8
{
	Center,
	Top,
	Bottom,
	Left,
	Right
};

/** Stores positional anchors that can be used to snap align multiple nodes with each other. */
struct FNodeAlignmentAnchors
{
public:
	FVector2D Center;
	FVector2D Top;
	FVector2D Bottom;
	FVector2D Left;
	FVector2D Right;

public:

	const FVector2D& GetAnchorPosition(EAlignmentAnchor Anchor) const;
	const FVector2D& GetAdjacentAnchorPosition(EAlignmentAnchor Anchor) const;

	FNodeAlignmentAnchors ShiftBy(const FVector2D& Offset);
	FNodeAlignmentAnchors ExpandBy(const FVector2D& SizeChange);
};

/** Parameters to customize how nodes can be aligned with other nodes. */
struct FNodeAlignmentParams
{
public:
	/** Distance along snapped axis that snapping occurs. */
	float SnapProximity;

	/** The maximum distance two nodes can be that snapping can occur. */
	float MaxSnapRadius;

	/** Amount of padding to add between nodes when snapping their adjacent edges together. */
	float SnapAdjacentEdgesPadding;

	/** Indicates whether nodes' adjacent edges (e.g. right edge to left edge) should snap. */
	bool bCanSnapAdjacentEdges;

	/** Indicates whether nodes' equivalent edges (e.g top edge to top edge) should snap. */
	bool bCanSnapSameEdges;

	/** A list of anchors that won't be aligned with. */
	TSet<EAlignmentAnchor> AnchorsToIgnore;

	FNodeAlignmentParams() :
		SnapProximity(10),
		MaxSnapRadius(FLT_MAX),
		SnapAdjacentEdgesPadding(10),
		bCanSnapAdjacentEdges(true),
		bCanSnapSameEdges(true)
	{}
};

struct FNodeAlignment
{
public:
	TWeakObjectPtr<const UDisplayClusterConfiguratorBaseNode> TargetNode;
	EAlignmentAnchor AlignedAnchor;
	bool bIsAdjacent;
	float AlignmentOffset;

	FNodeAlignment() :
		TargetNode(nullptr),
		AlignedAnchor(EAlignmentAnchor::Center),
		bIsAdjacent(false),
		AlignmentOffset(0)
	{ }

	FNodeAlignment(const UDisplayClusterConfiguratorBaseNode* InTargetNode, EAlignmentAnchor InAlignedAnchor, bool bInIsAdjacent, float InAlignmentOffset) :
		TargetNode(InTargetNode),
		AlignedAnchor(InAlignedAnchor),
		bIsAdjacent(bInIsAdjacent),
		AlignmentOffset(InAlignmentOffset)
	{ }

	bool IsValid() const { return TargetNode.IsValid(); }
};

struct FNodeAlignmentPair
{
public:
	FNodeAlignment XAlignment;
	FNodeAlignment YAlignment;

	FNodeAlignmentPair(FNodeAlignment InXAlignment, FNodeAlignment InYAlignment) :
		XAlignment(InXAlignment),
		YAlignment(InYAlignment)
	{ }

	FVector2D GetOffset() const;
	bool HasAlignments() const;
};

/**
 * Stores a list of all possible alignment options for a given node to choose from. Each axis stores a separate list of alignments,
 * allowing a node to be aligned two different nodes at the same time, one along each axis.
 */
struct FDisplayClusterConfiguratorNodeAlignmentHelper
{
public:
	FDisplayClusterConfiguratorNodeAlignmentHelper(const UDisplayClusterConfiguratorBaseNode* InNodeToAlign, const FNodeAlignmentAnchors& InNodeAnchors, const FNodeAlignmentParams& InAlignmentParams);

	void AddAlignmentsToNode(const UDisplayClusterConfiguratorBaseNode* TargetNode);
	void AddAlignmentsToParent(const UDisplayClusterConfiguratorBaseNode* TargetParent);

	FNodeAlignmentPair GetAlignments() const;
	bool HasAlignments() const;

private:
	void AddAlignment(FNodeAlignment NewAlignment, int32 AlignmentAxis);

	void TryAlignAjacentEdge(const UDisplayClusterConfiguratorBaseNode* TargetNode, EAlignmentAnchor Anchor, bool bIsParent);
	void TryAlignSameEdge(const UDisplayClusterConfiguratorBaseNode* TargetNode, EAlignmentAnchor Anchor, bool bIsParent);

private:
	TWeakObjectPtr<const UDisplayClusterConfiguratorBaseNode> NodeToAlign;
	FNodeAlignmentAnchors NodeAnchors;
	FNodeAlignmentParams AlignmentParams;

	TArray<FNodeAlignment> XAxisAlignments;
	TArray<FNodeAlignment> YAxisAlignments;
};