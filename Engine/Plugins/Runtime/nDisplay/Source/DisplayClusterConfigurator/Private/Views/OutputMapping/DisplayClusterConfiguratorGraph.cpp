// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/OutputMapping/DisplayClusterConfiguratorGraph.h"

#include "DisplayClusterConfiguratorBlueprintEditor.h"
#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterConfiguratorStyle.h"
#include "ClusterConfiguration/DisplayClusterConfiguratorClusterUtils.h"
#include "Views/OutputMapping/DisplayClusterConfiguratorViewOutputMapping.h"
#include "Views/OutputMapping/EdNodes/DisplayClusterConfiguratorBaseNode.h"
#include "Views/OutputMapping/EdNodes/DisplayClusterConfiguratorCanvasNode.h"
#include "Views/OutputMapping/EdNodes/DisplayClusterConfiguratorHostNode.h"
#include "Views/OutputMapping/EdNodes/DisplayClusterConfiguratorWindowNode.h"
#include "Views/OutputMapping/EdNodes/DisplayClusterConfiguratorViewportNode.h"

#include "Algo/MaxElement.h"

void UDisplayClusterConfiguratorGraph::Initialize(const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& InToolkit)
{
	ToolkitPtr = InToolkit;
}

void UDisplayClusterConfiguratorGraph::PostEditUndo()
{
	Super::PostEditUndo();

	NotifyGraphChanged();
}

void UDisplayClusterConfiguratorGraph::Cleanup()
{
	for (UEdGraphNode* Node : Nodes)
	{
		if (UDisplayClusterConfiguratorBaseNode* BaseNode = Cast<UDisplayClusterConfiguratorBaseNode>(Node))
		{
			BaseNode->Cleanup();
		}
	}
}

void UDisplayClusterConfiguratorGraph::Empty()
{
	// Manually remove nodes here instead of using built in RemoveNode method to avoid invoking any GraphChanged delegates for node removal.
	// Call Modify on this graph and all removed nodes to ensure that, if need be, the graph is reinstated to its previous configuration on an undo.
	Modify();

	for (UEdGraphNode* Node : Nodes)
	{
		Node->Modify();
	}

	Nodes.Empty();
	NotifyGraphChanged();
}

void UDisplayClusterConfiguratorGraph::RebuildGraph()
{
	Empty();

	if (UDisplayClusterConfigurationData* Config = ToolkitPtr.Pin()->GetConfig())
	{
		UDisplayClusterConfiguratorCanvasNode* CanvasNode = BuildCanvasNode(Config->Cluster);

		TMap<FString, TMap<FString, UDisplayClusterConfigurationClusterNode*>> SortedClusterNodes;
		UE::DisplayClusterConfiguratorClusterUtils::SortClusterNodesByHost(Config->Cluster->Nodes, SortedClusterNodes);

		// Keep track of the count of each cluster item to use as a z-index for the node.
		int HostIndex = 0;
		int WindowIndex = 0;
		int ViewportIndex = 0;

		for (const TPair<FString, TMap<FString, UDisplayClusterConfigurationClusterNode*>>& SortedPair : SortedClusterNodes)
		{
			UDisplayClusterConfigurationHostDisplayData* DisplayData = UE::DisplayClusterConfiguratorClusterUtils::FindOrCreateHostDisplayData(Config->Cluster, SortedPair.Key);
			UDisplayClusterConfiguratorHostNode* HostNode = BuildHostNode(CanvasNode, SortedPair.Key, HostIndex, DisplayData);
			++HostIndex;

			for (const TPair<FString, UDisplayClusterConfigurationClusterNode*>& ClusterNodePair : SortedPair.Value)
			{
				UDisplayClusterConfiguratorWindowNode* WindowNode = BuildWindowNode(HostNode, ClusterNodePair.Key, WindowIndex, ClusterNodePair.Value);
				++WindowIndex;

				for (const TPair<FString, TObjectPtr<UDisplayClusterConfigurationViewport>>& ViewportPair : ClusterNodePair.Value->Viewports)
				{
					BuildViewportNode(WindowNode, ViewportPair.Key, ViewportIndex, ViewportPair.Value);
					++ViewportIndex;
				}
			}
		}

		if (TSharedPtr<IDisplayClusterConfiguratorViewOutputMapping> OutputMappingView = ToolkitPtr.Pin()->GetViewOutputMapping())
		{
			OutputMappingView->GetOnOutputMappingBuiltDelegate().Broadcast();
		}
	}
}

void UDisplayClusterConfiguratorGraph::TickNodePositions()
{
	ForEachGraphNode([](UDisplayClusterConfiguratorBaseNode* Node)
	{
		Node->TickPosition();
	});
}

void UDisplayClusterConfiguratorGraph::RefreshNodePositions()
{
	ForEachGraphNode([](UDisplayClusterConfiguratorBaseNode* Node)
	{
		Node->UpdateNode();
	});
}

UDisplayClusterConfiguratorBaseNode* UDisplayClusterConfiguratorGraph::GetNodeFromObject(UObject* InObject)
{
	for (UEdGraphNode* Node : Nodes)
	{
		if (UDisplayClusterConfiguratorBaseNode* BaseNode = Cast<UDisplayClusterConfiguratorBaseNode>(Node))
		{
			if (BaseNode->GetObject() == InObject)
			{
				return BaseNode;
			}
		}
	}

	return nullptr;
}

UDisplayClusterConfiguratorCanvasNode* UDisplayClusterConfiguratorGraph::GetRootNode() const
{
	for (UEdGraphNode* Node : Nodes)
	{
		if (UDisplayClusterConfiguratorCanvasNode* CanvasNode = Cast<UDisplayClusterConfiguratorCanvasNode>(Node))
		{
			return CanvasNode;
		}
	}

	return nullptr;
}

void UDisplayClusterConfiguratorGraph::ForEachGraphNode(TFunction<void(UDisplayClusterConfiguratorBaseNode* Node)> Predicate)
{
	if (UDisplayClusterConfiguratorCanvasNode* RootNode = GetRootNode())
	{
		Predicate(RootNode);
		ForEachChildNode(RootNode, Predicate);
	}
}

void UDisplayClusterConfiguratorGraph::ForEachChildNode(UDisplayClusterConfiguratorBaseNode* Node, TFunction<void(UDisplayClusterConfiguratorBaseNode* Node)> Predicate)
{
	for (UDisplayClusterConfiguratorBaseNode* ChildNode : Node->GetChildren())
	{
		Predicate(ChildNode);
		ForEachChildNode(ChildNode, Predicate);
	}
}

UDisplayClusterConfiguratorCanvasNode* UDisplayClusterConfiguratorGraph::BuildCanvasNode(UDisplayClusterConfigurationCluster* ClusterConfig)
{
	FGraphNodeCreator<UDisplayClusterConfiguratorCanvasNode> NodeCreator(*this);
	UDisplayClusterConfiguratorCanvasNode* NewNode = NodeCreator.CreateNode(false);
	NewNode->Initialize(FString(), 0, ClusterConfig, ToolkitPtr.Pin().ToSharedRef());

	NodeCreator.Finalize();

	return NewNode;
}

UDisplayClusterConfiguratorHostNode* UDisplayClusterConfiguratorGraph::BuildHostNode(UDisplayClusterConfiguratorBaseNode* ParentNode, FString NodeName, int32 NodeIndex, UDisplayClusterConfigurationHostDisplayData* HostDisplayData)
{
	FGraphNodeCreator<UDisplayClusterConfiguratorHostNode> NodeCreator(*this);
	UDisplayClusterConfiguratorHostNode* NewNode = NodeCreator.CreateNode(false);
	NewNode->Initialize(NodeName, NodeIndex, HostDisplayData, ToolkitPtr.Pin().ToSharedRef());

	ParentNode->AddChild(NewNode);

	NodeCreator.Finalize();

	return NewNode;
}

UDisplayClusterConfiguratorWindowNode* UDisplayClusterConfiguratorGraph::BuildWindowNode(UDisplayClusterConfiguratorBaseNode* ParentNode, FString NodeName, int32 NodeIndex, UDisplayClusterConfigurationClusterNode* ClusterNodeConfig)
{
	FGraphNodeCreator<UDisplayClusterConfiguratorWindowNode> NodeCreator(*this);
	UDisplayClusterConfiguratorWindowNode* NewNode = NodeCreator.CreateNode(false);
	NewNode->Initialize(NodeName, NodeIndex, ClusterNodeConfig, ToolkitPtr.Pin().ToSharedRef());

	ParentNode->AddChild(NewNode);

	NodeCreator.Finalize();

	return NewNode;
}

UDisplayClusterConfiguratorViewportNode* UDisplayClusterConfiguratorGraph::BuildViewportNode(UDisplayClusterConfiguratorBaseNode* ParentNode, FString NodeName, int32 NodeIndex, UDisplayClusterConfigurationViewport* ViewportConfig)
{
	if (!ensure(ViewportConfig))
	{
		// TODO: Remove or verify this is okay.. When deleting viewports this can be null during a rebuild.
		return nullptr;
	}
	
	FGraphNodeCreator<UDisplayClusterConfiguratorViewportNode> NodeCreator(*this);
	UDisplayClusterConfiguratorViewportNode* NewNode = NodeCreator.CreateNode(false);
	NewNode->Initialize(NodeName, NodeIndex, ViewportConfig, ToolkitPtr.Pin().ToSharedRef());

	ParentNode->AddChild(NewNode);

	NodeCreator.Finalize();

	return NewNode;
}