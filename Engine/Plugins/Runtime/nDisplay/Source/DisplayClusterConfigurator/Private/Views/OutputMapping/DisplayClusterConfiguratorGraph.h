// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraph/EdGraph.h"
#include "UObject/StrongObjectPtr.h"

#include "Views/OutputMapping/EdNodes/DisplayClusterConfiguratorBaseNode.h"

#include "DisplayClusterConfiguratorGraph.generated.h"

class FDisplayClusterConfiguratorBlueprintEditor;
class UDisplayClusterConfigurationData;
class UDisplayClusterConfigurationCluster;
class UDisplayClusterConfigurationClusterNode;
class UDisplayClusterConfigurationViewport;
class UDisplayClusterConfigurationHostDisplayData;
class UDisplayClusterConfiguratorCanvasNode;
class UDisplayClusterConfiguratorHostNode;
class UDisplayClusterConfiguratorWindowNode;
class UDisplayClusterConfiguratorViewportNode;

UCLASS()
class UDisplayClusterConfiguratorGraph
	: public UEdGraph
{
	GENERATED_BODY()

public:
	void Initialize(const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& InToolkit);

	// Beign UObject Interface
	virtual void PostEditUndo() override;
	// End UObject Interface

	/** Cleans up the graph and its nodes. */
	void Cleanup();

	/** Removes all nodes from the graph. */
	void Empty();

	/** Rebuilds the graph using the current cluster configuration. */
	void RebuildGraph();

	/** Allows each node on the graph to reposition itself during a tick. */
	void TickNodePositions();

	/** Recomputes the global positions of all graph nodes. */
	void RefreshNodePositions();

	/** Gets the graph node that contains the specified object. */
	UDisplayClusterConfiguratorBaseNode* GetNodeFromObject(UObject* InObject);

	/** @return The root canvas node of the graph. */
	UDisplayClusterConfiguratorCanvasNode* GetRootNode() const;

	/** Iterates over all nodes in the graph through the node hierarchy and applies the specified predicate. */
	void ForEachGraphNode(TFunction<void(UDisplayClusterConfiguratorBaseNode* Node)> Predicate);

	/** Iterates over the child nodes of the specified node and applies the specified predicate. */
	void ForEachChildNode(UDisplayClusterConfiguratorBaseNode* Node, TFunction<void(UDisplayClusterConfiguratorBaseNode* Node)> Predicate);

	/** @return The toolkit used by this graph */
	TWeakPtr<FDisplayClusterConfiguratorBlueprintEditor> GetToolkit() const { return ToolkitPtr; }

private:
	UDisplayClusterConfiguratorCanvasNode* BuildCanvasNode(UDisplayClusterConfigurationCluster* ClusterConfig);
	UDisplayClusterConfiguratorHostNode* BuildHostNode(UDisplayClusterConfiguratorBaseNode* ParentNode, FString NodeName, int32 NodeIndex, UDisplayClusterConfigurationHostDisplayData* HostDisplayData);
	UDisplayClusterConfiguratorWindowNode* BuildWindowNode(UDisplayClusterConfiguratorBaseNode* ParentNode, FString NodeName, int32 NodeIndex, UDisplayClusterConfigurationClusterNode* ClusterNodeConfig);
	UDisplayClusterConfiguratorViewportNode* BuildViewportNode(UDisplayClusterConfiguratorBaseNode* ParentNode, FString NodeName, int32 NodeIndex, UDisplayClusterConfigurationViewport* ViewportConfig);

private:
	TWeakPtr<FDisplayClusterConfiguratorBlueprintEditor> ToolkitPtr;
};
