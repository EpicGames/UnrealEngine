// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/OutputMapping/DisplayClusterConfiguratorGraphSchema.h"

#include "DisplayClusterConfiguratorBlueprintEditor.h"
#include "DisplayClusterConfiguratorGraph.h"
#include "DisplayClusterConfigurationTypes.h"
#include "ClusterConfiguration/DisplayClusterConfiguratorClusterEditorUtils.h"
#include "ClusterConfiguration/SDisplayClusterConfiguratorNewClusterItemDialog.h"

#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "SDisplayClusterConfiguratorNewClusterItemDialog"

FDisplayClusterConfiguratorSchemaAction_NewNode::FDisplayClusterConfiguratorSchemaAction_NewNode() :
	FEdGraphSchemaAction(),
	ItemType(EClusterItemType::ClusterNode),
	PresetSize(FVector2f::ZeroVector)
{ }

FDisplayClusterConfiguratorSchemaAction_NewNode::FDisplayClusterConfiguratorSchemaAction_NewNode(EClusterItemType InItemType, const UE::Slate::FDeprecateVector2DParameter& InPresetSize, FText InDescription, FText InTooltip) :
	FEdGraphSchemaAction(LOCTEXT("NewNodeActionCategory", "nDisplay"), MoveTemp(InDescription), MoveTemp(InTooltip), 0),
	ItemType(InItemType),
	PresetSize(InPresetSize)
{ }

UEdGraphNode* FDisplayClusterConfiguratorSchemaAction_NewNode::PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode)
{
	const UDisplayClusterConfiguratorGraph* ClusterGraph = CastChecked<UDisplayClusterConfiguratorGraph>(ParentGraph);
	TSharedPtr<FDisplayClusterConfiguratorBlueprintEditor> Toolkit = ClusterGraph->GetToolkit().Pin();
	UDisplayClusterConfigurationCluster* Cluster = Toolkit->GetEditorData()->Cluster;

	TSharedPtr<FScopedTransaction> Transaction;
	UObject* CreatedObject = nullptr;
	FDisplayClusterConfigurationRectangle PresetRect = FDisplayClusterConfigurationRectangle(0, 0, PresetSize.X, PresetSize.Y);
	switch (ItemType)
	{
	case EClusterItemType::ClusterNode:
		CreatedObject = UE::DisplayClusterConfiguratorClusterEditorUtils::CreateNewClusterNodeFromDialog(Toolkit.ToSharedRef(), Cluster, PresetRect, Transaction);
		break;

	case EClusterItemType::Viewport:
		CreatedObject = UE::DisplayClusterConfiguratorClusterEditorUtils::CreateNewViewportFromDialog(Toolkit.ToSharedRef(), nullptr, PresetRect, Transaction);
		break;
	}

	if (CreatedObject)
	{
		Toolkit->GetEditorData()->MarkPackageDirty();
		Toolkit->ClusterChanged();
	}

	Transaction.Reset();

	// We are invoking OnConfigChanged, which will ultimately rebuild the graph, so no need to return the actual node here, as it will be created during the rebuild
	return nullptr;
}

void UDisplayClusterConfiguratorGraphSchema::GetGraphContextActions(FGraphContextMenuBuilder& ContextMenuBuilder) const
{
	const FVector2f PresetSize = UE::Slate::CastToVector2f(FDisplayClusterConfiguratorPresetSize::CommonPresets[FDisplayClusterConfiguratorPresetSize::DefaultPreset].Size);

	// Action for adding a new cluster node
	{
		FText Desc = LOCTEXT("NewClusterNode_Desc", "New Cluster Node");
		FText Tooltip = LOCTEXT("NewClusterNode_Tooltip", "Add a new cluster node");

		TSharedPtr<FDisplayClusterConfiguratorSchemaAction_NewNode> NewClusterNodeAction = TSharedPtr<FDisplayClusterConfiguratorSchemaAction_NewNode>(new FDisplayClusterConfiguratorSchemaAction_NewNode(EClusterItemType::ClusterNode, PresetSize, Desc, Tooltip));
		ContextMenuBuilder.AddAction(NewClusterNodeAction);
	}

	// Action for adding a new viewport
	const UDisplayClusterConfiguratorGraph* ClusterGraph = CastChecked<UDisplayClusterConfiguratorGraph>(ContextMenuBuilder.CurrentGraph);
	if (UE::DisplayClusterConfiguratorClusterEditorUtils::CanAddNewViewportToToolkit(ClusterGraph->GetToolkit()))
	{
		FText Desc = LOCTEXT("NewViewport_Desc", "New Viewport");
		FText Tooltip = LOCTEXT("NewViewport_Tooltip", "Add a new viewport");

		TSharedPtr<FDisplayClusterConfiguratorSchemaAction_NewNode> NewViewportAction = TSharedPtr<FDisplayClusterConfiguratorSchemaAction_NewNode>(new FDisplayClusterConfiguratorSchemaAction_NewNode(EClusterItemType::Viewport, PresetSize, Desc, Tooltip));
		ContextMenuBuilder.AddAction(NewViewportAction);
	}
}

#undef LOCTEXT_NAMESPACE