// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "UObject/ObjectMacros.h"
#include "EdGraph/EdGraphSchema.h"

#include "DisplayClusterConfiguratorGraphSchema.generated.h"

class UEdGraph;

UENUM()
enum class EClusterItemType : uint8
{
	ClusterNode,
	Viewport
};

USTRUCT()
struct FDisplayClusterConfiguratorSchemaAction_NewNode : public FEdGraphSchemaAction
{
	GENERATED_BODY()

public:
	FDisplayClusterConfiguratorSchemaAction_NewNode();
	FDisplayClusterConfiguratorSchemaAction_NewNode(EClusterItemType InItemType, const UE::Slate::FDeprecateVector2DParameter& InPresetSize, FText InDescription, FText InTooltip);

	//~ Begin FEdGraphSchemaAction Interface
	virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	//~ End FEdGraphSchemaAction Interface

private:
	UPROPERTY()
	EClusterItemType ItemType;

	UPROPERTY()
	FDeprecateSlateVector2D PresetSize;
};

UCLASS()
class UDisplayClusterConfiguratorGraphSchema
	: public UEdGraphSchema
{
	GENERATED_BODY()


public:
	virtual void GetGraphContextActions(FGraphContextMenuBuilder& ContextMenuBuilder) const override;
};
