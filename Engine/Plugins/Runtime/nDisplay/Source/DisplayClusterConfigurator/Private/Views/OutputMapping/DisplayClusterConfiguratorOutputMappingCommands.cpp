// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfiguratorOutputMappingCommands.h"

#define LOCTEXT_NAMESPACE "DisplayClusterConfiguratorOutputMappingCommands"


void FDisplayClusterConfiguratorOutputMappingCommands::RegisterCommands()
{	
	UI_COMMAND(ShowWindowInfo, "Info Bar", "Shows the information bar on cluster nodes", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(ShowWindowCorner, "Corner", "Shows only the corner nub on cluster nodes", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(ShowWindowNone, "None", "Hides both the information bar and corner nub on cluster nodes", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(ToggleOutsideViewports, "Show Viewports outside the Window", "Toggles displaying viewports that are entirely outside a cluster node", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::R));
	UI_COMMAND(ToggleClusterItemOverlap, "Allow Cluster Item Overlap", "Enables or disables allowing cluster items to overlap when being manipulated", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleLockClusterNodesInHosts, "Keep Cluster Nodes inside Hosts", "Prevents cluster nodes from being moved outside of hosts when being manipulated", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleTintViewports, "Tint Selected Viewports", "Toggles tinting selected viewports orange to better indicate that they are selected", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ZoomToFit, "Zoom To Fit", "Zoom To Fit In Graph", EUserInterfaceActionType::Button, FInputChord(EKeys::Z));

	UI_COMMAND(BrowseDocumentation, "Documentation", "Opens the documentation reference documentation", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(ToggleAdjacentEdgeSnapping, "Toggle Adjacent Edge Snapping", "Enables or disables snapping adjacent viewport edges together", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleSameEdgeSnapping, "Toggle Same Edge Snapping", "Enables or disables snapping equivalent viewport edges together", EUserInterfaceActionType::ToggleButton, FInputChord());

	UI_COMMAND(FillParentNode, "Fill Parent", "Resizes and positions this node to fill its parent", EUserInterfaceActionType::Button, FInputChord(EKeys::F, EModifierKey::Shift));
	UI_COMMAND(SizeToChildNodes, "Size to Children", "Resizes this node to completely wrap its children", EUserInterfaceActionType::Button, FInputChord(EKeys::C, EModifierKey::Shift));
	
	UI_COMMAND(RotateViewport90CW, "Rotate 90\u00b0 Clockwise", "Rotates this viewport clockwise by 90 degrees", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RotateViewport90CCW, "Rotate 90\u00b0 Counter Clockwise", "Rotates this viewport counter-clockwise by 90 degrees", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RotateViewport180, "Rotate 180\u00b0", "Rotates this viewport by 180 degrees", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(FlipViewportHorizontal, "Flip Horizontal", "Flips this viewport horizontally", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(FlipViewportVertical, "Flip Vertical", "Flips this viewport vertically", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ResetViewportTransform, "Reset Transform", "Resets the viewport's transform", EUserInterfaceActionType::Button, FInputChord());
}

#undef LOCTEXT_NAMESPACE