// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"

#include "DisplayClusterConfiguratorStyle.h"

class FDisplayClusterConfiguratorOutputMappingCommands 
	: public TCommands<FDisplayClusterConfiguratorOutputMappingCommands>
{
public:
	FDisplayClusterConfiguratorOutputMappingCommands()
		: TCommands<FDisplayClusterConfiguratorOutputMappingCommands>(TEXT("DisplayClusterConfigurator.OutputMapping"), 
			NSLOCTEXT("Contexts", "DisplayClusterConfiguratorOutputMapping", "Display Cluster Configurator Output Mapping"), NAME_None, FDisplayClusterConfiguratorStyle::Get().GetStyleSetName())
	{ }

	virtual void RegisterCommands() override;

public:
	TSharedPtr<FUICommandInfo> ShowWindowInfo;
	TSharedPtr<FUICommandInfo> ShowWindowCorner;
	TSharedPtr<FUICommandInfo> ShowWindowNone;
	TSharedPtr<FUICommandInfo> ToggleOutsideViewports;
	TSharedPtr<FUICommandInfo> ToggleClusterItemOverlap;
	TSharedPtr<FUICommandInfo> ToggleLockClusterNodesInHosts;
	TSharedPtr<FUICommandInfo> ToggleTintViewports;
	TSharedPtr<FUICommandInfo> ZoomToFit;
	TSharedPtr<FUICommandInfo> BrowseDocumentation;

	TSharedPtr<FUICommandInfo> ToggleAdjacentEdgeSnapping;
	TSharedPtr<FUICommandInfo> ToggleSameEdgeSnapping;

	TSharedPtr<FUICommandInfo> FillParentNode;
	TSharedPtr<FUICommandInfo> SizeToChildNodes;

	TSharedPtr<FUICommandInfo> RotateViewport90CW;
	TSharedPtr<FUICommandInfo> RotateViewport90CCW;
	TSharedPtr<FUICommandInfo> RotateViewport180;
	TSharedPtr<FUICommandInfo> FlipViewportHorizontal;
	TSharedPtr<FUICommandInfo> FlipViewportVertical;
	TSharedPtr<FUICommandInfo> ResetViewportTransform;
};