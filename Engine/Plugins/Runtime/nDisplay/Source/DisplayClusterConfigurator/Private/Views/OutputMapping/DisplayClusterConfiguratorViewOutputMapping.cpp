// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/OutputMapping/DisplayClusterConfiguratorViewOutputMapping.h"

#include "DisplayClusterConfiguratorBlueprintEditor.h"
#include "IDisplayClusterConfigurator.h"
#include "Views/OutputMapping/DisplayClusterConfiguratorGraph.h"
#include "Views/OutputMapping/DisplayClusterConfiguratorGraphSchema.h"
#include "Views/OutputMapping/DisplayClusterConfiguratorOutputMappingCommands.h"
#include "Views/OutputMapping/SDisplayClusterConfiguratorViewOutputMapping.h"
#include "Views/OutputMapping/SDisplayClusterConfiguratorGraphEditor.h"

#include "Framework/Commands/UICommandList.h"
#include "Misc/ConfigCacheIni.h"

#define LOCTEXT_NAMESPACE "FDisplayClusterConfiguratorViewOutputMapping"

#define SAVE_SETTING(SettingType, Setting) GConfig->Set##SettingType(TEXT("nDisplay.OutputMapping"), TEXT(#Setting), Setting, GEditorPerProjectIni)
#define LOAD_SETTING(SettingType, Setting) GConfig->Get##SettingType(TEXT("nDisplay.OutputMapping"), TEXT(#Setting), Setting, GEditorPerProjectIni)
#define MAP_TOGGLE_COMMAND(Command, Setting) CommandList->MapAction(Command, FExecuteAction::CreateLambda([this]() { ToggleFlag(Setting); }), FCanExecuteAction(), FIsActionChecked::CreateLambda([this]() { return Setting; }));

FDisplayClusterConfiguratorViewOutputMapping::FDisplayClusterConfiguratorViewOutputMapping(const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& InToolkit)
	: ToolkitPtr(InToolkit)
{
	// Create the graph
	FName UniqueGraphName = MakeUniqueObjectName(GetTransientPackage(), UWorld::StaticClass(), FName(*(LOCTEXT("DisplayClusterConfiguratorGraph", "Graph").ToString())));
	GraphObj = TStrongObjectPtr<UDisplayClusterConfiguratorGraph>(NewObject< UDisplayClusterConfiguratorGraph >(GetTransientPackage(), UniqueGraphName, RF_Transactional));
	GraphObj->Schema = UDisplayClusterConfiguratorGraphSchema::StaticClass();
	GraphObj->Initialize(InToolkit);

	LoadSettings();
	BindCommands();
}

FDisplayClusterConfiguratorViewOutputMapping::~FDisplayClusterConfiguratorViewOutputMapping()
{
	SaveSettings();
}

TSharedRef<SWidget> FDisplayClusterConfiguratorViewOutputMapping::CreateWidget()
{
	if (!ViewOutputMapping.IsValid())
	{
		GraphEditor = SNew(SDisplayClusterConfiguratorGraphEditor, ToolkitPtr.Pin().ToSharedRef(), SharedThis(this))
			.GraphToEdit(GraphObj.Get());

		 SAssignNew(ViewOutputMapping, SDisplayClusterConfiguratorViewOutputMapping, ToolkitPtr.Pin().ToSharedRef(), GraphEditor.ToSharedRef(), SharedThis(this))
			 .AdditionalCommands(CommandList);
	}

	return ViewOutputMapping.ToSharedRef();
}

TSharedRef<SWidget> FDisplayClusterConfiguratorViewOutputMapping::GetWidget()
{
	return ViewOutputMapping.ToSharedRef();
}

void FDisplayClusterConfiguratorViewOutputMapping::SetEnabled(bool bInEnabled)
{
	if (GraphEditor.IsValid())
	{
		GraphEditor->SetEnabled(bInEnabled);
	}
}

void FDisplayClusterConfiguratorViewOutputMapping::Cleanup()
{
	if (GraphObj.IsValid())
	{
		GraphObj->Cleanup();
	}
}

FDelegateHandle FDisplayClusterConfiguratorViewOutputMapping::RegisterOnOutputMappingBuilt(const FOnOutputMappingBuiltDelegate& Delegate)
{
	return OnOutputMappingBuilt.Add(Delegate);
}

void FDisplayClusterConfiguratorViewOutputMapping::UnregisterOnOutputMappingBuilt(FDelegateHandle DelegateHandle)
{
	OnOutputMappingBuilt.Remove(DelegateHandle);
}

void FDisplayClusterConfiguratorViewOutputMapping::FindAndSelectObjects(const TArray<UObject*>& ObjectsToSelect)
{
	if (GraphEditor.IsValid())
	{
		GraphEditor->FindAndSelectObjects(ObjectsToSelect);
	}
}

void FDisplayClusterConfiguratorViewOutputMapping::JumpToObject(UObject* InObject)
{
	if (GraphEditor.IsValid())
	{
		GraphEditor->JumpToObject(InObject);
	}
}

void FDisplayClusterConfiguratorViewOutputMapping::RefreshNodePositions()
{
	if (GraphObj.IsValid())
	{
		GraphObj->RefreshNodePositions();
	}
}

void FDisplayClusterConfiguratorViewOutputMapping::BindCommands()
{
	CommandList = MakeShareable(new FUICommandList);

	const FDisplayClusterConfiguratorOutputMappingCommands& Commands = FDisplayClusterConfiguratorOutputMappingCommands::Get();

	CommandList->MapAction(
		Commands.ShowWindowInfo, 
		FExecuteAction::CreateLambda([this]() { OutputMappingSettings.bShowWindowInfo = true; OutputMappingSettings.bShowWindowCornerImage = true; }), 
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([this]() { return OutputMappingSettings.bShowWindowInfo && OutputMappingSettings.bShowWindowCornerImage; }));

	CommandList->MapAction(
		Commands.ShowWindowCorner, 
		FExecuteAction::CreateLambda([this]() { OutputMappingSettings.bShowWindowInfo = false; OutputMappingSettings.bShowWindowCornerImage = true; }), 
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([this]() { return !OutputMappingSettings.bShowWindowInfo && OutputMappingSettings.bShowWindowCornerImage; }));
	
	CommandList->MapAction(
		Commands.ShowWindowNone, 
		FExecuteAction::CreateLambda([this]() { OutputMappingSettings.bShowWindowInfo = false; OutputMappingSettings.bShowWindowCornerImage = false; }), 
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([this]() { return !OutputMappingSettings.bShowWindowInfo && !OutputMappingSettings.bShowWindowCornerImage; }));

	MAP_TOGGLE_COMMAND(Commands.ToggleOutsideViewports, OutputMappingSettings.bShowOutsideViewports);
	MAP_TOGGLE_COMMAND(Commands.ToggleClusterItemOverlap, OutputMappingSettings.bAllowClusterItemOverlap);
	MAP_TOGGLE_COMMAND(Commands.ToggleLockClusterNodesInHosts, OutputMappingSettings.bKeepClusterNodesInHosts);
	MAP_TOGGLE_COMMAND(Commands.ToggleTintViewports, OutputMappingSettings.bTintSelectedViewports);

	MAP_TOGGLE_COMMAND(Commands.ToggleAdjacentEdgeSnapping, NodeAlignmentSettings.bSnapAdjacentEdges);
	MAP_TOGGLE_COMMAND(Commands.ToggleSameEdgeSnapping, NodeAlignmentSettings.bSnapSameEdges);
}

void FDisplayClusterConfiguratorViewOutputMapping::LoadSettings()
{
	LOAD_SETTING(Bool, OutputMappingSettings.bShowRuler);
	LOAD_SETTING(Bool, OutputMappingSettings.bShowWindowInfo);
	LOAD_SETTING(Bool, OutputMappingSettings.bShowWindowCornerImage);
	LOAD_SETTING(Bool, OutputMappingSettings.bShowOutsideViewports);
	LOAD_SETTING(Bool, OutputMappingSettings.bAllowClusterItemOverlap);
	LOAD_SETTING(Bool, OutputMappingSettings.bKeepClusterNodesInHosts);
	LOAD_SETTING(Bool, OutputMappingSettings.bLockViewports);
	LOAD_SETTING(Bool, OutputMappingSettings.bLockClusterNodes);
	LOAD_SETTING(Bool, OutputMappingSettings.bTintSelectedViewports);
	LOAD_SETTING(Float, OutputMappingSettings.ViewScale);

	int32 EnumSetting = 0;
	GConfig->GetInt(TEXT("nDisplay.OutputMapping"), TEXT("HostArrangementSettings.ArrnagementType"), EnumSetting, GEditorPerProjectIni);
	HostArrangementSettings.ArrangementType = (EHostArrangementType)EnumSetting;
	LOAD_SETTING(Float, HostArrangementSettings.WrapThreshold);
	LOAD_SETTING(Int, HostArrangementSettings.GridSize);

	LOAD_SETTING(Int, NodeAlignmentSettings.SnapProximity);
	LOAD_SETTING(Int, NodeAlignmentSettings.AdjacentEdgesSnapPadding);
	LOAD_SETTING(Bool, NodeAlignmentSettings.bSnapAdjacentEdges);
	LOAD_SETTING(Bool, NodeAlignmentSettings.bSnapSameEdges);
}

void FDisplayClusterConfiguratorViewOutputMapping::SaveSettings()
{
	SAVE_SETTING(Bool, OutputMappingSettings.bShowRuler);
	SAVE_SETTING(Bool, OutputMappingSettings.bShowWindowInfo);
	SAVE_SETTING(Bool, OutputMappingSettings.bShowWindowCornerImage);
	SAVE_SETTING(Bool, OutputMappingSettings.bShowOutsideViewports);
	SAVE_SETTING(Bool, OutputMappingSettings.bAllowClusterItemOverlap);
	SAVE_SETTING(Bool, OutputMappingSettings.bKeepClusterNodesInHosts);
	SAVE_SETTING(Bool, OutputMappingSettings.bLockViewports);
	SAVE_SETTING(Bool, OutputMappingSettings.bLockClusterNodes);
	SAVE_SETTING(Bool, OutputMappingSettings.bTintSelectedViewports);
	SAVE_SETTING(Float, OutputMappingSettings.ViewScale);
	
	int32 EnumSetting = (int32)HostArrangementSettings.ArrangementType;
	GConfig->SetInt(TEXT("nDisplay.OutputMapping"), TEXT("HostArrangementSettings.ArrnagementType"), EnumSetting, GEditorPerProjectIni);
	SAVE_SETTING(Float, HostArrangementSettings.WrapThreshold);
	SAVE_SETTING(Int, HostArrangementSettings.GridSize);

	SAVE_SETTING(Int, NodeAlignmentSettings.SnapProximity);
	SAVE_SETTING(Int, NodeAlignmentSettings.AdjacentEdgesSnapPadding);
	SAVE_SETTING(Bool, NodeAlignmentSettings.bSnapAdjacentEdges);
	SAVE_SETTING(Bool, NodeAlignmentSettings.bSnapSameEdges);
}

void FDisplayClusterConfiguratorViewOutputMapping::ToggleFlag(bool& bFlag)
{
	bFlag = !bFlag;
}

ECheckBoxState FDisplayClusterConfiguratorViewOutputMapping::FlagToCheckBoxState(bool bFlag) const
{
	return bFlag ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

#undef LOCTEXT_NAMESPACE