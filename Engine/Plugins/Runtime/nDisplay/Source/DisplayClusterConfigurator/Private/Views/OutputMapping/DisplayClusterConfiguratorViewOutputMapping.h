// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Views/OutputMapping/IDisplayClusterConfiguratorViewOutputMapping.h"

#include "UObject/StrongObjectPtr.h"
#include "Styling/SlateTypes.h"

class FUICommandList;
class FDisplayClusterConfiguratorBlueprintEditor;
class SDisplayClusterConfiguratorGraphEditor;
class SDisplayClusterConfiguratorViewOutputMapping;
class UDisplayClusterConfiguratorGraph;

class FDisplayClusterConfiguratorViewOutputMapping
	: public IDisplayClusterConfiguratorViewOutputMapping
{
public:
	FDisplayClusterConfiguratorViewOutputMapping(const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& InToolkit);
	~FDisplayClusterConfiguratorViewOutputMapping();

	//~ Begin IDisplayClusterConfiguratorView Interface
	virtual TSharedRef<SWidget> CreateWidget() override;
	virtual TSharedRef<SWidget> GetWidget() override;
	virtual void SetEnabled(bool bInEnabled) override;
	//~ End IDisplayClusterConfiguratorView Interface

	//~ Begin IDisplayClusterConfiguratorView Interface
	virtual void Cleanup() override;

	virtual FOnOutputMappingBuilt& GetOnOutputMappingBuiltDelegate() override { return OnOutputMappingBuilt; }
	virtual FDelegateHandle RegisterOnOutputMappingBuilt(const FOnOutputMappingBuiltDelegate& Delegate) override;
	virtual void UnregisterOnOutputMappingBuilt(FDelegateHandle DelegateHandle) override;

	virtual const FOutputMappingSettings& GetOutputMappingSettings() const override { return OutputMappingSettings; }
	virtual FOutputMappingSettings& GetOutputMappingSettings() override { return OutputMappingSettings; }

	virtual const FHostNodeArrangementSettings& GetHostArrangementSettings() const override { return HostArrangementSettings; }
	virtual FHostNodeArrangementSettings& GetHostArrangementSettings() override { return HostArrangementSettings; }

	virtual const FNodeAlignmentSettings& GetNodeAlignmentSettings() const override { return NodeAlignmentSettings; }
	virtual FNodeAlignmentSettings& GetNodeAlignmentSettings() override { return NodeAlignmentSettings; }

	virtual void FindAndSelectObjects(const TArray<UObject*>& ObjectsToSelect) override;

	virtual void JumpToObject(UObject* InObject) override;

	//~ End IDisplayClusterConfiguratorView Interface

	void RefreshNodePositions();

private:
	void BindCommands();

	void LoadSettings();
	void SaveSettings();

	void ToggleFlag(bool& bFlag);
	ECheckBoxState FlagToCheckBoxState(bool bFlag) const;

private:
	TSharedPtr<SDisplayClusterConfiguratorViewOutputMapping> ViewOutputMapping;
	TWeakPtr<FDisplayClusterConfiguratorBlueprintEditor> ToolkitPtr;
	TSharedPtr<SDisplayClusterConfiguratorGraphEditor> GraphEditor;
	TStrongObjectPtr<UDisplayClusterConfiguratorGraph> GraphObj;

	FOutputMappingSettings OutputMappingSettings;
	FHostNodeArrangementSettings HostArrangementSettings;
	FNodeAlignmentSettings NodeAlignmentSettings;

	TSharedPtr<FUICommandList> CommandList;

	FOnOutputMappingBuilt OnOutputMappingBuilt;
};
