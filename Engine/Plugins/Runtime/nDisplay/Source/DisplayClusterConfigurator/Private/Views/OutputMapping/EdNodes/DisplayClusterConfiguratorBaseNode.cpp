// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/OutputMapping/EdNodes/DisplayClusterConfiguratorBaseNode.h"

#include "Views/OutputMapping/Alignment/DisplayClusterConfiguratorNodeAlignmentHelper.h"
#include "Views/OutputMapping/IDisplayClusterConfiguratorViewOutputMapping.h"
#include "Views/TreeViews/IDisplayClusterConfiguratorViewTree.h"
#include "Views/TreeViews/IDisplayClusterConfiguratorTreeItem.h"

#include "DisplayClusterConfiguratorBlueprintEditor.h"

void UDisplayClusterConfiguratorBaseNode::Initialize(const FString& InNodeName, int32 InNodeZIndex, UObject* InObject, const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& InToolkit)
{
	ObjectToEdit = InObject;
	NodeName = InNodeName;
	NodeZIndex = InNodeZIndex;
	ToolkitPtr = InToolkit;
}

void UDisplayClusterConfiguratorBaseNode::BeginDestroy()
{
	Super::BeginDestroy();

	Cleanup();
}

#if WITH_EDITOR
void UDisplayClusterConfiguratorBaseNode::PostEditUndo()
{
	Super::PostEditUndo();

	// Make sure the undo operation isn't going to a state where this node or the object it edits isn't valid
	// before attempting to update the object or its children
	//
	// The ToolkitPtr could be invalid if the node was deleted, the BP editor was closed, then the operation undone.
	// This node is invalid in this case and will be recreated in the new BP editor instance.
	if (IsObjectValid() && ToolkitPtr.IsValid())
	{
		CleanupChildrenNodes();
		
		UpdateObject();

		// Don't update the child nodes if this node is auto-positioned because this node is probably in the undo stack as part of
		// a change to a child, and we don't want to overwrite the child's undo. If the children need updating, it will be handled on
		// the next position tick.
		if (!IsNodeAutoPositioned())
		{
			UpdateChildNodes();
		}
	}
}
#endif

void UDisplayClusterConfiguratorBaseNode::PostPlacedNewNode()
{
	Super::PostPlacedNewNode();

	ReadNodeStateFromObject();
}

void UDisplayClusterConfiguratorBaseNode::ResizeNode(const FVector2f& NewSize)
{
	Modify();

	NodeHeight = UE::LWC::FloatToIntCastChecked<int32>(NewSize.Y);
	NodeWidth = UE::LWC::FloatToIntCastChecked<int32>(NewSize.X);

	UpdateObject();
}

void UDisplayClusterConfiguratorBaseNode::OnSelection()
{
	TArray<UObject*> SelectedObjects;
	if (UObject* Object = GetObject())
	{
		SelectedObjects.Add(Object);
	}

	ToolkitPtr.Pin()->SelectObjects(SelectedObjects);
}

bool UDisplayClusterConfiguratorBaseNode::IsSelected()
{
	const TArray<UObject*>& SelectedObjects = ToolkitPtr.Pin()->GetSelectedObjects();

	UObject* const* SelectedObject = SelectedObjects.FindByPredicate([this](const UObject* InObject)
	{
		return InObject == GetObject();
	});

	if (SelectedObject != nullptr)
	{
		UObject* Obj = *SelectedObject;

		return Obj != nullptr;
	}

	return false;
}

const FString& UDisplayClusterConfiguratorBaseNode::GetNodeName() const
{
	return NodeName;
}

void UDisplayClusterConfiguratorBaseNode::SetParent(UDisplayClusterConfiguratorBaseNode* InParent)
{
	Parent = InParent;
}

UDisplayClusterConfiguratorBaseNode* UDisplayClusterConfiguratorBaseNode::GetParent() const
{
	if (Parent.IsValid())
	{
		return Parent.Get();
	}
	
	return nullptr;
}

void UDisplayClusterConfiguratorBaseNode::AddChild(UDisplayClusterConfiguratorBaseNode* InChild)
{
	InChild->SetParent(this);
	Children.Add(InChild);
}

const TArray<UDisplayClusterConfiguratorBaseNode*>& UDisplayClusterConfiguratorBaseNode::GetChildren() const
{
	return Children;
}

FVector2D UDisplayClusterConfiguratorBaseNode::TransformPointToLocal(FVector2D GlobalPosition) const
{
	const FVector2D ParentPosition = Parent.IsValid() ? Parent->GetNodePosition() + Parent->GetTranslationOffset() : FVector2D::ZeroVector;
	const float ViewScale = GetViewScale();
	return (GlobalPosition - ParentPosition) / ViewScale;
}

FVector2D UDisplayClusterConfiguratorBaseNode::TransformPointToGlobal(FVector2D LocalPosition) const
{
	FVector2D ParentPosition = Parent.IsValid() ? Parent->GetNodePosition() + Parent->GetTranslationOffset() : FVector2D::ZeroVector;
	const float ViewScale = GetViewScale();
	return LocalPosition * ViewScale + ParentPosition;
}

FVector2D UDisplayClusterConfiguratorBaseNode::TransformSizeToLocal(FVector2D GlobalSize) const
{
	const float ViewScale = GetViewScale();
	return GlobalSize / ViewScale;
}

FVector2D UDisplayClusterConfiguratorBaseNode::TransformSizeToGlobal(FVector2D LocalSize) const
{
	const float ViewScale = GetViewScale();
	return LocalSize * ViewScale;
}

FBox2D UDisplayClusterConfiguratorBaseNode::GetNodeBounds(bool bAsParent) const
{
	FVector2D Min = FVector2D(NodePosX, NodePosY);
	FVector2D Max = Min + FVector2D(NodeWidth, NodeHeight);
	return FBox2D(Min, Max);
}

FVector2D UDisplayClusterConfiguratorBaseNode::GetNodePosition() const
{
	return FVector2D(NodePosX, NodePosY);
}

FVector2D UDisplayClusterConfiguratorBaseNode::GetNodeLocalPosition() const
{
	return TransformPointToLocal(GetNodePosition());
}

FVector2D UDisplayClusterConfiguratorBaseNode::GetNodeSize() const
{
	return FVector2D(NodeWidth, NodeHeight);
}

FVector2D UDisplayClusterConfiguratorBaseNode::GetNodeLocalSize() const
{
	return TransformSizeToLocal(GetNodeSize());
}

FNodeAlignmentAnchors UDisplayClusterConfiguratorBaseNode::GetNodeAlignmentAnchors(bool bAsParent) const
{
	FNodeAlignmentAnchors Anchors;

	const FVector2D NodePos = GetNodePosition();
	const FVector2D NodeExtent = 0.5f * GetNodeSize();

	Anchors.Center = NodePos + NodeExtent;
	Anchors.Top = Anchors.Center - FVector2D(0, NodeExtent.Y);
	Anchors.Bottom = Anchors.Center + FVector2D(0, NodeExtent.Y);
	Anchors.Left = Anchors.Center - FVector2D(NodeExtent.X, 0);
	Anchors.Right = Anchors.Center + FVector2D(NodeExtent.X, 0);

	return Anchors;
}

int32 UDisplayClusterConfiguratorBaseNode::GetNodeLayer(const TSet<UObject*>& SelectionSet, bool bIncludeZIndex) const
{
	int32 ZIndex = 0;
	if (bIncludeZIndex)
	{
		// Don't let the z-index exceed a value that would push the node into the next layer.
		ZIndex = FMath::Min(NodeZIndex * DisplayClusterConfiguratorGraphLayers::ZIndexSize, DisplayClusterConfiguratorGraphLayers::LayerSize - DisplayClusterConfiguratorGraphLayers::ZIndexSize);
	}

	bool bIsSelected = SelectionSet.Contains(this);

	if (Parent.IsValid())
	{
		int32 LayerIndex = Parent->GetNodeLayer(SelectionSet, false) + DisplayClusterConfiguratorGraphLayers::LayerSize + ZIndex;

		if (bIsSelected && LayerIndex < DisplayClusterConfiguratorGraphLayers::SelectedLayerIndex)
		{
			LayerIndex += DisplayClusterConfiguratorGraphLayers::SelectedLayerIndex;
		}

		return LayerIndex;
	}

	return bIsSelected ? DisplayClusterConfiguratorGraphLayers::SelectedLayerIndex + ZIndex : DisplayClusterConfiguratorGraphLayers::BaseLayerIndex + ZIndex;
}

int32 UDisplayClusterConfiguratorBaseNode::GetAuxiliaryLayer(const TSet<UObject*>& SelectionSet) const
{
	bool bIsChildSelected = false;
	for (UDisplayClusterConfiguratorBaseNode* Child : Children)
	{
		if (SelectionSet.Contains(Child))
		{
			bIsChildSelected = true;
			break;
		}
	}

	int32 LayerIndex = GetNodeLayer(SelectionSet);
	int32 AuxLayerIndex = DisplayClusterConfiguratorGraphLayers::AuxiliaryLayerIndex;

	if (LayerIndex >= DisplayClusterConfiguratorGraphLayers::SelectedLayerIndex || bIsChildSelected)
	{
		AuxLayerIndex += DisplayClusterConfiguratorGraphLayers::SelectedLayerIndex;
	}

	return AuxLayerIndex;
}

void UDisplayClusterConfiguratorBaseNode::FillParent(bool bRepositionNode)
{
	if (Parent.IsValid())
	{
		const FBox2D ParentBounds = Parent->GetNodeBounds(true);
		const FVector2D ParentSize = ParentBounds.GetSize();

		Modify();

		NodeWidth = ParentSize.X;
		NodeHeight = ParentSize.Y;

		if (bRepositionNode)
		{
			NodePosX = ParentBounds.Min.X;
			NodePosY = ParentBounds.Min.Y;

			// Call UpdateObject on this node's children, which will force them to update their backing objects' states and recompute any local coordinates
			// that may have changed when their parent was repositioned. No need to edit the node's position, since that is in global coordinates and 
			// we want to keep the children located at the same global position as they were before.
			for (UDisplayClusterConfiguratorBaseNode* Child : Children)
			{
				Child->Modify();
				Child->UpdateObject();
			}
		}

		UpdateObject();
	}
}

void UDisplayClusterConfiguratorBaseNode::SizeToChildren(bool bRepositionNode)
{
	const FBox2D ChildBounds = GetChildBounds();
	if (ChildBounds.bIsValid)
	{
		Modify();

		NodeWidth = ChildBounds.GetSize().X;
		NodeHeight = ChildBounds.GetSize().Y;

		if (bRepositionNode)
		{
			NodePosX = ChildBounds.Min.X;
			NodePosY = ChildBounds.Min.Y;

			// Call UpdateObject on this node's children, which will force them to update their backing objects' states and recompute any local coordinates
			// that may have changed when their parent was repositioned. No need to edit the node's position, since that is in global coordinates and 
			// we want to keep the children located at the same global position as they were before.
			for (UDisplayClusterConfiguratorBaseNode* Child : Children)
			{
				Child->Modify();
				Child->UpdateObject();
			}
		}

		UpdateObject();
	}
}

FBox2D UDisplayClusterConfiguratorBaseNode::GetChildBounds() const
{
	FBox2D ChildBounds;
	ChildBounds.Init();

	for (const UDisplayClusterConfiguratorBaseNode* ChildNode : Children)
	{
		// Only add non-zero sized children, so that if all children are zero-sized, the returned bounds have bIsValid set to false
		// allowing the caller to test if all children had zero bounds.
		if (!ChildNode->GetNodeSize().IsZero())
		{
			ChildBounds += ChildNode->GetNodeBounds();
		}
	}

	return ChildBounds;
}

FBox2D UDisplayClusterConfiguratorBaseNode::GetDescendentBounds() const
{
	// Similar to GetChildBounds, but adds the bounds of all descendents of this node, not just its direct children
	FBox2D ChildBounds;
	ChildBounds.Init();

	for (const UDisplayClusterConfiguratorBaseNode* ChildNode : Children)
	{
		// Only add non-zero sized children, so that if all children are zero-sized, the returned bounds have bIsValid set to false
		// allowing the caller to test if all children had zero bounds.
		if (!ChildNode->GetNodeSize().IsZero())
		{
			ChildBounds += ChildNode->GetNodeBounds();
		}

		ChildBounds += ChildNode->GetDescendentBounds();
	}

	return ChildBounds;
}

bool UDisplayClusterConfiguratorBaseNode::IsOutsideParent() const
{
	if (Parent.IsValid())
	{
		FBox2D Bounds = GetNodeBounds();
		FBox2D ParentBounds = Parent->GetNodeBounds();

		if (ParentBounds.GetSize().IsZero())
		{
			return false;
		}

		if (Bounds.Min.X > ParentBounds.Max.X || Bounds.Min.Y > ParentBounds.Max.Y)
		{
			return true;
		}

		if (Bounds.Max.X < ParentBounds.Min.X || Bounds.Max.Y < ParentBounds.Min.Y)
		{
			return true;
		}
	}

	return false;
}

bool UDisplayClusterConfiguratorBaseNode::IsOutsideParentBoundary() const
{
	if (Parent.IsValid())
	{
		FBox2D Bounds = GetNodeBounds();
		FBox2D ParentBounds = Parent->GetNodeBounds();

		if (ParentBounds.GetSize().IsZero())
		{
			return false;
		}

		if (Bounds.Min.X < ParentBounds.Min.X || Bounds.Min.Y < ParentBounds.Min.Y)
		{
			return true;
		}

		if (Bounds.Max.X > ParentBounds.Max.X || Bounds.Max.Y > ParentBounds.Max.Y)
		{
			return true;
		}
	}

	return false;
}

void UDisplayClusterConfiguratorBaseNode::UpdateChildNodes()
{
	CleanupChildrenNodes();

	for (UDisplayClusterConfiguratorBaseNode* ChildNode : Children)
	{
		ChildNode->UpdateNode();
	}
}

bool UDisplayClusterConfiguratorBaseNode::IsUserInteractingWithNode(bool bCheckDescendents) const
{
	if (!bCheckDescendents)
	{
		return bIsUserInteractingWithNode;
	}

	bool bIsInteractingWithDescendents = false;
	for (UDisplayClusterConfiguratorBaseNode* ChildNode : Children)
	{
		if (ChildNode->IsUserInteractingWithNode(bCheckDescendents))
		{
			bIsInteractingWithDescendents = true;
			break;
		}
	}

	return bIsUserInteractingWithNode || bIsInteractingWithDescendents;
}

void UDisplayClusterConfiguratorBaseNode::UpdateNode()
{
	if (IsObjectValid())
	{
		ReadNodeStateFromObject();

		UpdateChildNodes();
	}
}

void UDisplayClusterConfiguratorBaseNode::UpdateObject()
{
	if (IsObjectValid())
	{
		WriteNodeStateToObject();

		if (ObjectToEdit.IsValid())
		{
			ObjectToEdit->MarkPackageDirty();
		}
	}
}

bool UDisplayClusterConfiguratorBaseNode::IsObjectValid() const
{
	return IsValidChecked(this) && ObjectToEdit.IsValid();
}

void UDisplayClusterConfiguratorBaseNode::OnNodeAligned(bool bUpdateChildren)
{
	UpdateObject();

	if (bUpdateChildren)
	{
		UpdateChildNodes();
	}
}

namespace
{
	bool Intrudes(FBox2D BoxA, FBox2D BoxB)
	{
		// Similar to FBox2D::Intersects, but ignores the case where the box edges are touching.

		// Special case if both boxes are directly on top of each other, which is considered an intrusion.
		if (BoxA == BoxB)
		{
			return true;
		}

		if ((BoxA.Min.X >= BoxB.Max.X) || (BoxB.Min.X >= BoxA.Max.X))
		{
			return false;
		}

		if ((BoxA.Min.Y >= BoxB.Max.Y) || (BoxB.Min.Y >= BoxA.Max.Y))
		{
			return false;
		}

		return true;
	}

	bool IsOutside(FVector2D Point, FBox2D Box)
	{
		return (Point.X < Box.Min.X) || (Point.X > Box.Max.X) || (Point.Y < Box.Min.Y) || (Point.Y > Box.Min.Y);
	}

	bool IsOutside(FBox2D BoxA, FBox2D BoxB)
	{
		return IsOutside(BoxA.Min, BoxB) || IsOutside(BoxA.Max, BoxB);
	}
}

bool UDisplayClusterConfiguratorBaseNode::WillOverlap(UDisplayClusterConfiguratorBaseNode* InNode, const FVector2D& InDesiredOffset, const FVector2D& InDesiredSizeChange) const
{
	const FBox2D Bounds = GetNodeBounds();

	FBox2D NodeBounds = InNode->GetNodeBounds();
	NodeBounds.Min += InDesiredOffset;
	NodeBounds.Max += InDesiredOffset + InDesiredSizeChange;

	return Intrudes(NodeBounds, Bounds);
}

FVector2D UDisplayClusterConfiguratorBaseNode::FindNonOverlappingOffset(UDisplayClusterConfiguratorBaseNode* InNode, const FVector2D& InDesiredOffset) const
{
	FVector2D BestOffset = InDesiredOffset;

	const FBox2D NodeBounds = InNode->GetNodeBounds().ShiftBy(BestOffset);
	const FVector2D NodeCenter = NodeBounds.GetCenter();

	const FBox2D Bounds = GetNodeBounds();
	const FVector2D Center = Bounds.GetCenter();

	if (Intrudes(NodeBounds, Bounds))
	{
		// If there is an intersection, we want to modify the desired offset in such a way that the new offset leaves the slot outside of the child's bounds.
		// Best way to do this is to move the slot either along the x or y axis a distance equal to the penetration depth along that axis. We must pick the 
		// axis that results in a new offset that is smaller than the desired one.
		const FBox2D IntersectionBox = FBox2D(FVector2D::Max(NodeBounds.Min, Bounds.Min), FVector2D::Min(NodeBounds.Max, Bounds.Max));
		const FVector2D AxisDepths = IntersectionBox.GetSize();

		// The direction determines which way we need to move the slot to avoid intersection, which is always in the direction that moves the slot bound's
		// center away from the child bound's center.
		const FVector2D Direction = FVector2D(FMath::Sign(NodeCenter.X - Center.X), FMath::Sign(NodeCenter.Y - Center.Y));
		const FVector2D XShift = BestOffset + Direction.X * FVector2D(AxisDepths.X, 0);
		const FVector2D YShift = BestOffset + Direction.Y * FVector2D(0, AxisDepths.Y);

		BestOffset = XShift.Size() < YShift.Size() ? XShift : YShift;
	}

	return BestOffset;
}

FVector2D UDisplayClusterConfiguratorBaseNode::FindNonOverlappingSize(UDisplayClusterConfiguratorBaseNode* InNode, const FVector2D& InDesiredSize, const bool bFixedApsectRatio) const
{
	FVector2D BestSize = InDesiredSize;

	const FVector2D OriginalSlotSize = InNode->GetNodeSize();
	const float AspectRatio = OriginalSlotSize.X / OriginalSlotSize.Y;
	FVector2D SizeChange = BestSize - OriginalSlotSize;

	if (SizeChange.GetMax() < 0)
	{
		return BestSize;
	}

	const FBox2D Bounds = GetNodeBounds();
	FBox2D NodeBounds = InNode->GetNodeBounds();
	NodeBounds.Max += SizeChange;

	if (Intrudes(NodeBounds, Bounds))
	{
		// If there is an intersection, we want to modify the desired size in such a way that the new size leaves the slot outside of the child's bounds.
		// Best way to do this is to change the slot size along the x or y axis a distance equal to the penetration depth along that axis. We must pick the 
		// axis that results in a new size change that is smaller than the desired one. In the case where aspect ratio is fixed, we need to shift the other
		// axis as well a proportional amount.
		const FBox2D IntersectionBox = FBox2D(Bounds.Min, NodeBounds.Max);
		const FVector2D AxisDepths = IntersectionBox.GetSize();

		const FVector2D XShift = SizeChange - FVector2D(AxisDepths.X, bFixedApsectRatio ? AxisDepths.X / AspectRatio : 0);
		const FVector2D YShift = SizeChange - FVector2D(bFixedApsectRatio ? AxisDepths.Y * AspectRatio : 0, AxisDepths.Y);

		SizeChange = XShift.Size() < YShift.Size() ? XShift : YShift;
		BestSize = OriginalSlotSize + SizeChange;
	}

	return BestSize;
}

FVector2D UDisplayClusterConfiguratorBaseNode::FindNonOverlappingOffsetFromParent(const FVector2D& InDesiredOffset, const TSet<UDisplayClusterConfiguratorBaseNode*>& NodesToIgnore)
{
	FVector2D BestOffset = InDesiredOffset;

	if (!Parent.IsValid())
	{
		return BestOffset;
	}

	for (const UDisplayClusterConfiguratorBaseNode* SiblingNode : Parent->GetChildren())
	{
		if (SiblingNode == this)
		{
			continue;
		}

		// Skip siblings that are not visible or enabled, as they are not visible in the graph editor and is confusing when a node is blocked by something invisible.
		if (!SiblingNode->IsNodeEnabled() || !SiblingNode->IsNodeVisible() || NodesToIgnore.Contains(SiblingNode))
		{
			continue;
		}

		BestOffset = SiblingNode->FindNonOverlappingOffset(this, BestOffset);

		// Break if we have hit a best offset of zero; there is no offset that can be performed that doesn't cause intersection.
		if (BestOffset.IsNearlyZero())
		{
			return FVector2D::ZeroVector;
		}
	}

	// In some cases, the node may still be intersecting with other nodes if its offset change pushed it into another node that it previously
	// checked. Do a final intersection check over all nodes and return zero offset if the calculated best offset still causes intersection.
	for (const UDisplayClusterConfiguratorBaseNode* SiblingNode : Parent->GetChildren())
	{
		if (SiblingNode == this)
		{
			continue;
		}

		// Skip siblings that are not visible or enabled, as they are not visible in the graph editor and is confusing when a node is blocked by something invisible.
		if (!SiblingNode->IsNodeEnabled() || !SiblingNode->IsNodeVisible() || NodesToIgnore.Contains(SiblingNode))
		{
			continue;
		}

		if (SiblingNode->WillOverlap(this, BestOffset))
		{
			return FVector2D::ZeroVector;
		}
	}

	return BestOffset;
}

FVector2D UDisplayClusterConfiguratorBaseNode::FindBoundedOffsetFromParent(const FVector2D& InDesiredOffset)
{
	FVector2D BestOffset = InDesiredOffset;

	if (!Parent.IsValid())
	{
		return BestOffset;
	}

	const FBox2D ParentBounds = Parent->GetNodeBounds(true);
	const FVector2D NodeCenter = ParentBounds.GetCenter();

	const FBox2D Bounds = GetNodeBounds().ShiftBy(BestOffset);
	const FVector2D Center = Bounds.GetCenter();
	
	const bool bIsParentAutosized = Parent->IsNodeAutosized();
	const bool bCanExceedBottomRightBounds = CanNodeExceedParentBounds() && !CanNodeHaveNegativePosition();

	if (IsOutside(Bounds, ParentBounds))
	{
		// We ignore bounding violations on the max (right/bottom) sides if the parent node is being autosized
		// because presumably the parent will resize to bound this node when its position is changed.

		// If the parent is autosized, keep track of the size change it would need to accommodate the translation, so that it can be tested for overlap later
		FVector2D DesiredParentSizeChange = FVector2D::ZeroVector;

		float XShift = 0;
		if (Bounds.Min.X < ParentBounds.Min.X)
		{
			XShift = ParentBounds.Min.X - Bounds.Min.X;
		}
		else if (Bounds.Max.X > ParentBounds.Max.X)
		{
			if (bIsParentAutosized)
			{
				DesiredParentSizeChange.X = Bounds.Max.X - ParentBounds.Max.X;
			}
			else if (!bCanExceedBottomRightBounds)
			{
				XShift = ParentBounds.Max.X - Bounds.Max.X;
			}
		}

		float YShift = 0;
		if (Bounds.Min.Y < ParentBounds.Min.Y)
		{
			YShift = ParentBounds.Min.Y - Bounds.Min.Y;
		}
		else if (Bounds.Max.Y > ParentBounds.Max.Y)
		{
			if (bIsParentAutosized)
			{
				DesiredParentSizeChange.Y = Bounds.Max.Y - ParentBounds.Max.Y;
			}
			else if (!bCanExceedBottomRightBounds)
			{
				YShift = ParentBounds.Max.Y - Bounds.Max.Y;
			}
		}

		if (!DesiredParentSizeChange.IsZero())
		{
			// If the parent is autosized and will change size to accommodate the translation of this node, make sure that doing so 
			// would not cause the parent to overlap its siblings.
			const FVector2D ParentCurrentSize = Parent->GetNodeSize();
			const FVector2D ParentMaxSize = Parent->FindNonOverlappingSizeFromParent(ParentCurrentSize + DesiredParentSizeChange, false);
			const FVector2D AllowedParentSizeChange = ParentMaxSize - ParentCurrentSize;
			XShift -= DesiredParentSizeChange.X - AllowedParentSizeChange.X;
			YShift -= DesiredParentSizeChange.Y - AllowedParentSizeChange.Y;
		}

		BestOffset += FVector2D(XShift, YShift);
	}

	return BestOffset;
}

FVector2D UDisplayClusterConfiguratorBaseNode::FindNonOverlappingSizeFromParent(const FVector2D& InDesiredSize, const bool bFixedApsectRatio)
{
	FVector2D BestSize = InDesiredSize;
	const FVector2D NodeSize = GetNodeSize();

	// If desired size is smaller in both dimensions to the slot's current size, can return it immediately, as shrinking a slot can't cause any new intersections.
	if (BestSize.ComponentwiseAllLessThan(NodeSize))
	{
		return BestSize;
	}

	for (const UDisplayClusterConfiguratorBaseNode* SiblingNode : Parent->GetChildren())
	{
		if (SiblingNode == this)
		{
			continue;
		}

		// Skip siblings that are not visible or enabled, as they are not visible in the graph editor and is confusing when a node is blocked by something invisible.
		// Also skip siblings that are auto-positioned because their position could be automatically updated to avoid overlap
		if (!SiblingNode->IsNodeEnabled() || !SiblingNode->IsNodeVisible() || SiblingNode->IsNodeAutoPositioned())
		{
			continue;
		}

		BestSize = SiblingNode->FindNonOverlappingSize(this, BestSize, bFixedApsectRatio);

		// If the best size has shrunk to be equal to current size, simply return the current size, as there is no larger size
		// that doesn't cause intersection.
		if (BestSize.Equals(NodeSize))
		{
			break;
		}
	}

	return BestSize;
}

FVector2D UDisplayClusterConfiguratorBaseNode::FindBoundedSizeFromParent(const FVector2D& InDesiredSize, const bool bFixedApsectRatio)
{
	FVector2D BestSize = InDesiredSize;
	const FVector2D NodeSize = GetNodeSize();
	const float AspectRatio = NodeSize.X / NodeSize.Y;
	FVector2D SizeChange = BestSize - NodeSize;

	// If desired size is smaller in both dimensions to the slot's current size, can return it immediately, as shrinking a slot can't cause any bound exceeding.
	if (SizeChange.GetMax() < 0)
	{
		return BestSize;
	}

	if (!Parent.IsValid())
	{
		// If the parent is autosized, don't adjust the desired size, as the parent node will presumably resize to bound the node's new size
		return BestSize;
	}

	const FBox2D ParentBounds = Parent->GetNodeBounds(true);
	FBox2D Bounds = GetNodeBounds();
	Bounds.Max += SizeChange;

	const bool bIsParentAutosized = Parent->IsNodeAutosized();

	if (IsOutside(Bounds, ParentBounds))
	{
		// If the parent is autosized, keep track of the size change it would need to accommodate this node's size change, so that it can be tested for overlap later
		FVector2D DesiredParentSizeChange = FVector2D::ZeroVector;

		float XShift = 0;
		if (Bounds.Max.X > ParentBounds.Max.X)
		{
			if (bIsParentAutosized)
			{
				DesiredParentSizeChange.X = Bounds.Max.X - ParentBounds.Max.X;
			}
			else
			{
				XShift = ParentBounds.Max.X - Bounds.Max.X;
			}
		}

		float YShift = 0;
		if (Bounds.Max.Y > ParentBounds.Max.Y)
		{
			if (bIsParentAutosized)
			{
				DesiredParentSizeChange.Y = Bounds.Max.Y - ParentBounds.Max.Y;
			}
			else
			{
				YShift = ParentBounds.Max.Y - Bounds.Max.Y;
			}
		}

		if (!DesiredParentSizeChange.IsZero())
		{
			// If the parent is autosized and will change size to accommodate the translation of this node, make sure that doing so 
			// would not cause the parent to overlap its siblings.
			const FVector2D ParentCurrentSize = Parent->GetNodeSize();
			const FVector2D ParentMaxSize = Parent->FindNonOverlappingSizeFromParent(ParentCurrentSize + DesiredParentSizeChange, false);
			const FVector2D AllowedParentSizeChange = ParentMaxSize - ParentCurrentSize;
			XShift -= DesiredParentSizeChange.X - AllowedParentSizeChange.X;
			YShift -= DesiredParentSizeChange.Y - AllowedParentSizeChange.Y;
		}

		if (bFixedApsectRatio)
		{
			if (YShift * AspectRatio < XShift)
			{
				SizeChange = FVector2D(YShift * AspectRatio, YShift);
			}
			else
			{
				SizeChange = FVector2D(XShift, XShift / AspectRatio);
			}
		}
		else
		{
			SizeChange = FVector2D(XShift, YShift);
		}

		BestSize += SizeChange;
	}

	return BestSize;
}

FVector2D UDisplayClusterConfiguratorBaseNode::FindBoundedSizeFromChildren(const FVector2D& InDesiredSize, const bool bFixedApsectRatio)
{
	FVector2D BestSize = InDesiredSize;
	const FVector2D NodeSize = GetNodeSize();
	const float AspectRatio = NodeSize.X / NodeSize.Y;
	FVector2D SizeChange = BestSize - NodeSize;

	// If desired size is bigger in both dimensions to the slot's current size, can return it immediately, as growing a slot can't cause any bound exceeding.
	if (SizeChange.GetMin() > 0)
	{
		return BestSize;
	}

	const FBox2D ChildBounds = GetChildBounds();
	FBox2D Bounds = GetNodeBounds(true);
	Bounds.Max += SizeChange;

	const bool bIsParentAutosized = Parent->IsNodeAutosized();

	if (Bounds.Max.X < ChildBounds.Max.X || Bounds.Max.Y < ChildBounds.Max.Y)
	{
		float XShift = 0;
		if (Bounds.Max.X < ChildBounds.Max.X)
		{
			XShift = ChildBounds.Max.X - Bounds.Max.X;
		}

		float YShift = 0;
		if (Bounds.Max.Y < ChildBounds.Max.Y)
		{
			YShift = ChildBounds.Max.Y - Bounds.Max.Y;
		}

		if (bFixedApsectRatio)
		{
			if (YShift * AspectRatio < XShift)
			{
				SizeChange = FVector2D(YShift * AspectRatio, YShift);
			}
			else
			{
				SizeChange = FVector2D(XShift, XShift / AspectRatio);
			}
		}
		else
		{
			SizeChange = FVector2D(XShift, YShift);
		}

		BestSize += SizeChange;
	}

	return BestSize;
}

FNodeAlignmentPair UDisplayClusterConfiguratorBaseNode::GetTranslationAlignments(const FVector2D& InOffset, const FNodeAlignmentParams& AlignmentParams, const TSet<UDisplayClusterConfiguratorBaseNode*>& NodesToIgnore) const
{
	const FNodeAlignmentAnchors ShiftedAnchors = GetNodeAlignmentAnchors().ShiftBy(InOffset);
	return GetAlignments(ShiftedAnchors, AlignmentParams, NodesToIgnore);
}

FNodeAlignmentPair UDisplayClusterConfiguratorBaseNode::GetResizeAlignments(const FVector2D& InSizeChange, const FNodeAlignmentParams& AlignmentParams, const TSet<UDisplayClusterConfiguratorBaseNode*>& NodesToIgnore) const
{
	const FNodeAlignmentAnchors ExpandedAnchors = GetNodeAlignmentAnchors().ExpandBy(InSizeChange);

	// When resizing, we want to exclude top, left, and center anchors from being aligned.
	FNodeAlignmentParams NewParams(AlignmentParams);

	NewParams.AnchorsToIgnore.Add(EAlignmentAnchor::Center);
	NewParams.AnchorsToIgnore.Add(EAlignmentAnchor::Top);
	NewParams.AnchorsToIgnore.Add(EAlignmentAnchor::Left);

	return GetAlignments(ExpandedAnchors, NewParams, NodesToIgnore);
}

FNodeAlignmentPair UDisplayClusterConfiguratorBaseNode::GetAlignments(const FNodeAlignmentAnchors& TransformedAnchors, const FNodeAlignmentParams& AlignmentParams, const TSet<UDisplayClusterConfiguratorBaseNode*>& NodesToIgnore) const
{
	FDisplayClusterConfiguratorNodeAlignmentHelper AlignmentHelper(this, TransformedAnchors, AlignmentParams);

	// Align to parent, but only with equivalent edges, not with adjacent edges.
	if (CanAlignWithParent() && Parent.IsValid())
	{
		AlignmentHelper.AddAlignmentsToParent(Parent.Get());
	}

	for (const UDisplayClusterConfiguratorBaseNode* SiblingNode : Parent->GetChildren())
	{
		if (SiblingNode == this)
		{
			continue;
		}

		// Skip siblings that are not visible or enabled, as they are not visible in the graph editor and is confusing when a node snap aligns to an invisible node.
		if (!SiblingNode->IsNodeEnabled() || !SiblingNode->IsNodeVisible() || NodesToIgnore.Contains(SiblingNode))
		{
			continue;
		}

		AlignmentHelper.AddAlignmentsToNode(SiblingNode);
	}

	return AlignmentHelper.GetAlignments();
}

float UDisplayClusterConfiguratorBaseNode::GetViewScale() const
{
	TSharedPtr<FDisplayClusterConfiguratorBlueprintEditor> Toolkit = ToolkitPtr.Pin();
	check(Toolkit.IsValid());

	TSharedRef<IDisplayClusterConfiguratorViewOutputMapping> OutputMapping = Toolkit->GetViewOutputMapping();

	return OutputMapping->GetOutputMappingSettings().ViewScale;
}

void UDisplayClusterConfiguratorBaseNode::CleanupChildrenNodes()
{
	// In rare cases, the child pointer may be in the process of being killed (such as if the user undoes an add operation)
	// so remove any dead or dying children nodes.
	Children.RemoveAll([](UDisplayClusterConfiguratorBaseNode* Child)
	{
		return !IsValid(Child);
	});
}
