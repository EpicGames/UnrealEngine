// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Views/OutputMapping/IDisplayClusterConfiguratorOutputMappingItem.h"
#include "EdGraph/EdGraphNode.h"

//#include "Views/OutputMapping/Alignment/DisplayClusterConfiguratorNodeAlignmentHelper.h"

#include "DisplayClusterConfiguratorBaseNode.generated.h"

class FDisplayClusterConfiguratorBlueprintEditor;
class IDisplayClusterConfiguratorOutputMappingSlot;
class IDisplayClusterConfiguratorTreeItem;

struct FNodeAlignmentAnchors;
struct FNodeAlignmentPair;
struct FNodeAlignmentParams;

namespace DisplayClusterConfiguratorGraphLayers
{
	// Layer size is set to 1000 because the graph nodes rely on SOverlay widgets for rendering, and every slot in an overlay
	// increments the render layer by 10. Using a layer size smaller than 1000 can easily cause elements of nodes to bleed into 
	// layers above them. For a similar reason, the ZIndexSize must be big enough to give every node room in the layer to render 
	// all of its components without overlapping other nodes.
	const int32 BaseLayerIndex = 0;
	const int32 LayerSize = 1000;
	const int32 ZIndexSize = 30;

	// There are five default layers:
	// - Canvas
	// - Host
	// - Window
	// - Viewport
	// - Auxiliary
	// which are computed by the nodes themselves using their position in the hierarchy (see UDisplayClusterConfiguratorBaseNode::GetNodeLayer)

	// The auxiliary layer is meant for widgets that are always displayed on top of the viewport layer, such as window titlebars. 
	const int32 AuxiliaryLayerIndex = BaseLayerIndex + LayerSize * 4;

	// When a node is selected, a fixed value is added to its layer index to elevate it and its children above the normal layers.
	const int32 SelectedLayerIndex = BaseLayerIndex + LayerSize * 5;

	// The ornament layer is meant for widgets such as the resize handler that should sit above everything in the layer stack, even selected items.
	const int32 OrnamentLayerIndex = SelectedLayerIndex + LayerSize * 5;
}

UCLASS(MinimalAPI)
class UDisplayClusterConfiguratorBaseNode
	: public UEdGraphNode 
	, public IDisplayClusterConfiguratorOutputMappingItem
{
	GENERATED_BODY()

public:
	virtual void Initialize(const FString& InNodeName, int32 InNodeZIndex, UObject* InObject, const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& InToolkit);
	virtual void Cleanup() { }

	//~ Begin UObject Interface
	virtual void BeginDestroy() override;

#if WITH_EDITOR
	virtual void PostEditUndo() override;
#endif
	//~ End UObject Interface

	//~ Begin UEdGraphNode Interface
	virtual void PostPlacedNewNode() override;
	virtual void ResizeNode(const FVector2f& NewSize) override;
	virtual bool CanDuplicateNode() const override { return false; }
	virtual bool CanUserDeleteNode() const override { return false; }
	//~ End UEdGraphNode Interface

	//~ Begin IDisplayClusterConfiguratorItem Interface
	virtual void OnSelection() override;
	virtual UObject* GetObject() const override { return ObjectToEdit.Get(); }
	virtual bool IsSelected() override;
	//~ End IDisplayClusterConfiguratorItem Interface

	//~ Begin IDisplayClusterConfiguratorOutputMappingItem Interface
	virtual const FString& GetNodeName() const override;
	//~ End IDisplayClusterConfiguratorOutputMappingItem Interface

	virtual void SetParent(UDisplayClusterConfiguratorBaseNode* InParent);
	virtual UDisplayClusterConfiguratorBaseNode* GetParent() const;
	virtual void AddChild(UDisplayClusterConfiguratorBaseNode* InChild);
	virtual const TArray<UDisplayClusterConfiguratorBaseNode*>& GetChildren() const;

	virtual FVector2D TransformPointToLocal(FVector2D GlobalPosition) const;
	virtual FVector2D TransformPointToGlobal(FVector2D LocalPosition) const;
	virtual FVector2D TransformSizeToLocal(FVector2D GlobalSize) const;
	virtual FVector2D TransformSizeToGlobal(FVector2D LocalSize) const;
	virtual FVector2D GetTranslationOffset() const { return FVector2D::ZeroVector; }

	virtual FBox2D GetNodeBounds(bool bAsParent = false) const;
	virtual FVector2D GetNodePosition() const;
	virtual FVector2D GetNodeLocalPosition() const;
	virtual FVector2D GetNodeSize() const;
	virtual FVector2D GetNodeLocalSize() const;
	virtual FNodeAlignmentAnchors GetNodeAlignmentAnchors(bool bAsParent = false) const;
	virtual int32 GetNodeLayer(const TSet<UObject*>& SelectionSet, bool bIncludeZIndex = true) const;
	virtual int32 GetAuxiliaryLayer(const TSet<UObject*>& SelectionSet) const;
	virtual bool IsNodeVisible() const { return true; }
	virtual bool IsNodeEnabled() const { return true; }
	virtual bool IsNodeUnlocked() const { return true; }
	virtual bool IsNodeAutoPositioned() const { return false; }
	virtual bool IsNodeAutosized() const { return false; }
	virtual bool CanNodeOverlapSiblings() const { return true; }
	virtual bool CanNodeExceedParentBounds() const { return true; }
	virtual bool CanNodeHaveNegativePosition() const { return true; }
	virtual bool CanNodeEncroachChildBounds() const { return true; }

	virtual void FillParent(bool bRepositionNode = true);
	virtual void SizeToChildren(bool bRepositionNode = true);

	virtual FBox2D GetChildBounds() const;
	virtual FBox2D GetDescendentBounds() const;
	virtual bool IsOutsideParent() const;
	virtual bool IsOutsideParentBoundary() const;
	virtual void UpdateChildNodes();

	virtual void TickPosition() { }

	virtual bool IsUserInteractingWithNode(bool bCheckDescendents = false) const;
	virtual bool IsUserDirectlyInteractingWithNode() const { return bIsDirectInteraction; }
	virtual void MarkUserInteractingWithNode(bool bInIsDirectInteraction) { bIsUserInteractingWithNode = true; bIsDirectInteraction = bInIsDirectInteraction; }
	virtual void ClearUserInteractingWithNode() { bIsUserInteractingWithNode = bIsDirectInteraction = false;  }

	virtual void UpdateNode();
	virtual void UpdateObject();
	virtual void DeleteObject() { }
	virtual bool IsObjectValid() const;

	virtual void OnNodeAligned(bool bUpdateChildren = false);

	virtual bool WillOverlap(UDisplayClusterConfiguratorBaseNode* InNode, const FVector2D& InDesiredOffset = FVector2D::ZeroVector, const FVector2D& InDesiredSizeChange = FVector2D::ZeroVector) const;
	virtual FVector2D FindNonOverlappingOffset(UDisplayClusterConfiguratorBaseNode* InNode, const FVector2D& InDesiredOffset) const;
	virtual FVector2D FindNonOverlappingSize(UDisplayClusterConfiguratorBaseNode* InNode, const FVector2D& InDesiredSize, const bool bFixedApsectRatio) const;

	virtual FVector2D FindNonOverlappingOffsetFromParent(const FVector2D& InDesiredOffset, const TSet<UDisplayClusterConfiguratorBaseNode*>& NodesToIgnore = TSet<UDisplayClusterConfiguratorBaseNode*>());
	virtual FVector2D FindBoundedOffsetFromParent(const FVector2D& InDesiredOffset);
	virtual FVector2D FindNonOverlappingSizeFromParent(const FVector2D& InDesiredSize, const bool bFixedApsectRatio);
	virtual FVector2D FindBoundedSizeFromParent(const FVector2D& InDesiredSize, const bool bFixedApsectRatio);
	virtual FVector2D FindBoundedSizeFromChildren(const FVector2D& InDesiredSize, const bool bFixedApsectRatio);

	virtual FNodeAlignmentPair GetTranslationAlignments(const FVector2D& InOffset, const FNodeAlignmentParams& AlignmentParams, const TSet<UDisplayClusterConfiguratorBaseNode*>& NodesToIgnore = TSet<UDisplayClusterConfiguratorBaseNode*>()) const;
	virtual FNodeAlignmentPair GetResizeAlignments(const FVector2D& InSizeChange, const FNodeAlignmentParams& AlignmentParams, const TSet<UDisplayClusterConfiguratorBaseNode*>& NodesToIgnore = TSet<UDisplayClusterConfiguratorBaseNode*>()) const;

protected:
	virtual bool CanAlignWithParent() const { return false; }
	virtual FNodeAlignmentPair GetAlignments(const FNodeAlignmentAnchors& TransformedAnchors, const FNodeAlignmentParams& AlignmentParams, const TSet<UDisplayClusterConfiguratorBaseNode*>& NodesToIgnore) const;

	virtual void WriteNodeStateToObject() { }
	virtual void ReadNodeStateFromObject() { }

	virtual float GetViewScale() const;

	/** Remove invalid nodes. */
	void CleanupChildrenNodes();
	
public:
	template<class TObjectType>
	TObjectType* GetObjectChecked() const
	{
		TObjectType* CastedObject = Cast<TObjectType>(ObjectToEdit.Get());
		check(CastedObject);
		return CastedObject;
	}

	template<class TParentType>
	TParentType* GetParentChecked() const
	{
		TParentType* CastedParent = Cast<TParentType>(Parent.Get());
		check(CastedParent);
		return CastedParent;
	}

protected:
	TWeakObjectPtr<UObject> ObjectToEdit;
	TWeakPtr<FDisplayClusterConfiguratorBlueprintEditor> ToolkitPtr;

	TWeakObjectPtr<UDisplayClusterConfiguratorBaseNode> Parent;

	UPROPERTY(Transient, NonTransactional)
	TArray<TObjectPtr<UDisplayClusterConfiguratorBaseNode>> Children;

	FString NodeName;
	int32 NodeZIndex;

	bool bIsUserInteractingWithNode;
	bool bIsDirectInteraction;
};
