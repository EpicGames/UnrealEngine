// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/OutputMapping/EdNodes/DisplayClusterConfiguratorCanvasNode.h"

#include "DisplayClusterConfiguratorBlueprintEditor.h"
#include "Views/OutputMapping/IDisplayClusterConfiguratorViewOutputMapping.h"
#include "Views/OutputMapping/EdNodes/DisplayClusterConfiguratorHostNode.h"
#include "Views/OutputMapping/EdNodes/DisplayClusterConfiguratorWindowNode.h"
#include "Views/OutputMapping/GraphNodes/SDisplayClusterConfiguratorCanvasNode.h"
#include "Views/OutputMapping/Alignment/DisplayClusterConfiguratorHostNodeArrangementHelper.h"

#include "DisplayClusterConfigurationTypes.h"

TSharedPtr<SGraphNode> UDisplayClusterConfiguratorCanvasNode::CreateVisualWidget()
{
	return SNew(SDisplayClusterConfiguratorCanvasNode, this, ToolkitPtr.Pin().ToSharedRef());;
}

void UDisplayClusterConfiguratorCanvasNode::TickPosition()
{
	// Resize canvas slot
	FBox2D CanvasBounds = GetChildBounds();

	// If all of the canvas's direct children are zero sized, use the bounds of its indirect descendents instead
	if (!CanvasBounds.bIsValid)
	{
		CanvasBounds = GetDescendentBounds();
	}

	NodePosX = CanvasBounds.Min.X;
	NodePosY = CanvasBounds.Min.Y;
	NodeWidth = CanvasBounds.Max.X - CanvasBounds.Min.X;
	NodeHeight = CanvasBounds.Max.Y - CanvasBounds.Min.Y;

	Resolution = FVector2D::ZeroVector;

	for (UDisplayClusterConfiguratorBaseNode* Child : Children)
	{
		Resolution += Child->GetNodeLocalSize();
	}

	// Now position the child host nodes
	TArray<UDisplayClusterConfiguratorHostNode*> ChildHosts;
	for (UDisplayClusterConfiguratorBaseNode* Child : Children)
	{
		if (UDisplayClusterConfiguratorHostNode* HostChild = Cast<UDisplayClusterConfiguratorHostNode>(Child))
		{
			ChildHosts.Add(HostChild);
		}
	}

	TSharedPtr<FDisplayClusterConfiguratorBlueprintEditor> Toolkit = ToolkitPtr.Pin();
	check(Toolkit.IsValid());

	TSharedRef<IDisplayClusterConfiguratorViewOutputMapping> OutputMapping = Toolkit->GetViewOutputMapping();
	const FHostNodeArrangementSettings& HostArrangementSettings = OutputMapping->GetHostArrangementSettings();

	FDisplayClusterConfiguratorHostNodeArrangementHelper ArrangementHelper(HostArrangementSettings);
	ArrangementHelper.PlaceNodes(ChildHosts);
}