// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Views/OutputMapping/EdNodes/DisplayClusterConfiguratorBaseNode.h"

#include "DisplayClusterConfiguratorCanvasNode.generated.h"

class UDisplayClusterConfigurationCluster;
class UDisplayClusterConfiguratorWindowNode;

UCLASS()
class UDisplayClusterConfiguratorCanvasNode final
	: public UDisplayClusterConfiguratorBaseNode
{
	GENERATED_BODY()

public:
	//~ Begin EdGraphNode Interface
	virtual TSharedPtr<SGraphNode> CreateVisualWidget() override;
	//~ End EdGraphNode Interface

	//~ Begin UDisplayClusterConfiguratorBaseNode Interface
	virtual void TickPosition() override;
	virtual bool IsNodeAutoPositioned() const { return true; }
	virtual bool IsNodeAutosized() const override { return true; }
	//~ End UDisplayClusterConfiguratorBaseNode Interface

	const FVector2D& GetResolution() const { return Resolution; }

private:
	FVector2D Resolution;
};
