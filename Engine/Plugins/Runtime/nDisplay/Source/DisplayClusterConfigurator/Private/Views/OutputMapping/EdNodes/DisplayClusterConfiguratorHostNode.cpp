// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfiguratorHostNode.h"

#include "DisplayClusterConfiguratorBlueprintEditor.h"
#include "DisplayClusterConfigurationTypes.h"
#include "ClusterConfiguration/DisplayClusterConfiguratorClusterUtils.h"
#include "Views/OutputMapping/Alignment/DisplayClusterConfiguratorNodeAlignmentHelper.h"
#include "Views/OutputMapping/IDisplayClusterConfiguratorViewOutputMapping.h"
#include "Views/TreeViews/IDisplayClusterConfiguratorViewTree.h"
#include "Views/TreeViews/IDisplayClusterConfiguratorTreeItem.h"

#include "Views/OutputMapping/GraphNodes/SDisplayClusterConfiguratorHostNode.h"

const FMargin UDisplayClusterConfiguratorHostNode::VisualMargin = FMargin(25, 110, 25, 25);
const float UDisplayClusterConfiguratorHostNode::DefaultSpaceBetweenHosts = 40.0f;
const float UDisplayClusterConfiguratorHostNode::HorizontalSpanBetweenHosts = VisualMargin.Left + VisualMargin.Right + DefaultSpaceBetweenHosts;
const float UDisplayClusterConfiguratorHostNode::VerticalSpanBetweenHosts = VisualMargin.Top + VisualMargin.Bottom + DefaultSpaceBetweenHosts;

void UDisplayClusterConfiguratorHostNode::Initialize(const FString& InNodeName, int32 InNodeZIndex, UObject* InObject, const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& InToolkit)
{
	UDisplayClusterConfiguratorBaseNode::Initialize(InNodeName, InNodeZIndex, InObject, InToolkit);

	UDisplayClusterConfigurationHostDisplayData* HostDisplayData = GetObjectChecked<UDisplayClusterConfigurationHostDisplayData>();
	HostDisplayData->OnPostEditChangeChainProperty.Add(UDisplayClusterConfigurationHostDisplayData::FOnPostEditChangeChainProperty::FDelegate::CreateUObject(this, &UDisplayClusterConfiguratorHostNode::OnPostEditChangeChainProperty));
}

void UDisplayClusterConfiguratorHostNode::Cleanup()
{
	if (ObjectToEdit.IsValid())
	{
		UDisplayClusterConfigurationHostDisplayData* HostDisplayData = GetObjectChecked<UDisplayClusterConfigurationHostDisplayData>();
		HostDisplayData->OnPostEditChangeChainProperty.RemoveAll(this);
	}
}

TSharedPtr<SGraphNode> UDisplayClusterConfiguratorHostNode::CreateVisualWidget()
{
	return SNew(SDisplayClusterConfiguratorHostNode, this, ToolkitPtr.Pin().ToSharedRef())
		.BorderThickness(VisualMargin);
}

FLinearColor UDisplayClusterConfiguratorHostNode::GetHostColor() const
{
	if (!IsObjectValid())
	{
		return FLinearColor();
	}

	UDisplayClusterConfigurationHostDisplayData* HostDisplayData = GetObjectChecked<UDisplayClusterConfigurationHostDisplayData>();
	return HostDisplayData->Color;
}

FText UDisplayClusterConfiguratorHostNode::GetHostName() const
{
	if (!IsObjectValid())
	{
		return FText::GetEmpty();
	}

	UDisplayClusterConfigurationHostDisplayData* HostDisplayData = GetObjectChecked<UDisplayClusterConfigurationHostDisplayData>();
	return HostDisplayData->HostName;
}

FVector2D UDisplayClusterConfiguratorHostNode::GetHostOrigin(bool bInGlobalCoordinates) const
{
	if (!IsObjectValid())
	{
		return FVector2D::ZeroVector;
	}

	UDisplayClusterConfigurationHostDisplayData* HostDisplayData = GetObjectChecked<UDisplayClusterConfigurationHostDisplayData>();
	float Scale = bInGlobalCoordinates ? GetViewScale() : 1.0f;

	return HostDisplayData->Origin * Scale;
}

void UDisplayClusterConfiguratorHostNode::SetHostOrigin(const FVector2D& NewOrigin, bool bInGlobalCoordinates)
{
	if (!IsObjectValid())
	{
		return;
	}

	UDisplayClusterConfigurationHostDisplayData* HostDisplayData = GetObjectChecked<UDisplayClusterConfigurationHostDisplayData>();

	float Scale = bInGlobalCoordinates ? GetViewScale() : 1.0f;

	HostDisplayData->Modify();
	HostDisplayData->Origin = NewOrigin / Scale;
	HostDisplayData->MarkPackageDirty();

	UpdateChildNodes();
}

bool UDisplayClusterConfiguratorHostNode::CanUserMoveNode() const
{
	if (!IsObjectValid())
	{
		return false;
	}

	UDisplayClusterConfigurationHostDisplayData* HostDisplayData = GetObjectChecked<UDisplayClusterConfigurationHostDisplayData>();
	return HostDisplayData->bAllowManualPlacement;
}

bool UDisplayClusterConfiguratorHostNode::CanUserResizeNode() const
{
	if (!IsObjectValid())
	{
		return false;
	}

	UDisplayClusterConfigurationHostDisplayData* HostDisplayData = GetObjectChecked<UDisplayClusterConfigurationHostDisplayData>();
	return HostDisplayData->bAllowManualSizing;
}

FVector2D UDisplayClusterConfiguratorHostNode::TransformPointToLocal(FVector2D GlobalPosition) const
{
	// Overridden to ignore the parent position in the computation, as the parent Canvas node's position includes the host's visual margin
	// which causes incorrect transforms.
	const float ViewScale = GetViewScale();
	return GlobalPosition / ViewScale;
}

FVector2D UDisplayClusterConfiguratorHostNode::TransformPointToGlobal(FVector2D LocalPosition) const
{
	// Overridden to ignore the parent position in the computation, as the parent Canvas node's position includes the host's visual margin
	// which causes incorrect transforms.
	const float ViewScale = GetViewScale();
	return LocalPosition * ViewScale;
}

FBox2D UDisplayClusterConfiguratorHostNode::GetNodeBounds(bool bAsParent) const
{
	// Inflate the bounds by the visual margin to prevent visual overlap in the graph editor
	FBox2D Bounds = Super::GetNodeBounds();

	// If the bounds aren't being used by a child node, increase the bounds to account for the visual margin around the
	// edge of the host node
	if (!bAsParent)
	{
		Bounds.Min -= FVector2D(VisualMargin.Left, VisualMargin.Top);
		Bounds.Max += FVector2D(VisualMargin.Right, VisualMargin.Bottom);
	}

	return Bounds;
}

FNodeAlignmentAnchors UDisplayClusterConfiguratorHostNode::GetNodeAlignmentAnchors(bool bAsParent) const
{
	FNodeAlignmentAnchors Anchors = Super::GetNodeAlignmentAnchors();

	if (!bAsParent)
	{
		Anchors.Top -= FVector2D(0, VisualMargin.Top);
		Anchors.Bottom += FVector2D(0, VisualMargin.Bottom);
		Anchors.Left -= FVector2D(VisualMargin.Left, 0);
		Anchors.Right += FVector2D(VisualMargin.Right, 0);
	}

	return Anchors;
}
bool UDisplayClusterConfiguratorHostNode::IsNodeVisible() const
{
	if (!IsObjectValid())
	{
		return false;
	}

	UDisplayClusterConfigurationHostDisplayData* HostDisplayData = GetObjectChecked<UDisplayClusterConfigurationHostDisplayData>();

	if (HostDisplayData->bIsVisible)
	{
		return true;
	}

	// If all children of the host node are invisible, make the host node invisible as well.
	bool bIsChildVisible = false;
	for (UDisplayClusterConfiguratorBaseNode* Child : Children)
	{
		if (Child->IsNodeVisible())
		{
			bIsChildVisible = true;
			break;
		}
	}

	return bIsChildVisible;
}

bool UDisplayClusterConfiguratorHostNode::IsNodeUnlocked() const
{
	if (!IsObjectValid())
	{
		return false;
	}

	UDisplayClusterConfigurationHostDisplayData* HostDisplayData = GetObjectChecked<UDisplayClusterConfigurationHostDisplayData>();
	return HostDisplayData->bIsUnlocked;
}

void UDisplayClusterConfiguratorHostNode::DeleteObject()
{
	FString HostStr = GetNodeName();

	UDisplayClusterConfigurationHostDisplayData* HostDisplayData = GetObjectChecked<UDisplayClusterConfigurationHostDisplayData>();
	if (UDisplayClusterConfigurationCluster* Cluster = Cast<UDisplayClusterConfigurationCluster>(HostDisplayData->GetOuter()))
	{
		UE::DisplayClusterConfiguratorClusterUtils::RemoveHost(Cluster, HostStr);
	}
}

void UDisplayClusterConfiguratorHostNode::WriteNodeStateToObject()
{
	UDisplayClusterConfigurationHostDisplayData* HostDisplayData = GetObjectChecked<UDisplayClusterConfigurationHostDisplayData>();
	const FVector2D LocalPosition = TransformPointToLocal(GetNodePosition());
	const FVector2D LocalSize = TransformSizeToLocal(GetNodeSize());

	HostDisplayData->Position = LocalPosition;
	HostDisplayData->HostResolution = LocalSize;
}

void UDisplayClusterConfiguratorHostNode::ReadNodeStateFromObject()
{
	UDisplayClusterConfigurationHostDisplayData* HostDisplayData = GetObjectChecked<UDisplayClusterConfigurationHostDisplayData>();
	const FVector2D GlobalPosition = TransformPointToGlobal(HostDisplayData->Position);
	const FVector2D GlobalSize = TransformSizeToGlobal(HostDisplayData->HostResolution);

	NodePosX = GlobalPosition.X;
	NodePosY = GlobalPosition.Y;
	NodeWidth = GlobalSize.X;
	NodeHeight = GlobalSize.Y;
}

void UDisplayClusterConfiguratorHostNode::OnPostEditChangeChainProperty(const FPropertyChangedChainEvent& PropertyChangedEvent)
{
	// If the pointer to the blueprint editor is no longer valid, its likely that the editor this node was created for was closed,
	// and this node is orphaned and will eventually be GCed.
	if (!ToolkitPtr.IsValid())
	{
		return;
	}

	// If the object is no longer valid, don't attempt to sync properties
	if (!IsObjectValid())
	{
		return;
	}

	UDisplayClusterConfigurationHostDisplayData* HostDisplayData = GetObjectChecked<UDisplayClusterConfigurationHostDisplayData>();

	const FName& PropertyName = PropertyChangedEvent.PropertyChain.GetActiveMemberNode()->GetValue()->GetFName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationHostDisplayData, Position))
	{
		Modify();

		// Change slots and children position, config object already updated
		const FVector2D GlobalPosition = TransformPointToGlobal(HostDisplayData->Position);
		NodePosX = GlobalPosition.X;
		NodePosY = GlobalPosition.Y;

		UpdateChildNodes();
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationHostDisplayData, HostResolution))
	{
		Modify();

		// Change node slot size, config object already updated 
		const FVector2D GlobalSize = TransformSizeToGlobal(HostDisplayData->HostResolution);
		NodeWidth = GlobalSize.X;
		NodeHeight = GlobalSize.Y;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationHostDisplayData, bAllowManualPlacement))
	{
		// If the user has enabled manual placement, we need to update the node's position to match the configurated size
		if (HostDisplayData->bAllowManualPlacement)
		{
			// Update the configuration with the current node position and size, which will have been computed in the last tick.
			UpdateObject();
		}
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationHostDisplayData, bAllowManualSizing))
	{
		// If the user has enabled manual sizing, we need to update the node's size to match the configurated size
		if (HostDisplayData->bAllowManualSizing)
		{
			// Update the configuration with the current node position and size, which will have been computed in the last tick.
			UpdateObject();
		}
	}
}