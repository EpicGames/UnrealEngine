// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Views/OutputMapping/EdNodes/DisplayClusterConfiguratorBaseNode.h"
#include "Layout/Margin.h"

#include "DisplayClusterConfiguratorHostNode.generated.h"

class UDisplayClusterConfigurationHostDisplayData;

UCLASS()
class UDisplayClusterConfiguratorHostNode final
	: public UDisplayClusterConfiguratorBaseNode
{
	GENERATED_BODY()

public:
	virtual void Initialize(const FString& InNodeName, int32 InNodeZIndex, UObject* InObject, const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& InToolkit) override;
	virtual void Cleanup() override;

	//~ Begin EdGraphNode Interface
	virtual TSharedPtr<SGraphNode> CreateVisualWidget() override;
	virtual bool CanUserDeleteNode() const override { return true; }
	//~ End EdGraphNode Interface

	FLinearColor GetHostColor() const;
	FText GetHostName() const;
	FVector2D GetHostOrigin(bool bInGlobalCoordinates = false) const;
	void SetHostOrigin(const FVector2D& NewOrigin, bool bInGlobalCoordinates = false);
	bool CanUserMoveNode() const;
	bool CanUserResizeNode() const;

	//~ Begin UDisplayClusterConfiguratorBaseNode Interface
	virtual FVector2D TransformPointToLocal(FVector2D GlobalPosition) const override;
	virtual FVector2D TransformPointToGlobal(FVector2D LocalPosition) const override;
	virtual FVector2D GetTranslationOffset() const override { return GetHostOrigin(true); }

	virtual FBox2D GetNodeBounds(bool bAsParent = false) const override;
	virtual FNodeAlignmentAnchors GetNodeAlignmentAnchors(bool bAsParent = false) const override;
	virtual bool IsNodeVisible() const override;
	virtual bool IsNodeUnlocked() const override;
	virtual bool IsNodeAutoPositioned() const { return !CanUserMoveNode(); }
	virtual bool IsNodeAutosized() const override { return !CanUserResizeNode(); }
	virtual bool CanNodeOverlapSiblings() const override { return false; }
	virtual bool CanNodeEncroachChildBounds() const { return false; }

	virtual void DeleteObject() override;

protected:
	virtual void WriteNodeStateToObject() override;
	virtual void ReadNodeStateFromObject() override;
	//~ End UDisplayClusterConfiguratorBaseNode Interface

private:
	void OnPostEditChangeChainProperty(const FPropertyChangedChainEvent& PropertyChangedEvent);

public:
	/** An extra padding around the node to display visual elements like borders */
	static const FMargin VisualMargin;

	/** When host are being programmatically positioned, this is the amount of space to place between hosts */
	static const float DefaultSpaceBetweenHosts;

	/** Total amount of horizontal space between adjacent hosts, including the visual margins of each */
	static const float HorizontalSpanBetweenHosts;

	/** Total amount of vertical space between adjacent hosts, including the visual margins of each */
	static const float VerticalSpanBetweenHosts;
};