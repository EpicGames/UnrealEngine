// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/OutputMapping/EdNodes/DisplayClusterConfiguratorViewportNode.h"

#include "ClusterConfiguration/DisplayClusterConfiguratorClusterUtils.h"
#include "ClusterConfiguration/ViewModels/DisplayClusterConfiguratorViewportViewModel.h"
#include "Views/OutputMapping/EdNodes/DisplayClusterConfiguratorWindowNode.h"
#include "Views/OutputMapping/GraphNodes/SDisplayClusterConfiguratorViewportNode.h"
#include "Views/TreeViews/IDisplayClusterConfiguratorViewTree.h"
#include "Views/TreeViews/IDisplayClusterConfiguratorTreeItem.h"

#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterConfigurationTypes_ViewportRemap.h"
#include "DisplayClusterConfiguratorBlueprintEditor.h"
#include "DisplayClusterRootActor.h"

#include "Render/Viewport/IDisplayClusterViewportPreview.h"
#include "Engine/TextureRenderTarget2D.h"


void UDisplayClusterConfiguratorViewportNode::Initialize(const FString& InNodeName, int32 InNodeZIndex, UObject* InObject, const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& InToolkit)
{
	UDisplayClusterConfiguratorBaseNode::Initialize(InNodeName, InNodeZIndex, InObject, InToolkit);

	UDisplayClusterConfigurationViewport* CfgViewport = GetObjectChecked<UDisplayClusterConfigurationViewport>();
	CfgViewport->OnPostEditChangeChainProperty.Add(UDisplayClusterConfigurationViewport::FOnPostEditChangeChainProperty::FDelegate::CreateUObject(this, &UDisplayClusterConfiguratorViewportNode::OnPostEditChangeChainProperty));

	ViewportVM = MakeShareable(new FDisplayClusterConfiguratorViewportViewModel(CfgViewport));
}

void UDisplayClusterConfiguratorViewportNode::Cleanup()
{
	if (ObjectToEdit.IsValid())
	{
		UDisplayClusterConfigurationViewport* CfgViewport = GetObjectChecked<UDisplayClusterConfigurationViewport>();
		CfgViewport->OnPostEditChangeChainProperty.RemoveAll(this);
	}
}

TSharedPtr<SGraphNode> UDisplayClusterConfiguratorViewportNode::CreateVisualWidget()
{
	return SNew(SDisplayClusterConfiguratorViewportNode, this, ToolkitPtr.Pin().ToSharedRef());
}

void UDisplayClusterConfiguratorViewportNode::ResizeNode(const FVector2f& NewSize)
{
	// If the viewport is rotated, there are some sizes the node (which represents the bounding box of the rotated viewport)
	// cannot take on, because such sizes cause the viewport to have negative width or height. Find an appropriate size for the
	// node by computing the possible viewport size, and clamping it to valid values, and recompute a corrected node size from the
	// clamped viewport size, to ensure the viewport size never becomes negative

	const FDisplayClusterConfigurationViewport_RemapData& CfgRemap = GetCfgViewportRemap();
	const FMatrix2x2 RotMat = GetSizeRotationMatrix(CfgRemap.Angle);

	FVector2f CorrectedNewSize = NewSize;
	if (IsRotationAngleDegenerate(CfgRemap.Angle) || RotMat.Determinant() == 0.0f)
	{
		// The bounding box for a 45 degree rotation is always a square, so correct the size to the smallest square.
		CorrectedNewSize = FVector2f(NewSize.GetMin());
	}
	else
	{
		const FMatrix2x2 InvRotMat = RotMat.Inverse();
		const FVector2f UnrotatedSize = InvRotMat.TransformVector(NewSize);
		const FVector2f ClampedSize = UnrotatedSize.ClampAxes(UDisplayClusterConfigurationViewport::ViewportMinimumSize, UDisplayClusterConfigurationViewport::ViewportMaximumSize);

		CorrectedNewSize = RotMat.TransformVector(ClampedSize);
	}

	UDisplayClusterConfiguratorBaseNode::ResizeNode(CorrectedNewSize);
}

bool UDisplayClusterConfiguratorViewportNode::IsNodeVisible() const
{
	if (!IsObjectValid())
	{
		return false;
	}

	UDisplayClusterConfigurationViewport* Viewport = GetObjectChecked<UDisplayClusterConfigurationViewport>();
	return Viewport->bIsVisible;
}

bool UDisplayClusterConfiguratorViewportNode::IsNodeUnlocked() const
{
	if (!IsObjectValid())
	{
		return false;
	}

	UDisplayClusterConfigurationViewport* Viewport = GetObjectChecked<UDisplayClusterConfigurationViewport>();
	return Viewport->bIsUnlocked;
}

void UDisplayClusterConfiguratorViewportNode::DeleteObject()
{
	if (!IsObjectValid())
	{
		return;
	}

	UDisplayClusterConfigurationViewport* Viewport = GetObjectChecked<UDisplayClusterConfigurationViewport>();
	UE::DisplayClusterConfiguratorClusterUtils::RemoveViewportFromClusterNode(Viewport);
}

void UDisplayClusterConfiguratorViewportNode::WriteNodeStateToObject()
{
	UDisplayClusterConfigurationViewport* CfgViewport = GetObjectChecked<UDisplayClusterConfigurationViewport>();
	const FVector2D LocalPosition = GetNodeLocalPosition();
	const FVector2D LocalSize = TransformSizeToLocal(GetNodeSize());
	
	// The node's width and height refer to the absolute horizontal and vertical size on the graph editor, even if the viewport is rotated,
	// so we must "unrotate" the size when writing it back to the viewport configuration
	const FDisplayClusterConfigurationViewport_RemapData& CfgRemap = GetCfgViewportRemap();
	const FMatrix2x2 RotMat = GetSizeRotationMatrix(CfgRemap.Angle);

	FVector2D UnrotatedSize;
	if (IsRotationAngleDegenerate(CfgRemap.Angle) || RotMat.Determinant() == 0.0f)
	{
		// At a rotation of 45 degrees (or 135 degrees, or 225 degrees, etc), the determinant will be zero, indicating
		// that the system is degerate; all 45 degree rotated rectangles will give a square bounding box, so we can't know
		// which rectangle to use just given the bounding box, we need another constraint. We can use the aspect ratio of 
		// the viewport, keeping it constant while the bounding box is resized, to determine the viewport size.
		constexpr float TwoOverRootTwo = 2.0f * UE_INV_SQRT_2;

		UnrotatedSize.Y = TwoOverRootTwo * LocalSize.X / (1 + ViewportAspectRatio);
		UnrotatedSize.X = UnrotatedSize.Y * ViewportAspectRatio;
	}
	else
	{
		const FMatrix2x2 InvRotMat = RotMat.Inverse();
		UnrotatedSize = InvRotMat.TransformVector(LocalSize);
	}

	FDisplayClusterConfigurationRectangle NewRegion(LocalPosition.X, LocalPosition.Y, UnrotatedSize.X, UnrotatedSize.Y);
	ViewportVM->SetRegion(NewRegion);
}

void UDisplayClusterConfiguratorViewportNode::ReadNodeStateFromObject()
{
	const FDisplayClusterConfigurationRectangle& CfgRegion = GetCfgViewportRegion();
	const FDisplayClusterConfigurationViewport_RemapData& CfgRemap = GetCfgViewportRemap();

	const FVector2D GlobalPosition = TransformPointToGlobal(FVector2D(CfgRegion.X, CfgRegion.Y));
	const FVector2D GlobalSize = TransformSizeToGlobal(FVector2D(CfgRegion.W, CfgRegion.H));

	const FMatrix2x2 RotMat = GetSizeRotationMatrix(CfgRemap.Angle);
	const FVector2D RotatedSize = RotMat.TransformVector(GlobalSize);

	NodePosX = GlobalPosition.X;
	NodePosY = GlobalPosition.Y;
	NodeWidth = FMath::RoundToInt(RotatedSize.X);
	NodeHeight = FMath::RoundToInt(RotatedSize.Y);
	ViewportAspectRatio = (float)CfgRegion.W / (float)CfgRegion.H;
}

const FDisplayClusterConfigurationRectangle& UDisplayClusterConfiguratorViewportNode::GetCfgViewportRegion() const
{
	UDisplayClusterConfigurationViewport* CfgViewport = GetObjectChecked<UDisplayClusterConfigurationViewport>();
	return CfgViewport->Region;
}


const FDisplayClusterConfigurationViewport_RemapData& UDisplayClusterConfiguratorViewportNode::GetCfgViewportRemap() const
{
	UDisplayClusterConfigurationViewport* CfgViewport = GetObjectChecked<UDisplayClusterConfigurationViewport>();
	return CfgViewport->ViewportRemap.BaseRemap;
}

bool UDisplayClusterConfiguratorViewportNode::IsFixedAspectRatio() const
{
	if (!IsObjectValid())
	{
		return false;
	}

	UDisplayClusterConfigurationViewport* CfgViewport = GetObjectChecked<UDisplayClusterConfigurationViewport>();
	return CfgViewport->bFixedAspectRatio;
}

void UDisplayClusterConfiguratorViewportNode::RotateViewport(float InRotation)
{
	UDisplayClusterConfigurationViewport* CfgViewport = GetObjectChecked<UDisplayClusterConfigurationViewport>();
	
	FDisplayClusterConfigurationViewport_RemapData NewRemap(CfgViewport->ViewportRemap.BaseRemap);
	NewRemap.Angle = FRotator::ClampAxis(CfgViewport->ViewportRemap.BaseRemap.Angle + InRotation);

	ViewportVM->SetRemap(NewRemap);

	ReadNodeStateFromObject();
}

void UDisplayClusterConfiguratorViewportNode::FlipViewport(bool bFlipHorizontal, bool bFlipVertical)
{
	UDisplayClusterConfigurationViewport* CfgViewport = GetObjectChecked<UDisplayClusterConfigurationViewport>();

	FDisplayClusterConfigurationViewport_RemapData NewRemap(CfgViewport->ViewportRemap.BaseRemap);
	if (bFlipHorizontal)
	{
		NewRemap.bFlipH = !CfgViewport->ViewportRemap.BaseRemap.bFlipH;
	}

	if (bFlipVertical)
	{
		NewRemap.bFlipV = !CfgViewport->ViewportRemap.BaseRemap.bFlipV;
	}

	ViewportVM->SetRemap(NewRemap);
}

void UDisplayClusterConfiguratorViewportNode::ResetTransform()
{
	UDisplayClusterConfigurationViewport* CfgViewport = GetObjectChecked<UDisplayClusterConfigurationViewport>();

	FDisplayClusterConfigurationViewport_RemapData NewRemap(CfgViewport->ViewportRemap.BaseRemap);

	NewRemap.Angle = 0.0f;
	NewRemap.bFlipH = false;
	NewRemap.bFlipV = false;

	ViewportVM->SetRemap(NewRemap);
}

UTexture* UDisplayClusterConfiguratorViewportNode::GetPreviewTexture() const
{
	TSharedPtr<FDisplayClusterConfiguratorBlueprintEditor> Toolkit = ToolkitPtr.Pin();
	check(Toolkit.IsValid());

	if (ADisplayClusterRootActor* RootActor = Cast<ADisplayClusterRootActor>(Toolkit->GetPreviewActor()))
	{
		if (IDisplayClusterViewportManager* ViewportManager = RootActor->GetViewportManager())
		{
			const FString ViewportId = GetNodeName();
			if (IDisplayClusterViewport* Viewport = ViewportManager->FindViewport(ViewportId))
			{
				return Viewport->GetViewportPreview().GetPreviewTextureRenderTarget2D();
			}
		}
	}

	return nullptr;
}

void UDisplayClusterConfiguratorViewportNode::OnPostEditChangeChainProperty(const FPropertyChangedChainEvent& PropertyChangedEvent)
{
	// If the pointer to the blueprint editor is no longer valid, its likely that the editor this node was created for was closed,
	// and this node is orphaned and will eventually be GCed.
	if (!ToolkitPtr.IsValid())
	{
		return;
	}
	
	// If the object is no longer valid, don't attempt to sync properties
	if (!IsObjectValid())
	{
		return;
	}

	// Only respond to interactive changes
	if ((PropertyChangedEvent.ChangeType & EPropertyChangeType::Interactive) == 0)
	{
		return;
	}

	const UDisplayClusterConfigurationViewport* CfgViewport = GetObjectChecked<UDisplayClusterConfigurationViewport>();

	const FName& PropertyName = PropertyChangedEvent.GetPropertyName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationRectangle, X) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationRectangle, Y))
	{
		Modify();

		// Change slots and children position, config object already updated 
		const FVector2D GlobalPosition = TransformPointToGlobal(FVector2D(CfgViewport->Region.X, CfgViewport->Region.Y));
		NodePosX = GlobalPosition.X;
		NodePosY = GlobalPosition.Y;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationRectangle, W) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationRectangle, H) || 
		PropertyName == GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_RemapData, Angle))
	{
		Modify();

		// Change node slot size, config object already updated
		const FDisplayClusterConfigurationViewport_RemapData& CfgRemap = GetCfgViewportRemap();

		const FVector2D GlobalSize = TransformSizeToGlobal(FVector2D(CfgViewport->Region.W, CfgViewport->Region.H));

		const FMatrix2x2 RotMat = GetSizeRotationMatrix(CfgRemap.Angle);
		const FVector2D RotatedSize = RotMat.TransformVector(GlobalSize);

		NodeWidth = FMath::RoundToInt(RotatedSize.X);
		NodeHeight = FMath::RoundToInt(RotatedSize.Y);
		ViewportAspectRatio = (float)CfgViewport->Region.W / (float)CfgViewport->Region.H;
	}
}

FMatrix2x2 UDisplayClusterConfiguratorViewportNode::GetSizeRotationMatrix(float AngleInDegrees) const
{
	float SinAngle, CosAngle;
	FMath::SinCos(&SinAngle, &CosAngle, FMath::DegreesToRadians(AngleInDegrees));

	return FMatrix2x2(
		FMath::Abs(CosAngle), FMath::Abs(SinAngle),
		FMath::Abs(SinAngle), FMath::Abs(CosAngle));
}

bool UDisplayClusterConfiguratorViewportNode::IsRotationAngleDegenerate(float AngleInDegrees) const
{
	float ClampedAngle = FMath::Fmod(AngleInDegrees, 90.0f);
	if (ClampedAngle < 0.f)
	{
		ClampedAngle += 90;
	}

	return FMath::Abs(ClampedAngle - 45.0f) < KINDA_SMALL_NUMBER;
}
