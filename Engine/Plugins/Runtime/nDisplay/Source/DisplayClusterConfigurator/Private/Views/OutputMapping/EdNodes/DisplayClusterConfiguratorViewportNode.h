// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/TransformCalculus2D.h"
#include "Views/OutputMapping/EdNodes/DisplayClusterConfiguratorBaseNode.h"

#include "Engine/Texture.h"

#include "DisplayClusterConfiguratorViewportNode.generated.h"

class UDisplayClusterConfigurationViewport;
class FDisplayClusterConfiguratorBlueprintEditor;
class FDisplayClusterConfiguratorViewportViewModel;
struct FDisplayClusterConfigurationRectangle;
struct FDisplayClusterConfigurationViewport_RemapData;

UCLASS(MinimalAPI)
class UDisplayClusterConfiguratorViewportNode final
	: public UDisplayClusterConfiguratorBaseNode
{
	GENERATED_BODY()

public:
	virtual void Initialize(const FString& InNodeName, int32 InNodeZIndex, UObject* InObject, const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& InToolkit) override;
	virtual void Cleanup() override;

	//~ Begin EdGraphNode Interface
	virtual TSharedPtr<SGraphNode> CreateVisualWidget() override;
	virtual void ResizeNode(const FVector2f& NewSize) override;
	virtual bool CanDuplicateNode() const override { return true; }
	virtual bool CanUserDeleteNode() const override { return true; }
	//~ End EdGraphNode Interface
	
	//~ Begin UDisplayClusterConfiguratorBaseNode Interface
	virtual bool IsNodeVisible() const override;
	virtual bool IsNodeUnlocked() const override;
	virtual bool CanNodeOverlapSiblings() const override { return false; }
	virtual bool CanNodeHaveNegativePosition() const { return false; }

	virtual void DeleteObject() override;

protected:
	virtual bool CanAlignWithParent() const override { return true; }
	virtual void WriteNodeStateToObject() override;
	virtual void ReadNodeStateFromObject() override;
	//~ End UDisplayClusterConfiguratorBaseNode Interface

public:
	/** Gets the underlying viewport configuration's region */
	const FDisplayClusterConfigurationRectangle& GetCfgViewportRegion() const;

	/** Gets the underlying viewport configuration's base remapping configuration */
	const FDisplayClusterConfigurationViewport_RemapData& GetCfgViewportRemap() const;

	/** Gets whether the aspect ratio of the viewport's region is fixed */
	bool IsFixedAspectRatio() const;

	/** Rotates the viewport by the specified angle in degrees */
	void RotateViewport(float InRotation);

	/** Flips the viewport along the specified axis */
	void FlipViewport(bool bFlipHorizontal, bool bFlipVertical);

	/** Resets the viewport's transform */
	void ResetTransform();

	/** Gets the preview texture that has been created for the viewport from the display cluster root actor */
	UTexture* GetPreviewTexture() const;

private:
	/** Raised when a property within the viewport configuration object is changed */
	void OnPostEditChangeChainProperty(const FPropertyChangedChainEvent& PropertyChangedEvent);

	/** Gets the "rotation" matrix to use on the viewport's size vector to compute the size of the bounding box encapsulating the rotated viewport */
	FMatrix2x2 GetSizeRotationMatrix(float AngleInDegrees) const;

	/** Determines if the specified angle will create a degenerate size rotation matrix that has a determinant of 0, which happens at angles of 45 degree intervals */
	bool IsRotationAngleDegenerate(float AngleInDegrees) const;

private:
	/** The view model used to propagate property change events when the viewport configuration has changed */
	TSharedPtr<FDisplayClusterConfiguratorViewportViewModel> ViewportVM;

	/** The current aspect ratio of the viewport's region */
	float ViewportAspectRatio;
};
