// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/OutputMapping/EdNodes/DisplayClusterConfiguratorWindowNode.h"

#include "DisplayClusterConfiguratorBlueprintEditor.h"
#include "ClusterConfiguration/DisplayClusterConfiguratorClusterUtils.h"
#include "ClusterConfiguration/ViewModels/DisplayClusterConfiguratorClusterNodeViewModel.h"
#include "Views/OutputMapping/IDisplayClusterConfiguratorViewOutputMapping.h"
#include "Views/OutputMapping/EdNodes/DisplayClusterConfiguratorCanvasNode.h"
#include "Views/OutputMapping/EdNodes/DisplayClusterConfiguratorViewportNode.h"
#include "Views/OutputMapping/GraphNodes/SDisplayClusterConfiguratorWindowNode.h"

#include "DisplayClusterConfigurationTypes.h"

void UDisplayClusterConfiguratorWindowNode::Initialize(const FString& InNodeName, int32 InNodeZIndex, UObject* InObject, const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& InToolkit)
{
	UDisplayClusterConfiguratorBaseNode::Initialize(InNodeName, InNodeZIndex, InObject, InToolkit);

	UDisplayClusterConfigurationClusterNode* CfgNode = GetObjectChecked<UDisplayClusterConfigurationClusterNode>();
	CfgNode->OnPostEditChangeChainProperty.Add(UDisplayClusterConfigurationViewport::FOnPostEditChangeChainProperty::FDelegate::CreateUObject(this, &UDisplayClusterConfiguratorWindowNode::OnPostEditChangeChainProperty));

	ClusterNodeVM = MakeShareable(new FDisplayClusterConfiguratorClusterNodeViewModel(CfgNode));
}

void UDisplayClusterConfiguratorWindowNode::Cleanup()
{
	if (ObjectToEdit.IsValid())
	{
		UDisplayClusterConfigurationClusterNode* ClusterNode = GetObjectChecked<UDisplayClusterConfigurationClusterNode>();
		ClusterNode->OnPostEditChangeChainProperty.RemoveAll(this);
	}
}

TSharedPtr<SGraphNode> UDisplayClusterConfiguratorWindowNode::CreateVisualWidget()
{
	return SNew(SDisplayClusterConfiguratorWindowNode, this, ToolkitPtr.Pin().ToSharedRef());;
}

const FDisplayClusterConfigurationRectangle& UDisplayClusterConfiguratorWindowNode::GetCfgWindowRect() const
{
	UDisplayClusterConfigurationClusterNode* CfgClusterNode = GetObjectChecked<UDisplayClusterConfigurationClusterNode>();
	return CfgClusterNode->WindowRect;
}

FString UDisplayClusterConfiguratorWindowNode::GetCfgHost() const
{
	if (!IsObjectValid())
	{
		return TEXT("");
	}

	UDisplayClusterConfigurationClusterNode* CfgClusterNode = GetObjectChecked<UDisplayClusterConfigurationClusterNode>();
	return CfgClusterNode->Host;
}

const FString& UDisplayClusterConfiguratorWindowNode::GetPreviewImagePath() const
{
	UDisplayClusterConfigurationClusterNode* CfgClusterNode = GetObjectChecked<UDisplayClusterConfigurationClusterNode>();
	return CfgClusterNode->PreviewImage.ImagePath;
}

bool UDisplayClusterConfiguratorWindowNode::IsFixedAspectRatio() const
{
	if (!IsObjectValid())
	{
		return false;
	}

	UDisplayClusterConfigurationClusterNode* CfgClusterNode = GetObjectChecked<UDisplayClusterConfigurationClusterNode>();
	return CfgClusterNode->bFixedAspectRatio;
}

bool UDisplayClusterConfiguratorWindowNode::IsPrimary() const
{
	if (!IsObjectValid())
	{
		return false;
	}

	UDisplayClusterConfigurationClusterNode* ClusterNode = GetObjectChecked<UDisplayClusterConfigurationClusterNode>();
	return UE::DisplayClusterConfiguratorClusterUtils::IsClusterNodePrimary(ClusterNode);
}

FDelegateHandle UDisplayClusterConfiguratorWindowNode::RegisterOnPreviewImageChanged(const FOnPreviewImageChangedDelegate& Delegate)
{
	return PreviewImageChanged.Add(Delegate);
}

void UDisplayClusterConfiguratorWindowNode::UnregisterOnPreviewImageChanged(FDelegateHandle DelegateHandle)
{
	PreviewImageChanged.Remove(DelegateHandle);
}

bool UDisplayClusterConfiguratorWindowNode::IsNodeVisible() const
{
	if (!IsObjectValid())
	{
		return false;
	}

	UDisplayClusterConfigurationClusterNode* ClusterNode = GetObjectChecked<UDisplayClusterConfigurationClusterNode>();

	if (ClusterNode->bIsVisible)
	{
		return true;
	}

	// If this node is marked as invisible but it has a child that is visible, This node needs to remain visible.
	bool bIsChildVisible = false;
	for (UDisplayClusterConfiguratorBaseNode* Child : Children)
	{
		if (Child->IsNodeVisible())
		{
			bIsChildVisible = true;
			break;
		}
	}
	
	return bIsChildVisible;
}

bool UDisplayClusterConfiguratorWindowNode::IsNodeUnlocked() const
{
	if (!IsObjectValid())
	{
		return false;
	}

	UDisplayClusterConfigurationClusterNode* ClusterNode = GetObjectChecked<UDisplayClusterConfigurationClusterNode>();
	return ClusterNode->bIsUnlocked;
}

bool UDisplayClusterConfiguratorWindowNode::CanNodeExceedParentBounds() const
{
	TSharedPtr<FDisplayClusterConfiguratorBlueprintEditor> Toolkit = ToolkitPtr.Pin();
	check(Toolkit.IsValid());

	TSharedRef<IDisplayClusterConfiguratorViewOutputMapping> OutputMapping = Toolkit->GetViewOutputMapping();
	return !OutputMapping->GetOutputMappingSettings().bKeepClusterNodesInHosts;
}

void UDisplayClusterConfiguratorWindowNode::DeleteObject()
{
	if (!IsObjectValid())
	{
		return;
	}

	UDisplayClusterConfigurationClusterNode* ClusterNode = GetObjectChecked<UDisplayClusterConfigurationClusterNode>();
	UE::DisplayClusterConfiguratorClusterUtils::RemoveClusterNodeFromCluster(ClusterNode);
}

void UDisplayClusterConfiguratorWindowNode::WriteNodeStateToObject()
{
	UDisplayClusterConfigurationClusterNode* CfgClusterNode = GetObjectChecked<UDisplayClusterConfigurationClusterNode>();
	const FVector2D LocalPosition = GetNodeLocalPosition();
	const FVector2D LocalSize = TransformSizeToLocal(GetNodeSize());

	FDisplayClusterConfigurationRectangle NewWindowRect(LocalPosition.X, LocalPosition.Y, LocalSize.X, LocalSize.Y);
	ClusterNodeVM->SetWindowRect(NewWindowRect);
}

void UDisplayClusterConfiguratorWindowNode::ReadNodeStateFromObject()
{
	const FDisplayClusterConfigurationRectangle& WindowRect = GetCfgWindowRect();
	const FVector2D GlobalPosition = TransformPointToGlobal(FVector2D(WindowRect.X, WindowRect.Y));
	const FVector2D GlobalSize = TransformSizeToGlobal(FVector2D(WindowRect.W, WindowRect.H));

	NodePosX = GlobalPosition.X;
	NodePosY = GlobalPosition.Y;
	NodeWidth = GlobalSize.X;
	NodeHeight = GlobalSize.Y;
}

void UDisplayClusterConfiguratorWindowNode::OnPostEditChangeChainProperty(const FPropertyChangedChainEvent& PropertyChangedEvent)
{
	// If the pointer to the blueprint editor is no longer valid, its likely that the editor this node was created for was closed,
	// and this node is orphaned and will eventually be GCed.
	if (!ToolkitPtr.IsValid())
	{
		return;
	}

	// If the object is no longer valid, don't attempt to sync properties
	if (!IsObjectValid())
	{
		return;
	}

	UDisplayClusterConfigurationClusterNode* CfgClusterNode = GetObjectChecked<UDisplayClusterConfigurationClusterNode>();

	const FName& PropertyName = PropertyChangedEvent.GetPropertyName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationRectangle, X) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationRectangle, Y))
	{
		Modify();

		// Change slots and children position, config object already updated 
		const FVector2D GlobalPosition = TransformPointToGlobal(FVector2D(CfgClusterNode->WindowRect.X, CfgClusterNode->WindowRect.Y));

		NodePosX = GlobalPosition.X;
		NodePosY = GlobalPosition.Y;

		UpdateChildNodes();
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationRectangle, W) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationRectangle, H))
	{
		Modify();

		// Change node slot size, config object already updated 
		const FVector2D GlobalSize = TransformSizeToGlobal(FVector2D(CfgClusterNode->WindowRect.W, CfgClusterNode->WindowRect.H));
		NodeWidth = GlobalSize.X;
		NodeHeight = GlobalSize.Y;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationClusterNode, Host))
	{
		// If the host property on a window node has changed, invoke a config refresh, which will allow the graph editor
		// to rebuild with the correct hierarchy.
		if (ToolkitPtr.IsValid())
		{
			ToolkitPtr.Pin()->ClusterChanged();
		}
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationExternalImage, ImagePath))
	{
		PreviewImageChanged.Broadcast();
	}
}