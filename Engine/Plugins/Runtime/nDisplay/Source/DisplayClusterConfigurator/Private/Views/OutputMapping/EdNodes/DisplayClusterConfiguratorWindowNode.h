// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Views/OutputMapping/EdNodes/DisplayClusterConfiguratorBaseNode.h"

#include "DisplayClusterConfiguratorWindowNode.generated.h"

class UDisplayClusterConfigurationClusterNode;
class FDisplayClusterConfiguratorClusterNodeViewModel;
struct FDisplayClusterConfigurationRectangle;

UCLASS()
class UDisplayClusterConfiguratorWindowNode final
	: public UDisplayClusterConfiguratorBaseNode
{
	GENERATED_BODY()

public:
	DECLARE_MULTICAST_DELEGATE(FOnPreviewImageChanged);
	using FOnPreviewImageChangedDelegate = FOnPreviewImageChanged::FDelegate;

public:
	virtual void Initialize(const FString& InNodeName, int32 InNodeZIndex, UObject* InObject, const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& InToolkit) override;
	virtual void Cleanup() override;

	//~ Begin EdGraphNode Interface
	virtual TSharedPtr<SGraphNode> CreateVisualWidget() override;
	virtual bool CanDuplicateNode() const override { return true; }
	virtual bool CanUserDeleteNode() const override { return true; }
	//~ End EdGraphNode Interface

	const FDisplayClusterConfigurationRectangle& GetCfgWindowRect() const;
	FString GetCfgHost() const;
	const FString& GetPreviewImagePath() const;
	bool IsFixedAspectRatio() const;
	bool IsPrimary() const;

	FDelegateHandle RegisterOnPreviewImageChanged(const FOnPreviewImageChangedDelegate& Delegate);
	void UnregisterOnPreviewImageChanged(FDelegateHandle DelegateHandle);

	//~ Begin UDisplayClusterConfiguratorBaseNode Interface
	virtual bool IsNodeVisible() const override;
	virtual bool IsNodeUnlocked() const override;
	virtual bool CanNodeOverlapSiblings() const override { return false; }
	virtual bool CanNodeExceedParentBounds() const override;

	virtual void DeleteObject() override;

protected:
	virtual bool CanAlignWithParent() const override { return true; }
	virtual void WriteNodeStateToObject() override;
	virtual void ReadNodeStateFromObject() override;
	//~ End UDisplayClusterConfiguratorBaseNode Interface

private:
	void OnPostEditChangeChainProperty(const FPropertyChangedChainEvent& PropertyChangedEvent);

private:
	TSharedPtr<FDisplayClusterConfiguratorClusterNodeViewModel> ClusterNodeVM;
	FOnPreviewImageChanged PreviewImageChanged;
};

