// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/OutputMapping/GraphNodes/SDisplayClusterConfiguratorBaseNode.h"

#include "DisplayClusterConfiguratorStyle.h"
#include "DisplayClusterConfiguratorBlueprintEditor.h"
#include "Framework/Application/SlateApplication.h"
#include "Views/TreeViews/IDisplayClusterConfiguratorViewTree.h"
#include "Views/TreeViews/IDisplayClusterConfiguratorTreeItem.h"
#include "Views/OutputMapping/Alignment/DisplayClusterConfiguratorNodeAlignmentHelper.h"
#include "Views/OutputMapping/IDisplayClusterConfiguratorViewOutputMapping.h"
#include "Views/OutputMapping/EdNodes/DisplayClusterConfiguratorBaseNode.h"
#include "Views/OutputMapping/Widgets/SDisplayClusterConfiguratorResizer.h"
#include "Views/OutputMapping/Widgets/SDisplayClusterConfiguratorLayeringBox.h"

#include "Widgets/Images/SImage.h"
#include "SGraphPanel.h"

void SAlignmentRuler::Construct(const FArguments& InArgs)
{
	Orientation = InArgs._Orientation;
	Length = InArgs._Length;
	Thickness = InArgs._Thickness;

	ChildSlot
	[
		SNew(SBorder)
		.Padding(FMargin(0.5f))
		.BorderImage(FAppStyle::GetBrush("WhiteBrush"))
		.BorderBackgroundColor(InArgs._ColorAndOpacity)
		[
			SAssignNew(BoxWidget, SBox)
		]
	];

	if (Orientation.Get(EOrientation::Orient_Horizontal) == EOrientation::Orient_Horizontal)
	{
		BoxWidget->SetWidthOverride(Length);
		BoxWidget->SetHeightOverride(Thickness);
	}
	else
	{
		BoxWidget->SetWidthOverride(Thickness);
		BoxWidget->SetHeightOverride(Length);
	}
}

void SAlignmentRuler::SetOrientation(TAttribute<EOrientation> InOrientation)
{
	Orientation = InOrientation;
	if (Orientation.Get(EOrientation::Orient_Horizontal) == EOrientation::Orient_Horizontal)
	{
		BoxWidget->SetWidthOverride(Length);
		BoxWidget->SetHeightOverride(Thickness);
	}
	else
	{
		BoxWidget->SetWidthOverride(Thickness);
		BoxWidget->SetHeightOverride(Length);
	}
}

EOrientation SAlignmentRuler::GetOrientation() const
{
	return Orientation.Get(EOrientation::Orient_Horizontal);
}

void SAlignmentRuler::SetLength(TAttribute<FOptionalSize> InLength)
{
	Length = InLength;
	if (Orientation.Get(EOrientation::Orient_Horizontal) == EOrientation::Orient_Horizontal)
	{
		BoxWidget->SetWidthOverride(Length);
	}
	else
	{
		BoxWidget->SetHeightOverride(Length);
	}
}

void SAlignmentRuler::SetThickness(TAttribute<FOptionalSize> InThickness)
{
	Thickness = InThickness;
	if (Orientation.Get(EOrientation::Orient_Horizontal) == EOrientation::Orient_Horizontal)
	{
		BoxWidget->SetHeightOverride(Thickness);
	}
	else
	{
		BoxWidget->SetWidthOverride(Thickness);
	}
}

const float SDisplayClusterConfiguratorBaseNode::ResizeHandleSize = 20;

void SDisplayClusterConfiguratorBaseNode::Construct(const FArguments& InArgs, UDisplayClusterConfiguratorBaseNode* InBaseNode, const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& InToolkit)
{
	SGraphNode::Construct();

	ToolkitPtr = InToolkit;

	GraphNode = InBaseNode;
	check(GraphNode);

	SetCursor(TAttribute<TOptional<EMouseCursor::Type>>(this, &SDisplayClusterConfiguratorBaseNode::GetCursor));
}

FCursorReply SDisplayClusterConfiguratorBaseNode::OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const
{
	return SWidget::OnCursorQuery(MyGeometry, CursorEvent);
}

void SDisplayClusterConfiguratorBaseNode::UpdateGraphNode()
{
	InputPins.Empty();
	OutputPins.Empty();

	RightNodeBox.Reset();
	LeftNodeBox.Reset();

	XAlignmentRuler = SNew(SAlignmentRuler)
		.Orientation(EOrientation::Orient_Vertical)
		.Thickness(2)
		.ColorAndOpacity(FLinearColor::Yellow);

	YAlignmentRuler = SNew(SAlignmentRuler)
		.Orientation(EOrientation::Orient_Horizontal)
		.Thickness(2)
		.ColorAndOpacity(FLinearColor::Yellow);

	SetVisibility(MakeAttributeSP(this, &SDisplayClusterConfiguratorBaseNode::GetNodeVisibility));
	SetEnabled(MakeAttributeSP(this, &SDisplayClusterConfiguratorBaseNode::IsNodeEnabled));

	GetOrAddSlot(ENodeZone::BottomRight)
		.SlotSize2f(FVector2f(ResizeHandleSize))
		.SlotOffset2f(TAttribute<FVector2f>(this, &SDisplayClusterConfiguratorBaseNode::GetReizeHandleOffset))
		.VAlign(VAlign_Top)
		.HAlign(HAlign_Left)
		.AllowScaling(false)
		[
			SNew(SDisplayClusterConfiguratorLayeringBox)
			.LayerOffset(DisplayClusterConfiguratorGraphLayers::OrnamentLayerIndex)
			.Visibility(this, &SDisplayClusterConfiguratorBaseNode::GetResizeHandleVisibility)
			[
				SNew(SDisplayClusterConfiguratorResizer, ToolkitPtr.Pin().ToSharedRef(), SharedThis(this))
				.MinimumSize(this, &SDisplayClusterConfiguratorBaseNode::GetNodeMinimumSize)
				.MaximumSize(this, &SDisplayClusterConfiguratorBaseNode::GetNodeMaximumSize)
				.IsFixedAspectRatio(this, &SDisplayClusterConfiguratorBaseNode::IsAspectRatioFixed)
			]
		];

	// The rest of widgets should be created by child classes
}

FVector2D SDisplayClusterConfiguratorBaseNode::ComputeDesiredSize(float) const
{
	return GetSize();
}

void SDisplayClusterConfiguratorBaseNode::MoveTo(const FVector2f& NewPosition, FNodeSet& NodeFilter, bool bMarkDirty)
{
	if (!IsGraphNodeValid())
	{
		return;
	}

	if (!IsNodeUnlocked())
	{
		NodeFilter.Add(SharedThis(this));
		return;
	}

	TSharedPtr<FDisplayClusterConfiguratorBlueprintEditor> Toolkit = ToolkitPtr.Pin();
	check(Toolkit.IsValid());

	TSharedRef<IDisplayClusterConfiguratorViewOutputMapping> OutputMapping = Toolkit->GetViewOutputMapping();

	FVector2f CurrentPosition = FVector2f(GraphNode->NodePosX, GraphNode->NodePosY);
	const FVector2f Offset = NewPosition - CurrentPosition;

	UDisplayClusterConfiguratorBaseNode* EdNode = GetGraphNodeChecked<UDisplayClusterConfiguratorBaseNode>();
	UDisplayClusterConfiguratorBaseNode* ParentEdNode = EdNode->GetParent();

	const FGraphPanelSelectionSet& SelectedNodes = GetOwnerPanel()->SelectionManager.GetSelectedNodes();

	TSet<UDisplayClusterConfiguratorBaseNode*> SelectedBaseNodes;
	for (UObject* SelectedObject : SelectedNodes)
	{
		if (UDisplayClusterConfiguratorBaseNode* BaseNode = Cast<UDisplayClusterConfiguratorBaseNode>(SelectedObject))
		{
			SelectedBaseNodes.Add(BaseNode);
		}
	}

	// If the parent node is also being moved, we don't want to move this node; otherwise, weird translations happen as the parent tries to update its child positions.
	const bool bIsParentSelected = SelectedBaseNodes.Contains(EdNode->GetParent());
	if (bIsParentSelected)
	{
		return;
	}

	// The graph editor applies the "true" movement of the node after the user is done dragging the node around,
	// first resetting the selected nodes back to their starting position, then moving them to their final position,
	// at which point the nodes should be marking themselves as modified for the undo buffer. In this case,
	// all positions have been properly computed and the NewPosition can be applied directly to each node.
	if (!FSlateApplication::Get().GetPressedMouseButtons().Contains(EKeys::LeftMouseButton))
	{
		SGraphNode::MoveTo(NewPosition, NodeFilter, bMarkDirty);

		// If the parent node is being auto-positioned, add it to the undo stack here because we need to store its old position with the this node's old position
		// so that if the move operation is undone, this node can appropriately reset the backing config object's position without requiring a full auto-positioning pass.
		if (ParentEdNode && ParentEdNode->IsNodeAutoPositioned())
		{
			ParentEdNode->Modify(bMarkDirty);
		}

		EdNode->UpdateObject();

		return;
	}

	const bool bIsNodeFiltered = NodeFilter.Contains(SharedThis(this));
	if (!bIsNodeFiltered && !EdNode->IsUserInteractingWithNode())
	{
		BeginUserInteraction();
	}

	// If the user is not directly interacting with this node (clicking it directly to drag it), don't update the node's position.
	// In order to correctly move multiple selected nodes at once while preventing overlap or aligning nodes,  the node being directly
	// moved will handle the move offset and apply it to the entire group of selected nodes instead of allowing each node to determine
	// its own movement.
	if (!EdNode->IsUserDirectlyInteractingWithNode())
	{
		return;
	}

	FVector2D BestOffset = FVector2D(Offset);
	if (!bIsNodeFiltered)
	{
		const bool bIsOverlappingAllowed = OutputMapping->GetOutputMappingSettings().bAllowClusterItemOverlap;

		// Check for overlapping across the entire group of nodes that are being moved, and iteratively update the best offset to prevent
		// any node being moved from overlapping with any of its siblings
		for (UDisplayClusterConfiguratorBaseNode* BaseNode : SelectedBaseNodes)
		{
			if (!BaseNode->CanNodeExceedParentBounds() || !BaseNode->CanNodeHaveNegativePosition())
			{
				BestOffset = BaseNode->FindBoundedOffsetFromParent(BestOffset);
			}

			if (!BaseNode->CanNodeOverlapSiblings() && !bIsOverlappingAllowed)
			{
				BestOffset = BaseNode->FindNonOverlappingOffsetFromParent(BestOffset, SelectedBaseNodes);
			}
		}
	}

	FVector2f AlignmentOffset = FVector2f::ZeroVector;
	if (CanSnapAlign() && CanNodeBeSnapAligned() && !bIsNodeFiltered)
	{
		const FNodeAlignmentSettings& AlignmentSettings = OutputMapping->GetNodeAlignmentSettings();

		FNodeAlignmentParams Params;
		Params.bCanSnapSameEdges = AlignmentSettings.bSnapSameEdges;
		Params.bCanSnapAdjacentEdges = AlignmentSettings.bSnapAdjacentEdges;
		Params.SnapProximity = AlignmentSettings.SnapProximity;
		Params.SnapAdjacentEdgesPadding = AlignmentSettings.AdjacentEdgesSnapPadding;

		FNodeAlignmentPair Alignments = EdNode->GetTranslationAlignments(BestOffset, Params, SelectedBaseNodes);
		AlignmentOffset = UE::Slate::CastToVector2f(Alignments.GetOffset());

		UpdateAlignmentTarget(XAlignmentTarget, Alignments.XAlignment, Alignments.XAlignment.TargetNode == ParentEdNode);
		UpdateAlignmentTarget(YAlignmentTarget, Alignments.YAlignment, Alignments.YAlignment.TargetNode == ParentEdNode);
	}
	else
	{
		XAlignmentTarget.TargetNode.Reset();
		YAlignmentTarget.TargetNode.Reset();
	}

	SGraphNode::MoveTo(CurrentPosition + UE::Slate::CastToVector2f(BestOffset) + AlignmentOffset, NodeFilter, bMarkDirty);

	// Nodes that aren't being directly interacted with aren't allowed to update their positions themselves since that might cause
	// issues with overlapping and alignment, so the node being directly dragged and moved needs to update the positions of all selected
	// nodes to move them as a cohesive group
	if (!bIsNodeFiltered)
	{
		for (UDisplayClusterConfiguratorBaseNode* BaseNode : SelectedBaseNodes)
		{
			if (BaseNode != EdNode)
			{
				BaseNode->Modify(bMarkDirty);
				BaseNode->NodePosX += BestOffset.X + AlignmentOffset.X;
				BaseNode->NodePosY += BestOffset.Y + AlignmentOffset.Y;

				BaseNode->UpdateObject();
				BaseNode->UpdateChildNodes();
			}
		}
	}


	if (!bIsNodeFiltered)
	{
		EdNode->UpdateObject();
		EdNode->UpdateChildNodes();
	}
}

void SDisplayClusterConfiguratorBaseNode::EndUserInteraction() const
{
	SGraphNode::EndUserInteraction();

	// EndUserInteraction is only called on the widget the user initially clicked on, but since nodes are marked as being 
	// interacted with if they are moved, all selected nodes need their interaction flag cleared.
	const FGraphPanelSelectionSet& SelectedNodes = GetOwnerPanel()->SelectionManager.GetSelectedNodes();
	for (UObject* SelectedNode : SelectedNodes)
	{
		if (UDisplayClusterConfiguratorBaseNode* BaseNode = Cast<UDisplayClusterConfiguratorBaseNode>(SelectedNode))
		{
			BaseNode->ClearUserInteractingWithNode();
		}
	}

	XAlignmentTarget.TargetNode = nullptr;
	YAlignmentTarget.TargetNode = nullptr;
}

const FSlateBrush* SDisplayClusterConfiguratorBaseNode::GetShadowBrush(bool bSelected) const
{
	return FAppStyle::GetNoBrush();
}

bool SDisplayClusterConfiguratorBaseNode::CanBeSelected(const FVector2f& MousePositionInNode) const
{
	return IsNodeEnabled() && IsNodeUnlocked();
}

bool SDisplayClusterConfiguratorBaseNode::ShouldAllowCulling() const
{
	return false;
}

int32 SDisplayClusterConfiguratorBaseNode::GetSortDepth() const
{
	return GetNodeLogicalLayer();
}

TArray<FOverlayWidgetInfo> SDisplayClusterConfiguratorBaseNode::GetOverlayWidgets(bool bSelected, const FVector2f& WidgetSize) const
{
	TArray<FOverlayWidgetInfo> Widgets = SGraphNode::GetOverlayWidgets(bSelected, WidgetSize);

	const bool bIsSnapMovingNode = FSlateApplication::Get().GetPressedMouseButtons().Contains(EKeys::LeftMouseButton) && FSlateApplication::Get().GetModifierKeys().IsShiftDown();
	if (bIsSnapMovingNode)
	{
		AddAlignmentRulerToOverlay(Widgets, XAlignmentRuler, XAlignmentTarget, WidgetSize);
		AddAlignmentRulerToOverlay(Widgets, YAlignmentRuler, YAlignmentTarget, WidgetSize);
	}

	return Widgets;
}

void SDisplayClusterConfiguratorBaseNode::BeginUserInteraction() const
{
	// If the user isn't attempting to click on the node, do not mark the node as being interacted
	if (!FSlateApplication::Get().GetPressedMouseButtons().Contains(EKeys::LeftMouseButton))
	{
		return;
	}

	if (!IsGraphNodeValid())
	{
		return;
	}

	UDisplayClusterConfiguratorBaseNode* EdNode = GetGraphNodeChecked<UDisplayClusterConfiguratorBaseNode>();

	bool bIsInteractingDirectly = false;
	const FVector2D CursorPos = GetOwnerPanel()->PanelCoordToGraphCoord(GetOwnerPanel()->GetTickSpaceGeometry().AbsoluteToLocal(FSlateApplication::Get().GetCursorPos()));

	if (EdNode->GetNodeBounds().IsInside(CursorPos))
	{
		bIsInteractingDirectly = true;

		// Check that there isn't another node above this node that the user might be clicking on
		const FGraphPanelSelectionSet& SelectedNodes = GetOwnerPanel()->SelectionManager.GetSelectedNodes();
		const int32 EdNodeLayer = EdNode->GetNodeLayer(SelectedNodes);

		for (UObject* SelectedNode : SelectedNodes)
		{
			if (UDisplayClusterConfiguratorBaseNode* BaseNode = Cast<UDisplayClusterConfiguratorBaseNode>(SelectedNode))
			{
				if (BaseNode->GetNodeBounds().IsInside(CursorPos))
				{
					if (BaseNode->GetNodeLayer(SelectedNodes) > EdNodeLayer)
					{
						bIsInteractingDirectly = false;
					}
				}
			}
		}
	}

	EdNode->MarkUserInteractingWithNode(bIsInteractingDirectly);
}

void SDisplayClusterConfiguratorBaseNode::SetNodeSize(const FVector2D InLocalSize, bool bFixedAspectRatio)
{
	if (!IsGraphNodeValid())
	{
		return;
	}

	TSharedPtr<FDisplayClusterConfiguratorBlueprintEditor> Toolkit = ToolkitPtr.Pin();
	check(Toolkit.IsValid());

	TSharedRef<IDisplayClusterConfiguratorViewOutputMapping> OutputMapping = Toolkit->GetViewOutputMapping();
	const bool bIsOverlappingAllowed = OutputMapping->GetOutputMappingSettings().bAllowClusterItemOverlap;

	UDisplayClusterConfiguratorBaseNode* EdNode = GetGraphNodeChecked<UDisplayClusterConfiguratorBaseNode>();
	UDisplayClusterConfiguratorBaseNode* ParentEdNode = EdNode->GetParent();

	const FVector2D CurrentSize = EdNode->GetNodeSize();

	FVector2D BestSize = InLocalSize;
	if (!EdNode->CanNodeExceedParentBounds())
	{
		BestSize = EdNode->FindBoundedSizeFromParent(BestSize, bFixedAspectRatio);
	}

	if (!EdNode->CanNodeEncroachChildBounds())
	{
		BestSize = EdNode->FindBoundedSizeFromChildren(BestSize, bFixedAspectRatio);
	}

	if (!EdNode->CanNodeOverlapSiblings() && !bIsOverlappingAllowed)
	{
		BestSize = EdNode->FindNonOverlappingSizeFromParent(BestSize, bFixedAspectRatio);
	}

	FVector2D AlignmentOffset = FVector2D::ZeroVector;
	if (CanSnapAlign() && CanNodeBeSnapAligned())
	{
		const FNodeAlignmentSettings& AlignmentSettings = OutputMapping->GetNodeAlignmentSettings();

		FNodeAlignmentParams Params;
		Params.bCanSnapSameEdges = AlignmentSettings.bSnapSameEdges;
		Params.bCanSnapAdjacentEdges = AlignmentSettings.bSnapAdjacentEdges;
		Params.SnapProximity = AlignmentSettings.SnapProximity;
		Params.SnapAdjacentEdgesPadding = AlignmentSettings.AdjacentEdgesSnapPadding;

		FNodeAlignmentPair Alignments = EdNode->GetResizeAlignments(BestSize - CurrentSize, Params);
		AlignmentOffset = Alignments.GetOffset();

		UpdateAlignmentTarget(XAlignmentTarget, Alignments.XAlignment, Alignments.XAlignment.TargetNode == ParentEdNode);
		UpdateAlignmentTarget(YAlignmentTarget, Alignments.YAlignment, Alignments.YAlignment.TargetNode == ParentEdNode);

		// Make sure the alignment offset never causes a negative size.
		AlignmentOffset.X = FMath::Max(AlignmentOffset.X, -BestSize.X);
		AlignmentOffset.Y = FMath::Max(AlignmentOffset.Y, -BestSize.Y);
	}

	GraphNode->ResizeNode(UE::Slate::FDeprecateVector2DParameter(BestSize + AlignmentOffset));

	// If the parent node is being auto-positioned, add it to the undo stack here because we need to store its old position with the this node's old position
	// so that if the move operation is undone, this node can appropriately reset the backing config object's position without requiring a full auto-positioning pass.
	if (ParentEdNode && ParentEdNode->IsNodeAutoPositioned())
	{
		ParentEdNode->Modify();
	}
}

bool SDisplayClusterConfiguratorBaseNode::IsNodeVisible() const
{
	if (!IsGraphNodeValid())
	{
		return false;
	}

	UDisplayClusterConfiguratorBaseNode* EdNode = GetGraphNodeChecked<UDisplayClusterConfiguratorBaseNode>();
	return EdNode->IsNodeVisible();
}

bool SDisplayClusterConfiguratorBaseNode::IsNodeEnabled() const
{
	if (!IsGraphNodeValid())
	{
		return false;
	}

	UDisplayClusterConfiguratorBaseNode* EdNode = GetGraphNodeChecked<UDisplayClusterConfiguratorBaseNode>();
	return EdNode->IsNodeEnabled();
}

bool SDisplayClusterConfiguratorBaseNode::IsNodeUnlocked() const
{
	if (!IsGraphNodeValid())
	{
		return false;
	}

	UDisplayClusterConfiguratorBaseNode* EdNode = GetGraphNodeChecked<UDisplayClusterConfiguratorBaseNode>();
	return EdNode->IsNodeUnlocked();
}

FVector2D SDisplayClusterConfiguratorBaseNode::GetSize() const
{
	return FVector2D(GraphNode->NodeWidth, GraphNode->NodeHeight);
}

EVisibility SDisplayClusterConfiguratorBaseNode::GetNodeVisibility() const
{
	if (IsNodeVisible())
	{
		return EVisibility::Visible;
	}

	return EVisibility::Hidden;
}

EVisibility SDisplayClusterConfiguratorBaseNode::GetSelectionVisibility() const
{
	if (GetOwnerPanel()->SelectionManager.SelectedNodes.Contains(GraphNode))
	{
		return EVisibility::Visible;
	}

	return EVisibility::Hidden;
}

TOptional<EMouseCursor::Type> SDisplayClusterConfiguratorBaseNode::GetCursor() const
{
	if (IsNodeEnabled())
	{
		return EMouseCursor::CardinalCross;
	}

	return EMouseCursor::Default;
}

int32 SDisplayClusterConfiguratorBaseNode::GetNodeLogicalLayer() const
{
	if (!IsGraphNodeValid())
	{
		return 0;
	}

	UDisplayClusterConfiguratorBaseNode* EdNode = GetGraphNodeChecked<UDisplayClusterConfiguratorBaseNode>();
	return EdNode->GetNodeLayer(ObjectPtrDecay(GetOwnerPanel()->SelectionManager.SelectedNodes));
}

int32 SDisplayClusterConfiguratorBaseNode::GetNodeVisualLayer() const
{
	if (!IsGraphNodeValid())
	{
		return 0;
	}

	UDisplayClusterConfiguratorBaseNode* EdNode = GetGraphNodeChecked<UDisplayClusterConfiguratorBaseNode>();
	return EdNode->GetNodeLayer(ObjectPtrDecay(GetOwnerPanel()->SelectionManager.SelectedNodes));
}

FVector2f SDisplayClusterConfiguratorBaseNode::GetReizeHandleOffset() const
{
	const FVector2f NodeSize = UE::Slate::CastToVector2f(ComputeDesiredSize(FSlateApplication::Get().GetApplicationScale()));
	const float GraphZoom = GetOwnerPanel()->GetZoomAmount();
	return FVector2f(NodeSize.X, NodeSize.Y) * GraphZoom;
}

EVisibility SDisplayClusterConfiguratorBaseNode::GetResizeHandleVisibility() const
{
	if (!CanNodeBeResized())
	{
		return EVisibility::Collapsed;
	}

	return GetSelectionVisibility();
}

bool SDisplayClusterConfiguratorBaseNode::CanSnapAlign() const
{
	return FSlateApplication::Get().GetModifierKeys().IsShiftDown();
}

void SDisplayClusterConfiguratorBaseNode::UpdateAlignmentTarget(FAlignmentRulerTarget& OutTarget, const FNodeAlignment& Alignment, bool bIsTargetingParent)
{
	if (Alignment.IsValid())
	{
		OutTarget.TargetNode = Alignment.TargetNode;
		OutTarget.bIsAdjacent = Alignment.bIsAdjacent;
		OutTarget.bIsTargetingParent = bIsTargetingParent;

		if (Alignment.AlignedAnchor == EAlignmentAnchor::Center)
		{
			OutTarget.Position = 0.5f;
		}
		else if (Alignment.AlignedAnchor == EAlignmentAnchor::Bottom || Alignment.AlignedAnchor == EAlignmentAnchor::Right)
		{
			OutTarget.Position = 1.f;
		}
		else
		{
			OutTarget.Position = 0.f;
		}
	}
	else
	{
		OutTarget.TargetNode.Reset();
	}
}

void SDisplayClusterConfiguratorBaseNode::AddAlignmentRulerToOverlay(TArray<FOverlayWidgetInfo>& OverlayWidgets, TSharedPtr<SAlignmentRuler> RulerWidget, const FAlignmentRulerTarget& Target, const FVector2f& WidgetSize) const
{
	if (!Target.TargetNode.IsValid())
	{
		return;
	}

	const int32 XAxis = 0;
	const int32 YAxis = 1;

	// Keep track of which alignment axis the ruler is representing. If the ruler is horizontal, that means the y axis is being aligned, vertical for x axis.
	// The cross axis refers to the other axis, and can be computed by 1 - Axis (1 - XAxis = YAxis, 1 - YAxis = XAxis)
	const int32 Axis = RulerWidget->GetOrientation() == EOrientation::Orient_Horizontal ? YAxis : XAxis;
	const int32 CrossAxis = 1 - Axis;

	float RulerLength = 0;
	FVector2f RulerOffset = FVector2f::ZeroVector;

	if (Target.bIsAdjacent)
	{
		RulerLength = WidgetSize[CrossAxis];
		RulerOffset[Axis] = WidgetSize[Axis] * Target.Position;
	}
	else
	{
		const FVector2f ThisNodePosition = GetPosition2f();
		const FVector2f TargetNodePosition = UE::Slate::CastToVector2f(Target.TargetNode->GetNodePosition());
		const FVector2f TargetNodeSize = UE::Slate::CastToVector2f(Target.TargetNode->GetNodeSize());

		if (Target.bIsTargetingParent)
		{
			RulerLength = TargetNodeSize[CrossAxis];
			RulerOffset = TargetNodePosition - ThisNodePosition;
			RulerOffset[Axis] += TargetNodeSize[Axis] * Target.Position;
		}
		else if (TargetNodePosition[CrossAxis] < ThisNodePosition[CrossAxis])
		{
			RulerLength = ThisNodePosition[CrossAxis] + WidgetSize[CrossAxis] - TargetNodePosition[CrossAxis];
			RulerOffset[Axis] = WidgetSize[Axis] * Target.Position;
			RulerOffset[CrossAxis] = -(RulerLength - WidgetSize[CrossAxis]);
		}
		else
		{
			RulerLength = TargetNodePosition[CrossAxis] + TargetNodeSize[CrossAxis] - ThisNodePosition[CrossAxis];
			RulerOffset[Axis] = WidgetSize[Axis] * Target.Position;
			RulerOffset[CrossAxis] = 0;
		}
	}

	RulerWidget->SetLength(RulerLength);

	FOverlayWidgetInfo AlignmentRulerInfo;
	AlignmentRulerInfo.OverlayOffset = RulerOffset;
	AlignmentRulerInfo.Widget = RulerWidget;

	OverlayWidgets.Add(AlignmentRulerInfo);
}

bool SDisplayClusterConfiguratorBaseNode::IsGraphNodeValid() const
{
	return IsValid(GraphNode) && !GraphNode->HasAnyFlags(RF_BeginDestroyed | RF_FinishDestroyed);
}
