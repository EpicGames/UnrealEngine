// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SGraphNode.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Types/SlateStructs.h"

class FDisplayClusterConfiguratorBlueprintEditor;
class IDisplayClusterConfiguratorTreeItem;
class SBox;
class UDisplayClusterConfiguratorBaseNode;
class UTexture;
struct FSlateColor;
struct FNodeAlignment;

class SAlignmentRuler
	: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAlignmentRuler)
		: _ColorAndOpacity(FLinearColor::White)
		, _Orientation(EOrientation::Orient_Horizontal)
		, _Length(100)
		, _Thickness(1)
	{ }
		SLATE_ARGUMENT(FSlateColor, ColorAndOpacity)
		SLATE_ATTRIBUTE(EOrientation, Orientation)
		SLATE_ATTRIBUTE(FOptionalSize, Length)
		SLATE_ATTRIBUTE(FOptionalSize, Thickness)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	void SetOrientation(TAttribute<EOrientation> InOrientation);
	EOrientation GetOrientation() const;

	void SetLength(TAttribute<FOptionalSize> InLength);
	void SetThickness(TAttribute<FOptionalSize> InThickness);

private:
	TSharedPtr<SBox> BoxWidget;

	TAttribute<EOrientation> Orientation;
	TAttribute<FOptionalSize> Length;
	TAttribute<FOptionalSize> Thickness;
};

struct FAlignmentRulerTarget
{
	TWeakObjectPtr<const UDisplayClusterConfiguratorBaseNode> TargetNode;
	bool bIsTargetingParent;
	bool bIsAdjacent;
	float Position;
};

class SDisplayClusterConfiguratorBaseNode
	: public SGraphNode
{
public:
	SLATE_BEGIN_ARGS(SDisplayClusterConfiguratorBaseNode)
		{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UDisplayClusterConfiguratorBaseNode* InBaseNode, const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& InToolkit);

	//~ Begin SWidget interface
	virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override;
	//~ End of SWidget interface

	//~ Begin SGraphNode interface
	virtual void UpdateGraphNode() override;
	virtual FVector2D ComputeDesiredSize(float) const override;
	virtual void MoveTo(const FVector2f& NewPosition, FNodeSet& NodeFilter, bool bMarkDirty = true) override;
	virtual void EndUserInteraction() const override;
	virtual const FSlateBrush* GetShadowBrush(bool bSelected) const override;
	virtual bool CanBeSelected(const FVector2f& MousePositionInNode) const override;
	virtual bool ShouldAllowCulling() const override;
	virtual int32 GetSortDepth() const override;
	virtual TArray<FOverlayWidgetInfo> GetOverlayWidgets(bool bSelected, const FVector2f& WidgetSize) const override;
	//~ End SGraphNode interface

	/**
	 * Raised whenever the user begins interacting with this node
	 */
	virtual void BeginUserInteraction() const;

	/**
	 * Apply new node size
	 *
	 * @param InLocalSize - Size in local space
	 * @param bFixedAspectRatio - Indicates the node should have a fixed aspect ratio
	 */
	virtual void SetNodeSize(const FVector2D InLocalSize, bool bFixedAspectRatio);

	/**
	 * @return true if the node should be visible
	 */
	virtual bool IsNodeVisible() const;

	/**
	 * @return true if the node should be enabled
	 */
	virtual bool IsNodeEnabled() const;

	/**
	 * @return true if the node should be locked, meaning it can't be selected, moved, or resized
	 */
	virtual bool IsNodeUnlocked() const;

	/**
	 * @return The intended size of the node, taken from the backing EdGraphNode 
	 */
	virtual FVector2D GetSize() const;

	/**
	 * @return Wether this node can be snap aligned when the user activates snap aligning
	 */
	virtual bool CanNodeBeSnapAligned() const { return false; }

	/** @return Whether this node can be resized using the resize widget. */
	virtual bool CanNodeBeResized() const { return true; }

	/** @return The minimum size this node can be resized to. */
	virtual float GetNodeMinimumSize() const { return 0; }

	/** @return The maximum size this node can be resized to. */
	virtual float GetNodeMaximumSize() const { return FLT_MAX; }

	/** @return Whether this node's size is fixed to a specific aspect ratio. */
	virtual bool IsAspectRatioFixed() const { return false; }

protected:
	EVisibility GetNodeVisibility() const;
	EVisibility GetSelectionVisibility() const;
	TOptional<EMouseCursor::Type> GetCursor() const;

	virtual int32 GetNodeLogicalLayer() const;
	virtual int32 GetNodeVisualLayer() const;

	FVector2f GetReizeHandleOffset() const;
	EVisibility GetResizeHandleVisibility() const;

	bool CanSnapAlign() const;
	void UpdateAlignmentTarget(FAlignmentRulerTarget& OutTarget, const FNodeAlignment& Alignment, bool bIsTargetingParent);
	void AddAlignmentRulerToOverlay(TArray<FOverlayWidgetInfo>& OverlayWidgets, TSharedPtr<SAlignmentRuler> RulerWidget, const FAlignmentRulerTarget& Target, const FVector2f& WidgetSize) const;

	template<class TObjectType>
	TObjectType* GetGraphNodeChecked() const
	{
		TObjectType* CastedNode = Cast<TObjectType>(GraphNode);
		check(CastedNode);
		return CastedNode;
	}

	/** Checks that the graph node this widget displays is valid and has not been destroyed or garbage collected */
	bool IsGraphNodeValid() const;
	
protected:
	TWeakPtr<FDisplayClusterConfiguratorBlueprintEditor> ToolkitPtr;

	TSharedPtr<SAlignmentRuler> XAlignmentRuler;
	TSharedPtr<SAlignmentRuler> YAlignmentRuler;

	// These need to be mutable so they can be cleared in the node's EndUserInteraction call, which is a const function
	mutable FAlignmentRulerTarget XAlignmentTarget;
	mutable FAlignmentRulerTarget YAlignmentTarget;

private:
	static const float ResizeHandleSize;
};
