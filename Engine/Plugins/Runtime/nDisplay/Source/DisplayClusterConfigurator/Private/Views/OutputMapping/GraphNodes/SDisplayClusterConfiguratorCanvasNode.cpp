// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/OutputMapping/GraphNodes/SDisplayClusterConfiguratorCanvasNode.h"

#include "Views/OutputMapping/EdNodes/DisplayClusterConfiguratorWindowNode.h"
#include "Views/OutputMapping/EdNodes/DisplayClusterConfiguratorViewportNode.h"

#include "DisplayClusterConfiguratorBlueprintEditor.h"
#include "DisplayClusterConfiguratorStyle.h"
#include "Views/TreeViews/IDisplayClusterConfiguratorTreeItem.h"
#include "Views/OutputMapping/EdNodes/DisplayClusterConfiguratorCanvasNode.h"
#include "SGraphPanel.h"

#define LOCTEXT_NAMESPACE "SDisplayClusterConfiguratorCanvasNode"

void SDisplayClusterConfiguratorCanvasNode::Construct(const FArguments& InArgs, UDisplayClusterConfiguratorCanvasNode* InNode, const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& InToolkit)
{	
	SDisplayClusterConfiguratorBaseNode::Construct(SDisplayClusterConfiguratorBaseNode::FArguments(), InNode, InToolkit);

	// Add padding to the canvas node's rendered size, ensuring the borders are visible when wrapping its children
	CanvasPadding = FMargin(75, 75, 75, 75);

	UpdateGraphNode();
}

void SDisplayClusterConfiguratorCanvasNode::UpdateGraphNode()
{
	SDisplayClusterConfiguratorBaseNode::UpdateGraphNode();
	
	TAttribute<const FSlateBrush*> SelectedBrush = TAttribute<const FSlateBrush*>::Create(TAttribute<const FSlateBrush*>::FGetter::CreateSP(this, &SDisplayClusterConfiguratorCanvasNode::GetSelectedBrush));

	CanvasSizeTextWidget = SNew(SBorder)
	.BorderImage(FAppStyle::GetBrush("NoBorder"))
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.HAlign(HAlign_Center)
		.AutoHeight()
		[
			SNew(SHorizontalBox)
											
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(5.f, 5.f, 5.f, 2.f)
			.HAlign(EHorizontalAlignment::HAlign_Center)
			[
				SNew(STextBlock)
				.Text(this, &SDisplayClusterConfiguratorCanvasNode::GetCanvasSizeText)
				.TextStyle(&FDisplayClusterConfiguratorStyle::Get().GetWidgetStyle<FTextBlockStyle>("DisplayClusterConfigurator.Node.Text.Regular"))
				.Justification(ETextJustify::Center)
			]
		]
	];

	GetOrAddSlot(ENodeZone::Center)
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Fill)
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("NoBorder"))
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.FillHeight(1)
			[
				SNew(SBox)
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					SNew(SBorder)
					.BorderImage(SelectedBrush)
				]
			]
		]
	];
}

void SDisplayClusterConfiguratorCanvasNode::MoveTo(const FVector2f& NewPosition, FNodeSet& NodeFilter, bool bMarkDirty)
{
	// Canvas node is not allowed to be moved in general, so add it to the node filter
	NodeFilter.Add(SharedThis(this));

	SGraphNode::MoveTo(NewPosition, NodeFilter, bMarkDirty);
}

FVector2D SDisplayClusterConfiguratorCanvasNode::ComputeDesiredSize(float) const
{
	const FVector2D NodeSize = GetSize();
	return FVector2D(NodeSize.X + CanvasPadding.Left + CanvasPadding.Right, NodeSize.Y + CanvasPadding.Top + CanvasPadding.Bottom);
}

FVector2f SDisplayClusterConfiguratorCanvasNode::GetPosition2f() const
{
	const FVector2f NodePosition = SDisplayClusterConfiguratorBaseNode::GetPosition2f();

	// Offset node position by the top and left margin of the canvas padding
	return NodePosition - FVector2f(CanvasPadding.Left, CanvasPadding.Top);
}

TArray<FOverlayWidgetInfo> SDisplayClusterConfiguratorCanvasNode::GetOverlayWidgets(bool bSelected, const FVector2f& WidgetSize) const
{
	TArray<FOverlayWidgetInfo> Widgets = SDisplayClusterConfiguratorBaseNode::GetOverlayWidgets(bSelected, WidgetSize);

	const FVector2f TextSize = CanvasSizeTextWidget->GetDesiredSize();

	FOverlayWidgetInfo Info;
	Info.OverlayOffset = FVector2f(0.5f * (WidgetSize.X - TextSize.X), WidgetSize.Y);
	Info.Widget = CanvasSizeTextWidget;

	Widgets.Add(Info);

	return Widgets;
}

const FSlateBrush* SDisplayClusterConfiguratorCanvasNode::GetSelectedBrush() const
{
	if (GetOwnerPanel()->SelectionManager.SelectedNodes.Contains(GetNodeObj()))
	{
		// Selected Case
		return FDisplayClusterConfiguratorStyle::Get().GetBrush("DisplayClusterConfigurator.Selected.Canvas.Brush");
	}

	// Regular case
	return FDisplayClusterConfiguratorStyle::Get().GetBrush("DisplayClusterConfigurator.Regular.Canvas.Brush");
}

FMargin SDisplayClusterConfiguratorCanvasNode::GetBackgroundPosition() const
{
	const FVector2D Size = ComputeDesiredSize(0);
	return FMargin(0, 0, Size.X, Size.Y);
}

FText SDisplayClusterConfiguratorCanvasNode::GetCanvasSizeText() const
{
	if (!IsGraphNodeValid())
	{
		return FText::GetEmpty();
	}

	UDisplayClusterConfiguratorCanvasNode* CanvasNode = GetGraphNodeChecked<UDisplayClusterConfiguratorCanvasNode>();
	const FVector2D& Resolution = CanvasNode->GetResolution();
	return FText::Format(LOCTEXT("ClusterResolution", "Cluster Resolution {0} x {1}"), FText::AsNumber(FMath::RoundToInt(Resolution.X)), FText::AsNumber(FMath::RoundToInt(Resolution.Y)));
}

#undef LOCTEXT_NAMESPACE