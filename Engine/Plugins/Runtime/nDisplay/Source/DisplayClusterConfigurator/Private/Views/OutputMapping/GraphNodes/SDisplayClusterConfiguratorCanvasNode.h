// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Views/OutputMapping/GraphNodes/SDisplayClusterConfiguratorBaseNode.h"

#include "UObject/WeakObjectPtr.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class IDisplayClusterConfiguratorOutputMappingSlot;
class FDisplayClusterConfiguratorBlueprintEditor;
class FDisplayClusterConfiguratorOutputMappingBuilder;
class UDisplayClusterConfiguratorCanvasNode;
class UDisplayClusterConfigurationCluster;

class SDisplayClusterConfiguratorCanvasNode
	: public SDisplayClusterConfiguratorBaseNode
{
public:
	SLATE_BEGIN_ARGS(SDisplayClusterConfiguratorCanvasNode)
	{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UDisplayClusterConfiguratorCanvasNode* InNode, const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& InToolkit);

	//~ SGraphNode interface
	virtual void UpdateGraphNode() override;
	virtual void MoveTo(const FVector2f& NewPosition, FNodeSet& NodeFilter, bool bMarkDirty = true) override;
	virtual FVector2D ComputeDesiredSize(float) const override;
	virtual FVector2f GetPosition2f() const override;
	virtual TArray<FOverlayWidgetInfo> GetOverlayWidgets(bool bSelected, const FVector2f& WidgetSize) const override;
	//~ End SGraphNode interface

	//~ Begin SDisplayClusterConfiguratorBaseNode interface
	virtual bool CanNodeBeResized() const { return false; }
	//~ End of SDisplayClusterConfiguratorBaseNode interface

private:
	const FSlateBrush* GetSelectedBrush() const;
	FMargin GetBackgroundPosition() const;
	FText GetCanvasSizeText() const;

private:
	TSharedPtr<SWidget> CanvasSizeTextWidget;

	FMargin CanvasPadding;
};
