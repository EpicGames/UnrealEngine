// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDisplayClusterConfiguratorHostNode.h"

#include "DisplayClusterConfiguratorStyle.h"
#include "Views/OutputMapping/EdNodes/DisplayClusterConfiguratorHostNode.h"
#include "Views/OutputMapping/Widgets/SDisplayClusterConfiguratorResizer.h"
#include "Views/OutputMapping/Widgets/SDisplayClusterConfiguratorLayeringBox.h"

#include "SGraphPanel.h"
#include "ScopedTransaction.h"
#include "Framework/Application/SlateApplication.h"
#include "Rendering/DrawElements.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SConstraintCanvas.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/Layout/SSpacer.h"

#define LOCTEXT_NAMESPACE "SDisplayClusterConfiguratorCanvasNode"

class SCrosshair : public SBox
{
public:
	SLATE_BEGIN_ARGS(SCrosshair)
		: _Thickness(1)
		, _Size(1)
		, _Alignment(0.5f)
		, _ColorAndOpacity(FLinearColor::White)
	{ }
		SLATE_ARGUMENT(float, Thickness)
		SLATE_ARGUMENT(float, Size)
		SLATE_ARGUMENT(FVector2D, Alignment)
		SLATE_ATTRIBUTE(FSlateColor, ColorAndOpacity)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		Thickness = InArgs._Thickness;
		Alignment = InArgs._Alignment;
		ColorAndOpacity = InArgs._ColorAndOpacity;
		
		SBox::Construct(SBox::FArguments().WidthOverride(InArgs._Size).HeightOverride(InArgs._Size));
	}

	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override
	{
		const FVector2D LocalSize = AllottedGeometry.GetLocalSize();
		const FLinearColor Tint = ColorAndOpacity.Get(FLinearColor::White).GetColor(InWidgetStyle);

		++LayerId;
		TArray<FVector2D> VerticalLine = { FVector2D(Alignment.X * LocalSize.X, 0.0f), FVector2D(Alignment.X * LocalSize.X, LocalSize.Y) };
		TArray<FVector2D> HorizontalLine = { FVector2D(0.0f, Alignment.Y * LocalSize.Y), FVector2D(LocalSize.X, Alignment.Y * LocalSize.Y) };

		FSlateDrawElement::MakeLines(
			OutDrawElements,
			LayerId,
			AllottedGeometry.ToPaintGeometry(),
			VerticalLine,
			ESlateDrawEffect::None,
			Tint,
			true,
			Thickness
		);

		FSlateDrawElement::MakeLines(
			OutDrawElements,
			LayerId,
			AllottedGeometry.ToPaintGeometry(),
			HorizontalLine,
			ESlateDrawEffect::None,
			Tint,
			true,
			Thickness
		);

		return SBox::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
	}

private:
	TAttribute<FSlateColor> ColorAndOpacity;
	float Thickness;
	FVector2D Alignment;
};

class SNodeOrigin : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNodeOrigin)
	{ }
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<SDisplayClusterConfiguratorHostNode>& InParentNode)
	{
		ParentNodePtr = InParentNode;
		bIsMovingOrigin = false;

		SetCursor(EMouseCursor::GrabHandClosed);

		FVector2D CrosshairAlignment(0.25f);
		ChildSlot
		[
			SNew(SConstraintCanvas)

			+ SConstraintCanvas::Slot()
			.Alignment(CrosshairAlignment)
			.AutoSize(true)
			[
				SNew(SCrosshair)
				.Size(48)
				.Thickness(3)
				.Alignment(CrosshairAlignment)
			]

			+ SConstraintCanvas::Slot()
			.Alignment(FVector2D(0.0f, 1.0f))
			.AutoSize(true)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("NoBorder"))
				.Padding(5)
				[
					SNew(STextBlock)
					.Text(this, &SNodeOrigin::GetOriginText)
					.TextStyle(&FDisplayClusterConfiguratorStyle::Get().GetWidgetStyle<FTextBlockStyle>("DisplayClusterConfigurator.Node.Text.Small"))
				]
			]
		];
	}

	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
		{
			bIsMovingOrigin = true;
			return FReply::Handled().CaptureMouse(SharedThis(this));
		}

		return FReply::Unhandled();
	}

	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
		{
			bIsMovingOrigin = false;
			ScopedTransaction.Reset();
			return FReply::Handled().ReleaseMouseCapture();
		}

		return FReply::Unhandled();
	}

	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		if (bIsMovingOrigin)
		{
			TSharedPtr<SDisplayClusterConfiguratorHostNode> ParentNode = ParentNodePtr.Pin();
			check(ParentNode.IsValid());

			TSharedPtr<SGraphPanel> GraphPanel = ParentNode->GetOwnerPanel();
			check(GraphPanel.IsValid());

			if (!ParentNode->IsGraphNodeValid())
			{
				return FReply::Unhandled();
			}

			const float DPIScale = GetDPIScale();

			FVector2D NewOrigin = MouseEvent.GetScreenSpacePosition() - ParentNode->GetTickSpaceGeometry().GetAbsolutePosition();
			NewOrigin /= (GraphPanel->GetZoomAmount() * DPIScale);
			NewOrigin -= FVector2D(UDisplayClusterConfiguratorHostNode::VisualMargin.Left, UDisplayClusterConfiguratorHostNode::VisualMargin.Top);

			// Never let node origin exceed bounds of the host node
			UDisplayClusterConfiguratorHostNode* HostEdNode = ParentNode->GetGraphNodeChecked<UDisplayClusterConfiguratorHostNode>();
			if (NewOrigin.X < 0.f)
			{
				NewOrigin.X = 0.f;
			}
			else if (NewOrigin.X > HostEdNode->NodeWidth)
			{
				NewOrigin.X = HostEdNode->NodeWidth;
			}

			if (NewOrigin.Y < 0.f)
			{
				NewOrigin.Y = 0.f;
			}
			else if (NewOrigin.Y > HostEdNode->NodeHeight)
			{
				NewOrigin.Y = HostEdNode->NodeHeight;
			}

			// If we don't have a scoped transaction for the move, create a new one.
			if (!ScopedTransaction.IsValid())
			{
				ScopedTransaction = MakeShareable(new FScopedTransaction(LOCTEXT("MoveHostOriginAction", "Move Host Origin")));
			}

			HostEdNode->SetHostOrigin(NewOrigin, true);

			return FReply::Handled();
		}

		return FReply::Unhandled();
	}

private:
	float GetDPIScale() const
	{
		float DPIScale = 1.0f;
		TSharedPtr<SWindow> WidgetWindow = FSlateApplication::Get().FindWidgetWindow(SharedThis(this));
		if (WidgetWindow.IsValid())
		{
			DPIScale = WidgetWindow->GetNativeWindow()->GetDPIScaleFactor();
		}

		return DPIScale;
	}

	FText GetOriginText() const
	{
		TSharedPtr<SDisplayClusterConfiguratorHostNode> ParentNode = ParentNodePtr.Pin();
		check(ParentNode.IsValid());

		if (!ParentNode->IsGraphNodeValid())
		{
			return FText::GetEmpty();
		}

		UDisplayClusterConfiguratorHostNode* HostEdNode = ParentNode->GetGraphNodeChecked<UDisplayClusterConfiguratorHostNode>();
		const FVector2D& Origin = HostEdNode->GetHostOrigin();
		return FText::Format(LOCTEXT("OriginLabel", "Host Origin @ {0}, {1}"), FMath::FloorToFloat(Origin.X), FMath::FloorToFloat(Origin.Y));
	}

private:
	TWeakPtr<SDisplayClusterConfiguratorHostNode> ParentNodePtr;
	TSharedPtr<FScopedTransaction> ScopedTransaction;
	bool bIsMovingOrigin;
};

void SDisplayClusterConfiguratorHostNode::Construct(const FArguments& InArgs, UDisplayClusterConfiguratorHostNode* InHostNode, const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& InToolkit)
{
	SDisplayClusterConfiguratorBaseNode::Construct(SDisplayClusterConfiguratorBaseNode::FArguments(), InHostNode, InToolkit);

	BorderThickness = InArgs._BorderThickness;

	UpdateGraphNode();
}

void SDisplayClusterConfiguratorHostNode::UpdateGraphNode()
{
	SDisplayClusterConfiguratorBaseNode::UpdateGraphNode();

	GetOrAddSlot(ENodeZone::Center)
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Fill)
	[
		SNew(SDisplayClusterConfiguratorLayeringBox)
		.LayerOffset(this, &SDisplayClusterConfiguratorHostNode::GetNodeVisualLayer)
		[
			SNew(SConstraintCanvas)

			+ SConstraintCanvas::Slot()
			.Offset(TAttribute<FMargin>::Create(TAttribute<FMargin>::FGetter::CreateSP(this, &SDisplayClusterConfiguratorHostNode::GetBackgroundPosition)))
			.Alignment(FVector2D::ZeroVector)
			[
				SNew(SBox)
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Fill)
				[
					SNew(SOverlay)	

					+ SOverlay::Slot()
					.VAlign(VAlign_Fill)
					.HAlign(HAlign_Fill)
					[
						SNew(SImage)
						.ColorAndOpacity(this, &SDisplayClusterConfiguratorHostNode::GetBorderColor)
						.Image(FAppStyle::Get().GetBrush("WhiteBrush"))
					]

					+ SOverlay::Slot()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Top)
					.Padding(BorderThickness.Left, 15, BorderThickness.Right, 10)
					[
						SNew(SScaleBox)
						.HAlign(HAlign_Fill)
						.Stretch(EStretch::ScaleToFit)
						.StretchDirection(EStretchDirection::Both)
						.VAlign(VAlign_Center)
						[
							SNew(SBorder)
							.BorderImage(FAppStyle::Get().GetBrush("NoBorder"))
							.ForegroundColor(this, &SDisplayClusterConfiguratorHostNode::GetTextColor)
							[
								SNew(SHorizontalBox)

								+ SHorizontalBox::Slot()
								.AutoWidth()
								.HAlign(HAlign_Center)
								.VAlign(VAlign_Center)
								[
									SNew(SBox)
									.WidthOverride(48)
									.HeightOverride(48)
									[
										SNew(SImage)
										.Image(FDisplayClusterConfiguratorStyle::Get().GetBrush(TEXT("DisplayClusterConfigurator.TreeItems.Host")))
										.ColorAndOpacity(FSlateColor::UseForeground())
									]
								]

								+ SHorizontalBox::Slot()
								.AutoWidth()
								[
									SNew(SSpacer)
									.Size(FVector2D(25, 1))
								]

								+ SHorizontalBox::Slot()
								.AutoWidth()
								.VAlign(VAlign_Center)
								[
									SNew(STextBlock)
									.Text(this, &SDisplayClusterConfiguratorHostNode::GetHostNameText)
									.TextStyle(&FDisplayClusterConfiguratorStyle::Get().GetWidgetStyle<FTextBlockStyle>("DisplayClusterConfigurator.Host.Text.Title"))
									.ColorAndOpacity(FSlateColor::UseForeground())
								]

								+ SHorizontalBox::Slot()
								.AutoWidth()
								[
									SNew(SSpacer)
									.Size(FVector2D(25, 1))
								]

								+ SHorizontalBox::Slot()
								.AutoWidth()
								.VAlign(VAlign_Center)
								[
									SNew(SBox)
									.WidthOverride(48)
									.HeightOverride(48)
									.Visibility(this, &SDisplayClusterConfiguratorHostNode::GetLockIconVisibility)
									[
										SNew(SImage)
										.Image(FAppStyle::Get().GetBrush(TEXT("PropertyWindow.Locked")))
										.ColorAndOpacity(FSlateColor::UseForeground())
									]
								]

								+ SHorizontalBox::Slot()
								.FillWidth(1)
								.Padding(20, 0)
								[
									SNew(SSpacer)
								]

								+ SHorizontalBox::Slot()
								.AutoWidth()
								.VAlign(VAlign_Center)
								[
									SNew(STextBlock)
									.Text(this, &SDisplayClusterConfiguratorHostNode::GetHostResolutionText)
									.TextStyle(&FDisplayClusterConfiguratorStyle::Get().GetWidgetStyle<FTextBlockStyle>("DisplayClusterConfigurator.Host.Text.Title"))
									.ColorAndOpacity(FSlateColor::UseForeground())
								]

								+ SHorizontalBox::Slot()
								.FillWidth(1)
								.Padding(20, 0)
								[
									SNew(SSpacer)
								]

								+ SHorizontalBox::Slot()
								.AutoWidth()
								.VAlign(VAlign_Center)
								[
									SNew(STextBlock)
									.Text(this, &SDisplayClusterConfiguratorHostNode::GetHostIPText)
									.TextStyle(&FDisplayClusterConfiguratorStyle::Get().GetWidgetStyle<FTextBlockStyle>("DisplayClusterConfigurator.Host.Text.Title"))
									.ColorAndOpacity(FSlateColor::UseForeground())
								]
							]
						]
					]

					+ SOverlay::Slot()
					.VAlign(VAlign_Fill)
					.HAlign(HAlign_Fill)
					.Padding(BorderThickness)
					[
						SNew(SImage)
						.ColorAndOpacity(FDisplayClusterConfiguratorStyle::Get().GetColor("DisplayClusterConfigurator.Node.Host.Inner.Background"))
						.Image(FAppStyle::Get().GetBrush("WhiteBrush"))
					]
				]
			]

			+ SConstraintCanvas::Slot()
			.Offset(TAttribute<FMargin>::Create(TAttribute<FMargin>::FGetter::CreateSP(this, &SDisplayClusterConfiguratorHostNode::GetNodeOriginPosition)))
			.AutoSize(true)
			.Alignment(FVector2D::ZeroVector)
			[
				SNew(SDisplayClusterConfiguratorLayeringBox)
				.LayerOffset(this, &SDisplayClusterConfiguratorHostNode::GetNodeOriginLayerOffset)
				[
					SNew(SNodeOrigin, SharedThis(this))
					.Visibility(this, &SDisplayClusterConfiguratorHostNode::GetSelectionVisibility)
				]
			]
		]
	];
}

void SDisplayClusterConfiguratorHostNode::MoveTo(const FVector2f& NewPosition, FNodeSet& NodeFilter, bool bMarkDirty)
{
	if (!IsGraphNodeValid())
	{
		return;
	}

	UDisplayClusterConfiguratorHostNode* HostEdNode = GetGraphNodeChecked<UDisplayClusterConfiguratorHostNode>();
	if (!HostEdNode->CanUserMoveNode())
	{
		NodeFilter.Add(SharedThis(this));
	}

	SDisplayClusterConfiguratorBaseNode::MoveTo(NewPosition, NodeFilter, bMarkDirty);
}

FVector2D SDisplayClusterConfiguratorHostNode::ComputeDesiredSize(float) const
{
	return GetSize() + FVector2D(BorderThickness.Left + BorderThickness.Right, BorderThickness.Top + BorderThickness.Bottom);
}

FVector2f SDisplayClusterConfiguratorHostNode::GetPosition2f() const
{
	return SDisplayClusterConfiguratorBaseNode::GetPosition2f() - FVector2f(BorderThickness.Left, BorderThickness.Top);
}

void SDisplayClusterConfiguratorHostNode::SetNodeSize(const FVector2D InLocalSize, bool bFixedAspectRatio)
{
	// The size the EdNode stores should not contain its visual margin, so factor that out of the size being set
	const FVector2D AdjustedSize = InLocalSize - UDisplayClusterConfiguratorHostNode::VisualMargin.GetDesiredSize();
	SDisplayClusterConfiguratorBaseNode::SetNodeSize(AdjustedSize, bFixedAspectRatio);
}

bool SDisplayClusterConfiguratorHostNode::CanNodeBeResized() const
{
	if (!IsGraphNodeValid())
	{
		return false;
	}

	UDisplayClusterConfiguratorHostNode* HostEdNode = GetGraphNodeChecked<UDisplayClusterConfiguratorHostNode>();
	return HostEdNode->CanUserResizeNode();
}

FMargin SDisplayClusterConfiguratorHostNode::GetBackgroundPosition() const
{
	FVector2D NodeSize = ComputeDesiredSize(FSlateApplication::Get().GetApplicationScale());
	return FMargin(0.f, 0.f, NodeSize.X, NodeSize.Y);
}

FMargin SDisplayClusterConfiguratorHostNode::GetNodeOriginPosition() const
{
	if (!IsGraphNodeValid())
	{
		return FMargin();
	}

	UDisplayClusterConfiguratorHostNode* HostEdNode = GetGraphNodeChecked<UDisplayClusterConfiguratorHostNode>();
	const FVector2D GlobalOrigin = HostEdNode->GetHostOrigin(true);
	
	return FMargin(GlobalOrigin.X + UDisplayClusterConfiguratorHostNode::VisualMargin.Left, GlobalOrigin.Y + UDisplayClusterConfiguratorHostNode::VisualMargin.Top, 0.0f, 0.0f);
}

int32 SDisplayClusterConfiguratorHostNode::GetNodeOriginLayerOffset() const
{
	if (!IsGraphNodeValid())
	{
		return 0;
	}

	UDisplayClusterConfiguratorHostNode* HostEdNode = GetGraphNodeChecked<UDisplayClusterConfiguratorHostNode>();
	int32 NodeLayerIndex = HostEdNode->GetNodeLayer(ObjectPtrDecay(GetOwnerPanel()->SelectionManager.SelectedNodes));
	int32 OrnamentLayerIndex = DisplayClusterConfiguratorGraphLayers::OrnamentLayerIndex;

	return OrnamentLayerIndex - NodeLayerIndex;
}

EVisibility SDisplayClusterConfiguratorHostNode::GetHostNameVisibility() const
{
	if (!IsGraphNodeValid())
	{
		return EVisibility::Collapsed;
	}

	UDisplayClusterConfiguratorHostNode* PCEdNode = GetGraphNodeChecked<UDisplayClusterConfiguratorHostNode>();
	const FText HostName = PCEdNode->GetHostName();

	return HostName.IsEmptyOrWhitespace() ? EVisibility::Collapsed : EVisibility::Visible;
}

FText SDisplayClusterConfiguratorHostNode::GetHostNameText() const
{
	if (!IsGraphNodeValid())
	{
		return FText::GetEmpty();
	}

	UDisplayClusterConfiguratorHostNode* PCEdNode = GetGraphNodeChecked<UDisplayClusterConfiguratorHostNode>();
	return PCEdNode->GetHostName();
}

FText SDisplayClusterConfiguratorHostNode::GetHostIPText() const
{
	if (!IsGraphNodeValid())
	{
		return FText::GetEmpty();
	}

	UDisplayClusterConfiguratorHostNode* PCEdNode = GetGraphNodeChecked<UDisplayClusterConfiguratorHostNode>();
	FString IPAddress = PCEdNode->GetNodeName();
	return FText::FromString(*IPAddress);
}

FText SDisplayClusterConfiguratorHostNode::GetHostResolutionText() const
{
	if (!IsGraphNodeValid())
	{
		return FText::GetEmpty();
	}

	UDisplayClusterConfiguratorHostNode* HostEdNode = GetGraphNodeChecked<UDisplayClusterConfiguratorHostNode>();
	const FVector2D NodeSize = HostEdNode->GetNodeLocalSize();
	return FText::Format(LOCTEXT("HostResolution_Formatted", "[{0} x {1}]"), FText::AsNumber(FMath::RoundToInt(NodeSize.X)), FText::AsNumber(FMath::RoundToInt(NodeSize.Y)));
}

FSlateColor SDisplayClusterConfiguratorHostNode::GetBorderColor() const
{
	if (!IsGraphNodeValid())
	{
		return FSlateColor();
	}

	if (GetOwnerPanel()->SelectionManager.SelectedNodes.Contains(GraphNode))
	{
		// Selected Case
		return FDisplayClusterConfiguratorStyle::Get().GetColor("DisplayClusterConfigurator.Node.Color.Selected");
	}

	// Regular case
	UDisplayClusterConfiguratorHostNode* PCEdNode = GetGraphNodeChecked<UDisplayClusterConfiguratorHostNode>();
	return PCEdNode->GetHostColor();
}


FSlateColor SDisplayClusterConfiguratorHostNode::GetTextColor() const
{
	if (!IsGraphNodeValid())
	{
		return FSlateColor();
	}

	if (GetOwnerPanel()->SelectionManager.SelectedNodes.Contains(GraphNode))
	{
		// Selected Case
		return FDisplayClusterConfiguratorStyle::Get().GetColor("DisplayClusterConfigurator.Node.Text.Color.Selected");
	}

	return FDisplayClusterConfiguratorStyle::Get().GetColor("DisplayClusterConfigurator.Node.Text.Color.Regular");
}

EVisibility SDisplayClusterConfiguratorHostNode::GetLockIconVisibility() const
{
	return !IsNodeUnlocked() ? EVisibility::Visible : EVisibility::Hidden;
}

#undef LOCTEXT_NAMESPACE
