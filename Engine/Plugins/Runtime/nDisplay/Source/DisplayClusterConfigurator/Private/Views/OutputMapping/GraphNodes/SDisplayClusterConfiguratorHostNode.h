// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Views/OutputMapping/GraphNodes/SDisplayClusterConfiguratorBaseNode.h"

class FDisplayClusterConfiguratorBlueprintEditor;
class UDisplayClusterConfiguratorHostNode;

class SDisplayClusterConfiguratorHostNode
	: public SDisplayClusterConfiguratorBaseNode
{
public:
	friend class SNodeOrigin;

	SLATE_BEGIN_ARGS(SDisplayClusterConfiguratorHostNode)
		: _BorderThickness(FMargin(1))
	{}
		SLATE_ARGUMENT(FMargin, BorderThickness)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UDisplayClusterConfiguratorHostNode* InHostNode, const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& InToolkit);

	//~ Begin SGraphNode interface
	virtual void UpdateGraphNode() override;
	virtual void MoveTo(const FVector2f& NewPosition, FNodeSet& NodeFilter, bool bMarkDirty = true) override;
	virtual FVector2D ComputeDesiredSize(float) const override;
	virtual FVector2f GetPosition2f() const override;
	//~ End SGraphNode interface

	//~ Begin SDisplayClusterConfiguratorBaseNode interface
	virtual void SetNodeSize(const FVector2D InLocalSize, bool bFixedAspectRatio) override;
	virtual bool CanNodeBeSnapAligned() const override { return true; }
	virtual bool CanNodeBeResized() const;
	//~ End SDisplayClusterConfiguratorBaseNode interface

private:
	FMargin GetBackgroundPosition() const;
	FMargin GetNodeOriginPosition() const;
	int32 GetNodeOriginLayerOffset() const;

	EVisibility GetHostNameVisibility() const;
	FText GetHostNameText() const;
	FText GetHostIPText() const;
	FText GetHostResolutionText() const;

	FSlateColor GetBorderColor() const;
	FSlateColor GetTextColor() const;

	EVisibility GetLockIconVisibility() const;

private:
	FMargin BorderThickness;
};