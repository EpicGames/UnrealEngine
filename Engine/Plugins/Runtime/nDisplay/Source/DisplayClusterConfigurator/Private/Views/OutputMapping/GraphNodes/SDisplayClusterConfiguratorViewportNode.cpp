// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/OutputMapping/GraphNodes/SDisplayClusterConfiguratorViewportNode.h"

#include "DisplayClusterConfiguratorStyle.h"
#include "DisplayClusterConfiguratorBlueprintEditor.h"
#include "DisplayClusterConfigurationTypes.h"
#include "TextureResource.h"
#include "Views/TreeViews/IDisplayClusterConfiguratorTreeItem.h"
#include "Views/OutputMapping/IDisplayClusterConfiguratorViewOutputMapping.h"
#include "Views/OutputMapping/EdNodes/DisplayClusterConfiguratorViewportNode.h"
#include "Views/OutputMapping/EdNodes/DisplayClusterConfiguratorWindowNode.h"
#include "Views/OutputMapping/Widgets/SDisplayClusterConfiguratorResizer.h"
#include "Views/OutputMapping/Widgets/SDisplayClusterConfiguratorLayeringBox.h"

#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SConstraintCanvas.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Images/SImage.h"
#include "SGraphPanel.h"

#define LOCTEXT_NAMESPACE "SDisplayClusterConfiguratorViewportNode"

void SDisplayClusterConfiguratorViewportNode::Construct(const FArguments& InArgs,
                                                        UDisplayClusterConfiguratorViewportNode* InViewportNode,
                                                        const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& InToolkit)
{
	SDisplayClusterConfiguratorBaseNode::Construct(SDisplayClusterConfiguratorBaseNode::FArguments(), InViewportNode, InToolkit);

	UpdateGraphNode();
}

void SDisplayClusterConfiguratorViewportNode::UpdateGraphNode()
{
	SDisplayClusterConfiguratorBaseNode::UpdateGraphNode();

	if (!IsGraphNodeValid())
	{
		return;
	}

	BackgroundImage = SNew(SImage)
		.ColorAndOpacity(this, &SDisplayClusterConfiguratorViewportNode::GetBackgroundColor)
		.Image(this, &SDisplayClusterConfiguratorViewportNode::GetBackgroundBrush)
		.RenderTransform(this, &SDisplayClusterConfiguratorViewportNode::GetBackgroundRenderTransform)
		.RenderTransformPivot(FVector2D(0.5f, 0.5f));

	UDisplayClusterConfiguratorViewportNode* ViewportEdNode = GetGraphNodeChecked<UDisplayClusterConfiguratorViewportNode>();

	UpdatePreviewTexture();

	GetOrAddSlot( ENodeZone::Center )
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Fill)
	[
		SNew(SDisplayClusterConfiguratorLayeringBox)
		.LayerOffset(this, &SDisplayClusterConfiguratorViewportNode::GetNodeVisualLayer)
		.ShadowBrush(this, &SDisplayClusterConfiguratorViewportNode::GetNodeShadowBrush)
		[
			SNew(SBox)
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				[
					SNew(SOverlay)

					+ SOverlay::Slot()
					.VAlign(VAlign_Fill)
					.HAlign(HAlign_Fill)
					[
						SNew(SBorder)
						.Padding(FMargin(0.f))
						[
							BackgroundImage.ToSharedRef()
						]
					]

					+ SOverlay::Slot()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Center)
					.Padding(FMargin(15.f, 12.f))
					[
						SNew(SBox)
						[
							SNew(SScaleBox)
							.Stretch(EStretch::ScaleToFit)
							.StretchDirection(EStretchDirection::DownOnly)
							.VAlign(VAlign_Center)
							[
								SNew(SBorder)
								.BorderImage(FAppStyle::GetBrush("WhiteBrush"))
								.BorderBackgroundColor(this, &SDisplayClusterConfiguratorViewportNode::GetTextBoxColor)
								.ForegroundColor(this, &SDisplayClusterConfiguratorViewportNode::GetTextColor)
								.Padding(8.0f)
								[
									SNew( SVerticalBox )

									+ SVerticalBox::Slot()
									.VAlign(VAlign_Center)
									.Padding(5.f, 2.f)
									[
										SNew(STextBlock)
										.Text(FText::FromString(ViewportEdNode->GetNodeName()))
										.Justification(ETextJustify::Center)
										.TextStyle(&FDisplayClusterConfiguratorStyle::Get().GetWidgetStyle<FTextBlockStyle>("DisplayClusterConfigurator.Node.Text.Bold"))
										.ColorAndOpacity(FSlateColor::UseForeground())
									]

									+ SVerticalBox::Slot()
									.VAlign(VAlign_Center)
									.Padding(5.f, 2.f)
									[
										SNew(STextBlock)
										.Text(this, &SDisplayClusterConfiguratorViewportNode::GetPositionAndSizeText)
										.Justification(ETextJustify::Center)
										.TextStyle(&FDisplayClusterConfiguratorStyle::Get().GetWidgetStyle<FTextBlockStyle>("DisplayClusterConfigurator.Node.Text.Regular"))
										.ColorAndOpacity(FSlateColor::UseSubduedForeground())
									]

									+ SVerticalBox::Slot()
									.VAlign(VAlign_Center)
									.Padding(5.f, 2.f)
									[
										SNew(STextBlock)
										.Text(this, &SDisplayClusterConfiguratorViewportNode::GetTransformText)
										.Justification(ETextJustify::Center)
										.TextStyle(&FDisplayClusterConfiguratorStyle::Get().GetWidgetStyle<FTextBlockStyle>("DisplayClusterConfigurator.Node.Text.Regular"))
										.ColorAndOpacity(FSlateColor::UseSubduedForeground())
										.Visibility(this, &SDisplayClusterConfiguratorViewportNode::GetTransformTextVisibility)
									]

									+ SVerticalBox::Slot()
									.VAlign(VAlign_Center)
									.HAlign(HAlign_Center)
									.AutoHeight()
									.Padding(5.0f, 2.0f)
									[
										SNew(SBox)
										.WidthOverride(32)
										.HeightOverride(32)
										.Visibility(this, &SDisplayClusterConfiguratorViewportNode::GetLockIconVisibility)
										[
											SNew(SImage)
											.Image(FAppStyle::Get().GetBrush(TEXT("PropertyWindow.Locked")))
											.ColorAndOpacity(FSlateColor::UseSubduedForeground())
										]
									]
								]
							]
						]
					]

					+ SOverlay::Slot()
					.VAlign(VAlign_Fill)
					.HAlign(HAlign_Fill)
					[
						SNew(SBorder)
						.BorderImage(this, &SDisplayClusterConfiguratorViewportNode::GetBorderBrush)
					]
				]
			]
		]
	];
}

void SDisplayClusterConfiguratorViewportNode::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SDisplayClusterConfiguratorBaseNode::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	UpdatePreviewTexture();
}

bool SDisplayClusterConfiguratorViewportNode::IsNodeVisible() const
{
	if (!IsGraphNodeValid())
	{
		return false;
	}

	UDisplayClusterConfiguratorViewportNode* ViewportEdNode = GetGraphNodeChecked<UDisplayClusterConfiguratorViewportNode>();
	TSharedPtr<FDisplayClusterConfiguratorBlueprintEditor> Toolkit = ToolkitPtr.Pin();
	check(Toolkit.IsValid());

	TSharedRef<IDisplayClusterConfiguratorViewOutputMapping> OutputMapping = Toolkit->GetViewOutputMapping();

	const bool bIsSelected = GetOwnerPanel()->SelectionManager.SelectedNodes.Contains(GraphNode);
	const bool bIsVisible = bIsSelected || OutputMapping->GetOutputMappingSettings().bShowOutsideViewports || !ViewportEdNode->IsOutsideParent();
	return SDisplayClusterConfiguratorBaseNode::IsNodeVisible() && bIsVisible;
}

float SDisplayClusterConfiguratorViewportNode::GetNodeMinimumSize() const
{
	return UDisplayClusterConfigurationViewport::ViewportMinimumSize;
}

float SDisplayClusterConfiguratorViewportNode::GetNodeMaximumSize() const
{
	return UDisplayClusterConfigurationViewport::ViewportMaximumSize;
}

bool SDisplayClusterConfiguratorViewportNode::IsAspectRatioFixed() const
{
	if (!IsGraphNodeValid())
	{
		return false;
	}

	UDisplayClusterConfiguratorViewportNode* ViewportEdNode = GetGraphNodeChecked<UDisplayClusterConfiguratorViewportNode>();
	return ViewportEdNode->IsFixedAspectRatio();
}

void SDisplayClusterConfiguratorViewportNode::BeginUserInteraction() const
{
	SDisplayClusterConfiguratorBaseNode::BeginUserInteraction();

	if (!IsGraphNodeValid())
	{
		return;
	}

	const UDisplayClusterConfiguratorViewportNode* ViewportEdNode = GetGraphNodeChecked<UDisplayClusterConfiguratorViewportNode>();
	UDisplayClusterConfigurationViewport* Viewport = ViewportEdNode->GetObjectChecked<UDisplayClusterConfigurationViewport>();
	bIsManagingPreviewTexture = Viewport->DisablePreviewTexture();
}

void SDisplayClusterConfiguratorViewportNode::EndUserInteraction() const
{
	SDisplayClusterConfiguratorBaseNode::EndUserInteraction();
	
	if (!IsGraphNodeValid())
	{
		return;
	}

	if (bIsManagingPreviewTexture)
	{
		const UDisplayClusterConfiguratorViewportNode* ViewportEdNode = GetGraphNodeChecked<UDisplayClusterConfiguratorViewportNode>();
		UDisplayClusterConfigurationViewport* Viewport = ViewportEdNode->GetObjectChecked<UDisplayClusterConfigurationViewport>();
		
		Viewport->EnablePreviewTexture();
		bIsManagingPreviewTexture = false;
	}
}

FSlateColor SDisplayClusterConfiguratorViewportNode::GetBackgroundColor() const
{
	TSharedPtr<FDisplayClusterConfiguratorBlueprintEditor> Toolkit = ToolkitPtr.Pin();
	check(Toolkit.IsValid());

	if (!IsGraphNodeValid())
	{
		return FSlateColor();
	}

	TSharedRef<IDisplayClusterConfiguratorViewOutputMapping> OutputMapping = Toolkit->GetViewOutputMapping();

	const bool bIsSelected = GetOwnerPanel()->SelectionManager.SelectedNodes.Contains(GraphNode);
	const bool bHasImageBackground = BackgroundActiveBrush.GetResourceObject() != nullptr;
	const bool bIsLocked = !IsNodeUnlocked();
	const bool bTintBackground = OutputMapping->GetOutputMappingSettings().bTintSelectedViewports;

	UDisplayClusterConfiguratorViewportNode* ViewportEdNode = GetGraphNodeChecked<UDisplayClusterConfiguratorViewportNode>();
	if (ViewportEdNode->IsOutsideParentBoundary())
	{
		if (bIsSelected)
		{
			// Selected Case
			return FDisplayClusterConfiguratorStyle::Get().GetColor("DisplayClusterConfigurator.Node.Viewport.OutsideBackgroundColor.Selected");
		}
		else
		{
			// Regular case
			return FDisplayClusterConfiguratorStyle::Get().GetColor("DisplayClusterConfigurator.Node.Viewport.OutsideBackgroundColor.Regular");
		}
	}
	else
	{
		if (bHasImageBackground)
		{
			if (bIsSelected && bTintBackground)
			{
				return FDisplayClusterConfiguratorStyle::Get().GetColor("DisplayClusterConfigurator.Node.Viewport.BackgroundImage.Selected");
			}
			else if (bIsLocked)
			{
				return FDisplayClusterConfiguratorStyle::Get().GetColor("DisplayClusterConfigurator.Node.Viewport.BackgroundImage.Locked");
			}
			else
			{
				return FLinearColor::White;
			}
		}
		else
		{
			if (bIsSelected && bTintBackground)
			{
				// Selected Case
				return FDisplayClusterConfiguratorStyle::Get().GetColor("DisplayClusterConfigurator.Node.Viewport.BackgroundColor.Selected");
			}
			else
			{
				// Regular case
				return FDisplayClusterConfiguratorStyle::Get().GetColor("DisplayClusterConfigurator.Node.Viewport.BackgroundColor.Regular");
			}
		}
	}

}

const FSlateBrush* SDisplayClusterConfiguratorViewportNode::GetBackgroundBrush() const
{
	if (BackgroundActiveBrush.GetResourceObject() != nullptr)
	{
		return &BackgroundActiveBrush;
	}
	else
	{
		return FAppStyle::Get().GetBrush("WhiteBrush");
	}
}

TOptional<FSlateRenderTransform> SDisplayClusterConfiguratorViewportNode::GetBackgroundRenderTransform() const
{
	if (!IsGraphNodeValid())
	{
		return TOptional<FSlateRenderTransform>();
	}

	UDisplayClusterConfiguratorViewportNode* ViewportEdNode = GetGraphNodeChecked<UDisplayClusterConfiguratorViewportNode>();
	const FDisplayClusterConfigurationRectangle& Region = ViewportEdNode->GetCfgViewportRegion();
	const FDisplayClusterConfigurationViewport_RemapData& RemapData = ViewportEdNode->GetCfgViewportRemap();

	FMatrix2x2 TransformMat = FMatrix2x2();
	
	if (RemapData.IsFlipping())
	{
		FScale2D Scale = FScale2D(RemapData.bFlipH ? -1.0f : 1.0f, RemapData.bFlipV ? -1.0f : 1.0f);
		TransformMat = TransformMat.Concatenate(FMatrix2x2(Scale));
	}

	if (RemapData.IsRotating())
	{
		// Since the size of the node is changing to match the bounds of the rotated viewport, which scales the image,
		// we need to undo that scaling to make the image the appropriate size before rotating the image
		float SinAngle, CosAngle;
		FMath::SinCos(&SinAngle, &CosAngle, FMath::DegreesToRadians(RemapData.Angle));

		FVector2D RotatedSize;
		RotatedSize.X = Region.W * FMath::Abs(CosAngle) + Region.H * FMath::Abs(SinAngle);
		RotatedSize.Y = Region.W * FMath::Abs(SinAngle) + Region.H * FMath::Abs(CosAngle);

		FMatrix2x2 RotMat = FMatrix2x2(FQuat2D(FMath::DegreesToRadians(RemapData.Angle)));
		FMatrix2x2 ScaleMat = FMatrix2x2(FScale2D(Region.W / RotatedSize.X, Region.H / RotatedSize.Y));

		TransformMat = TransformMat.Concatenate(ScaleMat.Concatenate(RotMat));
	}

	return FSlateRenderTransform(TransformMat);
}

const FSlateBrush* SDisplayClusterConfiguratorViewportNode::GetNodeShadowBrush() const
{
	return FAppStyle::GetBrush(TEXT("Graph.Node.Shadow"));
}

const FSlateBrush* SDisplayClusterConfiguratorViewportNode::GetBorderBrush() const
{
	if (!IsGraphNodeValid())
	{
		return nullptr;
	}

	if (GetOwnerPanel()->SelectionManager.SelectedNodes.Contains(GraphNode))
	{
		return FDisplayClusterConfiguratorStyle::Get().GetBrush("DisplayClusterConfigurator.Node.Viewport.Border.Brush.Selected");
	}
	else
	{
		UDisplayClusterConfiguratorViewportNode* ViewportEdNode = GetGraphNodeChecked<UDisplayClusterConfiguratorViewportNode>();
		if (ViewportEdNode->IsOutsideParentBoundary())
		{
			return FDisplayClusterConfiguratorStyle::Get().GetBrush("DisplayClusterConfigurator.Node.Viewport.Border.Brush.Outside");
		}
		else
		{
			return FDisplayClusterConfiguratorStyle::Get().GetBrush("DisplayClusterConfigurator.Node.Viewport.Border.Brush.Regular");
		}
	}
}

FSlateColor SDisplayClusterConfiguratorViewportNode::GetTextBoxColor() const
{
	if (GetOwnerPanel()->SelectionManager.SelectedNodes.Contains(GraphNode))
	{
		return FDisplayClusterConfiguratorStyle::Get().GetColor("DisplayClusterConfigurator.Node.Color.Selected");
	}

	return FDisplayClusterConfiguratorStyle::Get().GetColor("DisplayClusterConfigurator.Node.Viewport.Text.Background");
}

FSlateColor SDisplayClusterConfiguratorViewportNode::GetTextColor() const
{
	if (GetOwnerPanel()->SelectionManager.SelectedNodes.Contains(GraphNode))
	{
		return FDisplayClusterConfiguratorStyle::Get().GetColor("DisplayClusterConfigurator.Node.Text.Color.Selected");
	}

	return FDisplayClusterConfiguratorStyle::Get().GetColor("DisplayClusterConfigurator.Node.Text.Color.Regular");
}

FText SDisplayClusterConfiguratorViewportNode::GetPositionAndSizeText() const
{
	if (!IsGraphNodeValid())
	{
		return FText::GetEmpty();
	}

	UDisplayClusterConfiguratorViewportNode* ViewportEdNode = GetGraphNodeChecked<UDisplayClusterConfiguratorViewportNode>();
	const FDisplayClusterConfigurationRectangle& CfgViewportRegion = ViewportEdNode->GetCfgViewportRegion();

	return FText::Format(LOCTEXT("ResAndOffset", "[{0} x {1}] @ {2}, {3}"), CfgViewportRegion.W, CfgViewportRegion.H, CfgViewportRegion.X, CfgViewportRegion.Y);
}

FText SDisplayClusterConfiguratorViewportNode::GetTransformText() const
{
	if (!IsGraphNodeValid())
	{
		return FText::GetEmpty();
	}

	UDisplayClusterConfiguratorViewportNode* ViewportEdNode = GetGraphNodeChecked<UDisplayClusterConfiguratorViewportNode>();
	const FDisplayClusterConfigurationViewport_RemapData& RemapData = ViewportEdNode->GetCfgViewportRemap();

	TArray<FText> TransformText;
	if (RemapData.IsRotating())
	{
		// Get the rotation angle expressed from -180 to 180 degrees
		const float RotAngle = FRotator::NormalizeAxis(RemapData.Angle);

		FText RotDirectionText;
		if (RotAngle < 0)
		{
			RotDirectionText = LOCTEXT("ConterClockwiseLabel", "CCW");
		}
		else
		{
			RotDirectionText = LOCTEXT("ClockwiseLabel", "CW");
		}

		TransformText.Add(FText::Format(LOCTEXT("RotationFormat", "Rotated {0}\u00b0 {1}"), FMath::Abs(RotAngle), RotDirectionText));
	}

	if (RemapData.IsFlipping())
	{
		if (RemapData.bFlipH)
		{
			TransformText.Add(LOCTEXT("HorizontalFlip", "Flipped Horizontally"));
		}

		if (RemapData.bFlipV)
		{
			TransformText.Add(LOCTEXT("VerticalFlip", "Flipped Vertically"));
		}
	}

	return FText::Join(FText::FromString(TEXT(", ")), TransformText);
}

EVisibility SDisplayClusterConfiguratorViewportNode::GetTransformTextVisibility() const
{
	if (!IsGraphNodeValid())
	{
		return EVisibility::Collapsed;
	}

	UDisplayClusterConfiguratorViewportNode* ViewportEdNode = GetGraphNodeChecked<UDisplayClusterConfiguratorViewportNode>();
	const FDisplayClusterConfigurationViewport_RemapData& RemapData = ViewportEdNode->GetCfgViewportRemap();
	return RemapData.IsValid() ? EVisibility::Visible : EVisibility::Collapsed;
}

FMargin SDisplayClusterConfiguratorViewportNode::GetBackgroundPosition() const
{
	const FVector2D NodeSize = GetSize();
	return FMargin(0.f, 0.f, NodeSize.X, NodeSize.Y);
}

FMargin SDisplayClusterConfiguratorViewportNode::GetAreaResizeHandlePosition() const
{
	const FVector2D NodeSize = GetSize();
	return FMargin(NodeSize.X, NodeSize.Y, 0.f, 0.f);
}

bool SDisplayClusterConfiguratorViewportNode::IsViewportLocked() const
{
	TSharedPtr<FDisplayClusterConfiguratorBlueprintEditor> Toolkit = ToolkitPtr.Pin();
	check(Toolkit.IsValid());

	TSharedRef<IDisplayClusterConfiguratorViewOutputMapping> OutputMapping = Toolkit->GetViewOutputMapping();
	return OutputMapping->GetOutputMappingSettings().bLockViewports;
}

EVisibility SDisplayClusterConfiguratorViewportNode::GetLockIconVisibility() const
{
	return !IsNodeUnlocked() ? EVisibility::Visible : EVisibility::Collapsed;
}

void SDisplayClusterConfiguratorViewportNode::UpdatePreviewTexture()
{
	if (!IsGraphNodeValid())
	{
		return;
	}

	UDisplayClusterConfiguratorViewportNode* ViewportEdNode = GetGraphNodeChecked<UDisplayClusterConfiguratorViewportNode>();
	UTexture* CurrentTexture = ViewportEdNode->GetPreviewTexture();

	if (CachedTexture != CurrentTexture)
	{
		CachedTexture = CurrentTexture;
		if (CachedTexture != nullptr)
		{
			if (BackgroundActiveBrush.GetResourceObject() != CachedTexture && CachedTexture->GetResource() != nullptr)
			{
				BackgroundActiveBrush = FSlateBrush();
				BackgroundActiveBrush.SetResourceObject(CachedTexture);
				BackgroundActiveBrush.ImageSize.X = CachedTexture->GetResource()->GetSizeX();
				BackgroundActiveBrush.ImageSize.Y = CachedTexture->GetResource()->GetSizeY();
			}
		}
		else
		{
			// Reset the brush to be empty.
			BackgroundActiveBrush = FSlateBrush();
		}
	}
}

#undef LOCTEXT_NAMESPACE
