// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Views/OutputMapping/GraphNodes/SDisplayClusterConfiguratorBaseNode.h"

#include "UObject/WeakObjectPtr.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class FDisplayClusterConfiguratorOutputMappingViewportSlot;
class FDisplayClusterConfiguratorBlueprintEditor;
class IDisplayClusterConfiguratorTreeItem;
class SImage;
class UDisplayClusterConfigurationViewport;
class UDisplayClusterConfiguratorViewportNode;
class UTexture;

class SDisplayClusterConfiguratorViewportNode
	: public SDisplayClusterConfiguratorBaseNode
{
public:
	SLATE_BEGIN_ARGS(SDisplayClusterConfiguratorViewportNode)
	{}
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs,
		UDisplayClusterConfiguratorViewportNode* InViewportNode,
		const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& InToolkit);

	//~ Begin SGraphNode interface
	virtual void UpdateGraphNode() override;
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	//~ End of SGraphNode interface

	//~ Begin SDisplayClusterConfiguratorBaseNode interface
	virtual bool IsNodeVisible() const override;
	virtual bool CanNodeBeSnapAligned() const override { return true; }
	virtual bool CanNodeBeResized() const { return IsNodeUnlocked(); }
	virtual float GetNodeMinimumSize() const override;
	virtual float GetNodeMaximumSize() const override;
	virtual bool IsAspectRatioFixed() const override;
	virtual void BeginUserInteraction() const override;
	virtual void EndUserInteraction() const override;
	//~ End of SDisplayClusterConfiguratorBaseNode interface

private:
	FSlateColor GetBackgroundColor() const;
	const FSlateBrush* GetBackgroundBrush() const;
	TOptional<FSlateRenderTransform> GetBackgroundRenderTransform() const;
	const FSlateBrush* GetNodeShadowBrush() const;
	const FSlateBrush* GetBorderBrush() const;
	FSlateColor GetTextBoxColor() const;
	FSlateColor GetTextColor() const;
	FText GetPositionAndSizeText() const;
	FText GetTransformText() const;
	EVisibility GetTransformTextVisibility() const;
	FMargin GetBackgroundPosition() const;
	FMargin GetAreaResizeHandlePosition() const;
	bool IsViewportLocked() const;
	EVisibility GetLockIconVisibility() const;

	void UpdatePreviewTexture();

private:
	FSlateBrush BackgroundActiveBrush;
	TSharedPtr<SImage> BackgroundImage;

	UTexture* CachedTexture;
	
	/** If this object is managing the preview texture, such as through resizing. */
	mutable bool bIsManagingPreviewTexture = false;
};
