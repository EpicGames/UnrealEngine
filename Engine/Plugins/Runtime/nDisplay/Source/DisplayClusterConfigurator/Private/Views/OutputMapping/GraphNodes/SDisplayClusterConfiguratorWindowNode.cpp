// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/OutputMapping/GraphNodes/SDisplayClusterConfiguratorWindowNode.h"

#include "DisplayClusterConfiguratorStyle.h"
#include "DisplayClusterConfiguratorBlueprintEditor.h"
#include "DisplayClusterConfigurationTypes.h"
#include "Framework/Application/SlateApplication.h"
#include "Views/TreeViews/IDisplayClusterConfiguratorTreeItem.h"
#include "Views/OutputMapping/IDisplayClusterConfiguratorViewOutputMapping.h"
#include "Views/OutputMapping/EdNodes/DisplayClusterConfiguratorWindowNode.h"
#include "Views/OutputMapping/EdNodes/DisplayClusterConfiguratorViewportNode.h"
#include "Views/OutputMapping/GraphNodes/SDisplayClusterConfiguratorViewportNode.h"
#include "Views/OutputMapping/Widgets/SDisplayClusterConfiguratorResizer.h"
#include "Views/OutputMapping/Widgets/SDisplayClusterConfiguratorLayeringBox.h"
#include "Views/OutputMapping/Widgets/SDisplayClusterConfiguratorExternalImage.h"

#include "SGraphPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SConstraintCanvas.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/Layout/SSpacer.h"

#define LOCTEXT_NAMESPACE "SDisplayClusterConfiguratorWindowNode"

class SCornerImage
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SCornerImage)
		: _ColorAndOpacity(FLinearColor::White)
		, _Size(FVector2D(60.f))
	{ }
		SLATE_ATTRIBUTE(FSlateColor, ColorAndOpacity)
		SLATE_ARGUMENT(FVector2D, Size)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<SDisplayClusterConfiguratorWindowNode> InParentNode)
	{
		ParentNode = InParentNode;

		SetCursor(EMouseCursor::CardinalCross);

		ChildSlot
		[
			SNew(SBox)
			.WidthOverride(InArgs._Size.X)
			.HeightOverride(InArgs._Size.Y)
			[
				SNew(SImage)
				.ColorAndOpacity(InArgs._ColorAndOpacity)
				.Image(FDisplayClusterConfiguratorStyle::Get().GetBrush("DisplayClusterConfigurator.Node.Brush.Corner"))
			]
		];
	}

	virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		// A little hack to ensure that the user can select or drag the parent window node by clicking on the corner widget. The SNodePanel that manages
		// mouse interaction for the graph editor sorts the node widgets by their sort depth to determine which node widget to select and drag, and overlay
		// widgets are not hit tested. By default, windows are always lower than viewports in their sort order to ensure viewports are always selectable over
		// windows, but the one exception is when the user clicks on the corner widget. To ensure that the window widget is selected, increase the window's 
		// z-index temporarily as long as the mouse is over the corner widget.
		SCompoundWidget::OnMouseEnter(MyGeometry, MouseEvent);
		ParentNode->bLayerAboveViewports = true;
	}

	void OnMouseLeave(const FPointerEvent& MouseEvent)
	{
		SCompoundWidget::OnMouseLeave(MouseEvent);
		ParentNode->bLayerAboveViewports = false;
	}

private:
	TSharedPtr<SDisplayClusterConfiguratorWindowNode> ParentNode;
};

class SNodeInfo
	: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNodeInfo)
		: _ColorAndOpacity(FLinearColor::White)
		, _ZIndexOffset(0)
	{ }
		SLATE_ATTRIBUTE(FSlateColor, ColorAndOpacity)
		SLATE_ATTRIBUTE(FSlateColor, ForegroundColor)
		SLATE_ARGUMENT(int32, ZIndexOffset)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<SDisplayClusterConfiguratorWindowNode>& InParentNode)
	{
		ParentNode = InParentNode;
		ZIndexOffset = InArgs._ZIndexOffset;

		SetCursor(EMouseCursor::CardinalCross);

		ChildSlot
		[
			SNew(SBox)
			.WidthOverride(this, &SNodeInfo::GetTitleWidth)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
				SNew(SScaleBox)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Fill)
				.Stretch(EStretch::ScaleToFill)
				.StretchDirection(EStretchDirection::DownOnly)
				[
					SNew(SBorder)
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.BorderImage(FAppStyle::GetBrush("WhiteBrush"))
					.BorderBackgroundColor(InArgs._ColorAndOpacity)
					.Padding(FMargin(20, 10, 30, 10))
					.ForegroundColor(InArgs._ForegroundColor)
					[
						SNew(SHorizontalBox)

						+ SHorizontalBox::Slot()
						.AutoWidth()
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						[
							SNew(SBox)
							.WidthOverride(36)
							.HeightOverride(36)
							[
								SNew(SImage)
								.Image(FDisplayClusterConfiguratorStyle::Get().GetBrush(TEXT("DisplayClusterConfigurator.TreeItems.ClusterNode")))
								.ColorAndOpacity(FSlateColor::UseForeground())
							]
						]
		
						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(SSpacer)
							.Size(FVector2D(15, 1))
						]

						+ SHorizontalBox::Slot()
						.AutoWidth()
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Text(this, &SNodeInfo::GetNodeName)
							.TextStyle(&FDisplayClusterConfiguratorStyle::Get().GetWidgetStyle<FTextBlockStyle>("DisplayClusterConfigurator.Node.Text.Bold"))
							.ColorAndOpacity(FSlateColor::UseForeground())
						]
		
						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(SSpacer)
							.Size(FVector2D(25, 1))
						]

						+ SHorizontalBox::Slot()
						.AutoWidth()
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Text(this, &SNodeInfo::GetPositionAndSizeText)
							.TextStyle(&FDisplayClusterConfiguratorStyle::Get().GetWidgetStyle<FTextBlockStyle>("DisplayClusterConfigurator.Node.Text.Regular"))
							.ColorAndOpacity(FSlateColor::UseSubduedForeground())
						]
		
						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(SSpacer)
							.Size(FVector2D(25, 1))
						]

						+ SHorizontalBox::Slot()
						.AutoWidth()
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						[
							SNew(SBox)
							.WidthOverride(36)
							.HeightOverride(36)
							.Visibility(this, &SNodeInfo::GetLockIconVisibility)
							[
								SNew(SImage)
								.Image(FAppStyle::Get().GetBrush(TEXT("PropertyWindow.Locked")))
								.ColorAndOpacity(FSlateColor::UseSubduedForeground())
							]
						]
					]
				]
			]
		];
	}

	virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		// A little hack to ensure that the user can select or drag the parent window node by clicking on the corner widget. The SNodePanel that manages
		// mouse interaction for the graph editor sorts the node widgets by their sort depth to determine which node widget to select and drag, and overlay
		// widgets are not hit tested. By default, windows are always lower than viewports in their sort order to ensure viewports are always selectable over
		// windows, but the one exception is when the user clicks on the corner widget. To ensure that the window widget is selected, increase the window's 
		// z-index temporarily as long as the mouse is over the corner widget.
		SCompoundWidget::OnMouseEnter(MyGeometry, MouseEvent);
		ParentNode->bLayerAboveViewports = true;
	}

	void OnMouseLeave(const FPointerEvent& MouseEvent)
	{
		SCompoundWidget::OnMouseLeave(MouseEvent);
		ParentNode->bLayerAboveViewports = false;
	}

	FText GetNodeName() const
	{
		if (!ParentNode->IsGraphNodeValid())
		{
			return FText::GetEmpty();
		}

		UDisplayClusterConfiguratorWindowNode* WindowEdNode = ParentNode->GetGraphNodeChecked<UDisplayClusterConfiguratorWindowNode>();
		const FText NodeName = FText::FromString(WindowEdNode->GetNodeName());
		if (WindowEdNode->IsPrimary())
		{
			return FText::Format(LOCTEXT("WindowNameWithPrimary", "{0} (Primary)"), NodeName);
		}
		else
		{
			return NodeName;
		}
	}

	FText GetPositionAndSizeText() const
	{
		if (!ParentNode->IsGraphNodeValid())
		{
			return FText::GetEmpty();
		}

		UDisplayClusterConfiguratorWindowNode* WindowEdNode = ParentNode->GetGraphNodeChecked<UDisplayClusterConfiguratorWindowNode>();
		FDisplayClusterConfigurationRectangle WindowRect = WindowEdNode->GetCfgWindowRect();
		return FText::Format(LOCTEXT("ResAndOffset", "[{0} x {1}] @ {2}, {3}"), WindowRect.W, WindowRect.H, WindowRect.X, WindowRect.Y);
	}

	FOptionalSize GetTitleWidth() const
	{
		if (!ParentNode->IsGraphNodeValid())
		{
			return FOptionalSize();
		}

		UDisplayClusterConfiguratorWindowNode* WindowEdNode = ParentNode->GetGraphNodeChecked<UDisplayClusterConfiguratorWindowNode>();
		return WindowEdNode->NodeWidth;
	}

	EVisibility GetLockIconVisibility() const
	{
		if (!ParentNode->IsNodeUnlocked())
		{
			return EVisibility::Visible;
		}

		return EVisibility::Collapsed;
	}

private:
	TSharedPtr<SDisplayClusterConfiguratorWindowNode> ParentNode;
	int32 ZIndexOffset;
};

SDisplayClusterConfiguratorWindowNode::~SDisplayClusterConfiguratorWindowNode()
{
	if (!IsGraphNodeValid())
	{
		return;
	}

	UDisplayClusterConfiguratorWindowNode* WindowEdNode = GetGraphNodeChecked<UDisplayClusterConfiguratorWindowNode>();
	WindowEdNode->UnregisterOnPreviewImageChanged(ImageChangedHandle);
}

void SDisplayClusterConfiguratorWindowNode::Construct(const FArguments& InArgs,
	UDisplayClusterConfiguratorWindowNode* InWindowNode,
	const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& InToolkit)
{
	SDisplayClusterConfiguratorBaseNode::Construct(SDisplayClusterConfiguratorBaseNode::FArguments(), InWindowNode, InToolkit);

	WindowScaleFactor = FVector2D(1, 1);

	InWindowNode->RegisterOnPreviewImageChanged(UDisplayClusterConfiguratorWindowNode::FOnPreviewImageChangedDelegate::CreateSP(this,
		&SDisplayClusterConfiguratorWindowNode::OnPreviewImageChanged));

	UpdateGraphNode();
}

void SDisplayClusterConfiguratorWindowNode::UpdateGraphNode()
{
	SDisplayClusterConfiguratorBaseNode::UpdateGraphNode();

	// Add the corner image and node info widgets to the top-left slot of the node.
	GetOrAddSlot(ENodeZone::TopLeft)
	.HAlign(HAlign_Left)
	.VAlign(VAlign_Top)
	[
		SNew(SDisplayClusterConfiguratorLayeringBox)
		.LayerOffset(this, &SDisplayClusterConfiguratorWindowNode::GetNodeTitleLayerOffset)
		.IsEnabled(this, &SDisplayClusterConfiguratorWindowNode::IsNodeEnabled)
		[
			SNew(SOverlay)

			+SOverlay::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Top)
			[
				SNew(SCornerImage, SharedThis(this))
				.Visibility(this, &SDisplayClusterConfiguratorWindowNode::GetCornerImageVisibility)
				.ColorAndOpacity(this, &SDisplayClusterConfiguratorWindowNode::GetCornerColor)
				.Size(FVector2D(128))
			]

			+SOverlay::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Top)
			[
				SNew(SNodeInfo, SharedThis(this))
				.Visibility(this, &SDisplayClusterConfiguratorWindowNode::GetNodeInfoVisibility)
				.ColorAndOpacity(this, &SDisplayClusterConfiguratorWindowNode::GetCornerColor)
				.ForegroundColor(this, &SDisplayClusterConfiguratorWindowNode::GetTextColor)
			]
		]
	];

	GetOrAddSlot(ENodeZone::Center)
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Fill)
	[
		SNew(SDisplayClusterConfiguratorLayeringBox)
		.LayerOffset(this, &SDisplayClusterConfiguratorWindowNode::GetNodeVisualLayer)
		.ShadowBrush(this, &SDisplayClusterConfiguratorWindowNode::GetNodeShadowBrush)
		[
			SNew(SBox)
			.VAlign(VAlign_Fill)
			.HAlign(HAlign_Fill)
			[
				SNew(SOverlay)

				+ SOverlay::Slot()
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Fill)
				[
					CreateBackground(FDisplayClusterConfiguratorStyle::Get().GetColor("DisplayClusterConfigurator.Node.Window.Inner.Background"))
				]

				+ SOverlay::Slot()
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Fill)
				[
					SNew(SDisplayClusterConfiguratorLayeringBox)
					.LayerOffset(this, &SDisplayClusterConfiguratorWindowNode::GetBorderLayerOffset)
					[
						SNew(SBorder)
						.BorderImage(this, &SDisplayClusterConfiguratorWindowNode::GetBorderBrush)
					]
				]
			]
		]
	];
}

FVector2D SDisplayClusterConfiguratorWindowNode::ComputeDesiredSize(float) const
{
	return GetSize() * WindowScaleFactor;
}

FVector2f SDisplayClusterConfiguratorWindowNode::GetPosition2f() const
{
	return SDisplayClusterConfiguratorBaseNode::GetPosition2f() * UE::Slate::CastToVector2f(WindowScaleFactor);
}

bool SDisplayClusterConfiguratorWindowNode::IsAspectRatioFixed() const
{
	if (!IsGraphNodeValid())
	{
		return false;
	}

	UDisplayClusterConfiguratorWindowNode* WindowEdNode = GetGraphNodeChecked<UDisplayClusterConfiguratorWindowNode>();
	return WindowEdNode->IsFixedAspectRatio();
}

int32 SDisplayClusterConfiguratorWindowNode::GetNodeLogicalLayer() const
{
	if (!IsGraphNodeValid())
	{
		return 0;
	}

	UDisplayClusterConfiguratorWindowNode* WindowEdNode = GetGraphNodeChecked<UDisplayClusterConfiguratorWindowNode>();
	if (bLayerAboveViewports)
	{
		return WindowEdNode->GetAuxiliaryLayer(ObjectPtrDecay(GetOwnerPanel()->SelectionManager.SelectedNodes));
	}

	TSharedPtr<FDisplayClusterConfiguratorBlueprintEditor> Toolkit = ToolkitPtr.Pin();
	check(Toolkit.IsValid());

	TSharedRef<IDisplayClusterConfiguratorViewOutputMapping> OutputMapping = Toolkit->GetViewOutputMapping();
	bool bAreViewportsLocked = OutputMapping->GetOutputMappingSettings().bLockViewports;

	// If the alt key is down or viewports are locked, put the window in the aux layer so that it is above the viewports, allowing
	// users to select and drag it even if a viewport is in the way.
	if (FSlateApplication::Get().GetModifierKeys().IsAltDown() || bAreViewportsLocked)
	{
		return WindowEdNode->GetAuxiliaryLayer(ObjectPtrDecay(GetOwnerPanel()->SelectionManager.SelectedNodes));
	}

	return SDisplayClusterConfiguratorBaseNode::GetNodeLogicalLayer();
}

TSharedRef<SWidget> SDisplayClusterConfiguratorWindowNode::CreateBackground(const TAttribute<FSlateColor>& InColorAndOpacity)
{
	if (!IsGraphNodeValid())
	{
		return SNullWidget::NullWidget;
	}

	UDisplayClusterConfiguratorWindowNode* WindowEdNode = GetGraphNodeChecked<UDisplayClusterConfiguratorWindowNode>();

	return SNew(SOverlay)
		+SOverlay::Slot()
		.VAlign(VAlign_Fill)
		.HAlign(HAlign_Fill)
		.Padding(0.f)
		[
			SNew(SImage)
			.ColorAndOpacity(InColorAndOpacity)
			.Image(FAppStyle::Get().GetBrush("WhiteBrush"))
		]
	
		+SOverlay::Slot()
		.VAlign(VAlign_Fill)
		.HAlign(HAlign_Fill)
		.Padding(0.f)
		[
			SNew(SScaleBox)
			.Stretch(EStretch::ScaleToFit)
			.StretchDirection(EStretchDirection::Both)
			.Visibility(this, &SDisplayClusterConfiguratorWindowNode::GetPreviewImageVisibility)
			[
				SAssignNew(PreviewImageWidget, SDisplayClusterConfiguratorExternalImage)
				.ImagePath(WindowEdNode->GetPreviewImagePath())
				.ShowShadow(false)
				.MinImageSize(FVector2D::ZeroVector)
				.MaxImageSize(this, &SDisplayClusterConfiguratorWindowNode::GetPreviewImageSize)
			]
		];
}

const FSlateBrush* SDisplayClusterConfiguratorWindowNode::GetBorderBrush() const
{
	if (GetOwnerPanel()->SelectionManager.SelectedNodes.Contains(GraphNode))
	{
		// Selected Case
		return FDisplayClusterConfiguratorStyle::Get().GetBrush("DisplayClusterConfigurator.Node.Window.Border.Brush.Selected");
	}

	// Regular case
	return FDisplayClusterConfiguratorStyle::Get().GetBrush("DisplayClusterConfigurator.Node.Window.Border.Brush.Regular");
}

int32 SDisplayClusterConfiguratorWindowNode::GetBorderLayerOffset() const
{
	if (!IsGraphNodeValid())
	{
		return 0;
	}

	// If the window node is selected, we want to render the border at the same layer as the viewport nodes to ensure it is visible
	// in the case that the child viewport nodes completely fill the window, since the border is a key indicator that the window node is selected.
	if (GetOwnerPanel()->SelectionManager.SelectedNodes.Contains(GraphNode))
	{
		// The offset needs to factor out the window's base layer offset, as the border is part of the node's widget hierarchy, which is already offset to be in the correct 
		// layer; we need to compute the offset needed to bump the border into the auxiliary layer.
		UDisplayClusterConfiguratorWindowNode* WindowEdNode = GetGraphNodeChecked<UDisplayClusterConfiguratorWindowNode>();
		int32 NodeLayerIndex = WindowEdNode->GetNodeLayer(ObjectPtrDecay(GetOwnerPanel()->SelectionManager.SelectedNodes));
		int32 AuxLayerIndex = WindowEdNode->GetAuxiliaryLayer(ObjectPtrDecay(GetOwnerPanel()->SelectionManager.SelectedNodes));

		// Subtract an extra buffer of 100 here to make sure the border doesn't bleed into another layer.
		return AuxLayerIndex - NodeLayerIndex - 100;
	}

	return 0;
}

const FSlateBrush* SDisplayClusterConfiguratorWindowNode::GetNodeShadowBrush() const
{
	return FAppStyle::GetBrush(TEXT("Graph.Node.Shadow"));
}

FMargin SDisplayClusterConfiguratorWindowNode::GetBackgroundPosition() const
{
	FVector2D NodeSize = GetSize();
	return FMargin(0.f, 0.f, NodeSize.X, NodeSize.Y);
}

FSlateColor SDisplayClusterConfiguratorWindowNode::GetCornerColor() const
{
	if (GetOwnerPanel()->SelectionManager.SelectedNodes.Contains(GraphNode))
	{
		return FDisplayClusterConfiguratorStyle::Get().GetColor("DisplayClusterConfigurator.Node.Color.Selected");
	}

	return FDisplayClusterConfiguratorStyle::Get().GetColor("DisplayClusterConfigurator.Node.Window.Corner.Color");
}

FSlateColor SDisplayClusterConfiguratorWindowNode::GetTextColor() const
{
	if (GetOwnerPanel()->SelectionManager.SelectedNodes.Contains(GraphNode))
	{
		return FDisplayClusterConfiguratorStyle::Get().GetColor("DisplayClusterConfigurator.Node.Text.Color.Selected");
	}

	return FDisplayClusterConfiguratorStyle::Get().GetColor("DisplayClusterConfigurator.Node.Text.Color.Regular");
}

FVector2D SDisplayClusterConfiguratorWindowNode::GetPreviewImageSize() const
{
	return GetSize();
}

EVisibility SDisplayClusterConfiguratorWindowNode::GetPreviewImageVisibility() const
{
	if (!IsGraphNodeValid())
	{
		return EVisibility::Collapsed;
	}

	UDisplayClusterConfiguratorWindowNode* WindowEdNode = GetGraphNodeChecked<UDisplayClusterConfiguratorWindowNode>();
	const FString& PreviewImagePath = WindowEdNode->GetPreviewImagePath();
	return !PreviewImagePath.IsEmpty() ? EVisibility::Visible : EVisibility::Hidden;
}


int32 SDisplayClusterConfiguratorWindowNode::GetNodeTitleLayerOffset() const
{
	if (!IsGraphNodeValid())
	{
		return 0;
	}

	UDisplayClusterConfiguratorWindowNode* WindowEdNode = GetGraphNodeChecked<UDisplayClusterConfiguratorWindowNode>();
	return WindowEdNode->GetAuxiliaryLayer(ObjectPtrDecay(GetOwnerPanel()->SelectionManager.SelectedNodes));
}

EVisibility SDisplayClusterConfiguratorWindowNode::GetNodeInfoVisibility() const
{
	return CanShowInfoWidget() ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SDisplayClusterConfiguratorWindowNode::GetCornerImageVisibility() const
{
	return CanShowCornerImageWidget() ? EVisibility::Visible : EVisibility::Collapsed;
}

bool SDisplayClusterConfiguratorWindowNode::CanShowInfoWidget() const
{
	TSharedPtr<FDisplayClusterConfiguratorBlueprintEditor> Toolkit = ToolkitPtr.Pin();
	check(Toolkit.IsValid());

	TSharedRef<IDisplayClusterConfiguratorViewOutputMapping> OutputMapping = Toolkit->GetViewOutputMapping();
	FVector2D NodeSize = GetSize();

	return IsNodeVisible() && OutputMapping->GetOutputMappingSettings().bShowWindowInfo && NodeSize.GetMin() > 0;
}

bool SDisplayClusterConfiguratorWindowNode::CanShowCornerImageWidget() const
{
	TSharedPtr<FDisplayClusterConfiguratorBlueprintEditor> Toolkit = ToolkitPtr.Pin();
	check(Toolkit.IsValid());

	TSharedRef<IDisplayClusterConfiguratorViewOutputMapping> OutputMapping = Toolkit->GetViewOutputMapping();
	FVector2D NodeSize = GetSize();

	return IsNodeVisible() && OutputMapping->GetOutputMappingSettings().bShowWindowCornerImage && NodeSize.GetMin() > 0;
}

bool SDisplayClusterConfiguratorWindowNode::IsClusterNodeLocked() const
{
	TSharedPtr<FDisplayClusterConfiguratorBlueprintEditor> Toolkit = ToolkitPtr.Pin();
	check(Toolkit.IsValid());

	TSharedRef<IDisplayClusterConfiguratorViewOutputMapping> OutputMapping = Toolkit->GetViewOutputMapping();
	return OutputMapping->GetOutputMappingSettings().bLockClusterNodes;
}

void SDisplayClusterConfiguratorWindowNode::OnPreviewImageChanged()
{
	if (!IsGraphNodeValid())
	{
		return;
	}

	if (PreviewImageWidget.IsValid())
	{
		UDisplayClusterConfiguratorWindowNode* WindowEdNode = GetGraphNodeChecked<UDisplayClusterConfiguratorWindowNode>();
		PreviewImageWidget->SetImagePath(WindowEdNode->GetPreviewImagePath());
	}
}

#undef LOCTEXT_NAMESPACE
