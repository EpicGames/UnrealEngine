// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Views/OutputMapping/GraphNodes/SDisplayClusterConfiguratorBaseNode.h"

#include "UObject/WeakObjectPtr.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class FDisplayClusterConfiguratorOutputMappingWindowSlot;
class FDisplayClusterConfiguratorBlueprintEditor;
class IDisplayClusterConfiguratorTreeItem;
class UDisplayClusterConfigurationClusterNode;
class UDisplayClusterConfiguratorWindowNode;
class SDisplayClusterConfiguratorExternalImage;

class SDisplayClusterConfiguratorWindowNode
	: public SDisplayClusterConfiguratorBaseNode
{
public:
	friend class SCornerImage;
	friend class SNodeInfo;

	~SDisplayClusterConfiguratorWindowNode();

	SLATE_BEGIN_ARGS(SDisplayClusterConfiguratorWindowNode)
	{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs,
		UDisplayClusterConfiguratorWindowNode* InWindowNode,
		const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& InToolkit);

	//~ Begin SGraphNode interface
	virtual void UpdateGraphNode() override;
	virtual FVector2D ComputeDesiredSize(float) const override;
	virtual FVector2f GetPosition2f() const override;
	//~ End SGraphNode interface

	//~ Begin SDisplayClusterConfiguratorBaseNode interface
	virtual bool CanNodeBeSnapAligned() const override { return true; }
	virtual bool CanNodeBeResized() const { return IsNodeUnlocked(); }
	virtual bool IsAspectRatioFixed() const override;

protected:
	virtual int32 GetNodeLogicalLayer() const override;
	//~ End SDisplayClusterConfiguratorBaseNode interface

private:
	TSharedRef<SWidget> CreateBackground(const TAttribute<FSlateColor>& ColorAndOpacity);

	const FSlateBrush* GetBorderBrush() const;
	int32 GetBorderLayerOffset() const;
	const FSlateBrush* GetNodeShadowBrush() const;
	FMargin GetBackgroundPosition() const;
	FSlateColor GetCornerColor() const;
	FSlateColor GetTextColor() const;
	FVector2D GetPreviewImageSize() const;
	EVisibility GetPreviewImageVisibility() const;
	int32 GetNodeTitleLayerOffset() const;
	EVisibility GetNodeInfoVisibility() const;
	EVisibility GetCornerImageVisibility() const;

	bool CanShowInfoWidget() const;
	bool CanShowCornerImageWidget() const;
	bool IsClusterNodeLocked() const;

	void OnPreviewImageChanged();

private:
	TSharedPtr<SDisplayClusterConfiguratorExternalImage> PreviewImageWidget;

	FVector2D WindowScaleFactor;

	FDelegateHandle ImageChangedHandle;

	bool bLayerAboveViewports = false;
};
