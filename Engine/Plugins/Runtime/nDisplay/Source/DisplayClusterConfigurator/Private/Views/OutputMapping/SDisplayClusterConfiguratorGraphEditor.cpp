// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/OutputMapping/SDisplayClusterConfiguratorGraphEditor.h"

#include "DisplayClusterConfiguratorCommands.h"
#include "DisplayClusterConfiguratorBlueprintEditor.h"
#include "DisplayClusterConfigurationTypes.h"
#include "IDisplayClusterConfigurator.h"
#include "ClusterConfiguration/DisplayClusterConfiguratorClusterEditorUtils.h"
#include "ClusterConfiguration/SDisplayClusterConfiguratorNewClusterItemDialog.h"
#include "Views/OutputMapping/DisplayClusterConfiguratorGraph.h"
#include "Views/OutputMapping/DisplayClusterConfiguratorOutputMappingCommands.h"
#include "Views/OutputMapping/DisplayClusterConfiguratorViewOutputMapping.h"
#include "Views/OutputMapping/EdNodes/DisplayClusterConfiguratorBaseNode.h"
#include "Views/OutputMapping/EdNodes/DisplayClusterConfiguratorCanvasNode.h"
#include "Views/OutputMapping/EdNodes/DisplayClusterConfiguratorHostNode.h"
#include "Views/OutputMapping/EdNodes/DisplayClusterConfiguratorWindowNode.h"
#include "Views/OutputMapping/EdNodes/DisplayClusterConfiguratorViewportNode.h"

#include "ScopedTransaction.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GraphEditorActions.h"
#include "SGraphPanel.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Optional.h"
#include "IDocumentation.h"

#include "Views/OutputMapping/IDisplayClusterConfiguratorViewOutputMapping.h"


#define LOCTEXT_NAMESPACE "SDisplayClusterConfiguratorGraphEditor"

void SDisplayClusterConfiguratorGraphEditor::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SortNodes();

	SGraphEditor::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	// Every tick, give the graph nodes an opportunity to update their position and size, to allow procedurally positioned nodes a chance to reposition if need be.
	// Do this here instead of the node widgets because widgets can only tick when they are on screen, and nodes need to update their position even when they are 
	// not in the graph's view.
	if (ClusterConfiguratorGraph.IsValid())
	{
		ClusterConfiguratorGraph->TickNodePositions();
	}
}

void SDisplayClusterConfiguratorGraphEditor::PostUndo(bool bSuccess)
{
	// Clear out the selection set, as it may be holding references to nodes that are marked as pending kill.
	ClearSelectionSet();
}

void SDisplayClusterConfiguratorGraphEditor::FindAndSelectObjects(const TArray<UObject*>& ObjectsToSelect)
{
	UDisplayClusterConfiguratorGraph* ConfiguratorGraph = ClusterConfiguratorGraph.Get();
	check(ConfiguratorGraph != nullptr);

	bSelectionSetDirectly = true;
	// Clear the current selection or stale data might persist after a compile.
	ClearSelectionSet();
	ConfiguratorGraph->ForEachGraphNode([this, &ObjectsToSelect](UDisplayClusterConfiguratorBaseNode* Node)
	{
		if (ObjectsToSelect.Contains(Node->GetObject()))
		{
			SetNodeSelection(Node, true);
		}
		else
		{
			SetNodeSelection(Node, false);
		}
	});
	bSelectionSetDirectly = false;
}

void SDisplayClusterConfiguratorGraphEditor::JumpToObject(UObject* InObject)
{
	UDisplayClusterConfiguratorGraph* ConfiguratorGraph = ClusterConfiguratorGraph.Get();
	check(ConfiguratorGraph != nullptr);

	if (UDisplayClusterConfiguratorBaseNode* GraphNode = ConfiguratorGraph->GetNodeFromObject(InObject))
	{
		JumpToNode(GraphNode, false, false);
	}
}

void SDisplayClusterConfiguratorGraphEditor::OnSelectedNodesChanged(const TSet<UObject*>& NewSelection)
{
	// If the selection has been set directly in code, don't propagate the selection change to the toolkit.
	if (bSelectionSetDirectly)
	{
		return;
	}

	TArray<UObject*> Selection;
	for (UObject* GraphNode : NewSelection)
	{
		if (UDisplayClusterConfiguratorBaseNode* BaseNode = Cast<UDisplayClusterConfiguratorBaseNode>(GraphNode))
		{
			Selection.Add(BaseNode->GetObject());
		}
	}

	ToolkitPtr.Pin()->SelectObjects(Selection);
}

void SDisplayClusterConfiguratorGraphEditor::OnNodeDoubleClicked(UEdGraphNode* ClickedNode)
{
	JumpToNode(ClickedNode, false, false);
}

void SDisplayClusterConfiguratorGraphEditor::OnObjectSelected()
{
	TArray<UObject*> SelectedObjects = ToolkitPtr.Pin()->GetSelectedObjects();

	if (!SelectedObjects.Num())
	{
		bSelectionSetDirectly = true;
		ClearSelectionSet();
		bSelectionSetDirectly = false;

		return;
	}

	UDisplayClusterConfiguratorGraph* ConfiguratorGraph = ClusterConfiguratorGraph.Get();
	check(ConfiguratorGraph != nullptr);

	bSelectionSetDirectly = true;
	ConfiguratorGraph->ForEachGraphNode([this, &SelectedObjects](UDisplayClusterConfiguratorBaseNode* Node)
	{
		if (SelectedObjects.Contains(Node->GetObject()))
		{
			SetNodeSelection(Node, true);
		}
		else
		{
			SetNodeSelection(Node, false);
		}
	});
	bSelectionSetDirectly = false;

	TSharedPtr<FDisplayClusterConfiguratorViewOutputMapping> OutputMapping = ViewOutputMappingPtr.Pin();
	check(OutputMapping.IsValid());
}

void SDisplayClusterConfiguratorGraphEditor::OnConfigReloaded()
{
	RebuildGraph();

	// Reselect any graph nodes that correspond to the selected objects in the editor
	FindAndSelectObjects(ToolkitPtr.Pin()->GetSelectedObjects());
}

void SDisplayClusterConfiguratorGraphEditor::OnClusterChanged()
{
	RebuildGraph();

	// Reselect any graph nodes that correspond to the selected objects in the editor
	FindAndSelectObjects(ToolkitPtr.Pin()->GetSelectedObjects());
}

void SDisplayClusterConfiguratorGraphEditor::RebuildGraph()
{
	UDisplayClusterConfiguratorGraph* ConfiguratorGraph = ClusterConfiguratorGraph.Get();
	check(ConfiguratorGraph != nullptr);

	ConfiguratorGraph->RebuildGraph();
}

FActionMenuContent SDisplayClusterConfiguratorGraphEditor::OnCreateNodeOrPinMenu(UEdGraph* CurrentGraph, const UEdGraphNode* InGraphNode, const UEdGraphPin* InGraphPin, FMenuBuilder* MenuBuilder, bool bIsDebugging)
{
	if (InGraphNode->IsA(UDisplayClusterConfiguratorBaseNode::StaticClass()))
	{
		MenuBuilder->BeginSection(FName(TEXT("Add")), LOCTEXT("AddSectionLabel", "Add"));
		{
			if (InGraphNode->IsA<UDisplayClusterConfiguratorHostNode>() || InGraphNode->IsA<UDisplayClusterConfiguratorCanvasNode>())
			{
				MenuBuilder->AddMenuEntry(FDisplayClusterConfiguratorCommands::Get().AddNewClusterNode);
			}

			if (InGraphNode->IsA<UDisplayClusterConfiguratorWindowNode>() || InGraphNode->IsA<UDisplayClusterConfiguratorCanvasNode>())
			{
				MenuBuilder->AddMenuEntry(FDisplayClusterConfiguratorCommands::Get().AddNewViewport);
			}
		}
		MenuBuilder->EndSection();

		MenuBuilder->BeginSection(FName(TEXT("Documentation")), LOCTEXT("Documentation", "Documentation"));
		{
			MenuBuilder->AddMenuEntry(
				FDisplayClusterConfiguratorOutputMappingCommands::Get().BrowseDocumentation,
				NAME_None,
				LOCTEXT("GoToDocsForActor", "View Documentation"),
				LOCTEXT("GoToDocsForActor_ToolTip", "Click to open documentation for nDisplay"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Help")
				);
		}
		MenuBuilder->EndSection();

		MenuBuilder->BeginSection(FName(TEXT("CommonSection")), LOCTEXT("CommonSection", "Common"));
		{
			MenuBuilder->AddMenuEntry(FGenericCommands::Get().Copy);
			MenuBuilder->AddMenuEntry(FGenericCommands::Get().Cut);
			MenuBuilder->AddMenuEntry(FGenericCommands::Get().Paste);
			MenuBuilder->AddMenuEntry(FGenericCommands::Get().Duplicate);
			MenuBuilder->AddMenuEntry(FGenericCommands::Get().Delete);
		}
		MenuBuilder->EndSection();

		MenuBuilder->BeginSection("EdGraphSchemaOrganization", LOCTEXT("OrganizationHeader", "Organization"));
		{
			MenuBuilder->AddSubMenu(LOCTEXT("AlignmentHeader", "Alignment"), FText(), FNewMenuDelegate::CreateLambda([](FMenuBuilder& InSubMenuBuilder)
			{
				InSubMenuBuilder.BeginSection("EdGraphSchemaAlignment", LOCTEXT("AlignHeader", "Align"));
				{
					InSubMenuBuilder.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesTop);
					InSubMenuBuilder.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesMiddle);
					InSubMenuBuilder.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesBottom);
					InSubMenuBuilder.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesLeft);
					InSubMenuBuilder.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesCenter);
					InSubMenuBuilder.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesRight);
				}
				InSubMenuBuilder.EndSection();

				InSubMenuBuilder.BeginSection("EdGraphSchemaSizing", LOCTEXT("SizeHeader", "Size"));
				{
					InSubMenuBuilder.AddMenuEntry(FDisplayClusterConfiguratorOutputMappingCommands::Get().FillParentNode);
					InSubMenuBuilder.AddMenuEntry(FDisplayClusterConfiguratorOutputMappingCommands::Get().SizeToChildNodes);
				}
				InSubMenuBuilder.EndSection();
			}));
		}
		MenuBuilder->EndSection();

		MenuBuilder->BeginSection(FName(TEXT("Transformation")), LOCTEXT("TransformationHeader", "Transform"));
		{
			if (InGraphNode->IsA<UDisplayClusterConfiguratorViewportNode>())
			{
				MenuBuilder->AddMenuEntry(FDisplayClusterConfiguratorOutputMappingCommands::Get().RotateViewport90CW);
				MenuBuilder->AddMenuEntry(FDisplayClusterConfiguratorOutputMappingCommands::Get().RotateViewport90CCW);
				MenuBuilder->AddMenuEntry(FDisplayClusterConfiguratorOutputMappingCommands::Get().RotateViewport180);
				MenuBuilder->AddMenuEntry(FDisplayClusterConfiguratorOutputMappingCommands::Get().FlipViewportHorizontal);
				MenuBuilder->AddMenuEntry(FDisplayClusterConfiguratorOutputMappingCommands::Get().FlipViewportVertical);
				MenuBuilder->AddSeparator();
				MenuBuilder->AddMenuEntry(FDisplayClusterConfiguratorOutputMappingCommands::Get().ResetViewportTransform);
			}
		}
		MenuBuilder->EndSection();

		return FActionMenuContent(MenuBuilder->MakeWidget());
	}

	return FActionMenuContent();
}
void SDisplayClusterConfiguratorGraphEditor::BindCommands()
{
	CommandList = MakeShareable(new FUICommandList);

	CommandList->MapAction(
		FDisplayClusterConfiguratorCommands::Get().AddNewClusterNode,
		FExecuteAction::CreateSP(this, &SDisplayClusterConfiguratorGraphEditor::AddNewClusterNode),
		FCanExecuteAction::CreateSP(this, &SDisplayClusterConfiguratorGraphEditor::CanAddNewClusterNode)
		);

	CommandList->MapAction(
		FDisplayClusterConfiguratorCommands::Get().AddNewViewport,
		FExecuteAction::CreateSP(this, &SDisplayClusterConfiguratorGraphEditor::AddNewViewport),
		FCanExecuteAction::CreateSP(this, &SDisplayClusterConfiguratorGraphEditor::CanAddNewViewport)
		);

	CommandList->MapAction(
		FDisplayClusterConfiguratorOutputMappingCommands::Get().BrowseDocumentation,
		FExecuteAction::CreateSP(this, &SDisplayClusterConfiguratorGraphEditor::BrowseDocumentation),
		FCanExecuteAction()
		);

	CommandList->MapAction(
		FGenericCommands::Get().Delete,
		FExecuteAction::CreateSP(this, &SDisplayClusterConfiguratorGraphEditor::DeleteSelectedNodes),
		FCanExecuteAction::CreateSP(this, &SDisplayClusterConfiguratorGraphEditor::CanDeleteNodes)
		);

	CommandList->MapAction(
		FGenericCommands::Get().Copy,
		FExecuteAction::CreateSP(this, &SDisplayClusterConfiguratorGraphEditor::CopySelectedNodes),
		FCanExecuteAction::CreateSP(this, &SDisplayClusterConfiguratorGraphEditor::CanCopyNodes)
		);

	CommandList->MapAction(
		FGenericCommands::Get().Cut,
		FExecuteAction::CreateSP(this, &SDisplayClusterConfiguratorGraphEditor::CutSelectedNodes),
		FCanExecuteAction::CreateSP(this, &SDisplayClusterConfiguratorGraphEditor::CanCutNodes)
		);

	CommandList->MapAction(
		FGenericCommands::Get().Paste,
		FExecuteAction::CreateSP(this, &SDisplayClusterConfiguratorGraphEditor::PasteNodes),
		FCanExecuteAction::CreateSP(this, &SDisplayClusterConfiguratorGraphEditor::CanPasteNodes)
		);

	CommandList->MapAction(
		FGenericCommands::Get().Duplicate,
		FExecuteAction::CreateSP(this, &SDisplayClusterConfiguratorGraphEditor::DuplicateNodes),
		FCanExecuteAction::CreateSP(this, &SDisplayClusterConfiguratorGraphEditor::CanDuplicateNodes)
		);

	CommandList->MapAction(
		FGraphEditorCommands::Get().AlignNodesTop,
		FExecuteAction::CreateSP(this, &SDisplayClusterConfiguratorGraphEditor::AlignNodes, ENodeAlignment::Top),
		FCanExecuteAction::CreateSP(this, &SDisplayClusterConfiguratorGraphEditor::CanAlignNodes)
		);

	CommandList->MapAction(
		FGraphEditorCommands::Get().AlignNodesMiddle,
		FExecuteAction::CreateSP(this, &SDisplayClusterConfiguratorGraphEditor::AlignNodes, ENodeAlignment::Middle),
		FCanExecuteAction::CreateSP(this, &SDisplayClusterConfiguratorGraphEditor::CanAlignNodes)
		);

	CommandList->MapAction(
		FGraphEditorCommands::Get().AlignNodesBottom,
		FExecuteAction::CreateSP(this, &SDisplayClusterConfiguratorGraphEditor::AlignNodes, ENodeAlignment::Bottom),
		FCanExecuteAction::CreateSP(this, &SDisplayClusterConfiguratorGraphEditor::CanAlignNodes)
		);

	CommandList->MapAction(
		FGraphEditorCommands::Get().AlignNodesLeft,
		FExecuteAction::CreateSP(this, &SDisplayClusterConfiguratorGraphEditor::AlignNodes, ENodeAlignment::Left),
		FCanExecuteAction::CreateSP(this, &SDisplayClusterConfiguratorGraphEditor::CanAlignNodes)
		);

	CommandList->MapAction(
		FGraphEditorCommands::Get().AlignNodesCenter,
		FExecuteAction::CreateSP(this, &SDisplayClusterConfiguratorGraphEditor::AlignNodes, ENodeAlignment::Center),
		FCanExecuteAction::CreateSP(this, &SDisplayClusterConfiguratorGraphEditor::CanAlignNodes)
		);

	CommandList->MapAction(
		FGraphEditorCommands::Get().AlignNodesRight,
		FExecuteAction::CreateSP(this, &SDisplayClusterConfiguratorGraphEditor::AlignNodes, ENodeAlignment::Right),
		FCanExecuteAction::CreateSP(this, &SDisplayClusterConfiguratorGraphEditor::CanAlignNodes)
		);

	CommandList->MapAction(
		FDisplayClusterConfiguratorOutputMappingCommands::Get().FillParentNode,
		FExecuteAction::CreateSP(this, &SDisplayClusterConfiguratorGraphEditor::FillParentNode),
		FCanExecuteAction::CreateSP(this, &SDisplayClusterConfiguratorGraphEditor::CanFillParentNode)
	);

	CommandList->MapAction(
		FDisplayClusterConfiguratorOutputMappingCommands::Get().SizeToChildNodes,
		FExecuteAction::CreateSP(this, &SDisplayClusterConfiguratorGraphEditor::SizeToChildNodes),
		FCanExecuteAction::CreateSP(this, &SDisplayClusterConfiguratorGraphEditor::CanSizeToChildNodes)
	);

	CommandList->MapAction(
		FDisplayClusterConfiguratorOutputMappingCommands::Get().RotateViewport90CW,
		FExecuteAction::CreateSP(this, &SDisplayClusterConfiguratorGraphEditor::RotateNode, 90.0f),
		FCanExecuteAction::CreateSP(this, &SDisplayClusterConfiguratorGraphEditor::CanTransformNode)
	);

	CommandList->MapAction(
		FDisplayClusterConfiguratorOutputMappingCommands::Get().RotateViewport90CCW,
		FExecuteAction::CreateSP(this, &SDisplayClusterConfiguratorGraphEditor::RotateNode, -90.0f),
		FCanExecuteAction::CreateSP(this, &SDisplayClusterConfiguratorGraphEditor::CanTransformNode)
	);

	CommandList->MapAction(
		FDisplayClusterConfiguratorOutputMappingCommands::Get().RotateViewport180,
		FExecuteAction::CreateSP(this, &SDisplayClusterConfiguratorGraphEditor::RotateNode, 180.0f),
		FCanExecuteAction::CreateSP(this, &SDisplayClusterConfiguratorGraphEditor::CanTransformNode)
	);

	CommandList->MapAction(
		FDisplayClusterConfiguratorOutputMappingCommands::Get().FlipViewportHorizontal,
		FExecuteAction::CreateSP(this, &SDisplayClusterConfiguratorGraphEditor::FlipNode, true, false),
		FCanExecuteAction::CreateSP(this, &SDisplayClusterConfiguratorGraphEditor::CanTransformNode)
	);

	CommandList->MapAction(
		FDisplayClusterConfiguratorOutputMappingCommands::Get().FlipViewportVertical,
		FExecuteAction::CreateSP(this, &SDisplayClusterConfiguratorGraphEditor::FlipNode, false, true),
		FCanExecuteAction::CreateSP(this, &SDisplayClusterConfiguratorGraphEditor::CanTransformNode)
	);

	CommandList->MapAction(
		FDisplayClusterConfiguratorOutputMappingCommands::Get().ResetViewportTransform,
		FExecuteAction::CreateSP(this, &SDisplayClusterConfiguratorGraphEditor::ResetNodeTransform),
		FCanExecuteAction::CreateSP(this, &SDisplayClusterConfiguratorGraphEditor::CanTransformNode)
	);

	CommandList->MapAction(
		FDisplayClusterConfiguratorOutputMappingCommands::Get().ZoomToFit,
		FExecuteAction::CreateSP(this, &SDisplayClusterConfiguratorGraphEditor::ZoomToFit, false)
	);
}

void SDisplayClusterConfiguratorGraphEditor::AddNewClusterNode()
{
	const TSet<UObject*>& SelectedNodes = GetSelectedNodes();
	if (SelectedNodes.Num() == 1)
	{
		FString Host;
		if (const UDisplayClusterConfiguratorHostNode* HostNode = Cast<UDisplayClusterConfiguratorHostNode>(*SelectedNodes.begin()))
		{
			Host = HostNode->GetNodeName();
		}

		const TSharedPtr<FDisplayClusterConfiguratorBlueprintEditor> Toolkit = ToolkitPtr.Pin();
		UDisplayClusterConfigurationCluster* Cluster = Toolkit->GetEditorData()->Cluster;
		const FVector2D PresetSize = FDisplayClusterConfiguratorPresetSize::CommonPresets[FDisplayClusterConfiguratorPresetSize::DefaultPreset].Size;
		const FDisplayClusterConfigurationRectangle PresetRect = FDisplayClusterConfigurationRectangle(0, 0, PresetSize.X, PresetSize.Y);

		TSharedPtr<FScopedTransaction> Transaction;
		if (UDisplayClusterConfigurationClusterNode* NewNode =
			UE::DisplayClusterConfiguratorClusterEditorUtils::CreateNewClusterNodeFromDialog(Toolkit.ToSharedRef(), Cluster, PresetRect,Transaction, Host))
		{
			// Mark the cluster configuration data as dirty, allowing user to save the changes, and fire off a cluster changed event to let other
			// parts of the UI update as well
			Toolkit->GetEditorData()->MarkPackageDirty();
			Toolkit->ClusterChanged();
		}
	}
}

bool SDisplayClusterConfiguratorGraphEditor::CanAddNewClusterNode() const
{
	const TSet<UObject*>& SelectedNodes = GetSelectedNodes();
	if (SelectedNodes.Num() == 1)
	{
		UObject* Node = *SelectedNodes.begin();
		return Node->IsA<UDisplayClusterConfiguratorHostNode>() || Node->IsA<UDisplayClusterConfiguratorCanvasNode>();
	}

	return false;
}

void SDisplayClusterConfiguratorGraphEditor::AddNewViewport()
{
	const TSet<UObject*>& SelectedNodes = GetSelectedNodes();
	if (SelectedNodes.Num() == 1)
	{
		UDisplayClusterConfigurationClusterNode* SelectedClusterNode = nullptr;
		if (const UDisplayClusterConfiguratorWindowNode* WindowNode = Cast<UDisplayClusterConfiguratorWindowNode>(*SelectedNodes.begin()))
		{
			SelectedClusterNode = Cast<UDisplayClusterConfigurationClusterNode>(WindowNode->GetObject());
		}

		const TSharedPtr<FDisplayClusterConfiguratorBlueprintEditor> Toolkit = ToolkitPtr.Pin();
		const FVector2D PresetSize = FDisplayClusterConfiguratorPresetSize::CommonPresets[FDisplayClusterConfiguratorPresetSize::DefaultPreset].Size;
		const FDisplayClusterConfigurationRectangle PresetRect = FDisplayClusterConfigurationRectangle(0, 0, PresetSize.X, PresetSize.Y);

		TSharedPtr<FScopedTransaction> Transaction;
		if (UDisplayClusterConfigurationViewport* NewViewport = UE::DisplayClusterConfiguratorClusterEditorUtils::CreateNewViewportFromDialog(Toolkit.ToSharedRef(), SelectedClusterNode, PresetRect, Transaction))
		{
			// Mark the cluster configuration data as dirty, allowing user to save the changes, and fire off a cluster changed event to let other
			// parts of the UI update as well
			Toolkit->GetEditorData()->MarkPackageDirty();
			Toolkit->ClusterChanged();
		}
	}
}

bool SDisplayClusterConfiguratorGraphEditor::CanAddNewViewport() const
{
	const TSet<UObject*>& SelectedNodes = GetSelectedNodes();
	if (SelectedNodes.Num() == 1)
	{
		const UObject* Node = *SelectedNodes.begin();
		return UE::DisplayClusterConfiguratorClusterEditorUtils::CanAddNewViewportToToolkit(ToolkitPtr)
			&& (Node->IsA<UDisplayClusterConfiguratorWindowNode>() || Node->IsA<UDisplayClusterConfiguratorCanvasNode>());
	}

	return false;
}

void SDisplayClusterConfiguratorGraphEditor::BrowseDocumentation()
{
	const static FString NDisplayLink = TEXT("WorkingWithMedia/IntegratingMedia/nDisplay");
	IDocumentation::Get()->Open(NDisplayLink, FDocumentationSourceInfo(TEXT("ndisplay_config")));
}

void SDisplayClusterConfiguratorGraphEditor::DeleteSelectedNodes()
{
	const TSet<UObject*>& SelectedNodes = GetSelectedNodes();
	if (SelectedNodes.Num() > 0)
	{
		const FScopedTransaction Transaction(FText::Format(LOCTEXT("DeleteClusterItems", "Remove Cluster {0}|plural(one=Item, other=Items)"), SelectedNodes.Num()));

		for (UObject* Node : SelectedNodes)
		{
			UDisplayClusterConfiguratorBaseNode* BaseNode = CastChecked<UDisplayClusterConfiguratorBaseNode>(Node);
			if (BaseNode->CanUserDeleteNode())
			{
				BaseNode->DeleteObject();
			}
		}

		TSharedPtr<FDisplayClusterConfiguratorBlueprintEditor> Toolkit = ToolkitPtr.Pin();
		Toolkit->GetEditorData()->MarkPackageDirty();
		constexpr bool bStructureChange = true;
		Toolkit->ClusterChanged(bStructureChange);
		ClearSelectionSet();
	}
}

bool SDisplayClusterConfiguratorGraphEditor::CanDeleteNodes() const
{
	const TSet<UObject*>& SelectedNodes = GetSelectedNodes();
	if (SelectedNodes.Num() > 0)
	{
		for (const UObject* Node : SelectedNodes)
		{
			const UDisplayClusterConfiguratorBaseNode* BaseNode = CastChecked<UDisplayClusterConfiguratorBaseNode>(Node);

			if (BaseNode->CanUserDeleteNode())
			{
				return true;
			}
		}

		return false;
	}

	return false;
}

void SDisplayClusterConfiguratorGraphEditor::CopySelectedNodes()
{
	const TSet<UObject*>& SelectedNodes = GetSelectedNodes();
	if (SelectedNodes.Num() > 0)
	{
		TArray<UObject*> ObjectsToCopy;
		for (const UObject* Node : SelectedNodes)
		{
			const UDisplayClusterConfiguratorBaseNode* BaseNode = CastChecked<UDisplayClusterConfiguratorBaseNode>(Node);

			if (BaseNode->CanDuplicateNode())
			{
				ObjectsToCopy.Add(BaseNode->GetObject());
			}
		}

		UE::DisplayClusterConfiguratorClusterEditorUtils::CopyClusterItemsToClipboard(ObjectsToCopy);
	}
}

bool SDisplayClusterConfiguratorGraphEditor::CanCopyNodes() const
{
	const TSet<UObject*>& SelectedNodes = GetSelectedNodes();
	if (SelectedNodes.Num() > 0)
	{
		for (const UObject* Node : SelectedNodes)
		{
			const UDisplayClusterConfiguratorBaseNode* BaseNode = CastChecked<UDisplayClusterConfiguratorBaseNode>(Node);

			if (BaseNode->CanDuplicateNode())
			{
				return true;
			}
		}
	}

	return false;
}

void SDisplayClusterConfiguratorGraphEditor::CutSelectedNodes()
{
	CopySelectedNodes();
	DeleteSelectedNodes();
}

bool SDisplayClusterConfiguratorGraphEditor::CanCutNodes() const
{
	const TSet<UObject*>& SelectedNodes = GetSelectedNodes();
	if (SelectedNodes.Num() > 0)
	{
		for (const UObject* Node : SelectedNodes)
		{
			const UDisplayClusterConfiguratorBaseNode* BaseNode = CastChecked<UDisplayClusterConfiguratorBaseNode>(Node);

			if (BaseNode->CanDuplicateNode() && BaseNode->CanUserDeleteNode())
			{
				return true;
			}
		}

		return false;
	}

	return false;
}

void SDisplayClusterConfiguratorGraphEditor::PasteNodes()
{
	TSharedPtr<FDisplayClusterConfiguratorBlueprintEditor> Toolkit = ToolkitPtr.Pin();
	TSharedRef<IDisplayClusterConfiguratorViewOutputMapping> OutputMapping = Toolkit->GetViewOutputMapping();
	const float ViewScale = OutputMapping->GetOutputMappingSettings().ViewScale;

	TArray<UObject*> TargetObjects;
	const TSet<UObject*>& SelectedNodes = GetSelectedNodes();
	for (UObject* Node : SelectedNodes)
	{
		const UDisplayClusterConfiguratorBaseNode* BaseNode = CastChecked<UDisplayClusterConfiguratorBaseNode>(Node);
		if (BaseNode->GetObject())
		{
			// Though TSets aren't expected to be sorted in any particular order, adding the selected nodes in reverse order seems more likely
			// to put the most recently selected node at the top of the list, resulting in better results when pasting into mutliple nodes.
			TargetObjects.Insert(BaseNode->GetObject(), 0);
		}
	}

	// If there are no nodes selected, assume that we are pasting into the root cluster
	if (TargetObjects.Num() == 0)
	{
		TargetObjects.Add(Toolkit->GetEditorData()->Cluster);
	}

	int32 NumClusterItems;
	if (UE::DisplayClusterConfiguratorClusterEditorUtils::CanPasteClusterItemsFromClipboard(TargetObjects, NumClusterItems))
	{
		FScopedTransaction Transaction(FText::Format(LOCTEXT("PasteClusterItems", "Paste Cluster {0}|plural(one=Item, other=Items)"), NumClusterItems));

		FVector2D PasteLocation = GetPasteLocation2f() / ViewScale;
		TArray<UObject*> PastedObjects = UE::DisplayClusterConfiguratorClusterEditorUtils::PasteClusterItemsFromClipboard(TargetObjects, PasteLocation);
		if (PastedObjects.Num() > 0)
		{
			Toolkit->GetEditorData()->MarkPackageDirty();
			Toolkit->ClusterChanged();
			ClearSelectionSet();
		}
		else
		{
			Transaction.Cancel();
		}

		// Select all of the nodes that represent the copied objects.
		for (UEdGraphNode* Node : ClusterConfiguratorGraph->Nodes)
		{
			UDisplayClusterConfiguratorBaseNode* BaseNode = CastChecked<UDisplayClusterConfiguratorBaseNode>(Node);
			if (PastedObjects.Contains(BaseNode->GetObject()))
			{
				SetNodeSelection(Node, true);
			}
		}
	}
}

bool SDisplayClusterConfiguratorGraphEditor::CanPasteNodes() const
{
	TSharedPtr<FDisplayClusterConfiguratorBlueprintEditor> Toolkit = ToolkitPtr.Pin();

	TArray<UObject*> TargetObjects;
	const TSet<UObject*>& SelectedNodes = GetSelectedNodes();
	for (UObject* Node : SelectedNodes)
	{
		const UDisplayClusterConfiguratorBaseNode* BaseNode = CastChecked<UDisplayClusterConfiguratorBaseNode>(Node);
		if (BaseNode->GetObject())
		{
			TargetObjects.Add(BaseNode->GetObject());
		}
	}

	// If there are no nodes selected, assume that we are pasting into the root cluster
	if (TargetObjects.Num() == 0)
	{
		TargetObjects.Add(Toolkit->GetEditorData()->Cluster);
	}

	int32 NumClusterItems;
	return UE::DisplayClusterConfiguratorClusterEditorUtils::CanPasteClusterItemsFromClipboard(TargetObjects, NumClusterItems);
}

void SDisplayClusterConfiguratorGraphEditor::DuplicateNodes()
{
	CopySelectedNodes();
	PasteNodes();
}

bool SDisplayClusterConfiguratorGraphEditor::CanDuplicateNodes() const
{
	return CanCopyNodes();
}

bool SDisplayClusterConfiguratorGraphEditor::CanAlignNodes() const
{
	// We want nodes to be alignable only when all nodes are children of the same parent. Viewport nodes should only be aligned with
	// sibling viewport nodes or their parent window node, window nodes should only be aligned with their sibling windows, etc.
	const FGraphPanelSelectionSet& CurrentlySelectedNodes = GetSelectedNodes();

	TMap<FGuid, TArray<UDisplayClusterConfiguratorBaseNode*>> SelectionGroupedByParent;
	for (UObject* Node : CurrentlySelectedNodes)
	{
		if (UDisplayClusterConfiguratorBaseNode* BaseNode = Cast<UDisplayClusterConfiguratorBaseNode>(Node))
		{
			if (BaseNode->IsA<UDisplayClusterConfiguratorCanvasNode>())
			{
				// Canvas node cannot be aligned, as it is meant to be a stationary node.
				// Disable alignment if canvas node is among the selected nodes.
				return false;
			}

			if (UDisplayClusterConfiguratorBaseNode* ParentNode = BaseNode->GetParent())
			{
				if (!SelectionGroupedByParent.Contains(ParentNode->NodeGuid))
				{
					SelectionGroupedByParent.Add(ParentNode->NodeGuid, TArray<UDisplayClusterConfiguratorBaseNode*>());
				}

				SelectionGroupedByParent[ParentNode->NodeGuid].Add(BaseNode);
			}
		}
	}

	if (SelectionGroupedByParent.Num() == 1)
	{
		// All of the selected nodes are siblings, alignment is allowed.
		return true;
	}
	else if (SelectionGroupedByParent.Num() == 2)
	{
		// In this case, it could be that the parent node of the siblings is also selected. At least one of the arrays in the map
		// has to only contain one element in that case, and that element has to be the parent of the other map entry's list of nodes
		TArray<FGuid> Keys;
		SelectionGroupedByParent.GenerateKeyArray(Keys);

		// If one of the lists has a count of one, and that node is the parent of the other list, then we can align all selected nodes.
		if (SelectionGroupedByParent[Keys[0]].Num() == 1 && SelectionGroupedByParent[Keys[0]][0]->NodeGuid == Keys[1])
		{
			return true;
		}
		else if (SelectionGroupedByParent[Keys[1]].Num() == 1 && SelectionGroupedByParent[Keys[1]][0]->NodeGuid == Keys[0])
		{
			return true;
		}
		else
		{
			return false;
		}
	}
	else
	{
		// If there are more than two parent groups selected, we can't align all of the selected nodes together.
		return false;
	}
}

void SDisplayClusterConfiguratorGraphEditor::AlignNodes(ENodeAlignment Alignment)
{
	TArray<UDisplayClusterConfiguratorBaseNode*> NodesToUpdate;
	const FGraphPanelSelectionSet& CurrentlySelectedNodes = GetSelectedNodes();

	// Window nodes should also update their children if they are aligned except in the case that the window node is being aligned _with_ its children.
	bool bCanUpdateChildren = true;

	for (UObject* Node : CurrentlySelectedNodes)
	{
		if (UDisplayClusterConfiguratorBaseNode* BaseNode = Cast<UDisplayClusterConfiguratorBaseNode>(Node))
		{
			NodesToUpdate.Add(BaseNode);

			// If any of the nodes being aligned are viewport nodes, do not allow window nodes to update their children, as that can cause the viewport children to
			// be shifted out of alignment when the window nodes propagate the position change to their children.
			if (BaseNode->IsA<UDisplayClusterConfiguratorViewportNode>())
			{
				bCanUpdateChildren = false;
			}
		}
	}

	// Create a transaction in this scope to override the normal alignment transaction so that when nodes update their children's positions
	// as part of the OnNodeAligned call after the alignment, the children's Modify calls are in the same transaction as the original node alignment.
	// Need to do this as a pointer because we want a different transaction name for each alignment type, and FScopedTransactions don't allow the
	// transaction name to be changed after creation.
	TSharedPtr<FScopedTransaction> Transaction;

	switch (Alignment)
	{
	case ENodeAlignment::Top:
		Transaction = MakeShareable(new FScopedTransaction(FGraphEditorCommands::Get().AlignNodesTop->GetLabel()));
		OnAlignTop();
		break;

	case ENodeAlignment::Middle:
		Transaction = MakeShareable(new FScopedTransaction(FGraphEditorCommands::Get().AlignNodesMiddle->GetLabel()));
		OnAlignMiddle();
		break;

	case ENodeAlignment::Bottom:
		Transaction = MakeShareable(new FScopedTransaction(FGraphEditorCommands::Get().AlignNodesBottom->GetLabel()));
		OnAlignBottom();
		break;

	case ENodeAlignment::Left:
		Transaction = MakeShareable(new FScopedTransaction(FGraphEditorCommands::Get().AlignNodesLeft->GetLabel()));
		OnAlignLeft();
		break;

	case ENodeAlignment::Center:
		Transaction = MakeShareable(new FScopedTransaction(FGraphEditorCommands::Get().AlignNodesCenter->GetLabel()));
		OnAlignCenter();
		break;

	case ENodeAlignment::Right:
		Transaction = MakeShareable(new FScopedTransaction(FGraphEditorCommands::Get().AlignNodesRight->GetLabel()));
		OnAlignRight();
		break;
	}

	for (UDisplayClusterConfiguratorBaseNode* Node : NodesToUpdate)
	{
		Node->OnNodeAligned(bCanUpdateChildren);
	}

	Transaction.Reset();
}

bool SDisplayClusterConfiguratorGraphEditor::CanFillParentNode() const
{
	const TSet<UObject*>& SelectedNodes = GetSelectedNodes();
	for (UObject* Node : SelectedNodes)
	{
		// Viewport and window nodes can be resized to fill parents, so if any of the selected nodes is a viewport or window, return true
		if (Node->IsA<UDisplayClusterConfiguratorWindowNode>() || Node->IsA<UDisplayClusterConfiguratorViewportNode>())
		{
			return true;
		}
	}

	return false;
}

void SDisplayClusterConfiguratorGraphEditor::FillParentNode()
{
	const TSet<UObject*>& SelectedNodes = GetSelectedNodes();

	FScopedTransaction Transaction(FDisplayClusterConfiguratorOutputMappingCommands::Get().FillParentNode->GetLabel());
	bool bNodesChanged = false;

	for (UObject* Node : SelectedNodes)
	{
		UDisplayClusterConfiguratorBaseNode* BaseNode = CastChecked<UDisplayClusterConfiguratorBaseNode>(Node);
		if (BaseNode->IsA<UDisplayClusterConfiguratorWindowNode>() || BaseNode->IsA<UDisplayClusterConfiguratorViewportNode>())
		{
			BaseNode->FillParent();
			bNodesChanged = true;
		}
	}

	if (!bNodesChanged)
	{
		Transaction.Cancel();
	}
}

bool SDisplayClusterConfiguratorGraphEditor::CanSizeToChildNodes() const
{
	const TSet<UObject*>& SelectedNodes = GetSelectedNodes();
	for (UObject* Node : SelectedNodes)
	{
		// Host and window nodes can be resized to wrap their children, so if any of the selected nodes is a host or window, return true
		if (Node->IsA<UDisplayClusterConfiguratorHostNode>() || Node->IsA<UDisplayClusterConfiguratorWindowNode>())
		{
			return true;
		}
	}

	return false;
}

void SDisplayClusterConfiguratorGraphEditor::SizeToChildNodes()
{
	const TSet<UObject*>& SelectedNodes = GetSelectedNodes();

	FScopedTransaction Transaction(FDisplayClusterConfiguratorOutputMappingCommands::Get().SizeToChildNodes->GetLabel());
	bool bNodesChanged = false;

	for (UObject* Node : SelectedNodes)
	{
		UDisplayClusterConfiguratorBaseNode* BaseNode = CastChecked<UDisplayClusterConfiguratorBaseNode>(Node);
		if (BaseNode->IsA<UDisplayClusterConfiguratorHostNode>() || BaseNode->IsA<UDisplayClusterConfiguratorWindowNode>())
		{
			BaseNode->SizeToChildren();
			bNodesChanged = true;
		}
	}

	if (!bNodesChanged)
	{
		Transaction.Cancel();
	}
}

bool SDisplayClusterConfiguratorGraphEditor::CanTransformNode() const
{
	const TSet<UObject*>& SelectedNodes = GetSelectedNodes();

	if (!SelectedNodes.Num())
	{
		return false;
	}

	for (UObject* Node : SelectedNodes)
	{
		// Transforms can only be applied to viewports at this point.
		if (!Node->IsA<UDisplayClusterConfiguratorViewportNode>())
		{
			return false;
		}
	}

	return true;
}

void SDisplayClusterConfiguratorGraphEditor::RotateNode(float InRotation)
{
	const TSet<UObject*>& SelectedNodes = GetSelectedNodes();

	FScopedTransaction Transaction(LOCTEXT("RotateNodeTransaction", "Rotate Nodes"));
	bool bNodesChanged = false;

	for (UObject* Node : SelectedNodes)
	{
		if (UDisplayClusterConfiguratorViewportNode* ViewportNode = Cast<UDisplayClusterConfiguratorViewportNode>(Node))
		{
			ViewportNode->RotateViewport(InRotation);
			bNodesChanged = true;
		}
	}

	if (!bNodesChanged)
	{
		Transaction.Cancel();
	}
}

void SDisplayClusterConfiguratorGraphEditor::FlipNode(bool bFlipHorizontal, bool bFlipVertical)
{
	const TSet<UObject*>& SelectedNodes = GetSelectedNodes();

	FScopedTransaction Transaction(LOCTEXT("FlipNodeTransaction", "Flip Nodes"));
	bool bNodesChanged = false;

	for (UObject* Node : SelectedNodes)
	{
		if (UDisplayClusterConfiguratorViewportNode* ViewportNode = Cast<UDisplayClusterConfiguratorViewportNode>(Node))
		{
			ViewportNode->FlipViewport(bFlipHorizontal, bFlipVertical);
			bNodesChanged = true;
		}
	}

	if (!bNodesChanged)
	{
		Transaction.Cancel();
	}
}

void SDisplayClusterConfiguratorGraphEditor::ResetNodeTransform()
{
	const TSet<UObject*>& SelectedNodes = GetSelectedNodes();

	FScopedTransaction Transaction(LOCTEXT("ResetNodeTransformTransaction", "Reset Node Transform"));
	bool bNodesChanged = false;

	for (UObject* Node : SelectedNodes)
	{
		if (UDisplayClusterConfiguratorViewportNode* ViewportNode = Cast<UDisplayClusterConfiguratorViewportNode>(Node))
		{
			ViewportNode->ResetTransform();
			bNodesChanged = true;
		}
	}

	if (!bNodesChanged)
	{
		Transaction.Cancel();
	}
}

void SDisplayClusterConfiguratorGraphEditor::SortNodes()
{
	// Sort the nodes via their depth to ensure that overlapping nodes are properly rendered on top of each order and mouse interactions are properly handled
	TSlotlessChildren<SNodePanel::SNode>* VisibleChildren = static_cast<TSlotlessChildren<SNodePanel::SNode>*>(GetGraphPanel()->GetVisibleChildren());

	struct SNodeLessThanSort
	{
		FORCEINLINE bool operator()(const TSharedRef<SNodePanel::SNode>& A, const TSharedRef<SNodePanel::SNode>& B) const { return A.Get() < B.Get(); }
	};

	VisibleChildren->Sort(SNodeLessThanSort());
}

void SDisplayClusterConfiguratorGraphEditor::Construct(const FArguments& InArgs, const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& InToolkit, const TSharedRef<FDisplayClusterConfiguratorViewOutputMapping>& InViewOutputMapping)
{
	ToolkitPtr = InToolkit;
	ViewOutputMappingPtr = InViewOutputMapping;
	bSelectionSetDirectly = false;

	BindCommands();

	InToolkit->RegisterOnObjectSelected(IDisplayClusterConfiguratorBlueprintEditor::FOnObjectSelectedDelegate::CreateSP(this, &SDisplayClusterConfiguratorGraphEditor::OnObjectSelected));
	InToolkit->RegisterOnConfigReloaded(IDisplayClusterConfiguratorBlueprintEditor::FOnConfigReloadedDelegate::CreateSP(this, &SDisplayClusterConfiguratorGraphEditor::OnConfigReloaded));
	InToolkit->RegisterOnClusterChanged(IDisplayClusterConfiguratorBlueprintEditor::FOnClusterChangedDelegate::CreateSP(this, &SDisplayClusterConfiguratorGraphEditor::OnClusterChanged));

	ClusterConfiguratorGraph = CastChecked<UDisplayClusterConfiguratorGraph>(InArgs._GraphToEdit);

	SGraphEditor::FGraphEditorEvents GraphEvents;
	GraphEvents.OnSelectionChanged = SGraphEditor::FOnSelectionChanged::CreateSP(this, &SDisplayClusterConfiguratorGraphEditor::OnSelectedNodesChanged);
	GraphEvents.OnCreateNodeOrPinMenu = SGraphEditor::FOnCreateNodeOrPinMenu::CreateSP(this, &SDisplayClusterConfiguratorGraphEditor::OnCreateNodeOrPinMenu);
	GraphEvents.OnNodeDoubleClicked = FSingleNodeEvent::CreateSP(this, &SDisplayClusterConfiguratorGraphEditor::OnNodeDoubleClicked);

	FGraphAppearanceInfo AppearanceInfo;
	AppearanceInfo.CornerText = LOCTEXT("AppearanceCornerText", "STEP 3");

	SGraphEditor::FArguments Arguments;
	Arguments._GraphToEdit = InArgs._GraphToEdit;
	Arguments._AdditionalCommands = CommandList;
	Arguments._IsEditable = true;
	Arguments._GraphEvents = GraphEvents;
	Arguments._Appearance = AppearanceInfo;

	SGraphEditor::Construct(Arguments);

	SetCanTick(true);
	GEditor->RegisterForUndo(this);
}

SDisplayClusterConfiguratorGraphEditor::~SDisplayClusterConfiguratorGraphEditor()
{
	GEditor->UnregisterForUndo(this);
}

#undef LOCTEXT_NAMESPACE
