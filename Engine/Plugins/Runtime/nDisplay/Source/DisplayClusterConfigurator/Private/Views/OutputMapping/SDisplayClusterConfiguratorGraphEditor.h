// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "GraphEditor.h"
#include "NodeFactory.h"
#include "UObject/StrongObjectPtr.h"
#include "EditorUndoClient.h"

#include "Views/OutputMapping/EdNodes/DisplayClusterConfiguratorCanvasNode.h"

class FDisplayClusterConfiguratorViewOutputMapping;
class FDisplayClusterConfiguratorBlueprintEditor;
class FMenuBuilder;
class FUICommandList;
class SDisplayClusterConfiguratorGraphEditor;
class SDisplayClusterConfiguratorCanvasNode;
class SGraphNode;
class UDisplayClusterConfiguratorGraph;
class UEdGraph;
class UEdGraphNode;
class UTexture;

struct FActionMenuContent;

enum class ENodeAlignment : uint8
{
	Top,
	Middle,
	Bottom,
	Left,
	Center,
	Right
};

class SDisplayClusterConfiguratorGraphEditor
	: public SGraphEditor
	, public FEditorUndoClient
{
public:
	SLATE_BEGIN_ARGS(SDisplayClusterConfiguratorGraphEditor)
		: _GraphToEdit(nullptr)
	{}

		SLATE_ARGUMENT(UEdGraph*, GraphToEdit)
	SLATE_END_ARGS()

	void Construct(
		const FArguments& InArgs,
		const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& InToolkit,
		const TSharedRef<FDisplayClusterConfiguratorViewOutputMapping>& InViewOutputMapping);

	~SDisplayClusterConfiguratorGraphEditor();

	//~ SWidget interface
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	//~ End SWidget interface

	// FEditorUndoClient Interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override { PostUndo(bSuccess); }
	// End of FEditorUndoClient

	void FindAndSelectObjects(const TArray<UObject*>& ObjectsToSelect);
	void JumpToObject(UObject* InObject);

	TSharedPtr<FUICommandList> GetCommandList() const { return CommandList; }

private:
	void OnSelectedNodesChanged(const TSet<UObject*>& NewSelection);
	void OnNodeDoubleClicked(UEdGraphNode* ClickedNode);
	void OnObjectSelected();
	void OnConfigReloaded();
	void OnClusterChanged();
	void RebuildGraph();

	/** Callback to create contextual menu for graph nodes in graph panel */
	FActionMenuContent OnCreateNodeOrPinMenu(UEdGraph* CurrentGraph, const UEdGraphNode* InGraphNode, const UEdGraphPin* InGraphPin, FMenuBuilder* MenuBuilder, bool bIsDebugging);
	void BindCommands();

	void AddNewClusterNode();
	bool CanAddNewClusterNode() const;

	void AddNewViewport();
	bool CanAddNewViewport() const;

	void BrowseDocumentation();

	void DeleteSelectedNodes();
	bool CanDeleteNodes() const;

	void CopySelectedNodes();
	bool CanCopyNodes() const;

	void CutSelectedNodes();
	bool CanCutNodes() const;

	void PasteNodes();
	bool CanPasteNodes() const;

	void DuplicateNodes();
	bool CanDuplicateNodes() const;

	bool CanAlignNodes() const;
	void AlignNodes(ENodeAlignment Alignment);

	bool CanFillParentNode() const;
	void FillParentNode();

	bool CanSizeToChildNodes() const;
	void SizeToChildNodes();

	bool CanTransformNode() const;
	void RotateNode(float InRotation);
	void FlipNode(bool bFlipHorizontal, bool bFlipVertical);
	void ResetNodeTransform();

	void SortNodes();

private:
	TWeakPtr<FDisplayClusterConfiguratorBlueprintEditor> ToolkitPtr;

	TWeakObjectPtr<UDisplayClusterConfiguratorGraph> ClusterConfiguratorGraph;

	TWeakPtr<FDisplayClusterConfiguratorViewOutputMapping> ViewOutputMappingPtr;

	/** Indicates if the graph's SelectedNodesChange event was invoked from user action or changed directly through code. */
	bool bSelectionSetDirectly;

	/** The nodes menu command list */
	TSharedPtr<FUICommandList> CommandList;
};