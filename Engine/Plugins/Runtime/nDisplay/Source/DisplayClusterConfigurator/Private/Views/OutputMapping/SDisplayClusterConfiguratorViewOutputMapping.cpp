// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/OutputMapping/SDisplayClusterConfiguratorViewOutputMapping.h"

#include "DisplayClusterConfiguratorCommands.h"
#include "DisplayClusterConfiguratorBlueprintEditor.h"
#include "IDisplayClusterConfigurator.h"
#include "Views/OutputMapping/DisplayClusterConfiguratorViewOutputMapping.h"
#include "Views/OutputMapping/SDisplayClusterConfiguratorGraphEditor.h"
#include "Views/OutputMapping/Widgets/SDisplayClusterConfiguratorOutputMappingToolbar.h"
#include "Views/OutputMapping/Widgets/SDisplayClusterConfiguratorRuler.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UICommandList.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SDPIScaler.h"
#include "Widgets/Layout/SSpacer.h"

#define LOCTEXT_NAMESPACE "SDisplayClusterConfiguratorViewOutputMapping"

void SDisplayClusterConfiguratorViewOutputMapping::Construct(const FArguments& InArgs, const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& InToolkit, const TSharedRef<SDisplayClusterConfiguratorGraphEditor>& InGraphEditor, const TSharedRef<FDisplayClusterConfiguratorViewOutputMapping> InViewOutputMapping)
{
	ToolkitPtr = InToolkit;
	GraphEditorPtr = InGraphEditor;
	ViewOutputMappingPtr = InViewOutputMapping;

	BindCommands();

	const TSharedPtr<FUICommandList>& AdditionalCommands = InArgs._AdditionalCommands;
	if (AdditionalCommands.IsValid())
	{
		CommandList->Append(AdditionalCommands.ToSharedRef());
	}

	TSharedPtr<FUICommandList> GraphEditorCommands = InGraphEditor->GetCommandList();
	if (GraphEditorCommands.IsValid())
	{
		CommandList->Append(GraphEditorCommands.ToSharedRef());
	}

	SDisplayClusterConfiguratorViewBase::Construct(
		SDisplayClusterConfiguratorViewBase::FArguments()
		.Content()
		[

			SNew(SGridPanel)
			.FillColumn(1, 1.0f)
			.FillRow(1, 1.0f)

			// Corner
			+ SGridPanel::Slot(0, 0)
			[
				SNew(SBorder)
				.BorderImage(FCoreStyle::Get().GetBrush("GenericWhiteBox"))
				.BorderBackgroundColor(FLinearColor(FColor(48, 48, 48)))
				.Visibility(this, &SDisplayClusterConfiguratorViewOutputMapping::GetRulerVisibility)
			]

			// Top Ruler
			+ SGridPanel::Slot(1, 0)
			[
				SAssignNew(TopRuler, SDisplayClusterConfiguratorRuler)
				.Orientation(Orient_Horizontal)
				.Visibility(this, &SDisplayClusterConfiguratorViewOutputMapping::GetRulerVisibility)
			]

			// Side Ruler
			+ SGridPanel::Slot(0, 1)
			[
				SAssignNew(SideRuler, SDisplayClusterConfiguratorRuler)
				.Orientation(Orient_Vertical)
				.Visibility(this, &SDisplayClusterConfiguratorViewOutputMapping::GetRulerVisibility)
			]

			// Graph area
			+ SGridPanel::Slot(1, 1)
			[
				SNew(SOverlay)
				.Visibility(EVisibility::Visible)
				.Clipping(EWidgetClipping::ClipToBoundsAlways)

				+SOverlay::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					SAssignNew(PreviewSurface, SDPIScaler)
					.DPIScale(1.f)
					[
						InGraphEditor
					]
				]

				+ SOverlay::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					CreateOverlayUI()
				]
			]
		],
		InToolkit);

	// Set initial zoom
	FVector2f GraphEditorLocation = FVector2f::ZeroVector;
	float GraphEditorZoomAmount = 0;
	InGraphEditor->GetViewLocation(GraphEditorLocation, GraphEditorZoomAmount);
	InGraphEditor->SetViewLocation(GraphEditorLocation, 0.3f);
}

void SDisplayClusterConfiguratorViewOutputMapping::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	TSharedPtr<SDisplayClusterConfiguratorGraphEditor> GraphEditor = GraphEditorPtr.Pin();
	check(GraphEditor.IsValid());

	FVector2f GraphEditorLocation = FVector2f::ZeroVector;
	float GraphEditorZoomAmount = 0;
	GraphEditor->GetViewLocation(GraphEditorLocation, GraphEditorZoomAmount);

	const float ViewScale = GetViewScale();
	const float DPIScale = GetDPIScale();
	const float RulerScale = ViewScale * GraphEditorZoomAmount;

	// Compute the origin in absolute space.
	FGeometry RootGeometry = PreviewSurface->GetCachedGeometry();
	FVector2f AbsoluteOrigin = MakeGeometryWindowLocal(RootGeometry).LocalToAbsolute(FVector2f::ZeroVector);
	FVector2f AbsoluteOriginWithOffset = AbsoluteOrigin - GraphEditorLocation * GraphEditorZoomAmount * DPIScale;

	TopRuler->SetRuling(FDeprecateSlateVector2D(AbsoluteOriginWithOffset), 1.f / RulerScale);
	SideRuler->SetRuling(FDeprecateSlateVector2D(AbsoluteOriginWithOffset), 1.f / RulerScale);

	if (IsHovered())
	{
		// Get cursor in absolute window space.
		FVector2D CursorPos = FSlateApplication::Get().GetCursorPos();
		CursorPos = MakeGeometryWindowLocal(RootGeometry).LocalToAbsolute(RootGeometry.AbsoluteToLocal(CursorPos));

		TopRuler->SetCursor(CursorPos);
		SideRuler->SetCursor(CursorPos);
	}
	else
	{
		TopRuler->SetCursor(TOptional<FVector2D>());
		SideRuler->SetCursor(TOptional<FVector2D>());
	}
}

FReply SDisplayClusterConfiguratorViewOutputMapping::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

FGeometry SDisplayClusterConfiguratorViewOutputMapping::MakeGeometryWindowLocal(const FGeometry& WidgetGeometry) const
{
	FGeometry NewGeometry = WidgetGeometry;

	TSharedPtr<SWindow> WidgetWindow = FSlateApplication::Get().FindWidgetWindow(SharedThis(this));
	if (WidgetWindow.IsValid())
	{
		TSharedRef<SWindow> CurrentWindowRef = WidgetWindow.ToSharedRef();

		NewGeometry.AppendTransform(FSlateLayoutTransform(Inverse(CurrentWindowRef->GetPositionInScreen())));
	}

	return NewGeometry;
}

EVisibility SDisplayClusterConfiguratorViewOutputMapping::GetRulerVisibility() const
{
	TSharedPtr<FDisplayClusterConfiguratorViewOutputMapping> ViewOutputMapping = ViewOutputMappingPtr.Pin();
	check(ViewOutputMapping.IsValid());

	if (ViewOutputMapping->GetOutputMappingSettings().bShowRuler)
	{
		return EVisibility::Visible;
	}

	return EVisibility::Hidden;
}

TSharedRef<SWidget> SDisplayClusterConfiguratorViewOutputMapping::CreateOverlayUI()
{
	return SNew(SOverlay)

	// Top bar with buttons for changing the designer
	+ SOverlay::Slot()
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Top)
	.Padding(2.f)
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Top)
		[
			SNew(SDisplayClusterConfiguratorOutputMappingToolbar, ViewOutputMappingPtr)
			.CommandList(CommandList)
		]

		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		[
			SNew(SSpacer)
			.Size(FVector2D(1, 1))
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(0, 3, 15, 3)
		[
			SNew(STextBlock)
			.TextStyle(FAppStyle::Get(), "Graph.ZoomText")
			.Text(this, &SDisplayClusterConfiguratorViewOutputMapping::GetCursorPositionText)
			.ColorAndOpacity(FLinearColor(1.f, 1.f, 1.f, 0.25f))
			.Visibility(this, &SDisplayClusterConfiguratorViewOutputMapping::GetCursorPositionTextVisibility)
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(0, 3, 0, 3)
		[
			SNew(STextBlock)
			.TextStyle(FAppStyle::Get(), "Graph.ZoomText")
			.Text(this, &SDisplayClusterConfiguratorViewOutputMapping::GetViewScaleText)
			.ColorAndOpacity(FLinearColor(1.f, 1.f, 1.f, 0.25f))
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SSpacer)
			.Size(FVector2D(100, 1))
		]
	];
}
FText SDisplayClusterConfiguratorViewOutputMapping::GetCursorPositionText() const
{
	TSharedPtr<SDisplayClusterConfiguratorGraphEditor> GraphEditor = GraphEditorPtr.Pin();
	check(GraphEditor.IsValid());

	TSharedPtr<FDisplayClusterConfiguratorViewOutputMapping> ViewOutputMapping = ViewOutputMappingPtr.Pin();
	check(ViewOutputMapping != nullptr);

	const float ViewScale = ViewOutputMapping->GetOutputMappingSettings().ViewScale;

	FGeometry RootGeometry = PreviewSurface->GetCachedGeometry();
	const FVector2f CursorPos = RootGeometry.AbsoluteToLocal(FSlateApplication::Get().GetCursorPos());

	FVector2f GraphEditorLocation = FVector2f::ZeroVector;
	float GraphEditorZoomAmount = 0;
	GraphEditor->GetViewLocation(GraphEditorLocation, GraphEditorZoomAmount);

	FVector2f GraphPosition = ((CursorPos / GraphEditorZoomAmount) + GraphEditorLocation) / ViewScale;

	return FText::Format(LOCTEXT("CursorPositionFormat", "{0} x {1}"), FText::AsNumber(FMath::RoundToInt(GraphPosition.X)), FText::AsNumber(FMath::RoundToInt(GraphPosition.Y)));
}

EVisibility SDisplayClusterConfiguratorViewOutputMapping::GetCursorPositionTextVisibility() const
{
	return IsHovered() ? EVisibility::SelfHitTestInvisible : EVisibility::Collapsed;
}

float SDisplayClusterConfiguratorViewOutputMapping::GetViewScale() const
{
	TSharedPtr<FDisplayClusterConfiguratorViewOutputMapping> ViewOutputMapping = ViewOutputMappingPtr.Pin();
	check(ViewOutputMapping != nullptr);

	return ViewOutputMapping->GetOutputMappingSettings().ViewScale;
}

FText SDisplayClusterConfiguratorViewOutputMapping::GetViewScaleText() const
{
	TSharedPtr<FDisplayClusterConfiguratorViewOutputMapping> ViewOutputMapping = ViewOutputMappingPtr.Pin();
	check(ViewOutputMapping != nullptr);

	const float ViewScale = ViewOutputMapping->GetOutputMappingSettings().ViewScale;

	return FText::Format(LOCTEXT("ViewScaleFormat", "View Scale x{0}"), FText::AsNumber(ViewScale));
}

void SDisplayClusterConfiguratorViewOutputMapping::BindCommands()
{
	CommandList = MakeShareable(new FUICommandList);
}

float SDisplayClusterConfiguratorViewOutputMapping::GetDPIScale() const
{
	float DPIScale = 1.0f;
	TSharedPtr<SWindow> WidgetWindow = FSlateApplication::Get().FindWidgetWindow(SharedThis(this));
	if (WidgetWindow.IsValid())
	{
		DPIScale = WidgetWindow->GetNativeWindow()->GetDPIScaleFactor();
	}

	return DPIScale;
}

#undef LOCTEXT_NAMESPACE