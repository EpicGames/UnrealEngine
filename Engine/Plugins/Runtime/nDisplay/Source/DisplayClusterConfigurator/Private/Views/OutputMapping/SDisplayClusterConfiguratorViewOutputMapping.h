// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Views/SDisplayClusterConfiguratorViewBase.h"

#include "EditorUndoClient.h"

class FDisplayClusterConfiguratorBlueprintEditor;
class FDisplayClusterConfiguratorViewOutputMapping;
class FUICommandList;
class SDPIScaler;
class SDisplayClusterConfiguratorGraphEditor;
class SDisplayClusterConfiguratorRuler;

struct FGeometry;

class SDisplayClusterConfiguratorViewOutputMapping
	: public SDisplayClusterConfiguratorViewBase
	, public FEditorUndoClient
{
public:
	SLATE_BEGIN_ARGS(SDisplayClusterConfiguratorViewOutputMapping)
	{}
		SLATE_ARGUMENT(TSharedPtr<FUICommandList>, AdditionalCommands)
	SLATE_END_ARGS()

public:
	void Construct(
		const FArguments& InArgs, 
		const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& InToolkit, 
		const TSharedRef<SDisplayClusterConfiguratorGraphEditor>& InGraphEditor, 
		const TSharedRef<FDisplayClusterConfiguratorViewOutputMapping> InViewOutputMapping);

	//~ Begin SWidget Interface
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	//~ End SWidget Interface

private:
	FGeometry MakeGeometryWindowLocal(const FGeometry& WidgetGeometry) const;

	EVisibility GetRulerVisibility() const;

	/** Creates graph controls */
	TSharedRef<SWidget> CreateOverlayUI();

	void BindCommands();

	FText GetCursorPositionText() const;

	EVisibility GetCursorPositionTextVisibility() const;

	float GetViewScale() const;
	FText GetViewScaleText() const;

	float GetDPIScale() const;

private:
	TWeakPtr<FDisplayClusterConfiguratorBlueprintEditor> ToolkitPtr;

	TWeakPtr<SDisplayClusterConfiguratorGraphEditor> GraphEditorPtr;

	TWeakPtr<FDisplayClusterConfiguratorViewOutputMapping> ViewOutputMappingPtr;

	TSharedPtr<SDPIScaler> PreviewSurface;

	TSharedPtr<SDisplayClusterConfiguratorRuler> TopRuler;

	TSharedPtr<SDisplayClusterConfiguratorRuler> SideRuler;

	/** The graph toolbar command list */
	TSharedPtr<FUICommandList> CommandList;
};
