// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDisplayClusterConfiguratorExternalImage.h"

#include "IImageWrapper.h"
#include "IImageWrapperModule.h"
#include "Styling/AppStyle.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/FileHelper.h"
#include "Modules/ModuleManager.h"
#include "Brushes/SlateDynamicImageBrush.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SEnableBox.h"

void SDisplayClusterConfiguratorExternalImage::Construct(const FArguments& InArgs)
{
	ImagePath = InArgs._ImagePath;
	MaxImageSize = InArgs._MaxImageSize;
	MinImageSize = InArgs._MinImageSize;

	LoadImage();

	ChildSlot
	[
		SNew(SVerticalBox)

		+SVerticalBox::Slot()
		[
			SNew(SBorder)
			.BorderImage(InArgs._ShowShadow ? FAppStyle::Get().GetBrush("ExternalImagePicker.ThumbnailShadow") : FAppStyle::Get().GetBrush("NoBorder"))
			.Padding(4.0f)
			.Content()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("ExternalImagePicker.BlankImage"))
				.Padding(0.0f)
				.Content()
				[
					SNew(SBox)
					.WidthOverride(this, &SDisplayClusterConfiguratorExternalImage::GetImageWidth)
					.HeightOverride(this, &SDisplayClusterConfiguratorExternalImage::GetImageHeight)
					[
						SNew(SEnableBox)
						[
							SNew(SImage)
							.Image(this, &SDisplayClusterConfiguratorExternalImage::GetImageBrush)
						]
					]
				]
			]
		]
	];
}

void SDisplayClusterConfiguratorExternalImage::SetImagePath(const FString& NewImagePath)
{
	if (ImagePath != NewImagePath)
	{
		ImagePath = NewImagePath;
		LoadImage();
	}
}

void SDisplayClusterConfiguratorExternalImage::LoadImage()
{
	if (ImageBrush.IsValid())
	{
		FSlateApplication::Get().GetRenderer()->ReleaseDynamicResource(*ImageBrush.Get());
		ImageBrush.Reset();
	}

	if (!FPlatformFileManager::Get().GetPlatformFile().FileExists(*ImagePath))
	{
		return;
	}

	TArray64<uint8> RawFileData;
	if (FFileHelper::LoadFileToArray(RawFileData, *ImagePath))
	{
		IImageWrapperModule& ImageWrapperModule = FModuleManager::LoadModuleChecked<IImageWrapperModule>(FName("ImageWrapper"));
		EImageFormat Format = ImageWrapperModule.DetectImageFormat(RawFileData.GetData(), RawFileData.Num());

		if (Format == EImageFormat::Invalid)
		{
			return;
		}

		TSharedPtr<IImageWrapper> ImageWrapper = ImageWrapperModule.CreateImageWrapper(Format);
		if (ImageWrapper.IsValid() && ImageWrapper->SetCompressed(RawFileData.GetData(), RawFileData.Num()))
		{
			TArray<uint8> RawData;
			if (ImageWrapper->GetRaw(ERGBFormat::BGRA, 8, RawData))
			{
				if (FSlateApplication::Get().GetRenderer()->GenerateDynamicImageResource(*ImagePath, ImageWrapper->GetWidth(), ImageWrapper->GetHeight(), RawData))
				{
					ImageBrush = MakeShareable(new FSlateDynamicImageBrush(*ImagePath, FVector2D(ImageWrapper->GetWidth(), ImageWrapper->GetHeight())));
				}
			}
		}
	}
}

FVector2D SDisplayClusterConfiguratorExternalImage::GetConstrainedImageSize() const
{
	const FVector2D Size = GetImageBrush()->ImageSize;

	FVector2D MinSize = MinImageSize.Get(FVector2D(64.0f));
	FVector2D MaxSize = MaxImageSize.Get(FVector2D(128.0f));

	// make sure we have a valid size in case the image didn't get loaded
	const FVector2D ValidSize(FMath::Max(Size.X, MinSize.X), FMath::Max(Size.Y, MinSize.Y));

	// keep image aspect but don't display it above a reasonable size
	const float ImageAspect = ValidSize.X / ValidSize.Y;
	const FVector2D ConstrainedSize(FMath::Min(ValidSize.X, MaxSize.X), FMath::Min(ValidSize.Y, MaxSize.Y));
	return FVector2D(FMath::Min(ConstrainedSize.X, ConstrainedSize.Y * ImageAspect), FMath::Min(ConstrainedSize.Y, ConstrainedSize.X / ImageAspect));
}

const FSlateBrush* SDisplayClusterConfiguratorExternalImage::GetImageBrush() const
{
	return ImageBrush.IsValid() ? ImageBrush.Get() : FAppStyle::Get().GetBrush("ExternalImagePicker.BlankImage");
}

FOptionalSize SDisplayClusterConfiguratorExternalImage::GetImageWidth() const
{
	return GetConstrainedImageSize().X;
}

FOptionalSize SDisplayClusterConfiguratorExternalImage::GetImageHeight() const
{
	return GetConstrainedImageSize().Y;
}