// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Types/SlateStructs.h"

struct FSlateDynamicImageBrush;

class SDisplayClusterConfiguratorExternalImage : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDisplayClusterConfiguratorExternalImage)
		: _MaxImageSize(FVector2D(128.0f))
		, _MinImageSize(FVector2D(64.0f))
		, _ShowShadow(true)
	{ }
		SLATE_ARGUMENT(FString, ImagePath)
		SLATE_ATTRIBUTE(FVector2D, MaxImageSize)
		SLATE_ATTRIBUTE(FVector2D, MinImageSize)
		SLATE_ARGUMENT(bool, ShowShadow)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	const FString& GetImagePath() const { return ImagePath;}
	void SetImagePath(const FString& NewImagePath);

private:
	void LoadImage();

	FVector2D GetConstrainedImageSize() const;

	const FSlateBrush* GetImageBrush() const;
	FOptionalSize GetImageWidth() const;
	FOptionalSize GetImageHeight() const;

private:
	TSharedPtr<FSlateDynamicImageBrush> ImageBrush;
	FString ImagePath;
	TAttribute<FVector2D> MaxImageSize;
	TAttribute<FVector2D> MinImageSize;
};