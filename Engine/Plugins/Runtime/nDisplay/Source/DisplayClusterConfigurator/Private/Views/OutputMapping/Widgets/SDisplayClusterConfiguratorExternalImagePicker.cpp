// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDisplayClusterConfiguratorExternalImagePicker.h"

#include "SDisplayClusterConfiguratorExternalImage.h"

#include "DesktopPlatformModule.h"
#include "Styling/AppStyle.h"
#include "Framework/Application/SlateApplication.h"
#include "SResetToDefaultMenu.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"

#define LOCTEXT_NAMESPACE "SDisplayClusterConfiguratorExternalImagePicker"

void SDisplayClusterConfiguratorExternalImagePicker::Construct(const FArguments& InArgs)
{
	Extensions = InArgs._Extensions;
	OnImagePathPicked = InArgs._OnImagePathPicked;

	FString ImageExtension = FPaths::GetExtension(InArgs._ImagePath);
	if (!ImageExtension.IsEmpty() && !Extensions.Contains(ImageExtension))
	{
		Extensions.Add(ImageExtension);
	}

	if (!Extensions.Num())
	{
		Extensions.Add("png");
	}

    ChildSlot
	[
		SNew(SHorizontalBox)

		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SAssignNew(ExternalImage, SDisplayClusterConfiguratorExternalImage)
			.ImagePath(InArgs._ImagePath)
		]

		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.ToolTipText(LOCTEXT("FileButtonToolTipText", "Choose a file from this computer") )
			.OnClicked(FOnClicked::CreateSP(this, &SDisplayClusterConfiguratorExternalImagePicker::OpenFileDialog))
			.ContentPadding(2.0f)
			.ForegroundColor(FSlateColor::UseForeground())
			.IsFocusable(false)
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("ExternalImagePicker.PickImageButton"))
				.ColorAndOpacity( FSlateColor::UseForeground())
			]
		]

		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SResetToDefaultMenu)
			.OnResetToDefault(FSimpleDelegate::CreateSP(this, &SDisplayClusterConfiguratorExternalImagePicker::ResetToDefault))
			.DiffersFromDefault(this, &SDisplayClusterConfiguratorExternalImagePicker::DiffersFromDefault)
		]
	];
}

FReply SDisplayClusterConfiguratorExternalImagePicker::OpenFileDialog()
{
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (DesktopPlatform)
	{
		FText Title;
		FString TitleExtensions;
		FString AssociatedExtensions;
		if (Extensions.Num() > 1)
		{
			Title = LOCTEXT("Image", "Image");
			TitleExtensions = TEXT("*.") + FString::Join(Extensions, TEXT(", *."));
			AssociatedExtensions = TitleExtensions.Replace(TEXT(", "), TEXT(";"));
		}
		else
		{
			Title = FText::FromString(Extensions[0]);
			TitleExtensions = TEXT("*.") + Extensions[0];
			AssociatedExtensions = TitleExtensions;
		}

		TArray<FString> OutFiles;
		const FString Filter = FString::Printf(TEXT("%s files (%s)|%s"), *Title.ToString(), *TitleExtensions, *AssociatedExtensions);
		const FString DefaultPath = FPaths::GetPath(FPaths::GetProjectFilePath());

		TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
		void* ParentWindowHandle = (ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid()) ? ParentWindow->GetNativeWindow()->GetOSWindowHandle() : nullptr;

		if (DesktopPlatform->OpenFileDialog(ParentWindowHandle, FText::Format(LOCTEXT("ImagePickerDialogTitle", "Choose a {0} file"), Title).ToString(), DefaultPath, TEXT(""), Filter, EFileDialogFlags::None, OutFiles))
		{
			check(OutFiles.Num() == 1);

			FString SelectedImagePath = FPaths::ConvertRelativePathToFull(OutFiles[0]);

			OnImagePathPicked.ExecuteIfBound(SelectedImagePath);
			ExternalImage->SetImagePath(SelectedImagePath);
		}
	}

	return FReply::Handled();
}

void SDisplayClusterConfiguratorExternalImagePicker::ResetToDefault()
{
	OnImagePathPicked.ExecuteIfBound(TEXT(""));
	ExternalImage->SetImagePath(TEXT(""));
}

bool SDisplayClusterConfiguratorExternalImagePicker::DiffersFromDefault() const
{
	const FString& CurrentImagePath = ExternalImage->GetImagePath();
	return !CurrentImagePath.IsEmpty();
}

#undef LOCTEXT_NAMESPACE