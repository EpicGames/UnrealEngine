// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"


class SDisplayClusterConfiguratorExternalImage;

DECLARE_DELEGATE_OneParam(FOnImagePathPicked, const FString&);

class SDisplayClusterConfiguratorExternalImagePicker : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDisplayClusterConfiguratorExternalImagePicker) {}
		SLATE_ARGUMENT(FString, ImagePath)
		SLATE_ARGUMENT(TArray<FString>, Extensions)
		SLATE_EVENT(FOnImagePathPicked, OnImagePathPicked)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:
	FReply OpenFileDialog();
	void ResetToDefault();
	bool DiffersFromDefault() const;

private:
	TSharedPtr<SDisplayClusterConfiguratorExternalImage> ExternalImage;

	TArray<FString> Extensions;

	FOnImagePathPicked OnImagePathPicked;
};