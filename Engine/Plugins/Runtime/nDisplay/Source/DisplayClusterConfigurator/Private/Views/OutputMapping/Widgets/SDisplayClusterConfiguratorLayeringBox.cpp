// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDisplayClusterConfiguratorLayeringBox.h"

#include "Rendering/DrawElements.h"

void SDisplayClusterConfiguratorLayeringBox::Construct(const FArguments& InArgs)
{
	LayerOffset = InArgs._LayerOffset;
	ShadowBrush = InArgs._ShadowBrush;
	ShadowSize = InArgs._ShadowSize;

	SBox::Construct(SBox::FArguments().Content()[InArgs._Content.Widget]);
}

int32 SDisplayClusterConfiguratorLayeringBox::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	int32 ContentLayerId = LayerId + LayerOffset.Get(0);

	const FSlateBrush* ShadowResource = ShadowBrush.Get();
	FSlateDrawElement::MakeBox(
		OutDrawElements,
		ContentLayerId,
		AllottedGeometry.ToInflatedPaintGeometry(ShadowSize),
		ShadowResource,
		ESlateDrawEffect::None,
		FLinearColor(1.0f, 1.0f, 1.0f, 1.0f)
	);

	return SBox::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, ContentLayerId, InWidgetStyle, bParentEnabled);
}