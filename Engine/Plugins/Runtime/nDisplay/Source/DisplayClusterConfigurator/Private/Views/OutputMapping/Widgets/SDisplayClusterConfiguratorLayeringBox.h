// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SBox.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateTypes.h"

/**
 * An SBox that allows an offset to be added to the box's content when being painted, giving more control over what layer the content gets rendered at.
 * It also has the ability to draw a shadow that is appropriately layered behind the box.
 */
class SDisplayClusterConfiguratorLayeringBox : public SBox
{
public:
	SLATE_BEGIN_ARGS(SDisplayClusterConfiguratorLayeringBox)
		: _LayerOffset(0)
		, _ShadowBrush(FAppStyle::GetNoBrush())
		, _ShadowSize(FVector2D(12.0f))
	{}
		SLATE_DEFAULT_SLOT(FArguments, Content)
		SLATE_ATTRIBUTE(int32, LayerOffset)
		SLATE_ATTRIBUTE(const FSlateBrush*, ShadowBrush)
		SLATE_ARGUMENT(FVector2D, ShadowSize)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

public:
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

private:
	TAttribute<int32> LayerOffset;
	FInvalidatableBrushAttribute ShadowBrush;
	FVector2D ShadowSize;
};