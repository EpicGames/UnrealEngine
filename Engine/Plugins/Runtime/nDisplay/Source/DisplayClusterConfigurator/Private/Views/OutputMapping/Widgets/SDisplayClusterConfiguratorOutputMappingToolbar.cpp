// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/OutputMapping/Widgets/SDisplayClusterConfiguratorOutputMappingToolbar.h"

#include "DisplayClusterConfiguratorStyle.h"
#include "IDisplayClusterConfigurator.h"
#include "Views/OutputMapping/DisplayClusterConfiguratorOutputMappingCommands.h"
#include "Views/OutputMapping/DisplayClusterConfiguratorViewOutputMapping.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "SEditorViewportToolBarMenu.h"
#include "SViewportToolBarIconMenu.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/SSlider.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "SDisplayClusterConfiguratorOutputMappingToolbar"

const TArray<float> SDisplayClusterConfiguratorOutputMappingToolbar::ViewScales = { 2.0f, 1.5f, 1.0f, 0.75f, 0.5f, 1.0f / 3.0f };

void SDisplayClusterConfiguratorOutputMappingToolbar::Construct(const FArguments& InArgs, const TWeakPtr<FDisplayClusterConfiguratorViewOutputMapping>& InViewOutputMapping)
{
	ViewOutputMappingPtr = InViewOutputMapping;

	CommandList = InArgs._CommandList;

	ChildSlot
		[
			MakeToolBar(InArgs._Extenders)
		];

	SViewportToolBar::Construct(SViewportToolBar::FArguments());
}

TSharedRef<SWidget> SDisplayClusterConfiguratorOutputMappingToolbar::MakeToolBar(const TSharedPtr<FExtender> InExtenders)
{
	FToolBarBuilder ToolbarBuilder(CommandList, FMultiBoxCustomization::None, InExtenders);

	FName ToolBarStyle = "EditorViewportToolBar";
	ToolbarBuilder.SetStyle(&FAppStyle::Get(), ToolBarStyle);
	ToolbarBuilder.SetLabelVisibility(EVisibility::Collapsed);

	ToolbarBuilder.BeginSection("Advanced");
	{
		ToolbarBuilder.AddWidget(
			SNew(SEditorViewportToolbarMenu)
			.ParentToolBar(SharedThis(this))
			.Cursor(EMouseCursor::Default)
			.Image("EditorViewportToolBar.OptionsDropdown")
			.OnGetMenuContent(this, &SDisplayClusterConfiguratorOutputMappingToolbar::MakeAdvancedMenu)
		);
	}
	ToolbarBuilder.EndSection();

	ToolbarBuilder.BeginSection("ClusterItems");
	{
		ToolbarBuilder.AddWidget(
			SNew(SViewportToolBarIconMenu)
			.Cursor(EMouseCursor::Default)
			.Style(ToolBarStyle)
			.OnGetMenuContent(this, &SDisplayClusterConfiguratorOutputMappingToolbar::MakeWindowDisplayMenu)
			.Label(this, &SDisplayClusterConfiguratorOutputMappingToolbar::GetWindowDisplayText)
			.ToolTipText(LOCTEXT("WindowDisplayMenu_ToolTip", "Cluster Node Display"))
			.Icon(FSlateIcon(FDisplayClusterConfiguratorStyle::Get().GetStyleSetName(), "DisplayClusterConfigurator.OutputMapping.WindowDisplay"))
			.ParentToolBar(SharedThis(this))
		);

		ToolbarBuilder.AddToolBarButton(FDisplayClusterConfiguratorOutputMappingCommands::Get().ToggleOutsideViewports);
	}
	ToolbarBuilder.EndSection();

	ToolbarBuilder.BeginSection("View");
	{
		ToolbarBuilder.AddToolBarButton(FDisplayClusterConfiguratorOutputMappingCommands::Get().ZoomToFit);

		ToolbarBuilder.AddWidget(
			SNew(SViewportToolBarIconMenu)
			.Cursor(EMouseCursor::Default)
			.Style(ToolBarStyle)
			.OnGetMenuContent(this, &SDisplayClusterConfiguratorOutputMappingToolbar::MakeViewScaleMenu)
			.ToolTipText(LOCTEXT("ViewOptionsMenu_ToolTip", "View Scale"))
			.Icon(FSlateIcon(FDisplayClusterConfiguratorStyle::Get().GetStyleSetName(), "DisplayClusterConfigurator.OutputMapping.ViewScale"))
			.Label(this, &SDisplayClusterConfiguratorOutputMappingToolbar::GetViewScaleText)
			.ParentToolBar(SharedThis(this))
		);
	}
	ToolbarBuilder.EndSection();

	ToolbarBuilder.BeginSection("Transform");
	{
		ToolbarBuilder.AddWidget(
			SNew(SViewportToolBarIconMenu)
			.Cursor(EMouseCursor::Default)
			.Style(ToolBarStyle)
			.OnGetMenuContent(this, &SDisplayClusterConfiguratorOutputMappingToolbar::MakeTransformMenu)
			.ToolTipText(LOCTEXT("TransformMenu_ToolTip", "Transform Operations"))
			.Icon(FSlateIcon(FDisplayClusterConfiguratorStyle::Get().GetStyleSetName(), "DisplayClusterConfigurator.OutputMapping.Transform"))
			.Label(LOCTEXT("TransformSettings_Label", "Transform"))
			.ParentToolBar(SharedThis(this))
		);
	}
	ToolbarBuilder.EndSection();

	ToolbarBuilder.BeginSection("Snapping");
	{
		ToolbarBuilder.AddWidget(
			SNew(SViewportToolBarIconMenu)
			.Cursor(EMouseCursor::Default)
			.Style(ToolBarStyle)
			.OnGetMenuContent(this, &SDisplayClusterConfiguratorOutputMappingToolbar::MakeSnappingMenu)
			.ToolTipText(LOCTEXT("SnappingMenu_ToolTip", "Node Overlap and Snapping"))
			.Icon(FSlateIcon(FDisplayClusterConfiguratorStyle::Get().GetStyleSetName(), "DisplayClusterConfigurator.OutputMapping.Snapping"))
			.Label(LOCTEXT("AlignmentSettings_Label", "Overlap & Snapping"))
			.ParentToolBar(SharedThis(this))
		);
	}
	ToolbarBuilder.EndSection();

	return ToolbarBuilder.MakeWidget();
}

TSharedRef<SWidget> SDisplayClusterConfiguratorOutputMappingToolbar::MakeWindowDisplayMenu()
{
	const bool bShouldCloseWindowAfterMenuSelection = false;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, CommandList);

	MenuBuilder.BeginSection(TEXT("WindowDisplaySection"));
	{
		MenuBuilder.AddMenuEntry(FDisplayClusterConfiguratorOutputMappingCommands::Get().ShowWindowInfo);
		MenuBuilder.AddMenuEntry(FDisplayClusterConfiguratorOutputMappingCommands::Get().ShowWindowCorner);
		MenuBuilder.AddMenuEntry(FDisplayClusterConfiguratorOutputMappingCommands::Get().ShowWindowNone);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

FText SDisplayClusterConfiguratorOutputMappingToolbar::GetWindowDisplayText() const
{
	TSharedPtr<FDisplayClusterConfiguratorViewOutputMapping> ViewOutputMapping = ViewOutputMappingPtr.Pin();
	check(ViewOutputMapping != nullptr);

	FOutputMappingSettings& OutputMappingSettings = ViewOutputMapping->GetOutputMappingSettings();

	if (OutputMappingSettings.bShowWindowInfo)
	{
		return LOCTEXT("WindowDisplay_Info", "Info Bar");
	}
	else if (OutputMappingSettings.bShowWindowCornerImage)
	{
		return LOCTEXT("WindowDisplay_Corner", "Corner");
	}
	else
	{
		return LOCTEXT("WindowDisplay_None", "None");
	}
}

TSharedRef<SWidget> SDisplayClusterConfiguratorOutputMappingToolbar::MakeTransformMenu()
{
	const bool bShouldCloseWindowAfterMenuSelection = false;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, CommandList);

	MenuBuilder.BeginSection(TEXT("CommonTransformSection"));
	{
		MenuBuilder.AddMenuEntry(FDisplayClusterConfiguratorOutputMappingCommands::Get().RotateViewport90CCW);
		MenuBuilder.AddMenuEntry(FDisplayClusterConfiguratorOutputMappingCommands::Get().RotateViewport90CW);
		MenuBuilder.AddMenuEntry(FDisplayClusterConfiguratorOutputMappingCommands::Get().RotateViewport180);
		MenuBuilder.AddSeparator();
		MenuBuilder.AddMenuEntry(FDisplayClusterConfiguratorOutputMappingCommands::Get().FlipViewportHorizontal);
		MenuBuilder.AddMenuEntry(FDisplayClusterConfiguratorOutputMappingCommands::Get().FlipViewportVertical);
		MenuBuilder.AddSeparator();
		MenuBuilder.AddMenuEntry(FDisplayClusterConfiguratorOutputMappingCommands::Get().ResetViewportTransform);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> SDisplayClusterConfiguratorOutputMappingToolbar::MakeSnappingMenu()
{
	const bool bShouldCloseWindowAfterMenuSelection = false;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, CommandList);

	MenuBuilder.BeginSection(TEXT("OverlapBoundsSection"), LOCTEXT("OverlapBoundsSectionLabel", "Overlap & Bounds"));
	{
		MenuBuilder.AddMenuEntry(FDisplayClusterConfiguratorOutputMappingCommands::Get().ToggleClusterItemOverlap);
		MenuBuilder.AddMenuEntry(FDisplayClusterConfiguratorOutputMappingCommands::Get().ToggleLockClusterNodesInHosts);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection(TEXT("SnappingSection"), LOCTEXT("SnappingSectionLabel", "Snapping (To Activate, hold SHIFT)"));
	{
		MenuBuilder.AddMenuEntry(FDisplayClusterConfiguratorOutputMappingCommands::Get().ToggleAdjacentEdgeSnapping);

		MenuBuilder.AddWidget(
			SNew(SBox)
			.HAlign(HAlign_Right)
			[
				SNew(SBox)
				.Padding(FMargin(8.0f, 0.0f, 0.0f, 0.0f))
				.WidthOverride(100.0f)
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::Get().GetBrush("Menu.WidgetBorder"))
					.Padding(FMargin(1.0f))
					[
						SNew(SNumericEntryBox<int>)
						.Value(this, &SDisplayClusterConfiguratorOutputMappingToolbar::GetAdjacentEdgePadding)
						.OnValueChanged(this, &SDisplayClusterConfiguratorOutputMappingToolbar::SetAdjacentEdgePadding)
						.IsEnabled(this, &SDisplayClusterConfiguratorOutputMappingToolbar::IsAdjacentEdgeSnappingEnabled)
						.MinValue(0)
						.MaxValue(INT_MAX)
						.MaxSliderValue(100)
						.AllowSpin(true)
					]
				]
			],
			LOCTEXT("AlignmentSettings_AdjacentEdgePadding", "Adjacent Edge Padding")
		);
	}

	{
		MenuBuilder.AddMenuEntry(FDisplayClusterConfiguratorOutputMappingCommands::Get().ToggleSameEdgeSnapping);
	}

	{
		MenuBuilder.AddWidget(
			SNew(SBox)
			.HAlign(HAlign_Right)
			[
				SNew(SBox)
				.Padding(FMargin(8.0f, 0.0f, 0.0f, 0.0f))
				.WidthOverride(100.0f)
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::Get().GetBrush("Menu.WidgetBorder"))
					.Padding(FMargin(1.0f))
					[
						SNew(SNumericEntryBox<int>)
						.Value(this, &SDisplayClusterConfiguratorOutputMappingToolbar::GetSnapProximity)
						.OnValueChanged(this, &SDisplayClusterConfiguratorOutputMappingToolbar::SetSnapProximity)
						.IsEnabled(this, &SDisplayClusterConfiguratorOutputMappingToolbar::IsSnappingEnabled)
						.MinValue(0)
						.MaxValue(INT_MAX)
						.MaxSliderValue(100)
						.AllowSpin(true)
					]
				]
			],
			LOCTEXT("AlignmentSettings_SnapProximity", "Snap Proximity")
		);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

bool SDisplayClusterConfiguratorOutputMappingToolbar::IsSnappingEnabled() const
{
	TSharedPtr<FDisplayClusterConfiguratorViewOutputMapping> ViewOutputMapping = ViewOutputMappingPtr.Pin();
	check(ViewOutputMapping != nullptr);

	return ViewOutputMapping->GetNodeAlignmentSettings().bSnapAdjacentEdges || ViewOutputMapping->GetNodeAlignmentSettings().bSnapSameEdges;
}

bool SDisplayClusterConfiguratorOutputMappingToolbar::IsAdjacentEdgeSnappingEnabled() const
{
	TSharedPtr<FDisplayClusterConfiguratorViewOutputMapping> ViewOutputMapping = ViewOutputMappingPtr.Pin();
	check(ViewOutputMapping != nullptr);

	return ViewOutputMapping->GetNodeAlignmentSettings().bSnapAdjacentEdges;
}

TOptional<int> SDisplayClusterConfiguratorOutputMappingToolbar::GetAdjacentEdgePadding() const
{
	TSharedPtr<FDisplayClusterConfiguratorViewOutputMapping> ViewOutputMapping = ViewOutputMappingPtr.Pin();
	check(ViewOutputMapping != nullptr);

	return ViewOutputMapping->GetNodeAlignmentSettings().AdjacentEdgesSnapPadding;
}

void SDisplayClusterConfiguratorOutputMappingToolbar::SetAdjacentEdgePadding(int NewPadding)
{
	TSharedPtr<FDisplayClusterConfiguratorViewOutputMapping> ViewOutputMapping = ViewOutputMappingPtr.Pin();
	check(ViewOutputMapping != nullptr);

	ViewOutputMapping->GetNodeAlignmentSettings().AdjacentEdgesSnapPadding = NewPadding;
}

TOptional<int> SDisplayClusterConfiguratorOutputMappingToolbar::GetSnapProximity() const
{
	TSharedPtr<FDisplayClusterConfiguratorViewOutputMapping> ViewOutputMapping = ViewOutputMappingPtr.Pin();
	check(ViewOutputMapping != nullptr);

	return ViewOutputMapping->GetNodeAlignmentSettings().SnapProximity;
}

void SDisplayClusterConfiguratorOutputMappingToolbar::SetSnapProximity(int NewSnapProximity)
{
	TSharedPtr<FDisplayClusterConfiguratorViewOutputMapping> ViewOutputMapping = ViewOutputMappingPtr.Pin();
	check(ViewOutputMapping != nullptr);

	ViewOutputMapping->GetNodeAlignmentSettings().SnapProximity = NewSnapProximity;
}

TSharedRef<SWidget> SDisplayClusterConfiguratorOutputMappingToolbar::MakeAdvancedMenu()
{
	const bool bShouldCloseWindowAfterMenuSelection = false;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, CommandList);

	MenuBuilder.BeginSection(TEXT("General"), LOCTEXT("GeneralSectionLabel", "General"));
	{
		MenuBuilder.AddMenuEntry(FDisplayClusterConfiguratorOutputMappingCommands::Get().ToggleTintViewports);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection(TEXT("Hosts"), LOCTEXT("HostsSectionLabel", "Hosts"));
	{
		MenuBuilder.AddSubMenu(
			LOCTEXT("HostArrangementSubMenuLabel", "Host Arrangement"),
			LOCTEXT("HostArrangementSubMenuToolTip", "Indicates how hosts are arranged on the graph editor"),
			FNewMenuDelegate::CreateSP(this, &SDisplayClusterConfiguratorOutputMappingToolbar::MakeHostArrangementTypeSubMenu)
		);

		
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SDisplayClusterConfiguratorOutputMappingToolbar::MakeHostArrangementTypeSubMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection(TEXT("HostArrangementType"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("HostArrangementHorizontalLabel", "Horizontal"),
			LOCTEXT("HostArrangementHorizontalToolTip", "Arranges hosts horizontally on the graph editor"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SDisplayClusterConfiguratorOutputMappingToolbar::SetHostArrangementType, EHostArrangementType::Horizontal),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SDisplayClusterConfiguratorOutputMappingToolbar::IsHostArrangementTypeChecked, EHostArrangementType::Horizontal)
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("HostArrangementVerticalLabel", "Vertical"),
			LOCTEXT("HostArrangementVerticalToolTip", "Arranges hosts vertically on the graph editor"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SDisplayClusterConfiguratorOutputMappingToolbar::SetHostArrangementType, EHostArrangementType::Vertical),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SDisplayClusterConfiguratorOutputMappingToolbar::IsHostArrangementTypeChecked, EHostArrangementType::Vertical)
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("HostArrangementWrapLabel", "Wrap"),
			LOCTEXT("HostArrangementWrapToolTip", "Arranges hosts horizontally on the graph editor until the threshold is reached, then wraps them"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SDisplayClusterConfiguratorOutputMappingToolbar::SetHostArrangementType, EHostArrangementType::Wrap),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SDisplayClusterConfiguratorOutputMappingToolbar::IsHostArrangementTypeChecked, EHostArrangementType::Wrap)
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("HostArrangementGridLabel", "Grid"),
			LOCTEXT("HostArrangementGridToolTip", "Arranges hosts in a grid on the graph editor"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SDisplayClusterConfiguratorOutputMappingToolbar::SetHostArrangementType, EHostArrangementType::Grid),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SDisplayClusterConfiguratorOutputMappingToolbar::IsHostArrangementTypeChecked, EHostArrangementType::Grid)
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection(TEXT("HostArrangementSettings"));
	{
		MenuBuilder.AddWidget(
			SNew(SBox)
			.HAlign(HAlign_Right)
			[
				SNew(SBox)
				.Padding(FMargin(8.0f, 0.0f, 0.0f, 0.0f))
				.WidthOverride(100.0f)
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::Get().GetBrush("Menu.WidgetBorder"))
					.Padding(FMargin(1.0f))
					[
						SNew(SNumericEntryBox<int>)
						.Value(this, &SDisplayClusterConfiguratorOutputMappingToolbar::GetHostWrapThreshold)
						.OnValueChanged(this, &SDisplayClusterConfiguratorOutputMappingToolbar::SetHostWrapThreshold)
						.IsEnabled(this, &SDisplayClusterConfiguratorOutputMappingToolbar::IsHostArrangementTypeChecked, EHostArrangementType::Wrap)
						.MinValue(0)
						.MaxValue(INT_MAX)
						.MaxSliderValue(10000)
						.AllowSpin(true)
					]
				]
			],
			LOCTEXT("HostArrangementSettings_WrapThreshold", "Wrapping Threshold")
		);

		MenuBuilder.AddWidget(
			SNew(SBox)
			.HAlign(HAlign_Right)
			[
				SNew(SBox)
				.Padding(FMargin(8.0f, 0.0f, 0.0f, 0.0f))
				.WidthOverride(100.0f)
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::Get().GetBrush("Menu.WidgetBorder"))
					.Padding(FMargin(1.0f))
					[
						SNew(SNumericEntryBox<int>)
						.Value(this, &SDisplayClusterConfiguratorOutputMappingToolbar::GetHostGridSize)
						.OnValueChanged(this, &SDisplayClusterConfiguratorOutputMappingToolbar::SetHostGridSize)
						.IsEnabled(this, &SDisplayClusterConfiguratorOutputMappingToolbar::IsHostArrangementTypeChecked, EHostArrangementType::Grid)
						.MinValue(1)
						.MaxValue(10)
						.MinSliderValue(1)
						.MaxSliderValue(5)
						.AllowSpin(true)
					]
				]
			],
			LOCTEXT("HostArrangementSettings_GridSize", "Grid Size")
		);
	}
	MenuBuilder.EndSection();
}

bool SDisplayClusterConfiguratorOutputMappingToolbar::IsHostArrangementTypeChecked(EHostArrangementType ArrangementType) const
{
	TSharedPtr<FDisplayClusterConfiguratorViewOutputMapping> ViewOutputMapping = ViewOutputMappingPtr.Pin();
	check(ViewOutputMapping != nullptr);

	return ViewOutputMapping->GetHostArrangementSettings().ArrangementType == ArrangementType;
}

void SDisplayClusterConfiguratorOutputMappingToolbar::SetHostArrangementType(EHostArrangementType ArrangementType)
{
	TSharedPtr<FDisplayClusterConfiguratorViewOutputMapping> ViewOutputMapping = ViewOutputMappingPtr.Pin();
	check(ViewOutputMapping != nullptr);

	ViewOutputMapping->GetHostArrangementSettings().ArrangementType = ArrangementType;
}

TOptional<int> SDisplayClusterConfiguratorOutputMappingToolbar::GetHostWrapThreshold() const
{
	TSharedPtr<FDisplayClusterConfiguratorViewOutputMapping> ViewOutputMapping = ViewOutputMappingPtr.Pin();
	check(ViewOutputMapping != nullptr);

	return ViewOutputMapping->GetHostArrangementSettings().WrapThreshold;
}

void SDisplayClusterConfiguratorOutputMappingToolbar::SetHostWrapThreshold(int NewWrapThreshold)
{
	TSharedPtr<FDisplayClusterConfiguratorViewOutputMapping> ViewOutputMapping = ViewOutputMappingPtr.Pin();
	check(ViewOutputMapping != nullptr);

	ViewOutputMapping->GetHostArrangementSettings().WrapThreshold = NewWrapThreshold;
}

TOptional<int> SDisplayClusterConfiguratorOutputMappingToolbar::GetHostGridSize() const
{
	TSharedPtr<FDisplayClusterConfiguratorViewOutputMapping> ViewOutputMapping = ViewOutputMappingPtr.Pin();
	check(ViewOutputMapping != nullptr);

	return ViewOutputMapping->GetHostArrangementSettings().GridSize;
}

void SDisplayClusterConfiguratorOutputMappingToolbar::SetHostGridSize(int NewGridSize)
{
	TSharedPtr<FDisplayClusterConfiguratorViewOutputMapping> ViewOutputMapping = ViewOutputMappingPtr.Pin();
	check(ViewOutputMapping != nullptr);

	ViewOutputMapping->GetHostArrangementSettings().GridSize = NewGridSize;
}

TSharedRef<SWidget> SDisplayClusterConfiguratorOutputMappingToolbar::MakeViewScaleMenu()
{
	const bool bShouldCloseWindowAfterMenuSelection = false;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, CommandList);

	for (int Index = 0; Index < ViewScales.Num(); ++Index)
	{
		const float ViewScale = ViewScales[Index];

		FUIAction UIAction(
			FExecuteAction::CreateSP(this, &SDisplayClusterConfiguratorOutputMappingToolbar::SetViewScale, Index),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &SDisplayClusterConfiguratorOutputMappingToolbar::IsViewScaleChecked, Index)
		);

		MenuBuilder.AddMenuEntry(
			FText::AsNumber(ViewScale),
			FText::Format(LOCTEXT("PixelRatio_Tooltip", "Sets the visual scale to {0}"), FText::AsNumber(ViewScale)),
			FSlateIcon(),
			UIAction,
			NAME_None,
			EUserInterfaceActionType::RadioButton 
		);
	}

	return MenuBuilder.MakeWidget();
}

bool SDisplayClusterConfiguratorOutputMappingToolbar::IsViewScaleChecked(int32 Index) const
{
	TSharedPtr<FDisplayClusterConfiguratorViewOutputMapping> ViewOutputMapping = ViewOutputMappingPtr.Pin();
	check(ViewOutputMapping != nullptr);

	const float CurrentViewScale = ViewOutputMapping->GetOutputMappingSettings().ViewScale;
	
	return FMath::IsNearlyEqual(CurrentViewScale, ViewScales[Index]);
}

void SDisplayClusterConfiguratorOutputMappingToolbar::SetViewScale(int32 Index)
{
	TSharedPtr<FDisplayClusterConfiguratorViewOutputMapping> ViewOutputMapping = ViewOutputMappingPtr.Pin();
	check(ViewOutputMapping != nullptr);

	ViewOutputMapping->GetOutputMappingSettings().ViewScale = ViewScales[Index];

	ViewOutputMapping->RefreshNodePositions();
}

FText SDisplayClusterConfiguratorOutputMappingToolbar::GetViewScaleText() const
{
	TSharedPtr<FDisplayClusterConfiguratorViewOutputMapping> ViewOutputMapping = ViewOutputMappingPtr.Pin();
	check(ViewOutputMapping != nullptr);

	const float ViewScale = ViewOutputMapping->GetOutputMappingSettings().ViewScale;
	return FText::AsNumber(ViewScale);
}

#undef LOCTEXT_NAMESPACE
