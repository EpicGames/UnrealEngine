// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "SViewportToolBar.h"

#include "Views/OutputMapping/DisplayClusterConfiguratorViewOutputMapping.h"

class FExtender;
class FUICommandList;
class FDisplayClusterConfiguratorViewOutputMapping;
class FMenuBuilder;

class SDisplayClusterConfiguratorOutputMappingToolbar
	: public SViewportToolBar
{
public:
	SLATE_BEGIN_ARGS(SDisplayClusterConfiguratorOutputMappingToolbar) {}
		SLATE_ARGUMENT(TSharedPtr<FUICommandList>, CommandList)
		SLATE_ARGUMENT(TSharedPtr<FExtender>, Extenders)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TWeakPtr<FDisplayClusterConfiguratorViewOutputMapping>& InViewOutputMapping);

	/**
	 * Static: Creates a widget for the main tool bar
	 *
	 * @return	New widget
	 */
	TSharedRef< SWidget > MakeToolBar(const TSharedPtr< FExtender > InExtenders);

private:
	TSharedRef<SWidget> MakeWindowDisplayMenu();
	FText GetWindowDisplayText() const;

	TSharedRef<SWidget> MakeTransformMenu();

	TSharedRef<SWidget> MakeSnappingMenu();

	bool IsSnappingEnabled() const;

	bool IsAdjacentEdgeSnappingEnabled() const;
	TOptional<int> GetAdjacentEdgePadding() const;
	void SetAdjacentEdgePadding(int NewPadding);

	TOptional<int> GetSnapProximity() const;
	void SetSnapProximity(int NewSnapProximity);

	TSharedRef<SWidget> MakeAdvancedMenu();

	void MakeHostArrangementTypeSubMenu(FMenuBuilder& MenuBuilder);
	bool IsHostArrangementTypeChecked(EHostArrangementType ArrangementType) const;
	void SetHostArrangementType(EHostArrangementType ArrangementType);

	TOptional<int> GetHostWrapThreshold() const;
	void SetHostWrapThreshold(int NewWrapThreshold);

	TOptional<int> GetHostGridSize() const;
	void SetHostGridSize(int NewGridSize);

	TSharedRef<SWidget> MakeViewScaleMenu();

	bool IsViewScaleChecked(int32 Index) const;
	void SetViewScale(int32 Index);
	FText GetViewScaleText() const;

private:
	/** Command list */
	TSharedPtr<FUICommandList> CommandList;
	TWeakPtr<FDisplayClusterConfiguratorViewOutputMapping> ViewOutputMappingPtr;

private:
	static const TArray<float> ViewScales;
};
