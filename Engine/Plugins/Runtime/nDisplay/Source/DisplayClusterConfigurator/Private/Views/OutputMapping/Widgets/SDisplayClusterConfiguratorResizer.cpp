// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/OutputMapping/Widgets/SDisplayClusterConfiguratorResizer.h"

#include "DisplayClusterConfiguratorStyle.h"
#include "DisplayClusterConfiguratorBlueprintEditor.h"
#include "Views/OutputMapping/GraphNodes/SDisplayClusterConfiguratorBaseNode.h"

#include "Framework/Application/SlateApplication.h"
#include "Widgets/Images/SImage.h"
#include "SGraphPanel.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "SDisplayClusterConfiguratorResizer"

void SDisplayClusterConfiguratorResizer::Construct(const FArguments& InArgs, const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& InToolkit, const TSharedRef<SDisplayClusterConfiguratorBaseNode>& InBaseNode)
{
	ToolkitPtr = InToolkit;
	BaseNodePtr = InBaseNode;
	CurrentAspectRatio = 1;
	bResizing = false;

	MinimumSize = InArgs._MinimumSize;
	MaximumSize = InArgs._MaximumSize;
	IsFixedAspectRatio = InArgs._IsFixedAspectRatio;

	ScopedTransaction.Reset();

	SetCursor(EMouseCursor::GrabHandClosed);

	ChildSlot
		[
			SNew(SImage)
			.Image(FDisplayClusterConfiguratorStyle::Get().GetBrush("DisplayClusterConfigurator.OutputMapping.ResizeAreaHandle"))
		];
}

FReply SDisplayClusterConfiguratorResizer::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		bResizing = true;

		// Store the current aspect ratio here so that it isn't suspectible to drift due to float->int conversion while dragging.
		TSharedPtr<SDisplayClusterConfiguratorBaseNode> BaseNode = BaseNodePtr.Pin();
		check(BaseNode.IsValid());

		BaseNode->BeginUserInteraction();

		const FVector2D CurrentNodeSize = BaseNode->GetSize();
		if (CurrentNodeSize.Y != 0)
		{
			CurrentAspectRatio = CurrentNodeSize.X / CurrentNodeSize.Y;
		}
		else
		{
			CurrentAspectRatio = 1;
		}

		return FReply::Handled().CaptureMouse(SharedThis(this));
	}

	return FReply::Unhandled();
}

FReply SDisplayClusterConfiguratorResizer::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		bResizing = false;

		TSharedPtr<SDisplayClusterConfiguratorBaseNode> BaseNode = BaseNodePtr.Pin();
		check(BaseNode.IsValid());

		BaseNode->EndUserInteraction();

		ScopedTransaction.Reset();
		return FReply::Handled().ReleaseMouseCapture();
	}

	return FReply::Unhandled();
}

FReply SDisplayClusterConfiguratorResizer::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (bResizing)
	{
		TSharedPtr<SDisplayClusterConfiguratorBaseNode> BaseNode = BaseNodePtr.Pin();
		check(BaseNode.IsValid());

		TSharedPtr<SGraphPanel> GraphPanel = BaseNode->GetOwnerPanel();
		check(GraphPanel.IsValid());

		const float DPIScale = GetDPIScale();

		FVector2D NewNodeSize = MouseEvent.GetScreenSpacePosition() - BaseNode->GetTickSpaceGeometry().GetAbsolutePosition();

		bool bIsFixedAspectRatio = IsAspectRatioFixed();
		if (bIsFixedAspectRatio)
		{
			// If the aspect ratio is fixed, first get the node's current size to compute the ratio from,
			// then force the new node size to match that aspect ratio.
			const FVector2D CurrentNodeSize = BaseNode->GetSize();

			if (NewNodeSize.X > NewNodeSize.Y * CurrentAspectRatio)
			{
				NewNodeSize.X = NewNodeSize.Y * CurrentAspectRatio;
			}
			else
			{
				NewNodeSize.Y = NewNodeSize.X / CurrentAspectRatio;
			}
		}

		NewNodeSize /= (GraphPanel->GetZoomAmount() * DPIScale);

		// Bound the node's size between the minimum and maximum. Also never let the size become negative.
		float Minimum = FMath::Max(MinimumSize.Get(0.0f), 0.0f);
		float Maximum = FMath::Max(MaximumSize.Get(FLT_MAX), Minimum);
		
		NewNodeSize.X = FMath::Clamp(NewNodeSize.X, Minimum, Maximum);
		NewNodeSize.Y = FMath::Clamp(NewNodeSize.Y, Minimum, Maximum);

		// If we don't have a scoped transaction for the resize, create a new one.
		if (!ScopedTransaction.IsValid())
		{
			ScopedTransaction = MakeShareable(new FScopedTransaction(LOCTEXT("ResizeNodeAction", "Resize Node")));
		}

		BaseNode->SetNodeSize(NewNodeSize, bIsFixedAspectRatio);

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

float SDisplayClusterConfiguratorResizer::GetDPIScale() const
{
	float DPIScale = 1.0f;
	TSharedPtr<SWindow> WidgetWindow = FSlateApplication::Get().FindWidgetWindow(SharedThis(this));
	if (WidgetWindow.IsValid())
	{
		DPIScale = WidgetWindow->GetNativeWindow()->GetDPIScaleFactor();
	}

	return DPIScale;
}

bool SDisplayClusterConfiguratorResizer::IsAspectRatioFixed() const
{
	bool bIsShiftPressed = FSlateApplication::Get().GetModifierKeys().IsShiftDown();
	bool bIsFixedAspectRatio = IsFixedAspectRatio.Get(false);

	return bIsFixedAspectRatio || bIsShiftPressed;
}

#undef LOCTEXT_NAMESPACE
