// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GenericPlatform/ICursor.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FDisplayClusterConfiguratorBlueprintEditor;
class SDisplayClusterConfiguratorBaseNode;
class FScopedTransaction;

class SDisplayClusterConfiguratorResizer
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SDisplayClusterConfiguratorResizer)
		: _MinimumSize(0)
		, _MaximumSize(FLT_MAX)
		, _IsFixedAspectRatio(false)
	{}
		SLATE_ATTRIBUTE(float, MinimumSize)
		SLATE_ATTRIBUTE(float, MaximumSize)
		SLATE_ATTRIBUTE(bool, IsFixedAspectRatio)
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs, const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& InToolkit, const TSharedRef<SDisplayClusterConfiguratorBaseNode>& InBaseNode);

	//~ Begin SWidget Interface
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	//~ End SWidget Interface

private:
	float GetDPIScale() const;
	bool IsAspectRatioFixed() const;

private:
	TWeakPtr<FDisplayClusterConfiguratorBlueprintEditor> ToolkitPtr;
	TWeakPtr<SDisplayClusterConfiguratorBaseNode> BaseNodePtr;
	TSharedPtr<FScopedTransaction> ScopedTransaction;

	bool bResizing;
	float CurrentAspectRatio;

	TAttribute<float> MinimumSize;
	TAttribute<float> MaximumSize;
	TAttribute<bool> IsFixedAspectRatio;
};

