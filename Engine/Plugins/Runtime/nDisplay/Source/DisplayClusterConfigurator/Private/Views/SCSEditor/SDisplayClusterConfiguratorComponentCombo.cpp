// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDisplayClusterConfiguratorComponentCombo.h"

#include "Components/DisplayClusterCameraComponent.h"
#include "Components/DisplayClusterICVFXCameraComponent.h"
#include "Components/DisplayClusterScreenComponent.h"
#include "Components/DisplayClusterXformComponent.h"
#include "Render/DisplayDevice/Components/DisplayClusterDisplayDeviceBaseComponent.h"
#include "Components/DisplayClusterInFrustumFitCameraComponent.h"

#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SSeparator.h"

#include "Styling/AppStyle.h"
#include "Components/SceneComponent.h"
#include "Engine/Blueprint.h"
#include "Engine/Selection.h"
#include "Editor.h"
#include "Styling/SlateIconFinder.h"
#include "ComponentAssetBroker.h"
#include "ComponentTypeRegistry.h"
#include "EditorClassUtils.h"

#include "SListViewSelectorDropdownMenu.h"
#include "Misc/TextFilterExpressionEvaluator.h"
#include "SPositiveActionButton.h"

#define LOCTEXT_NAMESPACE "DisplayClusterConfiguratorComponentClassCombo"

void SDisplayClusterConfiguratorComponentClassCombo::Construct(const FArguments& InArgs)
{
	PrevSelectedIndex = INDEX_NONE;
	OnSubobjectClassSelected = InArgs._OnSubobjectClassSelected;
	TextFilter = MakeShared<FTextFilterExpressionEvaluator>(ETextFilterExpressionEvaluatorMode::BasicString);

	FComponentTypeRegistry::Get().SubscribeToComponentList(ComponentClassListPtr).AddRaw(this, &SDisplayClusterConfiguratorComponentClassCombo::UpdateComponentClassList);

	UpdateComponentClassList();

	SAssignNew(ComponentClassListView, SListView<FComponentClassComboEntryPtr>)
		.ListItemsSource(&FilteredComponentClassList)
		.OnSelectionChanged( this, &SDisplayClusterConfiguratorComponentClassCombo::OnAddComponentSelectionChanged )
		.OnGenerateRow( this, &SDisplayClusterConfiguratorComponentClassCombo::GenerateAddComponentRow )
		.SelectionMode(ESelectionMode::Single);

	SAssignNew(SearchBox, SSearchBox)
		.HintText( LOCTEXT( "BlueprintAddComponentSearchBoxHint", "Search Components" ) )
		.OnTextChanged( this, &SDisplayClusterConfiguratorComponentClassCombo::OnSearchBoxTextChanged )
		.OnTextCommitted( this, &SDisplayClusterConfiguratorComponentClassCombo::OnSearchBoxTextCommitted );

	ChildSlot
	[
		SAssignNew(AddNewButton, SPositiveActionButton)
		.Icon(FAppStyle::Get().GetBrush("Icons.Plus"))
		.Text(LOCTEXT("AddComponent", "Add"))
		.OnComboBoxOpened(this, &SDisplayClusterConfiguratorComponentClassCombo::ClearSelection)
		.MenuContent()
		[
			SNew(SListViewSelectorDropdownMenu<FComponentClassComboEntryPtr>, SearchBox, ComponentClassListView)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("Menu.Background"))
				.Padding(2)
				[
					SNew(SBox)
					.WidthOverride(250)
					[				
						SNew(SVerticalBox)
						+SVerticalBox::Slot()
						.Padding(1.f)
						.AutoHeight()
						[
							SearchBox.ToSharedRef()
						]
						+SVerticalBox::Slot()
						.MaxHeight(400)
						[
							ComponentClassListView.ToSharedRef()
						]
					]
				]
			]
		]
	];

	

	ComponentClassListView->EnableToolTipForceField( true );
	// The button can automatically handle setting focus to a specified control when the combo button is opened
	AddNewButton->SetMenuContentWidgetToFocus( SearchBox );
}

SDisplayClusterConfiguratorComponentClassCombo::~SDisplayClusterConfiguratorComponentClassCombo()
{
	FComponentTypeRegistry::Get().GetOnComponentTypeListChanged().RemoveAll(this);
	FilteredComponentClassList.Empty();
	ComponentClassList.Empty();
	ComponentClassListPtr = nullptr;
}

void SDisplayClusterConfiguratorComponentClassCombo::ClearSelection()
{
	SearchBox->SetText(FText::GetEmpty());

	PrevSelectedIndex = INDEX_NONE;
	
	// Clear the selection in such a way as to also clear the keyboard selector
	ComponentClassListView->SetSelection(nullptr, ESelectInfo::OnNavigation);

	// Make sure we scroll to the top
	if (ComponentClassList.Num() > 0)
	{
		ComponentClassListView->RequestScrollIntoView((ComponentClassList)[0]);
	}
}

void SDisplayClusterConfiguratorComponentClassCombo::GenerateFilteredComponentList()
{
	if ( TextFilter->GetFilterText().IsEmpty() )
	{
		FilteredComponentClassList = ComponentClassList;
	}
	else
	{
		FilteredComponentClassList.Empty();

		int32 LastHeadingIndex = INDEX_NONE;
		FComponentClassComboEntryPtr* LastHeadingPtr = nullptr;

		for (int32 ComponentIndex = 0; ComponentIndex < ComponentClassList.Num(); ComponentIndex++)
		{
			FComponentClassComboEntryPtr& CurrentEntry = (ComponentClassList)[ComponentIndex];

			if (CurrentEntry->IsHeading())
			{
				LastHeadingIndex = FilteredComponentClassList.Num();
				LastHeadingPtr = &CurrentEntry;
			}
			else if (CurrentEntry->IsClass() && CurrentEntry->IsIncludedInFilter())
			{
				FString FriendlyComponentName = SComponentClassCombo::GetSanitizedComponentName( CurrentEntry );

				if ( TextFilter->TestTextFilter(FBasicStringFilterExpressionContext(FriendlyComponentName)) )
				{
					// Add the heading first if it hasn't already been added
					if (LastHeadingPtr && LastHeadingIndex != INDEX_NONE)
					{
						FilteredComponentClassList.Insert(*LastHeadingPtr, LastHeadingIndex);
						LastHeadingIndex = INDEX_NONE;
						LastHeadingPtr = nullptr;
					}

					// Add the class
					FilteredComponentClassList.Add( CurrentEntry );
				}
			}
		}

		// Select the first non-category item that passed the filter
		for (FComponentClassComboEntryPtr& TestEntry : FilteredComponentClassList)
		{
			if (TestEntry->IsClass())
			{
				ComponentClassListView->SetSelection(TestEntry, ESelectInfo::OnNavigation);
				break;
			}
		}
	}
}

FText SDisplayClusterConfiguratorComponentClassCombo::GetCurrentSearchString() const
{
	return TextFilter->GetFilterText();
}

void SDisplayClusterConfiguratorComponentClassCombo::OnSearchBoxTextChanged( const FText& InSearchText )
{
	TextFilter->SetFilterText(InSearchText);
	SearchBox->SetError(TextFilter->GetFilterErrorText());

	// Generate a filtered list
	GenerateFilteredComponentList();

	// Ask the combo to update its contents on next tick
	ComponentClassListView->RequestListRefresh();
}

void SDisplayClusterConfiguratorComponentClassCombo::OnSearchBoxTextCommitted(const FText& NewText, ETextCommit::Type CommitInfo)
{
	if(CommitInfo == ETextCommit::OnEnter)
	{
		auto SelectedItems = ComponentClassListView->GetSelectedItems();
		if(SelectedItems.Num() > 0)
		{
			ComponentClassListView->SetSelection(SelectedItems[0]);
		}
	}
}

// @todo: use FKismetEditorUtilities if the main SCS editor combo also moves their version of this function.
static UClass* GetAuthoritativeBlueprintClass(UBlueprint const* const Blueprint)
{
	UClass* BpClass = (Blueprint->SkeletonGeneratedClass != nullptr) ? Blueprint->SkeletonGeneratedClass :
		Blueprint->GeneratedClass;

	if (BpClass == nullptr)
	{
		BpClass = Blueprint->ParentClass;
	}

	UClass* AuthoritativeClass = BpClass;
	if (BpClass != nullptr)
	{
		AuthoritativeClass = BpClass->GetAuthoritativeClass();
	}
	return AuthoritativeClass;
}

void SDisplayClusterConfiguratorComponentClassCombo::OnAddComponentSelectionChanged( FComponentClassComboEntryPtr InItem, ESelectInfo::Type SelectInfo )
{
	if ( InItem.IsValid() && InItem->IsClass() && SelectInfo != ESelectInfo::OnNavigation)
	{
		// We don't want the item to remain selected
		ClearSelection();

		if ( InItem->IsClass() )
		{
			// Neither do we want the combo dropdown staying open once the user has clicked on a valid option
			AddNewButton->SetIsMenuOpen(false, false);

			if( OnSubobjectClassSelected.IsBound() )
			{
				UClass* ComponentClass = InItem->GetComponentClass();
				if (ComponentClass == nullptr)
				{
					// The class is not loaded yet, so load it:
					if (UObject* LoadedObject = LoadObject<UObject>(nullptr, *InItem->GetComponentPath()))
					{
						if (UClass* LoadedClass = Cast<UClass>(LoadedObject))
						{
							ComponentClass = LoadedClass;
						}
						else if (const UBlueprint* LoadedBP = Cast<UBlueprint>(LoadedObject))
						{
							ComponentClass = GetAuthoritativeBlueprintClass(LoadedBP);
						}
					}
				}
				
				const FSubobjectDataHandle NewActorCompHandle =
					OnSubobjectClassSelected.IsBound() ?
					OnSubobjectClassSelected.Execute(ComponentClass, InItem->GetComponentCreateAction(), InItem->GetAssetOverride())
					: FSubobjectDataHandle::InvalidHandle;
				if(NewActorCompHandle.IsValid())
				{
					InItem->GetOnSubobjectCreated().ExecuteIfBound(NewActorCompHandle);
				}
			}
		}
	}
	else if ( InItem.IsValid() && SelectInfo != ESelectInfo::OnMouseClick )
	{
		int32 SelectedIdx = INDEX_NONE;
		if (FilteredComponentClassList.Find(InItem, /*out*/ SelectedIdx))
		{
			if (!InItem->IsClass())
			{
				int32 SelectionDirection = SelectedIdx - PrevSelectedIndex;

				// Update the previous selected index
				PrevSelectedIndex = SelectedIdx;

				// Make sure we select past the category header if we started filtering with it selected somehow (avoiding the infinite loop selecting the same item forever)
				if (SelectionDirection == 0)
				{
					SelectionDirection = 1;
				}

				if(SelectedIdx + SelectionDirection >= 0 && SelectedIdx + SelectionDirection < FilteredComponentClassList.Num())
				{
					ComponentClassListView->SetSelection(FilteredComponentClassList[SelectedIdx + SelectionDirection], ESelectInfo::OnNavigation);
				}
			}
			else
			{
				// Update the previous selected index
				PrevSelectedIndex = SelectedIdx;
			}
		}
	}
}

TSharedRef<ITableRow> SDisplayClusterConfiguratorComponentClassCombo::GenerateAddComponentRow( FComponentClassComboEntryPtr Entry, const TSharedRef<STableViewBase> &OwnerTable ) const
{
	check( Entry->IsHeading() || Entry->IsSeparator() || Entry->IsClass() );

	if ( Entry->IsHeading() )
	{
		return 
			SNew( STableRow< TSharedPtr<FString> >, OwnerTable )
				.Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.NoHoverTableRow"))
				.ShowSelection(false)
			[
				SNew(SBox)
				.Padding(1.f)
				[
					SNew(STextBlock)
					.Text(FText::FromString(Entry->GetHeadingText()))
					.TextStyle(FAppStyle::Get(), TEXT("Menu.Heading"))
				]
			];
	}
	else if ( Entry->IsSeparator() )
	{
		return 
			SNew( STableRow< TSharedPtr<FString> >, OwnerTable )
				.Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.NoHoverTableRow"))
				.ShowSelection(false)
			[
				SNew(SSeparator)
				.SeparatorImage(FAppStyle::Get().GetBrush("Menu.Separator"))
				.Thickness(1.0f)
			];
	}
	else
	{
		
		return
			SNew( SComboRow< TSharedPtr<FString> >, OwnerTable )
			.ToolTip( GetComponentToolTip(Entry) )
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SSpacer)
					.Size(FVector2D(8.0f,1.0f))
				]
				+SHorizontalBox::Slot()
				.Padding(1.0f)
				.AutoWidth()
				[
					SNew(SImage)
					.Image( FSlateIconFinder::FindIconBrushForClass( Entry->GetIconOverrideBrushName() == NAME_None ? Entry->GetIconClass() : nullptr, Entry->GetIconOverrideBrushName() ) )
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SSpacer)
					.Size(FVector2D(3.0f,1.0f))
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.HighlightText(this, &SDisplayClusterConfiguratorComponentClassCombo::GetCurrentSearchString)
					.Text(this, &SDisplayClusterConfiguratorComponentClassCombo::GetFriendlyComponentName, Entry)
				]
			];
	}
}

void SDisplayClusterConfiguratorComponentClassCombo::UpdateComponentClassList()
{
	GenerateComponentClassList();
	GenerateFilteredComponentList();
}

FText SDisplayClusterConfiguratorComponentClassCombo::GetFriendlyComponentName(FComponentClassComboEntryPtr Entry) const
{
	// Get a user friendly string from the component name
	FString FriendlyComponentName;

	if( Entry->GetComponentCreateAction() == EComponentCreateAction::CreateNewCPPClass )
	{
		FriendlyComponentName = LOCTEXT("NewCPPComponentFriendlyName", "New C++ Component...").ToString();
	}
	else if (Entry->GetComponentCreateAction() == EComponentCreateAction::CreateNewBlueprintClass )
	{
		FriendlyComponentName = LOCTEXT("NewBlueprintComponentFriendlyName", "New Blueprint Script Component...").ToString();
	}
	else
	{
		FriendlyComponentName = SComponentClassCombo::GetSanitizedComponentName(Entry);

		// Don't try to match up assets for USceneComponent it will match lots of things and doesn't have any nice behavior for asset adds 
		if (Entry->GetComponentClass() != USceneComponent::StaticClass() && Entry->GetComponentNameOverride().IsEmpty())
		{
			// Search the selected assets and look for any that can be used as a source asset for this type of component
			// If there is one we append the asset name to the component name, if there are many we append "Multiple Assets"
			FString AssetName;
			UObject* PreviousMatchingAsset = nullptr;

			FEditorDelegates::LoadSelectedAssetsIfNeeded.Broadcast();
			USelection* Selection = GEditor->GetSelectedObjects();
			for(FSelectionIterator ObjectIter(*Selection); ObjectIter; ++ObjectIter)
			{
				UObject* Object = *ObjectIter;
				check(Object);
				UClass* Class = Object->GetClass();

				TArray<TSubclassOf<UActorComponent> > ComponentClasses = FComponentAssetBrokerage::GetComponentsForAsset(Object);
				for(int32 ComponentIndex = 0; ComponentIndex < ComponentClasses.Num(); ComponentIndex++)
				{
					if(ComponentClasses[ComponentIndex]->IsChildOf(Entry->GetComponentClass()))
					{
						if(AssetName.IsEmpty())
						{
							// If there is no previous asset then we just accept the name
							AssetName = Object->GetName();
							PreviousMatchingAsset = Object;
						}
						else
						{
							// if there is a previous asset then check that we didn't just find multiple appropriate components
							// in a single asset - if the asset differs then we don't display the name, just "Multiple Assets"
							if(PreviousMatchingAsset != Object)
							{
								AssetName = LOCTEXT("MultipleAssetsForComponentAnnotation", "Multiple Assets").ToString();
								PreviousMatchingAsset = Object;
							}
						}
					}
				}
			}

			if(!AssetName.IsEmpty())
			{
				FriendlyComponentName += FString(" (") + AssetName + FString(")");
			}
		}
	}
	return FText::FromString(FriendlyComponentName);
}

void SDisplayClusterConfiguratorComponentClassCombo::GenerateComponentClassList()
{
	ComponentClassList.Reset();
	
	auto AddDCComp = [this](UClass* InClass)
	{
		TSharedPtr<FComponentClassComboEntry> Item = MakeShared<FComponentClassComboEntry>(InClass->GetName(), InClass, true, EComponentCreateAction::SpawnExistingClass);
		ComponentClassList.Add(Item);
	};

	auto AddHeader = [this](const FString& Name)
	{
		TSharedPtr<FComponentClassComboEntry> Item = MakeShared<FComponentClassComboEntry>(Name);
		ComponentClassList.Add(Item);
	};

	auto AddSep = [this]()
	{
		TSharedPtr<FComponentClassComboEntry> Item = MakeShared<FComponentClassComboEntry>();
		ComponentClassList.Add(Item);
	};

	AddHeader("nDisplay Components");
	AddDCComp(UDisplayClusterXformComponent::StaticClass());
	AddDCComp(UDisplayClusterScreenComponent::StaticClass());
	AddDCComp(UDisplayClusterCameraComponent::StaticClass());
	AddDCComp(UDisplayClusterInFrustumFitCameraComponent::StaticClass());

	AddHeader("nDisplay ICVFX Components");
	AddDCComp(UDisplayClusterICVFXCameraComponent::StaticClass());

	AddHeader("nDisplay Display Device Components");
	{
		// Locate all derived display device components. Only handles native classes. If we allow display devices
		// to be extended in BP then we'll need to handle unloaded assets.
		
		TArray<UClass*> DisplayDeviceClasses;
		GetDerivedClasses(UDisplayClusterDisplayDeviceBaseComponent::StaticClass(), DisplayDeviceClasses);

		for (UClass* Class : DisplayDeviceClasses)
		{
			if (!Class->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists))
			{
				AddDCComp(Class);
			}
		}
	}
	
	int32 CompIterationIdx = 0;
	for (const TSharedPtr<FComponentClassComboEntry>& Comp : *ComponentClassListPtr)
	{
		if (CompIterationIdx++ == 0)
		{
			// Hides the 'Scripting' section. We can't just check the name because GetClassName() isn't exported.
			continue;
		}
		
		if (Comp.IsValid())
		{
			if (UClass* Class = Comp->GetComponentClass())
			{
				if (Class->HasAnyClassFlags(CLASS_Abstract))
				{
					continue;
				}
			}
			
			ComponentClassList.AddUnique(Comp);
		}
	}
}

TSharedRef<SToolTip> SDisplayClusterConfiguratorComponentClassCombo::GetComponentToolTip(FComponentClassComboEntryPtr Entry) const
{
	// Special handling for the "New..." options
	if (Entry->GetComponentCreateAction() == EComponentCreateAction::CreateNewCPPClass)
	{
		return SNew(SToolTip)
			.Text(LOCTEXT("NewCPPComponentToolTip", "Create a custom actor component using C++"));
	}
	else if (Entry->GetComponentCreateAction() == EComponentCreateAction::CreateNewBlueprintClass)
	{
		return SNew(SToolTip)
			.Text(LOCTEXT("NewBlueprintComponentToolTip", "Create a custom actor component using Blueprints"));
	}
	
	// Handle components which have a currently loaded class
	if (const UClass* ComponentClass = Entry->GetComponentClass())
	{
		return FEditorClassUtils::GetTooltip(ComponentClass);
	}

	// Fallback for components that don't currently have a loaded class
	return SNew(SToolTip)
		.Text(FText::FromString("")); // Entry->GetClassName() Isn't exported or defined in header
}

#undef LOCTEXT_NAMESPACE
