// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SComponentClassCombo.h"

class FComponentClassComboEntry;
class SToolTip;
class FTextFilterExpressionEvaluator;

class SDisplayClusterConfiguratorComponentClassCombo : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SDisplayClusterConfiguratorComponentClassCombo )
		: _IncludeText(true)
	{}

		SLATE_ATTRIBUTE(bool, IncludeText)
		SLATE_EVENT( FSubobjectClassSelected, OnSubobjectClassSelected )

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	~SDisplayClusterConfiguratorComponentClassCombo();

	/** Clear the current combo list selection */
	void ClearSelection();

	/**
	 * Updates the filtered list of component names.
	 */
	void GenerateFilteredComponentList();

	FText GetCurrentSearchString() const;

	/**
	 * Called when the user changes the text in the search box.
	 * @param InSearchText The new search string.
	 */
	void OnSearchBoxTextChanged( const FText& InSearchText );

	/** Callback when the user commits the text in the searchbox */
	void OnSearchBoxTextCommitted(const FText& NewText, ETextCommit::Type CommitInfo);

	void OnAddComponentSelectionChanged( FComponentClassComboEntryPtr InItem, ESelectInfo::Type SelectInfo );

	TSharedRef<ITableRow> GenerateAddComponentRow( FComponentClassComboEntryPtr Entry, const TSharedRef<STableViewBase> &OwnerTable ) const;

	/** Update list of component classes */
	void UpdateComponentClassList();

protected:
	/** Generate the source component list to use for nDisplay. */
	void GenerateComponentClassList();
	
	FText GetFriendlyComponentName(FComponentClassComboEntryPtr Entry) const;

	TSharedRef<SToolTip> GetComponentToolTip(FComponentClassComboEntryPtr Entry) const;
	
	FSubobjectClassSelected OnSubobjectClassSelected;

	TArray<FComponentClassComboEntryPtr>* ComponentClassListPtr;
	
	/** List of component class names used by combo box */
	TArray<FComponentClassComboEntryPtr> ComponentClassList;

	/** List of component class names, filtered by the current search string */
	TArray<FComponentClassComboEntryPtr> FilteredComponentClassList;

	/** The current search string */
	TSharedPtr<FTextFilterExpressionEvaluator> TextFilter;

	/** The search box control - part of the combo drop down */
	TSharedPtr<SSearchBox> SearchBox;

	/** The Add combo button. */
	TSharedPtr<class SPositiveActionButton> AddNewButton;

	/** The component list control - part of the combo drop down */
	TSharedPtr< SListView<FComponentClassComboEntryPtr> > ComponentClassListView;

	/** Cached selection index used to skip over unselectable items */
	int32 PrevSelectedIndex;
};
