// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/SDisplayClusterConfiguratorViewBase.h"

#define LOCTEXT_NAMESPACE "SDisplayClusterConfiguratorViewBase"


void SDisplayClusterConfiguratorViewBase::Construct(const FArguments& InArgs, const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& InToolkit)
{
	ToolkitPtr = InToolkit;

	ConstructChildren(
		InArgs._Padding,
		InArgs._Content.Widget
	);
}

void SDisplayClusterConfiguratorViewBase::ConstructChildren(const TAttribute<FMargin>& InPadding, const TSharedRef<SWidget>& InContent)
{
	ChildSlot
		.Padding(InPadding)
		[
			InContent
		];
}

#undef LOCTEXT_NAMESPACE
