// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Views/IDisplayClusterConfiguratorViewBase.h"

#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class FDisplayClusterConfiguratorBlueprintEditor;

class SDisplayClusterConfiguratorViewBase
	: public IDisplayClusterConfiguratorViewBase
{

public:
	SLATE_BEGIN_ARGS(SDisplayClusterConfiguratorViewBase)
		: _Padding(FMargin(0))
		, _Content()
	{ }

	SLATE_ATTRIBUTE(FMargin, Padding)

	SLATE_DEFAULT_SLOT(SDisplayClusterConfiguratorViewBase::FArguments, Content)

	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& InToolkit);

	virtual void ConstructChildren(const TAttribute<FMargin>& InPadding, const TSharedRef<SWidget>& InContent) override;

protected:
	TWeakPtr<FDisplayClusterConfiguratorBlueprintEditor> ToolkitPtr;
};
