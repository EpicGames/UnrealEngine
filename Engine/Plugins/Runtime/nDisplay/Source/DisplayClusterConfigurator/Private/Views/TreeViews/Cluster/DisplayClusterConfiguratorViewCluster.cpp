// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/TreeViews/Cluster/DisplayClusterConfiguratorViewCluster.h"

#include "DisplayClusterConfiguratorBlueprintEditor.h"
#include "DisplayClusterConfiguratorUtils.h"
#include "DisplayClusterConfiguratorCommands.h"
#include "ClusterConfiguration/DisplayClusterConfiguratorClusterEditorUtils.h"
#include "ClusterConfiguration/DisplayClusterConfiguratorClusterUtils.h"
#include "ClusterConfiguration/SDisplayClusterConfiguratorNewClusterItemDialog.h"
#include "Views/TreeViews/IDisplayClusterConfiguratorTreeItem.h"
#include "Views/TreeViews/Cluster/DisplayClusterConfiguratorViewClusterBuilder.h"
#include "Views/TreeViews/DisplayClusterConfiguratorTreeViewCommands.h"
#include "Views/TreeViews/SDisplayClusterConfiguratorViewTree.h"

#include "Framework/Commands/GenericCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "UICommandList_Pinnable.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Exporters/Exporter.h"
#include "ScopedTransaction.h"
#include "Widgets/Images/SImage.h"

#define LOCTEXT_NAMESPACE "FDisplayClusterConfiguratorViewCluster"

const FName FDisplayClusterConfiguratorViewCluster::Columns::Host("Host");
const FName FDisplayClusterConfiguratorViewCluster::Columns::Visible("Visible");
const FName FDisplayClusterConfiguratorViewCluster::Columns::Enabled("Enabled");

FDisplayClusterConfiguratorViewCluster::FDisplayClusterConfiguratorViewCluster(const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& InToolkit)
	: FDisplayClusterConfiguratorViewTree(InToolkit)
{
	TreeBuilder = MakeShared<FDisplayClusterConfiguratorViewClusterBuilder>(InToolkit);
}

void FDisplayClusterConfiguratorViewCluster::PostUndo(bool bSuccess)
{
	ToolkitPtr.Pin()->ClusterChanged();
}

TSharedRef<SWidget> FDisplayClusterConfiguratorViewCluster::CreateWidget()
{
	ToolkitPtr.Pin()->RegisterOnClusterChanged(IDisplayClusterConfiguratorBlueprintEditor::FOnClusterChangedDelegate::CreateSP(this, &FDisplayClusterConfiguratorViewCluster::OnClusterChanged));

	return FDisplayClusterConfiguratorViewTree::CreateWidget();
}

void FDisplayClusterConfiguratorViewCluster::ConstructColumns(TArray<SHeaderRow::FColumn::FArguments>& OutColumnArgs)
{
	// For the cluster view, use the group column to indicate which host the cluster nodes belong to with a colored band
	OutColumnArgs.Add(SHeaderRow::Column(FDisplayClusterConfiguratorViewCluster::Columns::Host)
		.DefaultLabel(FText::GetEmpty())
		.DefaultTooltip(LOCTEXT("HostColumn_ToolTip", "Host"))
		.FixedWidth(12)
	);

	OutColumnArgs.Add(SHeaderRow::Column(FDisplayClusterConfiguratorViewCluster::Columns::Visible)
		.DefaultLabel(LOCTEXT("VisibleColumn_Label", "Visible"))
		.FixedWidth(24)
		.HAlignHeader(HAlign_Left)
		.VAlignHeader(VAlign_Center)
		.HAlignCell(HAlign_Center)
		.VAlignCell(VAlign_Center)
		.DefaultTooltip(LOCTEXT("VisibleColumn_ToolTip", "Visible"))
		.HeaderContentPadding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
		[
			SNew(SImage)
			.ColorAndOpacity(FSlateColor::UseForeground())
			.Image(FAppStyle::Get().GetBrush("Level.VisibleIcon16x"))
		]
	);

	OutColumnArgs.Add(SHeaderRow::Column(FDisplayClusterConfiguratorViewCluster::Columns::Enabled)
		.DefaultLabel(LOCTEXT("EnabledColumn_Label", "Locked"))
		.FixedWidth(24)
		.HAlignHeader(HAlign_Left)
		.VAlignHeader(VAlign_Center)
		.HAlignCell(HAlign_Center)
		.VAlignCell(VAlign_Center)
		.DefaultTooltip(LOCTEXT("EnabledColumn_ToolTip", "Locked"))
		.HeaderContentPadding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
		[
			SNew(SImage)
			.ColorAndOpacity(FSlateColor::UseForeground())
			.Image(FAppStyle::Get().GetBrush("PropertyWindow.Locked"))
		]
	);

	FDisplayClusterConfiguratorViewTree::ConstructColumns(OutColumnArgs);
}

void FDisplayClusterConfiguratorViewCluster::FillContextMenu(FMenuBuilder& MenuBuilder)
{
	const FDisplayClusterConfiguratorCommands& ConfiguratorCommands = FDisplayClusterConfiguratorCommands::Get();
	const FDisplayClusterConfiguratorTreeViewCommands& TreeViewCommands = FDisplayClusterConfiguratorTreeViewCommands::Get();

	TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>> SelectedTreeItems = ViewTree->GetSelectedItems();
	if (SelectedTreeItems.Num() == 1)
	{
		UObject* SelectedObject = SelectedTreeItems[0]->GetObject();
		MenuBuilder.BeginSection("Add", LOCTEXT("AddSection", "Add"));
		{
			if (SelectedObject->IsA<UDisplayClusterConfigurationCluster>() || SelectedObject->IsA<UDisplayClusterConfigurationHostDisplayData>())
			{
				MenuBuilder.AddMenuEntry(ConfiguratorCommands.AddNewClusterNode);
			}
			else if (SelectedObject->IsA<UDisplayClusterConfigurationClusterNode>())
			{
				MenuBuilder.AddMenuEntry(ConfiguratorCommands.AddNewViewport);
			}
		}
		MenuBuilder.EndSection();
	}

	MenuBuilder.BeginSection("Edit", LOCTEXT("EditSection", "Edit"));
	{
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Cut, "CutNode", LOCTEXT("CutNode", "Cut"), LOCTEXT("CutNodeToolTip", "Cuts the selected cluster node."));
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Copy, "CopyNode", LOCTEXT("CopyNode", "Copy"), LOCTEXT("CopyCompToolTip", "Copies the selected cluster node."));
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Paste, "PasteNode", LOCTEXT("PasteNode", "Paste"), LOCTEXT("PasteCompToolTop", "Adds the copied cluster node."));
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Duplicate, "DuplicateNode", LOCTEXT("DuplicateNode", "Duplicate"), LOCTEXT("DuplicateCompToolTip", "Duplicates the selected cluster nodes."));
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete, "DeleteNode", LOCTEXT("DeleteNode", "Delete"), LOCTEXT("DeleteCompToolTip", "Deletes the selected cluster nodes."));
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Rename, "RenameNode", LOCTEXT("RenameNode", "Rename"), LOCTEXT("RenameCompToolTip", "Renames the selected cluster nodes."));
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Visibility", LOCTEXT("VisibilitySection", "Visibility"));
	{
		MenuBuilder.AddMenuEntry(TreeViewCommands.ShowSelected);
		MenuBuilder.AddMenuEntry(TreeViewCommands.HideSelected);
		MenuBuilder.AddMenuEntry(TreeViewCommands.ShowSelectedOnly);
		MenuBuilder.AddMenuEntry(TreeViewCommands.ShowAll);
	}
	MenuBuilder.EndSection();

	if (SelectedTreeItems.Num() == 1)
	{
		UObject* SelectedObject = SelectedTreeItems[0]->GetObject();
		if (SelectedObject->IsA<UDisplayClusterConfigurationClusterNode>())
		{
			MenuBuilder.AddSeparator();
			MenuBuilder.AddMenuEntry(TreeViewCommands.SetAsPrimary);
		}
	}
}

void FDisplayClusterConfiguratorViewCluster::BindPinnableCommands(FUICommandList_Pinnable& CommandList)
{
	CommandList.MapAction(
		FGenericCommands::Get().Cut,
		FExecuteAction::CreateSP(this, &FDisplayClusterConfiguratorViewCluster::CutSelectedNodes),
		FCanExecuteAction::CreateSP(this, &FDisplayClusterConfiguratorViewCluster::CanCutSelectedNodes)
	);

	CommandList.MapAction(
		FGenericCommands::Get().Copy,
		FExecuteAction::CreateSP(this, &FDisplayClusterConfiguratorViewCluster::CopySelectedNodes),
		FCanExecuteAction::CreateSP(this, &FDisplayClusterConfiguratorViewCluster::CanCopySelectedNodes)
	);

	CommandList.MapAction(
		FGenericCommands::Get().Paste,
		FExecuteAction::CreateSP(this, &FDisplayClusterConfiguratorViewCluster::PasteNodes),
		FCanExecuteAction::CreateSP(this, &FDisplayClusterConfiguratorViewCluster::CanPasteNodes)
	);

	CommandList.MapAction(
		FGenericCommands::Get().Duplicate,
		FExecuteAction::CreateSP(this, &FDisplayClusterConfiguratorViewCluster::DuplicateSelectedNodes),
		FCanExecuteAction::CreateSP(this, &FDisplayClusterConfiguratorViewCluster::CanDuplicateSelectedNodes)
	);

	CommandList.MapAction(
		FGenericCommands::Get().Delete,
		FExecuteAction::CreateSP(this, &FDisplayClusterConfiguratorViewCluster::DeleteSelectedNodes),
		FCanExecuteAction::CreateSP(this, &FDisplayClusterConfiguratorViewCluster::CanDeleteSelectedNodes)
	);

	CommandList.MapAction(
		FGenericCommands::Get().Rename,
		FExecuteAction::CreateSP(this, &FDisplayClusterConfiguratorViewCluster::RenameSelectedNode),
		FCanExecuteAction::CreateSP(this, &FDisplayClusterConfiguratorViewCluster::CanRenameSelectedNode)
	);

	const FDisplayClusterConfiguratorCommands& ConfiguratorCommands = FDisplayClusterConfiguratorCommands::Get();
	const FDisplayClusterConfiguratorTreeViewCommands& TreeViewCommands = FDisplayClusterConfiguratorTreeViewCommands::Get();
	const FVector2D DefaultPresetSize = FVector2D(1920, 1080);

	// Base commands simply use a preset size of FHD
	CommandList.MapAction(
		ConfiguratorCommands.AddNewClusterNode,
		FExecuteAction::CreateSP(this, &FDisplayClusterConfiguratorViewCluster::AddNewClusterNode, DefaultPresetSize),
		FCanExecuteAction::CreateSP(this, &FDisplayClusterConfiguratorViewCluster::CanAddNewClusterNode)
	);

	CommandList.MapAction(
		ConfiguratorCommands.AddNewViewport,
		FExecuteAction::CreateSP(this, &FDisplayClusterConfiguratorViewCluster::AddNewViewport, DefaultPresetSize),
		FCanExecuteAction::CreateSP(this, &FDisplayClusterConfiguratorViewCluster::CanAddNewViewport)
	);

	CommandList.MapAction(
		TreeViewCommands.ShowSelected,
		FExecuteAction::CreateSP(this, &FDisplayClusterConfiguratorViewCluster::ShowSelected),
		FCanExecuteAction::CreateSP(this, &FDisplayClusterConfiguratorViewCluster::CanShowSelected)
	);

	CommandList.MapAction(
		TreeViewCommands.HideSelected,
		FExecuteAction::CreateSP(this, &FDisplayClusterConfiguratorViewCluster::HideSelected),
		FCanExecuteAction::CreateSP(this, &FDisplayClusterConfiguratorViewCluster::CanHideSelected)
	);

	CommandList.MapAction(
		TreeViewCommands.ShowSelectedOnly,
		FExecuteAction::CreateSP(this, &FDisplayClusterConfiguratorViewCluster::ShowSelectedOnly),
		FCanExecuteAction::CreateSP(this, &FDisplayClusterConfiguratorViewCluster::CanShowSelectedOnly)
	);

	CommandList.MapAction(
		TreeViewCommands.ShowAll,
		FExecuteAction::CreateSP(this, &FDisplayClusterConfiguratorViewCluster::ShowAll)
	);

	CommandList.MapAction(
		TreeViewCommands.SetAsPrimary,
		FExecuteAction::CreateSP(this, &FDisplayClusterConfiguratorViewCluster::SetAsPrimary),
		FCanExecuteAction::CreateSP(this, &FDisplayClusterConfiguratorViewCluster::CanSetAsPrimary)
	);
}

void FDisplayClusterConfiguratorViewCluster::FillAddNewMenu(FMenuBuilder& MenuBuilder)
{
	const FDisplayClusterConfiguratorCommands& ConfiguratorCommands = FDisplayClusterConfiguratorCommands::Get();

	MenuBuilder.AddMenuEntry(ConfiguratorCommands.AddNewClusterNode);
	MenuBuilder.AddMenuEntry(ConfiguratorCommands.AddNewViewport);
}

FText FDisplayClusterConfiguratorViewCluster::GetCornerText() const
{
	return LOCTEXT("CornerText", "STEP 2");
}

void FDisplayClusterConfiguratorViewCluster::OnClusterChanged()
{
	ClearSelection();
	ViewTree->Refresh();
}

void FDisplayClusterConfiguratorViewCluster::CutSelectedNodes()
{
	CopySelectedNodes();
	DeleteSelectedNodes();
}

bool FDisplayClusterConfiguratorViewCluster::CanCutSelectedNodes() const
{
	TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>> SelectedTreeItems = ViewTree->GetSelectedItems();

	for (TSharedPtr<IDisplayClusterConfiguratorTreeItem> TreeItem : SelectedTreeItems)
	{
		if (TreeItem->CanDuplicateItem() && TreeItem->CanDeleteItem())
		{
			return true;
		}
	}

	return false;
}

void FDisplayClusterConfiguratorViewCluster::CopySelectedNodes()
{
	TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>> SelectedTreeItems = ViewTree->GetSelectedItems();

	TArray<UObject*> ObjectsToCopy;
	for (const TSharedPtr<IDisplayClusterConfiguratorTreeItem>& TreeItem : SelectedTreeItems)
	{
		if (TreeItem->CanDuplicateItem())
		{
			ObjectsToCopy.Add(TreeItem->GetObject());
		}
	}

	UE::DisplayClusterConfiguratorClusterEditorUtils::CopyClusterItemsToClipboard(ObjectsToCopy);
}

bool FDisplayClusterConfiguratorViewCluster::CanCopySelectedNodes() const
{
	TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>> SelectedTreeItems = ViewTree->GetSelectedItems();

	for (TSharedPtr<IDisplayClusterConfiguratorTreeItem> TreeItem : SelectedTreeItems)
	{
		if (TreeItem->CanDuplicateItem())
		{
			return true;
		}
	}

	return false;
}

void FDisplayClusterConfiguratorViewCluster::PasteNodes()
{
	TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>> SelectedTreeItems = ViewTree->GetSelectedItems();
	TArray<UObject*> TargetObjects;
	for (TSharedPtr<IDisplayClusterConfiguratorTreeItem> TreeItem : SelectedTreeItems)
	{
		TargetObjects.Add(TreeItem->GetObject());
	}

	int32 NumClusterItems;
	if (UE::DisplayClusterConfiguratorClusterEditorUtils::CanPasteClusterItemsFromClipboard(TargetObjects, NumClusterItems))
	{
		FScopedTransaction Transaction(FText::Format(LOCTEXT("PasteClusterItems", "Paste Cluster {0}|plural(one=Item, other=Items)"), NumClusterItems));

		TArray<UObject*> PastedObjects = UE::DisplayClusterConfiguratorClusterEditorUtils::PasteClusterItemsFromClipboard(TargetObjects);
		if (PastedObjects.Num() > 0)
		{
			// Mark the cluster configuration data as dirty, allowing user to save the changes, and fire off a cluster changed event to let other
			// parts of the UI update as well
			GetEditorData()->MarkPackageDirty();
			ToolkitPtr.Pin()->ClusterChanged();

			// After we have pasted, we want to select the pasted items in the tree view. Find all corresponding tree view items
			// and select them.
			ClearSelection();

			TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>> AllItems = ViewTree->GetAllItemsFlattened();
			TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>> CopiedItems;

			for (const TSharedPtr<IDisplayClusterConfiguratorTreeItem>& Item : AllItems)
			{
				if (PastedObjects.Contains(Item->GetObject()))
				{
					CopiedItems.Add(Item);
				}
			}

			SetSelectedItems(CopiedItems);
		}
		else
		{
			Transaction.Cancel();
		}
	}
}

bool FDisplayClusterConfiguratorViewCluster::CanPasteNodes() const
{
	TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>> SelectedTreeItems = ViewTree->GetSelectedItems();
	TArray<UObject*> TargetObjects;
	for (TSharedPtr<IDisplayClusterConfiguratorTreeItem> TreeItem : SelectedTreeItems)
	{
		TargetObjects.Add(TreeItem->GetObject());
	}

	int32 NumClusterItems;
	return UE::DisplayClusterConfiguratorClusterEditorUtils::CanPasteClusterItemsFromClipboard(TargetObjects, NumClusterItems);
}

void FDisplayClusterConfiguratorViewCluster::DuplicateSelectedNodes()
{
	CopySelectedNodes();
	PasteNodes();
}

bool FDisplayClusterConfiguratorViewCluster::CanDuplicateSelectedNodes() const
{
	return CanCopySelectedNodes();
}

void FDisplayClusterConfiguratorViewCluster::DeleteSelectedNodes()
{
	TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>> SelectedTreeItems = ViewTree->GetSelectedItems();

	const FScopedTransaction Transaction(FText::Format(LOCTEXT("DeleteClusterNodes", "Remove Cluster {0}|plural(one=Node, other=Nodes)"), SelectedTreeItems.Num()));

	for (const TSharedPtr<IDisplayClusterConfiguratorTreeItem>& TreeItem : SelectedTreeItems)
	{
		if (TreeItem->CanDeleteItem())
		{
			TreeItem->DeleteItem();
		}
	}

	// Mark the cluster configuration data as dirty, allowing user to save the changes, and fire off a cluster changed event to let other
	// parts of the UI update as well
	GetEditorData()->MarkPackageDirty();
	constexpr bool bStructureChange = true;
	ToolkitPtr.Pin()->ClusterChanged(bStructureChange);

	// Clear the selection to ensure the deleted cluster item isn't still being displayed in the details panel
	ClearSelection();

	TArray<UObject*> Selection;
	ToolkitPtr.Pin()->SelectObjects(Selection);
}

bool FDisplayClusterConfiguratorViewCluster::CanDeleteSelectedNodes() const
{
	TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>> SelectedTreeItems = ViewTree->GetSelectedItems();

	for (TSharedPtr<IDisplayClusterConfiguratorTreeItem> TreeItem : SelectedTreeItems)
	{
		if (TreeItem->CanDeleteItem())
		{
			return true;
		}
	}

	return false;
}

void FDisplayClusterConfiguratorViewCluster::RenameSelectedNode()
{
	TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>> SelectedTreeItems = ViewTree->GetSelectedItems();

	if (SelectedTreeItems.Num() == 1)
	{
		SelectedTreeItems[0]->RequestRename();
	}
}

bool FDisplayClusterConfiguratorViewCluster::CanRenameSelectedNode() const
{
	TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>> SelectedTreeItems = ViewTree->GetSelectedItems();

	if (SelectedTreeItems.Num() == 1)
	{
		return SelectedTreeItems[0]->CanRenameItem();
	}

	return false;
}

void FDisplayClusterConfiguratorViewCluster::SetAsPrimary()
{
	TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>> SelectedTreeItems = ViewTree->GetSelectedItems();

	if (SelectedTreeItems.Num() == 1)
	{
		if (UDisplayClusterConfigurationClusterNode* ClusterNode = Cast<UDisplayClusterConfigurationClusterNode>(SelectedTreeItems[0]->GetObject()))
		{
			FScopedTransaction Transaction(LOCTEXT("SetPrimaryNode", "Set Primary Node"));

			if (!UE::DisplayClusterConfiguratorClusterUtils::SetClusterNodeAsPrimary(ClusterNode))
			{
				Transaction.Cancel();
			}
		}
	}
}

void FDisplayClusterConfiguratorViewCluster::HideSelected()
{
	TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>> SelectedTreeItems = ViewTree->GetSelectedItems();

	for (TSharedPtr<IDisplayClusterConfiguratorTreeItem> TreeItem : SelectedTreeItems)
	{
		if (TreeItem->CanHideItem())
		{
			TreeItem->SetItemHidden(true);
		}
	}
}

bool FDisplayClusterConfiguratorViewCluster::CanHideSelected() const
{
	TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>> SelectedTreeItems = ViewTree->GetSelectedItems();

	for (TSharedPtr<IDisplayClusterConfiguratorTreeItem> TreeItem : SelectedTreeItems)
	{
		if (TreeItem->CanHideItem())
		{
			return true;
		}
	}

	return false;
}

void FDisplayClusterConfiguratorViewCluster::ShowSelected()
{
	TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>> SelectedTreeItems = ViewTree->GetSelectedItems();

	for (TSharedPtr<IDisplayClusterConfiguratorTreeItem> TreeItem : SelectedTreeItems)
	{
		if (TreeItem->CanHideItem())
		{
			TreeItem->SetItemHidden(false);
		}
	}
}

bool FDisplayClusterConfiguratorViewCluster::CanShowSelected() const
{
	return CanHideSelected();
}

void FDisplayClusterConfiguratorViewCluster::ShowSelectedOnly()
{
	TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>> AllTreeItems = ViewTree->GetAllItemsFlattened();
	TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>> SelectedTreeItems = ViewTree->GetSelectedItems();

	for (TSharedPtr<IDisplayClusterConfiguratorTreeItem> TreeItem : AllTreeItems)
	{
		if (TreeItem->CanHideItem())
		{
			TreeItem->SetItemHidden(true);
		}
	}

	for (TSharedPtr<IDisplayClusterConfiguratorTreeItem> TreeItem : SelectedTreeItems)
	{
		if (TreeItem->CanHideItem())
		{
			TreeItem->SetItemHidden(false);
		}
	}
}

bool FDisplayClusterConfiguratorViewCluster::CanShowSelectedOnly() const
{
	return CanHideSelected();
}

void FDisplayClusterConfiguratorViewCluster::ShowAll()
{
	TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>> AllTreeItems = ViewTree->GetAllItemsFlattened();

	for (TSharedPtr<IDisplayClusterConfiguratorTreeItem> TreeItem : AllTreeItems)
	{
		if (TreeItem->CanHideItem())
		{
			TreeItem->SetItemHidden(false);
		}
	}
}

bool FDisplayClusterConfiguratorViewCluster::CanSetAsPrimary() const
{
	TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>> SelectedTreeItems = ViewTree->GetSelectedItems();

	if (SelectedTreeItems.Num() == 1)
	{
		if (const UDisplayClusterConfigurationClusterNode* ClusterNode = Cast<UDisplayClusterConfigurationClusterNode>(SelectedTreeItems[0]->GetObject()))
		{
			UDisplayClusterConfigurationCluster* Cluster = ToolkitPtr.Pin()->GetEditorData()->Cluster;
			const FString PrimaryNodeId = Cluster->PrimaryNode.Id;

			if (!Cluster->Nodes.Contains(PrimaryNodeId) || Cluster->Nodes[PrimaryNodeId] != ClusterNode)
			{
				return true;
			}
		}
	}

	return false;
}

void FDisplayClusterConfiguratorViewCluster::AddNewClusterNode(FVector2D PresetSize)
{
	const TSharedPtr<FDisplayClusterConfiguratorBlueprintEditor> Toolkit = ToolkitPtr.Pin();
	UDisplayClusterConfigurationCluster* Cluster = Toolkit->GetEditorData()->Cluster;
	const FDisplayClusterConfigurationRectangle PresetRect = FDisplayClusterConfigurationRectangle(0, 0, PresetSize.X, PresetSize.Y);
	
	FString HostAddress = NDISPLAY_DEFAULT_CLUSTER_HOST;
	TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>> SelectedTreeItems = ViewTree->GetSelectedItems();
	if (SelectedTreeItems.Num() == 1)
	{
		UObject* SelectedObject = SelectedTreeItems[0]->GetObject();
		if (UDisplayClusterConfigurationHostDisplayData* HostData = Cast<UDisplayClusterConfigurationHostDisplayData>(SelectedObject))
		{
			HostAddress = UE::DisplayClusterConfiguratorClusterUtils::GetAddressForHost(HostData);
		}
	}

	TSharedPtr<FScopedTransaction> Transaction;
	if (UDisplayClusterConfigurationClusterNode* NewNode = UE::DisplayClusterConfiguratorClusterEditorUtils::CreateNewClusterNodeFromDialog(Toolkit.ToSharedRef(), Cluster, PresetRect, Transaction, HostAddress))
	{
		// Mark the cluster configuration data as dirty, allowing user to save the changes, and fire off a cluster changed event to let other
		// parts of the UI update as well
		Toolkit->GetEditorData()->MarkPackageDirty();
		Toolkit->ClusterChanged();

		// After we have added the new cluster node, we want to select it in the tree view. Find all corresponding tree view item
		// and select it.
		ClearSelection();

		TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>> AllTreeItems = ViewTree->GetAllItemsFlattened();
		TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>> TreeItemsToSelect;

		for (const TSharedPtr<IDisplayClusterConfiguratorTreeItem>& TreeItem : AllTreeItems)
		{
			if (NewNode == TreeItem->GetObject())
			{
				TreeItemsToSelect.Add(TreeItem);
				break;
			}
		}

		SetSelectedItems(TreeItemsToSelect);
	}
}

bool FDisplayClusterConfiguratorViewCluster::CanAddNewClusterNode() const
{
	return true;
}

void FDisplayClusterConfiguratorViewCluster::AddNewViewport(FVector2D PresetSize)
{
	UDisplayClusterConfigurationClusterNode* SelectedClusterNode = nullptr;
	TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>> SelectedTreeItems = ViewTree->GetSelectedItems();
	if (SelectedTreeItems.Num() == 1)
	{
		if (UDisplayClusterConfigurationClusterNode* ClusterNode = Cast<UDisplayClusterConfigurationClusterNode>(SelectedTreeItems[0]->GetObject()))
		{
			SelectedClusterNode = ClusterNode;
		}
		else if (UDisplayClusterConfigurationViewport* Viewport = Cast<UDisplayClusterConfigurationViewport>(SelectedTreeItems[0]->GetObject()))
		{
			UDisplayClusterConfigurationClusterNode* ParentClusterNode = Cast<UDisplayClusterConfigurationClusterNode>(SelectedTreeItems[0]->GetParent()->GetObject());
			SelectedClusterNode = ParentClusterNode;
		}
	}

	const TSharedPtr<FDisplayClusterConfiguratorBlueprintEditor> Toolkit = ToolkitPtr.Pin();
	const FDisplayClusterConfigurationRectangle PresetRect = FDisplayClusterConfigurationRectangle(0, 0, PresetSize.X, PresetSize.Y);
	TSharedPtr<FScopedTransaction> Transaction;
	if (const UDisplayClusterConfigurationViewport* NewViewport = UE::DisplayClusterConfiguratorClusterEditorUtils::CreateNewViewportFromDialog(Toolkit.ToSharedRef(), SelectedClusterNode, PresetRect, Transaction))
	{
		// Mark the cluster configuration data as dirty, allowing user to save the changes, and fire off a cluster changed event to let other
		// parts of the UI update as well
		Toolkit->GetEditorData()->MarkPackageDirty();
		Toolkit->ClusterChanged();

		// After we have added the new viewport, we want to select it in the tree view. Find all corresponding tree view item
		// and select it.
		ClearSelection();

		TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>> AllTreeItems = ViewTree->GetAllItemsFlattened();
		TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>> TreeItemsToSelect;

		for (const TSharedPtr<IDisplayClusterConfiguratorTreeItem>& TreeItem : AllTreeItems)
		{
			if (NewViewport == TreeItem->GetObject())
			{
				TreeItemsToSelect.Add(TreeItem);
				break;
			}
		}

		SetSelectedItems(TreeItemsToSelect);
	}
}

bool FDisplayClusterConfiguratorViewCluster::CanAddNewViewport() const
{
	// Can only add a new viewport if there is at least one cluster node to add it to.
	return UE::DisplayClusterConfiguratorClusterEditorUtils::CanAddNewViewportToToolkit(ToolkitPtr);
}

#undef LOCTEXT_NAMESPACE