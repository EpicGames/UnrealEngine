// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Views/TreeViews/DisplayClusterConfiguratorViewTree.h"

class FDisplayClusterConfiguratorBlueprintEditor;
class SDisplayClusterConfiguratorNewClusterItemDialog;

class FDisplayClusterConfiguratorViewCluster
	: public FDisplayClusterConfiguratorViewTree
{
public:
	struct Columns
	{
		static const FName Host;
		static const FName Visible;
		static const FName Enabled;
	};

	FDisplayClusterConfiguratorViewCluster(const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& Toolkit);

	//~ Begin FEditorUndoClient interface
	virtual void PostUndo(bool bSuccess) override;
	//~ End FEditorUndoClient interface

	//~ Begin FDisplayClusterConfiguratorViewTree interface
	virtual TSharedRef<SWidget> CreateWidget() override;
	virtual void ConstructColumns(TArray<SHeaderRow::FColumn::FArguments>& OutColumnArgs) override;

	virtual void FillContextMenu(FMenuBuilder& MenuBuilder) override;
	virtual void BindPinnableCommands(FUICommandList_Pinnable& CommandList) override;
	virtual bool ShowAddNewButton() const override { return true; }
	virtual void FillAddNewMenu(FMenuBuilder& MenuBuilder) override;
	virtual FText GetCornerText() const override;
	//~ End FDisplayClusterConfiguratorViewTree interface

private:
	void OnClusterChanged();

	void CutSelectedNodes();
	bool CanCutSelectedNodes() const;

	void CopySelectedNodes();
	bool CanCopySelectedNodes() const;

	void PasteNodes();
	bool CanPasteNodes() const;

	void DuplicateSelectedNodes();
	bool CanDuplicateSelectedNodes() const;

	void DeleteSelectedNodes();
	bool CanDeleteSelectedNodes() const;

	void RenameSelectedNode();
	bool CanRenameSelectedNode() const;

	void HideSelected();
	bool CanHideSelected() const;

	void ShowSelected();
	bool CanShowSelected() const;

	void ShowSelectedOnly();
	bool CanShowSelectedOnly() const;

	void ShowAll();

	void SetAsPrimary();
	bool CanSetAsPrimary() const;

	void AddNewClusterNode(FVector2D PresetSize);
	bool CanAddNewClusterNode() const;

	void AddNewViewport(FVector2D PresetSize);
	bool CanAddNewViewport() const;

};
