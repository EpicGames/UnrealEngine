// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/TreeViews/Cluster/DisplayClusterConfiguratorViewClusterBuilder.h"

#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterConfiguratorBlueprintEditor.h"
#include "ClusterConfiguration/DisplayClusterConfiguratorClusterUtils.h"
#include "Views/TreeViews/IDisplayClusterConfiguratorTreeItem.h"
#include "Views/TreeViews/IDisplayClusterConfiguratorViewTree.h"
#include "Views/TreeViews/Cluster/TreeItems/DisplayClusterConfiguratorTreeItemCluster.h"
#include "Views/TreeViews/Cluster/TreeItems/DisplayClusterConfiguratorTreeItemHost.h"
#include "Views/TreeViews/Cluster/TreeItems/DisplayClusterConfiguratorTreeItemClusterNode.h"
#include "Views/TreeViews/Cluster/TreeItems/DisplayClusterConfiguratorTreeItemViewport.h"


FDisplayClusterConfiguratorViewClusterBuilder::FDisplayClusterConfiguratorViewClusterBuilder(const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& InToolkit)
	: FDisplayClusterConfiguratorTreeBuilder(InToolkit)
{
}

void FDisplayClusterConfiguratorViewClusterBuilder::Build(FDisplayClusterConfiguratorTreeBuilderOutput& Output)
{
	if (UDisplayClusterConfigurationData* EditorDataPtr = ConfiguratorTreePtr.Pin()->GetEditorData())
	{
		if (UDisplayClusterConfigurationData* Config = ToolkitPtr.Pin()->GetConfig())
		{
			if (Config->Cluster != nullptr)
			{
				AddCluster(Output, Config, Config->Cluster);
			}
		}
	}
}

void FDisplayClusterConfiguratorViewClusterBuilder::AddCluster(FDisplayClusterConfiguratorTreeBuilderOutput& Output, UDisplayClusterConfigurationData* InConfig, UObject* InObjectToEdit)
{
	FName ParentName = NAME_None;
	const FName NodeName = "Cluster";
	TSharedRef<IDisplayClusterConfiguratorTreeItem> DisplayNode = MakeShared<FDisplayClusterConfiguratorTreeItemCluster>(NodeName, ConfiguratorTreePtr.Pin().ToSharedRef(), ToolkitPtr.Pin().ToSharedRef(), InObjectToEdit, "DisplayClusterConfigurator.TreeItems.Cluster", true);
	Output.Add(DisplayNode, ParentName, FDisplayClusterConfiguratorTreeItem::GetTypeId());

	AddClusterNodes(Output, InConfig, InObjectToEdit);
}

void FDisplayClusterConfiguratorViewClusterBuilder::AddClusterNodes(FDisplayClusterConfiguratorTreeBuilderOutput& Output, UDisplayClusterConfigurationData* InConfig, UObject* InObjectToEdit)
{
	TMap<FString, TMap<FString, UDisplayClusterConfigurationClusterNode*>> SortedClusterNodes;
	UE::DisplayClusterConfiguratorClusterUtils::SortClusterNodesByHost(InConfig->Cluster->Nodes, SortedClusterNodes);

	for (const auto& HostPair : SortedClusterNodes)
	{
		UDisplayClusterConfigurationHostDisplayData* DisplayData = UE::DisplayClusterConfiguratorClusterUtils::FindOrCreateHostDisplayData(InConfig->Cluster, HostPair.Key);

		const FName HostParentName = "Cluster";
		const FName HostNodeName = *DisplayData->HostName.ToString();
		TSharedRef<IDisplayClusterConfiguratorTreeItem> HostDisplayNode = MakeShared<FDisplayClusterConfiguratorTreeItemHost>(HostNodeName, ConfiguratorTreePtr.Pin().ToSharedRef(), ToolkitPtr.Pin().ToSharedRef(), DisplayData);
		Output.Add(HostDisplayNode, HostParentName, FDisplayClusterConfiguratorTreeItemCluster::GetTypeId());

		for (const auto& Node : HostPair.Value)
		{
			const FName ParentName = *HostPair.Key;
			const FName NodeName = *Node.Key;
			TSharedRef<IDisplayClusterConfiguratorTreeItem> DisplayNode = MakeShared<FDisplayClusterConfiguratorTreeItemClusterNode>(NodeName, ConfiguratorTreePtr.Pin().ToSharedRef(), ToolkitPtr.Pin().ToSharedRef(), Node.Value, DisplayData);
			Output.Add(DisplayNode, ParentName, FDisplayClusterConfiguratorTreeItemHost::GetTypeId());

			for (const auto& Viewport : Node.Value->Viewports)
			{
				AddClusterNodeViewport(Output, InConfig, Viewport.Key, Node.Key, Viewport.Value);
			}
		}
	}
}

void FDisplayClusterConfiguratorViewClusterBuilder::AddClusterNodeViewport(FDisplayClusterConfiguratorTreeBuilderOutput& Output, UDisplayClusterConfigurationData* InConfig, const FString& NodeId, const FString& ParentNodeId, UObject* InObjectToEdit)
{
	const FName ParentName = *ParentNodeId;
	const FName NodeName = *NodeId;
	TSharedRef<IDisplayClusterConfiguratorTreeItem> DisplayNode = MakeShared<FDisplayClusterConfiguratorTreeItemViewport>(NodeName, ConfiguratorTreePtr.Pin().ToSharedRef(), ToolkitPtr.Pin().ToSharedRef(), InObjectToEdit);
	Output.Add(DisplayNode, ParentName, FDisplayClusterConfiguratorTreeItemClusterNode::GetTypeId());
}
