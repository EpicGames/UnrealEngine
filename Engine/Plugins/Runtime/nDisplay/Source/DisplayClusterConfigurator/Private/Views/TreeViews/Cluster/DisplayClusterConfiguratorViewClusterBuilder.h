// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Views/TreeViews/DisplayClusterConfiguratorTreeBuilder.h"

class UDisplayClusterConfigurationData;

class FDisplayClusterConfiguratorViewClusterBuilder
	: public FDisplayClusterConfiguratorTreeBuilder
{
public:
	FDisplayClusterConfiguratorViewClusterBuilder(const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& InToolkit);

	//~ Begin IDisplayClusterConfiguratorTreeBuilder Interface
	virtual void Build(FDisplayClusterConfiguratorTreeBuilderOutput& Output) override;
	//~ End IDisplayClusterConfiguratorTreeBuilder Interface

private:
	void AddCluster(FDisplayClusterConfiguratorTreeBuilderOutput& Output, UDisplayClusterConfigurationData* InConfigurationTempConfig, UObject* InObjectToEdit);
	void AddClusterNodes(FDisplayClusterConfiguratorTreeBuilderOutput& Output, UDisplayClusterConfigurationData* InConfigurationTempConfig, UObject* InObjectToEdit);
	void AddClusterNodeViewport(FDisplayClusterConfiguratorTreeBuilderOutput& Output, UDisplayClusterConfigurationData* InConfigurationTempConfig, const FString& NodeId, const FString& ParentNodeId, UObject* InObjectToEdit);
};
