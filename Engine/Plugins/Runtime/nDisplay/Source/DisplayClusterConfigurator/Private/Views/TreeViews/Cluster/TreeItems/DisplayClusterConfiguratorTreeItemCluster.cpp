// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/TreeViews/Cluster/TreeItems/DisplayClusterConfiguratorTreeItemCluster.h"

#include "DisplayClusterConfiguratorBlueprintEditor.h"
#include "DisplayClusterConfiguratorStyle.h"
#include "Views/OutputMapping/IDisplayClusterConfiguratorViewOutputMapping.h"
#include "Views/TreeViews/IDisplayClusterConfiguratorViewTree.h"
#include "Views/TreeViews/Cluster/DisplayClusterConfiguratorViewCluster.h"

#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"

#define LOCTEXT_NAMESPACE "FDisplayClusterConfiguratorTreeItemCluster"


class SToggleStateButton : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SToggleStateButton) 
		: _ToggleState(false)
	{}
		SLATE_ATTRIBUTE(const FSlateBrush*, ToggleOnImage)
		SLATE_ATTRIBUTE(const FSlateBrush*, ToggleOffImage)
		SLATE_ATTRIBUTE(bool, ToggleState)
		SLATE_EVENT(FSimpleDelegate, OnToggled)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		ToggleOnImageAttribute = InArgs._ToggleOnImage;
		ToggleOffImageAttribute = InArgs._ToggleOffImage;
		ToggleStateAttribute = InArgs._ToggleState;

		OnToggled = InArgs._OnToggled;

		ChildSlot
		[
			SNew(SButton)
			.ContentPadding(0)
			.ButtonStyle(FAppStyle::Get(), "NoBorder")
			.ToolTipText(InArgs._ToolTipText)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.OnClicked(this, &SToggleStateButton::OnButtonClicked)
			.Content()
			[
				SNew(SImage)
				.Image(this, &SToggleStateButton::GetImageBrush)
				.ColorAndOpacity(this, &SToggleStateButton::GetImageForegroundColor)
			]
		];
	}

public:
	bool GetToggleState() const { return ToggleStateAttribute.Get(); }
	void SetToggleState(TAttribute<bool> NewToggleState) { ToggleStateAttribute = NewToggleState; }

private:
	FReply OnButtonClicked()
	{
		OnToggled.ExecuteIfBound();

		return FReply::Handled();
	}

	const FSlateBrush* GetImageBrush() const	
	{
		return ToggleStateAttribute.Get(false) ? ToggleOnImageAttribute.Get() : ToggleOffImageAttribute.Get();
	}

	FSlateColor GetImageForegroundColor() const	
	{
		return IsHovered() ? FSlateColor::UseForeground() : FSlateColor::UseSubduedForeground();
	}

private:
	TAttribute<const FSlateBrush*> ToggleOnImageAttribute;
	TAttribute<const FSlateBrush*> ToggleOffImageAttribute;
	TAttribute<bool> ToggleStateAttribute;

	FSimpleDelegate OnToggled;
};


FDisplayClusterConfiguratorTreeItemCluster::FDisplayClusterConfiguratorTreeItemCluster(const FName& InName,
	const TSharedRef<IDisplayClusterConfiguratorViewTree>& InViewTree,
	const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& InToolkit,
	UObject* InObjectToEdit,
	FString InIconStyle,
	bool InbRoot)
	: FDisplayClusterConfiguratorTreeItem(InViewTree, InToolkit, InObjectToEdit, InbRoot)
	, Name(InName)
	, IconStyle(InIconStyle)
{}

TSharedRef<SWidget> FDisplayClusterConfiguratorTreeItemCluster::GenerateWidgetForColumn(const FName& ColumnName, TSharedPtr<ITableRow> TableRow, const TAttribute<FText>& FilterText, FIsSelected InIsSelected)
{
	if (ColumnName == FDisplayClusterConfiguratorViewCluster::Columns::Host && IsClusterItemGrouped())
	{
		return SNew(SBox)
			.HAlign(HAlign_Center)
			[
				SNew(SImage)
				.ColorAndOpacity(this, &FDisplayClusterConfiguratorTreeItemCluster::GetClusterItemGroupColor)
				.OnMouseButtonDown(this, &FDisplayClusterConfiguratorTreeItemCluster::OnClusterItemGroupClicked)
				.Image(FAppStyle::Get().GetBrush("WhiteBrush"))
				.DesiredSizeOverride(FVector2D(6, 20))
				.Cursor(EMouseCursor::Hand)
			];
	}
	else if (ColumnName == FDisplayClusterConfiguratorViewCluster::Columns::Visible && CanClusterItemBeHidden())
	{
		return SNew(SToggleStateButton)
			.ToggleOnImage(FAppStyle::Get().GetBrush(TEXT("Level.VisibleIcon16x")))
			.ToggleOffImage(FAppStyle::Get().GetBrush(TEXT("Level.NotVisibleIcon16x")))
			.ToggleState(this, &FDisplayClusterConfiguratorTreeItemCluster::IsClusterItemVisible)
			.Visibility(this, &FDisplayClusterConfiguratorTreeItemCluster::GetVisibleButtonVisibility)
			.ToolTipText(LOCTEXT("VisibilityButton_Tooltip", "Hides or shows this cluster item and its children in the Output Mapping editor"))
			.OnToggled(this, &FDisplayClusterConfiguratorTreeItemCluster::ToggleClusterItemVisibility);
	}
	else if (ColumnName == FDisplayClusterConfiguratorViewCluster::Columns::Enabled && CanClusterItemBeLocked())
	{
		return SNew(SToggleStateButton)
			.ToggleOnImage(FAppStyle::Get().GetBrush(TEXT("PropertyWindow.Unlocked")))
			.ToggleOffImage(FAppStyle::Get().GetBrush(TEXT("PropertyWindow.Locked")))
			.ToggleState(this, &FDisplayClusterConfiguratorTreeItemCluster::IsClusterItemUnlocked)
			.Visibility(this, &FDisplayClusterConfiguratorTreeItemCluster::GetLockButtonVisibility)
			.ToolTipText(LOCTEXT("LockButton_Tooltip", "Locks or unlocks this cluster item and its children in the Output Mapping editor"))
			.OnToggled(this, &FDisplayClusterConfiguratorTreeItemCluster::ToggleClusterItemLock);
	}

	return FDisplayClusterConfiguratorTreeItem::GenerateWidgetForColumn(ColumnName, TableRow, FilterText, InIsSelected);
}

void FDisplayClusterConfiguratorTreeItemCluster::OnItemDoubleClicked()
{
	if (ToolkitPtr.IsValid())
	{
		TSharedPtr<FDisplayClusterConfiguratorBlueprintEditor> Toolkit = ToolkitPtr.Pin();
		Toolkit->GetViewOutputMapping()->JumpToObject(GetObject());
	}
}

void FDisplayClusterConfiguratorTreeItemCluster::OnMouseEnter()
{
	ViewTreePtr.Pin()->SetHoveredItem(SharedThis(this));
}

void FDisplayClusterConfiguratorTreeItemCluster::OnMouseLeave()
{
	ViewTreePtr.Pin()->ClearHoveredItem();
}

bool FDisplayClusterConfiguratorTreeItemCluster::IsHovered() const
{
	if (TSharedPtr<IDisplayClusterConfiguratorTreeItem> HoveredItem = ViewTreePtr.Pin()->GetHoveredItem())
	{
		return HoveredItem.Get() == this;
	}

	return false;
}

EVisibility FDisplayClusterConfiguratorTreeItemCluster::GetVisibleButtonVisibility() const
{
	bool bIsHovered = IsHovered();
	bool bIsSelected = ViewTreePtr.Pin()->GetSelectedItems().Contains(SharedThis(this));

	if (bIsHovered || bIsSelected || !IsClusterItemVisible())
	{
		return EVisibility::Visible;
	}
	else
	{
		return EVisibility::Hidden;
	}
}

EVisibility FDisplayClusterConfiguratorTreeItemCluster::GetLockButtonVisibility() const
{
	bool bIsHovered = IsHovered();
	bool bIsSelected = ViewTreePtr.Pin()->GetSelectedItems().Contains(SharedThis(this));

	if (bIsHovered || bIsSelected || !IsClusterItemUnlocked())
	{
		return EVisibility::Visible;
	}
	else
	{
		return EVisibility::Hidden;
	}
}

#undef LOCTEXT_NAMESPACE