// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Views/TreeViews/DisplayClusterConfiguratorTreeItem.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class IDisplayClusterConfiguratorViewTree;

class FDisplayClusterConfiguratorTreeItemCluster
	: public FDisplayClusterConfiguratorTreeItem
{
public:
	NDISPLAY_TREE_ITEM_TYPE(FDisplayClusterConfiguratorTreeItemCluster, FDisplayClusterConfiguratorTreeItem)

	FDisplayClusterConfiguratorTreeItemCluster(const FName& InName,
		const TSharedRef<IDisplayClusterConfiguratorViewTree>& InViewTree,
		const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& InToolkit,
		UObject* InObjectToEdit,
		FString InIconStyle,
		bool InbRoot = false);

	//~ Begin IDisplayClusterConfiguratorTreeItem Interface
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName, TSharedPtr<ITableRow> TableRow, const TAttribute<FText>& FilterText, FIsSelected InIsSelected) override;
	virtual void OnItemDoubleClicked() override;
	virtual void OnMouseEnter() override;
	virtual void OnMouseLeave() override;
	virtual bool IsHovered() const override;

	virtual FName GetRowItemName() const override { return Name; }
	virtual FString GetIconStyle() const override { return IconStyle; }
	//~ End IDisplayClusterConfiguratorTreeItem Interface

protected:
	virtual bool CanClusterItemBeHidden() const { return false; }
	virtual bool CanClusterItemBeLocked() const { return false; }
	virtual bool IsClusterItemGrouped() const { return false; }

	virtual bool IsClusterItemVisible() const { return true; }
	virtual bool IsClusterItemUnlocked() const { return true; }

	virtual void ToggleClusterItemVisibility() { }
	virtual void ToggleClusterItemLock() { }

	virtual FSlateColor GetClusterItemGroupColor() const { return FLinearColor::Transparent; }
	virtual FReply OnClusterItemGroupClicked(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) { return FReply::Unhandled(); }

	EVisibility GetVisibleButtonVisibility() const;
	EVisibility GetLockButtonVisibility() const;

protected:
	FName Name;
	FString IconStyle;
};