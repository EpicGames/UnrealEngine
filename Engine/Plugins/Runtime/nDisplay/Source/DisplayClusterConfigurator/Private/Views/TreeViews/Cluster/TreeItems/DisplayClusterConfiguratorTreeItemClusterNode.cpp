// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfiguratorTreeItemClusterNode.h"

#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterConfiguratorStyle.h"
#include "DisplayClusterConfiguratorBlueprintEditor.h"
#include "DisplayClusterConfiguratorPropertyUtils.h"
#include "ISinglePropertyView.h"
#include "ClusterConfiguration/DisplayClusterConfiguratorClusterEditorUtils.h"
#include "ClusterConfiguration/DisplayClusterConfiguratorClusterUtils.h"
#include "Views/DragDrop/DisplayClusterConfiguratorValidatedDragDropOp.h"
#include "Views/DragDrop/DisplayClusterConfiguratorClusterNodeDragDropOp.h"
#include "Views/DragDrop/DisplayClusterConfiguratorViewportDragDropOp.h"
#include "Views/TreeViews/Cluster/DisplayClusterConfiguratorViewCluster.h"
#include "Views/TreeViews/Cluster/TreeItems/DisplayClusterConfiguratorTreeItemViewport.h"
#include "Views/TreeViews/IDisplayClusterConfiguratorViewTree.h"

#include "ScopedTransaction.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "ISinglePropertyView.h"

#define LOCTEXT_NAMESPACE "FDisplayClusterConfiguratorTreeItemClusterNode"

FDisplayClusterConfiguratorTreeItemClusterNode::FDisplayClusterConfiguratorTreeItemClusterNode(const FName& InName,
	const TSharedRef<IDisplayClusterConfiguratorViewTree>& InViewTree,
	const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& InToolkit,
	UObject* InObjectToEdit,
	UDisplayClusterConfigurationHostDisplayData* InHostObject)
	: FDisplayClusterConfiguratorTreeItemCluster(InName, InViewTree, InToolkit, InObjectToEdit, "DisplayClusterConfigurator.TreeItems.ClusterNode", false),
	HostObject(InHostObject)
{ }

UDisplayClusterConfigurationHostDisplayData* FDisplayClusterConfiguratorTreeItemClusterNode::GetHostDisplayData()
{
	return HostObject.Get();
}

void FDisplayClusterConfiguratorTreeItemClusterNode::SelectHostDisplayData()
{
	// Select the host display data from the toolkit, allowing it to show up in the details panel.
	TArray<UObject*> Objects { HostObject.Get() };
	ToolkitPtr.Pin()->SelectObjects(Objects);
}

void FDisplayClusterConfiguratorTreeItemClusterNode::SetVisible(bool bIsVisible)
{
	UDisplayClusterConfigurationClusterNode* ClusterNode = GetObjectChecked<UDisplayClusterConfigurationClusterNode>();

	// Use SaveToTransactionBuffer to avoid marking the package as dirty
	SaveToTransactionBuffer(ClusterNode, false);

	const TSharedPtr<ISinglePropertyView> PropertyView = UE::DisplayClusterConfiguratorPropertyUtils::GetPropertyView(
		ClusterNode, GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationClusterNode, bIsVisible));
	
	PropertyView->GetPropertyHandle()->SetValue(bIsVisible);

	// Set the visible state of this cluster node's children to match it.
	for (TSharedPtr<IDisplayClusterConfiguratorTreeItem> Child : Children)
	{
		TSharedPtr<FDisplayClusterConfiguratorTreeItemViewport> ViewportTreeItem = StaticCastSharedPtr<FDisplayClusterConfiguratorTreeItemViewport>(Child);
		if (ViewportTreeItem.IsValid())
		{
			ViewportTreeItem->SetVisible(ClusterNode->bIsVisible);
		}
	}
}

void FDisplayClusterConfiguratorTreeItemClusterNode::SetUnlocked(bool bIsUnlocked)
{
	UDisplayClusterConfigurationClusterNode* ClusterNode = GetObjectChecked<UDisplayClusterConfigurationClusterNode>();

	// Use SaveToTransactionBuffer to avoid marking the package as dirty
	SaveToTransactionBuffer(ClusterNode, false);

	const TSharedPtr<ISinglePropertyView> PropertyView = UE::DisplayClusterConfiguratorPropertyUtils::GetPropertyView(
		ClusterNode, GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationClusterNode, bIsUnlocked));

	PropertyView->GetPropertyHandle()->SetValue(bIsUnlocked);

	// Set the enabled state of this cluster node's children to match it.
	for (TSharedPtr<IDisplayClusterConfiguratorTreeItem> Child : Children)
	{
		TSharedPtr<FDisplayClusterConfiguratorTreeItemViewport> ViewportTreeItem = StaticCastSharedPtr<FDisplayClusterConfiguratorTreeItemViewport>(Child);
		if (ViewportTreeItem.IsValid())
		{
			ViewportTreeItem->SetUnlocked(ClusterNode->bIsUnlocked);
		}
	}
}

void FDisplayClusterConfiguratorTreeItemClusterNode::DeleteItem() const
{
	UDisplayClusterConfigurationClusterNode* ClusterNode = GetObjectChecked<UDisplayClusterConfigurationClusterNode>();
	UE::DisplayClusterConfiguratorClusterUtils::RemoveClusterNodeFromCluster(ClusterNode);
}

FReply FDisplayClusterConfiguratorTreeItemClusterNode::HandleDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	TSharedRef<IDisplayClusterConfiguratorViewTree> ViewTree = GetConfiguratorTree();
	if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>> SelectedItems = ViewTree->GetSelectedItems();

		TSharedPtr<FDisplayClusterConfiguratorTreeItemClusterNode> This = SharedThis(this);
		if (!SelectedItems.Contains(This))
		{
			SelectedItems.Insert(This, 0);
		}

		TArray<UObject*> SelectedObjects;
		for (TSharedPtr<IDisplayClusterConfiguratorTreeItem> SelectedItem : SelectedItems)
		{
			SelectedObjects.Add(SelectedItem->GetObject());
		}

		TSharedPtr<FDragDropOperation> DragDropOp = UE::DisplayClusterConfiguratorClusterEditorUtils::MakeDragDropOperation(SelectedObjects);

		if (DragDropOp.IsValid())
		{
			return FReply::Handled().BeginDragDrop(DragDropOp.ToSharedRef());
		}
		else
		{
			return FReply::Unhandled();
		}
	}

	return FReply::Unhandled();
}

void FDisplayClusterConfiguratorTreeItemClusterNode::HandleDragEnter(const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FDisplayClusterConfiguratorViewportDragDropOp> ViewportDragDropOp = DragDropEvent.GetOperationAs<FDisplayClusterConfiguratorViewportDragDropOp>();
	if (ViewportDragDropOp.IsValid())
	{
		FText ErrorMessage;
		if (!CanDropViewports(ViewportDragDropOp, ErrorMessage))
		{
			ViewportDragDropOp->SetDropAsInvalid(ErrorMessage);
		}
		else
		{
			ViewportDragDropOp->SetDropAsValid(FText::Format(LOCTEXT("ViewportDragDropOp_Message", "Move to Cluster Node {0}"), FText::FromName(Name)));
		}

		return;
	}

	TSharedPtr<FDisplayClusterConfiguratorClusterNodeDragDropOp> ClusterNodeDragDropOp = DragDropEvent.GetOperationAs<FDisplayClusterConfiguratorClusterNodeDragDropOp>();
	if (ClusterNodeDragDropOp.IsValid())
	{
		// In the case of a cluster node drag drop, we are attempting to drop the dragged cluster nodes as siblings of this cluster node. We can pass the drop event to this
		// cluster node's parent host tree item to handle.
		TSharedPtr<IDisplayClusterConfiguratorTreeItem> ParentItem = Parent.Pin();
		if (ParentItem.IsValid())
		{
			ParentItem->HandleDragEnter(DragDropEvent);
			return;
		}
	}

	FDisplayClusterConfiguratorTreeItemCluster::HandleDragEnter(DragDropEvent);
}

void FDisplayClusterConfiguratorTreeItemClusterNode::HandleDragLeave(const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FDisplayClusterConfiguratorValidatedDragDropOp> ValidatedDragDropOp = DragDropEvent.GetOperationAs<FDisplayClusterConfiguratorValidatedDragDropOp>();
	if (ValidatedDragDropOp.IsValid())
	{
		// Use an opt-in policy for drag-drop operations, always marking it as invalid until another widget marks it as a valid drop operation
		ValidatedDragDropOp->SetDropAsInvalid();
		return;
	}

	FDisplayClusterConfiguratorTreeItemCluster::HandleDragLeave(DragDropEvent);
}

TOptional<EItemDropZone> FDisplayClusterConfiguratorTreeItemClusterNode::HandleCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<IDisplayClusterConfiguratorTreeItem> TargetItem)
{
	TSharedPtr<FDisplayClusterConfiguratorViewportDragDropOp> ViewportDragDropOp = DragDropEvent.GetOperationAs<FDisplayClusterConfiguratorViewportDragDropOp>();
	if (ViewportDragDropOp.IsValid())
	{
		if (ViewportDragDropOp->CanBeDropped())
		{
			return EItemDropZone::OntoItem;
		}
		else
		{
			return TOptional<EItemDropZone>();
		}
	}

	TSharedPtr<FDisplayClusterConfiguratorClusterNodeDragDropOp> ClusterNodeDragDropOp = DragDropEvent.GetOperationAs<FDisplayClusterConfiguratorClusterNodeDragDropOp>();
	if (ClusterNodeDragDropOp.IsValid())
	{
		if (ClusterNodeDragDropOp->CanBeDropped())
		{
			return EItemDropZone::BelowItem;
		}
		else
		{
			return TOptional<EItemDropZone>();
		}
	}

	return FDisplayClusterConfiguratorTreeItemCluster::HandleCanAcceptDrop(DragDropEvent, DropZone, TargetItem);
}

FReply FDisplayClusterConfiguratorTreeItemClusterNode::HandleAcceptDrop(const FDragDropEvent& DragDropEvent, TSharedPtr<IDisplayClusterConfiguratorTreeItem> TargetItem)
{
	TSharedPtr<FDisplayClusterConfiguratorViewportDragDropOp> ViewportDragDropOp = DragDropEvent.GetOperationAs<FDisplayClusterConfiguratorViewportDragDropOp>();
	if (ViewportDragDropOp.IsValid())
	{
		FScopedTransaction Transaction(FText::Format(LOCTEXT("DropViewports", "Drop {0}|plural(one=Viewport, other=Viewports)"), ViewportDragDropOp->GetDraggedViewports().Num()));

		UDisplayClusterConfigurationClusterNode* ClusterNode = GetObjectChecked<UDisplayClusterConfigurationClusterNode>();
		bool bClusterModified = false;
		for (TWeakObjectPtr<UDisplayClusterConfigurationViewport> Viewport : ViewportDragDropOp->GetDraggedViewports())
		{
			if (Viewport.IsValid())
			{
				// Skip any viewports that already belong to this cluster node
				if (ClusterNode->Viewports.FindKey(Viewport.Get()))
				{
					continue;
				}

				UE::DisplayClusterConfiguratorClusterUtils::AddViewportToClusterNode(Viewport.Get(), ClusterNode);
				bClusterModified = true;
			}
		}

		if (bClusterModified)
		{
			ClusterNode->MarkPackageDirty();
			ToolkitPtr.Pin()->ClusterChanged();

			return FReply::Handled();
		}
		else
		{
			Transaction.Cancel();
		}
	}

	TSharedPtr<FDisplayClusterConfiguratorClusterNodeDragDropOp> ClusterNodeDragDropOp = DragDropEvent.GetOperationAs<FDisplayClusterConfiguratorClusterNodeDragDropOp>();
	if (ClusterNodeDragDropOp.IsValid())
	{
		// In the case of a cluster node drag drop, we are attempting to drop the dragged cluster nodes as siblings of this cluster node. We can pass the drop event to this
		// cluster node's parent host tree item to handle.
		TSharedPtr<IDisplayClusterConfiguratorTreeItem> ParentItem = Parent.Pin();
		if (ParentItem.IsValid())
		{
			return ParentItem->HandleAcceptDrop(DragDropEvent, TargetItem);
		}
	}

	return FDisplayClusterConfiguratorTreeItemCluster::HandleAcceptDrop(DragDropEvent, TargetItem);
}

void FDisplayClusterConfiguratorTreeItemClusterNode::FillItemColumn(TSharedPtr<SHorizontalBox> Box, const TAttribute<FText>& FilterText, FIsSelected InIsSelected)
{
	FDisplayClusterConfiguratorTreeItemCluster::FillItemColumn(Box, FilterText, InIsSelected);

	Box->AddSlot()
		.AutoWidth()
		.Padding(2, 0, 0, 0)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(NSLOCTEXT("FDisplayClusterConfiguratorTreeItemClusterNode", "PrimaryText", "(Primary)"))
			.Visibility(this, &FDisplayClusterConfiguratorTreeItemClusterNode::GetPrimaryLabelVisibility)
		];
}

void FDisplayClusterConfiguratorTreeItemClusterNode::OnDisplayNameCommitted(const FText& NewText, ETextCommit::Type CommitInfo)
{
	// A cluster node's "display name" is simply the key that it is stored under in its parent cluster. In order to change it,
	// the cluster node will need to be removed from the TMap and re-added under the new name/key.
	UDisplayClusterConfigurationClusterNode* ClusterNode = GetObjectChecked<UDisplayClusterConfigurationClusterNode>();

	FScopedTransaction Transaction(LOCTEXT("RenameClusterNode", "Rename Cluster Node"));
	FString NewName = NewText.ToString();

	if (UE::DisplayClusterConfiguratorClusterUtils::RenameClusterNode(ClusterNode, NewName))
	{
		Name = *NewName;

		ClusterNode->MarkPackageDirty();
		ToolkitPtr.Pin()->ClusterChanged();
	}
	else
	{
		Transaction.Cancel();
	}
}

EVisibility FDisplayClusterConfiguratorTreeItemClusterNode::GetPrimaryLabelVisibility() const
{
	UDisplayClusterConfigurationClusterNode* ClusterNode = GetObjectChecked<UDisplayClusterConfigurationClusterNode>();
	const bool bIsPrimary = UE::DisplayClusterConfiguratorClusterUtils::IsClusterNodePrimary(ClusterNode);

	return bIsPrimary ? EVisibility::Visible : EVisibility::Collapsed;
}

bool FDisplayClusterConfiguratorTreeItemClusterNode::CanDropViewports(TSharedPtr<FDisplayClusterConfiguratorViewportDragDropOp> ViewportDragDropOp, FText& OutErrorMessage) const
{
	if (ViewportDragDropOp.IsValid())
	{
		UDisplayClusterConfigurationClusterNode* ClusterNode = GetObjectChecked<UDisplayClusterConfigurationClusterNode>();
		bool bCanAcceptViewports = true;

		// Don't allow dropping the dragged viewports into this node if at least one of them has the same name/key
		for (TWeakObjectPtr<UDisplayClusterConfigurationViewport> Viewport : ViewportDragDropOp->GetDraggedViewports())
		{
			if (Viewport.IsValid())
			{
				FString ViewportName = UE::DisplayClusterConfiguratorClusterUtils::GetViewportName(Viewport.Get());

				if (ClusterNode->Viewports.Contains(ViewportName) && ClusterNode->Viewports[ViewportName] != Viewport.Get())
				{
					bCanAcceptViewports = false;
					OutErrorMessage = LOCTEXT("ViewportDuplicateName_ErrorMessage", "Can't drop viewport with duplicate name");
					break;
				}
			}
		}

		return bCanAcceptViewports;
	}

	return false;
}

bool FDisplayClusterConfiguratorTreeItemClusterNode::IsClusterItemVisible() const
{
	UDisplayClusterConfigurationClusterNode* ClusterNode = GetObjectChecked<UDisplayClusterConfigurationClusterNode>();
	return ClusterNode->bIsVisible;
}

void FDisplayClusterConfiguratorTreeItemClusterNode::ToggleClusterItemVisibility()
{
	const FScopedTransaction Transaction(LOCTEXT("ToggleClusterNodeVisibility", "Toggle Cluster Node Visibility"));

	UDisplayClusterConfigurationClusterNode* ClusterNode = GetObjectChecked<UDisplayClusterConfigurationClusterNode>();

	// Use SaveToTransactionBuffer to avoid marking the package as dirty
	SaveToTransactionBuffer(ClusterNode, false);

	const TSharedPtr<ISinglePropertyView> PropertyView = UE::DisplayClusterConfiguratorPropertyUtils::GetPropertyView(
		ClusterNode, GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationClusterNode, bIsVisible));

	PropertyView->GetPropertyHandle()->SetValue(!ClusterNode->bIsVisible);

	// Set the visible state of this cluster node's children to match it.
	for (TSharedPtr<IDisplayClusterConfiguratorTreeItem> Child : Children)
	{
		TSharedPtr<FDisplayClusterConfiguratorTreeItemViewport> ViewportTreeItem = StaticCastSharedPtr<FDisplayClusterConfiguratorTreeItemViewport>(Child);

		if (ViewportTreeItem.IsValid())
		{
			ViewportTreeItem->SetVisible(ClusterNode->bIsVisible);
		}
	}
}

bool FDisplayClusterConfiguratorTreeItemClusterNode::IsClusterItemUnlocked() const
{
	UDisplayClusterConfigurationClusterNode* ClusterNode = GetObjectChecked<UDisplayClusterConfigurationClusterNode>();
	return ClusterNode->bIsUnlocked;
}

void FDisplayClusterConfiguratorTreeItemClusterNode::ToggleClusterItemLock()
{
	const FScopedTransaction Transaction(LOCTEXT("ToggleClusterNodeLock", "Toggle Cluster Node Lock"));

	UDisplayClusterConfigurationClusterNode* ClusterNode = GetObjectChecked<UDisplayClusterConfigurationClusterNode>();

	// Use SaveToTransactionBuffer to avoid marking the package as dirty
	SaveToTransactionBuffer(ClusterNode, false);

	const TSharedPtr<ISinglePropertyView> PropertyView = UE::DisplayClusterConfiguratorPropertyUtils::GetPropertyView(
		ClusterNode, GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationClusterNode, bIsUnlocked));

	PropertyView->GetPropertyHandle()->SetValue(!ClusterNode->bIsUnlocked);

	// Set the enabled state of this cluster node's children to match it.
	for (TSharedPtr<IDisplayClusterConfiguratorTreeItem> Child : Children)
	{
		TSharedPtr<FDisplayClusterConfiguratorTreeItemViewport> ViewportTreeItem = StaticCastSharedPtr<FDisplayClusterConfiguratorTreeItemViewport>(Child);

		if (ViewportTreeItem.IsValid())
		{
			ViewportTreeItem->SetUnlocked(ClusterNode->bIsUnlocked);
		}
	}
}

FSlateColor FDisplayClusterConfiguratorTreeItemClusterNode::GetClusterItemGroupColor() const
{
	if (!HostObject.IsValid())
	{
		return FLinearColor::Transparent;
	}

	if (ToolkitPtr.Pin()->IsObjectSelected(HostObject.Get()))
	{
		return FDisplayClusterConfiguratorStyle::Get().GetColor("DisplayClusterConfigurator.Node.Color.Selected");
	}
	else
	{
		return HostObject.Get()->Color;
	}
}

FReply FDisplayClusterConfiguratorTreeItemClusterNode::OnClusterItemGroupClicked(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (!HostObject.IsValid())
	{
		return FReply::Unhandled();
	}

	SelectHostDisplayData();

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE