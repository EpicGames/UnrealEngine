// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfiguratorTreeItemHost.h"

#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterConfiguratorStyle.h"
#include "DisplayClusterConfiguratorBlueprintEditor.h"
#include "ClusterConfiguration/DisplayClusterConfiguratorClusterUtils.h"
#include "Views/DragDrop/DisplayClusterConfiguratorValidatedDragDropOp.h"
#include "Views/DragDrop/DisplayClusterConfiguratorClusterNodeDragDropOp.h"
#include "Views/TreeViews/Cluster/DisplayClusterConfiguratorViewCluster.h"
#include "Views/TreeViews/Cluster/TreeItems/DisplayClusterConfiguratorTreeItemClusterNode.h"

#include "ScopedTransaction.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"

#define LOCTEXT_NAMESPACE "FDisplayClusterConfiguratorTreeItemHost"

FDisplayClusterConfiguratorTreeItemHost::FDisplayClusterConfiguratorTreeItemHost(const FName& InName,
	const TSharedRef<IDisplayClusterConfiguratorViewTree>& InViewTree,
	const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& InToolkit,
	UObject* InObjectToEdit)
	: FDisplayClusterConfiguratorTreeItemCluster(InName, InViewTree, InToolkit, InObjectToEdit, "DisplayClusterConfigurator.TreeItems.Host", false)
{ }

void FDisplayClusterConfiguratorTreeItemHost::Initialize()
{
	UDisplayClusterConfigurationHostDisplayData* HostDisplayData = GetObjectChecked<UDisplayClusterConfigurationHostDisplayData>();
	HostDisplayData->OnPostEditChangeChainProperty.Add(UDisplayClusterConfigurationHostDisplayData::FOnPostEditChangeChainProperty::FDelegate::CreateSP(this, &FDisplayClusterConfiguratorTreeItemHost::OnPostEditChangeChainProperty));
}

void FDisplayClusterConfiguratorTreeItemHost::DeleteItem() const
{
	UDisplayClusterConfigurationHostDisplayData* Host = GetObjectChecked<UDisplayClusterConfigurationHostDisplayData>();
	if (UDisplayClusterConfigurationCluster* Cluster = Cast<UDisplayClusterConfigurationCluster>(Host->GetOuter()))
	{
		FString HostAddress = UE::DisplayClusterConfiguratorClusterUtils::GetAddressForHost(Host);
		UE::DisplayClusterConfiguratorClusterUtils::RemoveHost(Cluster, HostAddress);
	}
}

void FDisplayClusterConfiguratorTreeItemHost::HandleDragEnter(const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FDisplayClusterConfiguratorClusterNodeDragDropOp> ClusterNodeDragDropOp = DragDropEvent.GetOperationAs<FDisplayClusterConfiguratorClusterNodeDragDropOp>();
	if (ClusterNodeDragDropOp.IsValid())
	{
		FText ErrorMessage;
		if (!CanDropClusterNodes(ClusterNodeDragDropOp, ErrorMessage))
		{
			ClusterNodeDragDropOp->SetDropAsInvalid(ErrorMessage);
		}
		else
		{
			ClusterNodeDragDropOp->SetDropAsValid(FText::Format(LOCTEXT("ClusterNodeDragDropOp_Message", "Move to Host {0}"), FText::FromName(Name)));
		}

		return;
	}

	FDisplayClusterConfiguratorTreeItemCluster::HandleDragEnter(DragDropEvent);
}

void FDisplayClusterConfiguratorTreeItemHost::HandleDragLeave(const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FDisplayClusterConfiguratorValidatedDragDropOp> ValidatedDragDropOp = DragDropEvent.GetOperationAs<FDisplayClusterConfiguratorValidatedDragDropOp>();
	if (ValidatedDragDropOp.IsValid())
	{
		// Use an opt-in policy for drag-drop operations, always marking it as invalid until another widget marks it as a valid drop operation
		ValidatedDragDropOp->SetDropAsInvalid();
		return;
	}

	FDisplayClusterConfiguratorTreeItemCluster::HandleDragLeave(DragDropEvent);
}

TOptional<EItemDropZone> FDisplayClusterConfiguratorTreeItemHost::HandleCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<IDisplayClusterConfiguratorTreeItem> TargetItem)
{
	TSharedPtr<FDisplayClusterConfiguratorClusterNodeDragDropOp> ClusterNodeDragDropOp = DragDropEvent.GetOperationAs<FDisplayClusterConfiguratorClusterNodeDragDropOp>();
	if (ClusterNodeDragDropOp.IsValid())
	{
		if (ClusterNodeDragDropOp->CanBeDropped())
		{
			return EItemDropZone::OntoItem;
		}
		else
		{
			return TOptional<EItemDropZone>();
		}
	}

	return FDisplayClusterConfiguratorTreeItemCluster::HandleCanAcceptDrop(DragDropEvent, DropZone, TargetItem);
}

FReply FDisplayClusterConfiguratorTreeItemHost::HandleAcceptDrop(const FDragDropEvent& DragDropEvent, TSharedPtr<IDisplayClusterConfiguratorTreeItem> TargetItem)
{
	TSharedPtr<FDisplayClusterConfiguratorClusterNodeDragDropOp> ClusterNodeDragDropOp = DragDropEvent.GetOperationAs<FDisplayClusterConfiguratorClusterNodeDragDropOp>();
	if (ClusterNodeDragDropOp.IsValid())
	{
		FScopedTransaction Transaction(FText::Format(LOCTEXT("DropClusterNodes", "Drop {0}|plural(one=Cluster Node, other=Cluster Nodes)"), ClusterNodeDragDropOp->GetDraggedClusterNodes().Num()));

		UDisplayClusterConfigurationHostDisplayData* HostDisplayData = GetObjectChecked<UDisplayClusterConfigurationHostDisplayData>();
		const FString HostAddress = UE::DisplayClusterConfiguratorClusterUtils::GetAddressForHost(HostDisplayData);
		bool bClusterModified = false;
		for (TWeakObjectPtr<UDisplayClusterConfigurationClusterNode> ClusterNode : ClusterNodeDragDropOp->GetDraggedClusterNodes())
		{
			if (ClusterNode.IsValid())
			{
				// Skip any cluster nodes that already belong to this host
				if (ClusterNode->Host == HostAddress)
				{
					continue;
				}

				ClusterNode->Modify();
				ClusterNode->Host = HostAddress;
				bClusterModified = true;
			}
		}

		if (bClusterModified)
		{
			HostDisplayData->MarkPackageDirty();
			ToolkitPtr.Pin()->ClusterChanged();

			return FReply::Handled();
		}
		else
		{
			Transaction.Cancel();
		}
	}

	return FDisplayClusterConfiguratorTreeItemCluster::HandleAcceptDrop(DragDropEvent, TargetItem);
}

FName FDisplayClusterConfiguratorTreeItemHost::GetAttachName() const
{
	// Use the host's address as the attach name, to ensure that the name is unique (the host's HostName property has no uniqueness enforced on it)
	// so that child cluster nodes are correctly associated with their parent host nodes
	UDisplayClusterConfigurationHostDisplayData* Host = GetObjectChecked<UDisplayClusterConfigurationHostDisplayData>();
	FName AttachName = *UE::DisplayClusterConfiguratorClusterUtils::GetAddressForHost(Host);
	return AttachName;
}


void FDisplayClusterConfiguratorTreeItemHost::SetItemHidden(bool bIsHidden)
{
	UDisplayClusterConfigurationHostDisplayData* Host = GetObjectChecked<UDisplayClusterConfigurationHostDisplayData>();

	// Use SaveToTransactionBuffer to avoid marking the package as dirty
	SaveToTransactionBuffer(Host, false);
	Host->bIsVisible = !bIsHidden;

	// Set the visible state of this host's children to match it.
	for (TSharedPtr<IDisplayClusterConfiguratorTreeItem> Child : Children)
	{
		TSharedPtr<FDisplayClusterConfiguratorTreeItemClusterNode> ClusterNodeTreeItem = StaticCastSharedPtr<FDisplayClusterConfiguratorTreeItemClusterNode>(Child);
		if (ClusterNodeTreeItem.IsValid())
		{
			ClusterNodeTreeItem->SetVisible(Host->bIsVisible);
		}
	}
}

void FDisplayClusterConfiguratorTreeItemHost::FillItemColumn(TSharedPtr<SHorizontalBox> Box, const TAttribute<FText>& FilterText, FIsSelected InIsSelected)
{
	FDisplayClusterConfiguratorTreeItemCluster::FillItemColumn(Box, FilterText, InIsSelected);

	Box->AddSlot()
		.AutoWidth()
		.Padding(2, 0, 0, 0)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(this, &FDisplayClusterConfiguratorTreeItemHost::GetHostAddress)
		];
}

void FDisplayClusterConfiguratorTreeItemHost::OnDisplayNameCommitted(const FText& NewText, ETextCommit::Type CommitInfo)
{
	const FScopedTransaction Transaction(LOCTEXT("RenameHost", "Rename Host"));
	
	UDisplayClusterConfigurationHostDisplayData* Host = GetObjectChecked<UDisplayClusterConfigurationHostDisplayData>();

	Host->Modify();
	Host->HostName = NewText;
	Host->MarkPackageDirty();

	Name = *NewText.ToString();
}

bool FDisplayClusterConfiguratorTreeItemHost::IsClusterItemVisible() const
{
	UDisplayClusterConfigurationHostDisplayData* Host = GetObjectChecked<UDisplayClusterConfigurationHostDisplayData>();
	return Host->bIsVisible;
}

void FDisplayClusterConfiguratorTreeItemHost::ToggleClusterItemVisibility()
{
	const FScopedTransaction Transaction(LOCTEXT("ToggleHostVisibility", "Set Host Visibility"));

	UDisplayClusterConfigurationHostDisplayData* Host = GetObjectChecked<UDisplayClusterConfigurationHostDisplayData>();

	// Use SaveToTransactionBuffer to avoid marking the package as dirty
	SaveToTransactionBuffer(Host, false);
	Host->bIsVisible = !Host->bIsVisible;

	// Set the visible state of this host's children to match it.
	for (TSharedPtr<IDisplayClusterConfiguratorTreeItem> Child : Children)
	{
		TSharedPtr<FDisplayClusterConfiguratorTreeItemClusterNode> ClusterNodeTreeItem = StaticCastSharedPtr<FDisplayClusterConfiguratorTreeItemClusterNode>(Child);
		if (ClusterNodeTreeItem.IsValid())
		{
			ClusterNodeTreeItem->SetVisible(Host->bIsVisible);
		}
	}
}

bool FDisplayClusterConfiguratorTreeItemHost::IsClusterItemUnlocked() const
{
	UDisplayClusterConfigurationHostDisplayData* Host = GetObjectChecked<UDisplayClusterConfigurationHostDisplayData>();
	return Host->bIsUnlocked;
}

void FDisplayClusterConfiguratorTreeItemHost::ToggleClusterItemLock()
{
	const FScopedTransaction Transaction(LOCTEXT("ToggleHostEnabled", "Set Host Enabled"));

	UDisplayClusterConfigurationHostDisplayData* Host = GetObjectChecked<UDisplayClusterConfigurationHostDisplayData>();

	// Use SaveToTransactionBuffer to avoid marking the package as dirty
	SaveToTransactionBuffer(Host, false);
	Host->bIsUnlocked = !Host->bIsUnlocked;

	// Set the enabled state of this host's children to match it.
	for (TSharedPtr<IDisplayClusterConfiguratorTreeItem> Child : Children)
	{
		TSharedPtr<FDisplayClusterConfiguratorTreeItemClusterNode> ClusterNodeTreeItem = StaticCastSharedPtr<FDisplayClusterConfiguratorTreeItemClusterNode>(Child);
		if (ClusterNodeTreeItem.IsValid())
		{
			ClusterNodeTreeItem->SetUnlocked(Host->bIsUnlocked);
		}
	}
}

FSlateColor FDisplayClusterConfiguratorTreeItemHost::GetClusterItemGroupColor() const
{
	UDisplayClusterConfigurationHostDisplayData* Host = GetObjectChecked<UDisplayClusterConfigurationHostDisplayData>();
	if (ToolkitPtr.Pin()->IsObjectSelected(Host))
	{
		return FDisplayClusterConfiguratorStyle::Get().GetColor("DisplayClusterConfigurator.Node.Color.Selected");
	}
	else
	{
		return Host->Color;
	}
}

FReply FDisplayClusterConfiguratorTreeItemHost::OnClusterItemGroupClicked(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	UDisplayClusterConfigurationHostDisplayData* Host = GetObjectChecked<UDisplayClusterConfigurationHostDisplayData>();
	TArray<UObject*> Objects { Host };
	ToolkitPtr.Pin()->SelectObjects(Objects);

	return FReply::Handled();
}

FText FDisplayClusterConfiguratorTreeItemHost::GetHostAddress() const
{
	UDisplayClusterConfigurationHostDisplayData* Host = GetObjectChecked<UDisplayClusterConfigurationHostDisplayData>();
	return FText::Format(LOCTEXT("HostAddressFormattedLabel", "({0})"), FText::FromString(UE::DisplayClusterConfiguratorClusterUtils::GetAddressForHost(Host)));
}

bool FDisplayClusterConfiguratorTreeItemHost::CanDropClusterNodes(TSharedPtr<FDisplayClusterConfiguratorClusterNodeDragDropOp> ClusterNodeDragDropOp, FText& OutErrorMessage) const
{
	if (ClusterNodeDragDropOp.IsValid())
	{
		UDisplayClusterConfigurationHostDisplayData* Host = GetObjectChecked<UDisplayClusterConfigurationHostDisplayData>();
		bool bCanAcceptClusterNodes = true;

		return bCanAcceptClusterNodes;
	}

	return false;
}

void FDisplayClusterConfiguratorTreeItemHost::OnPostEditChangeChainProperty(const FPropertyChangedChainEvent& PropertyChangedEvent)
{
	UDisplayClusterConfigurationHostDisplayData* Host = GetObjectChecked<UDisplayClusterConfigurationHostDisplayData>();

	const FName& PropertyName = PropertyChangedEvent.PropertyChain.GetActiveMemberNode()->GetValue()->GetFName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationHostDisplayData, HostName))
	{
		Name = *Host->HostName.ToString();
	}
}

#undef LOCTEXT_NAMESPACE