// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterConfiguratorTreeItemViewport.h"

#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterConfiguratorStyle.h"
#include "DisplayClusterProjectionStrings.h"
#include "DisplayClusterConfiguratorBlueprintEditor.h"
#include "DisplayClusterConfiguratorPropertyUtils.h"
#include "ClusterConfiguration/DisplayClusterConfiguratorClusterEditorUtils.h"
#include "ClusterConfiguration/DisplayClusterConfiguratorClusterUtils.h"
#include "Views/DragDrop/DisplayClusterConfiguratorValidatedDragDropOp.h"
#include "Views/DragDrop/DisplayClusterConfiguratorViewportDragDropOp.h"
#include "Views/TreeViews/Cluster/DisplayClusterConfiguratorViewCluster.h"
#include "Views/TreeViews/Cluster/TreeItems/DisplayClusterConfiguratorTreeItemClusterNode.h"
#include "Views/TreeViews/IDisplayClusterConfiguratorViewTree.h"

#include "ScopedTransaction.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "ISinglePropertyView.h"

#define LOCTEXT_NAMESPACE "FDisplayClusterConfiguratorTreeItemViewport"

FDisplayClusterConfiguratorTreeItemViewport::FDisplayClusterConfiguratorTreeItemViewport(const FName& InName,
	const TSharedRef<IDisplayClusterConfiguratorViewTree>& InViewTree,
	const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& InToolkit,
	UObject* InObjectToEdit)
	: FDisplayClusterConfiguratorTreeItemCluster(InName, InViewTree, InToolkit, InObjectToEdit, "DisplayClusterConfigurator.TreeItems.Viewport", false)
{ }

void FDisplayClusterConfiguratorTreeItemViewport::SetVisible(bool bIsVisible)
{
	UDisplayClusterConfigurationViewport* Viewport = GetObjectChecked<UDisplayClusterConfigurationViewport>();

	// Use SaveToTransactionBuffer to avoid marking the package as dirty
	SaveToTransactionBuffer(Viewport, false);

	const TSharedPtr<ISinglePropertyView> PropertyView = UE::DisplayClusterConfiguratorPropertyUtils::GetPropertyView(
		Viewport, GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationViewport, bIsVisible));

	PropertyView->GetPropertyHandle()->SetValue(bIsVisible);
}

void FDisplayClusterConfiguratorTreeItemViewport::SetUnlocked(bool bIsUnlocked)
{
	UDisplayClusterConfigurationViewport* Viewport = GetObjectChecked<UDisplayClusterConfigurationViewport>();

	// Use SaveToTransactionBuffer to avoid marking the package as dirty
	SaveToTransactionBuffer(Viewport, false);

	const TSharedPtr<ISinglePropertyView> PropertyView = UE::DisplayClusterConfiguratorPropertyUtils::GetPropertyView(
		Viewport, GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationViewport, bIsUnlocked));
	
	PropertyView->GetPropertyHandle()->SetValue(bIsUnlocked);
}

void FDisplayClusterConfiguratorTreeItemViewport::OnSelection()
{
	FDisplayClusterConfiguratorTreeItemCluster::OnSelection();

	UDisplayClusterConfigurationViewport* Viewport = GetObjectChecked<UDisplayClusterConfigurationViewport>();

	TArray<FString> AssociatedComponents;
	if (Viewport->ProjectionPolicy.Type.Equals(DisplayClusterProjectionStrings::projection::Mesh, ESearchCase::IgnoreCase)
		&& Viewport->ProjectionPolicy.Parameters.Contains(DisplayClusterProjectionStrings::cfg::mesh::Component))
	{
		AssociatedComponents.Add(Viewport->ProjectionPolicy.Parameters[DisplayClusterProjectionStrings::cfg::mesh::Component]);
	}
	else if (Viewport->ProjectionPolicy.Type.Equals(DisplayClusterProjectionStrings::projection::Camera, ESearchCase::IgnoreCase)
		&& Viewport->ProjectionPolicy.Parameters.Contains(DisplayClusterProjectionStrings::cfg::camera::Component))
	{
		AssociatedComponents.Add(Viewport->ProjectionPolicy.Parameters[DisplayClusterProjectionStrings::cfg::camera::Component]);
	}

	if (AssociatedComponents.Num())
	{
		ToolkitPtr.Pin()->SelectAncillaryComponents(AssociatedComponents);
	}
}

void FDisplayClusterConfiguratorTreeItemViewport::DeleteItem() const
{
	UDisplayClusterConfigurationViewport* Viewport = GetObjectChecked<UDisplayClusterConfigurationViewport>();
	UE::DisplayClusterConfiguratorClusterUtils::RemoveViewportFromClusterNode(Viewport);
}

FReply FDisplayClusterConfiguratorTreeItemViewport::HandleDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	TSharedRef<IDisplayClusterConfiguratorViewTree> ViewTree = GetConfiguratorTree();
	if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>> SelectedItems = ViewTree->GetSelectedItems();

		TSharedPtr<FDisplayClusterConfiguratorTreeItemViewport> This = SharedThis(this);
		if (!SelectedItems.Contains(This))
		{
			SelectedItems.Insert(This, 0);
		}

		TArray<UObject*> SelectedObjects;
		for (TSharedPtr<IDisplayClusterConfiguratorTreeItem> SelectedItem : SelectedItems)
		{
			SelectedObjects.Add(SelectedItem->GetObject());
		}

		TSharedPtr<FDragDropOperation> DragDropOp = UE::DisplayClusterConfiguratorClusterEditorUtils::MakeDragDropOperation(SelectedObjects);

		if (DragDropOp.IsValid())
		{
			return FReply::Handled().BeginDragDrop(DragDropOp.ToSharedRef());
		}
		else
		{
			return FReply::Unhandled();
		}
	}

	return FReply::Unhandled();
}

void FDisplayClusterConfiguratorTreeItemViewport::HandleDragEnter(const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FDisplayClusterConfiguratorViewportDragDropOp> ViewportDragDropOp = DragDropEvent.GetOperationAs<FDisplayClusterConfiguratorViewportDragDropOp>();
	if (ViewportDragDropOp.IsValid())
	{
		// In the case of a viewport drag drop, we are attempting to drop the dragged viewports as siblings of this viewport. We can pass the drop event to this
		// viewport's parent cluster node tree item to handle.
		TSharedPtr<IDisplayClusterConfiguratorTreeItem> ParentItem = Parent.Pin();
		if (ParentItem.IsValid())
		{
			ParentItem->HandleDragEnter(DragDropEvent);
			return;
		}
	}

	FDisplayClusterConfiguratorTreeItemCluster::HandleDragEnter(DragDropEvent);
}

void FDisplayClusterConfiguratorTreeItemViewport::HandleDragLeave(const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FDisplayClusterConfiguratorValidatedDragDropOp> ValidatedDragDropOp = DragDropEvent.GetOperationAs<FDisplayClusterConfiguratorValidatedDragDropOp>();
	if (ValidatedDragDropOp.IsValid())
	{
		// Use an opt-in policy for drag-drop operations, always marking it as invalid until another widget marks it as a valid drop operation
		ValidatedDragDropOp->SetDropAsInvalid();
		return;
	}

	FDisplayClusterConfiguratorTreeItemCluster::HandleDragLeave(DragDropEvent);
}

TOptional<EItemDropZone> FDisplayClusterConfiguratorTreeItemViewport::HandleCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<IDisplayClusterConfiguratorTreeItem> TargetItem)
{
	TSharedPtr<FDisplayClusterConfiguratorViewportDragDropOp> ViewportDragDropOp = DragDropEvent.GetOperationAs<FDisplayClusterConfiguratorViewportDragDropOp>();
	if (ViewportDragDropOp.IsValid())
	{
		if (ViewportDragDropOp->CanBeDropped())
		{
			return EItemDropZone::BelowItem;
		}
		else
		{
			return TOptional<EItemDropZone>();
		}
	}

	return FDisplayClusterConfiguratorTreeItemCluster::HandleCanAcceptDrop(DragDropEvent, DropZone, TargetItem);
}

FReply FDisplayClusterConfiguratorTreeItemViewport::HandleAcceptDrop(const FDragDropEvent& DragDropEvent, TSharedPtr<IDisplayClusterConfiguratorTreeItem> TargetItem)
{
	TSharedPtr<FDisplayClusterConfiguratorViewportDragDropOp> ViewportDragDropOp = DragDropEvent.GetOperationAs<FDisplayClusterConfiguratorViewportDragDropOp>();
	if (ViewportDragDropOp.IsValid())
	{
		// In the case of a viewport drag drop, we are attempting to drop the dragged viewports as siblings of this viewport. We can pass the drop event to this
		// viewport's parent cluster node tree item to handle the dropping.
		TSharedPtr<IDisplayClusterConfiguratorTreeItem> ParentItem = Parent.Pin();
		if (ParentItem.IsValid())
		{
			return ParentItem->HandleAcceptDrop(DragDropEvent, TargetItem);
		}
	}

	return FDisplayClusterConfiguratorTreeItemCluster::HandleAcceptDrop(DragDropEvent, TargetItem);
}

void FDisplayClusterConfiguratorTreeItemViewport::OnDisplayNameCommitted(const FText& NewText, ETextCommit::Type CommitInfo)
{
	// A viewport's "display name" is simply the key that it is stored under in its parent cluster node. In order to change it,
	// the viewport will need to be removed from the TMap and re-added under the new name/key.
	UDisplayClusterConfigurationViewport* Viewport = GetObjectChecked<UDisplayClusterConfigurationViewport>();

	FScopedTransaction Transaction(NSLOCTEXT("FDisplayClusterConfiguratorViewCluster", "RenameClusterNode", "Rename Cluster Node"));
	FString NewName = NewText.ToString();

	if (UE::DisplayClusterConfiguratorClusterUtils::RenameViewport(Viewport, NewName))
	{
		Name = *NewName;

		Viewport->MarkPackageDirty();
		ToolkitPtr.Pin()->ClusterChanged();
	}
	else
	{
		Transaction.Cancel();
	}
}

bool FDisplayClusterConfiguratorTreeItemViewport::IsClusterItemVisible() const
{
	UDisplayClusterConfigurationViewport* Viewport = GetObjectChecked<UDisplayClusterConfigurationViewport>();
	return Viewport->bIsVisible;
}

void FDisplayClusterConfiguratorTreeItemViewport::ToggleClusterItemVisibility()
{
	const FScopedTransaction Transaction(LOCTEXT("ToggleViewportVisibility", "Toggle Viewport Visibilty"));

	UDisplayClusterConfigurationViewport* Viewport = GetObjectChecked<UDisplayClusterConfigurationViewport>();

	// Use SaveToTransactionBuffer to avoid marking the package as dirty
	SaveToTransactionBuffer(Viewport, false);

	const TSharedPtr<ISinglePropertyView> PropertyView = UE::DisplayClusterConfiguratorPropertyUtils::GetPropertyView(
		Viewport, GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationViewport, bIsVisible));

	PropertyView->GetPropertyHandle()->SetValue(!Viewport->bIsVisible);
}

bool FDisplayClusterConfiguratorTreeItemViewport::IsClusterItemUnlocked() const
{
	UDisplayClusterConfigurationViewport* Viewport = GetObjectChecked<UDisplayClusterConfigurationViewport>();
	return Viewport->bIsUnlocked;
}

void FDisplayClusterConfiguratorTreeItemViewport::ToggleClusterItemLock()
{
	const FScopedTransaction Transaction(LOCTEXT("ToggleViewportLock", "Toggle Viewport Lock"));

	UDisplayClusterConfigurationViewport* Viewport = GetObjectChecked<UDisplayClusterConfigurationViewport>();

	// Use SaveToTransactionBuffer to avoid marking the package as dirty
	SaveToTransactionBuffer(Viewport, false);

	const TSharedPtr<ISinglePropertyView> PropertyView = UE::DisplayClusterConfiguratorPropertyUtils::GetPropertyView(
		Viewport, GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationViewport, bIsUnlocked));

	PropertyView->GetPropertyHandle()->SetValue(!Viewport->bIsUnlocked);
}

FSlateColor FDisplayClusterConfiguratorTreeItemViewport::GetClusterItemGroupColor() const
{
	TSharedPtr<IDisplayClusterConfiguratorTreeItem> ParentItem = Parent.Pin();
	UDisplayClusterConfigurationHostDisplayData* HostDisplayData = nullptr;
	
	if (ParentItem.IsValid())
	{
		FDisplayClusterConfiguratorTreeItemClusterNode* ParentClusterNodeItem = StaticCast<FDisplayClusterConfiguratorTreeItemClusterNode*>(ParentItem.Get());
		HostDisplayData = ParentClusterNodeItem->GetHostDisplayData();
	}

	if (!HostDisplayData)
	{
		return FLinearColor::Transparent;
	}

	if (ToolkitPtr.Pin()->IsObjectSelected(HostDisplayData))
	{
		return FDisplayClusterConfiguratorStyle::Get().GetColor("DisplayClusterConfigurator.Node.Color.Selected");
	}
	else
	{
		return HostDisplayData->Color;
	}
}

FReply FDisplayClusterConfiguratorTreeItemViewport::OnClusterItemGroupClicked(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	TSharedPtr<IDisplayClusterConfiguratorTreeItem> ParentItem = GetParent();

	if (ParentItem.IsValid())
	{
		if (FDisplayClusterConfiguratorTreeItemClusterNode* ParentClusterNodeItem = StaticCast<FDisplayClusterConfiguratorTreeItemClusterNode*>(ParentItem.Get()))
		{
			ParentClusterNodeItem->SelectHostDisplayData();
			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

#undef LOCTEXT_NAMESPACE 