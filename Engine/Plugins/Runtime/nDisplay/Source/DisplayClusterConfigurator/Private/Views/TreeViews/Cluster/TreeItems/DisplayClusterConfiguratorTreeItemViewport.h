// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DisplayClusterConfiguratorTreeItemCluster.h"

class SButton;

class FDisplayClusterConfiguratorTreeItemViewport
	: public FDisplayClusterConfiguratorTreeItemCluster
{
public:
	NDISPLAY_TREE_ITEM_TYPE(FDisplayClusterConfiguratorTreeItemViewport, FDisplayClusterConfiguratorTreeItemCluster)

		FDisplayClusterConfiguratorTreeItemViewport(const FName& InName,
			const TSharedRef<IDisplayClusterConfiguratorViewTree>& InViewTree,
			const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& InToolkit,
			UObject* InObjectToEdit);

	void SetVisible(bool bIsVisible);
	void SetUnlocked(bool bIsUnlocked);

	//~ Begin FDisplayClusterConfiguratorTreeItem Interface
	virtual void OnSelection() override;

	virtual void DeleteItem() const override;
	virtual bool CanHideItem() const override { return true; }
	virtual void SetItemHidden(bool bIsHidden) { SetVisible(!bIsHidden); }

	virtual FReply HandleDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void HandleDragEnter(const FDragDropEvent& DragDropEvent) override;
	virtual void HandleDragLeave(const FDragDropEvent& DragDropEvent) override;
	virtual TOptional<EItemDropZone> HandleCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<IDisplayClusterConfiguratorTreeItem> TargetItem) override;
	virtual FReply HandleAcceptDrop(const FDragDropEvent& DragDropEvent, TSharedPtr<IDisplayClusterConfiguratorTreeItem> TargetItem) override;

protected:
	virtual void OnDisplayNameCommitted(const FText& NewText, ETextCommit::Type CommitInfo) override;

	virtual bool CanClusterItemBeHidden() const override { return true; }
	virtual bool CanClusterItemBeLocked() const override { return true; }
	virtual bool IsClusterItemGrouped() const override { return true; }

	virtual bool IsClusterItemVisible() const override;
	virtual bool IsClusterItemUnlocked() const override;

	virtual void ToggleClusterItemVisibility() override;
	virtual void ToggleClusterItemLock() override;

	virtual FSlateColor GetClusterItemGroupColor() const override;
	virtual FReply OnClusterItemGroupClicked(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	//~ End FDisplayClusterConfiguratorTreeItem Interface

private:
	FSlateColor GetHostColor() const;
	FReply OnHostClicked(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
};