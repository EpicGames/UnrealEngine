// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/TreeViews/DisplayClusterConfiguratorTreeBuilder.h"
#include "Views/TreeViews/IDisplayClusterConfiguratorTreeItem.h"

void FDisplayClusterConfiguratorTreeBuilderOutput::Add(const TSharedPtr<IDisplayClusterConfiguratorTreeItem>& InItem, const FName& InParentName, const FName& InParentType, bool bAddToHead)
{
	Add(InItem, InParentName, TArray<FName, TInlineAllocator<1>>({ InParentType }), bAddToHead);
}

void FDisplayClusterConfiguratorTreeBuilderOutput::Add(const TSharedPtr<IDisplayClusterConfiguratorTreeItem>& InItem, const FName& InParentName, TArrayView<const FName> InParentTypes, bool bAddToHead)
{
	TSharedPtr<IDisplayClusterConfiguratorTreeItem> ParentItem = Find(InParentName, InParentTypes);
	if (ParentItem.IsValid())
	{
		InItem->SetParent(ParentItem);

		if (bAddToHead)
		{
			ParentItem->GetChildren().Insert(InItem, 0);
		}
		else
		{
			ParentItem->GetChildren().Add(InItem);
		}
	}
	else
	{
		if (bAddToHead)
		{
			Items.Insert(InItem, 0);
		}
		else
		{
			Items.Add(InItem);
		}
	}

	LinearItems.Add(InItem);

	InItem->Initialize();
}

TSharedPtr<IDisplayClusterConfiguratorTreeItem> FDisplayClusterConfiguratorTreeBuilderOutput::Find(const FName& InName, const FName& InType) const
{
	return Find(InName, TArray<FName, TInlineAllocator<1>>({ InType }));
}

TSharedPtr<IDisplayClusterConfiguratorTreeItem> FDisplayClusterConfiguratorTreeBuilderOutput::Find(const FName& InName, TArrayView<const FName> InTypes) const
{
	for (const TSharedPtr<IDisplayClusterConfiguratorTreeItem>& Item : LinearItems)
	{
		bool bPassesType = (InTypes.Num() == 0);
		for (const FName& TypeName : InTypes)
		{
			if (Item->IsOfTypeByName(TypeName))
			{
				bPassesType = true;
				break;
			}
		}

		if (bPassesType && Item->GetAttachName() == InName)
		{
			return Item;
		}
	}

	return nullptr;
}

FDisplayClusterConfiguratorTreeBuilder::FDisplayClusterConfiguratorTreeBuilder(const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& InToolkit)
	: ToolkitPtr(InToolkit)
{
}

void FDisplayClusterConfiguratorTreeBuilder::Initialize(const TSharedRef<class IDisplayClusterConfiguratorViewTree>& InConfiguratorTree)
{
	ConfiguratorTreePtr = InConfiguratorTree;
}