// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Views/TreeViews/IDisplayClusterConfiguratorTreeBuilder.h"

class FDisplayClusterConfiguratorBlueprintEditor;
class IDisplayClusterConfiguratorTreeItem;
class IDisplayClusterConfiguratorViewTree;

enum class EDisplayClusterConfiguratorTreeFilterResult : uint8;

class FDisplayClusterConfiguratorTreeBuilder
	: public IDisplayClusterConfiguratorTreeBuilder
{
public:
	FDisplayClusterConfiguratorTreeBuilder(const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& InToolkit);

	//~ Begin IDisplayClusterConfiguratorTreeBuilder interface
	virtual void Initialize(const TSharedRef<IDisplayClusterConfiguratorViewTree>& InConfiguratorTree) override;
	//~ End IDisplayClusterConfiguratorTreeBuilder interface

protected:
	TWeakPtr<IDisplayClusterConfiguratorViewTree> ConfiguratorTreePtr;
	TWeakPtr<FDisplayClusterConfiguratorBlueprintEditor> ToolkitPtr;
};
