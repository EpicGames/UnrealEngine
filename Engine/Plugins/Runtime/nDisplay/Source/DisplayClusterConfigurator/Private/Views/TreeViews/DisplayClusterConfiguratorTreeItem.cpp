// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/TreeViews/DisplayClusterConfiguratorTreeItem.h"

#include "DisplayClusterConfiguratorStyle.h"
#include "DisplayClusterConfiguratorBlueprintEditor.h"
#include "Views/TreeViews/IDisplayClusterConfiguratorViewTree.h"
#include "Views/TreeViews/SDisplayClusterConfiguratorTreeItemRow.h"

#include "Misc/TextFilterExpressionEvaluator.h"
#include "UObject/Object.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Views/STableViewBase.h"

TSharedRef<ITableRow> FDisplayClusterConfiguratorTreeItem::MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, const TAttribute<FText>& InFilterText)
{
	return SNew(SDisplayClusterConfiguratorTreeItemRow, InOwnerTable)
		.FilterText(InFilterText)
		.Item(SharedThis(this));
}

TSharedRef<SWidget> FDisplayClusterConfiguratorTreeItem::GenerateWidgetForColumn(const FName& ColumnName, TSharedPtr<ITableRow> TableRow, const TAttribute<FText>& FilterText, FIsSelected InIsSelected)
{
	if (ColumnName == IDisplayClusterConfiguratorViewTree::Columns::Item)
	{
		TSharedPtr<SHorizontalBox> RowBox = SNew(SHorizontalBox);
		FillItemColumn(RowBox, FilterText, InIsSelected);

		return SNew(SBox)
			.MinDesiredHeight(20.0f)
			[
				SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(6, 0, 0, 0)
				[
					SNew(SExpanderArrow, TableRow)
					.IndentAmount(12)
				]

				+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				[
					RowBox.ToSharedRef()
				]
			];
	}

	return SNullWidget::NullWidget;
}

void FDisplayClusterConfiguratorTreeItem::GetParentObjectsRecursive(TArray<UObject*>& OutObjects) const
{
	if (TSharedPtr<IDisplayClusterConfiguratorTreeItem> ParentItem = GetParent())
	{
		OutObjects.Add(ParentItem->GetObject());
		ParentItem->GetParentObjectsRecursive(OutObjects);
	}
}

void FDisplayClusterConfiguratorTreeItem::GetChildrenObjectsRecursive(TArray<UObject*>& OutObjects) const
{
	const TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>>& ItemChildren = GetChildrenConst();
	for (const TSharedPtr<IDisplayClusterConfiguratorTreeItem>& TreeItem : ItemChildren)
	{
		OutObjects.Add(TreeItem->GetObject());
		TreeItem->GetChildrenObjectsRecursive(OutObjects);
	}
}

bool FDisplayClusterConfiguratorTreeItem::CanRenameItem() const
{
	return !bRoot;
}

bool FDisplayClusterConfiguratorTreeItem::CanDeleteItem() const
{
	return !bRoot;
}

bool FDisplayClusterConfiguratorTreeItem::CanDuplicateItem() const
{
	return !bRoot;
}

void FDisplayClusterConfiguratorTreeItem::RequestRename()
{
	if (CanRenameItem())
	{
		DisplayNameTextBlock->EnterEditingMode();
	}
}

bool FDisplayClusterConfiguratorTreeItem::IsChildOfRecursive(const TSharedRef<IDisplayClusterConfiguratorTreeItem>& InTreeItem) const
{
	if (TSharedPtr<IDisplayClusterConfiguratorTreeItem> ParentItem = GetParent())
	{
		if (ParentItem->GetRowItemName() == InTreeItem->GetRowItemName())
		{
			return true;
		}
		return ParentItem->IsChildOfRecursive(InTreeItem);
	}

	return false;
}

bool FDisplayClusterConfiguratorTreeItem::IsSelected()
{
	TArray<UObject*> SelectedObjects = ToolkitPtr.Pin()->GetSelectedObjects();

	UObject* const* SelectedObject = SelectedObjects.FindByPredicate([this](const UObject* InObject)
	{
		return InObject == GetObject();
	});

	if (SelectedObject != nullptr)
	{
		UObject* Obj = *SelectedObject;

		return Obj != nullptr;
	}

	return false;
}

EDisplayClusterConfiguratorTreeFilterResult FDisplayClusterConfiguratorTreeItem::ApplyFilter(const TSharedPtr<FTextFilterExpressionEvaluator>& TextFilter)
{
	EDisplayClusterConfiguratorTreeFilterResult Result = EDisplayClusterConfiguratorTreeFilterResult::Shown;

	if (TextFilter.IsValid())
	{
		if (TextFilter->TestTextFilter(FBasicStringFilterExpressionContext(GetRowItemName().ToString())))
		{
			Result = EDisplayClusterConfiguratorTreeFilterResult::ShownHighlighted;
		}
		else
		{
			Result = EDisplayClusterConfiguratorTreeFilterResult::Hidden;
		}
	}

	SetFilterResult(Result);

	return Result;
}

void FDisplayClusterConfiguratorTreeItem::FillItemColumn(TSharedPtr<SHorizontalBox> Box, const TAttribute<FText>& FilterText, FIsSelected InIsSelected)
{
	Box->AddSlot()
		.AutoWidth()
		.Padding(FMargin(0.f, 1.f, 6.f, 1.f))
		.VAlign(VAlign_Center)
		[
			SNew(SBox)
			.WidthOverride(16)
			.HeightOverride(16)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(FDisplayClusterConfiguratorStyle::Get().GetBrush(*GetIconStyle()))
			]
		];

	Box->AddSlot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SAssignNew(DisplayNameTextBlock, SInlineEditableTextBlock)
			.Text(this, &FDisplayClusterConfiguratorTreeItem::GetRowItemText)
			.HighlightText(FilterText)
			.IsReadOnly(this, &FDisplayClusterConfiguratorTreeItem::IsReadOnly)
			.OnTextCommitted(this, &FDisplayClusterConfiguratorTreeItem::OnDisplayNameCommitted)
		];
}

bool FDisplayClusterConfiguratorTreeItem::IsReadOnly() const
{
	return !CanRenameItem();
}

FText FDisplayClusterConfiguratorTreeItem::GetRowItemText() const
{
	return FText::FromName(GetRowItemName());
}