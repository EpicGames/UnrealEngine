// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Views/TreeViews/IDisplayClusterConfiguratorTreeItem.h"

#include "Widgets/SNullWidget.h"
#include "Widgets/SWidget.h"

class FDisplayClusterConfiguratorBlueprintEditor;
class IDisplayClusterConfiguratorViewTree;
class SInlineEditableTextBlock;

class FDisplayClusterConfiguratorTreeItem
	: public IDisplayClusterConfiguratorTreeItem
{
	friend class FDisplayClusterConfiguratorTreeBuilder;

public:
	FDisplayClusterConfiguratorTreeItem(
		const TSharedRef<IDisplayClusterConfiguratorViewTree>& InViewTree,
		const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& InToolkit,
		UObject* InObjectToEdit,
		bool InbRoot)
		: ViewTreePtr(InViewTree)
		, ToolkitPtr(InToolkit)
		, FilterResult(EDisplayClusterConfiguratorTreeFilterResult::Shown)
		, ObjectToEdit(InObjectToEdit)
		, bRoot(InbRoot)
	{ }

public:
	//~ Begin IDisplayClusterConfiguratorItem Interface
	virtual void OnSelection() override { }
	virtual UObject* GetObject() const override { return ObjectToEdit.Get(); }
	virtual bool IsSelected() override;
	//~ End IDisplayClusterConfiguratorItem Interface

public:
	//~ Begin IDisplayClusterConfiguratorTreeItem Interface
	virtual void Initialize() override {}
	virtual TSharedRef<ITableRow> MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, const TAttribute<FText>& InFilterText) override;
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName, TSharedPtr<ITableRow> TableRow, const TAttribute<FText>& FilterText, FIsSelected InIsSelected) override;
	virtual TSharedRef<SWidget> GenerateInlineEditWidget(const TAttribute<FText>& FilterText, FIsSelected InIsSelected) override { return SNullWidget::NullWidget; }
	virtual bool HasInlineEditor() const override { return false; }
	virtual void ToggleInlineEditorExpansion() override {}
	virtual bool IsInlineEditorExpanded() const override { return false; }
	virtual FName GetAttachName() const override { return GetRowItemName(); }
	virtual FString GetIconStyle() const override { return FString(); }
	virtual void GetParentObjectsRecursive(TArray<UObject*>& OutObjects) const override;
	virtual void GetChildrenObjectsRecursive(TArray<UObject*>& OutObjects) const override;
	virtual bool CanRenameItem() const override;
	virtual void RequestRename() override;
	virtual bool CanDeleteItem() const override;
	virtual void DeleteItem() const override { }
	bool CanDuplicateItem() const override;
	virtual bool CanHideItem() const override { return false; }
	virtual void SetItemHidden(bool bIsHidden) { }

	virtual void OnItemDoubleClicked() override {}
	virtual void OnMouseEnter() override {}
	virtual void OnMouseLeave() override {}
	virtual bool IsHovered() const override { return false; }
	virtual bool IsChildOfRecursive(const TSharedRef<IDisplayClusterConfiguratorTreeItem>& InParentTreeItem) const override;
	virtual bool IsRoot() const { return bRoot; }
	virtual FReply HandleDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override { return FReply::Unhandled(); }
	virtual void HandleDragEnter(const FDragDropEvent& DragDropEvent) override {}
	virtual void HandleDragLeave(const FDragDropEvent& DragDropEvent) override {}
	virtual TOptional<EItemDropZone> HandleCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<IDisplayClusterConfiguratorTreeItem> TargetItem) override { return TOptional<EItemDropZone>(); }
	virtual FReply HandleAcceptDrop(const FDragDropEvent& DragDropEvent, TSharedPtr<IDisplayClusterConfiguratorTreeItem> TargetItem) override { return FReply::Unhandled(); }
	virtual FReply HandleDrop(FDragDropEvent const& DragDropEvent) override { return FReply::Unhandled(); }
	virtual TSharedPtr<IDisplayClusterConfiguratorTreeItem> GetParent() const override { return Parent.Pin(); }
	virtual void SetParent(TSharedPtr<IDisplayClusterConfiguratorTreeItem> InParent) override { Parent = InParent; }
	virtual TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>>& GetChildren() override { return Children; }
	virtual const TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>>& GetChildrenConst() const { return Children; }
	virtual TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>>& GetFilteredChildren() override { return FilteredChildren; }
	virtual TSharedRef<IDisplayClusterConfiguratorViewTree> GetConfiguratorTree() const override { return ViewTreePtr.Pin().ToSharedRef(); }
	virtual EDisplayClusterConfiguratorTreeFilterResult ApplyFilter(const TSharedPtr<FTextFilterExpressionEvaluator>& TextFilter) override;
	virtual EDisplayClusterConfiguratorTreeFilterResult GetFilterResult() const override { return FilterResult; }
	virtual void SetFilterResult(EDisplayClusterConfiguratorTreeFilterResult InResult) override { FilterResult = InResult; }
	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) { return FReply::Unhandled(); }
	virtual bool IsInitiallyExpanded() const override { return true; }
	//~ End IDisplayClusterConfiguratorTreeItem Interface

protected:
	virtual void FillItemColumn(TSharedPtr<SHorizontalBox> Box, const TAttribute<FText>& FilterText, FIsSelected InIsSelected);

	virtual bool IsReadOnly() const;
	virtual void OnDisplayNameCommitted(const FText& NewText, ETextCommit::Type CommitInfo) { }

	virtual FText GetRowItemText() const;

	template<class TObjectType>
	TObjectType* GetObjectChecked() const
	{
		TObjectType* CastedObject = Cast<TObjectType>(ObjectToEdit.Get());
		check(CastedObject);
		return CastedObject;
	}

protected:
	/** The parent of this item */
	TWeakPtr<IDisplayClusterConfiguratorTreeItem> Parent;

	/** The children of this item */
	TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>> Children;

	/** The filtered children of this item */
	TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>> FilteredChildren;

	TWeakPtr<IDisplayClusterConfiguratorViewTree> ViewTreePtr;

	TWeakPtr<FDisplayClusterConfiguratorBlueprintEditor> ToolkitPtr;

	/** The current filter result */
	EDisplayClusterConfiguratorTreeFilterResult FilterResult;

	TWeakObjectPtr<UObject> ObjectToEdit;

	bool bRoot;

	TSharedPtr<SInlineEditableTextBlock> DisplayNameTextBlock;
};
