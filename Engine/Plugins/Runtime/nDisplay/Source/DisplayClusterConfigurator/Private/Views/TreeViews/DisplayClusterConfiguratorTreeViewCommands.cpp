// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/TreeViews/DisplayClusterConfiguratorTreeViewCommands.h"

#define LOCTEXT_NAMESPACE "FDisplayClusterConfiguratorTreeViewCommands"

void FDisplayClusterConfiguratorTreeViewCommands::RegisterCommands()
{
#if PLATFORM_MAC
	EModifierKey::Type PlatformControlKey = EModifierKey::Command;
#else
	EModifierKey::Type PlatformControlKey = EModifierKey::Control;
#endif

	UI_COMMAND(ShowAll, "Show All Cluster Items", "Shows all cluster items in the configuration", EUserInterfaceActionType::Button, FInputChord(PlatformControlKey, EKeys::H));
	UI_COMMAND(ShowSelectedOnly, "Show Only Selected", "Shows only the selected cluster items", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ShowSelected, "Show Selected", "Shows the selected cluster items", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::H));
	UI_COMMAND(HideSelected, "Hide Selected", "Hides the selected cluster items", EUserInterfaceActionType::Button, FInputChord(EKeys::H));

	UI_COMMAND(SetAsPrimary, "Set as Primary", "Sets this cluster node as the primary node", EUserInterfaceActionType::Button, FInputChord());
}

#undef LOCTEXT_NAMESPACE
