// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/AppStyle.h"
#include "Framework/Commands/Commands.h"


/** */
class FDisplayClusterConfiguratorTreeViewCommands
	: public TCommands<FDisplayClusterConfiguratorTreeViewCommands>
{
public:
	FDisplayClusterConfiguratorTreeViewCommands()
		: TCommands<FDisplayClusterConfiguratorTreeViewCommands>
		(
			TEXT("ConfiguratorViewTree"), // Context name for fast lookup
			NSLOCTEXT("Contexts", "ConfiguratorViewTree", "Configurator View Tree"), // Localized context name for displaying
			NAME_None, // Parent context name.  
			FAppStyle::GetAppStyleSetName() // Icon Style Set
		)
	{ }

public:
	/** Initialize commands */
	virtual void RegisterCommands() override;

	TSharedPtr<FUICommandInfo> ShowAll;
	TSharedPtr<FUICommandInfo> ShowSelectedOnly;
	TSharedPtr<FUICommandInfo> ShowSelected;
	TSharedPtr<FUICommandInfo> HideSelected;

	TSharedPtr<FUICommandInfo> SetAsPrimary;
};
