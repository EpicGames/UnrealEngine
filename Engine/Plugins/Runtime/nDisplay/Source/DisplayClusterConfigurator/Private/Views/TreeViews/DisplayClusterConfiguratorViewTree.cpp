// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/TreeViews/DisplayClusterConfiguratorViewTree.h"

#include "DisplayClusterConfiguratorBlueprintEditor.h"
#include "Views/TreeViews/DisplayClusterConfiguratorTreeItem.h"
#include "Views/TreeViews/DisplayClusterConfiguratorTreeBuilder.h"
#include "Views/TreeViews/IDisplayClusterConfiguratorViewTree.h"
#include "Views/TreeViews/SDisplayClusterConfiguratorViewTree.h"

#define LOCTEXT_NAMESPACE "FDisplayClusterConfiguratorViewTree"

const FName IDisplayClusterConfiguratorViewTree::Columns::Item("Item");
const FName IDisplayClusterConfiguratorViewTree::Columns::Group("Group");

FDisplayClusterConfiguratorViewTree::FDisplayClusterConfiguratorViewTree(const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& InToolkit)
	: ToolkitPtr(InToolkit)
	, bEnabled(false)
{
	GEditor->RegisterForUndo(this);

	SortedColumn = IDisplayClusterConfiguratorViewTree::Columns::Item;
	SortMode = EColumnSortMode::Ascending;
}

FDisplayClusterConfiguratorViewTree::~FDisplayClusterConfiguratorViewTree()
{
	GEditor->UnregisterForUndo(this);
}

TSharedRef<SWidget> FDisplayClusterConfiguratorViewTree::CreateWidget()
{
	ToolkitPtr.Pin()->RegisterOnConfigReloaded(IDisplayClusterConfiguratorBlueprintEditor::FOnConfigReloadedDelegate::CreateSP(this, &FDisplayClusterConfiguratorViewTree::OnConfigReloaded));
	ToolkitPtr.Pin()->RegisterOnObjectSelected(IDisplayClusterConfiguratorBlueprintEditor::FOnObjectSelectedDelegate::CreateSP(this, &FDisplayClusterConfiguratorViewTree::OnObjectSelected));

	if (!ViewTree.IsValid())
	{
		TreeBuilder->Initialize(SharedThis(this));

		SAssignNew(ViewTree, SDisplayClusterConfiguratorViewTree, ToolkitPtr.Pin().ToSharedRef(), TreeBuilder.ToSharedRef(), SharedThis(this));
	}

	return ViewTree.ToSharedRef();
}

TSharedRef<SWidget> FDisplayClusterConfiguratorViewTree::GetWidget()
{
	return ViewTree.ToSharedRef();
}

void FDisplayClusterConfiguratorViewTree::OnConfigReloaded()
{
	ViewTree->OnConfigReloaded();
}

void FDisplayClusterConfiguratorViewTree::OnObjectSelected()
{
	TArray<UObject*> SelectedObjects = ToolkitPtr.Pin()->GetSelectedObjects();

	if (!SelectedObjects.Num())
	{
		ClearSelection();
		return;
	}

	TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>> TreeItems = ViewTree->GetAllItemsFlattened();

	TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>> SelectedTreeItems;
	for (TSharedPtr<IDisplayClusterConfiguratorTreeItem> TreeItem : TreeItems)
	{
		UObject* Object = TreeItem->GetObject();
		if (SelectedObjects.Contains(Object))
		{
			SelectedTreeItems.Add(TreeItem);
		}
	}

	SetSelectedItems(SelectedTreeItems);
}

void FDisplayClusterConfiguratorViewTree::SetEnabled(bool bInEnabled)
{
	bEnabled = bInEnabled;
}

UDisplayClusterConfigurationData* FDisplayClusterConfiguratorViewTree::GetEditorData() const
{
	return ToolkitPtr.Pin()->GetEditorData();
}

void FDisplayClusterConfiguratorViewTree::ConstructColumns(TArray<SHeaderRow::FColumn::FArguments>& OutColumnArgs)
{
	OutColumnArgs.Add(SHeaderRow::Column(IDisplayClusterConfiguratorViewTree::Columns::Item)
		.DefaultLabel(LOCTEXT("DisplayClusterConfiguratorNameLabel", "Items"))
		.FillWidth(0.5f)
		.InitialSortMode(EColumnSortMode::Ascending)
		.SortMode(this, &FDisplayClusterConfiguratorViewTree::GetSortMode, IDisplayClusterConfiguratorViewTree::Columns::Item)
		.OnSort(this, &FDisplayClusterConfiguratorViewTree::OnSort));
}

void FDisplayClusterConfiguratorViewTree::SetHoveredItem(const TSharedRef<IDisplayClusterConfiguratorTreeItem>& InTreeItem)
{
	HoveredTreeItemPtr = InTreeItem;
	OnHoveredItemSet.Broadcast(InTreeItem);
}

void FDisplayClusterConfiguratorViewTree::ClearHoveredItem()
{
	HoveredTreeItemPtr = nullptr;
	OnHoveredItemCleared.Broadcast();
}

void FDisplayClusterConfiguratorViewTree::Filter(const FDisplayClusterConfiguratorTreeFilterArgs& InArgs, const TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>>& InItems, TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>>& OutFilteredItems)
{
	OutFilteredItems.Empty();

	for (const TSharedPtr<IDisplayClusterConfiguratorTreeItem>& Item : InItems)
	{
		FilterItem(Item, InArgs, OutFilteredItems);
	}
}

EDisplayClusterConfiguratorTreeFilterResult FDisplayClusterConfiguratorViewTree::FilterItem(const TSharedPtr<IDisplayClusterConfiguratorTreeItem>& InItem, const FDisplayClusterConfiguratorTreeFilterArgs& InArgs, TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>>& OutFilteredItems)
{
	const bool bIsFlatteningHierarchy = InArgs.TextFilter.IsValid() && InArgs.bFlattenHierarchyOnFilter;

	InItem->GetFilteredChildren().Empty();

	// Recursively filter the item's children first. If the hierarchy is not being flattened, add any non-hidden children to the item's list of filtered children.
	EDisplayClusterConfiguratorTreeFilterResult DescendantsFilterResult = EDisplayClusterConfiguratorTreeFilterResult::Hidden;
	for (const TSharedPtr<IDisplayClusterConfiguratorTreeItem>& Item : InItem->GetChildren())
	{
		EDisplayClusterConfiguratorTreeFilterResult ChildResult = FilterItem(Item, InArgs, OutFilteredItems);

		if (ChildResult != EDisplayClusterConfiguratorTreeFilterResult::Hidden && !bIsFlatteningHierarchy)
		{
			InItem->GetFilteredChildren().Add(Item);
		}

		if (ChildResult > DescendantsFilterResult)
		{
			DescendantsFilterResult = ChildResult;
		}
	}

	// Now filter the item itself. If the hierarchy is not being flattened, then this item needs to be shown if any of its descendents are being shown.
	EDisplayClusterConfiguratorTreeFilterResult ItemFilterResult = InItem->ApplyFilter(InArgs.TextFilter);
	if (!bIsFlatteningHierarchy && DescendantsFilterResult > ItemFilterResult)
	{
		ItemFilterResult = EDisplayClusterConfiguratorTreeFilterResult::ShownDescendant;
		InItem->SetFilterResult(ItemFilterResult);
	}

	// Finally, if the item is not being hidden and it is either a root item (has no parent) or the hierarchy is being flattened, add it to the top-level list of filtered items.
	if (ItemFilterResult != EDisplayClusterConfiguratorTreeFilterResult::Hidden)
	{
		if (bIsFlatteningHierarchy || !InItem->GetParent().IsValid())
		{
			OutFilteredItems.Add(InItem);
		}
	}

	return ItemFilterResult;
}

void FDisplayClusterConfiguratorViewTree::Sort(const TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>>& InItems, TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>>& OutSortedItems)
{
	for (const TSharedPtr<IDisplayClusterConfiguratorTreeItem>& Item : InItems)
	{
		SortItem(Item);
		OutSortedItems.Add(Item);
	}

	if (SortMode != EColumnSortMode::None)
	{
		if (SortedColumn == IDisplayClusterConfiguratorViewTree::Columns::Item)
		{
			Algo::SortBy(OutSortedItems, [this](const TSharedPtr<IDisplayClusterConfiguratorTreeItem>& Item) { return Item->GetRowItemName(); }, FNameLexicalLess());
		}

		if (SortMode == EColumnSortMode::Descending)
		{
			Algo::Reverse(OutSortedItems);
		}
	}
}

void FDisplayClusterConfiguratorViewTree::SortItem(const TSharedPtr<IDisplayClusterConfiguratorTreeItem>& InItem)
{
	if (InItem.IsValid() && InItem->GetChildren().Num() && SortMode != EColumnSortMode::None)
	{
		for (const TSharedPtr<IDisplayClusterConfiguratorTreeItem>& Child : InItem->GetChildren())
		{
			SortItem(Child);
		}

		if (SortedColumn == IDisplayClusterConfiguratorViewTree::Columns::Item)
		{
			Algo::SortBy(InItem->GetChildren(), [this](const TSharedPtr<IDisplayClusterConfiguratorTreeItem>& Child) { return Child->GetRowItemName(); }, FNameLexicalLess());
		}

		if (SortMode == EColumnSortMode::Descending)
		{
			Algo::Reverse(InItem->GetChildren());
		}
	}
}

EColumnSortMode::Type FDisplayClusterConfiguratorViewTree::GetSortMode(const FName InColumnId) const
{
	if (SortedColumn != InColumnId)
	{
		return EColumnSortMode::None;
	}

	return SortMode;
}

void FDisplayClusterConfiguratorViewTree::OnSort(const EColumnSortPriority::Type InSortPriority, const FName& InColumnId, const EColumnSortMode::Type InSortMode)
{
	SortedColumn = InColumnId;
	SortMode = InSortMode;

	ViewTree->ApplySortAndFilter();
}

void FDisplayClusterConfiguratorViewTree::RebuildTree()
{
	ViewTree->RebuildTree();
}

void FDisplayClusterConfiguratorViewTree::SetSelectedItems(const TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>>& InTreeItems)
{
	if (InTreeItems.Num() > 0)
	{
		ViewTree->ClearSelection();
		ViewTree->SetSelectedItems(InTreeItems);
	}
	else
	{
		ClearSelection();
	}
}

void FDisplayClusterConfiguratorViewTree::ClearSelection()
{
	ViewTree->ClearSelection();
}

TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>> FDisplayClusterConfiguratorViewTree::GetSelectedItems() const
{
	return ViewTree->GetSelectedItems();
}

void FDisplayClusterConfiguratorViewTree::GetSelectedObjects(TArray<UObject*>& OutObjects) const
{
	TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>> SelectedItems = ViewTree->GetSelectedItems();

	OutObjects.Empty(SelectedItems.Num());
	for (TSharedPtr<IDisplayClusterConfiguratorTreeItem> SelectedItem : SelectedItems)
	{
		if (SelectedItem.IsValid() && SelectedItem->GetObject())
		{
			OutObjects.Add(SelectedItem->GetObject());
		}
	}
}

void FDisplayClusterConfiguratorViewTree::FindAndSelectObjects(const TArray<UObject*>& ObjectsToSelect)
{
	TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>> AllItems = ViewTree->GetAllItemsFlattened();
	TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>> ItemsToSelect;

	for (const TSharedPtr<IDisplayClusterConfiguratorTreeItem>& Item : AllItems)
	{
		if (ObjectsToSelect.Contains(Item->GetObject()))
		{
			ItemsToSelect.Add(Item);
		}
	}

	SetSelectedItems(ItemsToSelect);
}

TSharedPtr<IDisplayClusterConfiguratorTreeItem> FDisplayClusterConfiguratorViewTree::GetHoveredItem() const
{
	return HoveredTreeItemPtr.Pin();
}

FDelegateHandle FDisplayClusterConfiguratorViewTree::RegisterOnHoveredItemSet(const FOnHoveredItemSetDelegate& Delegate)
{
	return OnHoveredItemSet.Add(Delegate);
}

void FDisplayClusterConfiguratorViewTree::UnregisterOnHoveredItemSet(FDelegateHandle DelegateHandle)
{
	OnHoveredItemSet.Remove(DelegateHandle);
}

FDelegateHandle FDisplayClusterConfiguratorViewTree::RegisterOnHoveredItemCleared(const FOnHoveredItemClearedDelegate& Delegate)
{
	return OnHoveredItemCleared.Add(Delegate);
}

void FDisplayClusterConfiguratorViewTree::UnregisterOnHoveredItemCleared(FDelegateHandle DelegateHandle)
{
	OnHoveredItemCleared.Remove(DelegateHandle);
}

#undef LOCTEXT_NAMESPACE
