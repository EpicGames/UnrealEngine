// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DisplayClusterConfigurationTypes.h"
#include "Views/TreeViews/IDisplayClusterConfiguratorViewTree.h"

#include "EditorUndoClient.h"

class IDisplayClusterConfiguratorTreeItem;
class FDisplayClusterConfiguratorTreeBuilder;
class FDisplayClusterConfiguratorBlueprintEditor;
class SDisplayClusterConfiguratorViewTree;

struct FDisplayClusterConfiguratorTreeFilterArgs;
enum class EDisplayClusterConfiguratorTreeFilterResult : uint8;

class FDisplayClusterConfiguratorViewTree
	: public IDisplayClusterConfiguratorViewTree
	, public FEditorUndoClient
{
public:
	FDisplayClusterConfiguratorViewTree(const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& Toolkit);
	virtual ~FDisplayClusterConfiguratorViewTree();

public:
	//~ Begin FEditorUndoClient interface
	virtual void PostUndo(bool bSuccess) override { }
	virtual void PostRedo(bool bSuccess) override { PostUndo(bSuccess); }
	//~ End FEditorUndoClient interface

	//~ Begin IDisplayClusterConfiguratorView Interface
	virtual TSharedRef<SWidget> CreateWidget() override;
	virtual TSharedRef<SWidget> GetWidget() override;
	virtual void SetEnabled(bool bInEnabled) override;

	virtual bool GetIsEnabled() const override { return bEnabled; }
	//~ End IDisplayClusterConfiguratorView Interface

	//~ IDisplayClusterConfiguratorViewTree
	virtual void RebuildTree() override;
	virtual UDisplayClusterConfigurationData* GetEditorData() const override;
	virtual void ConstructColumns(TArray<SHeaderRow::FColumn::FArguments>& OutColumnArgs) override;

	virtual void SetHoveredItem(const TSharedRef<IDisplayClusterConfiguratorTreeItem>& InTreeItem) override;
	virtual void ClearHoveredItem() override;
	virtual TSharedPtr<IDisplayClusterConfiguratorTreeItem> GetHoveredItem() const override;
	virtual void SetSelectedItems(const TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>>& InTreeItems) override;
	virtual void ClearSelection() override;
	virtual TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>> GetSelectedItems() const override;
	virtual void GetSelectedObjects(TArray<UObject*>& OutObjects) const override;
	virtual void FindAndSelectObjects(const TArray<UObject*>& ObjectsToSelect) override;
	virtual void Filter(const FDisplayClusterConfiguratorTreeFilterArgs& InArgs, const TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>>& InItems, TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>>& OutFilteredItems) override;
	virtual void Sort(const TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>>& InItems, TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>>& OutSortedItems) override;

	virtual FDelegateHandle RegisterOnHoveredItemSet(const FOnHoveredItemSetDelegate& Delegate) override;
	virtual void UnregisterOnHoveredItemSet(FDelegateHandle DelegateHandle) override;
	virtual FDelegateHandle RegisterOnHoveredItemCleared(const FOnHoveredItemClearedDelegate& Delegate) override;
	virtual void UnregisterOnHoveredItemCleared(FDelegateHandle DelegateHandle) override;

	virtual void FillContextMenu(FMenuBuilder& MenuBuilder) override { }
	virtual void BindPinnableCommands(FUICommandList_Pinnable& CommandList) override { }
	virtual bool ShowAddNewButton() const override { return false; }
	virtual void FillAddNewMenu(FMenuBuilder& MenuBuilder) override { }
	virtual bool ShowFilterOptionsButton() const override { return false; }
	virtual void FillFilterOptionsMenu(FMenuBuilder& MenuBuilder) override { }
	virtual bool ShowViewOptionsButton() const override { return false; }
	virtual void FillViewOptionsMenu(FMenuBuilder& MenuBuilder) override { }
	virtual FText GetCornerText() const override { return FText::GetEmpty(); }
	//~ IDisplayClusterConfiguratorViewTree

protected:
	virtual void OnConfigReloaded();
	virtual void OnObjectSelected();

	virtual EDisplayClusterConfiguratorTreeFilterResult FilterItem(const TSharedPtr<IDisplayClusterConfiguratorTreeItem>& InItem, const FDisplayClusterConfiguratorTreeFilterArgs& InArgs, TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>>& OutFilteredItems);

	/** Applies the current sort configuration to the specified item, sorting the item's children as necessary */
	virtual void SortItem(const TSharedPtr<IDisplayClusterConfiguratorTreeItem>& InItem);

	/** Gets the current sort mode for the specified column */
	virtual EColumnSortMode::Type GetSortMode(const FName InColumnId) const;

	/** Raised when a column in the tree view is sorted */
	virtual void OnSort(const EColumnSortPriority::Type InSortPriority, const FName& InColumnId, const EColumnSortMode::Type InSortMode);

protected:
	TWeakPtr<FDisplayClusterConfiguratorBlueprintEditor> ToolkitPtr;

	/** The builder we use to construct the tree */
	TSharedPtr<FDisplayClusterConfiguratorTreeBuilder> TreeBuilder;

	TWeakPtr<IDisplayClusterConfiguratorTreeItem> HoveredTreeItemPtr;

	FOnHoveredItemSet OnHoveredItemSet;

	FOnHoveredItemCleared OnHoveredItemCleared;

	TSharedPtr<SDisplayClusterConfiguratorViewTree> ViewTree;

	bool bEnabled;

	/** The name of the column in the tree view that is currently being sorted */
	FName SortedColumn;

	/** The current sort mode of the column being sorted in the tree view */
	EColumnSortMode::Type SortMode;
};
