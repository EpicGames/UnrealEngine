// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/TreeViews/SDisplayClusterConfiguratorTreeItemRow.h"

#include "Views/TreeViews/IDisplayClusterConfiguratorTreeItem.h"
#include "Views/TreeViews/IDisplayClusterConfiguratorViewTree.h"

#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "SDisplayClusterConfiguratorTreeItemRow"

void SDisplayClusterConfiguratorTreeItemRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	Item = InArgs._Item;
	FilterText = InArgs._FilterText;

	SetHover(TAttribute<bool>::CreateSP(this, &SDisplayClusterConfiguratorTreeItemRow::ShouldAppearHovered));

	check(Item.IsValid());

	SMultiColumnTableRow< TSharedPtr<IDisplayClusterConfiguratorTreeItem> >::Construct(FSuperRowType::FArguments()
		.Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("SceneOutliner.TableViewRow"))
		.OnCanAcceptDrop(this, &SDisplayClusterConfiguratorTreeItemRow::HandleCanAcceptDrop)
		.OnAcceptDrop(this, &SDisplayClusterConfiguratorTreeItemRow::HandleAcceptDrop)
		.OnDragDetected(this, &SDisplayClusterConfiguratorTreeItemRow::HandleDragDetected)
		.OnDragEnter(this, &SDisplayClusterConfiguratorTreeItemRow::HandleDragEnter)
		.OnDragLeave(this, &SDisplayClusterConfiguratorTreeItemRow::HandleDragLeave)
		.OnDrop(this, &SDisplayClusterConfiguratorTreeItemRow::HandleDrop)
		, InOwnerTableView);
}

void SDisplayClusterConfiguratorTreeItemRow::ConstructChildren(ETableViewMode::Type InOwnerTableMode, const TAttribute<FMargin>& InPadding, const TSharedRef<SWidget>& InContent)
{
	STableRow<TSharedPtr<IDisplayClusterConfiguratorTreeItem>>::Content = InContent;

	TSharedRef<SWidget> InlineEditWidget = Item.Pin()->GenerateInlineEditWidget(FilterText, FIsSelected::CreateSP(this, &STableRow::IsSelected));

	// MultiColumnRows let the user decide which column should contain the expander/indenter item.
	ChildSlot
		.Padding(InPadding)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				InContent
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				InlineEditWidget
			]
		];
}

bool SDisplayClusterConfiguratorTreeItemRow::ShouldAppearHovered() const
{
	return Item.Pin()->IsHovered();
}

void SDisplayClusterConfiguratorTreeItemRow::OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	Item.Pin()->OnMouseEnter();
}

void SDisplayClusterConfiguratorTreeItemRow::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	Item.Pin()->OnMouseLeave();
}

FReply SDisplayClusterConfiguratorTreeItemRow::HandleDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return Item.Pin()->HandleDragDetected(MyGeometry, MouseEvent);
}

void SDisplayClusterConfiguratorTreeItemRow::HandleDragEnter(const FDragDropEvent& DragDropEvent)
{
	Item.Pin()->HandleDragEnter(DragDropEvent);
}

void SDisplayClusterConfiguratorTreeItemRow::HandleDragLeave(const FDragDropEvent& DragDropEvent)
{
	Item.Pin()->HandleDragLeave(DragDropEvent);
}

TOptional<EItemDropZone> SDisplayClusterConfiguratorTreeItemRow::HandleCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<IDisplayClusterConfiguratorTreeItem> TargetItem)
{
	return Item.Pin()->HandleCanAcceptDrop(DragDropEvent, DropZone, TargetItem);
}

FReply SDisplayClusterConfiguratorTreeItemRow::HandleAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<IDisplayClusterConfiguratorTreeItem> TargetItem)
{
	return Item.Pin()->HandleAcceptDrop(DragDropEvent, TargetItem);
}

FReply SDisplayClusterConfiguratorTreeItemRow::HandleDrop(FDragDropEvent const& DragDropEvent)
{
	return Item.Pin()->HandleDrop(DragDropEvent);
}

TSharedRef<SWidget> SDisplayClusterConfiguratorTreeItemRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	return Item.Pin()->GenerateWidgetForColumn(ColumnName, SharedThis(this), FilterText, FIsSelected::CreateSP(this, &STableRow::IsSelectedExclusively));
}

#undef LOCTEXT_NAMESPACE
