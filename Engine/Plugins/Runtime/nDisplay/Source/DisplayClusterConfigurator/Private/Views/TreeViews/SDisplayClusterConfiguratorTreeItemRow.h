// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class STableViewBase;
template<typename ItemType>
class STableRow;
class IDisplayClusterConfiguratorTreeItem;

class SDisplayClusterConfiguratorTreeItemRow :
	public SMultiColumnTableRow<TSharedPtr<IDisplayClusterConfiguratorTreeItem>>
{
public:
	SLATE_BEGIN_ARGS(SDisplayClusterConfiguratorTreeItemRow)
		{}
		/** The item for this row **/
		SLATE_ARGUMENT(TSharedPtr<IDisplayClusterConfiguratorTreeItem>, Item)

		/** Filter text typed by the user into the parent tree's search widget */
		SLATE_ATTRIBUTE(FText, FilterText);

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView);

	//~ Begin SMultiColumnTableRow interface
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;
	//~ End SWidget interface

	//~ Begin STableRow interface
	virtual void ConstructChildren(ETableViewMode::Type InOwnerTableMode, const TAttribute<FMargin>& InPadding, const TSharedRef<SWidget>& InContent) override;
	//~ End SWidget interface

	// Begin SWidget
	virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;
	// End SWidget

private:
	bool ShouldAppearHovered() const;

	FReply HandleDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	void HandleDragEnter(const FDragDropEvent& DragDropEvent);

	void HandleDragLeave(const FDragDropEvent& DragDropEvent);

	TOptional<EItemDropZone> HandleCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<IDisplayClusterConfiguratorTreeItem> TargetItem);

	FReply HandleAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<IDisplayClusterConfiguratorTreeItem> TargetItem);

	virtual FReply HandleDrop(FDragDropEvent const& DragDropEvent);

private:
	/** The item this row is holding */
	TWeakPtr<IDisplayClusterConfiguratorTreeItem> Item;

	/** Text the user typed into the search box - used for text highlighting */
	TAttribute<FText> FilterText;
};