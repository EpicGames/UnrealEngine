// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/TreeViews/SDisplayClusterConfiguratorViewTree.h"

#include "DisplayClusterConfiguratorBlueprintEditor.h"
#include "Views/TreeViews/IDisplayClusterConfiguratorTreeBuilder.h"
#include "Views/TreeViews/IDisplayClusterConfiguratorTreeItem.h"
#include "Views/TreeViews/IDisplayClusterConfiguratorViewTree.h"
#include "Views/TreeViews/DisplayClusterConfiguratorTreeViewCommands.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IPinnedCommandList.h"
#include "Modules/ModuleManager.h"
#include "Misc/TextFilterExpressionEvaluator.h"
#include "UICommandList_Pinnable.h"
#include "UObject/UObjectGlobals.h"
#include "Types/ISlateMetaData.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Input/SComboButton.h"
#include "SPositiveActionButton.h"

#define LOCTEXT_NAMESPACE "SDisplayClusterConfiguratorViewTree"

void SDisplayClusterConfiguratorViewTree::Construct(const FArguments& InArgs,
	const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& InToolkit,
	const TSharedRef<IDisplayClusterConfiguratorTreeBuilder>& InBuilder,
	const TSharedRef<IDisplayClusterConfiguratorViewTree>& InViewTree,
	const FDisplayClusterConfiguratorTreeArgs& InTreeArgs)
{
	// Set properties
	BuilderPtr = InBuilder;
	ToolkitPtr = InToolkit;
	ViewTreePtr = InViewTree;

	Mode = InTreeArgs.Mode;

	ContextName = InTreeArgs.ContextName;

	TextFilterPtr = MakeShared<FTextFilterExpressionEvaluator>(ETextFilterExpressionEvaluatorMode::BasicString);

	// Register delegates
	FCoreUObjectDelegates::OnPackageReloaded.AddSP(this, &SDisplayClusterConfiguratorViewTree::HandlePackageReloaded);

	// Create our pinned commands before we bind commands
	IPinnedCommandListModule& PinnedCommandListModule = FModuleManager::LoadModuleChecked<IPinnedCommandListModule>(TEXT("PinnedCommandList"));
	PinnedCommands = PinnedCommandListModule.CreatePinnedCommandList(ContextName);

	// Register and bind all our menu commands
	UICommandList = MakeShareable(new FUICommandList_Pinnable);

	FDisplayClusterConfiguratorTreeViewCommands::Register();
	BindCommands();

	SDisplayClusterConfiguratorViewBase::Construct(
		SDisplayClusterConfiguratorViewBase::FArguments()
		.Padding(0.0f)
		.Content()
		[
			SNew( SOverlay )
			+SOverlay::Slot()
			[
				// Add a border if we are being used as a picker
				SNew(SBorder)
				.Visibility_Lambda([this](){ return Mode == EDisplayClusterConfiguratorTreeMode::Picker ? EVisibility::Visible: EVisibility::Collapsed; })
				.BorderImage(FAppStyle::Get().GetBrush("Menu.Background"))
			]
			+SOverlay::Slot()
			[
				SNew( SVerticalBox )
		
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(FMargin(0.0f, 0.0f, 0.0f, 2.0f))
				[
					SNew(SBorder)
					.Padding(0)
					.BorderImage(FAppStyle::Get().GetBrush("DetailsView.CategoryTop"))
					.BorderBackgroundColor(FLinearColor(0.6f, 0.6f, 0.6f, 1.0f))
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.Padding(3.0f, 3.0f)
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
					
							SNew(SPositiveActionButton)
							.Text(LOCTEXT("AddNewItem", "Add"))
							.Icon(FAppStyle::Get().GetBrush("Icons.Plus"))
							.ToolTipText(LOCTEXT("AddNewClusterItemTooltip", "Adds a new Cluster Item to the Cluster"))
							.Visibility(this, &SDisplayClusterConfiguratorViewTree::GetAddNewComboButtonVisibility)
							.OnGetMenuContent(this, &SDisplayClusterConfiguratorViewTree::CreateAddNewMenuContent)
						]

						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(3.0f, 3.0f)
						[
							SAssignNew(FilterComboButton, SComboButton)
							.ComboButtonStyle(FAppStyle::Get(), "GenericFilters.ComboButtonStyle")
							.ForegroundColor(FLinearColor::White)
							.Visibility(this, &SDisplayClusterConfiguratorViewTree::GetFilterOptionsComboButtonVisibility)
							.ContentPadding(0.0f)
							.OnGetMenuContent(this, &SDisplayClusterConfiguratorViewTree::CreateFilterMenuContent)
							.ToolTipText(this, &SDisplayClusterConfiguratorViewTree::GetFilterMenuTooltip)
							.AddMetaData<FTagMetaData>(TEXT("ConfiguratorViewTree.Items"))
							.ButtonContent()
							[
								SNew(SHorizontalBox)
								+SHorizontalBox::Slot()
								.AutoWidth()
								.VAlign(VAlign_Center)
								[
									SNew(STextBlock)
									.TextStyle(FAppStyle::Get(), "GenericFilters.TextStyle")
									.Font(FAppStyle::Get().GetFontStyle("FontAwesome.9"))
									.Text(FText::FromString(FString(TEXT("\xf0b0"))) /*fa-filter*/)
								]
								+SHorizontalBox::Slot()
								.AutoWidth()
								.Padding(2, 0, 0, 0)
								.VAlign(VAlign_Center)
								[
									SNew(STextBlock)
									.TextStyle(FAppStyle::Get(), "GenericFilters.TextStyle")
									.Text( LOCTEXT("FilterMenuLabel", "Options") )
								]
							]
						]

						+SHorizontalBox::Slot()
						.FillWidth(1.0f)
						.VAlign(VAlign_Center)
						.Padding(3.0f, 3.0f)
						[
							SAssignNew( NameFilterBox, SSearchBox )
							.SelectAllTextWhenFocused( true )
							.OnTextChanged( this, &SDisplayClusterConfiguratorViewTree::OnFilterTextChanged )
							.HintText( LOCTEXT( "SearchBoxHint", "Search Config Tree...") )
							.AddMetaData<FTagMetaData>(TEXT("ConfiguratorViewTree.Search"))
						]
					]
				]

				+ SVerticalBox::Slot()
				.Padding( FMargin( 0.0f, 2.0f, 0.0f, 0.0f ) )
				.AutoHeight()
				[
					PinnedCommands.ToSharedRef()
				]

				+ SVerticalBox::Slot()
				.Padding( FMargin( 0.0f, 2.0f, 0.0f, 0.0f ) )
				[
					SAssignNew(TreeHolder, SOverlay)
				]

				+ SVerticalBox::Slot()
				.Padding(FMargin(0.0f, 2.0f, 0.0f, 0.0f))
				.AutoHeight()
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					[
						SNew(SSpacer)
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SComboButton)
						.ComboButtonStyle(FAppStyle::Get(), "GenericFilters.ComboButtonStyle")
						.ForegroundColor(FLinearColor::White)
						.Visibility(this, &SDisplayClusterConfiguratorViewTree::GetViewOptionsComboButtonVisibility)
						.ContentPadding(0)
						.OnGetMenuContent(this, &SDisplayClusterConfiguratorViewTree::CreateViewOptionsMenuContent)
						.HasDownArrow(true)
						.ContentPadding(FMargin(1, 0))
						.ButtonContent()
						[
							SNew(SHorizontalBox)

							+ SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							[
								SNew(SImage).Image(FAppStyle::GetBrush("GenericViewButton"))
							]

							+ SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(2, 0, 0, 0)
							.VAlign(VAlign_Center)
							[
								SNew(STextBlock).Text(LOCTEXT("ViewOptions", "View Options"))
							]
						]
					]
				]
			]

			+ SOverlay::Slot()
			.Padding(10)
			.VAlign(VAlign_Bottom)
			.HAlign(HAlign_Right)
			[
				SNew(STextBlock)
				.Visibility(EVisibility::HitTestInvisible)
				.TextStyle(FAppStyle::Get(), "Graph.CornerText")
				.Text(this, &SDisplayClusterConfiguratorViewTree::GetCornerText)
			]
		],
		InToolkit);

	CreateTreeColumns();
}

FReply SDisplayClusterConfiguratorViewTree::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (UICommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

void SDisplayClusterConfiguratorViewTree::Refresh()
{
	RebuildTree();
}

void SDisplayClusterConfiguratorViewTree::CreateTreeColumns()
{
	TSharedRef<SHeaderRow> TreeHeaderRow = SNew(SHeaderRow);

	TArray<SHeaderRow::FColumn::FArguments> Columns;
	ViewTreePtr.Pin()->ConstructColumns(Columns);

	for (const auto& Args : Columns)
	{
		TreeHeaderRow->AddColumn(Args);
	}

	{
		ConfigTreeView = SNew(STreeView<TSharedPtr<IDisplayClusterConfiguratorTreeItem>>)
			.TreeItemsSource(&FilteredItems)
			.OnGenerateRow(this, &SDisplayClusterConfiguratorViewTree::MakeTreeRowWidget)
			.OnGetChildren(this, &SDisplayClusterConfiguratorViewTree::GetFilteredChildren)
			.OnContextMenuOpening(this, &SDisplayClusterConfiguratorViewTree::CreateContextMenu)
			.OnSelectionChanged(this, &SDisplayClusterConfiguratorViewTree::OnSelectionChanged)
			.OnItemScrolledIntoView(this, &SDisplayClusterConfiguratorViewTree::OnItemScrolledIntoView)
			.OnMouseButtonDoubleClick(this, &SDisplayClusterConfiguratorViewTree::OnTreeDoubleClick)
			.OnSetExpansionRecursive(this, &SDisplayClusterConfiguratorViewTree::SetTreeItemExpansionRecursive)
			.HighlightParentNodesForSelection(true)
			.HeaderRow
			(
				TreeHeaderRow
			);

		TreeHolder->ClearChildren();
		TreeHolder->AddSlot()
			[
				SNew(SScrollBorder, ConfigTreeView.ToSharedRef())
				[
					ConfigTreeView.ToSharedRef()
				]
			];
	}

	RebuildTree();
}

void SDisplayClusterConfiguratorViewTree::RebuildTree()
{
	// Save selected items
	Items.Empty();
	LinearItems.Empty();
	FilteredItems.Empty();

	FDisplayClusterConfiguratorTreeBuilderOutput Output(Items, LinearItems);
	BuilderPtr.Pin()->Build(Output);
	ApplySortAndFilter();
}

void SDisplayClusterConfiguratorViewTree::ApplySortAndFilter()
{
	TextFilterPtr->SetFilterText(FilterText);

	FilteredItems.Empty();

	TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>> SortedItems;
	ViewTreePtr.Pin()->Sort(Items, SortedItems);

	FDisplayClusterConfiguratorTreeFilterArgs FilterArgs(!FilterText.IsEmpty() ? TextFilterPtr : nullptr);
	ViewTreePtr.Pin()->Filter(FilterArgs, SortedItems, FilteredItems);

	if (!FilterText.IsEmpty())
	{
		for (TSharedPtr<IDisplayClusterConfiguratorTreeItem>& Item : LinearItems)
		{
			if (Item->GetFilterResult() > EDisplayClusterConfiguratorTreeFilterResult::Hidden)
			{
				ConfigTreeView->SetItemExpansion(Item, true);
			}
		}
	}
	else
	{
		SetInitialExpansionState();
	}

	ConfigTreeView->RequestTreeRefresh();
}

void SDisplayClusterConfiguratorViewTree::SetInitialExpansionState()
{
	for (TSharedPtr<IDisplayClusterConfiguratorTreeItem>& Item : LinearItems)
	{
		ConfigTreeView->SetItemExpansion(Item, Item->IsInitiallyExpanded());

		if (Item->IsSelected())
		{
			ConfigTreeView->SetItemSelection(Item, true, ESelectInfo::Direct);
			ConfigTreeView->RequestScrollIntoView(Item);
		}
	}
}

TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>> SDisplayClusterConfiguratorViewTree::GetSelectedItems() const
{
	return ConfigTreeView->GetSelectedItems();
}

void SDisplayClusterConfiguratorViewTree::SetSelectedItems(const TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>>& InTreeItems)
{
	ConfigTreeView->SetItemSelection(InTreeItems, true);
}

void SDisplayClusterConfiguratorViewTree::ClearSelection()
{
	ConfigTreeView->ClearSelection();
}

TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>> SDisplayClusterConfiguratorViewTree::GetAllItemsFlattened() const
{
	return LinearItems;
}

TSharedRef<ITableRow> SDisplayClusterConfiguratorViewTree::MakeTreeRowWidget(TSharedPtr<IDisplayClusterConfiguratorTreeItem> InInfo, const TSharedRef<STableViewBase>& OwnerTable)
{
	check(InInfo.IsValid());

	return InInfo->MakeTreeRowWidget(OwnerTable, TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateLambda([this]() { return FilterText; })));
}

void SDisplayClusterConfiguratorViewTree::GetFilteredChildren(TSharedPtr<IDisplayClusterConfiguratorTreeItem> InInfo, TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>>& OutChildren)
{
	check(InInfo.IsValid());
	OutChildren = InInfo->GetFilteredChildren();
}

TSharedPtr<SWidget> SDisplayClusterConfiguratorViewTree::CreateContextMenu()
{
	const bool CloseAfterSelection = true;
	FMenuBuilder MenuBuilder(CloseAfterSelection, UICommandList, Extenders);

	ViewTreePtr.Pin()->FillContextMenu(MenuBuilder);

	return MenuBuilder.MakeWidget();
}

void SDisplayClusterConfiguratorViewTree::OnSelectionChanged(TSharedPtr<IDisplayClusterConfiguratorTreeItem> Selection, ESelectInfo::Type SelectInfo)
{
	if (SelectInfo != ESelectInfo::Direct)
	{
		TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>> SelectedItems = ConfigTreeView->GetSelectedItems();
		TArray<UObject*> SelectedObjects;
		for (TSharedPtr<IDisplayClusterConfiguratorTreeItem>& Item : SelectedItems)
		{
			SelectedObjects.Add(Item->GetObject());
		}

		ToolkitPtr.Pin()->SelectObjects(SelectedObjects);

		for (TSharedPtr<IDisplayClusterConfiguratorTreeItem>& Item : SelectedItems)
		{
			Item->OnSelection();
		}
	}
}

void SDisplayClusterConfiguratorViewTree::OnTreeDoubleClick(TSharedPtr<IDisplayClusterConfiguratorTreeItem> InItem)
{
	if (InItem.IsValid())
	{
		InItem->OnItemDoubleClicked();
	}
}

void SDisplayClusterConfiguratorViewTree::SetTreeItemExpansionRecursive(TSharedPtr<IDisplayClusterConfiguratorTreeItem> TreeItem, bool bInExpansionState) const
{
	ConfigTreeView->SetItemExpansion(TreeItem, bInExpansionState);

	// Recursively go through the children.
	for (auto It = TreeItem->GetChildren().CreateIterator(); It; ++It)
	{
		SetTreeItemExpansionRecursive(*It, bInExpansionState);
	}
}

void SDisplayClusterConfiguratorViewTree::BindCommands()
{
	ViewTreePtr.Pin()->BindPinnableCommands(*UICommandList);
}

void SDisplayClusterConfiguratorViewTree::OnFilterTextChanged(const FText& SearchText)
{
	FilterText = SearchText;
	ApplySortAndFilter();
}

void SDisplayClusterConfiguratorViewTree::OnConfigReloaded()
{
	RebuildTree();
}

EVisibility SDisplayClusterConfiguratorViewTree::GetAddNewComboButtonVisibility() const
{
	bool showButton = ViewTreePtr.IsValid() && ViewTreePtr.Pin()->ShowAddNewButton();
	return showButton ? EVisibility::Visible : EVisibility::Collapsed;
}

TSharedRef<SWidget> SDisplayClusterConfiguratorViewTree::CreateAddNewMenuContent()
{
	const bool bCloseAfterSelection = true;
	FMenuBuilder MenuBuilder(bCloseAfterSelection, UICommandList, Extenders);

	ViewTreePtr.Pin()->FillAddNewMenu(MenuBuilder);

	return MenuBuilder.MakeWidget();
}

EVisibility SDisplayClusterConfiguratorViewTree::GetFilterOptionsComboButtonVisibility() const
{
	bool showButton = ViewTreePtr.IsValid() && ViewTreePtr.Pin()->ShowFilterOptionsButton();
	return showButton ? EVisibility::Visible : EVisibility::Collapsed;
}

TSharedRef<SWidget> SDisplayClusterConfiguratorViewTree::CreateFilterMenuContent()
{
	const FDisplayClusterConfiguratorTreeViewCommands& Actions = FDisplayClusterConfiguratorTreeViewCommands::Get();

	const bool bCloseAfterSelection = true;
	FMenuBuilder MenuBuilder(bCloseAfterSelection, UICommandList, Extenders);

	return MenuBuilder.MakeWidget();
}

FText SDisplayClusterConfiguratorViewTree::GetFilterMenuTooltip() const
{
	return FText::GetEmpty();
}

EVisibility SDisplayClusterConfiguratorViewTree::GetViewOptionsComboButtonVisibility() const
{
	bool showButton = ViewTreePtr.IsValid() && ViewTreePtr.Pin()->ShowViewOptionsButton();
	return showButton ? EVisibility::Visible : EVisibility::Collapsed;
}

TSharedRef<SWidget> SDisplayClusterConfiguratorViewTree::CreateViewOptionsMenuContent()
{
	const bool bCloseAfterSelection = true;
	FMenuBuilder MenuBuilder(bCloseAfterSelection, UICommandList, Extenders);

	TSharedPtr<IDisplayClusterConfiguratorViewTree> ViewTree = ViewTreePtr.Pin();
	if (ViewTree.IsValid())
	{
		ViewTree->FillViewOptionsMenu(MenuBuilder);
	}

	return MenuBuilder.MakeWidget();
}

FText SDisplayClusterConfiguratorViewTree::GetCornerText() const
{
	TSharedPtr<IDisplayClusterConfiguratorViewTree> ViewTree = ViewTreePtr.Pin();
	if (ViewTree.IsValid())
	{
		return ViewTree->GetCornerText();
	}

	return FText::GetEmpty();
}

#undef LOCTEXT_NAMESPACE
