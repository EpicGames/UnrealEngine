// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Views/SDisplayClusterConfiguratorViewBase.h"
#include "Views/TreeViews/IDisplayClusterConfiguratorViewTree.h"

#include "EditorUndoClient.h"

class IPinnedCommandList;
class IDisplayClusterConfiguratorTreeItem;
class ITableRow;

class FPackageReloadedEvent;
class FDisplayClusterConfiguratorBlueprintEditor;
class FDisplayClusterConfiguratorViewTree;
class FDisplayClusterConfiguratorViewBuilder;
class FTextFilterExpressionEvaluator;
class FUICommandList_Pinnable;
class FExtender;
template<typename ItemType>
class STreeView;
class SOverlay;
class SComboButton;
class SSearchBox;
class STableViewBase;

struct FDisplayClusterConfiguratorTreeFilterArgs;

enum class EPackageReloadPhase : uint8;
enum class EDisplayClusterConfiguratorTreeFilterResult : uint8;

class SDisplayClusterConfiguratorViewTree
	: public SDisplayClusterConfiguratorViewBase
	, public FEditorUndoClient
{
public:
	friend class FDisplayClusterConfiguratorViewTree;

	SLATE_BEGIN_ARGS(SDisplayClusterConfiguratorViewTree)
	{ }

	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs,
		const TSharedRef<FDisplayClusterConfiguratorBlueprintEditor>& InToolkit,
		const TSharedRef<IDisplayClusterConfiguratorTreeBuilder>& InBuilder,
		const TSharedRef<IDisplayClusterConfiguratorViewTree>& ViewTreePtr,
		const FDisplayClusterConfiguratorTreeArgs& InTreeArgs = FDisplayClusterConfiguratorTreeArgs());

public:
	//~ Begin SWidget interface
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	//~ End SWidget interface

	//~ Begin FEditorUndoClient interface
	virtual void PostUndo(bool bSuccess) override {}
	virtual void PostRedo(bool bSuccess) override {}
	//~ End FEditorUndoClient interface

public:
	virtual void Refresh();

	/** Creates the tree control and then populates */
	virtual void CreateTreeColumns();

	/** Function to build the Config tree widgets from the source config object */
	void RebuildTree();

	/** Apply sorting and filtering to the tree */
	virtual void ApplySortAndFilter();

	/** Set the initial expansion state of the tree items */
	virtual void SetInitialExpansionState();

	/** @return A list of all selected items in the tree view */
	virtual TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>> GetSelectedItems() const;

	/** Sets the currently selected items in the tree view
	 * @param InTreeItems - The items to select
	 */
	virtual void SetSelectedItems(const TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>>& InTreeItems);

	/** Clears the current selection in the tree view */
	virtual void ClearSelection();

	/** @return A list of all items in the tree view, flattened out into a single array */
	virtual TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>> GetAllItemsFlattened() const;

protected:
	/** Create a widget for an entry in the tree from an info */
	virtual TSharedRef<ITableRow> MakeTreeRowWidget(TSharedPtr<IDisplayClusterConfiguratorTreeItem> InInfo, const TSharedRef<STableViewBase>& OwnerTable);

	/** Get all children for a given entry in the list */
	virtual void GetFilteredChildren(TSharedPtr<IDisplayClusterConfiguratorTreeItem> InInfo, TArray< TSharedPtr<IDisplayClusterConfiguratorTreeItem> >& OutChildren);

	/** Called to display context menu when right clicking on the widget */
	virtual TSharedPtr<SWidget> CreateContextMenu();

	/** Callback function to be called when selection changes in the tree view widget. */
	virtual void OnSelectionChanged(TSharedPtr<IDisplayClusterConfiguratorTreeItem> Selection, ESelectInfo::Type SelectInfo);

	/** Callback when an item is scrolled into view, handling calls to rename items */
	virtual void OnItemScrolledIntoView(TSharedPtr<IDisplayClusterConfiguratorTreeItem> InItem, const TSharedPtr<ITableRow>& InWidget) {}

	/** Callback for when the user double-clicks on an item in the tree */
	virtual void OnTreeDoubleClick(TSharedPtr<IDisplayClusterConfiguratorTreeItem> InItem);

	/** Handle recursive expansion/contraction of the tree */
	virtual void SetTreeItemExpansionRecursive(TSharedPtr<IDisplayClusterConfiguratorTreeItem> TreeItem, bool bInExpansionState) const;

	/** Binds the commands in FDisplayClusterConfiguratorViewTreeCommands to functions in this class */
	virtual void BindCommands();

	/** Handle package reloading */
	virtual void HandlePackageReloaded(const EPackageReloadPhase InPackageReloadPhase, FPackageReloadedEvent* InPackageReloadedEvent) {}

	/** Filters the SListView when the user changes the search text box (NameFilterBox)	*/
	virtual void OnFilterTextChanged(const FText& SearchText);

	virtual void OnConfigReloaded();


	/** @return The visibility of the "Add New" combo button */
	virtual EVisibility GetAddNewComboButtonVisibility() const;

	/**
	 * Raised when creating the "Add New" combo button menu
	 * @return The widget to show in the menu
	 */
	virtual TSharedRef<SWidget> CreateAddNewMenuContent();

	/** @return The visibility of the filter options combo button */
	virtual EVisibility GetFilterOptionsComboButtonVisibility() const;

	/**
	 * Raised when creating the filter options combo button menu
	 * @return The widget to show in the menu
	 */
	virtual TSharedRef<SWidget> CreateFilterMenuContent();

	/** @return The current text for the filter button tooltip */
	virtual FText GetFilterMenuTooltip() const;

	/** @return The visibility of the "View Options" combo button */
	virtual EVisibility GetViewOptionsComboButtonVisibility() const;

	/**
	 * Raised when creating the "View Options" combo button menu
	 * @return The widget to show in the menu
	 */
	virtual TSharedRef<SWidget> CreateViewOptionsMenuContent();

	/**
	 * @return The watermark text to display in the bottom right corner
	 */
	virtual FText GetCornerText() const;

protected:
	/** Current text typed into NameFilterBox */
	FText FilterText;

	/** Widget user to hold the item tree */
	TSharedPtr<SOverlay> TreeHolder;

	/** Widget used to display the items hierarchy */
	TSharedPtr<STreeView<TSharedPtr<IDisplayClusterConfiguratorTreeItem>>> ConfigTreeView;

	/** A tree of unfiltered items */
	TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>> Items;

	/** A "mirror" of the tree as a flat array for easier searching */
	TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>> LinearItems;

	/** Filtered view of the items tree. This is what is actually used in the tree widget */
	TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>> FilteredItems;

	/** Compiled filter search terms. */
	TSharedPtr<FTextFilterExpressionEvaluator> TextFilterPtr;

	/** The mode that this item tree is in */
	EDisplayClusterConfiguratorTreeMode Mode;

	/** Hold onto the filter combo button to set its foreground color */
	TSharedPtr<SComboButton> FilterComboButton;

	/** SSearchBox to filter the tree */
	TSharedPtr<SSearchBox>	NameFilterBox;

	/** Extenders for menus */
	TSharedPtr<FExtender> Extenders;

	/** Commands that are bound to delegates*/
	TSharedPtr<FUICommandList_Pinnable> UICommandList;

	/** Pinned commands panel */
	TSharedPtr<IPinnedCommandList> PinnedCommands;

	/** Context name used to persist settings */
	FName ContextName;

	/** The builder we use to construct the tree */
	TWeakPtr<IDisplayClusterConfiguratorTreeBuilder> BuilderPtr;

	TWeakPtr<FDisplayClusterConfiguratorBlueprintEditor> ToolkitPtr;

	TWeakPtr<IDisplayClusterConfiguratorViewTree> ViewTreePtr;
};
