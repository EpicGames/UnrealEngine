// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/Viewport/DisplayClusterConfiguratorSCSEditorViewport.h"
#include "Views/Viewport/DisplayClusterConfiguratorSCSEditorViewportClient.h"
#include "DisplayClusterConfiguratorCommands.h"

#include "DisplayClusterConfiguratorUtils.h"
#include "DisplayClusterProjectionStrings.h"
#include "DisplayClusterConfiguratorBlueprintEditor.h"
#include "Views/DragDrop/DisplayClusterConfiguratorValidatedDragDropOp.h"
#include "Views/DragDrop/DisplayClusterConfiguratorViewportDragDropOp.h"
#include "ClusterConfiguration/ViewModels/DisplayClusterConfiguratorProjectionPolicyViewModel.h"

#include "Components/DisplayClusterScreenComponent.h"
#include "Components/DisplayClusterICVFXCameraComponent.h"

#include "Camera/CameraComponent.h"

#include "EngineUtils.h"
#include "SceneInterface.h"
#include "BlueprintEditor.h"
#include "SSubobjectEditor.h"
#include "EditorViewportCommands.h"
#include "STransformViewportToolbar.h"
#include "SEditorViewportToolBarMenu.h"
#include "SViewportToolBar.h"
#include "AssetEditorViewportLayout.h"
#include "ViewportTabContent.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Slate/SceneViewport.h"
#include "ViewportToolbar/UnrealEdViewportToolbar.h"
#include "Widgets/Input/SNumericEntryBox.h"

#define LOCTEXT_NAMESPACE "DisplayClusterSCSEditorViewport"

class SDisplayClusterConfiguratorSCSEditorViewportToolBar : public SViewportToolBar
{
public:
	SLATE_BEGIN_ARGS(SDisplayClusterConfiguratorSCSEditorViewportToolBar){}
		SLATE_ARGUMENT(TWeakPtr<SEditorViewport>, EditorViewport)
	SLATE_END_ARGS()

	/** Constructs this widget with the given parameters */
	void Construct(const FArguments& InArgs, TSharedPtr<FDisplayClusterConfiguratorSCSEditorViewportClient> InViewportClient)
	{
		ViewportClient = InViewportClient;
		EditorViewport = InArgs._EditorViewport;

		static const FName DefaultForegroundName("DefaultForeground");
		const FMargin ToolbarSlotPadding(4.0f, 1.0f);

		this->ChildSlot
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("EditorViewportToolBar.Background"))
			.ForegroundColor(FAppStyle::Get().GetSlateColor(DefaultForegroundName))
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(ToolbarSlotPadding)
				[
					SNew(SEditorViewportToolbarMenu)
					.ParentToolBar(SharedThis(this))
					.Cursor(EMouseCursor::Default)
					.Image("EditorViewportToolBar.OptionsDropdown")
					.OnGetMenuContent(this, &SDisplayClusterConfiguratorSCSEditorViewportToolBar::GeneratePreviewMenu)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(ToolbarSlotPadding)
				[
					SNew( SEditorViewportToolbarMenu )
					.ParentToolBar( SharedThis( this ) )
					.Cursor( EMouseCursor::Default )
					.Label(this, &SDisplayClusterConfiguratorSCSEditorViewportToolBar::GetCameraMenuLabel)
					.LabelIcon(this, &SDisplayClusterConfiguratorSCSEditorViewportToolBar::GetCameraMenuLabelIcon)
					.OnGetMenuContent(this, &SDisplayClusterConfiguratorSCSEditorViewportToolBar::GenerateCameraMenu)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(ToolbarSlotPadding)
				[
					SNew( SEditorViewportToolbarMenu )
					.ParentToolBar( SharedThis( this ) )
					.Cursor( EMouseCursor::Default )
					.Label(this, &SDisplayClusterConfiguratorSCSEditorViewportToolBar::GetViewMenuLabel)
					.LabelIcon(this, &SDisplayClusterConfiguratorSCSEditorViewportToolBar::GetViewMenuLabelIcon)
					.OnGetMenuContent(this, &SDisplayClusterConfiguratorSCSEditorViewportToolBar::GenerateViewMenu)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(ToolbarSlotPadding)
				[
					SNew( SEditorViewportToolbarMenu )
					.ParentToolBar( SharedThis( this ) )
					.Cursor( EMouseCursor::Default )
					.Label(this, &SDisplayClusterConfiguratorSCSEditorViewportToolBar::GetViewportsMenuLabel)
					.OnGetMenuContent(this, &SDisplayClusterConfiguratorSCSEditorViewportToolBar::GenerateViewportsMenu)
				]
				+ SHorizontalBox::Slot()
				.Padding(ToolbarSlotPadding)
				.HAlign(HAlign_Right)
				[
					SNew(STransformViewportToolBar)
					.Viewport(EditorViewport.Pin().ToSharedRef())
					.CommandList(EditorViewport.Pin()->GetCommandList())
				]
			]
		];

		SViewportToolBar::Construct(SViewportToolBar::FArguments());
	}

	/** Creates the preview menu */
	TSharedRef<SWidget> GeneratePreviewMenu() const
	{
		TSharedPtr<const FUICommandList> CommandList = EditorViewport.IsValid()? EditorViewport.Pin()->GetCommandList(): nullptr;

		const bool bInShouldCloseWindowAfterMenuSelection = true;

		FMenuBuilder PreviewOptionsMenuBuilder(bInShouldCloseWindowAfterMenuSelection, CommandList);
		{
			PreviewOptionsMenuBuilder.BeginSection("BlueprintEditorPreviewOptions", NSLOCTEXT("BlueprintEditor", "PreviewOptionsMenuHeader", "Preview Viewport Options"));
			{
				PreviewOptionsMenuBuilder.AddMenuEntry(FDisplayClusterConfiguratorCommands::Get().ResetCamera);
				PreviewOptionsMenuBuilder.AddMenuEntry(FDisplayClusterConfiguratorCommands::Get().ShowFloor);
				PreviewOptionsMenuBuilder.AddMenuEntry(FDisplayClusterConfiguratorCommands::Get().ShowGrid);
				PreviewOptionsMenuBuilder.AddMenuEntry(FDisplayClusterConfiguratorCommands::Get().ShowOrigin);
				PreviewOptionsMenuBuilder.AddMenuEntry(FDisplayClusterConfiguratorCommands::Get().EnableAA);
				
				{
					PreviewOptionsMenuBuilder.AddSubMenu(
						LOCTEXT("nDisplayConfigLayout", "Layouts"),
						LOCTEXT("nDisplayConfigsSubMenu", "Layouts"), 
						FNewMenuDelegate::CreateSP(this, &SDisplayClusterConfiguratorSCSEditorViewportToolBar::GenerateViewportConfigsMenu));
				}
			}
			PreviewOptionsMenuBuilder.EndSection();
		}

		return PreviewOptionsMenuBuilder.MakeWidget();
	}

	FText GetCameraMenuLabel() const
	{
		if(EditorViewport.IsValid())
		{
			return UE::UnrealEd::GetCameraSubmenuLabelFromViewportType(
				EditorViewport.Pin()->GetViewportClient()->GetViewportType()
			);
		}

		return NSLOCTEXT("BlueprintEditor", "CameraMenuTitle_Default", "Camera");
	}

	const FSlateBrush* GetCameraMenuLabelIcon() const
	{
		if(EditorViewport.IsValid())
		{
			return GetCameraMenuLabelIconFromViewportType( EditorViewport.Pin()->GetViewportClient()->GetViewportType() );
		}

		return FAppStyle::GetBrush(NAME_None);
	}

	TSharedRef<SWidget> GenerateCameraMenu() const
	{
		TSharedPtr<const FUICommandList> CommandList = EditorViewport.IsValid()? EditorViewport.Pin()->GetCommandList(): nullptr;

		const bool bInShouldCloseWindowAfterMenuSelection = true;
		FMenuBuilder CameraMenuBuilder(bInShouldCloseWindowAfterMenuSelection, CommandList);

		CameraMenuBuilder.AddMenuEntry(FEditorViewportCommands::Get().Perspective);

		CameraMenuBuilder.BeginSection("LevelViewportCameraType_Ortho", NSLOCTEXT("BlueprintEditor", "CameraTypeHeader_Ortho", "Orthographic"));
			CameraMenuBuilder.AddMenuEntry(FEditorViewportCommands::Get().Top);
			CameraMenuBuilder.AddMenuEntry(FEditorViewportCommands::Get().Bottom);
			CameraMenuBuilder.AddMenuEntry(FEditorViewportCommands::Get().Left);
			CameraMenuBuilder.AddMenuEntry(FEditorViewportCommands::Get().Right);
			CameraMenuBuilder.AddMenuEntry(FEditorViewportCommands::Get().Front);
			CameraMenuBuilder.AddMenuEntry(FEditorViewportCommands::Get().Back);
		CameraMenuBuilder.EndSection();

		return CameraMenuBuilder.MakeWidget();
	}

	FText GetViewMenuLabel() const
	{
		FText Label = NSLOCTEXT("BlueprintEditor", "ViewMenuTitle_Default", "View");

		if (EditorViewport.IsValid())
		{
			switch (EditorViewport.Pin()->GetViewportClient()->GetViewMode())
			{
			case VMI_Lit:
				Label = NSLOCTEXT("BlueprintEditor", "ViewMenuTitle_Lit", "Lit");
				break;

			case VMI_Unlit:
				Label = NSLOCTEXT("BlueprintEditor", "ViewMenuTitle_Unlit", "Unlit");
				break;

			case VMI_BrushWireframe:
				Label = NSLOCTEXT("BlueprintEditor", "ViewMenuTitle_Wireframe", "Wireframe");
				break;
			}
		}

		return Label;
	}

	const FSlateBrush* GetViewMenuLabelIcon() const
	{
		static FName LitModeIconName("EditorViewport.LitMode");
		static FName UnlitModeIconName("EditorViewport.UnlitMode");
		static FName WireframeModeIconName("EditorViewport.WireframeMode");
		static FName LitWireframeModeIconName("EditorViewport.LitWireframeMode");

		FName Icon = NAME_None;

		if (EditorViewport.IsValid())
		{
			switch (EditorViewport.Pin()->GetViewportClient()->GetViewMode())
			{
			case VMI_Lit:
				Icon = LitModeIconName;
				break;

			case VMI_Unlit:
				Icon = UnlitModeIconName;
				break;

			case VMI_BrushWireframe:
				Icon = WireframeModeIconName;
				break;

			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			case VMI_Lit_Wireframe:
				Icon = LitWireframeModeIconName;
				break;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}
		}

		return FAppStyle::GetBrush(Icon);
	}

	TSharedRef<SWidget> GenerateViewMenu() const
	{
		TSharedPtr<const FUICommandList> CommandList = EditorViewport.IsValid() ? EditorViewport.Pin()->GetCommandList() : nullptr;

		const bool bInShouldCloseWindowAfterMenuSelection = true;
		FMenuBuilder ViewMenuBuilder(bInShouldCloseWindowAfterMenuSelection, CommandList);

		ViewMenuBuilder.AddMenuEntry(FEditorViewportCommands::Get().LitMode, NAME_None, NSLOCTEXT("BlueprintEditor", "LitModeMenuOption", "Lit"));
		ViewMenuBuilder.AddMenuEntry(FEditorViewportCommands::Get().UnlitMode, NAME_None, NSLOCTEXT("BlueprintEditor", "UnlitModeMenuOption", "Unlit"));
		ViewMenuBuilder.AddMenuEntry(FEditorViewportCommands::Get().WireframeMode, NAME_None, NSLOCTEXT("BlueprintEditor", "WireframeModeMenuOption", "Wireframe"));

		return ViewMenuBuilder.MakeWidget();
	}

	FText GetViewportsMenuLabel() const
	{
		FText Label = NSLOCTEXT("BlueprintEditor", "ViewportsMenuTitle_Default", "Viewports");
		return Label;
	}

	TSharedRef<SWidget> GenerateViewportsMenu() const
	{
		const TSharedPtr<const FUICommandList> CommandList = EditorViewport.IsValid() ? EditorViewport.Pin()->GetCommandList() : nullptr;

		const bool bInShouldCloseWindowAfterMenuSelection = true;
		FMenuBuilder ViewportsMenuBuilder(bInShouldCloseWindowAfterMenuSelection, CommandList);

		ViewportsMenuBuilder.BeginSection(TEXT("PreviewScale"), LOCTEXT("PreviewScaleSection", "Preview"));
		{
			ViewportsMenuBuilder.AddMenuEntry(FDisplayClusterConfiguratorCommands::Get().ShowPreview);
			ViewportsMenuBuilder.AddMenuEntry(FDisplayClusterConfiguratorCommands::Get().Show3DViewportNames);

			ViewportsMenuBuilder.AddWidget(
				SNew(SBox)
				.HAlign(HAlign_Right)
				[
					SNew(SBox)
					.Padding(FMargin(8.0f, 0.0f, 0.0f, 0.0f))
					.WidthOverride(100.0f)
					[
						SNew(SBorder)
						.BorderImage(FAppStyle::Get().GetBrush("Menu.WidgetBorder"))
						.Padding(FMargin(1.0f))
						[
							SNew(SNumericEntryBox<float>)
							.Value(ViewportClient.Get(), &FDisplayClusterConfiguratorSCSEditorViewportClient::GetPreviewResolutionScale)
							.OnValueChanged(ViewportClient.Get(), &FDisplayClusterConfiguratorSCSEditorViewportClient::SetPreviewResolutionScale)
							.OnValueCommitted_Lambda([this](const float InValue, ETextCommit::Type)
							{
								if (ViewportClient.IsValid())
								{
									ViewportClient->Invalidate();
								}
							})

							.MinValue(0.05f)
							.MaxValue(1.f)
							.MinSliderValue(0.05f)
							.MaxSliderValue(1.f)
							.AllowSpin(true)
						]
					]
				],
				LOCTEXT("PreviewResolution_Label", "Preview Resolution")
		);
		}
		ViewportsMenuBuilder.EndSection();
		
		ViewportsMenuBuilder.BeginSection(TEXT("XformGizmo"), LOCTEXT("XformGizmoSection", "Xform"));
		{
			ViewportsMenuBuilder.AddMenuEntry(FDisplayClusterConfiguratorCommands::Get().ToggleShowXformGizmos);

			ViewportsMenuBuilder.AddWidget(
				SNew(SBox)
				.HAlign(HAlign_Right)
				[
					SNew(SBox)
					.Padding(FMargin(8.0f, 0.0f, 0.0f, 0.0f))
					.WidthOverride(100.0f)
					[
						SNew(SBorder)
						.BorderImage(FAppStyle::Get().GetBrush("Menu.WidgetBorder"))
						.Padding(FMargin(1.0f))
						[
							SNew(SBox)
							.MinDesiredWidth(64)
							[
								SNew(SNumericEntryBox<float>)
								.Value(ViewportClient.Get(), &FDisplayClusterConfiguratorSCSEditorViewportClient::GetXformGizmoScale)
								.OnValueChanged(ViewportClient.Get(), &FDisplayClusterConfiguratorSCSEditorViewportClient::SetXformGizmoScale)
								.MinValue(0)
								.MaxValue(FLT_MAX)
								.MinSliderValue(0)
								.MaxSliderValue(2)
								.AllowSpin(true)
							]
						]
					]
				],
				LOCTEXT("XformGizmoScale_Label", "Xform Gizmo Scale")
			);
		}
		ViewportsMenuBuilder.EndSection();

		return ViewportsMenuBuilder.MakeWidget();
	}

	
	void GenerateViewportConfigsMenu(FMenuBuilder& MenuBuilder) const
	{
		check(EditorViewport.IsValid());
		TSharedPtr<const FUICommandList> CommandList = EditorViewport.Pin()->GetCommandList();

		MenuBuilder.BeginSection("nDisplayEditorViewportOnePaneConfigs", LOCTEXT("OnePaneConfigHeader", "One Pane"));
		{
			FToolBarBuilder OnePaneButton(CommandList, FMultiBoxCustomization::None);
			OnePaneButton.SetLabelVisibility(EVisibility::Collapsed);
			OnePaneButton.SetStyle(&FAppStyle::Get(), "ViewportLayoutToolbar");

			OnePaneButton.AddToolBarButton(FEditorViewportCommands::Get().ViewportConfig_OnePane);

			MenuBuilder.AddWidget(
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					OnePaneButton.MakeWidget()
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1)
				[
					SNullWidget::NullWidget
				],
				FText::GetEmpty(), true
				);
		}
		MenuBuilder.EndSection();

		MenuBuilder.BeginSection("nDisplayEditorViewportTwoPaneConfigs", LOCTEXT("TwoPaneConfigHeader", "Two Panes"));
		{
			FToolBarBuilder TwoPaneButtons(CommandList, FMultiBoxCustomization::None);
			TwoPaneButtons.SetLabelVisibility(EVisibility::Collapsed);
			TwoPaneButtons.SetStyle(&FAppStyle::Get(), "ViewportLayoutToolbar");

			TwoPaneButtons.AddToolBarButton(FEditorViewportCommands::Get().ViewportConfig_TwoPanesH, NAME_None, FText());
			TwoPaneButtons.AddToolBarButton(FEditorViewportCommands::Get().ViewportConfig_TwoPanesV, NAME_None, FText());

			MenuBuilder.AddWidget(
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					TwoPaneButtons.MakeWidget()
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1)
				[
					SNullWidget::NullWidget
				],
				FText::GetEmpty(), true
				);
		}
		MenuBuilder.EndSection();

		MenuBuilder.BeginSection("nDisplayEditorViewportThreePaneConfigs", LOCTEXT("ThreePaneConfigHeader", "Three Panes"));
		{
			FToolBarBuilder ThreePaneButtons(CommandList, FMultiBoxCustomization::None);
			ThreePaneButtons.SetLabelVisibility(EVisibility::Collapsed);
			ThreePaneButtons.SetStyle(&FAppStyle::Get(), "ViewportLayoutToolbar");

			ThreePaneButtons.AddToolBarButton(FEditorViewportCommands::Get().ViewportConfig_ThreePanesLeft, NAME_None, FText());
			ThreePaneButtons.AddToolBarButton(FEditorViewportCommands::Get().ViewportConfig_ThreePanesRight, NAME_None, FText());
			ThreePaneButtons.AddToolBarButton(FEditorViewportCommands::Get().ViewportConfig_ThreePanesTop, NAME_None, FText());
			ThreePaneButtons.AddToolBarButton(FEditorViewportCommands::Get().ViewportConfig_ThreePanesBottom, NAME_None, FText());

			MenuBuilder.AddWidget(
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					ThreePaneButtons.MakeWidget()
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1)
				[
					SNullWidget::NullWidget
				],
				FText::GetEmpty(), true
				);
		}
		MenuBuilder.EndSection();

		MenuBuilder.BeginSection("nDisplayEditorViewportFourPaneConfigs", LOCTEXT("FourPaneConfigHeader", "Four Panes"));
		{
			FToolBarBuilder FourPaneButtons(CommandList, FMultiBoxCustomization::None);
			FourPaneButtons.SetLabelVisibility(EVisibility::Collapsed);
			FourPaneButtons.SetStyle(&FAppStyle::Get(), "ViewportLayoutToolbar");

			FourPaneButtons.AddToolBarButton(FEditorViewportCommands::Get().ViewportConfig_FourPanes2x2, NAME_None, FText());
			FourPaneButtons.AddToolBarButton(FEditorViewportCommands::Get().ViewportConfig_FourPanesLeft, NAME_None, FText());
			FourPaneButtons.AddToolBarButton(FEditorViewportCommands::Get().ViewportConfig_FourPanesRight, NAME_None, FText());
			FourPaneButtons.AddToolBarButton(FEditorViewportCommands::Get().ViewportConfig_FourPanesTop, NAME_None, FText());
			FourPaneButtons.AddToolBarButton(FEditorViewportCommands::Get().ViewportConfig_FourPanesBottom, NAME_None, FText());

			MenuBuilder.AddWidget(
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					FourPaneButtons.MakeWidget()
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1)
				[
					SNullWidget::NullWidget
				],
				FText::GetEmpty(), true
				);
		}
		MenuBuilder.EndSection();
	}

private:
	/** Reference to the parent viewport */
	TWeakPtr<SEditorViewport> EditorViewport;
	TSharedPtr<FDisplayClusterConfiguratorSCSEditorViewportClient> ViewportClient;
};

void SDisplayClusterConfiguratorSCSEditorViewport::Construct(const FArguments& InArgs, const FAssetEditorViewportConstructionArgs& InViewportConstructionArgs)
{
	bIsActiveTimerRegistered = false;

	// Save off the Blueprint editor reference, we'll need this later
	BlueprintEditorPtr = InArgs._BlueprintEditor;
	OwnerTab = InArgs._OwningTab;
	
	SAssetEditorViewport::Construct(SAssetEditorViewport::FArguments(), InViewportConstructionArgs);

	// Restore last used feature level
	if (ViewportClient.IsValid())
	{
		UWorld* World = ViewportClient->GetPreviewScene()->GetWorld();
		if (World != nullptr && GWorld != nullptr && GWorld->Scene != nullptr)
		{
			World->ShaderPlatformChanged(GWorld->Scene->GetShaderPlatform());
		}
	}

	// Use a delegate to inform the attached world of preview platform changes.
	UEditorEngine* Editor = (UEditorEngine*)GEngine;
	PreviewFeatureLevelChangedHandle = Editor->OnPreviewShaderPlatformChanged().AddLambda([this](EShaderPlatform NewShaderPlatform)
	{
		if (ViewportClient.IsValid())
		{
			UWorld* World = ViewportClient->GetPreviewScene()->GetWorld();
			if (World != nullptr)
			{
				World->ShaderPlatformChanged(NewShaderPlatform);

				// Refresh the preview scene. Don't change the camera.
				RequestRefresh(false);
			}
		}
	});

	// Refresh the preview scene
	RequestRefresh(true);
}

SDisplayClusterConfiguratorSCSEditorViewport::~SDisplayClusterConfiguratorSCSEditorViewport()
{
	UEditorEngine* Editor = (UEditorEngine*)GEngine;
	Editor->OnPreviewShaderPlatformChanged().Remove(PreviewFeatureLevelChangedHandle);

	OwnerTab.Reset();
}

void SDisplayClusterConfiguratorSCSEditorViewport::Invalidate()
{
	ViewportClient->Invalidate();
}

void SDisplayClusterConfiguratorSCSEditorViewport::RequestRefresh(bool bResetCamera, bool bRefreshNow)
{
	if (bRefreshNow)
	{
		if (ViewportClient.IsValid())
		{
			ViewportClient->InvalidatePreview(bResetCamera);
		}
	}
	else
	{
		// Defer the update until the next tick. This way we don't accidentally spawn the preview actor in the middle of a transaction, for example.
		if (!bIsActiveTimerRegistered)
		{
			bIsActiveTimerRegistered = true;
			RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateSP(this, &SDisplayClusterConfiguratorSCSEditorViewport::DeferredUpdatePreview, bResetCamera));
		}
	}
}

void SDisplayClusterConfiguratorSCSEditorViewport::SetOwnerTab(TSharedRef<SDockTab> Tab)
{
	OwnerTab = Tab;
}

TSharedPtr<SDockTab> SDisplayClusterConfiguratorSCSEditorViewport::GetOwnerTab() const
{
	return OwnerTab.Pin();
}

void SDisplayClusterConfiguratorSCSEditorViewport::OnComponentSelectionChanged()
{
	// When the component selection changes, make sure to invalidate hit proxies to sync with the current selection
	SceneViewport->Invalidate();
}

TSharedRef<FEditorViewportClient> SDisplayClusterConfiguratorSCSEditorViewport::MakeEditorViewportClient()
{
	FPreviewScene* PreviewScene = BlueprintEditorPtr.Pin()->GetPreviewScene();

	// Construct a new viewport client instance.
	ViewportClient = MakeShareable(new FDisplayClusterConfiguratorSCSEditorViewportClient(BlueprintEditorPtr, PreviewScene, SharedThis(this)));
	ViewportClient->SetRealtime(true);
	ViewportClient->bSetListenerPosition = false;
	ViewportClient->VisibilityDelegate.BindSP(this, &SDisplayClusterConfiguratorSCSEditorViewport::IsVisible);

	return ViewportClient.ToSharedRef();
}

void SDisplayClusterConfiguratorSCSEditorViewport::PopulateViewportOverlays(TSharedRef<SOverlay> Overlay)
{
	SEditorViewport::PopulateViewportOverlays(Overlay);

	Overlay->AddSlot()
		.VAlign(VAlign_Top)
		[
			SNew(SDisplayClusterConfiguratorSCSEditorViewportToolBar, ViewportClient)
			.EditorViewport(SharedThis(this))
			.IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute())
		];

	// add the feature level display widget
	Overlay->AddSlot()
		.VAlign(VAlign_Bottom)
		.HAlign(HAlign_Right)
		.Padding(5.0f)
		[
			BuildShaderPlatformWidget()
		];
}

void SDisplayClusterConfiguratorSCSEditorViewport::BindCommands()
{
	TSharedPtr<FBlueprintEditor> BlueprintEditor = BlueprintEditorPtr.Pin();
	CommandList->Append(BlueprintEditor->GetSubobjectEditor()->GetCommandList().ToSharedRef());
	CommandList->Append(BlueprintEditor->GetToolkitCommands());
	
	const FDisplayClusterConfiguratorCommands& Commands = FDisplayClusterConfiguratorCommands::Get();

	// Toggle camera lock on/off
	CommandList->MapAction(
		Commands.ResetCamera,
		FExecuteAction::CreateSP(ViewportClient.Get(), &FDisplayClusterConfiguratorSCSEditorViewportClient::ResetCamera));

	CommandList->MapAction(
		Commands.ShowFloor,
		FExecuteAction::CreateSP(ViewportClient.Get(), &FDisplayClusterConfiguratorSCSEditorViewportClient::ToggleShowFloor),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(ViewportClient.Get(), &FDisplayClusterConfiguratorSCSEditorViewportClient::GetShowFloor));

	CommandList->MapAction(
		Commands.ShowGrid,
		FExecuteAction::CreateSP(ViewportClient.Get(), &FDisplayClusterConfiguratorSCSEditorViewportClient::ToggleShowGrid),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(ViewportClient.Get(), &FDisplayClusterConfiguratorSCSEditorViewportClient::GetShowGrid));
	
	CommandList->MapAction(
		Commands.ShowOrigin,
		FExecuteAction::CreateSP(ViewportClient.Get(), &FDisplayClusterConfiguratorSCSEditorViewportClient::ToggleShowOrigin),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(ViewportClient.Get(), &FDisplayClusterConfiguratorSCSEditorViewportClient::GetShowOrigin));

	CommandList->MapAction(
		Commands.EnableAA,
		FExecuteAction::CreateSP(ViewportClient.Get(), &FDisplayClusterConfiguratorSCSEditorViewportClient::ToggleEnableAA),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(ViewportClient.Get(), &FDisplayClusterConfiguratorSCSEditorViewportClient::GetEnableAA));
	
	CommandList->MapAction(
		Commands.ShowPreview,
		FExecuteAction::CreateSP(ViewportClient.Get(), &FDisplayClusterConfiguratorSCSEditorViewportClient::ToggleShowPreview),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(ViewportClient.Get(), &FDisplayClusterConfiguratorSCSEditorViewportClient::GetShowPreview));

	CommandList->MapAction(
		Commands.Show3DViewportNames,
		FExecuteAction::CreateSP(ViewportClient.Get(), &FDisplayClusterConfiguratorSCSEditorViewportClient::ToggleShowViewportNames),
		FCanExecuteAction::CreateSP(ViewportClient.Get(), &FDisplayClusterConfiguratorSCSEditorViewportClient::CanToggleViewportNames),
		FIsActionChecked::CreateSP(ViewportClient.Get(), &FDisplayClusterConfiguratorSCSEditorViewportClient::GetShowViewportNames));

	CommandList->MapAction(
		Commands.ToggleShowXformGizmos,
		FExecuteAction::CreateSP(ViewportClient.Get(), &FDisplayClusterConfiguratorSCSEditorViewportClient::ToggleShowXformGizmos),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(ViewportClient.Get(), &FDisplayClusterConfiguratorSCSEditorViewportClient::IsShowingXformGizmos));
	
	SAssetEditorViewport::BindCommands();
}

void SDisplayClusterConfiguratorSCSEditorViewport::OnDragLeave(const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FDisplayClusterConfiguratorValidatedDragDropOp> ValidatedDragDropOp = DragDropEvent.GetOperationAs<FDisplayClusterConfiguratorValidatedDragDropOp>();
	if (ValidatedDragDropOp.IsValid())
	{
		// Use an opt-in policy for drag-drop operations, always marking it as invalid until another widget marks it as a valid drop operation
		ValidatedDragDropOp->SetDropAsInvalid();
		return;
	}

	SAssetEditorViewport::OnDragLeave(DragDropEvent);
}

FReply SDisplayClusterConfiguratorSCSEditorViewport::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FDisplayClusterConfiguratorViewportDragDropOp> ViewportDragDropOp = DragDropEvent.GetOperationAs<FDisplayClusterConfiguratorViewportDragDropOp>();
	if (ViewportDragDropOp.IsValid())
	{
		FIntPoint ViewportOrigin, ViewportSize;
		ViewportClient->GetViewportDimensions(ViewportOrigin, ViewportSize);

		const FVector2D MousePos = MyGeometry.AbsoluteToLocal(DragDropEvent.GetScreenSpacePosition()) * MyGeometry.Scale - ViewportOrigin;
		HHitProxy* HitProxy = ViewportClient->GetHitProxyWithoutGizmos(MousePos.X, MousePos.Y);
		if (HActor* ActorProxy = HitProxyCast<HActor>(HitProxy))
		{
			if (ActorProxy->Actor && ActorProxy->PrimComponent)
			{
				if (const UStaticMeshComponent* MeshComponent = Cast<UStaticMeshComponent>(ActorProxy->PrimComponent))
				{
					if (const UDisplayClusterScreenComponent* ScreenComponent = Cast<UDisplayClusterScreenComponent>(MeshComponent))
					{
						ViewportDragDropOp->SetDropAsValid(FText::Format(LOCTEXT("ViewportDragDropOp_ScreenMessage", "Project to screen {0}"), FText::FromName(ScreenComponent->GetFName())));
					}
					else
					{
						USceneComponent* AttachParent = MeshComponent->GetAttachParent();
						const bool bIsCamera = AttachParent && (AttachParent->IsA<UCameraComponent>());
						
						if (bIsCamera)
						{
							ViewportDragDropOp->SetDropAsValid(FText::Format(LOCTEXT("ViewportDragDropOp_CameraMessage", "Project to camera {0}"), FText::FromName(AttachParent->GetFName())));
						}
						else if (!MeshComponent->IsVisualizationComponent())
						{
							ViewportDragDropOp->SetDropAsValid(FText::Format(LOCTEXT("ViewportDragDropOp_MeshMessage", "Project to mesh {0}"), FText::FromName(MeshComponent->GetFName())));
						}
					}

					return FReply::Handled();
				}
			}
		}

		ViewportDragDropOp->SetDropAsInvalid();
	}

	return SAssetEditorViewport::OnDragOver(MyGeometry, DragDropEvent);
}

FReply SDisplayClusterConfiguratorSCSEditorViewport::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FDisplayClusterConfiguratorViewportDragDropOp> ViewportDragDropOp = DragDropEvent.GetOperationAs<FDisplayClusterConfiguratorViewportDragDropOp>();
	if (ViewportDragDropOp.IsValid())
	{
		FIntPoint ViewportOrigin, ViewportSize;
		ViewportClient->GetViewportDimensions(ViewportOrigin, ViewportSize);

		const FVector2D MousePos = MyGeometry.AbsoluteToLocal(DragDropEvent.GetScreenSpacePosition()) * MyGeometry.Scale - ViewportOrigin;
		HHitProxy* HitProxy = ViewportClient->GetHitProxyWithoutGizmos(MousePos.X, MousePos.Y);
		if (HActor* ActorProxy = HitProxyCast<HActor>(HitProxy))
		{
			if (ActorProxy->Actor && ActorProxy->PrimComponent)
			{
				if (const UStaticMeshComponent* MeshComponent = Cast<UStaticMeshComponent>(ActorProxy->PrimComponent))
				{
					FString PolicyType;
					FString ParameterKey;
					FString ComponentName;

					USceneComponent* AttachParent = MeshComponent->GetAttachParent();
					const bool bIsCamera = AttachParent && (AttachParent->IsA<UCameraComponent>());

					if (bIsCamera)
					{
						PolicyType = DisplayClusterProjectionStrings::projection::Camera;
						ParameterKey = DisplayClusterProjectionStrings::cfg::camera::Component;
						ComponentName = AttachParent->GetName();
					}
					else if (!MeshComponent->IsVisualizationComponent())
					{
						PolicyType = DisplayClusterProjectionStrings::projection::Mesh;
						ParameterKey = DisplayClusterProjectionStrings::cfg::mesh::Component;
						ComponentName = MeshComponent->GetName();
					}

					if (!PolicyType.IsEmpty() && !ParameterKey.IsEmpty() && !ComponentName.IsEmpty())
					{
						FScopedTransaction Transaction(FText::Format(LOCTEXT("ProjectViewports", "Project {0}|plural(one=Viewport, other=Viewports)"), ViewportDragDropOp->GetDraggedViewports().Num()));

						bool bClusterModified = false;
						for (TWeakObjectPtr<UDisplayClusterConfigurationViewport> Viewport : ViewportDragDropOp->GetDraggedViewports())
						{
							if (Viewport.IsValid())
							{
								bClusterModified = true;

								FDisplayClusterConfiguratorProjectionPolicyViewModel PPViewModel(Viewport.Get());
								PPViewModel.ClearParameters();
								PPViewModel.SetPolicyType(PolicyType);
								PPViewModel.SetIsCustom(false);
								PPViewModel.SetParameterValue(ParameterKey, ComponentName);
							}
						}

						if (bClusterModified)
						{
							if (TSharedPtr<FDisplayClusterConfiguratorBlueprintEditor> DCBlueprintEditor = StaticCastSharedPtr<FDisplayClusterConfiguratorBlueprintEditor>(BlueprintEditorPtr.Pin()))
							{
								DCBlueprintEditor->ClusterChanged();
								DCBlueprintEditor->RefreshDisplayClusterPreviewActor();
								DCBlueprintEditor->RefreshInspector();
							}

							return FReply::Handled();
						}
						else
						{
							Transaction.Cancel();
						}
					}
				}
			}
		}
	}

	return SAssetEditorViewport::OnDrop(MyGeometry, DragDropEvent);
}

void SDisplayClusterConfiguratorSCSEditorViewport::OnFocusViewportToSelection()
{
	ViewportClient->FocusViewportToSelection();
}

EActiveTimerReturnType SDisplayClusterConfiguratorSCSEditorViewport::DeferredUpdatePreview(double InCurrentTime,
                                                                                           float InDeltaTime, bool bResetCamera)
{
	if (ViewportClient.IsValid())
	{
		ViewportClient->InvalidatePreview(bResetCamera);
	}

	bIsActiveTimerRegistered = false;
	return EActiveTimerReturnType::Stop;
}

#undef LOCTEXT_NAMESPACE
