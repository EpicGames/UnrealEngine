// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "SAssetEditorViewport.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class FBlueprintEditor;
class FDisplayClusterConfiguratorSCSEditorViewportClient;

class SDisplayClusterConfiguratorSCSEditorViewport : public SAssetEditorViewport
{
	SLATE_BEGIN_ARGS(SDisplayClusterConfiguratorSCSEditorViewport) {}
		SLATE_ARGUMENT(TWeakPtr<FBlueprintEditor>, BlueprintEditor)
		SLATE_ARGUMENT(TSharedPtr<SDockTab>, OwningTab)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const FAssetEditorViewportConstructionArgs& InViewportConstructionArgs);

	~SDisplayClusterConfiguratorSCSEditorViewport();
	
	/**
	 * Invalidates the viewport client
	 */
	void Invalidate();
	
	/**
	 * Request a refresh of the preview scene/world. Will recreate actors as needed.
	 *
	 * @param bResetCamera If true, the camera will be reset to its default position based on the preview.
	 * @param bRefreshNow If true, the preview will be refreshed immediately. Otherwise, it will be deferred until the next tick (default behavior).
	 */
	void RequestRefresh(bool bResetCamera = false, bool bRefreshNow = false);
	
	/**
	 * Called when the selected component changes in the SCS editor.
	 */
	void OnComponentSelectionChanged();

	void SetOwnerTab(TSharedRef<SDockTab> Tab);
	TSharedPtr<SDockTab> GetOwnerTab() const;
	TSharedPtr<FDisplayClusterConfiguratorSCSEditorViewportClient> GetDisplayClusterViewportClient() const { return ViewportClient; }

protected:
	/** SEditorViewport interface */
	virtual TSharedRef<FEditorViewportClient> MakeEditorViewportClient() override;
	virtual void PopulateViewportOverlays(TSharedRef<SOverlay> Overlay) override;
	virtual void BindCommands() override;
	virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	/** End SEditorViewport interface */
	
	/**
	 * Focuses the viewport on the currently selected components
	 */
	virtual void OnFocusViewportToSelection() override;

private:
	/** One-off active timer to update the preview */
	EActiveTimerReturnType DeferredUpdatePreview(double InCurrentTime, float InDeltaTime, bool bResetCamera);

private:
	TWeakPtr<FBlueprintEditor> BlueprintEditorPtr;
	TSharedPtr<FDisplayClusterConfiguratorSCSEditorViewportClient> ViewportClient;

	/** Whether the active timer (for updating the preview) is registered */
	bool bIsActiveTimerRegistered;
	
	/** The owner dock tab for this viewport. */
	TWeakPtr<SDockTab> OwnerTab;
	/** Handle to the registered OnPreviewFeatureLevelChanged delegate. */
	FDelegateHandle PreviewFeatureLevelChangedHandle;
};