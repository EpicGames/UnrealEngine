// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/Viewport/DisplayClusterConfiguratorSCSEditorViewportClient.h"
#include "Materials/Material.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "Views/Viewport/DisplayClusterConfiguratorSCSEditorViewport.h"
#include "DisplayClusterConfiguratorBlueprintEditor.h"
#include "DisplayClusterWorldOriginComponent.h"
#include "Settings/DisplayClusterConfiguratorSettings.h"
#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterConfigurationStrings.h"
#include "DisplayClusterConfiguratorPropertyUtils.h"

#include "DisplayClusterRootActor.h"
#include "ProceduralMeshComponent.h"
#include "Components/DisplayClusterCameraComponent.h"
#include "Components/LineBatchComponent.h"

#include "AssetViewerSettings.h"
#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "Camera/CameraComponent.h"
#include "ComponentVisualizer.h"
#include "EditorModeManager.h"
#include "EngineUtils.h"
#include "ISCSEditorCustomization.h"
#include "SSubobjectEditor.h"
#include "UnrealEdGlobals.h"
#include "UnrealWidget.h"
#include "Components/PostProcessComponent.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "Editor/UnrealEdEngine.h"
#include "Kismet2/ComponentEditorUtils.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Materials/MaterialInstanceConstant.h"
#include "PhysicsEngine/PhysicsConstraintComponent.h"
#include "SKismetInspector.h"
#include "SnappingUtils.h"

FDisplayClusterConfiguratorSCSEditorViewportClient::FDisplayClusterConfiguratorSCSEditorViewportClient(
	TWeakPtr<FBlueprintEditor>& InBlueprintEditorPtr,
	FPreviewScene* InPreviewScene,
	const TSharedRef<SDisplayClusterConfiguratorSCSEditorViewport>& InSCSEditorViewport
)
	: FEditorViewportClient(
		InBlueprintEditorPtr.Pin() ? &StaticCastSharedPtr<FDisplayClusterConfiguratorBlueprintEditor>(InBlueprintEditorPtr.Pin())->GetModeTools().Get() : nullptr,
		InPreviewScene,
		StaticCastSharedRef<SEditorViewport>(InSCSEditorViewport))
	, BlueprintEditorPtr(StaticCastSharedPtr<FDisplayClusterConfiguratorBlueprintEditor>(InBlueprintEditorPtr.Pin()))
	, PreviewActorBounds(ForceInitToZero)
	, bIsManipulating(false)
	, ScopedTransaction(nullptr)
	, bIsSimulateEnabled(false)
{
	WidgetMode = UE::Widget::WM_Translate;
	WidgetCoordSystem = COORD_Local;

	check(Widget);
	Widget->SetSnapEnabled(true);

	// Selectively set particular show flags that we need
	EngineShowFlags.SetSelectionOutline(GetDefault<ULevelEditorViewportSettings>()->bUseSelectionOutline);

	DefaultSettings = UAssetViewerSettings::Get();
	check(DefaultSettings);

	const UDisplayClusterConfiguratorEditorSettings* DisplayClusterSettings = GetDefault<UDisplayClusterConfiguratorEditorSettings>();
	
	ShowWidget(true);

	SetViewMode(VMI_Lit);

	SyncEditorSettings();

	EngineShowFlags.AntiAliasing = DisplayClusterSettings->bEditorEnableAA;
	EngineShowFlags.EyeAdaptation = false;
	
	//OverrideNearClipPlane(1.0f);
	bUsingOrbitCamera = true;

	//This seems to be needed to get the correct world time in the preview.
	SetIsSimulateInEditorViewport(true);
	ResetCamera();
	
	const FTransform Transform(FRotator(0, 0, 0), FVector(0, 0, 0), FVector(1));
	
	CurrentProfileIndex = DefaultSettings->Profiles.IsValidIndex(CurrentProfileIndex) ? GetDefault<UEditorPerProjectUserSettings>()->AssetViewerProfileIndex : 0;
	ensureMsgf(DefaultSettings->Profiles.IsValidIndex(CurrentProfileIndex), TEXT("Invalid default settings pointer or current profile index"));
	FPreviewSceneProfile& Profile = DefaultSettings->Profiles[CurrentProfileIndex];
	Profile.LoadEnvironmentMap();

	// Always set up sky light using the set cube map texture, reusing the sky light from PreviewScene class
	PreviewScene->SetSkyCubemap(Profile.EnvironmentCubeMap.Get());
	PreviewScene->SetSkyBrightness(Profile.SkyLightIntensity);

	// Large scale to prevent sphere from clipping
	const FTransform SphereTransform(FRotator(0, 0, 0), FVector(0, 0, 0), FVector(2000));
	SkyComponent = NewObject<UStaticMeshComponent>(GetTransientPackage());
	// Set up sky sphere showing hte same cube map as used by the sky light
	UStaticMesh* SkySphere = LoadObject<UStaticMesh>(nullptr, TEXT("/Engine/EditorMeshes/AssetViewer/Sphere_inversenormals.Sphere_inversenormals"), nullptr, LOAD_None, nullptr);
	check(SkySphere);
	SkyComponent->SetStaticMesh(SkySphere);
	SkyComponent->SetCollisionEnabled(ECollisionEnabled::NoCollision);
	SkyComponent->bVisibleInRayTracing = false;
	
	UMaterial* SkyMaterial = LoadObject<UMaterial>(nullptr, TEXT("/Engine/EditorMaterials/AssetViewer/M_SkyBox.M_SkyBox"), nullptr, LOAD_None, nullptr);
	check(SkyMaterial);

	InstancedSkyMaterial = NewObject<UMaterialInstanceConstant>(GetTransientPackage());
	InstancedSkyMaterial->Parent = SkyMaterial;
	
	UTextureCube* DefaultTexture = LoadObject<UTextureCube>(nullptr, TEXT("/Engine/MapTemplates/Sky/SunsetAmbientCubemap.SunsetAmbientCubemap"));

	InstancedSkyMaterial->SetTextureParameterValueEditorOnly(FName("SkyBox"), (Profile.EnvironmentCubeMap.Get() != nullptr) ? Profile.EnvironmentCubeMap.Get() : DefaultTexture);
	InstancedSkyMaterial->SetScalarParameterValueEditorOnly(FName("CubemapRotation"), Profile.LightingRigRotation / 360.0f);
	InstancedSkyMaterial->SetScalarParameterValueEditorOnly(FName("Intensity"), Profile.SkyLightIntensity);
	InstancedSkyMaterial->PostLoad();
	SkyComponent->SetMaterial(0, InstancedSkyMaterial);
	PreviewScene->AddComponent(SkyComponent, SphereTransform);

	PostProcessComponent = NewObject<UPostProcessComponent>();
	PostProcessComponent->Settings = Profile.PostProcessingSettings;
	PostProcessComponent->bUnbound = true;
	PreviewScene->AddComponent(PostProcessComponent, Transform);
	
	EditorFloorComp = NewObject<UStaticMeshComponent>(GetTransientPackage(),
		MakeUniqueObjectName(GetTransientPackage(), UStaticMeshComponent::StaticClass(), TEXT("EditorFloorComp")));

#if 0

#if WITH_EDITOR
	//@todo: need decision - preview with grid or without?
	// Hide FloorGrid in preview
	EditorFloorComp->SetIsVisualizationComponent(true);
	EditorFloorComp->bHiddenInSceneCapture = true;
	EditorFloorComp->bHiddenInGame = true;
#endif /*WITH_EDITOR*/

#endif

	UStaticMesh* FloorMesh = LoadObject<UStaticMesh>(nullptr, TEXT("/nDisplay/Meshes/sm_nDisplayGrid.sm_nDisplayGrid"), nullptr, LOAD_None, nullptr);
	if (ensure(FloorMesh))
	{
		EditorFloorComp->SetStaticMesh(FloorMesh);
	}

	UMaterial* Material = LoadObject<UMaterial>(nullptr, TEXT("/nDisplay/Materials/Editor/M_nDisplayGrid.M_nDisplayGrid"), nullptr, LOAD_None, nullptr);
	if (ensure(Material))
	{
		EditorFloorComp->SetMaterial(0, Material);
	}
	EditorFloorComp->SetRelativeScale3D(FVector(1.f, 1.f, 1.f));
	EditorFloorComp->SetVisibility(DisplayClusterSettings->bEditorShowFloor);
	EditorFloorComp->SetCollisionEnabled(DisplayClusterSettings->bEditorShowFloor ? ECollisionEnabled::QueryAndPhysics : ECollisionEnabled::NoCollision);
	PreviewScene->AddComponent(EditorFloorComp, FTransform::Identity);

	WorldOriginComponent = NewObject<UDisplayClusterWorldOriginComponent>(GetTransientPackage(),
		MakeUniqueObjectName(GetTransientPackage(), UDisplayClusterWorldOriginComponent::StaticClass(), TEXT("WorldOriginComp")));
	WorldOriginComponent->SetVisibility(DisplayClusterSettings->bEditorShowWorldOrigin);
	WorldOriginComponent->SetCollisionEnabled(ECollisionEnabled::NoCollision);
	PreviewScene->AddComponent(WorldOriginComponent, FTransform::Identity);
}

FDisplayClusterConfiguratorSCSEditorViewportClient::~FDisplayClusterConfiguratorSCSEditorViewportClient()
{
	// Ensure that an in-progress transaction is ended
	EndTransaction();
}

namespace
{
	/** Automatic translation applied to the camera in the default editor viewport logic when orbit mode is enabled. */
	const float AutoViewportOrbitCameraTranslate = 256.0f;

	void DrawAngles(FCanvas* Canvas, int32 XPos, int32 YPos,
		EAxisList::Type ManipAxis, UE::Widget::EWidgetMode MoveMode,
		const FRotator& Rotation, const FVector& Translation, float DPI)
	{
		FString OutputString(TEXT(""));
		if (MoveMode == UE::Widget::WM_Rotate && Rotation.IsZero() == false)
		{
			//Only one value moves at a time
			const FVector EulerAngles = Rotation.Euler();
			if (ManipAxis == EAxisList::X)
			{
				OutputString += FString::Printf(TEXT("Roll: %0.2f"), EulerAngles.X);
			}
			else if (ManipAxis == EAxisList::Y)
			{
				OutputString += FString::Printf(TEXT("Pitch: %0.2f"), EulerAngles.Y);
			}
			else if (ManipAxis == EAxisList::Z)
			{
				OutputString += FString::Printf(TEXT("Yaw: %0.2f"), EulerAngles.Z);
			}
		}
		else if (MoveMode == UE::Widget::WM_Translate && Translation.IsZero() == false)
		{
			//Only one value moves at a time
			if (ManipAxis == EAxisList::X)
			{
				OutputString += FString::Printf(TEXT(" %0.2f"), Translation.X);
			}
			else if (ManipAxis == EAxisList::Y)
			{
				OutputString += FString::Printf(TEXT(" %0.2f"), Translation.Y);
			}
			else if (ManipAxis == EAxisList::Z)
			{
				OutputString += FString::Printf(TEXT(" %0.2f"), Translation.Z);
			}
		}

		if (OutputString.Len() > 0)
		{
			const FVector2D AdjustedPosition(XPos / DPI, YPos / DPI);
			
			FCanvasTextItem TextItem(AdjustedPosition, FText::FromString(OutputString), GEngine->GetSmallFont(), FLinearColor::White);
			Canvas->DrawItem(TextItem);
		}
	}

	// Determine whether or not the given node has a parent node that is not the root node, is editable and is selected
	bool IsMovableParentNodeSelected(const FSubobjectEditorTreeNodePtrType& NodePtr, const TArray<FSubobjectEditorTreeNodePtrType>& SelectedNodes)
	{
		if (NodePtr.IsValid())
		{
			// Check for a valid parent node
			FSubobjectEditorTreeNodePtrType ParentNodePtr = NodePtr->GetParent();
			FSubobjectData* ParentData = ParentNodePtr.IsValid() ? ParentNodePtr->GetDataSource() : nullptr;
			if (ParentData && !ParentData->IsRootComponent())
			{
				if (SelectedNodes.Contains(ParentNodePtr))
				{
					// The parent node is not the root node and is also selected; success
					return true;
				}
				else
				{
					// Recursively search for any other parent nodes farther up the tree that might be selected
					return IsMovableParentNodeSelected(ParentNodePtr, SelectedNodes);
				}
			}
		}

		return false;
	}
}

void FDisplayClusterConfiguratorSCSEditorViewportClient::Tick(float DeltaSeconds)
{
	FEditorViewportClient::Tick(DeltaSeconds);

	// Register the selection override delegate for the preview actor's components
	TSharedPtr<SSubobjectEditor> SubobjectEditor = BlueprintEditorPtr.Pin()->GetSubobjectEditor();
	AActor* PreviewActor = GetPreviewActor();
	if (PreviewActor != nullptr)
	{
		for (UActorComponent* Component : PreviewActor->GetComponents())
		{
			if (UPrimitiveComponent* PrimComponent = Cast<UPrimitiveComponent>(Component))
			{
				if (!PrimComponent->SelectionOverrideDelegate.IsBound())
				{
					SubobjectEditor->SetSelectionOverride(PrimComponent);
				}
			}
		}

		SyncShowPreview();
	}
	else
	{
		InvalidatePreview(false);
	}

	// Tick the preview scene world.
	if (!GIntraFrameDebuggingGameThread)
	{
		// Ensure that the preview actor instance is up-to-date for component editing (e.g. after compiling the Blueprint, the actor may be reinstanced outside of this class)
		if (PreviewActor != BlueprintEditorPtr.Pin()->GetBlueprintObj()->SimpleConstructionScript->GetComponentEditorActorInstance())
		{
			BlueprintEditorPtr.Pin()->GetBlueprintObj()->SimpleConstructionScript->SetComponentEditorActorInstance(PreviewActor);
		}

		// Allow full tick only if preview simulation is enabled and we're not currently in an active SIE or PIE session
		if (bIsSimulateEnabled && GEditor->PlayWorld == nullptr && !GEditor->bIsSimulatingInEditor)
		{
			PreviewScene->GetWorld()->Tick(IsRealtime() ? LEVELTICK_All : LEVELTICK_TimeOnly, DeltaSeconds);
		}
		else
		{
			PreviewScene->GetWorld()->Tick(IsRealtime() ? LEVELTICK_ViewportsOnly : LEVELTICK_TimeOnly, DeltaSeconds);
		}
	}
}

void FDisplayClusterConfiguratorSCSEditorViewportClient::Draw(const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	FEditorViewportClient::Draw(View, PDI);

	// mimic the behaviour of the FEditorViewportClient call used in editor viewport
	constexpr const UWorld::ELineBatcherType LineBatchersToFlush[] = { UWorld::ELineBatcherType::World, UWorld::ELineBatcherType::Foreground };
	PreviewScene->GetWorld()->FlushLineBatchers(LineBatchersToFlush);

	bool bHitTesting = PDI->IsHitTesting();
	AActor* PreviewActor = GetPreviewActor();
	if (PreviewActor)
	{
		if (GUnrealEd != nullptr)
		{
			TArray<FSubobjectEditorTreeNodePtrType> SelectedNodes = BlueprintEditorPtr.Pin()->GetSelectedSubobjectEditorTreeNodes();
			for (int32 SelectionIndex = 0; SelectionIndex < SelectedNodes.Num(); ++SelectionIndex)
			{
				FSubobjectEditorTreeNodePtrType SelectedNode = SelectedNodes[SelectionIndex];

				if (SelectedNode.IsValid())
				{
					if (const FSubobjectData* Data = SelectedNode->GetDataSource())
					{
						const UActorComponent* Comp = Data->FindComponentInstanceInActor(PreviewActor);
						if (Comp != nullptr && Comp->IsRegistered())
						{
							// Try and find a visualizer
							TSharedPtr<FComponentVisualizer> Visualizer = GUnrealEd->FindComponentVisualizer(Comp->GetClass());
							if (Visualizer.IsValid())
							{
								Visualizer->DrawVisualization(Comp, View, PDI);
							}
						}
					}
				}
			}
		}
	}
}

void FDisplayClusterConfiguratorSCSEditorViewportClient::DrawCanvas(FViewport& InViewport, FSceneView& View, FCanvas& Canvas)
{
	AActor* PreviewActor = GetPreviewActor();
	if (PreviewActor)
	{
		if (GUnrealEd != nullptr)
		{
			TArray<FSubobjectEditorTreeNodePtrType> SelectedNodes = BlueprintEditorPtr.Pin()->GetSelectedSubobjectEditorTreeNodes();
			for (int32 SelectionIndex = 0; SelectionIndex < SelectedNodes.Num(); ++SelectionIndex)
			{
				FSubobjectEditorTreeNodePtrType SelectedNode = SelectedNodes[SelectionIndex];
				FSubobjectData* Data = SelectedNode.IsValid() ? SelectedNode->GetDataSource() : nullptr;
				const UActorComponent* Comp = Data ? Data->FindComponentInstanceInActor(PreviewActor) : nullptr;
				if (Comp != nullptr && Comp->IsRegistered())
				{
					// Try and find a visualizer
					TSharedPtr<FComponentVisualizer> Visualizer = GUnrealEd->FindComponentVisualizer(Comp->GetClass());
					if (Visualizer.IsValid())
					{
						Visualizer->DrawVisualizationHUD(Comp, &InViewport, &View, &Canvas);
					}
				}
			}
		}

		TGuardValue<bool> AutoRestore(GAllowActorScriptExecutionInEditor, true);

		const int32 HalfX = 0.5f * Viewport->GetSizeXY().X;
		const int32 HalfY = 0.5f * Viewport->GetSizeXY().Y;

		const TArray<FSubobjectEditorTreeNodePtrType>& SelectedNodes = BlueprintEditorPtr.Pin()->GetSelectedSubobjectEditorTreeNodes();
		if (bIsManipulating && SelectedNodes.Num() > 0 && SelectedNodes[0].IsValid())
		{
			if (const FSubobjectData* Data = SelectedNodes[0]->GetDataSource())
			{
				if (const USceneComponent* SceneComp = Cast<USceneComponent>(Data->FindComponentInstanceInActor(PreviewActor)))
				{
					const FVector WidgetLocation = GetWidgetLocation();
					const FPlane Proj = View.Project(WidgetLocation);
					if (Proj.W > 0.0f)
					{
						const int32 XPos = HalfX + (HalfX * Proj.X);
						const int32 YPos = HalfY + (HalfY * (Proj.Y * -1));
						DrawAngles(
							&Canvas,
							XPos,
							YPos,
							GetCurrentWidgetAxis(),
							GetWidgetMode(),
							GetWidgetCoordSystem().Rotator(),
							WidgetLocation,
							UpdateViewportClientWindowDPIScale());
					}
				}
			}
		}
	}

	const UDisplayClusterConfiguratorEditorSettings* Settings = GetDefault<UDisplayClusterConfiguratorEditorSettings>();
	if (Settings->bEditorShow3DViewportNames)
	{
		DisplayViewportInformation(View, Canvas);
	}
}

bool FDisplayClusterConfiguratorSCSEditorViewportClient::InputKey(const FInputKeyEventArgs& EventArgs)
{
	bool bHandled = GUnrealEd->ComponentVisManager.HandleInputKey(this, EventArgs.Viewport, EventArgs.Key, EventArgs.Event);

	if (!bHandled)
	{
		bHandled = FEditorViewportClient::InputKey(EventArgs);
	}

	return bHandled;
}

void FDisplayClusterConfiguratorSCSEditorViewportClient::ProcessClick(class FSceneView& View, class HHitProxy* HitProxy, FKey Key, EInputEvent Event, uint32 HitX, uint32 HitY)
{
	const FViewportClick Click(&View, this, Key, Event, HitX, HitY);

	if (HitProxy)
	{
		/*
		if (HitProxy->IsA(HInstancedStaticMeshInstance::StaticGetType()))
		{
			HInstancedStaticMeshInstance* InstancedStaticMeshInstanceProxy = ((HInstancedStaticMeshInstance*)HitProxy);

			TSharedPtr<ISCSEditorCustomization> Customization = BlueprintEditorPtr.Pin()->CustomizeSCSEditor(InstancedStaticMeshInstanceProxy->Component);
			if (Customization.IsValid() && Customization->HandleViewportClick(AsShared(), View, HitProxy, Key, Event, HitX, HitY))
			{
				Invalidate();
			}

			return;
		*/
		if (HitProxy->IsA(HWidgetAxis::StaticGetType()))
		{
			const bool bOldModeWidgets1 = EngineShowFlags.ModeWidgets;
			const bool bOldModeWidgets2 = View.Family->EngineShowFlags.ModeWidgets;

			EngineShowFlags.SetModeWidgets(false);
			FSceneViewFamily* SceneViewFamily = const_cast<FSceneViewFamily*>(View.Family);
			SceneViewFamily->EngineShowFlags.SetModeWidgets(false);
			bool bWasWidgetDragging = Widget->IsDragging();
			Widget->SetDragging(false);

			// Invalidate the hit proxy map so it will be rendered out again when GetHitProxy
			// is called
			Viewport->InvalidateHitProxy();

			// This will actually re-render the viewport's hit proxies!
			HHitProxy* HitProxyWithoutAxisWidgets = Viewport->GetHitProxy(HitX, HitY);
			if (HitProxyWithoutAxisWidgets != nullptr && !HitProxyWithoutAxisWidgets->IsA(HWidgetAxis::StaticGetType()))
			{
				// Try this again, but without the widget this time!
				ProcessClick(View, HitProxyWithoutAxisWidgets, Key, Event, HitX, HitY);
			}

			// Undo the evil
			EngineShowFlags.SetModeWidgets(bOldModeWidgets1);
			SceneViewFamily->EngineShowFlags.SetModeWidgets(bOldModeWidgets2);

			Widget->SetDragging(bWasWidgetDragging);

			// Invalidate the hit proxy map again so that it'll be refreshed with the original
			// scene contents if we need it again later.
			Viewport->InvalidateHitProxy();
			return;
		}
		else if (HitProxy->IsA(HActor::StaticGetType()))
		{
			HActor* ActorProxy = (HActor*)HitProxy;
			AActor* PreviewActor = GetPreviewActor();
			if (ActorProxy && ActorProxy->Actor && ActorProxy->PrimComponent)
			{
				USceneComponent* SelectedCompInstance = nullptr;

				if (ActorProxy->Actor == PreviewActor)
				{
					UPrimitiveComponent* TestComponent = ConstCast(ActorProxy->PrimComponent);
					if (ActorProxy->Actor->GetComponents().Contains(TestComponent))
					{
						SelectedCompInstance = TestComponent;
					}
				}
				else if (ActorProxy->Actor->IsChildActor())
				{
					AActor* TestActor = ActorProxy->Actor;
					while (TestActor->GetParentActor()->IsChildActor())
					{
						TestActor = TestActor->GetParentActor();
					}

					if (TestActor->GetParentActor() == PreviewActor)
					{
						SelectedCompInstance = TestActor->GetParentComponent();
					}
				}
				
				if (SelectedCompInstance)
				{
					TSharedPtr<ISCSEditorCustomization> Customization = BlueprintEditorPtr.Pin()->CustomizeSubobjectEditor(SelectedCompInstance);
					if (!(Customization.IsValid() && Customization->HandleViewportClick(AsShared(), View, HitProxy, Key, Event, HitX, HitY)))
					{
						const bool bIsCtrlKeyDown = Viewport->KeyState(EKeys::LeftControl) || Viewport->KeyState(EKeys::RightControl);
						if (BlueprintEditorPtr.IsValid())
						{
							// Note: This will find and select any node associated with the component instance that's attached to the proxy (including visualizers)
							BlueprintEditorPtr.Pin()->FindAndSelectSubobjectEditorTreeNode(SelectedCompInstance, bIsCtrlKeyDown);
						}
					}

					if (Event == IE_DoubleClick)
					{
						if (USceneComponent* ParentComponent = SelectedCompInstance->GetAttachParent())
						{
							const bool bIsCamera = ParentComponent->IsA<UDisplayClusterCameraComponent>() || ParentComponent->IsA<UCameraComponent>();
							if (bIsCamera)
							{
								SetCameraToComponent(ParentComponent);
							}
						}
					}
				}
			}

			Invalidate();
			return;
		}
	}

	GUnrealEd->ComponentVisManager.HandleClick(this, HitProxy, Click);
}

bool FDisplayClusterConfiguratorSCSEditorViewportClient::InputWidgetDelta(FViewport* InViewport, EAxisList::Type CurrentAxis, FVector& Drag, FRotator& Rot, FVector& Scale)
{
	bool bHandled = false;
	if (bIsManipulating && CurrentAxis != EAxisList::None)
	{
		bHandled = true;
		AActor* PreviewActor = GetPreviewActor();
		TSharedPtr<FBlueprintEditor> BlueprintEditor = BlueprintEditorPtr.Pin();
		if (PreviewActor && BlueprintEditor.IsValid())
		{
			TArray<FSubobjectEditorTreeNodePtrType> SelectedNodes = BlueprintEditor->GetSelectedSubobjectEditorTreeNodes();
			if (SelectedNodes.Num() > 0)
			{
				FVector ModifiedScale = Scale;

				// (mirrored from Level Editor VPC) - we don't scale components when we only have a very small scale change
				if (!Scale.IsNearlyZero())
				{
					FSnappingUtils::AdjustPercentageBasedScale(ModifiedScale);
				}
				else
				{
					ModifiedScale = FVector::ZeroVector;
				}

				for (const FSubobjectEditorTreeNodePtrType& SelectedNodePtr : SelectedNodes)
				{
					const FSubobjectData* Data = SelectedNodePtr.IsValid() ? SelectedNodePtr->GetDataSource() : nullptr;
					if (Data == nullptr)
					{
						continue;
					}
					// Don't allow editing of a root node, inherited SCS node or child node that also has a editable (non-root) parent node selected
					const bool bCanEdit = GUnrealEd->ComponentVisManager.IsActive() ||
						(!Data->IsRootComponent() && !IsMovableParentNodeSelected(SelectedNodePtr, SelectedNodes));

					if (bCanEdit)
					{
						if (GUnrealEd->ComponentVisManager.HandleInputDelta(this, InViewport, Drag, Rot, Scale))
						{
							GUnrealEd->RedrawLevelEditingViewports();
							Invalidate();
							return true;
						}

						USceneComponent* SceneComp = const_cast<USceneComponent*>(Cast<USceneComponent>(Data->FindComponentInstanceInActor(PreviewActor)));
						USceneComponent* SelectedTemplate = const_cast<USceneComponent*>(Data->GetObjectForBlueprint<USceneComponent>(BlueprintEditor->GetBlueprintObj()));
						if (SceneComp != nullptr && SelectedTemplate != nullptr)
						{
							// Cache the current default values for propagation
							FVector OldRelativeLocation = SelectedTemplate->GetRelativeLocation();
							FRotator OldRelativeRotation = SelectedTemplate->GetRelativeRotation();
							FVector OldRelativeScale3D = SelectedTemplate->GetRelativeScale3D();

							// Adjust the deltas as necessary
							FComponentEditorUtils::AdjustComponentDelta(SceneComp, Drag, Rot);

							TSharedPtr<ISCSEditorCustomization> Customization = BlueprintEditor->CustomizeSubobjectEditor(SceneComp);
							if (Customization.IsValid() && Customization->HandleViewportDrag(SceneComp, SelectedTemplate, Drag, Rot, ModifiedScale, GetWidgetLocation()))
							{
								// Handled by SCS Editor customization
							}
							else
							{
								// Apply delta to the template component object 
								// (the preview scene component will be set in one of the ArchetypeInstances loops below... to keep the two in sync) 
								GEditor->ApplyDeltaToComponent(
									SelectedTemplate,
									true,
									&Drag,
									&Rot,
									&ModifiedScale,
									SelectedTemplate->GetRelativeLocation());
							}

							UBlueprint* PreviewBlueprint = UBlueprint::GetBlueprintFromClass(PreviewActor->GetClass());
							if (PreviewBlueprint != nullptr)
							{
								// Like PostEditMove(), but we only need to re-run construction scripts
								if (PreviewBlueprint && PreviewBlueprint->bRunConstructionScriptOnDrag)
								{
									PreviewActor->RerunConstructionScripts();
								}

								SceneComp->PostEditComponentMove(true); // @TODO HACK passing 'finished' every frame...

								// If a constraint, copy back updated constraint frames to template
								UPhysicsConstraintComponent* ConstraintComp = Cast<UPhysicsConstraintComponent>(SceneComp);
								UPhysicsConstraintComponent* TemplateComp = Cast<UPhysicsConstraintComponent>(SelectedTemplate);
								if (ConstraintComp && TemplateComp)
								{
									TemplateComp->ConstraintInstance.CopyConstraintGeometryFrom(&ConstraintComp->ConstraintInstance);
								}

								// Iterate over all the active archetype instances and propagate the change(s) to the matching component instance
								TArray<UObject*> ArchetypeInstances;
								if (SelectedTemplate->HasAnyFlags(RF_ArchetypeObject))
								{
									SelectedTemplate->GetArchetypeInstances(ArchetypeInstances);
									for (int32 InstanceIndex = 0; InstanceIndex < ArchetypeInstances.Num(); ++InstanceIndex)
									{
										SceneComp = Cast<USceneComponent>(ArchetypeInstances[InstanceIndex]);
										if (SceneComp != nullptr)
										{
											FComponentEditorUtils::ApplyDefaultValueChange(SceneComp, SceneComp->GetRelativeLocation_DirectMutable(), OldRelativeLocation, SelectedTemplate->GetRelativeLocation());
											FComponentEditorUtils::ApplyDefaultValueChange(SceneComp, SceneComp->GetRelativeRotation_DirectMutable(), OldRelativeRotation, SelectedTemplate->GetRelativeRotation());
											FComponentEditorUtils::ApplyDefaultValueChange(SceneComp, SceneComp->GetRelativeScale3D_DirectMutable(), OldRelativeScale3D, SelectedTemplate->GetRelativeScale3D());
										}
									}
								}
								else if (UObject* Outer = SelectedTemplate->GetOuter())
								{
									Outer->GetArchetypeInstances(ArchetypeInstances);
									for (int32 InstanceIndex = 0; InstanceIndex < ArchetypeInstances.Num(); ++InstanceIndex)
									{
										SceneComp = static_cast<USceneComponent*>(FindObjectWithOuter(ArchetypeInstances[InstanceIndex], SelectedTemplate->GetClass(), SelectedTemplate->GetFName()));
										if (SceneComp)
										{
											FComponentEditorUtils::ApplyDefaultValueChange(SceneComp, SceneComp->GetRelativeLocation_DirectMutable(), OldRelativeLocation, SelectedTemplate->GetRelativeLocation());
											FComponentEditorUtils::ApplyDefaultValueChange(SceneComp, SceneComp->GetRelativeRotation_DirectMutable(), OldRelativeRotation, SelectedTemplate->GetRelativeRotation());
											FComponentEditorUtils::ApplyDefaultValueChange(SceneComp, SceneComp->GetRelativeScale3D_DirectMutable(), OldRelativeScale3D, SelectedTemplate->GetRelativeScale3D());
										}
									}
								}

								if (PreviewBlueprint && PreviewBlueprint->bRunConstructionScriptOnDrag)
								{
									// Fix UEENT-4314: Display cluster screens won't have their visual components properly refreshed until after the default values are applied above.
									// This is sort of a hack and may not be necessary if we ever change how the visual (impl) screen components are used.
									PreviewActor->RerunConstructionScripts();
								}
							}
						}
					}
				}
				GUnrealEd->RedrawLevelEditingViewports();
			}
		}

		Invalidate();
	}

	return bHandled;
}

void FDisplayClusterConfiguratorSCSEditorViewportClient::TrackingStarted(const struct FInputEventState& InInputState, bool bIsDraggingWidget, bool bNudge)
{
	if (!bIsManipulating && bIsDraggingWidget)
	{
		// Suspend component modification during each delta step to avoid recording unnecessary overhead into the transaction buffer
		GEditor->DisableDeltaModification(true);

		// Begin transaction
		BeginTransaction(NSLOCTEXT("UnrealEd", "ModifyComponents", "Modify Component(s)"));
		bIsManipulating = true;
	}
}

void FDisplayClusterConfiguratorSCSEditorViewportClient::TrackingStopped()
{
	if (bIsManipulating)
	{
		// Re-run construction scripts if we haven't done so yet (so that the components in the preview actor can update their transforms)
		AActor* PreviewActor = GetPreviewActor();
		if (PreviewActor != nullptr)
		{
			UBlueprint* PreviewBlueprint = UBlueprint::GetBlueprintFromClass(PreviewActor->GetClass());
			if (PreviewBlueprint != nullptr && !PreviewBlueprint->bRunConstructionScriptOnDrag)
			{
				PreviewActor->RerunConstructionScripts();
			}
		}

		// End transaction
		bIsManipulating = false;
		EndTransaction();

		// Restore component delta modification
		GEditor->DisableDeltaModification(false);
	}
}

bool FDisplayClusterConfiguratorSCSEditorViewportClient::BeginTransform(const FGizmoState& InState)
{
	// Suspend component modification during each delta step to avoid recording unnecessary overhead into the transaction buffer
	GEditor->DisableDeltaModification(true);

	// Begin transaction
	BeginTransaction(NSLOCTEXT("UnrealEd", "ModifyComponents", "Modify Component(s)"));
	bIsManipulating = true;
	return true;
}

bool FDisplayClusterConfiguratorSCSEditorViewportClient::EndTransform(const FGizmoState& InState)
{
	// Re-run construction scripts if we haven't done so yet (so that the components in the preview actor can update their transforms)
	AActor* PreviewActor = GetPreviewActor();
	if (PreviewActor != nullptr)
	{
		UBlueprint* PreviewBlueprint = UBlueprint::GetBlueprintFromClass(PreviewActor->GetClass());
		if (PreviewBlueprint != nullptr && !PreviewBlueprint->bRunConstructionScriptOnDrag)
		{
			PreviewActor->RerunConstructionScripts();
		}
	}

	// End transaction
	bIsManipulating = false;
	EndTransaction();

	// Restore component delta modification
	GEditor->DisableDeltaModification(false);
	
	return true;
}

bool FDisplayClusterConfiguratorSCSEditorViewportClient::OnTransformCanceled(const FGizmoState& InState)
{
	return EndTransform(InState);
}

UE::Widget::EWidgetMode FDisplayClusterConfiguratorSCSEditorViewportClient::GetWidgetMode() const
{
	// Default to not drawing the widget
	UE::Widget::EWidgetMode ReturnWidgetMode = UE::Widget::WM_None;

	AActor* PreviewActor = GetPreviewActor();
	if (!bIsSimulateEnabled && PreviewActor)
	{
		const TSharedPtr<FDisplayClusterConfiguratorBlueprintEditor> BluePrintEditor = BlueprintEditorPtr.Pin();
		if (BluePrintEditor.IsValid())
		{
			TArray<FSubobjectEditorTreeNodePtrType> SelectedNodes = BluePrintEditor->GetSelectedSubobjectEditorTreeNodes();
			if (BluePrintEditor->GetSubobjectEditor()->GetSceneRootNode().IsValid())
			{
				const TArray<FSubobjectEditorTreeNodePtrType>& RootNodes = BluePrintEditor->GetSubobjectEditor()->GetRootNodes();

				if (GUnrealEd->ComponentVisManager.IsActive() &&
					GUnrealEd->ComponentVisManager.IsVisualizingArchetype())
				{
					// Component visualizer is active and editing the archetype
					ReturnWidgetMode = WidgetMode;
				}
				else
				{
					// if the selected nodes array is empty, or only contains entries from the
					// root nodes array, or isn't visible in the preview actor, then don't display a transform widget
					for (int32 CurrentNodeIndex = 0; CurrentNodeIndex < SelectedNodes.Num(); CurrentNodeIndex++)
					{
						FSubobjectEditorTreeNodePtrType CurrentNodePtr = SelectedNodes[CurrentNodeIndex];
						FSubobjectData* Data = CurrentNodePtr.IsValid() ? CurrentNodePtr->GetDataSource() : nullptr;
						if ((Data &&
							((!RootNodes.Contains(CurrentNodePtr) && !Data->IsRootComponent()) ||
								(Data->GetObject<UInstancedStaticMeshComponent>() && // show widget if we are editing individual instances even if it is the root component
									CastChecked<UInstancedStaticMeshComponent>(Data->FindComponentInstanceInActor(GetPreviewActor()))->SelectedInstances.Contains(true))) &&
							Data->CanEdit() &&
							Data->FindComponentInstanceInActor(PreviewActor)))
						{
							// a non-nullptr, non-root item is selected, draw the widget
							ReturnWidgetMode = WidgetMode;
							break;
						}
					}
				}
			}
		}
	}

	return ReturnWidgetMode;
}


void FDisplayClusterConfiguratorSCSEditorViewportClient::SetWidgetMode(UE::Widget::EWidgetMode NewMode)
{
	WidgetMode = NewMode;
}

void FDisplayClusterConfiguratorSCSEditorViewportClient::SetWidgetCoordSystemSpace(ECoordSystem NewCoordSystem)
{
	WidgetCoordSystem = NewCoordSystem;
}

FVector FDisplayClusterConfiguratorSCSEditorViewportClient::GetWidgetLocation() const
{
	FVector ComponentVisWidgetLocation;
	if (GUnrealEd->ComponentVisManager.IsVisualizingArchetype() &&
		GUnrealEd->ComponentVisManager.GetWidgetLocation(this, ComponentVisWidgetLocation))
	{
		return ComponentVisWidgetLocation;
	}

	FVector Location = FVector::ZeroVector;

	AActor* PreviewActor = GetPreviewActor();
	if (PreviewActor)
	{
		TArray<FSubobjectEditorTreeNodePtrType> SelectedNodes = BlueprintEditorPtr.Pin()->GetSelectedSubobjectEditorTreeNodes();
		if (SelectedNodes.Num() > 0 && SelectedNodes.Last().IsValid())
		{
			// Use the last selected item for the widget location
			if (const FSubobjectData* NodeData = SelectedNodes.Last()->GetDataSource())
			{
				const USceneComponent* SceneComp = Cast<USceneComponent>(NodeData->FindComponentInstanceInActor(PreviewActor));
				if (SceneComp)
				{
					TSharedPtr<ISCSEditorCustomization> Customization = BlueprintEditorPtr.Pin()->CustomizeSubobjectEditor(SceneComp);
					FVector CustomLocation;
					if (Customization.IsValid() && Customization->HandleGetWidgetLocation(SceneComp, CustomLocation))
					{
						Location = CustomLocation;
					}
					else
					{
						Location = SceneComp->GetComponentLocation();
					}
				}
			}
		}
	}

	return Location;
}

FMatrix FDisplayClusterConfiguratorSCSEditorViewportClient::GetWidgetCoordSystem() const
{
	FMatrix ComponentVisWidgetCoordSystem;
	if (GUnrealEd->ComponentVisManager.IsVisualizingArchetype() &&
		GUnrealEd->ComponentVisManager.GetCustomInputCoordinateSystem(this, ComponentVisWidgetCoordSystem))
	{
		return ComponentVisWidgetCoordSystem;
	}

	FMatrix Matrix = FMatrix::Identity;
	if (GetWidgetCoordSystemSpace() == COORD_Local)
	{
		AActor* PreviewActor = GetPreviewActor();
		TSharedPtr<FBlueprintEditor> BlueprintEditor = BlueprintEditorPtr.Pin();
		if (PreviewActor && BlueprintEditor.IsValid())
		{
			TArray<FSubobjectEditorTreeNodePtrType> SelectedNodes = BlueprintEditor->GetSelectedSubobjectEditorTreeNodes();
			if (SelectedNodes.Num() > 0)
			{
				const FSubobjectEditorTreeNodePtrType SelectedNode = SelectedNodes.Last();
				FSubobjectData* Data = SelectedNode.IsValid() ? SelectedNode->GetDataSource() : nullptr;
				const USceneComponent* SceneComp = Data ? Cast<USceneComponent>(Data->FindComponentInstanceInActor(PreviewActor)) : nullptr;
				if (SceneComp)
				{
					TSharedPtr<ISCSEditorCustomization> Customization = BlueprintEditor->CustomizeSubobjectEditor(SceneComp);
					FMatrix CustomTransform;
					if (Customization.IsValid() && Customization->HandleGetWidgetTransform(SceneComp, CustomTransform))
					{
						Matrix = CustomTransform;
					}
					else
					{
						Matrix = FQuatRotationMatrix(SceneComp->GetComponentQuat());
					}
				}
			}
		}
	}

	if (!Matrix.Equals(FMatrix::Identity))
	{
		Matrix.RemoveScaling();
	}

	return Matrix;
}

const FEditorViewportCameraSpeedSettings& FDisplayClusterConfiguratorSCSEditorViewportClient::GetCameraSpeedSettings() const
{
	return GetDefault<UEditorPerProjectUserSettings>()->SCSViewportCameraSpeedSettings;
}

void FDisplayClusterConfiguratorSCSEditorViewportClient::SetCameraSpeedSettings(const FEditorViewportCameraSpeedSettings& InCameraSpeedSettings)
{
	GetMutableDefault<UEditorPerProjectUserSettings>()->SCSViewportCameraSpeedSettings = InCameraSpeedSettings;
}

HHitProxy* FDisplayClusterConfiguratorSCSEditorViewportClient::GetHitProxyWithoutGizmos(int32 X, int32 Y)
{
	HHitProxy* HitProxy = Viewport->GetHitProxy(X, Y);

	if (!HitProxy)
	{
		return nullptr;
	}

	if (HitProxy->IsA(HWidgetAxis::StaticGetType()))
	{
		const bool bOldModeWidgets = EngineShowFlags.ModeWidgets;

		EngineShowFlags.SetModeWidgets(false);
		bool bWasWidgetDragging = Widget->IsDragging();
		Widget->SetDragging(false);

		// Invalidate the hit proxy map so it will be rendered out again when GetHitProxy
		// is called
		Viewport->InvalidateHitProxy();

		// This will actually re-render the viewport's hit proxies!
		HitProxy = Viewport->GetHitProxy(X, Y);

		// Undo the evil
		EngineShowFlags.SetModeWidgets(bOldModeWidgets);

		Widget->SetDragging(bWasWidgetDragging);

		// Invalidate the hit proxy map again so that it'll be refreshed with the original
		// scene contents if we need it again later.
		Viewport->InvalidateHitProxy();
	}

	return HitProxy;
}

void FDisplayClusterConfiguratorSCSEditorViewportClient::InvalidatePreview(bool bResetCamera)
{
	// Ensure that the editor is valid before continuing
	if (!BlueprintEditorPtr.IsValid())
	{
		return;
	}

	// Create or update the Blueprint actor instance in the preview scene
	BlueprintEditorPtr.Pin()->RefreshDisplayClusterPreviewActor();

	Invalidate();
	RefreshPreviewBounds();

	if (bResetCamera)
	{
		ResetCamera();
	}
}

void FDisplayClusterConfiguratorSCSEditorViewportClient::SyncEditorSettings()
{
	SyncShowGrid();
	SyncShowPreview();
}

void FDisplayClusterConfiguratorSCSEditorViewportClient::SetCameraToComponent(USceneComponent* InComponent)
{
	check(InComponent);
	SetViewLocation(InComponent->GetComponentLocation());
	SetViewRotation(InComponent->GetComponentRotation());
}

void FDisplayClusterConfiguratorSCSEditorViewportClient::ResetCamera()
{
	ToggleOrbitCamera(true);

	SetViewLocationForOrbiting(PreviewActorBounds.Origin);

	const UDisplayClusterConfiguratorEditorSettings* DisplayClusterSettings = GetDefault<UDisplayClusterConfiguratorEditorSettings>();
	SetViewLocation(EditorViewportDefs::DefaultPerspectiveViewLocation + DisplayClusterSettings->EditorDefaultCameraLocation);
	SetViewRotation(EditorViewportDefs::DefaultPerspectiveViewRotation + DisplayClusterSettings->EditorDefaultCameraRotation);

	Invalidate();
}

void FDisplayClusterConfiguratorSCSEditorViewportClient::ToggleRealtimePreview()
{
	SetRealtime(!IsRealtime());

	Invalidate();
}

AActor* FDisplayClusterConfiguratorSCSEditorViewportClient::GetPreviewActor() const
{
	return BlueprintEditorPtr.Pin()->GetPreviewActor();
}

void FDisplayClusterConfiguratorSCSEditorViewportClient::FocusViewportToSelection()
{
	AActor* PreviewActor = GetPreviewActor();
	if (PreviewActor)
	{
		TArray<FSubobjectEditorTreeNodePtrType> SelectedNodes = BlueprintEditorPtr.Pin()->GetSelectedSubobjectEditorTreeNodes();
		if (SelectedNodes.Num() > 0)
		{
			// Use the last selected item for the widget location
			const FSubobjectEditorTreeNodePtrType& LastNode = SelectedNodes.Last();
			if (LastNode.IsValid())
			{
				if (const FSubobjectData* Data = LastNode->GetDataSource())
				{
					if (const USceneComponent* SceneComp = Cast<USceneComponent>(Data->FindComponentInstanceInActor(PreviewActor)))
					{
						FocusViewportOnBox(SceneComp->Bounds.GetBox());
					}
				}
			}
		}
		else
		{
			FocusViewportOnBox(PreviewActor->GetComponentsBoundingBox(true));
		}
	}
}

bool FDisplayClusterConfiguratorSCSEditorViewportClient::GetShowFloor() const
{
	return GetDefault<UDisplayClusterConfiguratorEditorSettings>()->bEditorShowFloor;
}

void FDisplayClusterConfiguratorSCSEditorViewportClient::ToggleShowFloor()
{
	UDisplayClusterConfiguratorEditorSettings* Settings = GetMutableDefault<UDisplayClusterConfiguratorEditorSettings>();

	bool bShowFloor = Settings->bEditorShowFloor;
	bShowFloor = !bShowFloor;

	EditorFloorComp->SetVisibility(bShowFloor);
	EditorFloorComp->SetCollisionEnabled(bShowFloor ? ECollisionEnabled::QueryAndPhysics : ECollisionEnabled::NoCollision);

	Settings->bEditorShowFloor = bShowFloor;
	Settings->PostEditChange();
	Settings->SaveConfig();

	Invalidate();
}

bool FDisplayClusterConfiguratorSCSEditorViewportClient::GetShowGrid()
{
	const UDisplayClusterConfiguratorEditorSettings* Settings = GetDefault<UDisplayClusterConfiguratorEditorSettings>();
	return Settings->bEditorShowGrid;
}

void FDisplayClusterConfiguratorSCSEditorViewportClient::ToggleShowGrid()
{
	UDisplayClusterConfiguratorEditorSettings* Settings = GetMutableDefault<UDisplayClusterConfiguratorEditorSettings>();
	Settings->bEditorShowGrid = !Settings->bEditorShowGrid;

	Settings->PostEditChange();
	Settings->SaveConfig();

	if (BlueprintEditorPtr.IsValid())
	{
		BlueprintEditorPtr.Pin()->SyncViewports();
	}
}

void FDisplayClusterConfiguratorSCSEditorViewportClient::SyncShowGrid()
{
	const UDisplayClusterConfiguratorEditorSettings* Settings = GetDefault<UDisplayClusterConfiguratorEditorSettings>();
	if (Settings->bEditorShowGrid != IsSetShowGridChecked())
	{
		SetShowGrid();
	}
}

void FDisplayClusterConfiguratorSCSEditorViewportClient::ToggleShowOrigin()
{
	check(BlueprintEditorPtr.IsValid());

	UDisplayClusterConfiguratorEditorSettings* Settings = GetMutableDefault<UDisplayClusterConfiguratorEditorSettings>();
	bool bShowOrigin = Settings->bEditorShowWorldOrigin;
	bShowOrigin = !bShowOrigin;

	WorldOriginComponent->SetVisibility(bShowOrigin);
	
	Settings->bEditorShowWorldOrigin = bShowOrigin;
	Settings->PostEditChange();
	Settings->SaveConfig();
	
	Invalidate();
}

bool FDisplayClusterConfiguratorSCSEditorViewportClient::GetShowOrigin() const
{
	const UDisplayClusterConfiguratorEditorSettings* Settings = GetDefault<UDisplayClusterConfiguratorEditorSettings>();
	return Settings->bEditorShowWorldOrigin;
}

bool FDisplayClusterConfiguratorSCSEditorViewportClient::GetEnableAA() const
{
	const UDisplayClusterConfiguratorEditorSettings* Settings = GetDefault<UDisplayClusterConfiguratorEditorSettings>();
	return Settings->bEditorEnableAA;
}

void FDisplayClusterConfiguratorSCSEditorViewportClient::ToggleEnableAA()
{
	UDisplayClusterConfiguratorEditorSettings* Settings = GetMutableDefault<UDisplayClusterConfiguratorEditorSettings>();
	Settings->bEditorEnableAA = !Settings->bEditorEnableAA;
	EngineShowFlags.AntiAliasing = Settings->bEditorEnableAA;
	
	Settings->PostEditChange();
	Settings->SaveConfig();
}

void FDisplayClusterConfiguratorSCSEditorViewportClient::ToggleShowPreview()
{
	check(BlueprintEditorPtr.IsValid());

	UDisplayClusterConfiguratorEditorSettings* Settings = GetMutableDefault<UDisplayClusterConfiguratorEditorSettings>();
	bool bShowPreview = Settings->bEditorShowPreview;
	bShowPreview = !bShowPreview;

	Settings->bEditorShowPreview = bShowPreview;
	Settings->PostEditChange();
	Settings->SaveConfig();
	
	SyncShowPreview();
	Invalidate();
}

bool FDisplayClusterConfiguratorSCSEditorViewportClient::GetShowPreview() const
{
	const UDisplayClusterConfiguratorEditorSettings* Settings = GetDefault<UDisplayClusterConfiguratorEditorSettings>();
	return Settings->bEditorShowPreview;
}

void FDisplayClusterConfiguratorSCSEditorViewportClient::SyncShowPreview()
{
	if (BlueprintEditorPtr.IsValid())
	{
		const bool bShouldShowPreview = GetShowPreview();
		if (!bShouldShowPreview)
		{
			if (ADisplayClusterRootActor* Actor = Cast<ADisplayClusterRootActor>(BlueprintEditorPtr.Pin()->GetPreviewActor()))
			{
				if (Actor->PreviewNodeId != DisplayClusterConfigurationStrings::gui::preview::PreviewNodeNone)
				{
					Actor->PreviewNodeId = DisplayClusterConfigurationStrings::gui::preview::PreviewNodeNone;
				}
			}
		}
	}
}

bool FDisplayClusterConfiguratorSCSEditorViewportClient::GetShowViewportNames() const
{
	const UDisplayClusterConfiguratorEditorSettings* Settings = GetDefault<UDisplayClusterConfiguratorEditorSettings>();
	return Settings->bEditorShow3DViewportNames;
}

void FDisplayClusterConfiguratorSCSEditorViewportClient::ToggleShowViewportNames()
{
	UDisplayClusterConfiguratorEditorSettings* Settings = GetMutableDefault<UDisplayClusterConfiguratorEditorSettings>();
	Settings->bEditorShow3DViewportNames = !Settings->bEditorShow3DViewportNames;

	Settings->PostEditChange();
	Settings->SaveConfig();
}

bool FDisplayClusterConfiguratorSCSEditorViewportClient::CanToggleViewportNames() const
{
	return GetShowPreview();
}

TOptional<float> FDisplayClusterConfiguratorSCSEditorViewportClient::GetPreviewResolutionScale() const
{
	const TSharedPtr<FDisplayClusterConfiguratorBlueprintEditor> BlueprintEditor = BlueprintEditorPtr.Pin();
	if (BlueprintEditor.IsValid())
	{
		if (ADisplayClusterRootActor* CDO = BlueprintEditor->GetDefaultRootActor())
		{
			return CDO->PreviewRenderTargetRatioMult;
		}
	}

	return TOptional<float>();
}

void FDisplayClusterConfiguratorSCSEditorViewportClient::SetPreviewResolutionScale(float InScale)
{
	const TSharedPtr<FDisplayClusterConfiguratorBlueprintEditor> BlueprintEditor = BlueprintEditorPtr.Pin();
	if (BlueprintEditor.IsValid())
	{
		if (ADisplayClusterRootActor* CDO = BlueprintEditor->GetDefaultRootActor())
		{
			UE::DisplayClusterConfiguratorPropertyUtils::SetPropertyHandleValue(
				CDO, GET_MEMBER_NAME_CHECKED(ADisplayClusterRootActor, PreviewRenderTargetRatioMult), InScale);
		}
	}
}

TOptional<float> FDisplayClusterConfiguratorSCSEditorViewportClient::GetXformGizmoScale() const
{
	const UDisplayClusterConfiguratorEditorSettings* Settings = GetDefault<UDisplayClusterConfiguratorEditorSettings>();
	return Settings->VisXformScale;
}

void FDisplayClusterConfiguratorSCSEditorViewportClient::SetXformGizmoScale(float InScale)
{
	UDisplayClusterConfiguratorEditorSettings* Settings = GetMutableDefault<UDisplayClusterConfiguratorEditorSettings>();
	Settings->VisXformScale = FMath::Max(0.0f, InScale);

	Settings->PostEditChange();
	Settings->SaveConfig();

	TSharedPtr<FDisplayClusterConfiguratorBlueprintEditor> BlueprintEditor = BlueprintEditorPtr.Pin();
	if (BlueprintEditor.IsValid())
	{
		BlueprintEditor->UpdateXformGizmos();
		Invalidate();
	}
}

bool FDisplayClusterConfiguratorSCSEditorViewportClient::IsShowingXformGizmos() const
{
	const UDisplayClusterConfiguratorEditorSettings* Settings = GetDefault<UDisplayClusterConfiguratorEditorSettings>();
	return Settings->bShowVisXforms;
}

void FDisplayClusterConfiguratorSCSEditorViewportClient::ToggleShowXformGizmos()
{
	UDisplayClusterConfiguratorEditorSettings* Settings = GetMutableDefault<UDisplayClusterConfiguratorEditorSettings>();
	Settings->bShowVisXforms = !Settings->bShowVisXforms;

	Settings->PostEditChange();
	Settings->SaveConfig();

	TSharedPtr<FDisplayClusterConfiguratorBlueprintEditor> BlueprintEditor = BlueprintEditorPtr.Pin();
	if (BlueprintEditor.IsValid())
	{
		BlueprintEditor->UpdateXformGizmos();
		Invalidate();
	}
}

void FDisplayClusterConfiguratorSCSEditorViewportClient::BeginTransaction(const FText& Description)
{
	//UE_LOGF(LogSCSEditorViewport, Log, "FDisplayClusterConfiguratorSCSEditorViewportClient::BeginTransaction() pre: %ls %08x", SessionName, *((uint32*)&ScopedTransaction));

	if (!ScopedTransaction)
	{
		ScopedTransaction = new FScopedTransaction(Description);

		TSharedPtr<FBlueprintEditor> BlueprintEditor = BlueprintEditorPtr.Pin();
		if (BlueprintEditor.IsValid())
		{
			UBlueprint* PreviewBlueprint = BlueprintEditor->GetBlueprintObj();
			if (PreviewBlueprint != nullptr)
			{
				FBlueprintEditorUtils::MarkBlueprintAsModified(PreviewBlueprint);
			}

			TArray<FSubobjectEditorTreeNodePtrType> SelectedNodes = BlueprintEditor->GetSelectedSubobjectEditorTreeNodes();
			for (const FSubobjectEditorTreeNodePtrType& Node : SelectedNodes)
			{
				if (Node.IsValid())
				{
					if (const FSubobjectData* Data = Node->GetDataSource())
					{
						if (USCS_Node* SCS_Node = const_cast<USCS_Node*>(Data->GetObject<USCS_Node>()))
						{
							USimpleConstructionScript* SCS = SCS_Node->GetSCS();
							UBlueprint* Blueprint = SCS ? SCS->GetBlueprint() : nullptr;
							if (Blueprint == PreviewBlueprint)
							{
								SCS_Node->Modify();
							}
						}

						// Modify template, any instances will be reconstructed as part of PostUndo:
						UActorComponent* ComponentTemplate = const_cast<UActorComponent*>(Data->GetObjectForBlueprint<UActorComponent>(PreviewBlueprint));
					
						if (ComponentTemplate != nullptr)
						{
							ComponentTemplate->SetFlags(RF_Transactional);
							ComponentTemplate->Modify();
						}
					}
				}
			}
		}
	}

	//UE_LOGF(LogSCSEditorViewport, Log, "FDisplayClusterConfiguratorSCSEditorViewportClient::BeginTransaction() post: %ls %08x", SessionName, *((uint32*)&ScopedTransaction));
}

void FDisplayClusterConfiguratorSCSEditorViewportClient::EndTransaction()
{
	//UE_LOGF(LogSCSEditorViewport, Log, "FDisplayClusterConfiguratorSCSEditorViewportClient::EndTransaction(): %08x", *((uint32*)&ScopedTransaction));

	if (ScopedTransaction)
	{
		delete ScopedTransaction;
		ScopedTransaction = nullptr;
	}
}

void FDisplayClusterConfiguratorSCSEditorViewportClient::RefreshPreviewBounds()
{
	AActor* PreviewActor = GetPreviewActor();

	if (PreviewActor)
	{
		// Compute actor bounds as the sum of its visible parts
		FBoxSphereBounds::Builder BoundsBuilder;
		for (UActorComponent* Component : PreviewActor->GetComponents())
		{
			// Aggregate primitive components that either have collision enabled or are otherwise visible components in-game
			if (UPrimitiveComponent* PrimComp = Cast<UPrimitiveComponent>(Component))
			{
				if (PrimComp->IsRegistered() && (!PrimComp->bHiddenInGame || PrimComp->IsCollisionEnabled()) && PrimComp->Bounds.SphereRadius < HALF_WORLD_MAX)
				{
					BoundsBuilder += PrimComp->Bounds;
				}
			}
		}
		PreviewActorBounds = BoundsBuilder;
	}
}

void FDisplayClusterConfiguratorSCSEditorViewportClient::DisplayViewportInformation(FSceneView& SceneView, FCanvas& Canvas)
{
	if (!BlueprintEditorPtr.IsValid())
	{
		return;
	}

	ADisplayClusterRootActor* RootActor = Cast<ADisplayClusterRootActor>(BlueprintEditorPtr.Pin()->GetPreviewActor());
	if (IDisplayClusterViewportManager* ViewportManager = RootActor ? RootActor->GetViewportManager() : nullptr)
	{
		for (TSharedPtr<IDisplayClusterViewport, ESPMode::ThreadSafe>& ViewportIt : ViewportManager->GetEntireClusterViewports())
		{
			if (UMeshComponent* PreviewMesh = ViewportIt.IsValid() ? ViewportIt->GetViewportPreview().GetPreviewMeshComponent() : nullptr)
			{
				UFont* Font = GEngine->GetSmallFont();
				const FLinearColor Color = FLinearColor::Red;
				const FString DisplayText = ViewportIt->GetId();

				// Display at center of bounding box of mesh with support for various pivot points.
				FVector WorldLocation = PreviewMesh->GetComponentLocation();
				if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(PreviewMesh))
				{
					if (UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh())
					{
						FVector MeshOrigin = StaticMesh->GetBounds().Origin * StaticMeshComponent->GetComponentScale();
						FRotator ComponentRotation = StaticMeshComponent->GetComponentRotation();
						WorldLocation += ComponentRotation.RotateVector(MeshOrigin);
					}
				}
				else if (UProceduralMeshComponent* ProceduralMeshComponent = Cast<UProceduralMeshComponent>(PreviewMesh))
				{
					// All projection policies except "mesh" and "simple" use ProceduralMeshComponent to create the mesh at runtime.
					const FBoxSphereBounds& Bounds = ProceduralMeshComponent->Bounds;
					FVector MeshOrigin = Bounds.Origin * ProceduralMeshComponent->GetComponentScale();
					FRotator ComponentRotation = ProceduralMeshComponent->GetComponentRotation();
					WorldLocation += ComponentRotation.RotateVector(MeshOrigin);
				}

				const FPlane Proj = SceneView.Project(WorldLocation);
				if (Proj.W > 0.0f)
				{
					int32 TextWidth, TextHeight;
					StringSize(Font, TextWidth, TextHeight, *DisplayText);

					const int32 HalfX = 0.5f * Viewport->GetSizeXY().X;
					const int32 HalfY = 0.5f * Viewport->GetSizeXY().Y;
					const int32 XPos = HalfX + (HalfX * Proj.X) - TextWidth / 2;
					const int32 YPos = HalfY + (HalfY * (Proj.Y * -1));

					const float DPI = UpdateViewportClientWindowDPIScale();
					FVector2D AdjustedPosition(XPos / DPI, YPos / DPI);

					FCanvasTextItem TextItem(AdjustedPosition, FText::FromString(DisplayText), Font, Color);
					TextItem.Draw(&Canvas);
				}
			}
		}
	}
}
