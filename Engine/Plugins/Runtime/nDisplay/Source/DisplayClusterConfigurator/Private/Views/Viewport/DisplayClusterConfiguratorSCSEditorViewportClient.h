// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Editor/Kismet/Private/SCSEditorViewportClient.h"
#include "UnrealWidgetFwd.h"

class UMaterialInstanceConstant;
class UPostProcessComponent;
class UAssetViewerSettings;
class UDisplayClusterWorldOriginComponent;
class SDisplayClusterConfiguratorSCSEditorViewport;
struct FGizmoState;

class FDisplayClusterConfiguratorSCSEditorViewportClient : public FEditorViewportClient, public TSharedFromThis<FDisplayClusterConfiguratorSCSEditorViewportClient>
{
public:
	/**
	 * Constructor.
	 *
	 * @param InBlueprintEditorPtr A weak reference to the Blueprint Editor context.
	 * @param InPreviewScene The preview scene to use.
	 */
	FDisplayClusterConfiguratorSCSEditorViewportClient(TWeakPtr<class FBlueprintEditor>& InBlueprintEditorPtr, FPreviewScene* InPreviewScene, const TSharedRef<SDisplayClusterConfiguratorSCSEditorViewport>& InSCSEditorViewport);


	/**
	 * Destructor.
	 */
	virtual ~FDisplayClusterConfiguratorSCSEditorViewportClient();

	// FEditorViewportClient interface
	virtual void Tick(float DeltaSeconds) override;
	virtual void Draw(const FSceneView* View, FPrimitiveDrawInterface* PDI) override;
	virtual void DrawCanvas(FViewport& InViewport, FSceneView& View, FCanvas& Canvas) override;
	virtual bool InputKey(const FInputKeyEventArgs& EventArgs) override;
	virtual void ProcessClick(class FSceneView& View, class HHitProxy* HitProxy, FKey Key, EInputEvent Event, uint32 HitX, uint32 HitY) override;
	virtual bool InputWidgetDelta(FViewport* Viewport, EAxisList::Type CurrentAxis, FVector& Drag, FRotator& Rot, FVector& Scale) override;
	virtual void TrackingStarted(const struct FInputEventState& InInputState, bool bIsDragging, bool bNudge) override;
	virtual void TrackingStopped() override;
	virtual bool BeginTransform(const FGizmoState& InState) override;
	virtual bool EndTransform(const FGizmoState& InState) override;
	virtual bool OnTransformCanceled(const FGizmoState& InState) override;
	virtual UE::Widget::EWidgetMode GetWidgetMode() const override;
	virtual void SetWidgetMode(UE::Widget::EWidgetMode NewMode) override;
	virtual void SetWidgetCoordSystemSpace(ECoordSystem NewCoordSystem) override;
	virtual FVector GetWidgetLocation() const override;
	virtual FMatrix GetWidgetCoordSystem() const override;
	virtual ECoordSystem GetWidgetCoordSystemSpace() const override { return WidgetCoordSystem; }
	virtual const FEditorViewportCameraSpeedSettings& GetCameraSpeedSettings() const override;
	virtual void SetCameraSpeedSettings(const FEditorViewportCameraSpeedSettings& InCameraSpeedSettings) override;


	/**
	 * Gets the hit proxy at the specified viewport coordinates when viewport gizmos such as the axis widget are filtered out.
	 */
	HHitProxy* GetHitProxyWithoutGizmos(int32 X, int32 Y);

	/**
	 * Recreates the preview scene and invalidates the owning viewport.
	 *
	 * @param bResetCamera Whether or not to reset the camera after recreating the preview scene.
	 */
	void InvalidatePreview(bool bResetCamera = true);

	/**
	 * Syncs local properties to editor settings.
	 */
	void SyncEditorSettings();

	/**
	 * Sets the local scene camera to the exact component location & rotation.
	 *
	 * @param InComponent The component with the transform to match the view to.
	 */
	void SetCameraToComponent(USceneComponent* InComponent);

	/**
	 * Resets the camera position
	 */
	void ResetCamera();

	/**
	 * Determines whether or not realtime preview is enabled.
	 *
	 * @return true if realtime preview is enabled, false otherwise.
	 */
	bool GetRealtimePreview() const
	{
		return IsRealtime();
	}

	/**
	 * Toggles realtime preview on/off.
	 */
	void ToggleRealtimePreview();

	/**
	 * Focuses the viewport on the selected components
	 */
	void FocusViewportToSelection();

	/**
	 * Returns true if the floor is currently visible in the viewport
	 */
	bool GetShowFloor() const;

	/**
	 * Will toggle the floor's visibility in the viewport
	 */
	void ToggleShowFloor();

	/**
	 * Returns true if the grid is currently visible in the viewport
	 */
	bool GetShowGrid();

	/**
	 * Will toggle the grid's visibility in the viewport
	 */
	void ToggleShowGrid();

	/**
	 * Syncs the local grid option with our global grid setting.
	 */
	void SyncShowGrid();
	
	/**
	 * Shows the world origin location.
	 */
	void ToggleShowOrigin();

	/**
	 * Returns true if the world origin is shown.
	 */
	bool GetShowOrigin() const;

	/**
	 * Returns true if AA enabled.
	 */
	bool GetEnableAA() const;

	/**
	 * Enables or disables AA.
	 */
	void ToggleEnableAA();
	
	/**
	 * Shows the preview components.
	 */
	void ToggleShowPreview();

	/**
	 * Returns true if preview components are visible.
	 */
	bool GetShowPreview() const;

	/**
	 * Syncs local preview with global preview setting.
	 */
	void SyncShowPreview();

	/**
	 * Display viewport names under preview components.
	 */
	bool GetShowViewportNames() const;

	/**
	 * Toggle whether to display viewport names or not.
	 */
	void ToggleShowViewportNames();

	/**
	 * Requires previews enabled.
	 */
	bool CanToggleViewportNames() const;

	/** @return The current preview scale. */
	TOptional<float> GetPreviewResolutionScale() const;

	/** Sets the preview scale. */
	void SetPreviewResolutionScale(float InScale);
	
	/** @return The current xform gizmo scale. */
	TOptional<float> GetXformGizmoScale() const;

	/** Sets the current xform gizmo scale. */
	void SetXformGizmoScale(float InScale);

	/** @return Whether xform gizmos are being shown */
	bool IsShowingXformGizmos() const;

	/** Toggle whether to display xform gizmos or not. */
	void ToggleShowXformGizmos();
	
	/**
	 * Gets the current preview actor instance.
	 */
	AActor* GetPreviewActor() const;

protected:
	/**
	 * Initiates a transaction.
	 */
	void BeginTransaction(const FText& Description);

	/**
	 * Ends the current transaction, if one exists.
	 */
	void EndTransaction();

	/**
	 * Updates preview bounds and floor positioning
	 */
	void RefreshPreviewBounds();

	/**
	 * Displays relevant viewport information.
	 */
	void DisplayViewportInformation(FSceneView& SceneView, FCanvas& Canvas);

protected:
	UE::Widget::EWidgetMode WidgetMode;
	ECoordSystem WidgetCoordSystem;

	/** Weak reference to the editor hosting the viewport */
	TWeakPtr<class FDisplayClusterConfiguratorBlueprintEditor> BlueprintEditorPtr;

	/** The full bounds of the preview scene (encompasses all visible components) */
	FBoxSphereBounds PreviewActorBounds;

	/** If true then we are manipulating a specific property or component */
	bool bIsManipulating;

	/** The current transaction for undo/redo */
	FScopedTransaction* ScopedTransaction;

	/** Floor static mesh component */
	UStaticMeshComponent* EditorFloorComp;

	/** World origin static mesh component */
	UDisplayClusterWorldOriginComponent* WorldOriginComponent;
	
	/** If true, the physics simulation gets ticked */
	bool bIsSimulateEnabled;
	
	UStaticMeshComponent* SkyComponent;
	UMaterialInstanceConstant* InstancedSkyMaterial;
	UPostProcessComponent* PostProcessComponent;
	UAssetViewerSettings* DefaultSettings;

	int32 CurrentProfileIndex;
};
