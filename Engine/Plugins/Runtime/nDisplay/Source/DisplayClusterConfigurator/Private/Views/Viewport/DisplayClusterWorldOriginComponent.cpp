// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterWorldOriginComponent.h"
#include "Components/StaticMeshComponent.h"

#include "Engine/StaticMesh.h"
#include "Materials/MaterialInterface.h"
#include "Materials/Material.h"
#include "UObject/ConstructorHelpers.h"


UDisplayClusterWorldOriginComponent::UDisplayClusterWorldOriginComponent()
	: Super()
{
	PrimaryComponentTick.bCanEverTick = false;

	static ConstructorHelpers::FObjectFinder<UStaticMesh> OriginMesh(TEXT("/Engine/EditorMeshes/Axis_Guide"));
	static ConstructorHelpers::FObjectFinder<UMaterial> OriginMaterial(TEXT("/Engine/EngineMaterials/GizmoMaterial"));
	SetMaterial(0, OriginMaterial.Object);
	SetMaterial(1, OriginMaterial.Object);
	SetRelativeLocationAndRotation(FVector::ZeroVector, FRotator::ZeroRotator, false);
	SetRelativeScale3D(FVector(1.0f, 1.0f, 1.0f));
	SetStaticMesh(OriginMesh.Object);
	SetMobility(EComponentMobility::Static);
	SetCollisionEnabled(ECollisionEnabled::NoCollision);
	SetVisibility(true);
	SetCastShadow(false);
}