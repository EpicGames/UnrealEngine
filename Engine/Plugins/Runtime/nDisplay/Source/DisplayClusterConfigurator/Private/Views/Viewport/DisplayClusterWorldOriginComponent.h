// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/StaticMeshComponent.h"
#include "DisplayClusterWorldOriginComponent.generated.h"


/**
 * Display cluster world origin component (for in-Editor visualization)
 */
UCLASS(ClassGroup = (DisplayCluster))
class UDisplayClusterWorldOriginComponent
	: public UStaticMeshComponent
{
	GENERATED_BODY()

public:
	UDisplayClusterWorldOriginComponent();
};
