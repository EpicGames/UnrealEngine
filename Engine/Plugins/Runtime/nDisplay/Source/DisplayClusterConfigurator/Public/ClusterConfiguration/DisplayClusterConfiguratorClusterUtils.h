// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class UDisplayClusterConfigurationData_Base;
class UDisplayClusterConfigurationCluster;
class UDisplayClusterConfigurationHostDisplayData;
class UDisplayClusterConfigurationClusterNode;
class UDisplayClusterConfigurationViewport;

namespace UE::DisplayClusterConfiguratorClusterUtils
{
	/**
	 * Adds a cluster node to a cluster, removing it from its current cluster if it belongs to one.
	 * @param ClusterNode - The cluster node to add
	 * @param Cluster - The cluster to add the cluster node to
	 * @param NewClusterNodeName - Optional name to add the new cluster node under in the cluster. If not supplied, the cluster node's current name is used, if possible.
	 * @return An updated pointer to the cluster node added to the cluster.
	 */
	DISPLAYCLUSTERCONFIGURATOR_API UDisplayClusterConfigurationClusterNode* AddClusterNodeToCluster(UDisplayClusterConfigurationClusterNode* ClusterNode, UDisplayClusterConfigurationCluster* Cluster, FString NewClusterNodeName = "");

	/**
	 * Removes a cluster node from the cluster that owns it. The owning cluster is retrieved using the cluster node's Outer.
	 * @param ClusterNode - The cluster node to remove
	 * @returns true if the cluster node was successfully removed from its owning cluster
	 */
	DISPLAYCLUSTERCONFIGURATOR_API bool RemoveClusterNodeFromCluster(UDisplayClusterConfigurationClusterNode* ClusterNode);

	/**
	 * Renames a cluster node, which involves changing the key that the cluster node is stored under in its parent cluster.
	 * @param ClusterNode - The cluster node to rename
	 * @param NewClusterNodeName - The desired name to change to. The final name will be a unique-ified version of this name.
	 * @return true if the cluster node was successfully renamed
	 */
	DISPLAYCLUSTERCONFIGURATOR_API bool RenameClusterNode(UDisplayClusterConfigurationClusterNode* ClusterNode, FString NewClusterNodeName);

	/**
	 * Sets the cluster node as the primary node.
	 * @param ClusterNode - The cluster node to set as the primary node
	 * @return true if the cluster node was successfully set as the primary node
	 */
	DISPLAYCLUSTERCONFIGURATOR_API bool SetClusterNodeAsPrimary(UDisplayClusterConfigurationClusterNode* ClusterNode);

	/**
	 * Determines if a cluster node is the primary node in its cluster.
	 * @param ClusterNode - The cluster node to check.
	 * @return true if the node is the primary; otherwise, false.
	 */
	DISPLAYCLUSTERCONFIGURATOR_API bool IsClusterNodePrimary(UDisplayClusterConfigurationClusterNode* ClusterNode);
	
	/**
	 * Adds a viewport to a cluster node, removing it from its current cluster node if it belongs to one.
	 * @param Viewport - The viewport to add
	 * @param ClusterNode - The cluster node to add the viewport to
	 * @param NewViewportName - Optional name to add the new viewport under in the cluster node. If not supplied, the viewport's current name is used, if possible.
	 * @return An updated pointer to the viewport added to the cluster.
	 */
	DISPLAYCLUSTERCONFIGURATOR_API UDisplayClusterConfigurationViewport* AddViewportToClusterNode(UDisplayClusterConfigurationViewport* Viewport, UDisplayClusterConfigurationClusterNode* ClusterNode, FString NewViewportName = "");

	/**
	 * Removes a viewport from the cluster node that owns it. The owning cluster node is retrieved using the viewport's Outer.
	 * @param Viewport - The viewport to remove
	 * @returns true if the viewport was successfully removed from its owning cluster
	 */
	DISPLAYCLUSTERCONFIGURATOR_API bool RemoveViewportFromClusterNode(UDisplayClusterConfigurationViewport* Viewport);

	/**
	 * Renames a viewport, which involves changing the key that the viewport is stored under in its parent cluster node.
	 * @param Viewport - The viewport to rename
	 * @param NewViewportName - The desired name to change to. The final name will be a unique-ified version of this name.
	 * @return true if the viewport was successfully renamed
	 */
	DISPLAYCLUSTERCONFIGURATOR_API bool RenameViewport(UDisplayClusterConfigurationViewport* Viewport, FString NewViewportName);

	/**
	 * Attempts to find a preexisting display data object for the specified host, and if one can't be found, creates a new one.
	 * @param Cluster - The cluster to find the host display data object in
	 * @param HostIPAddress - The IP address of the host to find the display data for
	 * @return The display data that was found or created
	 */
	DISPLAYCLUSTERCONFIGURATOR_API UDisplayClusterConfigurationHostDisplayData* FindOrCreateHostDisplayData(UDisplayClusterConfigurationCluster* Cluster, FString HostIPAddress);

	/**
	 * Removes all host display data objects from the cluster that aren't being used by any of the cluster nodes.
	 * @param Cluster - The cluster to clean up the host display data for
	 * @return true if any host display data was removed; otherwise, false
	 */
	DISPLAYCLUSTERCONFIGURATOR_API bool RemoveUnusedHostDisplayData(UDisplayClusterConfigurationCluster* Cluster);

	/**
	 * Gets a unique name for a host owned by the specified cluster.
	 * @param InitialName - The initial name the host wants
	 * @param ParentCluster - The cluster the host belongs to
	 * @param bAddZero - Whether to add an "_0" to the initial name if it is unique
	 * @returns A unique name for the host
	 */
	DISPLAYCLUSTERCONFIGURATOR_API FString GetUniqueNameForHost(FString InitialName, UDisplayClusterConfigurationCluster* ParentCluster, bool bAddZero = false);

	/**
	 * Gets the address of a host using its host display data.
	 * @param HostDisplayData - The display data of the host to get the address for
	 * @return The address of the host, or an empty string if no host address was found
	 */
	DISPLAYCLUSTERCONFIGURATOR_API FString GetAddressForHost(UDisplayClusterConfigurationHostDisplayData* HostDisplayData);

	/**
	 * Removes all cluster nodes and host display data from the cluster for the specified host.
	 * @param Cluster - The cluster to remove the host from.
	 * @param Host - The host to remove.
	 * @return true if any cluster nodes or host display data was removed; otherwise, false
	 */
	DISPLAYCLUSTERCONFIGURATOR_API bool RemoveHost(UDisplayClusterConfigurationCluster* Cluster, FString Host);

	/**
	 * Attempts to get the host display data for the host of a cluster node.
	 * @param ClusterNode - The cluster node whose host to get the display data for.
	 * @return The display data for the cluster node's host, or null if no display data was found.
	 */
	DISPLAYCLUSTERCONFIGURATOR_API UDisplayClusterConfigurationHostDisplayData* GetHostDisplayDataForClusterNode(UDisplayClusterConfigurationClusterNode* ClusterNode);

	/**
	 * Gets the name that the cluster node is stored under in its parent cluster.
	 * @return The cluster node's storage name
	 */
	DISPLAYCLUSTERCONFIGURATOR_API FString GetClusterNodeName(UDisplayClusterConfigurationClusterNode* ClusterNode);

	/**
	 * Gets a unique name for a cluster node owned by the specified cluster.
	 * @param InitialName - The initial name the cluster node wants
	 * @param ParentCluster - The cluster the cluster node belongs to
	 * @param bAddZero - Whether to add an "_0" to the initial name if it is unique
	 * @returns A unique name for the cluster node
	 */
	DISPLAYCLUSTERCONFIGURATOR_API FString GetUniqueNameForClusterNode(FString InitialName, UDisplayClusterConfigurationCluster* ParentCluster, bool bAddZero = false);

	/**
	 * Gets the name that the viewport is stored under in its parent cluster node.
	 * @return The viewport's storage name
	 */
	DISPLAYCLUSTERCONFIGURATOR_API FString GetViewportName(UDisplayClusterConfigurationViewport* Viewport);

	/**
	 * Gets a unique name for a viewport owned by the specified cluster node.
	 * @param InitialName - The initial name the viewport wants
	 * @param ParentClusterNode - The cluster node the viewport belongs to
	 * @param bAddZero - Whether to add an "_0" to the initial name if it is unique
	 * @returns A unique name for the viewport
	 */
	DISPLAYCLUSTERCONFIGURATOR_API FString GetUniqueNameForViewport(FString InitialName, UDisplayClusterConfigurationClusterNode* ParentClusterNode, bool bAddZero = false);

	/**
	 * Sorts a list of cluster nodes by their host.
	 * @param InClusterNodes - The list of nodes to sort
	 * @param OutSortedNodes - The sorted nodes, indexed by the Host property of the cluster nodes
	 */
	DISPLAYCLUSTERCONFIGURATOR_API void SortClusterNodesByHost(const TMap<FString, UDisplayClusterConfigurationClusterNode*>& InClusterNodes, TMap<FString, TMap<FString, UDisplayClusterConfigurationClusterNode*>>& OutSortedNodes);

	/**
	 * Sorts a list of cluster nodes by their host.
	 * @param InClusterNodes - The list of nodes to sort
	 * @param OutSortedNodes - The sorted nodes, indexed by the Host property of the cluster nodes
	 */
	DISPLAYCLUSTERCONFIGURATOR_API void SortClusterNodesByHost(const TMap<FString, TObjectPtr<UDisplayClusterConfigurationClusterNode>>& InClusterNodes, TMap<FString, TMap<FString, UDisplayClusterConfigurationClusterNode*>>& OutSortedNodes);
};