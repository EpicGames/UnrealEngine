// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "EditorReimportHandler.h"

#include "DisplayClusterConfiguratorFactory.generated.h"

class UDisplayClusterBlueprint;

UCLASS(MinimalApi)
class UDisplayClusterConfiguratorFactory
	: public UFactory
{
	GENERATED_BODY()

public:
	UDisplayClusterConfiguratorFactory();
	~UDisplayClusterConfiguratorFactory();
	
	//~ Begin UFactory Interface
	virtual bool ConfigureProperties() override;
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context,
		FFeedbackContext* Warn, FName CallingContext) override;
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context,
		FFeedbackContext* Warn) override;
	virtual bool DoesSupportClass(UClass* Class) override;
	virtual UClass* ResolveSupportedClass() override;
	virtual UObject* FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, const FString& InFilename, const TCHAR* Parms, FFeedbackContext* Warn, bool& bOutOperationCanceled) override;
	virtual bool FactoryCanImport(const FString& Filename) override { return true; }
	virtual FString GetDefaultNewAssetName() const override;
	virtual uint32 GetMenuCategories() const override;
	//~ End UFactory Interface

	/** Create initial config data and components. */
	static void SetupNewBlueprint(UDisplayClusterBlueprint* NewBlueprint);

	/** Perform one time setup of open documents. */
	static void SetupInitialBlueprintDocuments(UDisplayClusterBlueprint* NewBlueprint);
private:
	void OnConfigureNewAssetRequest(UFactory* InFactory);

private:
	// The parent class of the created blueprint
	UPROPERTY()
	TSubclassOf<class ADisplayClusterRootActor> ParentClass;

	UPROPERTY(Transient)
	TObjectPtr<UDisplayClusterBlueprint> BlueprintToCopy;

	FDelegateHandle OnConfigureNewAssetRequestHandle;
	bool bIsConfigureNewAssetRequest;
};

UCLASS(MinimalApi)
class UDisplayClusterConfiguratorReimportFactory
	: public UDisplayClusterConfiguratorFactory
	, public FReimportHandler
{
	GENERATED_BODY()

	UDisplayClusterConfiguratorReimportFactory();

	//~ Begin FReimportHandler Interface
	virtual bool CanReimport(UObject* Obj, TArray<FString>& OutFilenames) override;
	virtual void SetReimportPaths(UObject* Obj, const TArray<FString>& NewReimportPaths) override;
	virtual EReimportResult::Type Reimport(UObject* Obj) override;
	virtual int32 GetPriority() const override;
	//~ End FReimportHandler Interface
};
