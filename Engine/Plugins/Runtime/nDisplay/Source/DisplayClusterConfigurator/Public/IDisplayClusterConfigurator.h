// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class FDisplayClusterConfiguratorCommands;

// Read only delegate
DECLARE_MULTICAST_DELEGATE_OneParam(FOnDisplayClusterConfiguratorReadOnlyChanged, bool /* bInEnabled */);
using FOnDisplayClusterConfiguratorReadOnlyChangedDelegate = FOnDisplayClusterConfiguratorReadOnlyChanged::FDelegate;

/**
 * Configurator editor module
 */
class IDisplayClusterConfigurator :
	public IModuleInterface
{
public:
	static constexpr const TCHAR* ModuleName = TEXT("DisplayClusterConfigurator");

public:
	virtual ~IDisplayClusterConfigurator() = default;

public:
	/**
	* Singleton-like access to this module's interface.  This is just for convenience!
	* Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	*
	* @return Returns singleton instance, loading the module on demand if needed
	*/
	static inline IDisplayClusterConfigurator& Get()
	{
		return FModuleManager::GetModuleChecked<IDisplayClusterConfigurator>(IDisplayClusterConfigurator::ModuleName);
	}

	/**
	* Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	*
	* @return True if the module is loaded and ready to use
	*/
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded(IDisplayClusterConfigurator::ModuleName);
	}

public:
	DISPLAYCLUSTERCONFIGURATOR_API virtual const FDisplayClusterConfiguratorCommands& GetCommands() const = 0;
	DISPLAYCLUSTERCONFIGURATOR_API virtual TSharedPtr<class FExtensibilityManager> GetMenuExtensibilityManager() const = 0;
	DISPLAYCLUSTERCONFIGURATOR_API virtual TSharedPtr<class FExtensibilityManager> GetToolBarExtensibilityManager() const = 0;
};
