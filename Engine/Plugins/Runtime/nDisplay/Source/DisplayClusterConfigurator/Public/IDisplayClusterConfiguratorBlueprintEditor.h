// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BlueprintEditor.h"

/**
 * TODO: Remove this completely, don't expose the BP Editor outside of the configuration module.
 */
class IDisplayClusterConfiguratorBlueprintEditor
	: public FBlueprintEditor
{
public:
	virtual ~IDisplayClusterConfiguratorBlueprintEditor() = default;

public:
	DECLARE_MULTICAST_DELEGATE(FOnConfigReloaded);
	DECLARE_MULTICAST_DELEGATE(FOnObjectSelected);
	DECLARE_MULTICAST_DELEGATE(FOnInvalidateViews);
	DECLARE_MULTICAST_DELEGATE(FOnClearViewportSelection);
	DECLARE_MULTICAST_DELEGATE(FOnClusterChanged);

	using FOnConfigReloadedDelegate = FOnConfigReloaded::FDelegate;
	using FOnObjectSelectedDelegate = FOnObjectSelected::FDelegate;
	using FOnInvalidateViewsDelegate = FOnInvalidateViews::FDelegate;
	using FOnClearViewportSelectionDelegate = FOnClearViewportSelection::FDelegate;
	using FOnClusterChangedDelegate = FOnClusterChanged::FDelegate;

public:
	virtual TArray<UObject*> GetSelectedObjects() const = 0;

	virtual bool IsObjectSelected(UObject* Obj) const = 0;
};
