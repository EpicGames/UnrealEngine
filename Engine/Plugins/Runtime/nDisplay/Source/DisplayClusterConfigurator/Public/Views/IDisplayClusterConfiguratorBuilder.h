// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class IDisplayClusterConfiguratorBuilder
	: public TSharedFromThis<IDisplayClusterConfiguratorBuilder>
{
public:
	virtual ~IDisplayClusterConfiguratorBuilder() = default;
};
