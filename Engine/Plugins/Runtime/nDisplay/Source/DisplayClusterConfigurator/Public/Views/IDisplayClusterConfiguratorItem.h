// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"


class IDisplayClusterConfiguratorItem
	: public TSharedFromThis<IDisplayClusterConfiguratorItem>
{
public:
	virtual ~IDisplayClusterConfiguratorItem() = default;

public:
	/** Get the object represented by this item, if any */
	virtual UObject* GetObject() const = 0;

	virtual void OnSelection() = 0;

	virtual bool IsSelected() = 0;
};
