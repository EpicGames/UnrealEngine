// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class SWidget;


class IDisplayClusterConfiguratorView
	: public TSharedFromThis<IDisplayClusterConfiguratorView>
{
public:
	virtual ~IDisplayClusterConfiguratorView() = default;

public:
	virtual TSharedRef<SWidget> CreateWidget() = 0;
	virtual TSharedRef<SWidget> GetWidget() = 0;
	
	virtual void SetEnabled(bool bInEnabled) {};
	virtual bool GetIsEnabled() const { return false; };
};
