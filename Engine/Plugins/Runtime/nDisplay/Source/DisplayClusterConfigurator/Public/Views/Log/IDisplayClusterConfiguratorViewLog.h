// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Views/IDisplayClusterConfiguratorView.h"

class IMessageLogListing;

class IDisplayClusterConfiguratorViewLog
	: public IDisplayClusterConfiguratorView
{
public:
	virtual ~IDisplayClusterConfiguratorViewLog() = default;

public:
	enum class EVerbosityLevel
	{
		Log,
		Warning,
		Error
	};

public:
	virtual TSharedRef<IMessageLogListing> GetMessageLogListing() const = 0;

	virtual TSharedRef<IMessageLogListing> CreateLogListing() = 0;

	virtual void Log(const FText& Message, EVerbosityLevel Verbosity = EVerbosityLevel::Log) = 0;
};
