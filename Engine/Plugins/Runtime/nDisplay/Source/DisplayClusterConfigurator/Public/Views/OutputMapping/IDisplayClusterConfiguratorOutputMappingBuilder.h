// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Views/IDisplayClusterConfiguratorBuilder.h"

class IDisplayClusterConfiguratorOutputMappingSlot;
class SWidget;
class UObject;

/**
 * The Interface for generation of nodes from config
 */
class IDisplayClusterConfiguratorOutputMappingBuilder
	: public IDisplayClusterConfiguratorBuilder
{
public:
	virtual ~IDisplayClusterConfiguratorOutputMappingBuilder() = default;

public:
	struct FSlot
	{
		static const FName Canvas;
		static const FName Window;
		static const FName Viewport;
	};

	virtual void Build() = 0;

	virtual TSharedRef<SWidget> GetCanvasWidget() const = 0;

	virtual const TArray<TSharedPtr<IDisplayClusterConfiguratorOutputMappingSlot>>& GetAllSlots() const = 0;
};
