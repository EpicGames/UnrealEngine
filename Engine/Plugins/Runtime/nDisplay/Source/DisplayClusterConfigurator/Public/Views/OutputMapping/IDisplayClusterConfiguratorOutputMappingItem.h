// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Views/IDisplayClusterConfiguratorItem.h"

class IDisplayClusterConfiguratorOutputMappingSlot;

/**
 * The Interface holsd object and setting from Details View
 */
class IDisplayClusterConfiguratorOutputMappingItem
	: public IDisplayClusterConfiguratorItem
{
public:
	virtual ~IDisplayClusterConfiguratorOutputMappingItem() = default;

public:
	virtual const FString& GetNodeName() const = 0;
};
