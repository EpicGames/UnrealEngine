// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Views/IDisplayClusterConfiguratorView.h"

class UTexture;

struct FOutputMappingSettings
{
	bool bShowRuler;
	bool bShowWindowInfo;
	bool bShowWindowCornerImage;
	bool bShowOutsideViewports;
	bool bAllowClusterItemOverlap;
	bool bKeepClusterNodesInHosts;
	bool bLockViewports;
	bool bLockClusterNodes;
	bool bTintSelectedViewports;
	float ViewScale;

	FOutputMappingSettings() :
		bShowRuler(true),
		bShowWindowInfo(true),
		bShowWindowCornerImage(true),
		bShowOutsideViewports(false),
		bAllowClusterItemOverlap(false),
		bKeepClusterNodesInHosts(true),
		bLockViewports(false),
		bLockClusterNodes(false),
		bTintSelectedViewports(true),
		ViewScale(1.0f)
	{ }
};

enum class EHostArrangementType : uint8
{
	Horizontal,
	Vertical,
	Wrap,
	Grid
};

struct FHostNodeArrangementSettings
{
	EHostArrangementType ArrangementType;
	float WrapThreshold;
	int GridSize;

	FHostNodeArrangementSettings() :
		ArrangementType(EHostArrangementType::Wrap),
		WrapThreshold(5400),
		GridSize(4)
	{ }
};

struct FNodeAlignmentSettings
{
	int32 SnapProximity;
	int32 AdjacentEdgesSnapPadding;

	bool bSnapAdjacentEdges;
	bool bSnapSameEdges;

	FNodeAlignmentSettings() :
		SnapProximity(25),
		AdjacentEdgesSnapPadding(0),
		bSnapAdjacentEdges(true),
		bSnapSameEdges(true)
	{ }
};

/**
 * The Interface for controll the Output Mapping Мiew
 */
class IDisplayClusterConfiguratorViewOutputMapping
	: public IDisplayClusterConfiguratorView
{
public:
	virtual ~IDisplayClusterConfiguratorViewOutputMapping() = default;

public:
	DECLARE_MULTICAST_DELEGATE(FOnOutputMappingBuilt);

	using FOnOutputMappingBuiltDelegate = FOnOutputMappingBuilt::FDelegate;

public:
	virtual void Cleanup() = 0;

	virtual const FOutputMappingSettings& GetOutputMappingSettings() const = 0;
	virtual FOutputMappingSettings& GetOutputMappingSettings() = 0;

	virtual const FHostNodeArrangementSettings& GetHostArrangementSettings() const = 0;
	virtual FHostNodeArrangementSettings& GetHostArrangementSettings() = 0;

	virtual const FNodeAlignmentSettings& GetNodeAlignmentSettings() const = 0;
	virtual FNodeAlignmentSettings& GetNodeAlignmentSettings() = 0;

	virtual FOnOutputMappingBuilt& GetOnOutputMappingBuiltDelegate() = 0;
	virtual FDelegateHandle RegisterOnOutputMappingBuilt(const FOnOutputMappingBuiltDelegate& Delegate) = 0;
	virtual void UnregisterOnOutputMappingBuilt(FDelegateHandle DelegateHandle) = 0;

	virtual void FindAndSelectObjects(const TArray<UObject*>& ObjectsToSelect) = 0;

	virtual void JumpToObject(UObject* InObject) = 0;
};
