// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Views/IDisplayClusterConfiguratorBuilder.h"
#include "Containers/ArrayView.h"

class IDisplayClusterConfiguratorTreeItem;
class IDisplayClusterConfiguratorViewTree;

/** Output struct for builders to use */
struct DISPLAYCLUSTERCONFIGURATOR_API FDisplayClusterConfiguratorTreeBuilderOutput
{
	FDisplayClusterConfiguratorTreeBuilderOutput(TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>>& InItems, TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>>& InLinearItems)
		: Items(InItems)
		, LinearItems(InLinearItems)
	{ }

	/**
	 * Add an item to the output
	 * @param	InItem			The item to add
	 * @param	InParentName	The name of the item's parent
	 * @param	InParentTypes	The types of items to search. If this is empty all items will be searched.
	 * @param	bAddToHead		Whether to add the item to the start or end of the parent's children array
	 */
	void Add(const TSharedPtr<IDisplayClusterConfiguratorTreeItem>& InItem, const FName& InParentName, TArrayView<const FName> InParentTypes, bool bAddToHead = false);

	/**
	 * Add an item to the output
	 * @param	InItem			The item to add
	 * @param	InParentName	The name of the item's parent
	 * @param	InParentTypes	The types of items to search. If this is empty all items will be searched.
	 * @param	bAddToHead		Whether to add the item to the start or end of the parent's children array
	 */
	void Add(const TSharedPtr<IDisplayClusterConfiguratorTreeItem>& InItem, const FName& InParentName, std::initializer_list<FName> InParentTypes, bool bAddToHead = false)
	{
		Add(InItem, InParentName, MakeArrayView(InParentTypes), bAddToHead);
	}

	/**
	 * Add an item to the output
	 * @param	InItem			The item to add
	 * @param	InParentName	The name of the item's parent
	 * @param	InParentType	The type of items to search. If this is NAME_None all items will be searched.
	 * @param	bAddToHead		Whether to add the item to the start or end of the parent's children array
	 */
	void Add(const TSharedPtr<IDisplayClusterConfiguratorTreeItem>& InItem, const FName& InParentName, const FName& InParentType, bool bAddToHead = false);

	/**
	 * Find the item with the specified name
	 * @param	InName	The item's name
	 * @param	InTypes	The types of items to search. If this is empty all items will be searched.
	 * @return the item found, or an invalid ptr if it was not found.
	 */
	TSharedPtr<IDisplayClusterConfiguratorTreeItem> Find(const FName& InName, TArrayView<const FName> InTypes) const;

	/**
	 * Find the item with the specified name
	 * @param	InName	The item's name
	 * @param	InTypes	The types of items to search. If this is empty all items will be searched.
	 * @return the item found, or an invalid ptr if it was not found.
	 */
	TSharedPtr<IDisplayClusterConfiguratorTreeItem> Find(const FName& InName, std::initializer_list<FName> InTypes) const
	{
		return Find(InName, MakeArrayView(InTypes));
	}

	/**
	 * Find the item with the specified name
	 * @param	InName	The item's name
	 * @param	InType	The type of items to search. If this is NAME_None all items will be searched.
	 * @return the item found, or an invalid ptr if it was not found.
	 */
	TSharedPtr<IDisplayClusterConfiguratorTreeItem> Find(const FName& InName, const FName& InType) const;

private:
	/** The items that are built by this builder */
	TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>>& Items;

	/** A linearized list of all items in OutItems (for easier searching) */
	TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>>& LinearItems;
};

/** 
 * Interface to implement to provide custom build logic to tree views 
 */
class IDisplayClusterConfiguratorTreeBuilder
	: public IDisplayClusterConfiguratorBuilder
{
public:
	virtual ~IDisplayClusterConfiguratorTreeBuilder() = default;

public:
	/** Setup this builder with links to the tree */
	virtual void Initialize(const TSharedRef<IDisplayClusterConfiguratorViewTree>& InConfiguratorTree) = 0;

	/**
	 * Build an array of item tree items to display in the tree.
	 * @param	Output			The items that are built by this builder
	 */
	virtual void Build(FDisplayClusterConfiguratorTreeBuilderOutput& Output) = 0;
};
