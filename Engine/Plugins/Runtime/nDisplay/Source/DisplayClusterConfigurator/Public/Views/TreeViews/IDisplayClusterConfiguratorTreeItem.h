// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Views/IDisplayClusterConfiguratorItem.h"

#include "Framework/SlateDelegates.h"

enum class EItemDropZone;
class IDisplayClusterConfiguratorViewTree;
class ITableRow;
class SHorizontalBox;
class STableViewBase;
class SWidget;
class FTextFilterExpressionEvaluator;
class FDragDropEvent;

template< typename ObjectType >
class TAttribute;


enum class EDisplayClusterConfiguratorTreeFilterResult : uint8
{
	/** Hide the item */
	Hidden,

	/** Show the item because child items were shown */
	ShownDescendant,

	/** Show the item */
	Shown,

	/** Show the item and highlight search text */
	ShownHighlighted,
};


#define NDISPLAY_TREE_BASE_ITEM_TYPE(TYPE) \
	static const FName& GetTypeId() { static FName Type(TEXT(#TYPE)); return Type; } \
	virtual bool IsOfTypeByName(const FName& Type) const { return TYPE::GetTypeId() == Type; } \
	virtual FName GetTypeName() const { return TYPE::GetTypeId(); }


class IDisplayClusterConfiguratorTreeItem
	: public IDisplayClusterConfiguratorItem
{
public:
	virtual ~IDisplayClusterConfiguratorTreeItem() = default;

public:
	NDISPLAY_TREE_BASE_ITEM_TYPE(IDisplayClusterConfiguratorTreeItem)

	/** Initializes the tree item */
	virtual void Initialize() = 0;

	/** Builds the table row widget to display this info */
	virtual TSharedRef<ITableRow> MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, const TAttribute<FText>& InFilterText) = 0;

	/**
	 * Builds a widget for the specified column.
	 * @param ColumnName - The name of the column to build the widget for.
	 * @param TableRow - The table row the widget is being generated for.
	 * @param FilterText - The current filter being applied to the tree view.
	 * @param InIsSelected - The FIsSelected delegate for the current table row.
	 */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName, TSharedPtr<ITableRow> TableRow, const TAttribute<FText>& FilterText, FIsSelected InIsSelected) = 0;

	/** Builds the slate widget for any inline data editing */
	virtual TSharedRef< SWidget > GenerateInlineEditWidget(const TAttribute<FText>& FilterText, FIsSelected InIsSelected) = 0;

	/** @return true if the item has an in-line editor widget */
	virtual bool HasInlineEditor() const = 0;

	/** Toggle the expansion state of the inline editor */
	virtual void ToggleInlineEditorExpansion() = 0;

	/** Get the expansion state of the inline editor */
	virtual bool IsInlineEditorExpanded() const = 0;

	/** Get the name of the item that this row represents */
	virtual FName GetRowItemName() const = 0;

	/** Get the style for image icon */
	virtual FString GetIconStyle() const = 0;

	/** Return the name used to attach to this item */
	virtual FName GetAttachName() const = 0;

	/** @return true if this item can be renamed */
	virtual bool CanRenameItem() const = 0;

	/** Requests a rename on the the tree row item */
	virtual void RequestRename() = 0;

	/** @return true if this item can be deleted */
	virtual bool CanDeleteItem() const = 0;

	/** Deletes the tree row item and its backing object. */
	virtual void DeleteItem() const = 0;

	/** @return true if this item can be duplicated */
	virtual bool CanDuplicateItem() const = 0;

	/** @return true if this item can be hidden */
	virtual bool CanHideItem() const = 0;
	
	/** Sets the item's visibility */
	virtual void SetItemHidden(bool bIsHidden) = 0;

	/** Get the objects of all parents of this item */
	virtual void GetParentObjectsRecursive(TArray<UObject*>& OutObjects) const = 0;

	/** Get all Children object  */
	virtual void GetChildrenObjectsRecursive(TArray<UObject*>& OutObjects) const = 0;

	/** Handler for when the user double clicks on this item in the tree */
	virtual void OnItemDoubleClicked() = 0;

	/** Handler for when the user enter the mouse on this item in the tree */
	virtual void OnMouseEnter() = 0;

	/** Handler for when the user leave the mouse on this item in the tree */
	virtual void OnMouseLeave() = 0;

	/** Return true if tree item is hovered */
	virtual bool IsHovered() const = 0;

	/** Return true if that is root tree item */
	virtual bool IsRoot() const = 0;

	/** Return true if this item is a child of the given parent tree item */
	virtual bool IsChildOfRecursive(const TSharedRef<IDisplayClusterConfiguratorTreeItem>& InParentTreeItem) const = 0;

	/** Handle a drag and drop detected event */
	virtual FReply HandleDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) = 0;

	/** Handle a drag and drop enter event */
	virtual void HandleDragEnter(const FDragDropEvent& DragDropEvent) = 0;

	/** Handle a drag and drop leave event */
	virtual void HandleDragLeave(const FDragDropEvent& DragDropEvent) = 0;

	/** Handles the CanAcceptDrop callback for the tree item */
	virtual TOptional<EItemDropZone> HandleCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<IDisplayClusterConfiguratorTreeItem> TargetItem) = 0;

	/** Handle Accep drag and drop drop event */
	virtual FReply HandleAcceptDrop(const FDragDropEvent& DragDropEvent, TSharedPtr<IDisplayClusterConfiguratorTreeItem> TargetItem) = 0;

	/** Handle a drag and drop drop event */
	virtual FReply HandleDrop(FDragDropEvent const& DragDropEvent) = 0;

	/** Get this item's parent  */
	virtual TSharedPtr<IDisplayClusterConfiguratorTreeItem> GetParent() const = 0;

	/** Set this item's parent */
	virtual void SetParent(TSharedPtr<IDisplayClusterConfiguratorTreeItem> InParent) = 0;

	/** The array of children for this item */
	virtual TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>>& GetChildren() = 0;

	/** The const array of children for this item */
	virtual const TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>>& GetChildrenConst() const = 0;

	/** The filtered array of children for this item */
	virtual TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>>& GetFilteredChildren() = 0;

	/** The owning view tree */
	virtual TSharedRef<IDisplayClusterConfiguratorViewTree> GetConfiguratorTree() const = 0;

	/**
	 * Applies the specified text filter to the tree item and returns the results
	 * @param TextFilter - The filter to apply
	 * @return The results of applying the filter to this tree item
	 */
	virtual EDisplayClusterConfiguratorTreeFilterResult ApplyFilter(const TSharedPtr<FTextFilterExpressionEvaluator>& TextFilter) = 0;

	/** Get the current filter result */
	virtual EDisplayClusterConfiguratorTreeFilterResult GetFilterResult() const = 0;

	/** Set the current filter result */
	virtual void SetFilterResult(EDisplayClusterConfiguratorTreeFilterResult InResult) = 0;

	/** Get whether this item begins expanded or not */
	virtual bool IsInitiallyExpanded() const = 0;
};

#define NDISPLAY_TREE_ITEM_TYPE(TYPE, BASE) \
	static const FName& GetTypeId() { static FName Type(TEXT(#TYPE)); return Type; } \
	virtual bool IsOfTypeByName(const FName& Type) const override { return GetTypeId() == Type || BASE::IsOfTypeByName(Type); } \
	virtual FName GetTypeName() const { return TYPE::GetTypeId(); }
