// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Views/IDisplayClusterConfiguratorView.h"

#include "Widgets/Views/SHeaderRow.h"

class IDisplayClusterConfiguratorTreeBuilder;
class IDisplayClusterConfiguratorTreeItem;
class UDisplayClusterConfigurationData;
struct FDisplayClusterConfiguratorTreeFilterArgs;
class FUICommandList_Pinnable;
class FMenuBuilder;
class FTextFilterExpressionEvaluator;

enum class EDisplayClusterConfiguratorTreeFilterResult : uint8;

enum class EDisplayClusterConfiguratorTreeMode
{
	Editor,
	Picker
};

struct FDisplayClusterConfiguratorTreeArgs
{
	FDisplayClusterConfiguratorTreeArgs()
		: Mode(EDisplayClusterConfiguratorTreeMode::Editor)
		, ContextName(TEXT("ConfiguratorTree"))
	{ }

	/** Optional builder to allow for custom tree construction */
	TSharedPtr<IDisplayClusterConfiguratorTreeBuilder> Builder;

	/** The mode that this items tree is in */
	EDisplayClusterConfiguratorTreeMode Mode;

	/** Context name used to persist settings */
	FName ContextName;
};


/** Basic filter used when re-filtering the tree */
struct FDisplayClusterConfiguratorTreeFilterArgs
{
	FDisplayClusterConfiguratorTreeFilterArgs(TSharedPtr<FTextFilterExpressionEvaluator> InTextFilter)
		: TextFilter(InTextFilter)
		, bFlattenHierarchyOnFilter(false)
	{}

	/** The text filter we are using, if any */
	TSharedPtr<FTextFilterExpressionEvaluator> TextFilter;

	/** Whether to flatten the hierarchy so filtered items appear in a linear list */
	bool bFlattenHierarchyOnFilter;
};

/*
 * Base interface for editor tree view
 */
class IDisplayClusterConfiguratorViewTree
	: public IDisplayClusterConfiguratorView
{
public:
	virtual ~IDisplayClusterConfiguratorViewTree() = default;

public:
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnHoveredItemSet, const TSharedRef<IDisplayClusterConfiguratorTreeItem>&);
	DECLARE_MULTICAST_DELEGATE(FOnHoveredItemCleared);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnItemsSelected, const TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>>&);
	DECLARE_MULTICAST_DELEGATE(FOnSelectionCleared);

	using FOnHoveredItemSetDelegate = FOnHoveredItemSet::FDelegate;
	using FOnHoveredItemClearedDelegate = FOnHoveredItemCleared::FDelegate;
	using FOnItemsSelectedDelegate = FOnItemsSelected::FDelegate;
	using FOnSelectionClearedDelegate = FOnSelectionCleared::FDelegate;

	/**
	 * Type of the tree item
	 */
	struct Columns
	{
		static const FName Item;
		static const FName Group;
	};

	/**
	 * Rebuild the tree from the config
	 */
	virtual void RebuildTree() = 0;

	/**
	 * @return The Editor editing UObject
	 */
	virtual UDisplayClusterConfigurationData* GetEditorData() const = 0;

	/**
	 * Generates a list of columns to add to the tree view.
	 * @param OutColumnArgs - The list of column arguments to construct the columns from.
	 */
	virtual void ConstructColumns(TArray<SHeaderRow::FColumn::FArguments>& OutColumnArgs) = 0;

	/**
	 * Sets currently hovered tree item
	 *
	 * @param InTreeItem	 hovered tree item
	 */
	virtual void SetHoveredItem(const TSharedRef<IDisplayClusterConfiguratorTreeItem>& InTreeItem) = 0;

	/**
	 * Remove hovered Item from this tree
	 */
	virtual void ClearHoveredItem() = 0;

	/**
	 * @return The hovered tree item
	 */
	virtual TSharedPtr<IDisplayClusterConfiguratorTreeItem> GetHoveredItem() const = 0;

	/**
	 * Sets currently selected tree items
	 *
	 * @param InTreeItems - List of selected tree items
	 */
	virtual void SetSelectedItems(const TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>>& InTreeItems) = 0;

	/**
	 * Removes all tree items from list of selected items
	 */
	virtual void ClearSelection() = 0;

	/**
	 * @return The list of currently selected tree items
	 */
	virtual TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>> GetSelectedItems() const = 0;

	/** Gets the objects that are currently selected in the tree view. */
	virtual void GetSelectedObjects(TArray<UObject*>& OutObjects) const = 0;

	/**
	 * Finds any tree items that represent the specified objects and selects them.
	 * @param ObjectsToSelect - The objects to select
	 */
	virtual void FindAndSelectObjects(const TArray<UObject*>& ObjectsToSelect) = 0;

	/**
	 * Filters the specified list of items based on the filter arguments, and outputs the filtered list.
	 * @param InArgs - The filtering arguments
	 * @param InItems - The list of items to filter
	 * @param OutFilteredItems - The output list of items that matched the filter
	 */
	virtual void Filter(const FDisplayClusterConfiguratorTreeFilterArgs& InArgs, const TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>>& InItems, TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>>& OutFilteredItems) = 0;

	/**
	 * Sorts the specified list of items based on the current sort configuration and outputs the sorted list.
	 * @param InItems - The list of items to sort
	 * @param OutSortedItems - The output list of sorted items
	 */
	virtual void Sort(const TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>>& InItems, TArray<TSharedPtr<IDisplayClusterConfiguratorTreeItem>>& OutSortedItems) = 0;

	/**
	 * Register the hovered tree item delegate
	 */
	virtual FDelegateHandle RegisterOnHoveredItemSet(const FOnHoveredItemSetDelegate& Delegate) = 0;

	/**
	 * Remove the hovered tree item delegate
	 */
	virtual void UnregisterOnHoveredItemSet(FDelegateHandle DelegateHandle) = 0;

	/**
	 * Register clear hovered tree item delegate
	 */
	virtual FDelegateHandle RegisterOnHoveredItemCleared(const FOnHoveredItemClearedDelegate& Delegate) = 0;

	/**
	 * Unregister clear hovered tree item delegate
	 */
	virtual void UnregisterOnHoveredItemCleared(FDelegateHandle DelegateHandle) = 0;


	/**
	 * Raised when the context menu is being shown for an item in the tree view.
	 * @param MenuBuilder - The menu builder of the context menu
	 */
	virtual void FillContextMenu(FMenuBuilder& MenuBuilder) = 0;

	/**
	 * Raised on tree view creation when binding UI commands
	 * @param CommandList - The command list to bind commands to
	 */
	virtual void BindPinnableCommands(FUICommandList_Pinnable& CommandList) = 0;

	/**
	 * @return true if the tree view should show an "Add New" button in its top bar
	 */
	virtual bool ShowAddNewButton() const = 0;

	/**
	 * Raised when the "Add New" combo button is clicked and its menu is being opened
	 */
	virtual void FillAddNewMenu(FMenuBuilder& MenuBuilder) = 0;

	/**
	 * @return true if the tree view should show a filter options button in its top bar
	 */
	virtual bool ShowFilterOptionsButton() const = 0;

	/**
	 * Raised when the filter options combo button is clicked and its menu is being opened
	 */
	virtual void FillFilterOptionsMenu(FMenuBuilder& MenuBuilder) = 0;

	/**
	 * @return true if the tree view should show a "View Options" button in its bottom bar
	 */
	virtual bool ShowViewOptionsButton() const = 0;

	/**
	 * Raised when the "View Options" combo button is clicked and its menu is being opened
	 */
	virtual void FillViewOptionsMenu(FMenuBuilder& MenuBuilder) = 0;

	/**
	 * @return The watermark text to display in the bottom right corner of the tree view
	 */
	virtual FText GetCornerText() const = 0;
};
