// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AdvancedPreviewScene.h"


class IDisplayClusterConfiguratorPreviewScene 
	: public FAdvancedPreviewScene
{
public:
	/** Constructor only here to pass ConstructionValues to base constructor */
	IDisplayClusterConfiguratorPreviewScene(ConstructionValues CVS)
		: FAdvancedPreviewScene(CVS)
	{ }

	virtual ~IDisplayClusterConfiguratorPreviewScene() = default;
};
