// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Views/IDisplayClusterConfiguratorView.h"

class IDisplayClusterConfiguratorPreviewScene;


class IDisplayClusterConfiguratorViewViewport
	: public IDisplayClusterConfiguratorView
{
public:
	virtual ~IDisplayClusterConfiguratorViewViewport() = default;

public:
	virtual TSharedRef<IDisplayClusterConfiguratorPreviewScene> GetPreviewScene() const = 0;
};
