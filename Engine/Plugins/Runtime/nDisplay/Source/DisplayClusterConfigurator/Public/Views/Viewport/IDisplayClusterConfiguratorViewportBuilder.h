// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Views/IDisplayClusterConfiguratorBuilder.h"


class IDisplayClusterConfiguratorViewportBuilder
	: public IDisplayClusterConfiguratorBuilder
{
public:
	virtual ~IDisplayClusterConfiguratorViewportBuilder() = default;

public:
	virtual void BuildViewport() = 0;

	virtual void ClearViewportSelection() = 0;
};
