// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class DisplayClusterDetails : ModuleRules
{
	public DisplayClusterDetails(ReadOnlyTargetRules ROTargetRules) : base(ROTargetRules)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"DisplayCluster",
				"DisplayClusterConfiguration",
				"DisplayClusterOperator",

				"ApplicationCore",
				"AppFramework",
				"ColorCorrectRegions",
				"ColorGradingEditor",
				"Core",
				"CoreUObject",
				"DetailCustomizations",
				"EditorStyle",
				"Engine",
				"InputCore",
				"Kismet",
				"PropertyEditor",
				"PropertyPath",
				"Slate",
				"SlateCore",
				"ToolWidgets",
				"UnrealEd",
				"WorkspaceMenuStructure",
			});
	}
}
