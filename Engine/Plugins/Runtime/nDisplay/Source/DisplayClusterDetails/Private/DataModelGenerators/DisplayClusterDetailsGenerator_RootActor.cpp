// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterDetailsGenerator_RootActor.h"

#include "DisplayClusterDetailsStyle.h"
#include "IDisplayClusterDetails.h"
#include "IDisplayClusterDetailsDrawerSingleton.h"

#include "DisplayClusterRootActor.h"
#include "Components/DisplayClusterICVFXCameraComponent.h"

#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterConfigurationTypes_ICVFX.h"
#include "DisplayClusterConfigurationTypes_Postprocess.h"

#include "Algo/Transform.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailCustomization.h"
#include "IDetailTreeNode.h"
#include "IPropertyRowGenerator.h"
#include "PropertyHandle.h"
#include "PropertyPathHelpers.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "DisplayClusterDetails"

#define GET_MEMBER_NAME_ARRAY_CHECKED(ClassName, MemberName, Index) FName(FString(GET_MEMBER_NAME_STRING_CHECKED(ClassName, MemberName)).Replace(TEXT(#Index), *FString::FromInt(Index), ESearchCase::CaseSensitive))

TSharedPtr<IDetailTreeNode> FDisplayClusterDetailsGenerator_Base::FindPropertyTreeNode(const TSharedRef<IDetailTreeNode>& Node, const FCachedPropertyPath& PropertyPath)
{
	if (Node->GetNodeType() == EDetailNodeType::Item)
	{
		if (Node->GetNodeName() == PropertyPath.GetLastSegment().GetName())
		{
			TSharedPtr<IPropertyHandle> FoundPropertyHandle = Node->CreatePropertyHandle();
			FString FoundPropertyPath = FoundPropertyHandle->GeneratePathToProperty();

			if (PropertyPath == FoundPropertyPath)
			{
				return Node;
			}
		}
		
		return nullptr;
	}
	else
	{
		TArray<TSharedRef<IDetailTreeNode>> Children;
		Node->GetChildren(Children);
		for (const TSharedRef<IDetailTreeNode>& Child : Children)
		{
			if (TSharedPtr<IDetailTreeNode> PropertyTreeNode = FindPropertyTreeNode(Child, PropertyPath))
			{
				return PropertyTreeNode;
			}
		}

		return nullptr;
	}
}

TSharedPtr<IPropertyHandle> FDisplayClusterDetailsGenerator_Base::FindPropertyHandle(IPropertyRowGenerator& PropertyRowGenerator, const FCachedPropertyPath& PropertyPath)
{
	const TArray<TSharedRef<IDetailTreeNode>>& RootNodes = PropertyRowGenerator.GetRootTreeNodes();

	for (const TSharedRef<IDetailTreeNode>& RootNode : RootNodes)
	{
		if (TSharedPtr<IDetailTreeNode> PropertyTreeNode = FindPropertyTreeNode(RootNode, PropertyPath))
		{
			return PropertyTreeNode->CreatePropertyHandle();
		}
	}

	return nullptr;
}

#define CREATE_PROPERTY_PATH(RootObjectClass, PropertyPath) FCachedPropertyPath(GET_MEMBER_NAME_STRING_CHECKED(RootObjectClass, PropertyPath))

TSharedPtr<IPropertyHandle> MakePropertyTransactional(const TSharedPtr<IPropertyHandle>& PropertyHandle)
{
	if (PropertyHandle.IsValid())
	{
		PropertyHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([PropertyHandle]
		{
			TArray<UObject*> OuterObjects;
			PropertyHandle->GetOuterObjects(OuterObjects);
			for (UObject* Object : OuterObjects)
			{
				if (!Object->HasAnyFlags(RF_Transactional))
				{
					Object->SetFlags(RF_Transactional);
				}

				SaveToTransactionBuffer(Object, false);
				SnapshotTransactionBuffer(Object);
			}
		}));
	}

	return PropertyHandle;
}

TSharedRef<IDisplayClusterDetailsDataModelGenerator> FDisplayClusterDetailsGenerator_RootActor::MakeInstance()
{
	return MakeShareable(new FDisplayClusterDetailsGenerator_RootActor());
}

/**
 * A detail customization that picks out only the necessary properties needed to display a root actor in the details drawer and hides all other properties
 * Also organizes the properties into custom categories that can be easily displayed in the details drawer
 */
class FRootActorDetailsCustomization : public IDetailCustomization
{
public:
	FRootActorDetailsCustomization(const TSharedRef<FDisplayClusterDetailsDataModel>& InDetailsDataModel)
		: DetailsDataModel(InDetailsDataModel)
	{ }

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override
	{
		auto AddProperty = [&DetailBuilder](IDetailCategoryBuilder& Category, FName PropertyName, bool bExpandChildProperties = false)
		{
			TSharedRef<IPropertyHandle> PropertyHandle = DetailBuilder.GetProperty(PropertyName, ADisplayClusterRootActor::StaticClass());

			if (bExpandChildProperties)
			{
				PropertyHandle->SetInstanceMetaData(TEXT("ShowOnlyInnerProperties"), TEXT("1"));
			}

			Category.AddProperty(PropertyHandle);
		};

		// Add root component transform properties to layout builder so that the details panel can find them when constructing TransformCommon
		const TArray<TWeakObjectPtr<UObject>>& SelectedObjects = DetailBuilder.GetSelectedObjects();
		{
			TArray<UObject*> RootComponents;
			Algo::Transform(SelectedObjects, RootComponents, [](TWeakObjectPtr<UObject> Obj)
			{
				USceneComponent* RootComponent = nullptr;
				if (ADisplayClusterRootActor* RootActor = Cast<ADisplayClusterRootActor>(Obj.Get()))
				{
					RootComponent = RootActor->GetRootComponent();
				}

				return RootComponent;
			});

			DetailBuilder.AddObjectPropertyData(RootComponents, USceneComponent::GetRelativeLocationPropertyName());
			DetailBuilder.AddObjectPropertyData(RootComponents, USceneComponent::GetRelativeRotationPropertyName());
			DetailBuilder.AddObjectPropertyData(RootComponents, USceneComponent::GetRelativeScale3DPropertyName());
		}
		
		IDetailCategoryBuilder& ViewportsCategoryBuilder = DetailBuilder.EditCategory(TEXT("CustomViewportsCategory"), LOCTEXT("CustomViewportsCategoryLabel", "Viewports"));
		AddProperty(ViewportsCategoryBuilder, TEXT("OuterViewportUpscalerSettingsRef"));
		AddProperty(ViewportsCategoryBuilder, TEXT("ViewportScreenPercentageMultiplierRef"));
		AddProperty(ViewportsCategoryBuilder, TEXT("FreezeRenderOuterViewportsRef"));

		IDetailCategoryBuilder& InnerFrustumCategoryBuilder = DetailBuilder.EditCategory(TEXT("CustomICVFXCategory"), LOCTEXT("CustomICVFXCategoryLabel", "In-Camera VFX"));
		AddProperty(InnerFrustumCategoryBuilder, TEXT("GlobalInnerFrustumUpscalerSettingsRef"));
		AddProperty(InnerFrustumCategoryBuilder, TEXT("ShowInnerFrustumOverlapsRef"));
		AddProperty(InnerFrustumCategoryBuilder, GET_MEMBER_NAME_CHECKED(ADisplayClusterRootActor, InnerFrustumPriority));


		IDetailCategoryBuilder& ViewportChromakeyCategoryBuilder = DetailBuilder.EditCategory(TEXT("CustomViewportChromakeyCategory"), LOCTEXT("CustomViewportChromakeyCategoryLabel", "Chromakey"));
		AddProperty(ViewportChromakeyCategoryBuilder, TEXT("GlobalChromakeyColorRef"));

		IDetailCategoryBuilder& ViewportChromakeyMarkersCategoryBuilder = DetailBuilder.EditCategory(TEXT("CustomViewportChromakeyMarkersCategory"), LOCTEXT("CustomViewportChromakeyMarkersCategoryLabel", "Chromakey Markers"));
		AddProperty(ViewportChromakeyMarkersCategoryBuilder, TEXT("GlobalChromakeyMarkersRef"), true);
	}

private:
	TWeakPtr<FDisplayClusterDetailsDataModel> DetailsDataModel;
};

void FDisplayClusterDetailsGenerator_RootActor::Initialize(const TSharedRef<class FDisplayClusterDetailsDataModel>& DetailsDataModel, const TSharedRef<IPropertyRowGenerator>& PropertyRowGenerator)
{
	PropertyRowGenerator->RegisterInstancedCustomPropertyLayout(ADisplayClusterRootActor::StaticClass(), FOnGetDetailCustomizationInstance::CreateLambda([DetailsDataModel]
	{
		return MakeShared<FRootActorDetailsCustomization>(DetailsDataModel);
	}));
}

void FDisplayClusterDetailsGenerator_RootActor::Destroy(const TSharedRef<class FDisplayClusterDetailsDataModel>& DetailsDataModel, const TSharedRef<IPropertyRowGenerator>& PropertyRowGenerator)
{
	PropertyRowGenerator->UnregisterInstancedCustomPropertyLayout(ADisplayClusterRootActor::StaticClass());
}

void FDisplayClusterDetailsGenerator_RootActor::GenerateDataModel(IPropertyRowGenerator& PropertyRowGenerator, FDisplayClusterDetailsDataModel& OutDetailsDataModel)
{
	RootActors.Empty();

	const TArray<TWeakObjectPtr<UObject>>& SelectedObjects = PropertyRowGenerator.GetSelectedObjects();
	for (const TWeakObjectPtr<UObject>& SelectedObject : SelectedObjects)
	{
		if (SelectedObject.IsValid() && SelectedObject->IsA<ADisplayClusterRootActor>())
		{
			TWeakObjectPtr<ADisplayClusterRootActor> SelectedRootActor = CastChecked<ADisplayClusterRootActor>(SelectedObject.Get());
			RootActors.Add(SelectedRootActor);
		}
	}

	{
		FDisplayClusterDetailsDataModel::FDetailsSection InnerFrustumDetailsSection;
		InnerFrustumDetailsSection.DisplayName = LOCTEXT("InnerFrustumDetailsSectionLabel", "Inner Frustum");
		InnerFrustumDetailsSection.Categories.Add(TEXT("CustomViewportsCategory"));
		InnerFrustumDetailsSection.Categories.Add(TEXT("CustomICVFXCategory"));
		InnerFrustumDetailsSection.EditConditionPropertyHandle = FindPropertyHandle(PropertyRowGenerator, CREATE_PROPERTY_PATH(UDisplayClusterConfigurationData, StageSettings.bEnableInnerFrustums));

		OutDetailsDataModel.DetailsSections.Add(InnerFrustumDetailsSection);
	}

	{
		FDisplayClusterDetailsDataModel::FDetailsSection TransformDetailsSection;
		TransformDetailsSection.Categories.Add(TEXT("TransformCommon"));

		OutDetailsDataModel.DetailsSections.Add(TransformDetailsSection);
	}

	{
		FDisplayClusterDetailsDataModel::FDetailsSection ChromakeyDetailsSection;
		ChromakeyDetailsSection.Categories.Add(TEXT("CustomViewportChromakeyCategory"));
		ChromakeyDetailsSection.Categories.Add(TEXT("CustomViewportChromakeyMarkersCategory"));

		OutDetailsDataModel.DetailsSections.Add(ChromakeyDetailsSection);
	}
}

TSharedRef<IDisplayClusterDetailsDataModelGenerator> FDisplayClusterDetailsGenerator_ICVFXCamera::MakeInstance()
{
	return MakeShareable(new FDisplayClusterDetailsGenerator_ICVFXCamera());
}

/**
 * A detail customization that picks out only the necessary properties needed to display a ICVFX camera component in the details drawer and hides all other properties
 * Also organizes the properties into custom categories that can be easily displayed in the details drawer
 */
class FICVFXCameraDetailsCustomization : public IDetailCustomization
{
public:
	FICVFXCameraDetailsCustomization(const TSharedRef<FDisplayClusterDetailsDataModel>& InDetailsDataModel)
		: DetailsDataModel(InDetailsDataModel)
	{ }

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override
	{
		auto AddProperty = [&DetailBuilder](IDetailCategoryBuilder& Category, FName PropertyName, bool bExpandChildProperties = false)
		{
			TSharedRef<IPropertyHandle> PropertyHandle = DetailBuilder.GetProperty(PropertyName, UDisplayClusterICVFXCameraComponent::StaticClass());

			if (bExpandChildProperties)
			{
				PropertyHandle->SetInstanceMetaData(TEXT("ShowOnlyInnerProperties"), TEXT("1"));
			}

			Category.AddProperty(PropertyHandle);
		};

		IDetailCategoryBuilder& ICVFXCategoryBuilder = DetailBuilder.EditCategory(TEXT("CustomICVFXCategory"), LOCTEXT("CustomICVFXCategoryLabel", "In-Camera VFX"));

		AddProperty(ICVFXCategoryBuilder, TEXT("UpscalerSettingsRef"));
		AddProperty(ICVFXCategoryBuilder, TEXT("BufferRatioRef"));

		AddProperty(ICVFXCategoryBuilder, TEXT("ExternalCameraActorRef"));
		AddProperty(ICVFXCategoryBuilder, TEXT("HiddenICVFXViewportsRef"));

		IDetailCategoryBuilder& SoftEdgeCategoryBuilder = DetailBuilder.EditCategory(TEXT("CustomSoftEdgeCategory"), LOCTEXT("CustomSoftEdgeCategoryLabel", "Soft Edge"));
		AddProperty(SoftEdgeCategoryBuilder, TEXT("SoftEdgeRef"), true);

		IDetailCategoryBuilder& BorderCategoryBuilder = DetailBuilder.EditCategory(TEXT("CustomBorderCategory"), LOCTEXT("CustomBorderCategoryLabel", "Border"));
		AddProperty(BorderCategoryBuilder, TEXT("BorderRef"), true);

		IDetailCategoryBuilder& OverscanCategoryBuilder = DetailBuilder.EditCategory(TEXT("CustomOverscanCategory"), LOCTEXT("CustomOverscanCategoryLabel", "Inner Frustum Overscan"));
		AddProperty(OverscanCategoryBuilder, TEXT("CustomFrustumRef"), true);

		IDetailCategoryBuilder& ChromakeyCategoryBuilder = DetailBuilder.EditCategory(TEXT("CustomChromakeyCategory"), LOCTEXT("CustomChromakeyCategoryLabel", "Chromakey"));
		AddProperty(ChromakeyCategoryBuilder, TEXT("ChromakeyTypeRef"));
		AddProperty(ChromakeyCategoryBuilder, TEXT("ChromakeySettingsSourceRef"));
		AddProperty(ChromakeyCategoryBuilder, TEXT("ChromakeyColorRef"));

		IDetailCategoryBuilder& ChromakeyMarkersCategoryBuilder = DetailBuilder.EditCategory(TEXT("CustomChromakeyMarkersCategory"), LOCTEXT("CustomChromakeyMarkersCategoryLabel", "ChromakeyMarkers"));
		AddProperty(ChromakeyMarkersCategoryBuilder, TEXT("ChromakeyMarkersRef"), true);
	}

private:
	TWeakPtr<FDisplayClusterDetailsDataModel> DetailsDataModel;
};

void FDisplayClusterDetailsGenerator_ICVFXCamera::Initialize(const TSharedRef<class FDisplayClusterDetailsDataModel>& DetailsDataModel, const TSharedRef<IPropertyRowGenerator>& PropertyRowGenerator)
{
	PropertyRowGenerator->RegisterInstancedCustomPropertyLayout(UDisplayClusterICVFXCameraComponent::StaticClass(), FOnGetDetailCustomizationInstance::CreateLambda([DetailsDataModel]
	{
		return MakeShared<FICVFXCameraDetailsCustomization>(DetailsDataModel);
	}));
}

void FDisplayClusterDetailsGenerator_ICVFXCamera::Destroy(const TSharedRef<class FDisplayClusterDetailsDataModel>& DetailsDataModel, const TSharedRef<IPropertyRowGenerator>& PropertyRowGenerator)
{
	PropertyRowGenerator->UnregisterInstancedCustomPropertyLayout(UDisplayClusterICVFXCameraComponent::StaticClass());
}

void FDisplayClusterDetailsGenerator_ICVFXCamera::GenerateDataModel(IPropertyRowGenerator& PropertyRowGenerator, FDisplayClusterDetailsDataModel& OutDetailsDataModel)
{
	CameraComponents.Empty();

	const TArray<TWeakObjectPtr<UObject>>& SelectedObjects = PropertyRowGenerator.GetSelectedObjects();
	for (const TWeakObjectPtr<UObject>& SelectedObject : SelectedObjects)
	{
		if (SelectedObject.IsValid() && SelectedObject->IsA<UDisplayClusterICVFXCameraComponent>())
		{
			TWeakObjectPtr<UDisplayClusterICVFXCameraComponent> SelectedCameraComponent = CastChecked<UDisplayClusterICVFXCameraComponent>(SelectedObject.Get());
			CameraComponents.Add(SelectedCameraComponent);
		}
	}

	{
		FDisplayClusterDetailsDataModel::FDetailsSection InnerFrustumDetailsSection;
		InnerFrustumDetailsSection.DisplayName = LOCTEXT("InnerFrustumDetailsSectionLabel", "Inner Frustum");
		InnerFrustumDetailsSection.EditConditionPropertyHandle = FindPropertyHandle(PropertyRowGenerator, CREATE_PROPERTY_PATH(UDisplayClusterICVFXCameraComponent, CameraSettings.bEnable));
		InnerFrustumDetailsSection.Categories = { TEXT("CustomICVFXCategory"), TEXT("CustomSoftEdgeCategory"), TEXT("CustomBorderCategory") };

		OutDetailsDataModel.DetailsSections.Add(InnerFrustumDetailsSection);
	}

	{
		FDisplayClusterDetailsDataModel::FDetailsSection InnerFrustumOverscanDetailsSection;
		InnerFrustumOverscanDetailsSection.DisplayName = LOCTEXT("InnerFrustumOverscanDetailsSectionLabel", "Inner Frustum Overscan");
		InnerFrustumOverscanDetailsSection.EditConditionPropertyHandle = FindPropertyHandle(PropertyRowGenerator, CREATE_PROPERTY_PATH(UDisplayClusterICVFXCameraComponent, CameraSettings.CustomFrustum.bEnable));
		InnerFrustumOverscanDetailsSection.Categories = { TEXT("CustomOverscanCategory") };

		OutDetailsDataModel.DetailsSections.Add(InnerFrustumOverscanDetailsSection);
	}

	{
		FDisplayClusterDetailsDataModel::FDetailsSection ChromakeyDetailsSection;
		ChromakeyDetailsSection.DisplayName = LOCTEXT("ChromakeyDetailsSectionLabel", "Chromakey");
		ChromakeyDetailsSection.EditConditionPropertyHandle = FindPropertyHandle(PropertyRowGenerator, CREATE_PROPERTY_PATH(UDisplayClusterICVFXCameraComponent, CameraSettings.Chromakey.bEnable));
		ChromakeyDetailsSection.Categories = { TEXT("CustomChromakeyCategory"), TEXT("CustomChromakeyMarkersCategory") };

		OutDetailsDataModel.DetailsSections.Add(ChromakeyDetailsSection);
	}
}

#undef LOCTEXT_NAMESPACE
