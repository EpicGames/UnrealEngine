// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DisplayClusterDetailsDataModel.h"

#include "Input/Reply.h"

class ADisplayClusterRootActor;
class IDetailTreeNode;
class IPropertyHandle;
class IPropertyRowGenerator;
class FDisplayClusterDetailsDataModel;
class UDisplayClusterICVFXCameraComponent;
struct FCachedPropertyPath;

/** Base generator containing helper functions for DisplayCluster data model generators */
class FDisplayClusterDetailsGenerator_Base : public IDisplayClusterDetailsDataModelGenerator
{
protected:
	/** Recursively searches the detail tree hierarchy for a property detail tree node whose name matches the specified name */
	TSharedPtr<IDetailTreeNode> FindPropertyTreeNode(const TSharedRef<IDetailTreeNode>& Node, const FCachedPropertyPath& PropertyPath);

	/** Finds a property handle in the specified property row generator whose name matches the specified name */
	TSharedPtr<IPropertyHandle> FindPropertyHandle(IPropertyRowGenerator& PropertyRowGenerator, const FCachedPropertyPath& PropertyPath);
};

/** Details data model generator for an nDisplay root actor */
class FDisplayClusterDetailsGenerator_RootActor : public FDisplayClusterDetailsGenerator_Base
{
public:
	static TSharedRef<IDisplayClusterDetailsDataModelGenerator> MakeInstance();

	//~ IDisplayClusterDetailsDataModelGenerator interface
	virtual void Initialize(const TSharedRef<class FDisplayClusterDetailsDataModel>& DetailsDataModel, const TSharedRef<IPropertyRowGenerator>& PropertyRowGenerator) override;
	virtual void Destroy(const TSharedRef<class FDisplayClusterDetailsDataModel>& DetailsDataModel, const TSharedRef<IPropertyRowGenerator>& PropertyRowGenerator) override;
	virtual void GenerateDataModel(IPropertyRowGenerator& PropertyRowGenerator, FDisplayClusterDetailsDataModel& OutDetailsDataModel) override;
	//~ End IDisplayClusterDetailsDataModelGenerator interface

private:
	/** A list of root actors that are being represented by the data model */
	TArray<TWeakObjectPtr<ADisplayClusterRootActor>> RootActors;
};

/** Details data model generator for an nDisplay ICVFX camera component */
class FDisplayClusterDetailsGenerator_ICVFXCamera : public FDisplayClusterDetailsGenerator_Base
{
public:
	static TSharedRef<IDisplayClusterDetailsDataModelGenerator> MakeInstance();

	//~ IDisplayClusterDetailsDataModelGenerator interface
	virtual void Initialize(const TSharedRef<class FDisplayClusterDetailsDataModel>& InDetailsDataModel, const TSharedRef<IPropertyRowGenerator>& InPropertyRowGenerator) override;
	virtual void Destroy(const TSharedRef<class FDisplayClusterDetailsDataModel>& InDetailsDataModel, const TSharedRef<IPropertyRowGenerator>& InPropertyRowGenerator) override;
	virtual void GenerateDataModel(IPropertyRowGenerator& PropertyRowGenerator, FDisplayClusterDetailsDataModel& OutDetailsDataModel) override;
	//~ End IDisplayClusterDetailsDataModelGenerator interface

private:
	/** A list of camera components that are being represented by the data model */
	TArray<TWeakObjectPtr<UDisplayClusterICVFXCameraComponent>> CameraComponents;
};