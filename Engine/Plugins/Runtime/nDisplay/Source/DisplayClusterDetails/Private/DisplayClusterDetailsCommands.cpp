// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterDetailsCommands.h"

#define LOCTEXT_NAMESPACE "DisplayClusterDetails"

void FDisplayClusterDetailsCommands::RegisterCommands()
{
	UI_COMMAND(OpenDetailsDrawer, "Open In-Camera VFX Drawer", "Opens the In-Camera VFX details drawer from the status bar", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::SpaceBar));
}

#undef LOCTEXT_NAMESPACE
