// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DisplayClusterDetailsStyle.h"
#include "Framework/Commands/Commands.h"
#include "Styling/AppStyle.h"

/** Command list for the details drawer */
class FDisplayClusterDetailsCommands
	: public TCommands<FDisplayClusterDetailsCommands>
{
public:
	FDisplayClusterDetailsCommands()
		: TCommands<FDisplayClusterDetailsCommands>(TEXT("DisplayClusterDetails"),
			NSLOCTEXT("Contexts", "DisplayClusterDetails", "Display Cluster Details"), NAME_None, FDisplayClusterDetailsStyle::Get().GetStyleSetName())
	{ }

	virtual void RegisterCommands() override;

public:
	TSharedPtr<FUICommandInfo> OpenDetailsDrawer;
};