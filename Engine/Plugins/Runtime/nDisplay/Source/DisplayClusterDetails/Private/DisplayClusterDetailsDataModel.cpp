// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterDetailsDataModel.h"

#include "Drawer/DisplayClusterDetailsDrawerState.h"

#include "DisplayClusterRootActor.h"

#include "DetailWidgetRow.h"
#include "IPropertyRowGenerator.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"


#define LOCTEXT_NAMESPACE "DisplayClusterDetails"

TMap<TWeakObjectPtr<UClass>, FGetDetailsDataModelGenerator> FDisplayClusterDetailsDataModel::RegisteredDataModelGenerators;

/** Detail customizer intended for color FVector4 properties that don't generate property nodes for the child components of the vector, to speed up property node tree generation */
class FFastColorStructCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FFastColorStructCustomization);
	}

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override
	{
		HeaderRow.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		];
	}

	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override
	{
	}
};

class FColorPropertyTypeIdentifier : public IPropertyTypeIdentifier
{
	virtual bool IsPropertyTypeCustomized(const IPropertyHandle& PropertyHandle) const override
	{
		return PropertyHandle.HasMetaData(TEXT("ColorGradingMode"));
	}
};

FDisplayClusterDetailsDataModel::FDisplayClusterDetailsDataModel()
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FPropertyRowGeneratorArgs Args;
	PropertyRowGenerator = PropertyEditorModule.CreatePropertyRowGenerator(Args);
	PropertyRowGenerator->OnRowsRefreshed().AddRaw(this, &FDisplayClusterDetailsDataModel::OnPropertyRowGeneratorRefreshed);
	
	TSharedRef<FColorPropertyTypeIdentifier> ColorPropertyTypeIdentifier = MakeShared<FColorPropertyTypeIdentifier>();

	// Since we don't display color grading controls at all, set a customizer for any color vectors to prevent the property row generator
	// from generating child properties or extraneous widgets, which drastically helps improve performance when loading object properties
	PropertyRowGenerator->RegisterInstancedCustomPropertyTypeLayout(NAME_Vector4, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FFastColorStructCustomization::MakeInstance), ColorPropertyTypeIdentifier);
	PropertyRowGenerator->RegisterInstancedCustomPropertyTypeLayout(NAME_Vector4f, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FFastColorStructCustomization::MakeInstance), ColorPropertyTypeIdentifier);
	PropertyRowGenerator->RegisterInstancedCustomPropertyTypeLayout(NAME_Vector4d, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FFastColorStructCustomization::MakeInstance), ColorPropertyTypeIdentifier);
}

TArray<TWeakObjectPtr<UObject>> FDisplayClusterDetailsDataModel::GetObjects() const
{
	if (PropertyRowGenerator.IsValid())
	{
		return PropertyRowGenerator->GetSelectedObjects();
	}

	return TArray<TWeakObjectPtr<UObject>>();
}

void FDisplayClusterDetailsDataModel::SetObjects(const TArray<UObject*>& InObjects)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDisplayClusterDetailsDataModel::SetObjects);

	// Only update the data model if the objects being set are new
	bool bUpdateDataModel = false;
	if (PropertyRowGenerator.IsValid())
	{
		const TArray<TWeakObjectPtr<UObject>>& CurrentObjects = PropertyRowGenerator->GetSelectedObjects();

		if (CurrentObjects.Num() != InObjects.Num())
		{
			bUpdateDataModel = true;
		}
		else
		{
			for (UObject* NewObject : InObjects)
			{
				if (!CurrentObjects.Contains(NewObject))
				{
					bUpdateDataModel = true;
					break;
				}
			}
		}
	}

	if (bUpdateDataModel)
	{
		Reset();

		for (const UObject* Object : InObjects)
		{
			if (Object)
			{
				InitializeDataModelGenerator(Object->GetClass());
			}
		}

		if (PropertyRowGenerator.IsValid())
		{
			PropertyRowGenerator->SetObjects(InObjects);
		}
	}
}

bool FDisplayClusterDetailsDataModel::HasObjectOfType(const UClass* InClass) const
{
	if (PropertyRowGenerator.IsValid())
	{
		const TArray<TWeakObjectPtr<UObject>> SelectedObjects = PropertyRowGenerator->GetSelectedObjects();

		for (const TWeakObjectPtr<UObject>& Object : SelectedObjects)
		{
			if (Object.IsValid() && Object->GetClass()->IsChildOf(InClass))
			{
				return true;
			}
		}
	}

	return false;
}

void FDisplayClusterDetailsDataModel::Reset()
{
	for (const TPair<TWeakObjectPtr<UClass>, TSharedPtr<IDisplayClusterDetailsDataModelGenerator>>& GeneratorInstance : DataModelGeneratorInstances)
	{
		GeneratorInstance.Value->Destroy(SharedThis(this), PropertyRowGenerator.ToSharedRef());
		PropertyRowGenerator->UnregisterInstancedCustomPropertyLayout(GeneratorInstance.Key.Get());
	}

	DataModelGeneratorInstances.Empty();
	DetailsSections.Empty();
}

void FDisplayClusterDetailsDataModel::GetDrawerState(FDisplayClusterDetailsDrawerState& OutDrawerState)
{
}

void FDisplayClusterDetailsDataModel::SetDrawerState(const FDisplayClusterDetailsDrawerState& InDrawerState)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDisplayClusterDetailsDataModel::SetDrawerState);

	TArray<UObject*> ObjectsToSelect;
	for (const TWeakObjectPtr<UObject>& Object : InDrawerState.SelectedObjects)
	{
		if (Object.IsValid())
		{
			ObjectsToSelect.Add(Object.Get());
		}
	}

	for (const UObject* Object : ObjectsToSelect)
	{
		if (Object)
		{
			InitializeDataModelGenerator(Object->GetClass());
		}
	}

	if (PropertyRowGenerator.IsValid())
	{
		PropertyRowGenerator->SetObjects(ObjectsToSelect);
	}
}

void FDisplayClusterDetailsDataModel::InitializeDataModelGenerator(UClass* InClass)
{
	UClass* CurrentClass = InClass;
	while (CurrentClass)
	{
		if (RegisteredDataModelGenerators.Contains(CurrentClass) && !DataModelGeneratorInstances.Contains(CurrentClass))
		{
			if (RegisteredDataModelGenerators[CurrentClass].IsBound())
			{
				TSharedRef<IDisplayClusterDetailsDataModelGenerator> Generator = RegisteredDataModelGenerators[CurrentClass].Execute();
				Generator->Initialize(SharedThis(this), PropertyRowGenerator.ToSharedRef());

				DataModelGeneratorInstances.Add(CurrentClass, Generator);
			}
		}

		CurrentClass = CurrentClass->GetSuperClass();
	}
}

TSharedPtr<IDisplayClusterDetailsDataModelGenerator> FDisplayClusterDetailsDataModel::GetDataModelGenerator(UClass* InClass) const
{
	UClass* CurrentClass = InClass;
	while (CurrentClass)
	{
		if (DataModelGeneratorInstances.Contains(CurrentClass))
		{
			return DataModelGeneratorInstances[CurrentClass];
		}

		CurrentClass = CurrentClass->GetSuperClass();
	}

	return nullptr;
}

void FDisplayClusterDetailsDataModel::OnPropertyRowGeneratorRefreshed()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDisplayClusterDetailsDataModel::OnPropertyRowGeneratorRefreshed);

	DetailsSections.Empty();

	const TArray<TWeakObjectPtr<UObject>>& SelectedObjects = PropertyRowGenerator->GetSelectedObjects();

	if (SelectedObjects.Num() == 1)
	{
		TWeakObjectPtr<UObject> SelectedObject = SelectedObjects[0];

		if (SelectedObject.IsValid())
		{
			if (TSharedPtr<IDisplayClusterDetailsDataModelGenerator> Generator = GetDataModelGenerator(SelectedObject->GetClass()))
			{
				Generator->GenerateDataModel(*PropertyRowGenerator, *this);
			}
		}
	}

	// TODO: Figure out what needs to be done to support multiple disparate types of objects being edited at the same time

	OnDataModelGeneratedDelegate.Broadcast();
}

#undef LOCTEXT_NAMESPACE