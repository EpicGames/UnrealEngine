// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PropertyEditorDelegates.h"

class IPropertyHandle;
class IPropertyRowGenerator;
class SWidget;
struct FDisplayClusterDetailsDrawerState;

/** Interface that allows detail data models to be generated for specific UObjects */
class IDisplayClusterDetailsDataModelGenerator : public TSharedFromThis<IDisplayClusterDetailsDataModelGenerator>
{
public:
	virtual ~IDisplayClusterDetailsDataModelGenerator() {}

	virtual void Initialize(const TSharedRef<class FDisplayClusterDetailsDataModel>& DetailsDataModel, const TSharedRef<IPropertyRowGenerator>& PropertyRowGenerator) = 0;
	virtual void Destroy(const TSharedRef<class FDisplayClusterDetailsDataModel>& DetailsDataModel, const TSharedRef<IPropertyRowGenerator>& PropertyRowGenerator) = 0;

	/** Called when the details data model should be generated by this generator */
	virtual void GenerateDataModel(IPropertyRowGenerator& PropertyRowGenerator, class FDisplayClusterDetailsDataModel& OutDetailsDataModel) = 0;
};

DECLARE_DELEGATE_RetVal(TSharedRef<IDisplayClusterDetailsDataModelGenerator>, FGetDetailsDataModelGenerator);

/**
 * A data model that stores the details properties from a UObject, which determines which properties will be displayed in the details panel.
 */
class FDisplayClusterDetailsDataModel : public TSharedFromThis<FDisplayClusterDetailsDataModel>
{
public:
	/** Registers a new data model generator used to populate a details data model for the specified class */
	template<class T>
	static void RegisterDetailsDataModelGenerator(FGetDetailsDataModelGenerator CreateGeneratorDelegate)
	{
		UClass* Class = T::StaticClass();
		RegisteredDataModelGenerators.Add(Class, CreateGeneratorDelegate);
	}

private:
	/** A list of data model generators, indexed by class name, that have been registered */
	static TMap<TWeakObjectPtr<UClass>, FGetDetailsDataModelGenerator> RegisteredDataModelGenerators;


public:
	/** Stores data used to generate a details subsection, which is a selectable element in a details section that modifies which properties are displayed in the section */
	struct FDetailsSubsection
	{
		/** The display name of the subsection */
		FText DisplayName;

		/** A delegate that creates the details customization for the details subsection */
		FOnGetDetailCustomizationInstance DetailCustomizationDelegate;

		/** A list of categories to display in the detail subsection */
		TArray<FName> Categories;
	};

	/** Stores data used to generate a details section in the drawer, which displays a specific subset of properties from the selected objects */
	struct FDetailsSection
	{
		/** The display name of the details section*/
		FText DisplayName = FText::GetEmpty();

		/** The handle of the property that controls the editing condition of the properties in the details group */
		TSharedPtr<IPropertyHandle> EditConditionPropertyHandle = nullptr;

		/** A list of selectable subsections to display in the details view */
		TArray<FDetailsSubsection> Subsections;

		/** A delegate that creates the details customization for the details section */
		FOnGetDetailCustomizationInstance DetailCustomizationDelegate;

		/** A list of categories to display in the detail subsection */
		TArray<FName> Categories;
	};

	DECLARE_MULTICAST_DELEGATE(FOnDataModelGenerated);

public:
	FDisplayClusterDetailsDataModel();

	/** Gets the property row generator that was used to generate this data model */
	TSharedRef<IPropertyRowGenerator> GetPropertyRowGenerator() const { return PropertyRowGenerator.ToSharedRef(); }

	/** Gets the list of objects from which the details properties were generated from */
	TArray<TWeakObjectPtr<UObject>> GetObjects() const;

	/** Sets the objects from which the details properties will be generated */
	void SetObjects(const TArray<UObject*>& InObjects);

	/** Gets whether the data model has objects of the specified class */
	bool HasObjectOfType(const UClass* InClass) const;

	/** Resets the data model to be empty */
	void Reset();

	/** Adds the state of the data model to the specified drawer state */
	void GetDrawerState(FDisplayClusterDetailsDrawerState& OutDrawerState);

	/** Sets the state of the data model from the specified drawer state */
	void SetDrawerState(const FDisplayClusterDetailsDrawerState& InDrawerState);

	/** Gets the delegate that is raised when the data model is generated */
	FOnDataModelGenerated& OnDataModelGenerated() { return OnDataModelGeneratedDelegate; }

private:
	/** Initializes any needed data model generators for the specified class */
	void InitializeDataModelGenerator(UClass* InClass);

	/** Attempts to find an instance of a data model generator that can be used for the specified class */
	TSharedPtr<IDisplayClusterDetailsDataModelGenerator> GetDataModelGenerator(UClass* InClass) const;

	/** Callback that is raised when the property row generator the data model is generated from has been refreshed */
	void OnPropertyRowGeneratorRefreshed();

public:
	/** A list of details sections in the details data model */
	TArray<FDetailsSection> DetailsSections;

private:
	/** A list of data model generators that have been created to generate the data model for the current objects */
	TMap<TWeakObjectPtr<UClass>, TSharedPtr<IDisplayClusterDetailsDataModelGenerator>> DataModelGeneratorInstances;

	/** The property row generator which manages the property handles of the details objects */
	TSharedPtr<IPropertyRowGenerator> PropertyRowGenerator;

	/** Delegate that is raised when the data model has been generated */
	FOnDataModelGenerated OnDataModelGeneratedDelegate;
};