// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterDetailsModule.h"

#include "DisplayClusterDetailsCommands.h"
#include "DataModelGenerators/DisplayClusterDetailsGenerator_RootActor.h"
#include "Drawer/DisplayClusterDetailsDrawerSingleton.h"
#include "Drawer/SDisplayClusterDetailsDrawer.h"

#include "DisplayClusterRootActor.h"
#include "Components/DisplayClusterICVFXCameraComponent.h"

#include "ColorCorrectRegion.h"
#include "Engine/PostProcessVolume.h"

#define LOCTEXT_NAMESPACE "DisplayClusterDetails"

void FDisplayClusterDetailsModule::StartupModule()
{
	DetailsDrawerSingleton = MakeUnique<FDisplayClusterDetailsDrawerSingleton>();

	FDisplayClusterDetailsDataModel::RegisterDetailsDataModelGenerator<ADisplayClusterRootActor>(
		FGetDetailsDataModelGenerator::CreateStatic(&FDisplayClusterDetailsGenerator_RootActor::MakeInstance));

	FDisplayClusterDetailsDataModel::RegisterDetailsDataModelGenerator<UDisplayClusterICVFXCameraComponent>(
		FGetDetailsDataModelGenerator::CreateStatic(&FDisplayClusterDetailsGenerator_ICVFXCamera::MakeInstance));

	FDisplayClusterDetailsCommands::Register();
}

void FDisplayClusterDetailsModule::ShutdownModule()
{
	DetailsDrawerSingleton.Reset();
}

IDisplayClusterDetailsDrawerSingleton& FDisplayClusterDetailsModule::GetDetailsDrawerSingleton() const
{
	return *DetailsDrawerSingleton.Get();
}

IMPLEMENT_MODULE(FDisplayClusterDetailsModule, DisplayClusterDetails);

#undef LOCTEXT_NAMESPACE
