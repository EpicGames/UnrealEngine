// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDisplayClusterDetails.h"

class FDisplayClusterDetailsDrawerSingleton;

/**
 * Module which adds the In-Camera VFX details drawer to the ICVFX panel
 */
class FDisplayClusterDetailsModule : public IDisplayClusterDetails
{
public:
	//~ IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface interface

	//~ IDisplayClusterDetails interface
	virtual IDisplayClusterDetailsDrawerSingleton& GetDetailsDrawerSingleton() const override;
	//~ End IDisplayClusterDetails interface

private:
	/** The details drawer singleton, which manages the details drawer widget */
	TUniquePtr<FDisplayClusterDetailsDrawerSingleton> DetailsDrawerSingleton;
};
