// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyle.h"
#include "Styling/ToolBarStyle.h"


/** Styleset for the nDisplay details UI elements */
class FDisplayClusterDetailsStyle final : public FSlateStyleSet
{
public:

	FDisplayClusterDetailsStyle()
		: FSlateStyleSet("DisplayClusterDetailsStyle")
	{
		const FVector2D Icon16x16(16.0f, 16.0f);

		SetParentStyleName(FAppStyle::GetAppStyleSetName());

		// Set miscellaneous icons
		SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Runtime/nDisplay/Content/Icons/"));
		SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));

		Set("DisplayClusterDetails.Icon", new IMAGE_BRUSH_SVG("Components/nDisplayCamera", Icon16x16));

		FSlateStyleRegistry::RegisterSlateStyle(*this);
	}

	virtual ~FDisplayClusterDetailsStyle()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	}

	static FDisplayClusterDetailsStyle& Get()
	{
		static FDisplayClusterDetailsStyle Inst;
		return Inst;
	}
};