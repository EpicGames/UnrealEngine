// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterDetailsDrawerSingleton.h"

#include "DisplayClusterDetailsCommands.h"
#include "DisplayClusterDetailsStyle.h"

#include "DisplayClusterRootActor.h"

#include "IDisplayClusterOperator.h"
#include "IDisplayClusterOperatorViewModel.h"
#include "DisplayClusterOperatorStatusBarExtender.h"
#include "Drawer/SDisplayClusterDetailsDrawer.h"

#include "Editor.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/LayoutExtender.h"
#include "Framework/Docking/TabManager.h"
#include "IPropertyRowGenerator.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "DisplayClusterDetails"

const FName FDisplayClusterDetailsDrawerSingleton::DetailsDrawerId = TEXT("DisplayClusterDetailsDrawer");
const FName FDisplayClusterDetailsDrawerSingleton::DetailsDrawerTab = TEXT("DisplayClusterDetailsDrawerTab");

FDisplayClusterDetailsDrawerSingleton::FDisplayClusterDetailsDrawerSingleton()
{
	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(DetailsDrawerTab, FOnSpawnTab::CreateRaw(this, &FDisplayClusterDetailsDrawerSingleton::SpawnDetailsDrawerTab))
		.SetIcon(FSlateIcon(FDisplayClusterDetailsStyle::Get().GetStyleSetName(), "DisplayClusterDetails.Icon"))
		.SetDisplayName(LOCTEXT("DisplayClusterDetailsDrawerTab_DisplayName", "In-Camera VFX"))
		.SetTooltipText(LOCTEXT("DisplayClusterDetailsDrawerTab_Tooltip", "Editing tools for in-camera VFX."))
		.SetMenuType(ETabSpawnerMenuType::Hidden);

	IDisplayClusterOperator::Get().OnRegisterLayoutExtensions().AddRaw(this, &FDisplayClusterDetailsDrawerSingleton::ExtendOperatorTabLayout);
	IDisplayClusterOperator::Get().OnRegisterStatusBarExtensions().AddRaw(this, &FDisplayClusterDetailsDrawerSingleton::ExtendOperatorStatusBar);
	IDisplayClusterOperator::Get().OnAppendOperatorPanelCommands().AddRaw(this, &FDisplayClusterDetailsDrawerSingleton::AppendOperatorPanelCommands);

	IDisplayClusterOperator::Get().GetOperatorViewModel()->OnActiveRootActorChanged().AddRaw(this, &FDisplayClusterDetailsDrawerSingleton::OnActiveRootActorChanged);
	IDisplayClusterOperator::Get().GetOperatorViewModel()->OnDetailObjectsChanged().AddRaw(this, &FDisplayClusterDetailsDrawerSingleton::OnDetailObjectsChanged);
}

FDisplayClusterDetailsDrawerSingleton::~FDisplayClusterDetailsDrawerSingleton()
{
	IDisplayClusterOperator::Get().OnRegisterLayoutExtensions().RemoveAll(this);
	IDisplayClusterOperator::Get().OnRegisterStatusBarExtensions().RemoveAll(this);
	IDisplayClusterOperator::Get().OnAppendOperatorPanelCommands().RemoveAll(this);
	IDisplayClusterOperator::Get().GetOperatorViewModel()->OnActiveRootActorChanged().RemoveAll(this);
	IDisplayClusterOperator::Get().GetOperatorViewModel()->OnDetailObjectsChanged().RemoveAll(this);

	if (FSlateApplication::IsInitialized())
	{
		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(DetailsDrawerTab);
	}
}

void FDisplayClusterDetailsDrawerSingleton::DockDetailsDrawer()
{
	if (TSharedPtr<FTabManager> OperatorPanelTabManager = IDisplayClusterOperator::Get().GetOperatorViewModel()->GetTabManager())
	{
		if (TSharedPtr<SDockTab> ExistingTab = OperatorPanelTabManager->FindExistingLiveTab(DetailsDrawerTab))
		{
			IDisplayClusterOperator::Get().ForceDismissDrawers();
			ExistingTab->ActivateInParent(ETabActivationCause::SetDirectly);
		}
		else
		{
			OperatorPanelTabManager->TryInvokeTab(DetailsDrawerTab);
		}
	}
}

void FDisplayClusterDetailsDrawerSingleton::RefreshDetailsDrawers(bool bPreserveDrawerState)
{
	if (DetailsDrawer.IsValid())
	{
		DetailsDrawer.Pin()->Refresh(bPreserveDrawerState);
	}

	if (TSharedPtr<FTabManager> OperatorPanelTabManager = IDisplayClusterOperator::Get().GetOperatorViewModel()->GetTabManager())
	{
		if (TSharedPtr<SDockTab> ExistingTab = OperatorPanelTabManager->FindExistingLiveTab(DetailsDrawerTab))
		{
			TSharedRef<SDisplayClusterDetailsDrawer> DockedDrawer = StaticCastSharedRef<SDisplayClusterDetailsDrawer>(ExistingTab->GetContent());
			DockedDrawer->Refresh(bPreserveDrawerState);
		}
	}
}

TSharedRef<SWidget> FDisplayClusterDetailsDrawerSingleton::CreateDrawerContent(bool bIsInDrawer, bool bCopyStateFromActiveDrawer)
{
	if (bIsInDrawer)
	{
		TSharedPtr<SDisplayClusterDetailsDrawer> Drawer = DetailsDrawer.IsValid() ? DetailsDrawer.Pin() : nullptr;

		if (!Drawer.IsValid())
		{
			Drawer = SNew(SDisplayClusterDetailsDrawer, true);
			DetailsDrawer = Drawer;
		}

		if (PreviousDrawerState.IsSet())
		{
			DetailsDrawer.Pin()->SetDrawerState(PreviousDrawerState.GetValue());
			PreviousDrawerState.Reset();
		}
		else
		{
			DetailsDrawer.Pin()->SetDrawerStateToDefault();
		}

		return Drawer.ToSharedRef();
	}
	else
	{
		TSharedRef<SDisplayClusterDetailsDrawer> NewDrawer = SNew(SDisplayClusterDetailsDrawer, false);

		if (bCopyStateFromActiveDrawer && DetailsDrawer.IsValid())
		{
			NewDrawer->SetDrawerState(DetailsDrawer.Pin()->GetDrawerState());
		}

		return NewDrawer;
	}
}

TSharedRef<SDockTab> FDisplayClusterDetailsDrawerSingleton::SpawnDetailsDrawerTab(const FSpawnTabArgs& SpawnTabArgs)
{
	const TSharedRef<SDockTab> MajorTab = SNew(SDockTab)
		.TabRole(ETabRole::NomadTab);

	MajorTab->SetContent(CreateDrawerContent(false, true));

	return MajorTab;
}

void FDisplayClusterDetailsDrawerSingleton::ExtendOperatorTabLayout(FLayoutExtender& InExtender)
{
	FTabManager::FTab NewTab(FTabId(DetailsDrawerTab, ETabIdFlags::SaveLayout), ETabState::ClosedTab);
	InExtender.ExtendStack(IDisplayClusterOperator::Get().GetAuxilliaryOperatorExtensionId(), ELayoutExtensionPosition::After, NewTab);
}

void FDisplayClusterDetailsDrawerSingleton::ExtendOperatorStatusBar(FDisplayClusterOperatorStatusBarExtender& StatusBarExtender)
{
	FWidgetDrawerConfig DetailsDrawerConfig(DetailsDrawerId);

	DetailsDrawerConfig.GetDrawerContentDelegate.BindRaw(this, &FDisplayClusterDetailsDrawerSingleton::CreateDrawerContent, true, false);
	DetailsDrawerConfig.OnDrawerDismissedDelegate.BindRaw(this, &FDisplayClusterDetailsDrawerSingleton::SaveDrawerState);
	DetailsDrawerConfig.ButtonText = LOCTEXT("DisplayClusterDetailsDrawer_ButtonText", "In-Camera VFX");
	DetailsDrawerConfig.Icon = FDisplayClusterDetailsStyle::Get().GetBrush("DisplayClusterDetails.Icon");

	StatusBarExtender.AddWidgetDrawer(DetailsDrawerConfig);
}

void FDisplayClusterDetailsDrawerSingleton::AppendOperatorPanelCommands(TSharedRef<FUICommandList> OperatorPanelCommandList)
{
	OperatorPanelCommandList->MapAction(
		FDisplayClusterDetailsCommands::Get().OpenDetailsDrawer,
		FExecuteAction::CreateRaw(this, &FDisplayClusterDetailsDrawerSingleton::OpenDetailsDrawer)
	);
}

void FDisplayClusterDetailsDrawerSingleton::OpenDetailsDrawer()
{
	IDisplayClusterOperator::Get().ToggleDrawer(DetailsDrawerId);
}

void FDisplayClusterDetailsDrawerSingleton::SaveDrawerState(const TSharedPtr<SWidget>& DrawerContent)
{
	if (DetailsDrawer.IsValid())
	{
		PreviousDrawerState = DetailsDrawer.Pin()->GetDrawerState();
	}
	else
	{
		PreviousDrawerState.Reset();
	}
}

void FDisplayClusterDetailsDrawerSingleton::OnActiveRootActorChanged(ADisplayClusterRootActor* NewRootActor)
{
	// Clear the previous drawer state when the active root actor is changed, since it is most likely invalid
	PreviousDrawerState.Reset();
}

void FDisplayClusterDetailsDrawerSingleton::OnDetailObjectsChanged(const TArray<UObject*>& NewObjects)
{
	// Clear the previous drawer state when the selected detail objects have changed
	PreviousDrawerState.Reset();
}

#undef LOCTEXT_NAMESPACE