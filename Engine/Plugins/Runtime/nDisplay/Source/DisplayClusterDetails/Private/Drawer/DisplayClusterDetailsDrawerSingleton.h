// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDisplayClusterDetailsDrawerSingleton.h"
#include "Drawer/SDisplayClusterDetailsDrawer.h"

class FDisplayClusterOperatorStatusBarExtender;
class FLayoutExtender;
class FSpawnTabArgs;
class FUICommandList;
class SWidget;
class SDockTab;

/** A singleton used to manage and store persistent state for the details drawer */
class FDisplayClusterDetailsDrawerSingleton : public IDisplayClusterDetailsDrawerSingleton
{
public:
	/** The ID of the details drawer when registered with the nDisplay operator panel's status bar */
	static const FName DetailsDrawerId;

	/** The ID of the details drawer when docked in the nDisplay operator panel's tab manager */
	static const FName DetailsDrawerTab;

public:
	FDisplayClusterDetailsDrawerSingleton();
	virtual ~FDisplayClusterDetailsDrawerSingleton();

	/** Docks the details drawer in the nDisplay operator window */
	virtual void DockDetailsDrawer() override;

	/** Refreshes the UI of any open details drawers */
	virtual void RefreshDetailsDrawers(bool bPreserveDrawerState) override;

private:
	/** Creates a new drawer widget to place in a drawer or in a tab */
	TSharedRef<SWidget> CreateDrawerContent(bool bIsInDrawer, bool bCopyStateFromActiveDrawer);

	/** Tab spawn delegate handler used to create the drawer tab when the drawer is docked in the operator panel */
	TSharedRef<SDockTab> SpawnDetailsDrawerTab(const FSpawnTabArgs& SpawnTabArgs);

	/** Tab extender delegate callback that registers the tab spawner with the operator panel's tab manager */
	void ExtendOperatorTabLayout(FLayoutExtender& InExtender);

	/** Status bar extender delegate callback that registers the drawer spawner with the operator panel's status bar */
	void ExtendOperatorStatusBar(FDisplayClusterOperatorStatusBarExtender& StatusBarExtender);

	/** Delegate callback that appends the operator panel command list to add details drawer commands */
	void AppendOperatorPanelCommands(TSharedRef<FUICommandList> OperatorPanelCommandList);

	/** Opens the details drawer */
	void OpenDetailsDrawer();

	/** Delegate callback when the drawer is closed to save the drawer state */
	void SaveDrawerState(const TSharedPtr<SWidget>& DrawerContent);

	/** Delegate callback that is raised when the active root actor of the operator panel has changed */
	void OnActiveRootActorChanged(ADisplayClusterRootActor* NewRootActor);

	/** Delegate callback that is raised when the list of objects displayed in the operator panel's details panel has changed */
	void OnDetailObjectsChanged(const TArray<UObject*>& NewObjects);

private:
	/** A weak pointer to the active details drawer that is open */
	TWeakPtr<SDisplayClusterDetailsDrawer> DetailsDrawer;

	/** The drawer state when the last instance of the details drawer was dismissed */
	TOptional<FDisplayClusterDetailsDrawerState> PreviousDrawerState;
};