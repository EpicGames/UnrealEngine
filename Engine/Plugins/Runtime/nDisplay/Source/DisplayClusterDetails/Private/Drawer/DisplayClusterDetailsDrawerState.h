// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateTypes.h"

/** Stores the state of the drawer UI that can be reloaded in cases where the drawer or any of its elements are reloaded (such as when the drawer is reopened or docked) */
struct FDisplayClusterDetailsDrawerState
{
	/** The objects that are selected in the list */
	TArray<TWeakObjectPtr<UObject>> SelectedObjects;

	/** Indicates which subsections were selected for each section in the details panel */
	TArray<int32> SelectedDetailsSubsections;
};