// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDisplayClusterDetailsDrawer.h"

#include "DisplayClusterDetailsDataModel.h"
#include "DisplayClusterDetailsStyle.h"
#include "Engine/Blueprint.h"
#include "IDisplayClusterDetails.h"
#include "IDisplayClusterDetailsDrawerSingleton.h"
#include "SDisplayClusterDetailsObjectList.h"
#include "SDisplayClusterDetailsPanel.h"

#include "IDisplayClusterOperator.h"
#include "IDisplayClusterOperatorViewModel.h"

#include "DisplayClusterRootActor.h"
#include "Components/DisplayClusterICVFXCameraComponent.h"
#include "DisplayClusterConfigurationTypes.h"

#include "ColorCorrectRegion.h"
#include "ColorCorrectWindow.h"
#include "DisplayClusterDetailsCommands.h"
#include "Engine/PostProcessVolume.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Modules/ModuleManager.h"
#include "ScopedTransaction.h"
#include "Framework/Commands/GenericCommands.h"
#include "Styling/AppStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

#define LOCTEXT_NAMESPACE "DisplayClusterDetails"

SDisplayClusterDetailsDrawer::~SDisplayClusterDetailsDrawer()
{
	OperatorViewModel = IDisplayClusterOperator::Get().GetOperatorViewModel();
	OperatorViewModel->OnActiveRootActorChanged().RemoveAll(this);

	FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);
	GEngine->OnLevelActorAdded().RemoveAll(this);
	GEngine->OnLevelActorDeleted().RemoveAll(this);

	GEditor->UnregisterForUndo(this);

	for (const FDisplayClusterDetailsListItemRef& DetailsItem : ObjectItemList)
	{
		if (DetailsItem->Component.IsValid())
		{
			UnbindBlueprintCompiledDelegate(DetailsItem->Component->GetClass());
		}

		if (DetailsItem->Actor.IsValid())
		{
			UnbindBlueprintCompiledDelegate(DetailsItem->Actor->GetClass());
		}
	}
}

void SDisplayClusterDetailsDrawer::Construct(const FArguments& InArgs, bool bInIsInDrawer)
{
	DetailsDataModel = MakeShared<FDisplayClusterDetailsDataModel>();
	DetailsDataModel->OnDataModelGenerated().AddSP(this, &SDisplayClusterDetailsDrawer::OnDetailsDataModelGenerated);

	bIsInDrawer = bInIsInDrawer;
	OperatorViewModel = IDisplayClusterOperator::Get().GetOperatorViewModel();
	OperatorViewModel->OnActiveRootActorChanged().AddSP(this, &SDisplayClusterDetailsDrawer::OnActiveRootActorChanged);

	FCoreUObjectDelegates::OnObjectsReplaced.AddSP(this, &SDisplayClusterDetailsDrawer::OnObjectsReplaced);
	GEngine->OnLevelActorAdded().AddSP(this, &SDisplayClusterDetailsDrawer::OnLevelActorAdded);
	GEngine->OnLevelActorDeleted().AddSP(this, &SDisplayClusterDetailsDrawer::OnLevelActorDeleted);

	GEditor->RegisterForUndo(this);

	RefreshObjectList();

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.Padding(FMargin(0.0f, 0.0f))
		[
			// Splitter to divide the object list and the color panel
			SNew(SSplitter)
			.Orientation(Orient_Horizontal)
			.PhysicalSplitterHandleSize(2.0f)

			+SSplitter::Slot()
			.Value(0.12f)
			[
				SNew(SBox)
				.Padding(FMargin(4.f))
				[
					SNew(SBorder)
					.Padding(FMargin(0.0f))
					.BorderImage(FAppStyle::GetBrush("Brushes.Recessed"))
					[
						SNew(SScrollBox)
						+ SScrollBox::Slot()
						[
							SNew(SExpandableArea)
							.BorderImage(FAppStyle::Get().GetBrush("Brushes.Header"))
							.BodyBorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
							.HeaderPadding(FMargin(4.0f, 2.0f))
							.InitiallyCollapsed(false)
							.AllowAnimatedTransition(false)
							.Visibility_Lambda([this]() { return ObjectItemList.Num() ? EVisibility::Visible : EVisibility::Collapsed; })
							.HeaderContent()
							[
								SNew(SBox)
								.HeightOverride(24.0f)
								.VAlign(VAlign_Center)
								[
									SNew(STextBlock)
									.Text(LOCTEXT("DisplayClusterDetailsObjectListLabel", "Objects"))
									.TextStyle(FAppStyle::Get(), "ButtonText")
									.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
								]
							]
							.BodyContent()
							[
								SAssignNew(ObjectListView, SDisplayClusterDetailsObjectList)
								.DetailsItemsSource(&ObjectItemList)
								.OnSelectionChanged(this, &SDisplayClusterDetailsDrawer::OnListSelectionChanged)
							]
						]
					]
				]
			]

			+SSplitter::Slot()
			.Value(0.88f)
			[
				SNew(SVerticalBox)

				// Toolbar slot for the main drawer toolbar
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0, 0, 0, 0)
				[
					SNew(SBorder)
					.Padding(FMargin(3))
					.BorderImage(bIsInDrawer ? FStyleDefaults::GetNoBrush() : FAppStyle::Get().GetBrush("Brushes.Panel"))
					[
						SNew(SBox)
						.HeightOverride(28.0f)
						.HAlign(HAlign_Right)
						.VAlign(VAlign_Center)
						[
							CreateDockInLayoutButton()
						]
					]
				]

				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SSeparator)
					.Thickness(2.0f)
				]

				// Slot for the details views
				+SVerticalBox::Slot()
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
					.Padding(FMargin(2.0f, 2.0f, 2.0f, 0.0f))
					[
						SAssignNew(DetailsPanel, SDisplayClusterDetailsPanel)
						.DetailsDataModelSource(DetailsDataModel)
					]
				]
			]
		]
	];
}

void SDisplayClusterDetailsDrawer::Refresh(bool bPreserveDrawerState)
{
	FDisplayClusterDetailsDrawerState DrawerState = GetDrawerState();

	DetailsDataModel->Reset();

	RefreshObjectList();

	if (DetailsPanel.IsValid())
	{
		DetailsPanel->Refresh();
	}

	if (bPreserveDrawerState)
	{
		SetDrawerState(DrawerState);
	}
	else
	{
		SetDrawerStateToDefault();
	}
}

void SDisplayClusterDetailsDrawer::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	if (bRefreshOnNextTick)
	{
		const bool bPreserveDrawerState = true;
		Refresh(bPreserveDrawerState);

		bRefreshOnNextTick = false;
	}
}

void SDisplayClusterDetailsDrawer::PostUndo(bool bSuccess)
{
	if (bSuccess)
	{
		const bool bPreserveDrawerState = true;
		Refresh(bPreserveDrawerState);
	}
}

void SDisplayClusterDetailsDrawer::PostRedo(bool bSuccess)
{
	if (bSuccess)
	{
		const bool bPreserveDrawerState = true;
		Refresh(bPreserveDrawerState);
	}
}

FDisplayClusterDetailsDrawerState SDisplayClusterDetailsDrawer::GetDrawerState() const
{
	FDisplayClusterDetailsDrawerState DrawerState;

	DetailsDataModel->GetDrawerState(DrawerState);

	if (DetailsPanel.IsValid())
	{
		DetailsPanel->GetDrawerState(DrawerState);
	}

	if (ObjectListView.IsValid())
	{
		TArray<FDisplayClusterDetailsListItemRef> SelectedItems = ObjectListView->GetSelectedItems();

		for (const FDisplayClusterDetailsListItemRef& SelectedItem : SelectedItems)
		{
			if (SelectedItem.IsValid())
			{
				if (SelectedItem->Component.IsValid())
				{
					DrawerState.SelectedObjects.Add(SelectedItem->Component);
				}
				else if (SelectedItem->Actor.IsValid())
				{
					DrawerState.SelectedObjects.Add(SelectedItem->Actor);
				}
			}
		}
	}

	return DrawerState;
}

void SDisplayClusterDetailsDrawer::SetDrawerState(const FDisplayClusterDetailsDrawerState& InDrawerState)
{
	TArray<FDisplayClusterDetailsListItemRef> ItemsToSelect;

	for (const TWeakObjectPtr<UObject>& SelectedObject : InDrawerState.SelectedObjects)
	{
		if (SelectedObject.IsValid())
		{
			auto FindDetailsItem = [&SelectedObject](const FDisplayClusterDetailsListItemRef& DetailsItem)
			{
				return DetailsItem->Actor == SelectedObject || DetailsItem->Component == SelectedObject;
			};

			if (FDisplayClusterDetailsListItemRef* FoundItem = ObjectItemList.FindByPredicate(FindDetailsItem))
			{
				ItemsToSelect.Add(*FoundItem);
				break;
			}
		}
	}

	ObjectListView->SetSelectedItems(ItemsToSelect);
	DetailsDataModel->SetDrawerState(InDrawerState);

	if (DetailsPanel.IsValid())
	{
		DetailsPanel->SetDrawerState(InDrawerState);
	}
}

void SDisplayClusterDetailsDrawer::SetDrawerStateToDefault()
{
	for (FDisplayClusterDetailsListItemRef& ListItem : ObjectItemList)
	{
		if (ListItem.IsValid() && ListItem->Actor.IsValid() && ListItem->Actor->IsA<ADisplayClusterRootActor>())
		{
			// The nDisplay stage actor is always the first item in the root actor details items list, so set that as the currently selected item
			ObjectListView->SetSelectedItems({ ListItem });
			SetDetailsDataModelObjects({ ListItem->Actor.Get() });
			break;
		}
	}
}

TSharedRef<SWidget> SDisplayClusterDetailsDrawer::CreateDockInLayoutButton()
{
	if (bIsInDrawer)
	{
		return SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.ToolTipText(LOCTEXT("DockInLayout_Tooltip", "Docks this panel in the current operator window, copying all settings from the drawer.\nThe drawer will still be usable."))
			.OnClicked(this, &SDisplayClusterDetailsDrawer::DockInLayout)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(4.0, 0.0f)
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FAppStyle::Get().GetBrush("Icons.Layout"))
				]

				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(4.0, 0.0f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("DockInLayout", "Dock in Layout"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			];
	}
	
	return SNullWidget::NullWidget;
}

void SDisplayClusterDetailsDrawer::BindBlueprintCompiledDelegate(const UClass* Class)
{
	if (UBlueprint* Blueprint = UBlueprint::GetBlueprintFromClass(Class))
	{
		if (!Blueprint->OnCompiled().IsBoundToObject(this))
		{
			Blueprint->OnCompiled().AddSP(this, &SDisplayClusterDetailsDrawer::OnBlueprintCompiled);
		}
	}
}

void SDisplayClusterDetailsDrawer::UnbindBlueprintCompiledDelegate(const UClass* Class)
{
	if (UBlueprint* Blueprint = UBlueprint::GetBlueprintFromClass(Class))
	{
		Blueprint->OnCompiled().RemoveAll(this);
	}
}

void SDisplayClusterDetailsDrawer::RefreshObjectList()
{
	for (const FDisplayClusterDetailsListItemRef& Item : ObjectItemList)
	{
		if (Item->Component.IsValid())
		{
			UnbindBlueprintCompiledDelegate(Item->Component->GetClass());
		}

		if (Item->Actor.IsValid())
		{
			UnbindBlueprintCompiledDelegate(Item->Actor->GetClass());
		}
	}

	ObjectItemList.Empty();

	if (ADisplayClusterRootActor* RootActor = OperatorViewModel->GetRootActor())
	{
		BindBlueprintCompiledDelegate(RootActor->GetClass());

		FDisplayClusterDetailsListItemRef RootActorListItemRef = MakeShared<FDisplayClusterDetailsListItem>(RootActor);
		ObjectItemList.Add(RootActorListItemRef);

		auto AlphabeticalSort = [](const FDisplayClusterDetailsListItemRef& A, const FDisplayClusterDetailsListItemRef& B)
		{
			if (A.IsValid() && B.IsValid())
			{
				return *A < *B;
			}
			else
			{
				return false;
			}
		};

		// Add any ICVFX camera component the root actor has to the details list
		{
			TArray<FDisplayClusterDetailsListItemRef> SortedICVFXCameras;
			RootActor->ForEachComponent<UDisplayClusterICVFXCameraComponent>(false, [this, RootActor, &SortedICVFXCameras](UDisplayClusterICVFXCameraComponent* ICVFXCameraComponent)
			{
				BindBlueprintCompiledDelegate(ICVFXCameraComponent->GetClass());

				FDisplayClusterDetailsListItemRef ICVFXCameraListItemRef = MakeShared<FDisplayClusterDetailsListItem>(RootActor, ICVFXCameraComponent);
				SortedICVFXCameras.Add(ICVFXCameraListItemRef);
			});

			SortedICVFXCameras.Sort(AlphabeticalSort);
			ObjectItemList.Append(SortedICVFXCameras);
		}
	}

	if (ObjectListView.IsValid())
	{
		ObjectListView->RefreshList();
	}
}

void SDisplayClusterDetailsDrawer::SetDetailsDataModelObjects(const TArray<UObject*>& Objects)
{
	DetailsDataModel->SetObjects(Objects);
}

void SDisplayClusterDetailsDrawer::OnObjectsReplaced(const TMap<UObject*, UObject*>& OldToNewInstanceMap)
{
	bool bNeedsFullRefresh = false;
	bool bNeedsListRefresh = false;

	const TArray<TWeakObjectPtr<UObject>> SelectedObjects = DetailsDataModel->GetObjects();

	for (const TPair<UObject*, UObject*>& Pair : OldToNewInstanceMap)
	{
		if (Pair.Key && Pair.Value)
		{
			FDisplayClusterDetailsListItemRef* FoundDetailsItemPtr = nullptr;

			// Must use GetEvenIfUnreachable on the weak pointers here because most of the time, the objects being replaced have already been marked for GC, and TWeakObjectPtr
			// will return nullptr from Get on GC-marked objects
			FoundDetailsItemPtr = ObjectItemList.FindByPredicate([&Pair](const FDisplayClusterDetailsListItemRef& DetailsItem)
			{
				return DetailsItem->Actor.GetEvenIfUnreachable() == Pair.Key || DetailsItem->Component.GetEvenIfUnreachable() == Pair.Key;
			});

			if (FoundDetailsItemPtr)
			{
				FDisplayClusterDetailsListItemRef FoundDetailsItem = *FoundDetailsItemPtr;
				if (FoundDetailsItem->Actor.GetEvenIfUnreachable() == Pair.Key)
				{
					FoundDetailsItem->Actor = Cast<AActor>(Pair.Value);
				}
				else if (FoundDetailsItem->Component.GetEvenIfUnreachable() == Pair.Key)
				{
					FoundDetailsItem->Component = Cast<UActorComponent>(Pair.Value);
				}

				bNeedsListRefresh = true;
			}

			if (SelectedObjects.Contains(Pair.Key))
			{
				bNeedsFullRefresh = true;
			}
		}
	}

	if (bNeedsFullRefresh)
	{
		// Wait until the next tick so that we aren't undercutting any details customizations that may want to do logic after invoking an object reconstruction
		bRefreshOnNextTick = true;
	}
	else if (bNeedsListRefresh && ObjectListView)
	{
		ObjectListView->RefreshList();
	}
}

void SDisplayClusterDetailsDrawer::OnLevelActorAdded(AActor* Actor)
{
	// Only refresh when the actor being added is being added to the root actor's world and is of a type this drawer cares about
	if (OperatorViewModel->HasRootActor())
	{
		if (UWorld* World = OperatorViewModel->GetRootActor()->GetWorld())
		{
			if (World == Actor->GetWorld())
			{
				if (Actor->IsA<ADisplayClusterRootActor>() || Actor->IsA<APostProcessVolume>() || Actor->IsA<AColorCorrectRegion>())
				{
					// Wait to refresh, as this event can be fired off for several actors in a row in certain cases, such as when the root actor is recompiled after a property change
					bRefreshOnNextTick = true;
				}
			}
		}
	}
}

void SDisplayClusterDetailsDrawer::OnLevelActorDeleted(AActor* Actor)
{
	auto ContainsActorRef = [Actor](const FDisplayClusterDetailsListItemRef& DetailsItem)
	{
		return DetailsItem->Actor.GetEvenIfUnreachable() == Actor;
	};

	if (ObjectItemList.ContainsByPredicate(ContainsActorRef))
	{
		// Must wait for next tick to refresh because the actor has not actually been removed from the level at this point
		bRefreshOnNextTick = true;
	}
}

void SDisplayClusterDetailsDrawer::OnBlueprintCompiled(UBlueprint* Blueprint)
{
	const bool bPreserveDrawerState = true;
	Refresh(bPreserveDrawerState);
}

void SDisplayClusterDetailsDrawer::OnActiveRootActorChanged(ADisplayClusterRootActor* NewRootActor)
{
	const bool bPreserveDrawerState = false;
	Refresh(bPreserveDrawerState);
}

void SDisplayClusterDetailsDrawer::OnDetailsDataModelGenerated()
{
	if (DetailsPanel.IsValid())
	{
		DetailsPanel->Refresh();
	}
}

void SDisplayClusterDetailsDrawer::OnListSelectionChanged(TSharedRef<SDisplayClusterDetailsObjectList> SourceList, FDisplayClusterDetailsListItemRef SelectedItem, ESelectInfo::Type SelectInfo)
{
	if (SelectInfo != ESelectInfo::Direct)
	{
		TArray<FDisplayClusterDetailsListItemRef> SelectedObjects = SourceList->GetSelectedItems();
		TArray<UObject*> ObjectsToColorGrade;
		for (const FDisplayClusterDetailsListItemRef& SelectedObject : SelectedObjects)
		{
			if (SelectedObject->Component.IsValid())
			{
				ObjectsToColorGrade.Add(SelectedObject->Component.Get());
			}
			else if (SelectedObject->Actor.IsValid())
			{
				ObjectsToColorGrade.Add(SelectedObject->Actor.Get());
			}
		}

		SetDetailsDataModelObjects(ObjectsToColorGrade);
	}
}

FReply SDisplayClusterDetailsDrawer::DockInLayout()
{
	IDisplayClusterDetails::Get().GetDetailsDrawerSingleton().DockDetailsDrawer();
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE