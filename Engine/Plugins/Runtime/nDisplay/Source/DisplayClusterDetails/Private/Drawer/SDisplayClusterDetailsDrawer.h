// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorUndoClient.h"
#include "Styling/SlateTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#include "DisplayClusterDetailsDrawerState.h"
#include "SDisplayClusterDetailsObjectList.h"

class SInlineEditableTextBlock;
class ADisplayClusterRootActor;
class FDisplayClusterOperatorStatusBarExtender;
class FDisplayClusterDetailsDataModel;
class IDisplayClusterOperatorViewModel;
class IPropertyRowGenerator;
class SHorizontalBox;
class SDisplayClusterDetailsPanel;

/** Details drawer widget, which displays a list of color gradable items, and the color wheel panel */
class SDisplayClusterDetailsDrawer : public SCompoundWidget, public FEditorUndoClient
{
public:
	~SDisplayClusterDetailsDrawer();

	SLATE_BEGIN_ARGS(SDisplayClusterDetailsDrawer)
	{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, bool bInIsInDrawer);

	/** Refreshes the drawer's UI to match the current state of the level and active root actor, optionally preserving UI state */
	void Refresh(bool bPreserveDrawerState = false);

	//~ SWidget interface
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	//~ End SWidget interface

	//~ FEditorUndoClient interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
	//~ End FEditorUndoClient interface

	/** Gets the state of the drawer UI */
	FDisplayClusterDetailsDrawerState GetDrawerState() const;

	/** Sets the state of the drawer UI */
	void SetDrawerState(const FDisplayClusterDetailsDrawerState& InDrawerState);

	/** Sets the state of the drawer UI to its default value, which is to have the nDisplay stage actor selected */
	void SetDrawerStateToDefault();

private:
	/** Creates the button used to dock the drawer in the operator panel */
	TSharedRef<SWidget> CreateDockInLayoutButton();

	/** Binds a callback to the BlueprintCompiled delegate of the specified class */
	void BindBlueprintCompiledDelegate(const UClass* Class);

	/** Unbinds a callback to the BlueprintCompiled delegate of the specified class */
	void UnbindBlueprintCompiledDelegate(const UClass* Class);

	/** Refreshes the object list, filling it with the current editable objects from the root actor and world  */
	void RefreshObjectList();

	/** Updates the details data model with the specified list of objects */
	void SetDetailsDataModelObjects(const TArray<UObject*>& Objects);

	/** Raised when the editor replaces any UObjects with new instantiations, usually when actors have been recompiled from blueprints */
	void OnObjectsReplaced(const TMap<UObject*, UObject*>& OldToNewInstanceMap);

	/** Raised when an actor is added to the current level */
	void OnLevelActorAdded(AActor* Actor);

	/** Raised when an actor has been deleted from the currnent level */
	void OnLevelActorDeleted(AActor* Actor);

	/** Raised when the specified blueprint has been recompiled */
	void OnBlueprintCompiled(UBlueprint* Blueprint);

	/** Raised when the user has changed the active root actor selected in the nDisplay operator panel */
	void OnActiveRootActorChanged(ADisplayClusterRootActor* NewRootActor);

	/** Raised when the details data model has been generated */
	void OnDetailsDataModelGenerated();

	/** Raised when the user has selected a new item in any of the drawer's list views */
	void OnListSelectionChanged(TSharedRef<SDisplayClusterDetailsObjectList> SourceList, FDisplayClusterDetailsListItemRef SelectedItem, ESelectInfo::Type SelectInfo);

	/** Raised when the "Dock in Layout" button has been clicked */
	FReply DockInLayout();

private:
	/** The operator panel's view model */
	TSharedPtr<IDisplayClusterOperatorViewModel> OperatorViewModel;

	/** List view of editable objects being displayed in the drawer's list panel */
	TSharedPtr<SDisplayClusterDetailsObjectList> ObjectListView;

	/** Source list for the details object widget */
	TArray<FDisplayClusterDetailsListItemRef> ObjectItemList;

	/** Panel containing selected object's details */
	TSharedPtr<SDisplayClusterDetailsPanel> DetailsPanel;

	/** The data model for the currently selected objects */
	TSharedPtr<FDisplayClusterDetailsDataModel> DetailsDataModel;

	/** Gets whether this widget is in a drawer or docked in a tab */
	bool bIsInDrawer = false;

	/** Indicates that the drawer should refresh itself on the next tick */
	bool bRefreshOnNextTick = false;

	/** Indicates if the details data model should update when a list item selection has changed */
	bool bUpdateDataModelOnSelectionChanged = true;

	/** Delegate handle for the OnActiveRootActorChanged delegate */
	FDelegateHandle ActiveRootActorChangedHandle;
};
