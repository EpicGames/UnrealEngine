// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDisplayClusterDetailsObjectList.h"

#include "ClassIconFinder.h"
#include "Components/ActorComponent.h"
#include "GameFramework/Actor.h"
#include "Styling/AppStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

#define LOCTEXT_NAMESPACE "DisplayClusterDetails"

bool FDisplayClusterDetailsListItem::operator<(const FDisplayClusterDetailsListItem& Other) const
{
	FString ThisName = TEXT("");
	FString OtherName = TEXT("");

	if (Component.IsValid())
	{
		ThisName = Component->GetName();
	}
	else if (Actor.IsValid())
	{
		ThisName = Actor->GetActorLabel();
	}

	if (Other.Component.IsValid())
	{
		OtherName = Other.Component->GetName();
	}
	else if (Other.Actor.IsValid())
	{
		OtherName = Other.Actor->GetActorLabel();
	}

	return ThisName < OtherName;
}

namespace DisplayClusterDetailsObjectListColumnNames
{
	const static FName ItemEnabled(TEXT("ItemEnabled"));
	const static FName ItemLabel(TEXT("ItemLabel"));
};

class SDetailsListItemRow : public SMultiColumnTableRow<FDisplayClusterDetailsListItemRef>
{
public:
	SLATE_BEGIN_ARGS(SDetailsListItemRow) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable, const FDisplayClusterDetailsListItemRef InListItem)
	{
		ListItem = InListItem;

		SMultiColumnTableRow<FDisplayClusterDetailsListItemRef>::Construct(
			SMultiColumnTableRow<FDisplayClusterDetailsListItemRef>::FArguments()
			.Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("SceneOutliner.TableViewRow"))
			, InOwnerTable);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		if (ColumnName == DisplayClusterDetailsObjectListColumnNames::ItemLabel)
		{
			return SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(FMargin(6, 1))
				.VAlign(VAlign_Center)
				[
					SNew(SBox)
					.WidthOverride(16)
					.HeightOverride(16)
					[
						SNew(SImage)
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Image(this, &SDetailsListItemRow::GetItemIcon)
					]
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Left)
				[
					SNew(STextBlock)
					.Text(this, &SDetailsListItemRow::GetItemLabel)
				];
		}

		return SNullWidget::NullWidget;
	}

private:
	FText GetItemLabel() const
	{
		FString ItemName = TEXT("");

		if (ListItem.IsValid())
		{
			if (ListItem->Component.IsValid())
			{
				ItemName = ListItem->Component->GetName();
			}
			else if (ListItem->Actor.IsValid())
			{
				ItemName = ListItem->Actor->GetActorLabel();
			}
		}

		return FText::FromString(*ItemName);
	}

	const FSlateBrush* GetItemIcon() const
	{
		if (ListItem.IsValid())
		{
			if (ListItem->Component.IsValid())
			{
				return FSlateIconFinder::FindIconBrushForClass(ListItem->Component->GetClass(), TEXT("SCS.Component"));
			}
			else if (ListItem->Actor.IsValid())
			{
				return FClassIconFinder::FindIconForActor(ListItem->Actor);
			}
		}

		return nullptr;
	}

private:
	FDisplayClusterDetailsListItemRef ListItem;
};

void SDisplayClusterDetailsObjectList::Construct(const FArguments& InArgs)
{
	OnSelectionChangedDelegate = InArgs._OnSelectionChanged;

	RefreshList();

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SAssignNew(ListView, SListView<FDisplayClusterDetailsListItemRef>)
			.ListItemsSource(InArgs._DetailsItemsSource)
			.SelectionMode(ESelectionMode::Single)
			.OnGenerateRow(this, &SDisplayClusterDetailsObjectList::GenerateListItemRow)
			.OnSelectionChanged(this, &SDisplayClusterDetailsObjectList::OnSelectionChanged)
			.HeaderRow
			(
				SNew(SHeaderRow)
				.Visibility(EVisibility::Collapsed)

				+ SHeaderRow::Column(DisplayClusterDetailsObjectListColumnNames::ItemLabel)
				.FillWidth(1.0f)
			)
		]
	];
}

void SDisplayClusterDetailsObjectList::RefreshList()
{
	if (ListView.IsValid())
	{
		ListView->RebuildList();
	}
}

TArray<FDisplayClusterDetailsListItemRef> SDisplayClusterDetailsObjectList::GetSelectedItems()
{
	return ListView->GetSelectedItems();
}

void SDisplayClusterDetailsObjectList::SetSelectedItems(const TArray<FDisplayClusterDetailsListItemRef>& InSelectedItems)
{
	ListView->ClearSelection();
	ListView->SetItemSelection(InSelectedItems, true);
}

TSharedRef<ITableRow> SDisplayClusterDetailsObjectList::GenerateListItemRow(FDisplayClusterDetailsListItemRef Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SDetailsListItemRow, OwnerTable, Item);
}

void SDisplayClusterDetailsObjectList::OnSelectionChanged(FDisplayClusterDetailsListItemRef SelectedItem, ESelectInfo::Type SelectInfo)
{
	OnSelectionChangedDelegate.ExecuteIfBound(SharedThis<SDisplayClusterDetailsObjectList>(this), SelectedItem, SelectInfo);
}

#undef LOCTEXT_NAMESPACE