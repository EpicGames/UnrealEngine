// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/ActorComponent.h"
#include "Framework/SlateDelegates.h"
#include "GameFramework/Actor.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class ITableRow;
class STableViewBase;

template<class T>
class SListView;

/** A structure to store references to color gradable actors and components */
struct FDisplayClusterDetailsListItem
{
	/** The actor that is color gradable */
	TWeakObjectPtr<AActor> Actor;

	/** The component that is color gradable */
	TWeakObjectPtr<UActorComponent> Component;

	FDisplayClusterDetailsListItem(AActor* InActor, UActorComponent* InComponent = nullptr)
		: Actor(InActor)
		, Component(InComponent)
	{ }

	/** Less than operator overload that compares list items alphabetically by their display names */
	bool operator<(const FDisplayClusterDetailsListItem& Other) const;
};

typedef TSharedPtr<FDisplayClusterDetailsListItem> FDisplayClusterDetailsListItemRef;

/** Displays a list of color gradable items */
class SDisplayClusterDetailsObjectList : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_ThreeParams(FOnSelectionChanged, TSharedRef<SDisplayClusterDetailsObjectList>, FDisplayClusterDetailsListItemRef, ESelectInfo::Type);

public:
	SLATE_BEGIN_ARGS(SDisplayClusterDetailsObjectList) {}
		SLATE_ARGUMENT(const TArray<FDisplayClusterDetailsListItemRef>*, DetailsItemsSource)
		SLATE_EVENT(FOnSelectionChanged, OnSelectionChanged)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	/** Refreshes the list, updating the UI to reflect the current state of the source items list*/
	void RefreshList();

	/** Gets a list of currently selected items */
	TArray<FDisplayClusterDetailsListItemRef> GetSelectedItems();

	/** Selects the specified list of items */
	void SetSelectedItems(const TArray<FDisplayClusterDetailsListItemRef>& InSelectedItems);

private:
	/** Generates the table row widget for the specified list item */
	TSharedRef<ITableRow> GenerateListItemRow(FDisplayClusterDetailsListItemRef Item, const TSharedRef<STableViewBase>& OwnerTable);

	/** Raised when the internal list view's selection has changed */
	void OnSelectionChanged(FDisplayClusterDetailsListItemRef SelectedItem, ESelectInfo::Type SelectInfo);

private:
	/** Internal list view used to display the list of color gradable items */
	TSharedPtr<SListView<FDisplayClusterDetailsListItemRef>> ListView;

	/** A delegate that is raised when the list of selected items is changed */
	FOnSelectionChanged OnSelectionChangedDelegate;
};