// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDisplayClusterDetailsPanel.h"

#include "DisplayClusterDetailsDataModel.h"
#include "Drawer/DisplayClusterDetailsDrawerState.h"
#include "DetailView/SColorGradingDetailView.h"

#include "IDetailTreeNode.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Colors/SColorPicker.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Text/STextBlock.h"

/** A widget that wraps a details view used to display the properties specified by a details section in the details data model */
class SDetailsSectionView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDetailsSectionView) { }
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<IPropertyRowGenerator> PropertyRowGenerator)
	{
		ChildSlot
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2, 4)
			[
				SAssignNew(HeaderBox, SBox)
			]

			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			.Padding(2, 0)
			[
				SAssignNew(DetailView, SColorGradingDetailView)
				.PropertyRowGeneratorSource(PropertyRowGenerator)
				.OnFilterDetailTreeNode(this, &SDetailsSectionView::FilterDetailTreeNode)
			]
		];
	}

	/** Fills the details view with the specified objects, and configures it used the specified details section */
	void FillDetailsSection(const TArray<TWeakObjectPtr<UObject>>& Objects, const FDisplayClusterDetailsDataModel::FDetailsSection& InDetailsSection)
	{
		DetailsSection = &InDetailsSection;

		// If a details subsection isn't already selected and the details section has subsections, set the selected subsection to the first subsection
		if (SelectedSubsectionIndex == INDEX_NONE && DetailsSection->Subsections.Num())
		{
			SelectedSubsectionIndex = 0;
		}

		HeaderBox->SetContent(CreateHeader());

		if (DetailView.IsValid())
		{
			DetailView->Refresh();
		}
	}

	/** Clears the details view and header */
	void EmptyDetailsSection()
	{
		DetailsSection = nullptr;
		SelectedSubsectionIndex = INDEX_NONE;

		if (HeaderBox.IsValid())
		{
			HeaderBox->SetContent(SNullWidget::NullWidget);
		}

		if (DetailView.IsValid())
		{
			DetailView->Refresh();
		}
	}

	/** Gets the selected subsection index */
	int32 GetSelectedSubsectionIndex() const
	{
		return SelectedSubsectionIndex;
	}

	/** Sets the selected subsection index*/
	void SetSelectedSubsectionIndex(int32 InSubsectionIndex)
	{
		if (DetailsSection)
		{
			SelectedSubsectionIndex = InSubsectionIndex;
			DetailView->Refresh();
		}
	}

private:
	TSharedRef<SWidget> CreateHeader()
	{
		if (DetailsSection)
		{
			// If there is nothing to display in the title bar, don't display one
			if (DetailsSection->DisplayName.IsEmpty() && !DetailsSection->EditConditionPropertyHandle.IsValid() && DetailsSection->Subsections.Num() == 0)
			{
				return SNullWidget::NullWidget;
			}

			TSharedRef<SHorizontalBox> HorizontalBox = SNew(SHorizontalBox);

			if (DetailsSection->EditConditionPropertyHandle.IsValid() && DetailsSection->EditConditionPropertyHandle->IsValidHandle())
			{
				HorizontalBox->AddSlot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(2, 0, 0, 0)
				[
					DetailsSection->EditConditionPropertyHandle->CreatePropertyValueWidget(false)
				];
			}

			HorizontalBox->AddSlot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			.Padding(6, 0, 0, 0)
			[
				SNew(STextBlock)
				.Text(DetailsSection->DisplayName)
				.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
			];

			if (DetailsSection->Subsections.Num())
			{
				HorizontalBox->AddSlot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					CreateSubsectionToolBar()
				];
			}

			return SNew(SBox)
				.HeightOverride(18)
				[
					HorizontalBox
				];
		}

		return SNullWidget::NullWidget;
	}

	TSharedRef<SWidget> CreateSubsectionToolBar()
	{
		if (DetailsSection && DetailsSection->Subsections.Num())
		{
			TSharedRef<SHorizontalBox> ToolBarBox = SNew(SHorizontalBox);

			for (int32 Index = 0; Index < DetailsSection->Subsections.Num(); ++Index)
			{
				ToolBarBox->AddSlot()
				.AutoWidth()
				.Padding(2.0f, 0.0f, 2.0f, 0.0f)
				[
					SNew(SCheckBox)
					.Style(FAppStyle::Get(), "DetailsView.SectionButton")
					.IsChecked(this, &SDetailsSectionView::IsSubsectionSelected, Index)
					.OnCheckStateChanged(this, &SDetailsSectionView::OnSelectedSubsectionCheckedChanged, Index)
					[
						SNew(STextBlock)
						.TextStyle(FAppStyle::Get(), "SmallText")
						.Text(DetailsSection->Subsections[Index].DisplayName)
					]
				];
			}

			return ToolBarBox;
		}

		return SNullWidget::NullWidget;
	}

	ECheckBoxState IsSubsectionSelected(int32 SubsectionIndex) const
	{
		return SelectedSubsectionIndex == SubsectionIndex ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	void OnSelectedSubsectionCheckedChanged(ECheckBoxState CheckBoxState, int32 SubsectionIndex)
	{
		if (CheckBoxState == ECheckBoxState::Checked)
		{
			SetSelectedSubsectionIndex(SubsectionIndex);
		}
	}

	/** Filters the detail tree nodes to display for the details section */
	bool FilterDetailTreeNode(const TSharedRef<IDetailTreeNode>& InDetailTreeNode)
	{
		if (DetailsSection)
		{
			// Filter out any categories that are not configured by the data model to be displayed in the details section or subsection.
			// All other nodes (which will be any child of the category), should be displayed.
			if (InDetailTreeNode->GetNodeType() == EDetailNodeType::Category)
			{
				if (DetailsSection->Subsections.Num() && SelectedSubsectionIndex > INDEX_NONE)
				{
					return DetailsSection->Subsections[SelectedSubsectionIndex].Categories.Contains(InDetailTreeNode->GetNodeName());
				}
				else
				{
					return DetailsSection->Categories.Contains(InDetailTreeNode->GetNodeName());
				}
			}
			else
			{
				return true;
			}
		}

		return false;
	}

private:
	TSharedPtr<SBox> HeaderBox;
	TSharedPtr<SColorGradingDetailView> DetailView;

	/** Pointer to the details section this view represents */
	const FDisplayClusterDetailsDataModel::FDetailsSection* DetailsSection = nullptr;

	/** The index of the currently selected details subsection */
	int32 SelectedSubsectionIndex = INDEX_NONE;
};

void SDisplayClusterDetailsPanel::Construct(const FArguments& InArgs)
{
	DetailsDataModel = InArgs._DetailsDataModelSource;

    ChildSlot
	[
		SAssignNew(Splitter, SSplitter)
		.PhysicalSplitterHandleSize(2.0f)
	];

	DetailsSectionViews.AddDefaulted(MaxNumDetailsSections);
	for (int32 Index = 0; Index < MaxNumDetailsSections; ++Index)
	{
		Splitter->AddSlot()
		[
			SAssignNew(DetailsSectionViews[Index], SDetailsSectionView, DetailsDataModel->GetPropertyRowGenerator())
			.Visibility(this, &SDisplayClusterDetailsPanel::GetDetailsSectionVisibility, Index)
		];
	}
}

void SDisplayClusterDetailsPanel::Refresh()
{
	FillDetailsSections();
}

void SDisplayClusterDetailsPanel::GetDrawerState(FDisplayClusterDetailsDrawerState& OutDrawerState)
{
	for (int32 Index = 0; Index < DetailsSectionViews.Num(); ++Index)
	{
		OutDrawerState.SelectedDetailsSubsections.Add(DetailsSectionViews[Index]->GetSelectedSubsectionIndex());
	}
}

void SDisplayClusterDetailsPanel::SetDrawerState(const FDisplayClusterDetailsDrawerState& InDrawerState)
{
	if (DetailsDataModel.IsValid())
	{
		if (InDrawerState.SelectedDetailsSubsections.Num() == DetailsDataModel->DetailsSections.Num())
		{
			const int32 NumDetailsSections = FMath::Min(DetailsDataModel->DetailsSections.Num(), MaxNumDetailsSections);
			for (int32 Index = 0; Index < NumDetailsSections; ++Index)
			{
				DetailsSectionViews[Index]->SetSelectedSubsectionIndex(InDrawerState.SelectedDetailsSubsections[Index]);
			}
		}
	}
}

void SDisplayClusterDetailsPanel::FillDetailsSections()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SDisplayClusterDetailsPanel::FillDetailsSections);
	
	TSharedPtr<SColorPicker> ExistingColorPicker = GetColorPicker();
	if (ExistingColorPicker.IsValid())
	{
		// Color picker has special handling -- FColorStructCustomization relies on listening to SColorPicker window closed
		// events in order to fully commit changes and close out an open transaction. If we destroy the details section views
		// before the window close event fires we will leave an open transaction and this will cause problems with undo history
		// and transacting changes to MU.
	
		// Similar handling is done in SDetailsView. On PreSetObject, GetOptionalOwningDetailsView is checked and the picker
		// closed if they do not own the picker, and then closed on PostSetObject if they do own it. Our implementation doesn't rely
		// on PreSet/PostSet, and there is never a case where we own the color picker, so we should always close it.
		
		DestroyColorPicker();
		ExistingColorPicker.Reset();
	}
	
	if (DetailsDataModel.IsValid())
	{
		TArray<TWeakObjectPtr<UObject>> Objects = DetailsDataModel->GetObjects();

		for (int32 Index = 0; Index < MaxNumDetailsSections; ++Index)
		{
			if (Index < DetailsDataModel->DetailsSections.Num())
			{
				const FDisplayClusterDetailsDataModel::FDetailsSection& DetailsSection = DetailsDataModel->DetailsSections[Index];

				DetailsSectionViews[Index]->FillDetailsSection(Objects, DetailsSection);
			}
			else
			{
				DetailsSectionViews[Index]->EmptyDetailsSection();
			}
		}
	}
}

EVisibility SDisplayClusterDetailsPanel::GetDetailsSectionVisibility(int32 SectionIndex) const
{
	if (DetailsDataModel.IsValid())
	{
		const int32 NumSections = FMath::Min(DetailsDataModel->DetailsSections.Num(), MaxNumDetailsSections);
		return SectionIndex < NumSections ? EVisibility::Visible : EVisibility::Collapsed;
	}

	return EVisibility::Collapsed;
}