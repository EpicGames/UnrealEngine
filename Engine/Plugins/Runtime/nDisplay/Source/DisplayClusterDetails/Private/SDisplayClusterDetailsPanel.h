// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FDisplayClusterDetailsDataModel;
class SDetailsSectionView;
class SSplitter;
struct FDisplayClusterDetailsDrawerState;

/** A panel that displays several property details views based on the details data model */
class SDisplayClusterDetailsPanel : public SCompoundWidget
{
public:
	/** The maximum number of details sections that are allowed to be displayed at the same time */
	static const int32 MaxNumDetailsSections = 3;

public:
	SLATE_BEGIN_ARGS(SDisplayClusterDetailsPanel) {}
		SLATE_ARGUMENT(TSharedPtr<FDisplayClusterDetailsDataModel>, DetailsDataModelSource)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	/** Refreshes the details panel to reflect the current state of the details data model */
	void Refresh();

	/** Adds the state of the details panel to the specified drawer state */
	void GetDrawerState(FDisplayClusterDetailsDrawerState& OutDrawerState);

	/** Sets the state of the details panel from the specified drawer state */
	void SetDrawerState(const FDisplayClusterDetailsDrawerState& InDrawerState);

private:
	/** Fills the details sections based on the current state of the details data model */
	void FillDetailsSections();

	/** Gets the visibility state of the specified details section */
	EVisibility GetDetailsSectionVisibility(int32 SectionIndex) const;

private:
	/** The details data model that the panel is displaying */
	TSharedPtr<FDisplayClusterDetailsDataModel> DetailsDataModel;

	TArray<TSharedPtr<SDetailsSectionView>> DetailsSectionViews;
	TSharedPtr<SSplitter> Splitter;
};