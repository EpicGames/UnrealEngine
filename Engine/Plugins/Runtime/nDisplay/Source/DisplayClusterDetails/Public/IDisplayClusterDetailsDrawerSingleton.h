// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class IPropertyRowGenerator;

/** A singleton used to manage and store persistent state for the details drawer */
class IDisplayClusterDetailsDrawerSingleton
{
public:
	/** Docks the details drawer in the nDisplay operator window */
	virtual void DockDetailsDrawer() = 0;

	/** Refreshes the UI of any open details drawers */
	virtual void RefreshDetailsDrawers(bool bPreserveDrawerState) = 0;
};