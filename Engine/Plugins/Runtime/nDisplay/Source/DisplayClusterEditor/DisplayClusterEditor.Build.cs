// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class DisplayClusterEditor : ModuleRules
{
	public DisplayClusterEditor(ReadOnlyTargetRules ROTargetRules) : base(ROTargetRules)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"DisplayCluster",
				"DisplayClusterConfiguration",
				"Engine",
				"EngineSettings",
				"EnhancedInput",
				"UnrealEd"
			});
	}
}
