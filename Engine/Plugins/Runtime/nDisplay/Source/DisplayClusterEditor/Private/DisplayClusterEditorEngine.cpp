// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterEditorEngine.h"
#include "DisplayClusterEditorLog.h"

#include "DisplayClusterRootActor.h"

#include "DisplayClusterConfigurationTypes.h"
#include "IDisplayClusterConfiguration.h"

#include "IPDisplayCluster.h"

#include "Engine/LevelStreaming.h"

#include "Editor.h"


void UDisplayClusterEditorEngine::Init(IEngineLoop* InEngineLoop)
{
	UE_LOGF(LogDisplayClusterEditorEngine, VeryVerbose, "UDisplayClusterEditorEngine::Init");

	// Initialize DisplayCluster module for editor mode
	DisplayClusterModule = static_cast<IPDisplayCluster*>(&IDisplayCluster::Get());
	if (!DisplayClusterModule)
	{
		UE_LOGF(LogDisplayClusterEditorEngine, Error, "Couldn't initialize DisplayCluster module");
		return;
	}

	// Initialize DisplayCluster module for operating in Editor mode
	const bool bResult = DisplayClusterModule->Init(EDisplayClusterOperationMode::Editor);
	if (!bResult)
	{
		UE_LOGF(LogDisplayClusterEditorEngine, Error, "An error occured during DisplayCluster initialization");
		return;
	}

	UE_LOGF(LogDisplayClusterEditorEngine, Log, "DisplayCluster module has been initialized");

	// Subscribe to PIE events
	BeginPIEDelegate = FEditorDelegates::BeginPIE.AddUObject(this, &UDisplayClusterEditorEngine::OnBeginPIE);
	EndPIEDelegate   = FEditorDelegates::EndPIE.AddUObject(this, &UDisplayClusterEditorEngine::OnEndPIE);

	return Super::Init(InEngineLoop);
}

void UDisplayClusterEditorEngine::PreExit()
{
	UE_LOGF(LogDisplayClusterEditorEngine, VeryVerbose, "UDisplayClusterEditorEngine::PreExit");

	Super::PreExit();
}

ADisplayClusterRootActor* UDisplayClusterEditorEngine::FindDisplayClusterRootActor(UWorld* InWorld)
{
	if (InWorld && InWorld->PersistentLevel)
	{
		for (AActor* const Actor : InWorld->PersistentLevel->Actors)
		{
			if (IsValid(Actor))
			{
				ADisplayClusterRootActor* RootActor = Cast<ADisplayClusterRootActor>(Actor);
				if (RootActor)
				{
					UE_LOGF(LogDisplayClusterEditorEngine, Log, "Found root actor - %ls", *RootActor->GetName());
					return RootActor;
				}
			}
		}
	}

	return nullptr;
}

void UDisplayClusterEditorEngine::StartPlayInEditorSession(FRequestPlaySessionParams& InRequestParams)
{
	UE_LOGF(LogDisplayClusterEditorEngine, VeryVerbose, "UDisplayClusterEditorEngine::StartPlayInEditorSession");

	// Reset session frame counter
	SessionFrameCounter = 0;

	// Get PIE world
	UWorld* EditorWorldPreDup = GetEditorWorldContext().World();

	if (DisplayClusterModule)
	{
		// Find any nDisplay root actor in the PIE world
		ADisplayClusterRootActor* RootActor = FindDisplayClusterRootActor(EditorWorldPreDup);
		if (!RootActor && EditorWorldPreDup)
		{
			// Also search inside streamed levels
			const TArray<ULevelStreaming*>& StreamingLevels = EditorWorldPreDup->GetStreamingLevels();
			for (const ULevelStreaming* const StreamingLevel : StreamingLevels)
			{
				if (StreamingLevel && StreamingLevel->GetLevelStreamingState() == ELevelStreamingState::LoadedVisible)
				{
					// Look for the actor in those sub-levels that have been loaded already
					const TSoftObjectPtr<UWorld>& SubWorldAsset = StreamingLevel->GetWorldAsset();
					RootActor = FindDisplayClusterRootActor(SubWorldAsset.Get());
				}

				if (RootActor)
				{
					break;
				}
			}
		}

		// If we found a root actor, start DisplayCluster PIE session
		if (RootActor)
		{
			// Get current config data from the root actor
			UDisplayClusterConfigurationData* ConfigData = DuplicateObject<UDisplayClusterConfigurationData>(RootActor->GetConfigData(), this);

			// And start PIE session with that config data
			if (ConfigData)
			{
				if (DisplayClusterModule->StartSession(ConfigData, ConfigData->Cluster->PrimaryNode.Id))
				{
					bIsNDisplayPIE = true;
				}
				else
				{
					DisplayClusterModule->EndSession();
					UE_LOGF(LogDisplayClusterEditorEngine, Error, "An error occurred during DisplayCluster session start");
				}
			}
			else
			{
				UE_LOGF(LogDisplayClusterEditorEngine, Error, "Couldn't load config data");
			}
		}
	}

	// Start PIE
	Super::StartPlayInEditorSession(InRequestParams);

	// Pass PIE world to nDisplay
	if (bIsNDisplayPIE)
	{
		for (FWorldContext const& Context : GEngine->GetWorldContexts())
		{
			if (Context.WorldType == EWorldType::PIE && Context.World())
			{
				DisplayClusterModule->StartScene(Context.World());
				break;
			}
		}
	}
}

bool UDisplayClusterEditorEngine::LoadMap(FWorldContext& WorldContext, FURL URL, class UPendingNetGame* Pending, FString& Error)
{
	if (bIsNDisplayPIE)
	{
		// Finish previous scene
		DisplayClusterModule->EndScene();

		// Perform map loading
		if (!Super::LoadMap(WorldContext, URL, Pending, Error))
		{
			return false;
		}

		// Start new scene
		DisplayClusterModule->StartScene(WorldContext.World());
	}
	else
	{
		return Super::LoadMap(WorldContext, URL, Pending, Error);
	}

	return true;
}

void UDisplayClusterEditorEngine::Tick(float DeltaSeconds, bool bIdleMode)
{
	// Perform nDisplay Tick
	if (DisplayClusterModule && bIsActivePIE && bIsNDisplayPIE)
	{
		DisplayClusterModule->StartFrame(SessionFrameCounter);
		DisplayClusterModule->PreTick(DeltaSeconds);
		DisplayClusterModule->Tick(DeltaSeconds);
		DisplayClusterModule->PostTick(DeltaSeconds);
		DisplayClusterModule->EndFrame(SessionFrameCounter);
	}

	Super::Tick(DeltaSeconds, bIdleMode);

	// Increment session frame counter
	++SessionFrameCounter;
}

void UDisplayClusterEditorEngine::OnBeginPIE(const bool bSimulate)
{
	UE_LOGF(LogDisplayClusterEditorEngine, VeryVerbose, "UDisplayClusterEditorEngine::OnBeginPIE");

	bIsActivePIE = true;
}

void UDisplayClusterEditorEngine::OnEndPIE(const bool bSimulate)
{
	UE_LOGF(LogDisplayClusterEditorEngine, VeryVerbose, "UDisplayClusterEditorEngine::OnEndPIE");

	if (bIsNDisplayPIE)
	{
		// Notify nDisplay about session end
		DisplayClusterModule->EndScene();
		DisplayClusterModule->EndSession();
	}

	// Reset PIE flags
	bIsActivePIE   = false;
	bIsNDisplayPIE = false;
}
