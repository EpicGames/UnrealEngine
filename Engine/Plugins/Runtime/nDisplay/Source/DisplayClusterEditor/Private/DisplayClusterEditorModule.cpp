// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterEditorModule.h"

#include "Settings/DisplayClusterEditorSettings.h"

#include "ISettingsModule.h"


#define LOCTEXT_NAMESPACE "DisplayClusterEditor"


FDisplayClusterEditorModule::FDisplayClusterEditorModule()
	: Container(TEXT("Project"))
	, Category(TEXT("Plugins"))
	, Section(TEXT("nDisplay"))
{
}

void FDisplayClusterEditorModule::StartupModule()
{
	RegisterSettings();
}

void FDisplayClusterEditorModule::ShutdownModule()
{
	if (UObjectInitialized())
	{
		UnregisterSettings();
	}
}


void FDisplayClusterEditorModule::RegisterSettings()
{
	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
	{
		SettingsModule->RegisterSettings(
			Container, Category, Section,
			LOCTEXT("RuntimeSettingsName", "nDisplay"),
			LOCTEXT("RuntimeSettingsDescription", "Configure nDisplay"),
			GetMutableDefault<UDisplayClusterEditorSettings>()
		);
	}
}

void FDisplayClusterEditorModule::UnregisterSettings()
{
	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
	{
		SettingsModule->UnregisterSettings(Container, Category, Section);
	}
}

IMPLEMENT_MODULE(FDisplayClusterEditorModule, DisplayClusterEditor);

#undef LOCTEXT_NAMESPACE
