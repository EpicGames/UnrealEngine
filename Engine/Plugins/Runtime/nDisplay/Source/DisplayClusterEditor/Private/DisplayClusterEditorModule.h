// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDisplayClusterEditor.h"


/**
 * Display Cluster editor module
 */
class FDisplayClusterEditorModule :
	public IDisplayClusterEditor
{
public:
	FDisplayClusterEditorModule();

public:
	//~ IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

protected:
	void RegisterSettings();
	void UnregisterSettings();

private:
	const FName Container;
	const FName Category;
	const FName Section;
};
