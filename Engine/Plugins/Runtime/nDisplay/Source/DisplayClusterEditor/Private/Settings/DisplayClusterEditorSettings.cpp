// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterEditorSettings.h"

#include "DisplayClusterEditorEngine.h"
#include "DisplayClusterGameEngine.h"
#include "DisplayClusterPlayerInput.h"
#include "DisplayClusterViewportClient.h"
#include "EnhancedPlayerInput.h"
#include "GeneralProjectSettings.h"

#include "Editor/UnrealEdEngine.h"
#include "Engine/GameEngine.h"
#include "Engine/GameViewportClient.h"
#include "GameFramework/InputSettings.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/ConfigCacheIni.h"
#include "Serialization/JsonSerializerMacros.h"


void UDisplayClusterEditorSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	SaveConfig();

	if (PropertyChangedEvent.Property != nullptr)
	{
		const FName PropertyName = PropertyChangedEvent.Property->GetFName();

		const FString DefaultEnginePath = FPaths::Combine(FPaths::ProjectConfigDir(), TEXT("DefaultEngine.ini"));
		const FString DefaultGamePath   = FPaths::Combine(FPaths::ProjectConfigDir(), TEXT("DefaultGame.ini"));
		const FString DefaultInputPath  = FPaths::Combine(FPaths::ProjectConfigDir(), TEXT("DefaultInput.ini"));

		// Reset read-only flag from the config files
		{
			const FString* ConfigFiles[] = { &DefaultEnginePath, &DefaultGamePath, &DefaultInputPath };

			for (const FString* ConfigFile : ConfigFiles)
			{
				if (FPlatformFileManager::Get().GetPlatformFile().IsReadOnly(**ConfigFile))
				{
					FPlatformFileManager::Get().GetPlatformFile().SetReadOnly(**ConfigFile, false);
				}
			}
		}

		if (PropertyName == GET_MEMBER_NAME_CHECKED(UDisplayClusterEditorSettings, bEnabled))
		{
			// DefaultEngine.ini
			{
				// GameEngine
				{
					const UClass* const Class2Set = bEnabled ? UDisplayClusterGameEngine::StaticClass() : UGameEngine::StaticClass();
					GConfig->SetString(TEXT("/Script/Engine.Engine"), TEXT("GameEngine"), *Class2Set->GetPathName(), DefaultEnginePath);
				}

				// UnrealEdEngine
				{
					const UClass* const Class2Set = bEnabled ? UDisplayClusterEditorEngine::StaticClass() : UUnrealEdEngine::StaticClass();
					GConfig->SetString(TEXT("/Script/Engine.Engine"), TEXT("UnrealEdEngine"), *Class2Set->GetPathName(), DefaultEnginePath);
				}

				// GameViewportClientClassName
				{
					
					const UClass* const Class2Set = bEnabled ? UDisplayClusterViewportClient::StaticClass() : UGameViewportClient::StaticClass();
					GConfig->SetString(TEXT("/Script/Engine.Engine"), TEXT("GameViewportClientClassName"), *Class2Set->GetPathName(), DefaultEnginePath);
				}

				// Other engine settings
				{
					GConfig->SetBool(TEXT("/Script/Engine.UserInterfaceSettings"), TEXT("bAllowHighDPIInGameMode"),     true, DefaultEnginePath);
					GConfig->SetBool(TEXT("/Script/Engine.UserInterfaceSettings"), TEXT("bAllowHighDpiWhenUnattended"), true, DefaultEnginePath);
				}

				GConfig->Flush(true, DefaultEnginePath);
				GConfig->LoadFile(DefaultEnginePath);
			}

			// DefaultGame.ini
			if (UGeneralProjectSettings* GeneralProjectSettings = GetMutableDefault<UGeneralProjectSettings>())
			{
				GeneralProjectSettings->bUseBorderlessWindow = bEnabled;
				GeneralProjectSettings->SaveConfig();
				GeneralProjectSettings->TryUpdateDefaultConfigFile();
			}

			// DefaultInput.ini
			if (UInputSettings* InputSettings = GetMutableDefault<UInputSettings>())
			{
				UClass* const Class2Set = bEnabled ? UDisplayClusterPlayerInput::StaticClass() : UEnhancedPlayerInput::StaticClass();
				InputSettings->SetDefaultPlayerInputClass(Class2Set);
				InputSettings->SaveConfig();
				InputSettings->TryUpdateDefaultConfigFile();
			}
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(UDisplayClusterEditorSettings, bClusterReplicationEnabled))
		{
			if (bClusterReplicationEnabled)
			{
				FJsonSerializableArray NetDriverDefinitions;
				NetDriverDefinitions.Add(TEXT("(DefName=GameNetDriver,DriverClassName=/Script/DisplayClusterReplication.DisplayClusterNetDriver,DriverClassNameFallback=/Script/OnlineSubsystemUtils.IpNetDriver)"));
				NetDriverDefinitions.Add(TEXT("(DefName=DemoNetDriver,DriverClassName=/Script/Engine.DemoNetDriver,DriverClassNameFallback=/Script/Engine.DemoNetDriver)"));

				GConfig->SetArray(TEXT("/Script/Engine.GameEngine"), TEXT("+NetDriverDefinitions"), NetDriverDefinitions, DefaultEnginePath);
				GConfig->SetString(TEXT("/Script/DisplayClusterReplication.DisplayClusterNetDriver"), TEXT("NetConnectionClassName"), TEXT("DisplayClusterReplication.DisplayClusterNetConnection"), DefaultEnginePath);
			}
			else
			{
				GConfig->RemoveKey(TEXT("/Script/Engine.GameEngine"), TEXT("!NetDriverDefinitions"), DefaultEnginePath);
				GConfig->RemoveKey(TEXT("/Script/Engine.GameEngine"), TEXT("+NetDriverDefinitions"), DefaultEnginePath);
				GConfig->RemoveKey(TEXT("/Script/DisplayClusterReplication.DisplayClusterNetDriver"), TEXT("NetConnectionClassName"), DefaultEnginePath);
			}

			GConfig->Flush(true, DefaultEnginePath);
			GConfig->LoadFile(DefaultEnginePath);
		}
	}
}
