// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "DisplayClusterEditorSettings.generated.h"


/**
 * Implements nDisplay settings
 **/
UCLASS(config = Engine, defaultconfig)
class DISPLAYCLUSTEREDITOR_API UDisplayClusterEditorSettings : public UObject
{
	GENERATED_BODY()

protected:
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;

protected:

	/** Used to set or reset additional nDisplay settings */
	UPROPERTY(config, EditAnywhere, Category = Main, meta = (ToolTip = "When enabled, automatically overrides some engine classes and settings", ConfigRestartRequired = true))
	bool bEnabled;

	/** When enabled, replaces the original NetDriver to DisplayClusterNetDriver, original NetConnection to DisplayClusterNetDriver */
	UPROPERTY(config, EditAnywhere, Category = Main, meta = (ToolTip = "When enabled, replaces the original NetDriver to DisplayClusterNetDriver, original NetConnection to DisplayClusterNetDriver", ConfigRestartRequired = true))
	bool bClusterReplicationEnabled;
};
