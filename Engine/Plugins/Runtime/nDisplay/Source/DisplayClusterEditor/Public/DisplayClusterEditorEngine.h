// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Editor/UnrealEdEngine.h"
#include "DisplayClusterEditorEngine.generated.h"

class IPDisplayCluster;
class ADisplayClusterRootActor;


/**
 * Extended editor engine
 */
UCLASS()
class DISPLAYCLUSTEREDITOR_API UDisplayClusterEditorEngine
	: public UUnrealEdEngine
{
	GENERATED_BODY()

public:
	virtual void Init(IEngineLoop* InEngineLoop) override;
	virtual void PreExit() override;
	virtual void StartPlayInEditorSession(FRequestPlaySessionParams& InRequestParams) override;
	virtual bool LoadMap(FWorldContext& WorldContext, FURL URL, class UPendingNetGame* Pending, FString& Error) override;
	virtual void Tick(float DeltaSeconds, bool bIdleMode) override;

private:
	ADisplayClusterRootActor* FindDisplayClusterRootActor(UWorld* InWorld);

private:
	IPDisplayCluster* DisplayClusterModule = nullptr;

private:
	// Begin PIE delegate
	FDelegateHandle BeginPIEDelegate;
	void OnBeginPIE(const bool bSimulate);

	// End PIE delegate
	FDelegateHandle EndPIEDelegate;
	void OnEndPIE(const bool bSimulate);

	bool bIsActivePIE   = false;
	bool bIsNDisplayPIE = false;

	uint64 SessionFrameCounter = 0;
};
