// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class DisplayClusterFillDerivedDataCache : ModuleRules
{
	public DisplayClusterFillDerivedDataCache(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"OutputLog", 
				"UnrealEd"
			}
		);
			
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AssetRegistry",
				"AssetTools",
				"CoreUObject",
				"ContentBrowser",
				"Engine",
				"EditorStyle",
				"EditorWidgets",
				"InputCore",
				"Kismet",
				"Messaging",
				"Networking",
				"Projects",
				"PropertyEditor",
				"Slate",
				"SlateCore",
				"ToolMenus", 
				"ToolWidgets",
				"UnrealEd",
				"WorkspaceMenuStructure"
			}
		);
	}
}
