// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterFillDerivedDataCacheModule.h"

#include "DisplayClusterFillDerivedDataCacheLog.h"
#include "DisplayClusterFillDerivedDataCacheWorker.h"

#include "Commandlets/DerivedDataCacheCommandlet.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Interfaces/IProjectManager.h"
#include "Modules/ModuleManager.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "FDisplayClusterFillDerivedDataCacheModule"

FDisplayClusterFillDerivedDataCacheModule& FDisplayClusterFillDerivedDataCacheModule::Get()
{
	return FModuleManager::GetModuleChecked<FDisplayClusterFillDerivedDataCacheModule>("DisplayClusterFillDerivedDataCacheModule");
}

void FDisplayClusterFillDerivedDataCacheModule::StartupModule()
{
	FCoreDelegates::OnFEngineLoopInitComplete.AddRaw(this, &FDisplayClusterFillDerivedDataCacheModule::OnFEngineLoopInitComplete);
}

void FDisplayClusterFillDerivedDataCacheModule::OnFEngineLoopInitComplete()
{
	UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("LevelEditor.MainMenu.Build");

	FToolMenuSection& Section = Menu->FindOrAddSection("LevelEditorAutomation");
		
	Section.AddMenuEntry(
		"FillDerivedDataCache",
		LOCTEXT("FillDerivedDataCache", "Fill Derived Data Cache (Prepare all Shaders)"),
		LOCTEXT("FillDerivedDataCacheTooltip", "Prepare all shaders and preprocess Nanite and other heavy data ahead of time for all assets in this project for all checked platforms in Project Settings > Supported Platforms. \nThis is useful to avoid shader compilation and pop-in when opening a level or running PIE/Standalone. \nNote that this operation can take a very long time to complete and can take up a lot of disk space, depending on the size of your project."),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateRaw(this, &FDisplayClusterFillDerivedDataCacheModule::CreateAsyncTaskWorker)));
}

void FDisplayClusterFillDerivedDataCacheModule::ShutdownModule()
{
	UToolMenus::UnregisterOwner(this);
	FCoreDelegates::OnFEngineLoopInitComplete.RemoveAll(this);

	if (AsyncTaskWorker)
	{
		AsyncTaskWorker->CancelTask();
		AsyncTaskWorker = nullptr;
	}
}

void FDisplayClusterFillDerivedDataCacheModule::CreateAsyncTaskWorker()
{
	if (AsyncTaskWorker)
	{
		AsyncTaskWorker->CancelTask();
	}
	AsyncTaskWorker = MakeUnique<FDisplayClusterFillDerivedDataCacheWorker>();
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FDisplayClusterFillDerivedDataCacheModule, DisplayClusterFillDerivedDataCache)
