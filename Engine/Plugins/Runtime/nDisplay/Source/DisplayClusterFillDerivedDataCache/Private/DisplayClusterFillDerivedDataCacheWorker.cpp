// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterFillDerivedDataCacheWorker.h"

#include "DisplayClusterFillDerivedDataCacheLog.h"

#include "Commandlets/DerivedDataCacheCommandlet.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/Notifications/NotificationManager.h"
#include "HAL/PlatformProcess.h"
#include "Interfaces/IProjectManager.h"
#include "Internationalization/Regex.h"
#include "Misc/App.h"
#include "Misc/AsyncTaskNotification.h"
#include "Misc/DataDrivenPlatformInfoRegistry.h"
#include "Misc/ScopedSlowTask.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "FDisplayClusterFillDerivedDataCacheWorker"

const FText EnumeratingAssetsTitle = LOCTEXT("EnumeratingAssetsTitle", "Enumerating Assets (Step 1/3)...");
const FText EnumeratingAssetsProgressFormat = LOCTEXT("EnumeratingAssetsProgressFormat", "{0} assets discovered...");
const FText LoadingAssetsTitle = LOCTEXT("LoadingAssetsTitle", "Loading Assets (Step 2/3)...");
const FText CompilingAssetsTitle = LOCTEXT("CompilingAssetsTitle", "Compiling Assets (Step 3/3)...");
const FText GenericProgressFormat = LOCTEXT("GenericProgressFormat", "{0}/{1} ({2}%)...");

const FRegexPattern EnumeratingAssetsPattern(TEXT("Display:\\s+([0-9]+)\\)\\s.+")); // ex. "146) /Engine/EditorBlueprintResources/ActorMacros"
const FRegexPattern LoadingTotalPattern(TEXT("Display:\\s([0-9]+)\\spackages to load from command line arguments")); // ex. "2833 packages to load from command line arguments"
const FRegexPattern LoadingProgressPattern(TEXT("Display:\\sLoading\\s\\(([0-9]+)\\)")); // ex. "Loading (2833)"
const FRegexPattern CompileProgressPattern(TEXT("Display:\\sWaiting for\\s([0-9]+)\\s.+ to finish")); // ex. "Waiting for 14163 Shaders to finish"

FDisplayClusterFillDerivedDataCacheWorker::FDisplayClusterFillDerivedDataCacheWorker()
{	
	FAsyncTaskNotificationConfig NotificationConfig;
	NotificationConfig.bCanCancel = true;
	NotificationConfig.TitleText = EnumeratingAssetsTitle;
	NotificationConfig.bKeepOpenOnFailure = true;
	NotificationConfig.bKeepOpenOnSuccess = true;
	NotificationConfig.ExpireDuration = 1.0f;
	NotificationConfig.ProgressText = FText::Format(EnumeratingAssetsProgressFormat, FText::AsNumber(0));
	ProgressNotification = MakeUnique<FAsyncTaskNotification>(NotificationConfig);
	
	verify(FPlatformProcess::CreatePipe(ReadPipe, WritePipe));

	CurrentExecutableName = FPlatformProcess::ExecutablePath();
	const FString ProjectPath = FPaths::SetExtension(
		FPaths::Combine(FPaths::ProjectDir(), FApp::GetProjectName()),".uproject");
	Arguments = FString::Printf(TEXT("\"%s\" %s"), *ProjectPath, *GetDdcCommandletParams());
	
	UE_LOGF(LogDisplayClusterFillDerivedDataCache, Display, "Running commandlet: %ls %ls", *CurrentExecutableName, *Arguments);

	uint32 ProcessID;
	const bool bLaunchDetached = true;
	const bool bLaunchHidden = false;
	const bool bLaunchReallyHidden = false;
	ProcessHandle = FPlatformProcess::CreateProc(
		*CurrentExecutableName, *Arguments, bLaunchDetached, bLaunchHidden, bLaunchReallyHidden, &ProcessID,
		0, nullptr, WritePipe);
	
	Thread = TUniquePtr<FRunnableThread>(FRunnableThread::Create(this, TEXT("FDisplayClusterFillDerivedDataCacheWorker")));
}

FDisplayClusterFillDerivedDataCacheWorker::~FDisplayClusterFillDerivedDataCacheWorker()
{
	if (Thread)
	{
		constexpr bool bShouldWait = true;
		Thread->Kill( bShouldWait );
		Thread = nullptr;
	}
}

void FDisplayClusterFillDerivedDataCacheWorker::CancelTask()
{
	bWasCancelled = true;
	FPlatformProcess::TerminateProc(ProcessHandle);
	CompleteCommandletAndShowNotification();
}

void FDisplayClusterFillDerivedDataCacheWorker::CompleteCommandletAndShowNotification()
{
	if (bWasCancelled)
	{
		UE_LOGF(LogDisplayClusterFillDerivedDataCache, Display, "#### Commandlet Process Cancelled ####");
		ProgressNotification->SetComplete(
			LOCTEXT("ToastCommandletCancelled", "Commandlet Process Cancelled"), FText(), false);

		// Close the notification automatically
		ProgressNotification->SetKeepOpenOnFailure(false);
		return;
	}
	
	FPlatformProcess::GetProcReturnCode(ProcessHandle, &ResultCode);
	const bool bFinishedWithFailures = ResultCode != 0;

	FAsyncNotificationStateData NotificationStateData;
	NotificationStateData.ProgressText = FText();
	NotificationStateData.TitleText = LOCTEXT("ToastCommandletSuccess", "Commandlet Finished Without Error!");
	NotificationStateData.State = EAsyncTaskNotificationState::Success;

	if (bFinishedWithFailures)
	{
		UE_LOGF(LogDisplayClusterFillDerivedDataCache, Error, "#### Commandlet Finished With Errors ####");
		UE_LOGF(LogDisplayClusterFillDerivedDataCache, Error, "%ls %ls", *CurrentExecutableName, *Arguments);
		UE_LOGF(LogDisplayClusterFillDerivedDataCache, Error, "Return Code: %i", ResultCode);

		NotificationStateData.TitleText = LOCTEXT("ToastCommandletFailure", "Commandlet Finished With Error(s)");
		NotificationStateData.ProgressText = FText::Format(LOCTEXT("ReturnCodeFormat", "Return Code: {0}"), FText::AsNumber(ResultCode));
		NotificationStateData.State = EAsyncTaskNotificationState::Failure;
		NotificationStateData.Hyperlink = FSimpleDelegate::CreateLambda([]() { FGlobalTabmanager::Get()->TryInvokeTab(FName("OutputLog")); });
		NotificationStateData.HyperlinkText = LOCTEXT("ShowOutputLogHyperlink", "Show Output Log");
	}
	else
	{
		UE_LOGF(LogDisplayClusterFillDerivedDataCache, Display, "#### Commandlet Finished Without Error ####");
	}

	ProgressNotification->SetNotificationState(NotificationStateData);
}

void FDisplayClusterFillDerivedDataCacheWorker::RegexParseForEnumerationCount(const FString& LogString)
{
	FRegexMatcher EnumerationCountRegex(EnumeratingAssetsPattern, *LogString);
	while (EnumerationCountRegex.FindNext())
	{
		ProgressNotification->SetTitleText(EnumeratingAssetsTitle);
		
		if (const int32 EnumerationCount = FCString::Atoi(*EnumerationCountRegex.GetCaptureGroup(1)); EnumerationCount > 0)
		{
			ProgressNotification->SetProgressText(
			   FText::Format(EnumeratingAssetsProgressFormat, FText::AsNumber(EnumerationCount))
		   );
		}
	}
}

void FDisplayClusterFillDerivedDataCacheWorker::RegexParseForLoadingProgress(const FString& LogString)
{
	FRegexMatcher LoadingProgressRegex(LoadingProgressPattern, *LogString);
	while (LoadingProgressRegex.FindNext())
	{
		ProgressNotification->SetTitleText(LoadingAssetsTitle);
		
		if (LoadingTotal > INDEX_NONE)
		{
			if (const int32 LoadingProgress = FCString::Atoi(*LoadingProgressRegex.GetCaptureGroup(1)); LoadingProgress > 0)
			{
				ProgressNotification->SetProgressText(
				   FText::Format(GenericProgressFormat,
					   FText::AsNumber(LoadingProgress),
					   FText::AsNumber(LoadingTotal),
					   FText::AsNumber(FMath::FloorToInt((float)LoadingProgress / (float)LoadingTotal * 100))
				   )
			   );
			}
		}
	}
}

void FDisplayClusterFillDerivedDataCacheWorker::RegexParseForCompilationProgress(const FString& LogString)
{
	FRegexMatcher CompileProgressRegex(CompileProgressPattern, *LogString);
	while (CompileProgressRegex.FindNext())
	{
		AssetsWaitingToCompile = FCString::Atoi(*CompileProgressRegex.GetCaptureGroup(1));
		if (CompileTotal == INDEX_NONE && AssetsWaitingToCompile > 0)
		{
			CompileTotal = AssetsWaitingToCompile;
		}

		if (CompileTotal > 0)
		{
			ProgressNotification->SetTitleText(CompilingAssetsTitle);

			if (const int32 Progress = CompileTotal - AssetsWaitingToCompile; Progress > -1)
			{
				ProgressNotification->SetProgressText(
					FText::Format(
						GenericProgressFormat,
						FText::AsNumber(Progress),
						FText::AsNumber(CompileTotal),
						FText::AsNumber(FMath::FloorToInt((float)Progress / (float)CompileTotal * 100))
					)
				);
			}
		}
	}
}

FString FDisplayClusterFillDerivedDataCacheWorker::GetDdcCommandletParams() const
{
	return FString::Printf(TEXT("-run=DerivedDataCache %s -fill -DDC=CreateInstalledProjectPak"), *GetTargetPlatformParams());
}

FString FDisplayClusterFillDerivedDataCacheWorker::GetTargetPlatformParams() const
{
	TArray<FString> SupportedPlatforms;
	FProjectStatus ProjectStatus;
	
	if(IProjectManager::Get().QueryStatusForCurrentProject(ProjectStatus))
	{
		for (const FDataDrivenPlatformInfo* DDPI : FDataDrivenPlatformInfoRegistry::GetSortedPlatformInfos(EPlatformInfoType::TruePlatformsOnly))
		{
			if (ProjectStatus.IsTargetPlatformSupported(DDPI->IniPlatformName))
			{
				const FString PlatformName = DDPI->IniPlatformName.ToString();
				SupportedPlatforms.Add(PlatformName);

				// Not all platforms have an editor target but they may in the future.
				// It doesn't hurt to add targets that don't exist as they are ignored.
				SupportedPlatforms.Add(PlatformName + "Editor");
			}
		}
	}

	if (SupportedPlatforms.Num() > 0)
	{
		return FString::Printf(TEXT("-TargetPlatform=%s"), *FString::Join(SupportedPlatforms, TEXT("+")));
	}
	
	return FString();
}

FString FDisplayClusterFillDerivedDataCacheWorker::GetLastWholeLogBlock(const FString& LogString)
{
	FString CumulativeString = LastPartialLogLine + LogString;

	// Look for a line break. If it exists, we can consider everything to the right of it to be a partial line while the left of it is a whole line
	const int32 IndexOfLastNewLine = CumulativeString.Find(TEXT("\n"), ESearchCase::IgnoreCase, ESearchDir::FromEnd);

	if (IndexOfLastNewLine > INDEX_NONE)
	{
		LastPartialLogLine = CumulativeString.RightChop(IndexOfLastNewLine).TrimStartAndEnd();
		
		return CumulativeString.Left(IndexOfLastNewLine).TrimStartAndEnd();
	}

	// If we don't find a break in the line, we should consider the entire line partial
	LastPartialLogLine = CumulativeString;

	// Then return nothing so we know we have nothing to do this iteration
	return FString();
}

void FDisplayClusterFillDerivedDataCacheWorker::ReadCommandletOutputAndUpdateEditorNotification()
{
	while (FPlatformProcess::IsProcRunning(ProcessHandle))
	{
		if (ProgressNotification->GetPromptAction() == EAsyncTaskNotificationPromptAction::Cancel)
		{
			UE_LOGF(LogDisplayClusterFillDerivedDataCache, Display, "----------CANCEL BUTTON PRESSED----------");
			CancelTask();
			break;
		}

		const FString LogString = FPlatformProcess::ReadPipe(ReadPipe);
		if (LogString.TrimStartAndEnd().IsEmpty())
		{
			continue;
		}
		
		const FString LastWholeLogBlock = GetLastWholeLogBlock(LogString);
		if (LastWholeLogBlock.TrimStartAndEnd().IsEmpty())
		{
			continue;
		}

		TArray<FString> EachLineInLogBlock;
		LastWholeLogBlock.ParseIntoArray(EachLineInLogBlock, TEXT("\n"), true);

		for (const FString& Line : EachLineInLogBlock)
		{
			if (Line.TrimStartAndEnd().IsEmpty())
			{
				continue;
			}
			
			UE_LOGF(LogDisplayClusterFillDerivedDataCache, Display, "Commandlet Output: %ls", *Line)

			if (LoadingTotal == INDEX_NONE)
			{			
				// Try to find total number of assets to load
				FRegexMatcher LoadingTotalRegex(LoadingTotalPattern, *Line);

				while (LoadingTotalRegex.FindNext())
				{
					LoadingTotal = FCString::Atoi(*LoadingTotalRegex.GetCaptureGroup(1));
				}
			}
		
			// Progress on asset enumeration
			RegexParseForEnumerationCount(Line);
		
			// Progress on asset loading
			RegexParseForLoadingProgress(Line);
		
			// Progress on asset compilation
			RegexParseForCompilationProgress(Line);
		}

		FPlatformProcess::Sleep(0.5f);
	}
	
	UE_LOGF(LogDisplayClusterFillDerivedDataCache, Display, "----------PROC NOT RUNNING----------");
	CompleteCommandletAndShowNotification();
}

#undef LOCTEXT_NAMESPACE
