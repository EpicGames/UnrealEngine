// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Templates/UniquePtr.h"

class FDisplayClusterFillDerivedDataCacheModule : public IModuleInterface
{
public:
	
	static FDisplayClusterFillDerivedDataCacheModule& Get();

	//~ Begin IModuleInterface Interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface Interface

protected:
	
	void CreateAsyncTaskWorker();
	TUniquePtr<class FDisplayClusterFillDerivedDataCacheWorker> AsyncTaskWorker;

private:
	
	void OnFEngineLoopInitComplete();
};
