// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "HAL/PlatformProcess.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "Misc/SingleThreadRunnable.h"
#include "Templates/UniquePtr.h"

class FAsyncTaskNotification;

struct FProcHandle;

class FDisplayClusterFillDerivedDataCacheWorker : public FRunnable, public FSingleThreadRunnable
{
public:

	FDisplayClusterFillDerivedDataCacheWorker();
	
	virtual ~FDisplayClusterFillDerivedDataCacheWorker() override;

	//~ Begin FRunnable implementation
	virtual uint32 Run() override
	{
		ReadCommandletOutputAndUpdateEditorNotification();
		return 0;
	}
	virtual void Stop() override
	{
		CancelTask();
	}
	//~ End FRunnable implementation
	
	//~ Begin FSingleThreadRunnable implementation
	virtual void Tick() override
	{
		ReadCommandletOutputAndUpdateEditorNotification();
	}
	virtual FSingleThreadRunnable* GetSingleThreadInterface() override
	{
		return this;
	}
	//~ End FSingleThreadRunnable implementation

	void CancelTask();
	void CompleteCommandletAndShowNotification();

protected:
	
	void RegexParseForEnumerationCount(const FString& LogString);
	void RegexParseForLoadingProgress(const FString& LogString);
	void RegexParseForCompilationProgress(const FString& LogString);

	FString GetDdcCommandletParams() const;
	FString GetTargetPlatformParams() const;

	FString GetLastWholeLogBlock(const FString& LogString);

	void ReadCommandletOutputAndUpdateEditorNotification();
	
	/** Progress notification */
	TUniquePtr<FAsyncTaskNotification> ProgressNotification;

private:

	TUniquePtr<FRunnableThread> Thread;
	
	int32 ResultCode = 0;
	bool bWasCancelled = false;
	FProcHandle ProcessHandle;

	void* ReadPipe = nullptr;
	void* WritePipe = nullptr;

	FString LastPartialLogLine;

	int32 LoadingTotal = INDEX_NONE;
	int32 CompileTotal = INDEX_NONE;
	int32 AssetsWaitingToCompile = INDEX_NONE;

	FString CurrentExecutableName;
	FString Arguments;
};
