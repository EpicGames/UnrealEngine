// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class DisplayClusterLightCardEditor : ModuleRules
{
	public DisplayClusterLightCardEditor(ReadOnlyTargetRules ROTargetRules) : base(ROTargetRules)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {

				"AdvancedPreviewScene",
				"ApplicationCore",
				"AppFramework",
				"Core",
				"CoreUObject",
				"DisplayCluster",
				"DisplayClusterConfiguration",
				"DisplayClusterOperator",
				"DisplayClusterLightCardExtender",
				"DisplayClusterLightCardEditorShaders",
				"DisplayClusterScenePreview",
				"EditorStyle",
				"Engine",
				"InputCore",
				"OpenCV",
				"OpenCVHelper",
				"ProceduralMeshComponent",
				"PropertyEditor",
				"RenderCore",
				"Renderer",
				"RHI",
				"SceneOutliner",
				"Slate",
				"SlateCore",
				"UMG",
				"UnrealEd",
				"ToolMenus",
				"ToolWidgets",
				"WorkspaceMenuStructure"
			});

		PrivateIncludePaths.AddRange(
			new string[] {
			}
		);
	}
}
