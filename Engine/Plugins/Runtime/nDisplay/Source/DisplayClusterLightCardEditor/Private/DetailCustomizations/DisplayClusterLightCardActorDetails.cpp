// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterLightCardActorDetails.h"

#include "DisplayClusterLightCardActor.h"
#include "IDisplayClusterLightCardActorExtender.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "PropertyHandle.h"
#include "Algo/Find.h"
#include "Features/IModularFeatures.h"

TSharedRef<IDetailCustomization> FDisplayClusterLightCardActorDetails::MakeInstance()
{
	return MakeShared<FDisplayClusterLightCardActorDetails>();
}

void FDisplayClusterLightCardActorDetails::CustomizeDetails(IDetailLayoutBuilder& InLayoutBuilder)
{
	// Expose translucency sort priority property of the light card component
	{
		TSharedRef<IPropertyHandle> LightCardComponentPropertyHandle = InLayoutBuilder.GetProperty(TEXT("LightCardComponent"));
		check(LightCardComponentPropertyHandle->IsValidHandle());

		TSharedPtr<IPropertyHandle> TranslucencySortPriorityPropertyHandle = LightCardComponentPropertyHandle->GetChildHandle(TEXT("TranslucencySortPriority"));
		check(TranslucencySortPriorityPropertyHandle->IsValidHandle());

		InLayoutBuilder
			.EditCategory(TEXT("Rendering"))
			.AddProperty(TranslucencySortPriorityPropertyHandle);
	}

	// Add the detail views of components added for Light Card Actor Extenders
	{
		const TSharedRef<IPropertyHandle> ExtenderNameToComponentMapHandle = InLayoutBuilder.GetProperty(ADisplayClusterLightCardActor::GetExtenderNameToComponentMapMemberName());
		ExtenderNameToComponentMapHandle->MarkHiddenByCustomization();

		TArray<const void*> RawDatas;
		ExtenderNameToComponentMapHandle->AccessRawData(RawDatas);

		if (!RawDatas.IsEmpty())
		{
			// Remember the native categories, so extender categories can be ordered bottom-most later
			TArray<FName> NativeCategories;
			InLayoutBuilder.GetCategoryNames(NativeCategories);

			IModularFeatures& ModularFeatures = IModularFeatures::Get();
			const TArray<IDisplayClusterLightCardActorExtender*> Extenders = ModularFeatures.GetModularFeatureImplementations<IDisplayClusterLightCardActorExtender>(IDisplayClusterLightCardActorExtender::ModularFeatureName);

			FName Category;
			bool bShouldShowSubcategories = false;
			TArray<UObject*> Components;
			TArray<FName> ExtenderCategories;

			for (const void* RawData : RawDatas)
			{
				TMap<FName, TObjectPtr<UActorComponent>>* ExtenderNameToComponentMapPtr = (TMap<FName, TObjectPtr<UActorComponent>>*)(RawData);
				check(ExtenderNameToComponentMapPtr);

				for (const TTuple<FName, TObjectPtr<UActorComponent>>& ExtenderNameToComponentPair : *ExtenderNameToComponentMapPtr)
				{
					if (!ensureMsgf(ExtenderNameToComponentPair.Value, TEXT("Trying to display component for Extender %s, but component is invalid."), *ExtenderNameToComponentPair.Key.ToString()))
					{
						continue;
					}

					IDisplayClusterLightCardActorExtender* const* ExtenderPtr =
						Algo::FindByPredicate(Extenders, [ExtenderNameToComponentPair](const IDisplayClusterLightCardActorExtender* Extender)
							{
								return Extender->GetExtenderName() == ExtenderNameToComponentPair.Key;
							});

					if (!ensureMsgf(ExtenderPtr, TEXT("Cannot find Extender %s expected to exist for component %s."), *ExtenderNameToComponentPair.Key.ToString(), *ExtenderNameToComponentPair.Value->GetName()))
					{
						continue;
					}

					bShouldShowSubcategories = (*ExtenderPtr)->ShouldShowSubcategories();
					Components.Add(ExtenderNameToComponentPair.Value);

					Category = (*ExtenderPtr)->GetCategory();
					if (!NativeCategories.Contains(Category))
					{
						ExtenderCategories.AddUnique(Category);
					}
				}
			}

			FAddPropertyParams AddPropertyParams;
			AddPropertyParams.CreateCategoryNodes(bShouldShowSubcategories);
			AddPropertyParams.HideRootObjectNode(true);

			InLayoutBuilder
				.EditCategory(Category)
				.AddExternalObjects(Components, EPropertyLocation::Default, AddPropertyParams);

			// Sort extender categories bottom-most
			InLayoutBuilder.SortCategories([ExtenderCategories](const TMap<FName, IDetailCategoryBuilder*>& CategoryMap)
				{
					int32 NativeCategorySortOrder = 1;
					int32 ExtenderCategorySortOrder = CategoryMap.Num();
					for (const TTuple<FName, IDetailCategoryBuilder*>& CategoryPair : CategoryMap)
					{
						if (ExtenderCategories.Contains(CategoryPair.Key))
						{
							CategoryPair.Value->SetSortOrder(ExtenderCategorySortOrder);
							ExtenderCategorySortOrder--;
						}
						else
						{
							CategoryPair.Value->SetSortOrder(NativeCategorySortOrder);
							NativeCategorySortOrder++;
						}
					}
				});
		}
	}
}
