// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterLightCardEditor.h"

#include "DisplayClusterChromakeyCardActor.h"
#include "DisplayClusterLightCardEditorCommands.h"
#include "DisplayClusterLightCardEditorUtils.h"
#include "DisplayClusterLightCardEditorStyle.h"
#include "IDisplayClusterLightCardEditor.h"
#include "LightCardTemplates/DisplayClusterLightCardTemplate.h"
#include "LightCardTemplates/DisplayClusterLightCardTemplateHelpers.h"
#include "LightCardTemplates/SDisplayClusterLightCardTemplateList.h"
#include "Outliner/SDisplayClusterLightCardOutliner.h"
#include "Settings/DisplayClusterLightCardEditorSettings.h"

#include "Viewport/DisplayClusterLightcardEditorViewport.h"
#include "Viewport/DisplayClusterLightCardEditorViewportClient.h"

#include "IDisplayClusterOperator.h"
#include "IDisplayClusterOperatorViewModel.h"

#include "DisplayClusterConfigurationTypes.h"

#include "DisplayClusterRootActor.h"
#include "Blueprints/DisplayClusterBlueprintLib.h"
#include "Components/DisplayClusterCameraComponent.h"
#include "Components/DisplayClusterICVFXCameraComponent.h"
#include "Components/DisplayClusterLabelComponent.h"

#include "ContentBrowserModule.h"
#include "EditorModeManager.h"
#include "FileHelpers.h"
#include "IContentBrowserSingleton.h"
#include "LevelEditorActions.h"
#include "ObjectEditorUtils.h"
#include "PropertyCustomizationHelpers.h"
#include "ScopedTransaction.h"
#include "Selection.h"
#include "UnrealEdGlobals.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Editor/Transactor.h"
#include "Editor/UnrealEdEngine.h"
#include "Engine/Blueprint.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Docking/LayoutExtender.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Layers/LayersSubsystem.h"
#include "Misc/FileHelper.h"
#include "Misc/ITransaction.h"
#include "Misc/TransactionObjectEvent.h"
#include "Styling/SlateIconFinder.h"
#include "Tests/AutomationCommon.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/Workflow/SWizard.h"

#define LOCTEXT_NAMESPACE "FDisplayClusterLightCardEditor"

const FName FDisplayClusterLightCardEditor::ViewportTabName = TEXT("DisplayClusterLightCardViewportTab");
const FName FDisplayClusterLightCardEditor::OutlinerTabName = TEXT("DisplayClusterLightCardOutlinerTab");

TSharedRef<IDisplayClusterOperatorApp> FDisplayClusterLightCardEditor::MakeInstance(
	TSharedRef<IDisplayClusterOperatorViewModel> InViewModel)
{
	TSharedRef<FDisplayClusterLightCardEditor> Instance = MakeShared<FDisplayClusterLightCardEditor>();
	Instance->Initialize(InViewModel); // Initialize necessary to perform any SP binding after SP construction
	return Instance;
}

FDisplayClusterLightCardEditor::~FDisplayClusterLightCardEditor()
{
	UnregisterTabSpawners();
	
	IDisplayClusterOperator::Get().GetOperatorViewModel()->OnActiveRootActorChanged().Remove(ActiveRootActorChangedHandle);
	IDisplayClusterOperator::Get().GetOperatorViewModel()->OnActorsSelectedExternally().Remove(ActorsSelectedExternallyHandle);
	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);

	if (GEngine != nullptr)
	{
		GEngine->OnLevelActorAdded().RemoveAll(this);
		GEngine->OnLevelActorDeleted().RemoveAll(this);
	}

	if (OnObjectTransactedHandle.IsValid())
	{
		FCoreUObjectDelegates::OnObjectTransacted.Remove(OnObjectTransactedHandle);
	}

	RemoveCompileDelegates();
}

void FDisplayClusterLightCardEditor::PostUndo(bool bSuccess)
{
	FEditorUndoClient::PostUndo(bSuccess);
	RefreshLabels();
}

void FDisplayClusterLightCardEditor::PostRedo(bool bSuccess)
{
	FEditorUndoClient::PostRedo(bSuccess);
	RefreshLabels();
}

void FDisplayClusterLightCardEditor::Initialize(TSharedRef<IDisplayClusterOperatorViewModel> InViewModel)
{
	OperatorViewModel = InViewModel;

	ActiveRootActorChangedHandle = InViewModel->OnActiveRootActorChanged().AddSP(this, &FDisplayClusterLightCardEditor::OnActiveRootActorChanged);
	ActorsSelectedExternallyHandle = InViewModel->OnActorsSelectedExternally().AddSP(this, &FDisplayClusterLightCardEditor::OnActorsSelectedExternally);
	if (GEngine != nullptr)
	{
		GEngine->OnLevelActorAdded().AddSP(this, &FDisplayClusterLightCardEditor::OnLevelActorAdded);
		GEngine->OnLevelActorDeleted().AddSP(this, &FDisplayClusterLightCardEditor::OnLevelActorDeleted);
	}
	
	OnObjectTransactedHandle = FCoreUObjectDelegates::OnObjectTransacted.AddSP(this, &FDisplayClusterLightCardEditor::OnObjectTransacted);

	BindCommands();
	RegisterTabSpawners();
	RegisterToolbarExtensions();
	RegisterMenuExtensions();

	BindCompileDelegates();

	RefreshLabels();

	GEditor->RegisterForUndo(this);

	
}

TArray<AActor*> FDisplayClusterLightCardEditor::FindAllManagedActors() const
{
	TArray<AActor*> ManagedActors;
	if (ActiveRootActor.IsValid())
	{
		// First populate light cards
		TSet<ADisplayClusterLightCardActor*> RootActorLightCardActors;
		UDisplayClusterBlueprintLib::FindLightCardsForRootActor(ActiveRootActor.Get(), RootActorLightCardActors);

		// Find chromakey cards
		{
			TSet<ADisplayClusterChromakeyCardActor*> ChromakeyCardActors;
			UDisplayClusterBlueprintLib::FindChromakeyCardsForRootActor(ActiveRootActor.Get(), ChromakeyCardActors);
			RootActorLightCardActors.Append(reinterpret_cast<TSet<ADisplayClusterLightCardActor*>&>(ChromakeyCardActors));
		}
		
		for (ADisplayClusterLightCardActor* LightCardActor : RootActorLightCardActors)
		{
			ManagedActors.Add(LightCardActor);
		}
		
		// Find other actors
		if (UWorld* World = ActiveRootActor->GetWorld())
		{
			for (const TWeakObjectPtr<AActor> WeakActor : TActorRange<AActor>(World))
			{
				if (WeakActor.IsValid() && !WeakActor->IsA<ADisplayClusterLightCardActor>() &&
					UE::DisplayClusterLightCardEditorUtils::IsManagedActor(WeakActor.Get()))
				{
					ManagedActors.Add(WeakActor.Get());
				}
			}
		}
	}

	return ManagedActors;
}

void FDisplayClusterLightCardEditor::SelectActors(const TArray<AActor*>& ActorsToSelect)
{
	if (LightCardOutliner)
	{
		LightCardOutliner->SelectActors(ActorsToSelect);
	}

	if (ActorsToSelect.Num() > 0)
	{
		SelectedActors = FObjectEditorUtils::GetTypedWeakObjectPtrs<AActor>(static_cast<TArray<UObject*>>(ActorsToSelect));
	}
	else
	{
		SelectedActors.Empty();
	}
	
	IDisplayClusterOperator::Get().GetOperatorViewModel()->ShowDetailsForObjects(*reinterpret_cast<const TArray<UObject*>*>(&ActorsToSelect));
}

void FDisplayClusterLightCardEditor::SelectActorProxies(const TArray<AActor*>& ActorsToSelect)
{
	if (ViewportView)
	{
		ViewportView->GetLightCardEditorViewportClient()->SelectActors(ActorsToSelect);
	}
}

void FDisplayClusterLightCardEditor::GetSelectedActors(TArray<AActor*>& OutSelectedActors)
{
	OutSelectedActors = GetSelectedActorsAs<AActor>();
}

void FDisplayClusterLightCardEditor::CenterActorInView(AActor* Actor)
{
	check(Actor);
	if (ViewportView)
	{
		ViewportView->GetLightCardEditorViewportClient()->CenterActorInView(Actor);
	}
}

AActor* FDisplayClusterLightCardEditor::SpawnActor(TSubclassOf<AActor> InActorClass, const FName& InActorName,
	const UDisplayClusterLightCardTemplate* InTemplate, ULevel* InLevel, bool bIsPreview)
{
	if (!ActiveRootActor.IsValid())
	{
		return nullptr;
	}

	const UClass* ClassToUse = InTemplate ? InTemplate->GetClass() : InActorClass.Get(); 
	
	FScopedTransaction Transaction(LOCTEXT("SpawnActorTransactionMessage", "Spawn Actor"));

	const UDisplayClusterLightCardEditorProjectSettings* Settings = GetDefault<UDisplayClusterLightCardEditorProjectSettings>();
	
	FDisplayClusterLightCardEditorHelper::FSpawnActorArgs SpawnArgs;
	{
		FString ActorNameStr = InActorName.IsNone() ? ClassToUse->GetDisplayNameText().ToString() : InActorName.ToString();
		ActorNameStr.RemoveSpacesInline();
		
		SpawnArgs.ActorClass = InActorClass;
		SpawnArgs.ActorName = *ActorNameStr;
		SpawnArgs.RootActor = ActiveRootActor.Get();
		SpawnArgs.Template = InTemplate;
		SpawnArgs.Level = InLevel;
		SpawnArgs.AddLightCardArgs.LabelConfiguration.bVisible = Settings->bDisplayLightCardLabels;
		SpawnArgs.AddLightCardArgs.LabelConfiguration.Scale = Settings->LightCardLabelScale;
		SpawnArgs.AddLightCardArgs.LabelConfiguration.LabelFlags = Settings->LightCardLabelFlags;
		SpawnArgs.AddLightCardArgs.LabelConfiguration.RootActor = ActiveRootActor.Get();
	
		if (ViewportView.IsValid())
		{
			SpawnArgs.ProjectionMode = ViewportView->GetLightCardEditorViewportClient()->GetProjectionMode();
		}
	}
	
	AActor* NewActor = FDisplayClusterLightCardEditorHelper::SpawnStageActor(MoveTemp(SpawnArgs));
	check(NewActor);

	RefreshPreviewStageActor(NewActor);
	
	if (!bIsPreview)
	{
		if (InTemplate)
		{
			// Only template recent items need to be handled, new actor spawning is handled from caller otherwise.
			FDisplayClusterLightCardEditorRecentItem RecentlyPlacedItem;
			RecentlyPlacedItem.ObjectPath = const_cast<UDisplayClusterLightCardTemplate*>(InTemplate);
			RecentlyPlacedItem.ItemType = FDisplayClusterLightCardEditorRecentItem::Type_LightCardTemplate;
		
			AddRecentlyPlacedItem(MoveTemp(RecentlyPlacedItem));
		}
		
		SelectActors({ NewActor });
	}
	
	return NewActor;
}

AActor* FDisplayClusterLightCardEditor::SpawnActor(const UDisplayClusterLightCardTemplate* InTemplate, ULevel* InLevel, bool bIsPreview)
{
	return SpawnActor(nullptr, NAME_None, InTemplate, InLevel, bIsPreview);
}

ADisplayClusterLightCardActor* FDisplayClusterLightCardEditor::AddNewLightCard()
{
	check(ActiveRootActor.IsValid());

	FScopedTransaction Transaction(LOCTEXT("AddNewLightCardTransactionMessage", "Add New Light Card"));

	const UDisplayClusterLightCardEditorProjectSettings* Settings = GetDefault<UDisplayClusterLightCardEditorProjectSettings>();
	const UDisplayClusterLightCardTemplate* Template = Settings->DefaultLightCardTemplate.LoadSynchronous();
	
	if (ADisplayClusterLightCardActor* NewLightCard = SpawnActorAs<ADisplayClusterLightCardActor>(TEXT("LightCard"), Template))
	{
		if (!Template)
		{
			// When adding a new lightcard, usually the desired location is in the middle of the viewport
			CenterActorInView(NewLightCard);
		}
		
		FDisplayClusterLightCardEditorRecentItem RecentlyPlacedItem;
		RecentlyPlacedItem.ObjectPath = NewLightCard->GetClass();
		RecentlyPlacedItem.ItemType = FDisplayClusterLightCardEditorRecentItem::Type_LightCard;
		
		AddRecentlyPlacedItem(MoveTemp(RecentlyPlacedItem));
		SelectActors({NewLightCard});

		return NewLightCard;
	}

	return nullptr;
}

void FDisplayClusterLightCardEditor::AddExistingLightCard()
{
	TSharedPtr<SWindow> PickerWindow;
	TWeakObjectPtr<ADisplayClusterLightCardActor> SelectedActorPtr;
	bool bFinished = false;
	
	const TSharedRef<SWidget> ActorPicker = PropertyCustomizationHelpers::MakeActorPickerWithMenu(
		nullptr,
		false,
		FOnShouldFilterActor::CreateLambda([&](const AActor* const InActor) -> bool // ActorFilter
		{
			const bool IsAllowed = InActor != nullptr && !InActor->IsChildActor() && InActor->IsA<ADisplayClusterLightCardActor>() &&
				!InActor->GetClass()->HasAnyClassFlags(CLASS_Interface)	&& !InActor->IsA<ADisplayClusterRootActor>();
			
			return IsAllowed;
		}),
		FOnActorSelected::CreateLambda([&](AActor* InActor) -> void // OnSet
		{
			SelectedActorPtr = Cast<ADisplayClusterLightCardActor>(InActor);
		}),
		FSimpleDelegate::CreateLambda([&]() -> void // OnClose
		{
		}),
		FSimpleDelegate::CreateLambda([&]() -> void // OnUseSelected
		{
			if (ADisplayClusterLightCardActor* Selection = Cast<ADisplayClusterLightCardActor>(GEditor->GetSelectedActors()->GetTop(ADisplayClusterLightCardActor::StaticClass())))
			{
				SelectedActorPtr = Selection;
			}
		}));
	
	PickerWindow = SNew(SWindow)
	.Title(LOCTEXT("AddExistingLightCard", "Select an existing Light Card actor"))
	.ClientSize(FVector2D(500, 525))
	.SupportsMinimize(false) .SupportsMaximize(false)
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("Menu.Background"))
		[
			SNew(SWizard)
			.FinishButtonText(LOCTEXT("FinishAddingExistingLightCard", "Add Actor"))
			.OnCanceled(FSimpleDelegate::CreateLambda([&]()
			{
				if (PickerWindow.IsValid())
				{
					PickerWindow->RequestDestroyWindow();
				}
			}))
			.OnFinished(FSimpleDelegate::CreateLambda([&]()
			{
				bFinished = true;
				if (PickerWindow.IsValid())
				{
					PickerWindow->RequestDestroyWindow();
				}
			}))
			.CanFinish(TAttribute<bool>::CreateLambda([&]()
			{
				return SelectedActorPtr.IsValid();
			}))
			.ShowPageList(false)
			+SWizard::Page()
			.CanShow(true)
			[
				SNew(SBorder)
				.VAlign(VAlign_Fill)
				.BorderImage(FAppStyle::GetBrush("Menu.Background"))
				[
					SNew(SVerticalBox)
					+SVerticalBox::Slot()
					.AutoHeight()
					.VAlign(VAlign_Fill)
					.HAlign(HAlign_Fill)
					[
						ActorPicker
					]
					+SVerticalBox::Slot()
					.AutoHeight()
					.VAlign(VAlign_Bottom)
					.Padding(0.f, 8.f)
					[
						SNew(STextBlock)
						.TextStyle(FAppStyle::Get(), "NormalText.Important")
						.Text_Lambda([&]
						{
							const FString Result = FString::Printf(TEXT("Selected Actor: %s"),
								SelectedActorPtr.IsValid() ? *SelectedActorPtr->GetActorLabel() : TEXT(""));
							return FText::FromString(Result);
						})
					]
				]
			]
		]
	];

	GEditor->EditorAddModalWindow(PickerWindow.ToSharedRef());
	if (bFinished && SelectedActorPtr.IsValid())
	{
		FScopedTransaction Transaction(LOCTEXT("AddExistingLightCardTransactionMessage", "Add Existing Light Card"));

		TArray<ADisplayClusterLightCardActor*> LightCards { SelectedActorPtr.Get() };
		AddLightCardsToActor(LightCards);
	}

	PickerWindow.Reset();
	SelectedActorPtr.Reset();
}

ADisplayClusterLightCardActor* FDisplayClusterLightCardEditor::AddNewFlag()
{
	check(ActiveRootActor.IsValid());

	FScopedTransaction Transaction(LOCTEXT("AddNewFlagTransactionMessage", "Add New Flag"));

	const UDisplayClusterLightCardEditorProjectSettings* Settings = GetDefault<UDisplayClusterLightCardEditorProjectSettings>();
	const UDisplayClusterLightCardTemplate* Template = Settings->DefaultFlagTemplate.LoadSynchronous();
	
	if (ADisplayClusterLightCardActor* NewLightCard = SpawnActorAs<ADisplayClusterLightCardActor>(TEXT("Flag"), Template))
	{
		if (!Template)
		{
			NewLightCard->Color = FLinearColor(0.f, 0.f, 0.f);

			// When adding a new lightcard, usually the desired location is in the middle of the viewport
			CenterActorInView(NewLightCard);
		}

		NewLightCard->SetIsLightCardFlag(true);
		
		FDisplayClusterLightCardEditorRecentItem RecentlyPlacedItem;
		RecentlyPlacedItem.ObjectPath = NewLightCard->GetClass();
		RecentlyPlacedItem.ItemType = FDisplayClusterLightCardEditorRecentItem::Type_Flag;
		
		AddRecentlyPlacedItem(MoveTemp(RecentlyPlacedItem));
		SelectActors({NewLightCard});
		return NewLightCard;
	}

	return nullptr;
}

AActor* FDisplayClusterLightCardEditor::AddNewDynamic(UClass* InClass)
{
	check(ActiveRootActor.IsValid());

	FScopedTransaction Transaction(LOCTEXT("AddNewActorMessage", "Add New Actor"));
	
	if (AActor* NewActor = SpawnActor(InClass))
	{
		CenterActorInView(NewActor);

		FDisplayClusterLightCardEditorRecentItem RecentlyPlacedItem;
		RecentlyPlacedItem.ObjectPath = NewActor->GetClass();
		RecentlyPlacedItem.ItemType = FDisplayClusterLightCardEditorRecentItem::Type_Dynamic;
		
		AddRecentlyPlacedItem(MoveTemp(RecentlyPlacedItem));
		SelectActors({NewActor});
		return NewActor;
	}

	return nullptr;
}

void FDisplayClusterLightCardEditor::AddLightCardsToActor(const TArray<ADisplayClusterLightCardActor*>& LightCards)
{
	if (ActiveRootActor.IsValid())
	{
		FDisplayClusterLightCardEditorHelper::FAddLightCardArgs AddLightCardArgs;
		AddLightCardArgs.LabelConfiguration.bVisible = ShouldShowLightCardLabels();
		AddLightCardArgs.LabelConfiguration.Scale = *GetLightCardLabelScale();
		AddLightCardArgs.LabelConfiguration.LabelFlags = GetLightCardLabelFlags();
		AddLightCardArgs.LabelConfiguration.RootActor = ActiveRootActor.Get();
		
		FDisplayClusterLightCardEditorHelper::AddLightCardsToRootActor(LightCards, ActiveRootActor.Get(), MoveTemp(AddLightCardArgs));
		
		for (ADisplayClusterLightCardActor* LightCard : LightCards)
		{
			check(LightCard);
			RefreshPreviewStageActor(LightCard);
		}
	}
}

bool FDisplayClusterLightCardEditor::CanAddNewActor() const
{
	return CanAddNewActor(nullptr);
}

bool FDisplayClusterLightCardEditor::CanAddNewActor(UClass* InClass) const
{
	bool bClassAllowed = true;
	if (InClass && InClass->IsChildOf(ADisplayClusterChromakeyCardActor::StaticClass()))
	{
		// @todo uv chromakey cards
		bClassAllowed = !ViewportView.IsValid() ||
			ViewportView->GetLightCardEditorViewportClient()->GetProjectionMode() != EDisplayClusterMeshProjectionType::UV;
	}

	return bClassAllowed && ActiveRootActor.IsValid() && ActiveRootActor->GetWorld() != nullptr;
}

void FDisplayClusterLightCardEditor::CutSelectedActors()
{
	CopySelectedActors(/* bShouldCut */ true);
}

bool FDisplayClusterLightCardEditor::CanCutSelectedActors()
{
	if (LightCardOutliner.IsValid() && LightCardOutliner->CanCutSelectedFolder())
	{
		// If only a folder is selected -- this isn't handled outside of the outliner
		return true;
	}
	
	return CanCopySelectedActors() && CanRemoveSelectedActors();
}

void FDisplayClusterLightCardEditor::CopySelectedActors(bool bShouldCut)
{
	const FText TransactionText = bShouldCut ? LOCTEXT("CutItemsTransactionMessage", "Cut Selected Items") : LOCTEXT("CopyItemsTransactionMessage", "Copy Selected Items");
	FScopedTransaction Transaction(TransactionText);

	if (DoOutlinerFoldersNeedEditorDelegates())
	{
		// We still must fire the delegates in case folders are selected
		if (bShouldCut)
		{
			FEditorDelegates::OnEditCutActorsBegin.Broadcast();
			FEditorDelegates::OnEditCutActorsEnd.Broadcast();
		}
		else
		{
			FEditorDelegates::OnEditCopyActorsBegin.Broadcast();
			FEditorDelegates::OnEditCopyActorsEnd.Broadcast();
		}
		
		return;
	}
	
	UWorld* EditorWorld = GEditor->GetEditorWorldContext().World();

	const bool bNoteSelectionChange = false;
	const bool bDeselectBSPSurfs = true;
	const bool bWarnAboutManyActors = false;
	GEditor->SelectNone(bNoteSelectionChange, bDeselectBSPSurfs, bWarnAboutManyActors);

	for (const TWeakObjectPtr<AActor>& LightCard : SelectedActors)
	{
		const bool bInSelected = true;
		const bool bNotify = false;
		const bool bSelectEvenIfHidden = true;
		GEditor->SelectActor(LightCard.Get(), bInSelected, bNotify, bSelectEvenIfHidden);
	}

	const bool bIsMove = false;
	const bool bWarnAboutReferences = false;

	if (bShouldCut)
	{
		FEditorDelegates::OnEditCutActorsBegin.Broadcast();
	}
	else
	{
		FEditorDelegates::OnEditCopyActorsBegin.Broadcast();
	}
	
	GEditor->CopySelectedActorsToClipboard(EditorWorld, bShouldCut, bIsMove, bWarnAboutReferences);

	if (bShouldCut)
	{
		FEditorDelegates::OnEditCutActorsEnd.Broadcast();
	}
	else
	{
		FEditorDelegates::OnEditCopyActorsEnd.Broadcast();
	}
}

bool FDisplayClusterLightCardEditor::CanCopySelectedActors() const
{
	if (LightCardOutliner.IsValid() && LightCardOutliner->CanCopySelectedFolder())
	{
		// True if only a folder is selected -- this isn't handled outside of the outliner
		return true;
	}
	
	return SelectedActors.Num() > 0;
}

TArray<AActor*> FDisplayClusterLightCardEditor::PasteActors()
{
	if (DoOutlinerFoldersNeedEditorDelegates() && LightCardOutliner->CanPasteSelectedFolder())
	{
		// We still must fire the delegates in case folders are selected
		FEditorDelegates::OnEditPasteActorsBegin.Broadcast();
		FEditorDelegates::OnEditPasteActorsEnd.Broadcast();
		return {};
	}

	UWorld* EditorWorld = GEditor->GetEditorWorldContext().World();
	
	FEditorDelegates::OnEditPasteActorsBegin.Broadcast();
	GEditor->PasteSelectedActorsFromClipboard(EditorWorld, LOCTEXT("PasteItemsTransactionMessage", "Paste Items"), EPasteTo::PT_OriginalLocation);
	FEditorDelegates::OnEditPasteActorsEnd.Broadcast();
	
	TArray<AActor*> PastedActors;
	TArray<ADisplayClusterLightCardActor*> PastedLightCards;
	for (FSelectionIterator It(GEditor->GetSelectedActorIterator()); It; ++It)
	{
		if (AActor* Actor = Cast<AActor>(*It))
		{
			if (!Actor->Implements<UDisplayClusterStageActor>())
			{
				continue;
			}

			PastedActors.Add(Actor);
			
			if (ADisplayClusterLightCardActor* LightCard = Cast<ADisplayClusterLightCardActor>(Actor))
			{
				PastedLightCards.Add(LightCard);
			}
		}
	}

	AddLightCardsToActor(PastedLightCards);
	SelectActors(PastedActors);

	return PastedActors;
}

bool FDisplayClusterLightCardEditor::CanPasteActors() const
{
	if (LightCardOutliner.IsValid() && LightCardOutliner->CanPasteSelectedFolder())
	{
		// If only a folder is copied -- this isn't handled outside of the outliner
		return true;
	}
	
	UWorld* EditorWorld = GEditor->GetEditorWorldContext().World();
	return GEditor->CanPasteSelectedActorsFromClipboard(EditorWorld);
}

void FDisplayClusterLightCardEditor::DuplicateSelectedActors()
{
	FScopedTransaction Transaction(LOCTEXT("DuplicateItemsTransactionMessage", "Duplicate Selected Items"));

	if (DoOutlinerFoldersNeedEditorDelegates())
	{
		// We still must fire the delegates in case folders are selected
		FEditorDelegates::OnDuplicateActorsBegin.Broadcast();
		FEditorDelegates::OnDuplicateActorsEnd.Broadcast();
		return;
	}
	
	FEditorDelegates::OnDuplicateActorsBegin.Broadcast();

	const UWorld* EditorWorld = GEditor->GetEditorWorldContext().World();

	TArray<AActor*> NewActors;
	TArray<ADisplayClusterLightCardActor*> DuplicatedLightCards;
	GUnrealEd->DuplicateActors(GetSelectedActorsAs<AActor>(), NewActors, EditorWorld->GetCurrentLevel(), FVector());

	for (AActor* Actor : NewActors)
	{
		if (!Actor->Implements<UDisplayClusterStageActor>())
		{
			continue;
		}
		
		IDisplayClusterStageActor* StageActor = CastChecked<IDisplayClusterStageActor>(Actor);
		// If the light card should be offset from its pasted location, offset its longitude and latitude by a number of
		// degrees equal to an arc length of 10 units (arc length = angle in radians * radius)
		const float AngleOffset = FMath::RadiansToDegrees(10.0f / FMath::Max(StageActor->GetDistanceFromCenter(), 1.0f));
		StageActor->SetLatitude(StageActor->GetLatitude() - AngleOffset);
		StageActor->SetLongitude(StageActor->GetLongitude() + AngleOffset);

		if (ADisplayClusterLightCardActor* LightCard = Cast<ADisplayClusterLightCardActor>(Actor))
		{
			DuplicatedLightCards.Add(LightCard);
		}
	}
	
	FEditorDelegates::OnDuplicateActorsEnd.Broadcast();
	
	AddLightCardsToActor(DuplicatedLightCards);
	SelectActors(NewActors);
}

bool FDisplayClusterLightCardEditor::CanDuplicateSelectedActors() const
{
	return CanCopySelectedActors();
}

void FDisplayClusterLightCardEditor::RenameSelectedItem()
{
	if (LightCardOutliner.IsValid())
	{
		return LightCardOutliner->RenameSelectedItem();
	}
}

bool FDisplayClusterLightCardEditor::CanRenameSelectedItem() const
{
	if (LightCardOutliner.IsValid())
	{
		return LightCardOutliner->CanRenameSelectedItem();
	}

	return false;
}

void FDisplayClusterLightCardEditor::RemoveSelectedActors(bool bDeleteLightCardActor)
{
	RemoveActors(SelectedActors, bDeleteLightCardActor);
}

void FDisplayClusterLightCardEditor::RemoveActors(
	const TArray<TWeakObjectPtr<AActor>>& InActorsToRemove, bool bDeleteActors)
{
	if (DoOutlinerFoldersNeedEditorDelegates())
	{
		// We still must fire the delegates in case folders are selected
		FEditorDelegates::OnDeleteActorsBegin.Broadcast();
		FEditorDelegates::OnDeleteActorsEnd.Broadcast();
		return;
	}
	
	if (InActorsToRemove.Num() == 0)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("RemoveItemsTransactionMessage", "Remove Item(s)"));

	USelection* EdSelectionManager = GEditor->GetSelectedActors();
	UWorld* WorldToUse = nullptr;

	if (bDeleteActors)
	{
		EdSelectionManager->BeginBatchSelectOperation();
		EdSelectionManager->Modify();
		EdSelectionManager->DeselectAll();
	}

	if (ActiveRootActor.IsValid())
	{
		UDisplayClusterConfigurationData* ConfigData = ActiveRootActor->GetConfigData();
		ConfigData->Modify();

		FDisplayClusterConfigurationICVFX_VisibilityList& RootActorLightCards = ConfigData->StageSettings.Lightcard.ShowOnlyList;

		for (const TWeakObjectPtr<AActor>& Actor : InActorsToRemove)
		{
			if (ADisplayClusterLightCardActor* LightCard = Cast<ADisplayClusterLightCardActor>(Actor.Get()))
			{
				RootActorLightCards.Actors.RemoveAll([&](const TSoftObjectPtr<AActor>& InActor)
					{
						return InActor.Get() == LightCard;
					});

				LightCard->RemoveFromRootActor();
				
				if (!bDeleteActors)
				{
					// Remove from any layers shared with the DCRA. If we aren't deleting this has the possibility to
					// remove from a layer used by another DCRA, but this is rare and acceptable.
					ULayersSubsystem* LayersSubsystem = GEditor->GetEditorSubsystem<ULayersSubsystem>();
					TArray<FName> LightCardLayerNames = LightCard->Layers;
					for (const FName& LightCardLayerName : LightCardLayerNames)
					{
						if (RootActorLightCards.ActorLayers.ContainsByPredicate([&](const FActorLayer& LightCardLayer)
						{
							return LightCardLayer.Name == LightCardLayerName;
						}))
						{
							LayersSubsystem->RemoveActorFromLayer(LightCard, LightCardLayerName);
						}
					}

					FDisplayClusterLabelConfiguration LabelArgs;
					{
						LabelArgs.bVisible = false;
						LabelArgs.Scale = *GetLightCardLabelScale();
						LabelArgs.RootActor = ActiveRootActor.Get();
						LabelArgs.LabelFlags = GetLightCardLabelFlags();
					}
					LightCard->ShowLightCardLabel(MoveTemp(LabelArgs));
				}
			}

			if (bDeleteActors)
			{
				WorldToUse = Actor->GetWorld();
				GEditor->SelectActor(Actor.Get(), /*bSelect =*/true, /*bNotifyForActor =*/false, /*bSelectEvenIfHidden =*/true);
			}
		}
	}

	if (bDeleteActors)
	{
		EdSelectionManager->EndBatchSelectOperation();

		if (WorldToUse)
		{
			FEditorDelegates::OnDeleteActorsBegin.Broadcast();
			GEditor->edactDeleteSelected(WorldToUse);
			FEditorDelegates::OnDeleteActorsEnd.Broadcast();
		}
	}

	RefreshPreviewActors(EDisplayClusterLightCardEditorProxyType::StageActor);
}

bool FDisplayClusterLightCardEditor::CanRemoveSelectedActors() const
{
	if (LightCardOutliner.IsValid() && LightCardOutliner->CanDeleteSelectedFolder())
	{
		// If only a folder is selected -- this isn't handled outside of the outliner
		return true;
	}
	
	for (const TWeakObjectPtr<AActor>& Actor : SelectedActors)
	{
		if (Actor.IsValid())
		{
			return true;
		}
	}
	
	return false;
}

bool FDisplayClusterLightCardEditor::CanRemoveSelectedLightCardFromActor() const
{
	for (const TWeakObjectPtr<AActor>& Actor : SelectedActors)
	{
		if (Actor.IsValid() && Actor->IsA<ADisplayClusterLightCardActor>())
		{
			return true;
		}
	}
	
	return false;
}

void FDisplayClusterLightCardEditor::CreateLightCardTemplate()
{
	TArray<ADisplayClusterLightCardActor*> SelectedLightCardActors = GetSelectedActorsAs<ADisplayClusterLightCardActor>();

	check(SelectedLightCardActors.Num() == 1);
	const ADisplayClusterLightCardActor* LightCardActor = SelectedLightCardActors[0];

	const UDisplayClusterLightCardEditorProjectSettings* Settings = GetDefault<UDisplayClusterLightCardEditorProjectSettings>();
	FString DefaultPath = Settings->LightCardTemplateDefaultPath.Path;
	if (DefaultPath.IsEmpty())
	{
		DefaultPath = TEXT("/Game");
	}
	
	FString DefaultAssetName = LightCardActor->GetActorLabel();
	DefaultAssetName.RemoveSpacesInline();
	if (!DefaultAssetName.EndsWith(TEXT("Template")))
	{
		DefaultAssetName += TEXT("Template");
	}

	FString PackageName;
	bool FilenameValid = false;
	while (!FilenameValid)
	{
		FSaveAssetDialogConfig SaveAssetDialogConfig;
		{
			SaveAssetDialogConfig.DefaultPath = MoveTemp(DefaultPath);
			SaveAssetDialogConfig.DefaultAssetName = MoveTemp(DefaultAssetName);
			SaveAssetDialogConfig.AssetClassNames.Add(UDisplayClusterLightCardTemplate::StaticClass()->GetClassPathName());
			SaveAssetDialogConfig.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::AllowButWarn;
			SaveAssetDialogConfig.DialogTitleOverride = LOCTEXT("SaveAssetDialogTitle", "Save Asset As");
		}

		const FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
		const FString SaveObjectPath = ContentBrowserModule.Get().CreateModalSaveAssetDialog(SaveAssetDialogConfig);

		if (SaveObjectPath.IsEmpty())
		{
			return;
		}

		PackageName = FPackageName::ObjectPathToPackageName(SaveObjectPath);
		
		FText OutError;
		FilenameValid = FFileHelper::IsFilenameValidForSaving(PackageName, OutError);
	}

	const FString NewAssetName = FPackageName::GetLongPackageAssetName(PackageName);
	UPackage* AssetPackage = CreatePackage(*PackageName);
	check(AssetPackage);

	const EObjectFlags Flags = RF_Public | RF_Standalone;
	UDisplayClusterLightCardTemplate* Template = NewObject<UDisplayClusterLightCardTemplate>(AssetPackage, FName(*NewAssetName), Flags);
	Template->LightCardActor = CastChecked<ADisplayClusterLightCardActor>(StaticDuplicateObject(LightCardActor, Template));

	FAssetRegistryModule::AssetCreated(Template);

	AssetPackage->MarkPackageDirty();

	FEditorFileUtils::PromptForCheckoutAndSave({AssetPackage}, true, false);
}

bool FDisplayClusterLightCardEditor::CanCreateLightCardTemplate() const
{
	const TArray<ADisplayClusterLightCardActor*> SelectedLightCards = GetSelectedActorsAs<ADisplayClusterLightCardActor>();
	return SelectedLightCards.Num() == 1;
}

void FDisplayClusterLightCardEditor::ToggleLightCardLabels()
{
	ShowLightCardLabels(!ShouldShowLightCardLabels());
}

void FDisplayClusterLightCardEditor::ShowLightCardLabels(bool bVisible)
{
	if (!GetActiveRootActor().IsValid())
	{
		return;
	}
	
	FScopedTransaction Transaction(LOCTEXT("ToggleLightCardLabelsTransactionMessage", "Toggle Light Card Labels"), !GIsTransacting);
	
	IDisplayClusterLightCardEditor& LightCardEditorModule = FModuleManager::GetModuleChecked<IDisplayClusterLightCardEditor>(IDisplayClusterLightCardEditor::ModuleName);

	IDisplayClusterLightCardEditor::FLabelArgs Args;
	Args.bVisible = bVisible;
	Args.LabelFlags = GetLightCardLabelFlags();
	Args.Scale = *GetLightCardLabelScale();
	Args.RootActor = GetActiveRootActor().Get();
	
	LightCardEditorModule.ShowLabels(MoveTemp(Args));

	RefreshPreviewActors(EDisplayClusterLightCardEditorProxyType::StageActor);
}

bool FDisplayClusterLightCardEditor::ShouldShowLightCardLabels() const
{
	return GetDefault<UDisplayClusterLightCardEditorProjectSettings>()->bDisplayLightCardLabels;
}

void FDisplayClusterLightCardEditor::ToggleLightCardLabelFlag(EDisplayClusterLabelFlags InFlag)
{
	SaveLightCardLabelFlags(GetLightCardLabelFlags() ^ InFlag);
}

void FDisplayClusterLightCardEditor::SaveLightCardLabelFlags(EDisplayClusterLabelFlags InFlags)
{
	GetMutableDefault<UDisplayClusterLightCardEditorProjectSettings>()->LightCardLabelFlags = InFlags;
	ShowLightCardLabels(ShouldShowLightCardLabels());
}

bool FDisplayClusterLightCardEditor::LightCardLabelHasFlag(EDisplayClusterLabelFlags FlagToCheck) const
{
	return EnumHasAnyFlags(GetLightCardLabelFlags(), FlagToCheck);
}

bool FDisplayClusterLightCardEditor::LightCardLabelDoesNotHaveFlag(EDisplayClusterLabelFlags FlagToCheck) const
{
	return !LightCardLabelHasFlag(FlagToCheck);
}

EDisplayClusterLabelFlags FDisplayClusterLightCardEditor::GetLightCardLabelFlags() const
{
	return GetDefault<UDisplayClusterLightCardEditorProjectSettings>()->LightCardLabelFlags;
}

TOptional<float> FDisplayClusterLightCardEditor::GetLightCardLabelScale() const
{
	return GetDefault<UDisplayClusterLightCardEditorProjectSettings>()->LightCardLabelScale;
}

void FDisplayClusterLightCardEditor::SetLightCardLabelScale(float NewValue)
{
	if (!GetActiveRootActor().IsValid())
	{
		return;
	}
	
	FScopedTransaction Transaction(LOCTEXT("ScaleLightCardLabelsTransactionMessage", "Scale Light Card Labels"), !GIsTransacting);
	
	IDisplayClusterLightCardEditor& LightCardEditorModule = FModuleManager::GetModuleChecked<IDisplayClusterLightCardEditor>(IDisplayClusterLightCardEditor::ModuleName);

	IDisplayClusterLightCardEditor::FLabelArgs Args;
	Args.bVisible = ShouldShowLightCardLabels();
	Args.Scale = NewValue;
	Args.LabelFlags = GetLightCardLabelFlags();
	Args.RootActor = GetActiveRootActor().Get();
	
	LightCardEditorModule.ShowLabels(MoveTemp(Args));
}

void FDisplayClusterLightCardEditor::ShowIcons(bool bVisible)
{
	UDisplayClusterLightCardEditorSettings* Settings = GetMutableDefault<UDisplayClusterLightCardEditorSettings>();
	Settings->bDisplayIcons = bVisible;
	Settings->PostEditChange();
	Settings->SaveConfig();
}

bool FDisplayClusterLightCardEditor::ShouldShowIcons() const
{
	return GetDefault<UDisplayClusterLightCardEditorSettings>()->bDisplayIcons;
}

TOptional<float> FDisplayClusterLightCardEditor::GetIconScale() const
{
	return GetDefault<UDisplayClusterLightCardEditorSettings>()->IconScale;
}

void FDisplayClusterLightCardEditor::SetIconScale(float NewValue)
{
	UDisplayClusterLightCardEditorSettings* Settings = GetMutableDefault<UDisplayClusterLightCardEditorSettings>();
	Settings->IconScale = NewValue;
	Settings->PostEditChange();
	Settings->SaveConfig();
}

void FDisplayClusterLightCardEditor::OnActiveRootActorChanged(ADisplayClusterRootActor* NewRootActor)
{
	if (NewRootActor == ActiveRootActor.GetEvenIfUnreachable())
	{
		return;
	}
	
	RemoveCompileDelegates();
	
	// The new root actor pointer could be null, indicating that it was deleted or the user didn't select a valid root actor
	ActiveRootActor = NewRootActor;
	if (LightCardOutliner)
	{
		LightCardOutliner->SetRootActor(NewRootActor);
	}
	
	if (ViewportView)
	{
		ViewportView->SetRootActor(NewRootActor);
	}
	
	BindCompileDelegates();

	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);
	FCoreUObjectDelegates::OnObjectPropertyChanged.AddSP(this, &FDisplayClusterLightCardEditor::OnActorPropertyChanged);

	RefreshLabels();
}

void FDisplayClusterLightCardEditor::OnActorsSelectedExternally(const TArray<AActor*>& Actors, bool bShouldSelect)
{
	if (!OperatorViewModel.IsValid())
	{
		return;
	}

	if (bShouldSelect)
	{
		LightCardOutliner->SelectActors(Actors);
	}
	else
	{
		TArray<AActor*> NewActors;
		LightCardOutliner->GetSelectedActors(NewActors);
		
		for (AActor* Actor : Actors)
		{
			NewActors.Remove(Actor);
		}

		LightCardOutliner->SelectActors(NewActors);
	}
}

void FDisplayClusterLightCardEditor::RegisterTabSpawners()
{
	IDisplayClusterOperator::Get().OnRegisterLayoutExtensions().AddStatic(&FDisplayClusterLightCardEditor::RegisterLayoutExtension);

	check(OperatorViewModel.IsValid());
	const TSharedRef<FWorkspaceItem> AppMenuGroup = OperatorViewModel.Pin()->GetWorkspaceMenuGroup().ToSharedRef();

	const TSharedPtr<FTabManager> TabManager = OperatorViewModel.Pin()->GetTabManager();
	
	TabManager->RegisterTabSpawner(ViewportTabName, FOnSpawnTab::CreateSP(this, &FDisplayClusterLightCardEditor::SpawnViewportTab))
	.SetDisplayName(LOCTEXT("ViewportTab_DisplayName", "Viewport"))
	.SetTooltipText(LOCTEXT("ViewportTab_Tooltip", "Light Card Editor Viewport"))
	.SetGroup(AppMenuGroup)
	.SetIcon(FSlateIcon(FAppStyle::Get().GetStyleSetName(), "LevelEditor.Tabs.Viewports"));
	
	TabManager->RegisterTabSpawner(OutlinerTabName, FOnSpawnTab::CreateSP(this, &FDisplayClusterLightCardEditor::SpawnOutlinerTab))
	.SetDisplayName(LOCTEXT("OutlinerTab_DisplayName", "Outliner"))
	.SetTooltipText(LOCTEXT("OutlinerTab_Tooltip", "Light Card Editor Outliner"))
	.SetGroup(AppMenuGroup)
	.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Outliner"));
}

void FDisplayClusterLightCardEditor::UnregisterTabSpawners()
{
	if (OperatorViewModel.IsValid())
	{
		if (const TSharedPtr<FTabManager> OperatorPanelTabManager = OperatorViewModel.Pin()->GetTabManager())
		{
			OperatorPanelTabManager->UnregisterTabSpawner(ViewportTabName);
			OperatorPanelTabManager->UnregisterTabSpawner(OutlinerTabName);
		}
	}
}

void FDisplayClusterLightCardEditor::RegisterLayoutExtension(FLayoutExtender& InExtender)
{
	const FTabManager::FTab ViewportTab(FTabId(ViewportTabName, ETabIdFlags::SaveLayout), ETabState::OpenedTab);
	InExtender.ExtendStack(IDisplayClusterOperator::Get().GetPrimaryOperatorExtensionId(), ELayoutExtensionPosition::After, ViewportTab);
	
	FTabManager::FTab OutlinerTab(FTabId(OutlinerTabName, ETabIdFlags::SaveLayout), ETabState::OpenedTab);
	InExtender.ExtendLayout(IDisplayClusterOperator::Get().GetDetailsTabId(), ELayoutExtensionPosition::Above, OutlinerTab);
}

TSharedRef<SDockTab> FDisplayClusterLightCardEditor::SpawnViewportTab(const FSpawnTabArgs& SpawnTabArgs)
{
	TSharedRef<SWidget> ViewportWidget = CreateViewportWidget();

	if (ViewportView.IsValid())
	{
		if (const TSharedPtr<FEditorViewportClient>& ViewportClient = ViewportView->GetViewportClient())
		{
			if (FEditorModeTools* const ModeTools = ViewportClient->GetModeTools())
			{
				// Opt out of new ITF gizmos and viewport inputs for now
				ModeTools->SetSupportsViewportITF(false);
			}
		}
	}

	return SNew(SDockTab)
	.TabRole(ETabRole::DocumentTab)
	[
		ViewportWidget
	];
}

TSharedRef<SDockTab> FDisplayClusterLightCardEditor::SpawnOutlinerTab(const FSpawnTabArgs& SpawnTabArgs)
{
	return SNew(SDockTab)
	.TabRole(ETabRole::DocumentTab)
	[
		CreateLightCardOutlinerWidget()
	];
}

TSharedRef<SWidget> FDisplayClusterLightCardEditor::CreateLightCardOutlinerWidget()
{
	return SAssignNew(LightCardOutliner, SDisplayClusterLightCardOutliner, SharedThis(this), CommandList);
}

TSharedRef<SWidget> FDisplayClusterLightCardEditor::CreateViewportWidget()
{
	return SAssignNew(ViewportView, SDisplayClusterLightCardEditorViewport, SharedThis(this), CommandList);
}

TSharedRef<SWidget> FDisplayClusterLightCardEditor::GeneratePlaceActorsMenu()
{
	const bool bInShouldCloseWindowAfterMenuSelection = true;

	FMenuBuilder MenuBuilder(bInShouldCloseWindowAfterMenuSelection, CommandList);
	MenuBuilder.BeginSection("PlaceActors", LOCTEXT("PlaceActorsMenuHeader", "Place Actors"));
	{
		FSlateIcon LightCardIcon = FSlateIconFinder::FindIconForClass(ADisplayClusterLightCardActor::StaticClass());
		FSlateIcon FlagIcon = FSlateIconFinder::FindIcon(TEXT("ClassIcon.DisplayClusterLightCardActor.Flag"));
		FSlateIcon UVLightCardIcon = FSlateIconFinder::FindIcon(TEXT("ClassIcon.DisplayClusterLightCardActor.UVLightCard"));

		bool bIsUVMode = false;
		if (ViewportView.IsValid())
		{
			bIsUVMode = ViewportView->GetLightCardEditorViewportClient()->GetProjectionMode() == EDisplayClusterMeshProjectionType::UV;
		}

		MenuBuilder.AddMenuEntry(FDisplayClusterLightCardEditorCommands::Get().AddNewFlag,
			NAME_None, TAttribute<FText>(), TAttribute<FText>(), FlagIcon);
		MenuBuilder.AddMenuEntry(FDisplayClusterLightCardEditorCommands::Get().AddNewLightCard,
			NAME_None, TAttribute<FText>(), TAttribute<FText>(), bIsUVMode ? UVLightCardIcon : LightCardIcon);
		
		TSet<UClass*> StageActorClasses = UE::DisplayClusterLightCardEditorUtils::GetAllStageActorClasses();
		for (UClass* Class : StageActorClasses)
		{
			if (Class == ADisplayClusterLightCardActor::StaticClass())
			{
				// Added manually already
				continue;
			}
			
			FText Label = Class->GetDisplayNameText();
			FSlateIcon StageActorIcon = FSlateIconFinder::FindIconForClass(Class);
			MenuBuilder.AddMenuEntry(Label, LOCTEXT("AddStageActorHeader", "Add a stage actor to the scene"), StageActorIcon,
				FUIAction(FExecuteAction::CreateSPLambda(this, [this, Class]()
				{
					AddNewDynamic(Class);
				}),
				FCanExecuteAction::CreateSP(this, &FDisplayClusterLightCardEditor::CanAddNewActor, Class)));
		}

		if (CanAddNewActor())
		{
			MenuBuilder.AddSubMenu(LOCTEXT("AllTemplatesLabel", "All Templates"),
			LOCTEXT("AllTemplatesTooltip", "Select a template"),
			FNewMenuDelegate::CreateSP(this, &FDisplayClusterLightCardEditor::GenerateTemplateSubMenu),
			false,
			FSlateIcon(FDisplayClusterLightCardEditorStyle::Get().GetStyleSetName(), TEXT("DisplayClusterLightCardEditor.Template")),
			true);
		}
	}
	MenuBuilder.EndSection();
	MenuBuilder.BeginSection("Favorites", LOCTEXT("FavoritesMenuHeader", "Favorites"));
	{
		TArray<UDisplayClusterLightCardTemplate*> LightCardTemplates =
		UE::DisplayClusterLightCardTemplateHelpers::GetLightCardTemplates(/*bFavoritesOnly*/ true);

		TemplateBrushes.Empty();
		
		for (UDisplayClusterLightCardTemplate* Template : LightCardTemplates)
		{
			// Create a brush if this template is using a custom texture.
			if (Template->LightCardActor != nullptr && Template->LightCardActor->Texture.Get() != nullptr)
			{
				TSharedPtr<FSlateBrush> SlateBrush = MakeShared<FSlateBrush>();

				SlateBrush->SetResourceObject(Template->LightCardActor->Texture.Get());
				SlateBrush->ImageSize = FVector2D(16.f, 16.f);
				TemplateBrushes.Add(Template, SlateBrush);
			}

			TWeakObjectPtr<UDisplayClusterLightCardTemplate> TemplateWeakPtr = MakeWeakObjectPtr(Template);

			const TSharedPtr<SWidget> TemplateWidget = SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SBox)
				.HeightOverride(16.f)
				.WidthOverride(16.f)
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(this, &FDisplayClusterLightCardEditor::GetLightCardTemplateIcon, TemplateWeakPtr)
				]
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(4.f, 0.f)
			[
				SNew(STextBlock)
				.Text(FText::FromString(Template->GetName()))
			];

			FMenuEntryParams EntryParams;
			EntryParams.EntryWidget = TemplateWidget;
			EntryParams.LabelOverride = FText::FromString(Template->GetName());
			EntryParams.ToolTipOverride = LOCTEXT("FavoriteEntryTooltip", "Spawn this favorite");
			EntryParams.DirectActions.ExecuteAction = FExecuteAction::CreateLambda([this, TemplateWeakPtr]()
			{
				if (TemplateWeakPtr.IsValid())
				{
					SpawnActor(TemplateWeakPtr.Get());
				}
			});
			EntryParams.DirectActions.CanExecuteAction = FCanExecuteAction::CreateSP(this, &FDisplayClusterLightCardEditor::CanAddNewActor);
			
			MenuBuilder.AddMenuEntry(EntryParams);
		}
	}
	MenuBuilder.EndSection();

	CleanupRecentlyPlacedItems();
	
	MenuBuilder.BeginSection("RecentlyPlaced", LOCTEXT("RecentlyPlacedMenuHeader", "Recently Placed"));
	{
		const UDisplayClusterLightCardEditorSettings* Settings = GetDefault<UDisplayClusterLightCardEditorSettings>();
		for (const FDisplayClusterLightCardEditorRecentItem& RecentlyPlacedItem : Settings->RecentlyPlacedItems)
		{
			TWeakObjectPtr<UObject> ObjectWeakPtr = MakeWeakObjectPtr(RecentlyPlacedItem.ObjectPath.LoadSynchronous());
			check(ObjectWeakPtr.IsValid());
			
			const TSharedPtr<SWidget> RecentItemWidget = SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SBox)
				.HeightOverride(16.f)
				.WidthOverride(16.f)
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(RecentlyPlacedItem.GetSlateBrush())
				]
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(4.f, 0.f)
			[
				SNew(STextBlock)
				.Text(RecentlyPlacedItem.GetItemDisplayName())
			];

			FMenuEntryParams EntryParams;
			EntryParams.EntryWidget = RecentItemWidget;
			EntryParams.LabelOverride = FText::FromString(ObjectWeakPtr->GetName());
			EntryParams.ToolTipOverride = LOCTEXT("RecentItemEntryTooltip", "Spawn this recently placed item");
			EntryParams.DirectActions.ExecuteAction = FExecuteAction::CreateLambda([this, RecentlyPlacedItem, ObjectWeakPtr]()
			{
				if (ObjectWeakPtr.IsValid())
				{
					if (RecentlyPlacedItem.ItemType == FDisplayClusterLightCardEditorRecentItem::Type_LightCard)
					{
						AddNewLightCard();
					}
					else if (RecentlyPlacedItem.ItemType == FDisplayClusterLightCardEditorRecentItem::Type_Flag)
					{
						AddNewFlag();
					}
					else if (RecentlyPlacedItem.ItemType == FDisplayClusterLightCardEditorRecentItem::Type_LightCardTemplate)
					{
						if (const UDisplayClusterLightCardTemplate* Template = Cast<UDisplayClusterLightCardTemplate>(ObjectWeakPtr.Get()))
						{
							SpawnActor(Template);
						}
					}
					else if (RecentlyPlacedItem.ItemType == FDisplayClusterLightCardEditorRecentItem::Type_Dynamic)
					{
						AddNewDynamic(Cast<UClass>(ObjectWeakPtr.Get()));
					}
				}
			});
			EntryParams.DirectActions.CanExecuteAction = FCanExecuteAction::CreateSP(this, &FDisplayClusterLightCardEditor::CanAddNewActor, Cast<UClass>(ObjectWeakPtr.Get()));
			
			MenuBuilder.AddMenuEntry(EntryParams);
		}
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

const FSlateBrush* FDisplayClusterLightCardEditor::GetLightCardTemplateIcon(const TWeakObjectPtr<UDisplayClusterLightCardTemplate> InTemplate) const
{
	if (!InTemplate.IsValid())
	{
		return nullptr;
	}
		
	if (const TSharedPtr<FSlateBrush>* Brush = TemplateBrushes.Find(InTemplate))
	{
		return (*Brush).Get();
	}
	
	return FSlateIconFinder::FindIconBrushForClass(InTemplate->GetClass());
}

void FDisplayClusterLightCardEditor::GenerateTemplateSubMenu(FMenuBuilder& InMenuBuilder)
{
	InMenuBuilder.BeginSection("Templates", LOCTEXT("TemplatesMenuHeader", "Templates"));
	{
		InMenuBuilder.AddWidget(CreateLightCardTemplateWidget(), FText::GetEmpty(), true);
	}
	InMenuBuilder.EndSection();
}

TSharedRef<SWidget> FDisplayClusterLightCardEditor::GenerateLabelsMenu()
{
	const bool bInShouldCloseWindowAfterMenuSelection = true;

	FMenuBuilder MenuBuilder(bInShouldCloseWindowAfterMenuSelection, CommandList);

	MenuBuilder.BeginSection("Labels", LOCTEXT("LabelsMenuHeader", "Labels"));
	{
		MenuBuilder.AddMenuEntry(FDisplayClusterLightCardEditorCommands::Get().ToggleLightCardLabels);
		MenuBuilder.AddMenuEntry(FDisplayClusterLightCardEditorCommands::Get().ToggleLabelsVisibleInEditor);
		MenuBuilder.AddMenuEntry(FDisplayClusterLightCardEditorCommands::Get().ToggleLabelsHiddenInGame);
		
		MenuBuilder.AddWidget(
		SNew(SBox)
		.HAlign(HAlign_Right)
		[
			SNew(SBox)
			.Padding(FMargin(8.0f, 0.0f, 0.0f, 0.0f))
			.WidthOverride(100.0f)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("Menu.WidgetBorder"))
				.Padding(FMargin(1.0f))
				[
					SNew(SBox)
					.MinDesiredWidth(64)
					[
						SNew(SNumericEntryBox<float>)
						.Value(this, &FDisplayClusterLightCardEditor::GetLightCardLabelScale)
						.OnValueChanged(this, &FDisplayClusterLightCardEditor::SetLightCardLabelScale)
						.MinValue(0)
						.MaxValue(FLT_MAX)
						.MinSliderValue(0)
						.MaxSliderValue(10)
						.AllowSpin(true)
					]
				]
			]
		],
		LOCTEXT("LightCardLabelScale_Label", "Label Scale"));
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Icons", LOCTEXT("IconsMenuHeader", "Icons"));
	{
		MenuBuilder.AddMenuEntry(FDisplayClusterLightCardEditorCommands::Get().ToggleIconVisibility,
			NAME_None, LOCTEXT("IconVisibilityLabel", "Actor Icons"));
	
		MenuBuilder.AddWidget(
		SNew(SBox)
		.HAlign(HAlign_Right)
		[
			SNew(SBox)
			.Padding(FMargin(8.0f, 0.0f, 0.0f, 0.0f))
			.WidthOverride(100.0f)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("Menu.WidgetBorder"))
				.Padding(FMargin(1.0f))
				[
					SNew(SBox)
					.MinDesiredWidth(64)
					[
						SNew(SNumericEntryBox<float>)
						.Value(this, &FDisplayClusterLightCardEditor::GetIconScale)
						.OnValueChanged(this, &FDisplayClusterLightCardEditor::SetIconScale)
						.MinValue(0)
						.MaxValue(FLT_MAX)
						.MinSliderValue(0)
						.MaxSliderValue(4)
						.AllowSpin(true)
					]
				]
			]
		],
		LOCTEXT("LightCardIconScale_Label", "Icon Scale"));
	}
	MenuBuilder.EndSection();
	
	return MenuBuilder.MakeWidget();
}

namespace UE::LightCardEditorFrustumTypes
{
	static FString UnderFrustumString = TEXT("Under Frustum");
	static FString OverFrustumString = TEXT("Over Frustum");
}

/**
 * A single widget representing a frustum selection
 */
class SFrustumItemWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SFrustumItemWidget) {}
	SLATE_ATTRIBUTE(FString, Selection)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		static FText UnderFrustumText = LOCTEXT("UnderFrustumText", "Under Frustum");
		static FText OverFrustumText = LOCTEXT("OverFrustumText", "Over Frustum");
		static FText NoFrustumText = LOCTEXT("NoBlendModeSelectedText", "Blend Mode");
		
		Selection = InArgs._Selection;
		ChildSlot
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image_Lambda([this]()
				{
					const FString CurrentSelection = Selection.Get();
					const FString BrushName = CurrentSelection == UE::LightCardEditorFrustumTypes::OverFrustumString ?
						"DisplayClusterLightCardEditor.FrustumOnTop" : "DisplayClusterLightCardEditor.FrustumUnderneath";
					return FDisplayClusterLightCardEditorStyle::Get().GetBrush(*BrushName);
				})
			]
			+SHorizontalBox::Slot()
			.Padding(4.f, 0.f, 0.f, 0.f)
			[
				SNew(SBox)
				.MinDesiredWidth(90.f) // Keeps the widget from resizing on different selections
				[
					SNew(STextBlock).Text_Lambda([this]()
					{
						const FString CurrentSelection = Selection.Get();
						const FText Text = CurrentSelection == UE::LightCardEditorFrustumTypes::OverFrustumString ? OverFrustumText :
						CurrentSelection == UE::LightCardEditorFrustumTypes::UnderFrustumString ? UnderFrustumText : NoFrustumText;
						return Text;
					})
				]
			]
		];
	}

private:
	TAttribute<FString> Selection;
};

TSharedRef<SWidget> FDisplayClusterLightCardEditor::CreateFrustumWidget()
{
	FrustumSelections.Add(MakeShared<FString>(UE::LightCardEditorFrustumTypes::OverFrustumString));
	FrustumSelections.Add(MakeShared<FString>(UE::LightCardEditorFrustumTypes::UnderFrustumString));

	auto GetFrustumStringFromRootActor = [this]()
	{
		if (ActiveRootActor.IsValid())
		{
			switch (ActiveRootActor->GetStageSettings().Lightcard.Blendingmode)
			{
			case EDisplayClusterConfigurationICVFX_LightcardRenderMode::Over:
				{
					return UE::LightCardEditorFrustumTypes::OverFrustumString;
				}
			case EDisplayClusterConfigurationICVFX_LightcardRenderMode::Under:
				{
					return UE::LightCardEditorFrustumTypes::UnderFrustumString;
				}
			}
		}

		return FString();
	};

	return SNew(SComboBox<TSharedPtr<FString>>)
		.OptionsSource(&FrustumSelections)
		.ToolTipText(LOCTEXT("BlendModeTooltip", "Blending Mode: Specify how to render Light Cards in relation to the inner frustum."))
		.IsEnabled_Lambda([this]()
		{
			return ActiveRootActor.IsValid();
		})
		.OnSelectionChanged_Lambda([this](TSharedPtr<FString> InItem, ESelectInfo::Type)
		{
			if (ActiveRootActor.IsValid())
			{
				checkSlow(InItem.IsValid());

				UDisplayClusterConfigurationData* ConfigData = ActiveRootActor->GetConfigData();
				check(ConfigData);

				FScopedTransaction Transaction(LOCTEXT("SetBlendModeTransactionMessage", "Set Light Card Blend Mode"), !GIsTransacting);
				ActiveRootActor->Modify();
				ConfigData->Modify();
				if (*InItem == UE::LightCardEditorFrustumTypes::UnderFrustumString)
				{
					ConfigData->StageSettings.Lightcard.Blendingmode = EDisplayClusterConfigurationICVFX_LightcardRenderMode::Under;
				}
				else if (*InItem == UE::LightCardEditorFrustumTypes::OverFrustumString)
				{
					ConfigData->StageSettings.Lightcard.Blendingmode = EDisplayClusterConfigurationICVFX_LightcardRenderMode::Over;
				}
			}
		})
		.OnGenerateWidget_Lambda([](TSharedPtr<FString> InItem)
		{
			return SNew(SFrustumItemWidget).Selection(*InItem.Get());
		})
		.Content()
		[
			SNew(SFrustumItemWidget)
			.Selection_Lambda(GetFrustumStringFromRootActor)
		];
}

TSharedRef<SWidget> FDisplayClusterLightCardEditor::CreateLightCardTemplateWidget()
{
	return SNew(SDisplayClusterLightCardTemplateList, SharedThis(this))
	.HideHeader(true)
	.SpawnOnSelection(true);
}

void FDisplayClusterLightCardEditor::BindCommands()
{
	CommandList = MakeShareable(new FUICommandList);

	CommandList->MapAction(
		FDisplayClusterLightCardEditorCommands::Get().AddNewLightCard,
		FExecuteAction::CreateSPLambda(this, [this]()
		{
			AddNewLightCard();
		}),
		FCanExecuteAction::CreateSP(this, &FDisplayClusterLightCardEditor::CanAddNewActor));

	CommandList->MapAction(
		FDisplayClusterLightCardEditorCommands::Get().AddNewFlag,
		FExecuteAction::CreateSPLambda(this, [this]()
		{
			AddNewFlag();
		}),
		FCanExecuteAction::CreateSP(this, &FDisplayClusterLightCardEditor::CanAddNewActor));
	
	CommandList->MapAction(
		FDisplayClusterLightCardEditorCommands::Get().AddExistingLightCard,
		FExecuteAction::CreateSP(this, &FDisplayClusterLightCardEditor::AddExistingLightCard),
		FCanExecuteAction::CreateSP(this, &FDisplayClusterLightCardEditor::CanAddNewActor));

	CommandList->MapAction(
		FDisplayClusterLightCardEditorCommands::Get().RemoveLightCard,
		FExecuteAction::CreateSP(this, &FDisplayClusterLightCardEditor::RemoveSelectedActors, false),
		FCanExecuteAction::CreateSP(this, &FDisplayClusterLightCardEditor::CanRemoveSelectedLightCardFromActor));

	CommandList->MapAction(
		FGenericCommands::Get().Cut,
		FExecuteAction::CreateSP(this, &FDisplayClusterLightCardEditor::CutSelectedActors),
		FCanExecuteAction::CreateSP(this, &FDisplayClusterLightCardEditor::CanCutSelectedActors));

	CommandList->MapAction(
		FGenericCommands::Get().Copy,
		FExecuteAction::CreateSP(this, &FDisplayClusterLightCardEditor::CopySelectedActors, false),
		FCanExecuteAction::CreateSP(this, &FDisplayClusterLightCardEditor::CanCopySelectedActors));

	CommandList->MapAction(
		FGenericCommands::Get().Paste,
		FExecuteAction::CreateSPLambda(this, [this]()
		{
			PasteActors();
		}),
		FCanExecuteAction::CreateSP(this, &FDisplayClusterLightCardEditor::CanPasteActors));

	CommandList->MapAction(
		FGenericCommands::Get().Duplicate,
		FExecuteAction::CreateSP(this, &FDisplayClusterLightCardEditor::DuplicateSelectedActors),
		FCanExecuteAction::CreateSP(this, &FDisplayClusterLightCardEditor::CanDuplicateSelectedActors));

	CommandList->MapAction(
		FGenericCommands::Get().Rename,
		FExecuteAction::CreateSP(this, &FDisplayClusterLightCardEditor::RenameSelectedItem),
		FCanExecuteAction::CreateSP(this, &FDisplayClusterLightCardEditor::CanRenameSelectedItem));
	
	CommandList->MapAction(
		FGenericCommands::Get().Delete,
		FExecuteAction::CreateSP(this, &FDisplayClusterLightCardEditor::RemoveSelectedActors, true),
		FCanExecuteAction::CreateSP(this, &FDisplayClusterLightCardEditor::CanRemoveSelectedActors));

	CommandList->MapAction(
		FDisplayClusterLightCardEditorCommands::Get().SaveLightCardTemplate,
		FExecuteAction::CreateSP(this, &FDisplayClusterLightCardEditor::CreateLightCardTemplate),
		FCanExecuteAction::CreateSP(this, &FDisplayClusterLightCardEditor::CanCreateLightCardTemplate));

	CommandList->MapAction(
		FDisplayClusterLightCardEditorCommands::Get().ToggleLightCardLabels,
		FExecuteAction::CreateSP(this, &FDisplayClusterLightCardEditor::ToggleLightCardLabels),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FDisplayClusterLightCardEditor::ShouldShowLightCardLabels));

	CommandList->MapAction(
		FDisplayClusterLightCardEditorCommands::Get().ToggleLabelsVisibleInEditor,
		FExecuteAction::CreateSP(this, &FDisplayClusterLightCardEditor::ToggleLightCardLabelFlag, EDisplayClusterLabelFlags::DisplayInEditor),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FDisplayClusterLightCardEditor::LightCardLabelHasFlag, EDisplayClusterLabelFlags::DisplayInEditor));

	CommandList->MapAction(
		FDisplayClusterLightCardEditorCommands::Get().ToggleLabelsHiddenInGame,
		FExecuteAction::CreateSP(this, &FDisplayClusterLightCardEditor::ToggleLightCardLabelFlag, EDisplayClusterLabelFlags::DisplayInGame),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FDisplayClusterLightCardEditor::LightCardLabelDoesNotHaveFlag, EDisplayClusterLabelFlags::DisplayInGame));
}

void FDisplayClusterLightCardEditor::RegisterToolbarExtensions()
{
	const TSharedPtr<FExtender> ToolbarExtender = MakeShareable(new FExtender);
	ToolbarExtender->AddToolBarExtension("General", EExtensionHook::After, nullptr,
		FToolBarExtensionDelegate::CreateSP(this, &FDisplayClusterLightCardEditor::ExtendToolbar));
	
	IDisplayClusterOperator::Get().GetOperatorToolBarExtensibilityManager()->AddExtender(ToolbarExtender);
}

void FDisplayClusterLightCardEditor::RegisterMenuExtensions()
{
	const TSharedPtr<FExtender> ToolbarExtender = MakeShareable(new FExtender);
	ToolbarExtender->AddMenuExtension("FileOpen", EExtensionHook::Before, CommandList,
		FMenuExtensionDelegate::CreateSP(this, &FDisplayClusterLightCardEditor::ExtendFileMenu));

	ToolbarExtender->AddMenuExtension("EditHistory", EExtensionHook::After, CommandList,
	FMenuExtensionDelegate::CreateSP(this, &FDisplayClusterLightCardEditor::ExtendEditMenu));
	
	IDisplayClusterOperator::Get().GetOperatorMenuExtensibilityManager()->AddExtender(ToolbarExtender);
}

void FDisplayClusterLightCardEditor::ExtendToolbar(FToolBarBuilder& ToolbarBuilder)
{
	ToolbarBuilder.AddSeparator();
	{
		const FUIAction DefaultAction;
		ToolbarBuilder.AddComboButton(
			DefaultAction,
			FOnGetContent::CreateSP(this, &FDisplayClusterLightCardEditor::GeneratePlaceActorsMenu),
			TAttribute<FText>(),
			LOCTEXT("PlaceActors_ToolTip", "Place actors in the scene"),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "LevelEditor.OpenAddContent.Background", NAME_None, "LevelEditor.OpenAddContent.Overlay"));
	}
	
	ToolbarBuilder.AddSeparator();
	{
		const FUIAction DefaultAction;
		ToolbarBuilder.AddComboButton(
			DefaultAction,
			FOnGetContent::CreateSP(this, &FDisplayClusterLightCardEditor::GenerateLabelsMenu),
			TAttribute<FText>(),
			LOCTEXT("Labels_ToolTip", "Configure options for labels"),
			FSlateIcon(FDisplayClusterLightCardEditorStyle::Get().GetStyleSetName(), "DisplayClusterLightCardEditor.Labels"));
		
		ToolbarBuilder.AddWidget(CreateFrustumWidget());
	}
	
}

void FDisplayClusterLightCardEditor::ExtendFileMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("Import", LOCTEXT("ImportHeading", "Import"));
	const FSlateIcon LightCardIcon = FSlateIconFinder::FindIconForClass(ADisplayClusterLightCardActor::StaticClass());
	MenuBuilder.AddMenuEntry(FDisplayClusterLightCardEditorCommands::Get().AddExistingLightCard,
		NAME_None, TAttribute<FText>(), TAttribute<FText>(), LightCardIcon);
	MenuBuilder.EndSection();
}

void FDisplayClusterLightCardEditor::ExtendEditMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("Edit", LOCTEXT("EditHeading", "Edit"));
	MenuBuilder.AddMenuEntry(FGenericCommands::Get().Cut);
	MenuBuilder.AddMenuEntry(FGenericCommands::Get().Copy);
	MenuBuilder.AddMenuEntry(FGenericCommands::Get().Paste);
	MenuBuilder.AddMenuEntry(FGenericCommands::Get().Duplicate);
	MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete);
	MenuBuilder.EndSection();
}

void FDisplayClusterLightCardEditor::RefreshPreviewActors(EDisplayClusterLightCardEditorProxyType ProxyType)
{
	RemoveCompileDelegates();
	
	if (ADisplayClusterRootActor* RootActor = GetActiveRootActor().Get())
	{
		if (LightCardOutliner.IsValid())
		{
			LightCardOutliner->SetRootActor(RootActor);
		}
			
		if (ViewportView.IsValid())
		{
			const bool bForce = true;
			ViewportView->GetLightCardEditorViewportClient()->UpdatePreviewActor(RootActor, bForce, ProxyType);
		}
	}
	
	BindCompileDelegates();
}

void FDisplayClusterLightCardEditor::RefreshPreviewStageActor(AActor* Actor)
{
	if (ADisplayClusterRootActor* RootActor = GetActiveRootActor().Get())
	{
		if (LightCardOutliner.IsValid())
		{
			LightCardOutliner->SetRootActor(RootActor);
		}

		if (ViewportView.IsValid())
		{
			ViewportView->GetLightCardEditorViewportClient()->UpdatePreviewActor(RootActor, /*bForce*/ true,
				EDisplayClusterLightCardEditorProxyType::StageActor, Actor);
		}
	}
}

void FDisplayClusterLightCardEditor::RefreshLabels()
{
	ShowLightCardLabels(ShouldShowLightCardLabels());
}

bool FDisplayClusterLightCardEditor::IsOurObject(UObject* InObject,
	EDisplayClusterLightCardEditorProxyType& OutProxyType) const
{
	auto IsOurActor = [InObject] (UObject* ObjectToCompare) -> bool
	{
		if (ObjectToCompare)
		{
			if (InObject == ObjectToCompare)
			{
				return true;
			}

			if (const UObject* RootActorOuter = InObject->GetTypedOuter(ObjectToCompare->GetClass()))
			{
				return RootActorOuter == ObjectToCompare;
			}
		}

		return false;
	};

	EDisplayClusterLightCardEditorProxyType ProxyType = EDisplayClusterLightCardEditorProxyType::All;
	
	bool bIsOurActor = IsOurActor(GetActiveRootActor().Get());
	if (!bIsOurActor)
	{
		if (LightCardOutliner.IsValid())
		{
			for (const TSharedPtr<SDisplayClusterLightCardOutliner::FStageActorTreeItem>& LightCard : LightCardOutliner->GetStageActorTreeItems())
			{
				if (LightCard.IsValid())
				{
					bIsOurActor = IsOurActor(LightCard->Actor.Get());
					if (bIsOurActor)
					{
						break;
					}
				}
			}
		}

		// Fallback to generic check
		if (!bIsOurActor && UE::DisplayClusterLightCardEditorUtils::IsManagedActor(Cast<AActor>(InObject)))
		{
			bIsOurActor = true;
		}

		if (bIsOurActor)
		{
			ProxyType = EDisplayClusterLightCardEditorProxyType::StageActor;
		}
	}

	OutProxyType = ProxyType;
	return bIsOurActor;
}

void FDisplayClusterLightCardEditor::BindCompileDelegates()
{
	if (LightCardOutliner.IsValid())
	{
		for (const TSharedPtr<SDisplayClusterLightCardOutliner::FStageActorTreeItem>& LightCardActor : LightCardOutliner->GetStageActorTreeItems())
		{
			if (LightCardActor.IsValid() && LightCardActor->Actor.IsValid())
			{
				if (UBlueprint* Blueprint = UBlueprint::GetBlueprintFromClass(LightCardActor->Actor->GetClass()))
				{
					Blueprint->OnCompiled().AddSP(this, &FDisplayClusterLightCardEditor::OnBlueprintCompiled);
				}
			}
		}
	}
}

void FDisplayClusterLightCardEditor::RemoveCompileDelegates()
{
	if (LightCardOutliner.IsValid())
	{
		for (const TSharedPtr<SDisplayClusterLightCardOutliner::FStageActorTreeItem>& LightCardActor : LightCardOutliner->GetStageActorTreeItems())
		{
			if (LightCardActor.IsValid() && LightCardActor->Actor.IsValid())
			{
				if (UBlueprint* Blueprint = UBlueprint::GetBlueprintFromClass(LightCardActor->Actor->GetClass()))
				{
					Blueprint->OnCompiled().RemoveAll(this);
				}
			}
		}
	}
}

void FDisplayClusterLightCardEditor::OnActorPropertyChanged(UObject* ObjectBeingModified,
                                                            FPropertyChangedEvent& PropertyChangedEvent)
{
	EDisplayClusterLightCardEditorProxyType ProxyType;
	// When a level instance actor property is changed we want to update our proxy version here. However,
	// sometimes the viewport client is responsible for the property change call and in that case we want to avoid
	// double processing an update.
	if (IsOurObject(ObjectBeingModified, ProxyType) && (!ViewportView.IsValid() ||
		!ViewportView->GetLightCardEditorViewportClient()->HasCalledPostEditChangeProperty()))
	{
		IDisplayClusterStageActor* StageActor = Cast<IDisplayClusterStageActor>(ObjectBeingModified);
		
		const FName PropertyName = PropertyChangedEvent.GetPropertyName();
		const bool bComponentTransformPropertyChanged = PropertyName == USceneComponent::GetRelativeLocationPropertyName()
		|| PropertyName == USceneComponent::GetRelativeRotationPropertyName()
		|| PropertyName == USceneComponent::GetRelativeScale3DPropertyName();
		
		const bool bTransformationChanged = (bComponentTransformPropertyChanged
				|| (StageActor && StageActor->GetPositionalPropertyNames().Contains(PropertyName)));
		
		if (PropertyChangedEvent.ChangeType == EPropertyChangeType::Interactive ||
			(bTransformationChanged && !ObjectBeingModified->IsA<ADisplayClusterRootActor>()))
		{
			if (bComponentTransformPropertyChanged && ObjectBeingModified->IsA<USceneComponent>())
			{
				// The owning scene component actor will also fire a property change. Prevent double processing.
				return;
			}
			
			// Real-time & efficient update when dragging a slider or when we know the property type doesn't
			// require a full refresh
			if (ViewportView.IsValid())
			{
				if (AActor* Actor = Cast<AActor>(ObjectBeingModified))
				{
					ViewportView->GetLightCardEditorViewportClient()->UpdateProxyTransformFromLevelInstance(Actor);
				}
				else
				{
					ViewportView->GetLightCardEditorViewportClient()->UpdateProxyTransforms();
				}
			}
		}
		else
		{
			if (StageActor == nullptr)
			{
				// Check if this object belongs to a stage actor
				StageActor = Cast<IDisplayClusterStageActor>(ObjectBeingModified->GetTypedOuter(UDisplayClusterStageActor::StaticClass()));
			}
			
			if (AActor* Actor = Cast<AActor>(StageActor))
			{
				// Full destroy and refresh of a single proxy
				RefreshPreviewStageActor(Actor);
			}
			else if (ObjectBeingModified->IsA<UActorComponent>() &&
				CastChecked<UActorComponent>(ObjectBeingModified)->GetOwner() == GetActiveRootActor().Get())
			{
				// Avoid recreating all stage proxies if we can determine it's just a component that has changed
				// Especially helps improve ICVFX property changes
				RefreshPreviewActors(EDisplayClusterLightCardEditorProxyType::RootActor);
			}
			else
			{
				// Full destroy and refresh of all relevant proxies
				RefreshPreviewActors(ProxyType);
			}
		}
	}
}

void FDisplayClusterLightCardEditor::OnLevelActorAdded(AActor* Actor)
{
	if (Actor && UE::DisplayClusterLightCardEditorUtils::IsManagedActor(Actor))
	{
		const UPackage* Package = Actor->GetPackage();
		if (Package && !Package->HasAnyFlags(RF_Transient) /** Snapshots can create a temporary preview world which we need to avoid */)
		{
			RefreshPreviewStageActor(Actor);
		}
	}
}

void FDisplayClusterLightCardEditor::OnLevelActorDeleted(AActor* Actor)
{
	if (Actor && UE::DisplayClusterLightCardEditorUtils::IsManagedActor(Actor) &&
		Actor->GetPackage() && !Actor->GetPackage()->HasAnyFlags(RF_Transient) /* Don't trigger if this is a proxy actor being destroyed */)
	{
		if (Actor->GetClass()->HasAnyClassFlags(CLASS_NewerVersionExists))
		{
			// When a blueprint class is regenerated instances are deleted and replaced.
			// In this case the OnCompiled() delegate will fire and refresh the actor.
			return;
		}
		
		if (Actor->GetWorld())
		{
			Actor->GetWorld()->GetTimerManager().SetTimerForNextTick([this]()
			{
				// Schedule for next tick so available selections are properly updated once the
				// actor is fully deleted.
				RefreshPreviewActors(EDisplayClusterLightCardEditorProxyType::StageActor);
			});
		}
	}
}

void FDisplayClusterLightCardEditor::OnBlueprintCompiled(UBlueprint* Blueprint)
{
	// Right now only LightCard blueprints are handled here.
	RefreshPreviewActors(EDisplayClusterLightCardEditorProxyType::StageActor);
}

void FDisplayClusterLightCardEditor::OnObjectTransacted(UObject* Object,
	const FTransactionObjectEvent& TransactionObjectEvent)
{
	if (TransactionObjectEvent.GetEventType() == ETransactionObjectEventType::UndoRedo)
	{
		// Always refresh on undo because the light card actor may not inherit our C++ class
		// so we can't easily distinguish it. This supports the case where the user deletes
		// a LightCard actor from the level manually then undoes it.
		RefreshPreviewActors();
	}
	else if (AActor* Actor = Cast<AActor>(Object))
	{
		if (TransactionObjectEvent.GetEventType() == ETransactionObjectEventType::Finalized &&
			TransactionObjectEvent.HasPropertyChanges() && ViewportView.IsValid() && GEditor && GEditor->Trans &&
			UE::DisplayClusterLightCardEditorUtils::IsManagedActor(Actor))
		{
			// Look for level snapshots restore transactions. Snapshots doesn't fire PostEditChange events so we need
			// to listen for the transaction instead.
			
			const int32 TransactionIndex = GEditor->Trans->FindTransactionIndex(TransactionObjectEvent.GetTransactionId());
			if (const FTransaction* Transaction = GEditor->Trans->GetTransaction(TransactionIndex))
			{
				const TOptional<FString> Namespace = FTextInspector::GetNamespace(Transaction->GetTitle());
				if (Namespace.IsSet() && *Namespace == TEXT("LevelSnapshotsEditor"))
				{
					const TOptional<FString> Key = FTextInspector::GetKey(Transaction->GetTitle());
					if (Key.IsSet() && *Key == TEXT("ApplyToWorldKey"))
					{
						ViewportView->GetLightCardEditorViewportClient()->UpdateProxyTransformFromLevelInstance(Actor);
					}
				}
			}
		}
	}
}

void FDisplayClusterLightCardEditor::AddRecentlyPlacedItem(const FDisplayClusterLightCardEditorRecentItem& InItem)
{
	UDisplayClusterLightCardEditorSettings* Settings = GetMutableDefault<UDisplayClusterLightCardEditorSettings>();
	
	Settings->RecentlyPlacedItems.RemoveAll([&](const FDisplayClusterLightCardEditorRecentItem& Compare)
	{
		return Compare == InItem;
	});
	Settings->RecentlyPlacedItems.Insert(InItem, 0);

	const int32 MaxRecentlyPlaced = 5;
	
	if (Settings->RecentlyPlacedItems.Num() > MaxRecentlyPlaced)
	{
		Settings->RecentlyPlacedItems.RemoveAt(MaxRecentlyPlaced - 1, Settings->RecentlyPlacedItems.Num() - MaxRecentlyPlaced);
	}

	Settings->PostEditChange();
	Settings->SaveConfig();
}

void FDisplayClusterLightCardEditor::CleanupRecentlyPlacedItems()
{
	UDisplayClusterLightCardEditorSettings* Settings = GetMutableDefault<UDisplayClusterLightCardEditorSettings>();
	Settings->RecentlyPlacedItems.RemoveAll([&](const FDisplayClusterLightCardEditorRecentItem& Compare)
	{
		return !Compare.ObjectPath.LoadSynchronous();
	});

	Settings->PostEditChange();
	Settings->SaveConfig();
}

bool FDisplayClusterLightCardEditor::DoOutlinerFoldersNeedEditorDelegates() const
{
	return LightCardOutliner.IsValid() && SelectedActors.Num() == 0;
}

#undef LOCTEXT_NAMESPACE
