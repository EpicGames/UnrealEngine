// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Actor.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

#include "DisplayClusterLightCardEditorProxyType.h"

#include "EditorUndoClient.h"
#include "IDisplayClusterOperatorApp.h"

enum class EDisplayClusterLabelFlags : uint8;
class FLayoutExtender;
class FMenuBuilder;
class FSpawnTabArgs;
class FTabManager;
class FToolBarBuilder;
class FUICommandList;
class SDockTab;
class SDisplayClusterLightCardOutliner;
class SDisplayClusterLightCardTemplateList;
class SDisplayClusterLightCardEditorViewport;
class ADisplayClusterRootActor;
class ADisplayClusterLightCardActor;
class UDisplayClusterLightCardTemplate;
class IDisplayClusterOperatorViewModel;
struct FSlateBrush;

struct FDisplayClusterLightCardEditorRecentItem;

/** A panel that can be spawned in a tab that contains all the UI elements that make up the 2D light cards editor */
class FDisplayClusterLightCardEditor : public IDisplayClusterOperatorApp, public FEditorUndoClient
{
public:
	/** The name of the tab that the viewport lives in */
	static const FName ViewportTabName;
	
	/** The name of the tab that the outliner lives in */
	static const FName OutlinerTabName;

	/** Construct an instance of the light card editor */
	static TSharedRef<IDisplayClusterOperatorApp> MakeInstance(TSharedRef<IDisplayClusterOperatorViewModel> InViewModel);
	
	~FDisplayClusterLightCardEditor();

	// FEditorUndoClient
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
	// ~FEditorUndoClient

	/** Initialize the light card editor instance */
	void Initialize(TSharedRef<IDisplayClusterOperatorViewModel> InViewModel);

	/** Return the command list for the light card editor */
	TSharedPtr<FUICommandList> GetCommandList() const { return CommandList; }
	
	/** The current active root actor for this light card editor */
	TWeakObjectPtr<ADisplayClusterRootActor> GetActiveRootActor() const { return ActiveRootActor; }

	/** Iterate the world actors looking for all actors this editor is managing */
	TArray<AActor*> FindAllManagedActors() const;

	/** Selects the specified actors in the outliner and details panel */
	void SelectActors(const TArray<AActor*>& ActorsToSelect);

	template<typename T>
	TArray<T*> GetSelectedActorsAs() const
	{
		TArray<T*> OutArray;
		Algo::TransformIf(SelectedActors, OutArray, [](const TWeakObjectPtr<AActor>& InItem)
		{
			return InItem.IsValid() && InItem->IsA<T>();
		},
		[](const TWeakObjectPtr<AActor>& InItem)
		{
			return CastChecked<T>(InItem.Get());
		});
		return OutArray;
	}
	
	/** Gets the actors that are selected in the outliner */
	void GetSelectedActors(TArray<AActor*>& OutSelectedActors);

	/** Selects the actor proxies that correspond to the specified actors */
	void SelectActorProxies(const TArray<AActor*>& ActorsToSelect);

	/** Places the given light card in the middle of the viewport */
	void CenterActorInView(AActor* Actor);

	/** Spawns a new actor and adds it to the root actor if it is a light card */
	AActor* SpawnActor(TSubclassOf<AActor> InActorClass, const FName& InActorName = NAME_None,
		const UDisplayClusterLightCardTemplate* InTemplate = nullptr, ULevel* InLevel = nullptr, bool bIsPreview = false);

	/** Spawn an actor from a template */
	AActor* SpawnActor(const UDisplayClusterLightCardTemplate* InTemplate, ULevel* InLevel = nullptr, bool bIsPreview = false);

	template<typename T>
	T* SpawnActorAs(const FName& InActorName = NAME_None, const UDisplayClusterLightCardTemplate* InTemplate = nullptr)
	{
		return Cast<T>(SpawnActor(T::StaticClass(), InActorName, InTemplate));
	}
	
	/** Adds a new light card to the root actor and centers it in the viewport */
	ADisplayClusterLightCardActor* AddNewLightCard();

	/** Select an existing Light Card from a menu */
	void AddExistingLightCard();

	/** Adds a new light card configured as a flag */
	ADisplayClusterLightCardActor* AddNewFlag();

	/** Add a new actor dynamically based on a class */
	AActor* AddNewDynamic(UClass* InClass);

	/** Adds the given Light Card to the root actor */
	void AddLightCardsToActor(const TArray<ADisplayClusterLightCardActor*>& LightCards);

	/** If a Light Card can currently be added */
	bool CanAddNewActor() const;
	bool CanAddNewActor(UClass* InClass) const;

	/** Copies any selected actors to the clipboard, and then deletes them */
	void CutSelectedActors();

	/** Determines if there are selected actors that can be cut */
	bool CanCutSelectedActors();

	/** Copies any selected actors to the clipboard */
	void CopySelectedActors(bool bShouldCut = false);

	/** Determines if there are selected actors that can be copied */
	bool CanCopySelectedActors() const;

	/** Pastes any actors in the clipboard to the current root actor */
	TArray<AActor*> PasteActors();

	/** Determines if there are any actors that can be pasted from the clipboard */
	bool CanPasteActors() const;

	/** Copies any selected actors to the clipboard and then pastes them */
	void DuplicateSelectedActors();

	/** Determines if there are selected actors that can be duplicated */
	bool CanDuplicateSelectedActors() const;

	/** Rename the selected item. Requires the outliner */
	void RenameSelectedItem();

	/** If the selected item can be renamed */
	bool CanRenameSelectedItem() const;
	
	/**
	 * Remove the light card from the actor
	 *@param bDeleteLightCardActor Delete the actor from the level
	 */
	void RemoveSelectedActors(bool bDeleteLightCardActor);

	/**
	 * Remove the given light cards from the actor
	 * @param InActorsToRemove Actors which should be removed
	 * @param bDeleteActors If the light cards should be deleted from the level
	 */
	void RemoveActors(const TArray<TWeakObjectPtr<AActor>>& InActorsToRemove, bool bDeleteActors);
	
	/** If the selected actors can be removed */
	bool CanRemoveSelectedActors() const;

	/** If the selected light card can be removed from the actor */
	bool CanRemoveSelectedLightCardFromActor() const;

	/** Creates a template of the selected light card */
	void CreateLightCardTemplate();

	/** Determines if there are selected light cards that can have templates created */
	bool CanCreateLightCardTemplate() const;

	/** Toggles the visible status of labels */
	void ToggleLightCardLabels();
	
	/** Display or hide labels for all included light cards */
	void ShowLightCardLabels(bool bVisible);

	/** If light card labels are currently toggled */
	bool ShouldShowLightCardLabels() const;

	/** Toggles light cards label flag */
	void ToggleLightCardLabelFlag(EDisplayClusterLabelFlags InFlag);
	
	/** Save the light card label flags to settings */
	void SaveLightCardLabelFlags(EDisplayClusterLabelFlags InFlags);
	
	/** If the light card label has a specific flag */
	bool LightCardLabelHasFlag(EDisplayClusterLabelFlags FlagToCheck) const;

	/** If the light card label is missing a specific flag */
	bool LightCardLabelDoesNotHaveFlag(EDisplayClusterLabelFlags FlagToCheck) const;
	
	/** The current light card label flags */
	EDisplayClusterLabelFlags GetLightCardLabelFlags() const;
	
	/** Return the current light card label scale */
	TOptional<float> GetLightCardLabelScale() const;

	/** Update light card label scale */
	void SetLightCardLabelScale(float NewValue);

	/** Set all icon visibility */
	void ShowIcons(bool bVisible);

	/** If all icons should be visible */
	bool ShouldShowIcons() const;

	/** Return the current icon scale */
	TOptional<float> GetIconScale() const;

	/** Update all icons scale */
	void SetIconScale(float NewValue);
	
private:
	/** Raised when the active Display cluster root actor has been changed in the operator panel */
	void OnActiveRootActorChanged(ADisplayClusterRootActor* NewRootActor);

	/** Raised when a request to select or deselect actors in the operator panel is raised from outside the panel */
	void OnActorsSelectedExternally(const TArray<AActor*>& Actors, bool bShouldSelect);
	
	/** Registers the light card editor with the global tab manager and adds it to the operator panel's extension tab stack */
	void RegisterTabSpawners();

	/** Unregisters the light card editor from the global tab manager */
	void UnregisterTabSpawners();
	
	/** Registers the light card editor tab with the operator panel using a layout extension */
	static void RegisterLayoutExtension(FLayoutExtender& InExtender);

	/** Spawns in the viewport tab */
	TSharedRef<SDockTab> SpawnViewportTab(const FSpawnTabArgs& SpawnTabArgs);
	
	/** Spawns in the outliner tab */
	TSharedRef<SDockTab> SpawnOutlinerTab(const FSpawnTabArgs& SpawnTabArgs);
	
	/** Creates the widget used to show the list of light cards associated with the active root actor */
	TSharedRef<SWidget> CreateLightCardOutlinerWidget();

	/** Create the widget for selecting light card templates */
	TSharedRef<SWidget> CreateLightCardTemplateWidget();
	
	/** Create the 3d viewport widget */
	TSharedRef<SWidget> CreateViewportWidget();

	/** Generate the place actors drop down menu */
	TSharedRef<SWidget> GeneratePlaceActorsMenu();

	/** Return the correct template icon to use */
	const FSlateBrush* GetLightCardTemplateIcon(const TWeakObjectPtr<UDisplayClusterLightCardTemplate> InTemplate) const;

	/** Generate the All Templates sub menu */
	void GenerateTemplateSubMenu(FMenuBuilder& InMenuBuilder);
	
	/** Generate the labels dropdown menu */
	TSharedRef<SWidget> GenerateLabelsMenu();

	/** Creates the frustum combo box */
	TSharedRef<SWidget> CreateFrustumWidget();

	/** Creates the editor's command list and binds commands to it */
	void BindCommands();

	/** Register any extensions for the toolbar */
	void RegisterToolbarExtensions();

	/** Register any extensions for the menu bar */
	void RegisterMenuExtensions();

	/** Extend the toolbar for this local instance */
	void ExtendToolbar(FToolBarBuilder& ToolbarBuilder);

	/** Extend the main file menu */
	void ExtendFileMenu(FMenuBuilder& MenuBuilder);
	
	/** Extend the main edit menu */
	void ExtendEditMenu(FMenuBuilder& MenuBuilder);
	
	/** Refresh all preview actors */
	void RefreshPreviewActors(EDisplayClusterLightCardEditorProxyType ProxyType = EDisplayClusterLightCardEditorProxyType::All);

	/** Refresh a specific preview stage actor */
	void RefreshPreviewStageActor(AActor* Actor);
	
	/** Reapply the current label settings */
	void RefreshLabels();
	
	/**
	 * Check if an object is managed by us
	 * @param InObject The object to compare
	 * @param OutProxyType The type of the object
	 * @return True if our object, false if not
	 */
	bool IsOurObject(UObject* InObject, EDisplayClusterLightCardEditorProxyType& OutProxyType) const;
	
	/** Bind delegates to when a BP compiles */
	void BindCompileDelegates();

	/** Remove compile delegates from a BP */
	void RemoveCompileDelegates();

	/** When a property on the actor has changed */
	void OnActorPropertyChanged(UObject* ObjectBeingModified, FPropertyChangedEvent& PropertyChangedEvent);

	/** Raised when the user adds an actor to the level */
	void OnLevelActorAdded(AActor* Actor);
	
	/** Raised when the user deletes an actor from the level */
	void OnLevelActorDeleted(AActor* Actor);

	/** Raised when a supported blueprint is compiled */
	void OnBlueprintCompiled(UBlueprint* Blueprint);

	/** Raised when any object is transacted */
	void OnObjectTransacted(UObject* Object, const FTransactionObjectEvent& TransactionObjectEvent);

	/** Add an item to recently placed list. Reorders matching items to the top of the list and trims the list */
	void AddRecentlyPlacedItem(const FDisplayClusterLightCardEditorRecentItem& InItem);

	/** Make sure no invalid entries are present */
	void CleanupRecentlyPlacedItems();

	/**
	 * Checks if it is possible only one or more folders - and no actors - are selected in the outliner
	 * They require editor delegates fired when executing some generic commands
	 *
	 * @return true if the outliner is valid and no actor is selected
	 */
	bool DoOutlinerFoldersNeedEditorDelegates() const;
	
private:
	/** The light card outliner widget */
	TSharedPtr<SDisplayClusterLightCardOutliner> LightCardOutliner;

	/** Templates to their icon brushes */
	TMap<TWeakObjectPtr<UDisplayClusterLightCardTemplate>, TSharedPtr<FSlateBrush>> TemplateBrushes;
	
	/** The 3d viewport */
	TSharedPtr<SDisplayClusterLightCardEditorViewport> ViewportView;

	/** The command list for editor commands */
	TSharedPtr<FUICommandList> CommandList;

	/** Stores the mouse position when the context menu was opened */
	TOptional<FIntPoint> CachedContextMenuMousePos;

	/** A reference to the root actor that is currently being operated on */
	TWeakObjectPtr<ADisplayClusterRootActor> ActiveRootActor;

	/** The view model for the operator panel */
	TWeakPtr<IDisplayClusterOperatorViewModel> OperatorViewModel;

	/** Selected actors */
	TArray<TWeakObjectPtr<AActor>> SelectedActors;

	/** Available frustums to choose */
	TArray<TSharedPtr<FString>> FrustumSelections;

	/** Delegate handle for the OnActiveRootActorChanged delegate */
	FDelegateHandle ActiveRootActorChangedHandle;

	/** Delegate handle for the OnActorsSelectedExternally delegate */
	FDelegateHandle ActorsSelectedExternallyHandle;

	/** Delegate handle for when an object is transacted */
	FDelegateHandle OnObjectTransactedHandle;
};