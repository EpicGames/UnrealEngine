// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterLightCardEditorCommands.h"

#define LOCTEXT_NAMESPACE "DisplayClusterLightCardEditorCommands"


void FDisplayClusterLightCardEditorCommands::RegisterCommands()
{
	UI_COMMAND(ResetCamera, "Reset Camera", "Resets the camera to focus on the mesh", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(FrameSelection, "Frame Selection", "Frames the selected items in the viewport", EUserInterfaceActionType::Button, FInputChord(EKeys::F));

	UI_COMMAND(PerspectiveProjection, "Perspective", "A perspective projection from the stage's view point", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(OrthographicProjection, "Orthographic", "An orthographic projection from the stage's view point", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(AzimuthalProjection, "Dome", "A hemispherical projection from the stage's view point", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(UVProjection, "UV", "A UV projection that flattens the stage's meshes based on their UV coordinates", EUserInterfaceActionType::RadioButton, FInputChord());

	UI_COMMAND(ViewOrientationTop, "Top", "Orient the view to look at the top of the stage", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Alt, EKeys::J));
	UI_COMMAND(ViewOrientationBottom, "Bottom", "Orient the view look at the bottom of the stage", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Alt | EModifierKey::Shift, EKeys::J));
	UI_COMMAND(ViewOrientationLeft, "Left", "Orient the view to look at the left of the stage", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Alt, EKeys::K));
	UI_COMMAND(ViewOrientationRight, "Right", "Orient the view to look at the right of the stage", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Alt | EModifierKey::Shift, EKeys::K));
	UI_COMMAND(ViewOrientationFront, "Front", "Orient the view to look at the front of the stage", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Alt, EKeys::H));
	UI_COMMAND(ViewOrientationBack, "Back", "Orient the view to look at the back of the stage", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Alt | EModifierKey::Shift, EKeys::H));

#if PLATFORM_MAC
	UI_COMMAND(CycleEditorWidgetCoordinateSystem, "Cycle Transform Coordinate System", "Cycles the transform gizmo coordinate systems between cartesian and spherical coordinates", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Command, EKeys::Tilde));
#else
	UI_COMMAND(CycleEditorWidgetCoordinateSystem, "Cycle Transform Coordinate System", "Cycles the transform gizmo coordinate systems between cartesian and spherical coordinates", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::Tilde));
#endif

	UI_COMMAND(SphericalCoordinateSystem, "Spherical Coordinate System", "Move objects using latitude and longitude", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(CartesianCoordinateSystem, "Cartesian Coordinate System", "Move objects using X, Y, and Z axes", EUserInterfaceActionType::RadioButton, FInputChord());

	UI_COMMAND(AddNewLightCard, "Light Card", "Add and assign a new Light Card to the actor", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(AddNewFlag, "Flag", "Add and assign a new light control flag", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(AddExistingLightCard, "Import Existing Content", "Add an existing Light Card to the actor", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RemoveLightCard, "Remove from Actor", "Remove the Light Card from the actor but do not delete it", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(PasteHere, "Paste Here", "Paste clipboard contents at the click location", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(SaveLightCardTemplate, "Save As Template", "Save a template of the light card's appearance", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control | EModifierKey::Alt, EKeys::S));

	UI_COMMAND(DrawLightCard, "Draw Light Card", "Draw polygon light card on viewport", EUserInterfaceActionType::ToggleButton, FInputChord());

	UI_COMMAND(ToggleAllLabels, "Labels", "Display labels in the preview and wall", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleLabelsVisibleInEditor, "Labels Visible in Editor", "Display labels in the editor", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleLabelsHiddenInGame, "Labels Hidden in Game", "Hide labels from displaying on the wall when running as -game", EUserInterfaceActionType::ToggleButton, FInputChord());
	
	// @todo label types -- ToggleLightCardLabels is meant for only light card types and the user text should be updated to reflect that when labels are supported on more than just light cards
	UI_COMMAND(ToggleLightCardLabels, "Show Labels", "Display labels on light cards in the preview and wall", EUserInterfaceActionType::ToggleButton, FInputChord());
	
	UI_COMMAND(ToggleIconVisibility, "Icons", "Display icons in the light card editor where applicable", EUserInterfaceActionType::ToggleButton, FInputChord());
}

#undef LOCTEXT_NAMESPACE
