// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"

#include "DisplayClusterLightCardEditorStyle.h"


class FDisplayClusterLightCardEditorCommands 
	: public TCommands<FDisplayClusterLightCardEditorCommands>
{
public:
	FDisplayClusterLightCardEditorCommands()
		: TCommands<FDisplayClusterLightCardEditorCommands>(TEXT("DisplayClusterLightCardEditor"), 
			NSLOCTEXT("Contexts", "DisplayClusterLightCardEditor", "Display Cluster LightCard Editor"),
			NAME_None,
			FDisplayClusterLightCardEditorStyle::Get().GetStyleSetName())
	{ }

	virtual void RegisterCommands() override;

public:
	// Viewport commands
	TSharedPtr<FUICommandInfo> ResetCamera;
	TSharedPtr<FUICommandInfo> FrameSelection;

	TSharedPtr<FUICommandInfo> PerspectiveProjection;
	TSharedPtr<FUICommandInfo> OrthographicProjection;
	TSharedPtr<FUICommandInfo> AzimuthalProjection;
	TSharedPtr<FUICommandInfo> UVProjection;

	TSharedPtr<FUICommandInfo> ViewOrientationTop;
	TSharedPtr<FUICommandInfo> ViewOrientationBottom;
	TSharedPtr<FUICommandInfo> ViewOrientationLeft;
	TSharedPtr<FUICommandInfo> ViewOrientationRight;
	TSharedPtr<FUICommandInfo> ViewOrientationFront;
	TSharedPtr<FUICommandInfo> ViewOrientationBack;

	TSharedPtr<FUICommandInfo> CycleEditorWidgetCoordinateSystem;
	TSharedPtr<FUICommandInfo> CartesianCoordinateSystem;
	TSharedPtr<FUICommandInfo> SphericalCoordinateSystem;

	TSharedPtr<FUICommandInfo> AddNewLightCard;
	TSharedPtr<FUICommandInfo> AddNewFlag;
	TSharedPtr<FUICommandInfo> AddExistingLightCard;
	TSharedPtr<FUICommandInfo> RemoveLightCard;
	TSharedPtr<FUICommandInfo> PasteHere;

	TSharedPtr<FUICommandInfo> SaveLightCardTemplate;
	
	TSharedPtr<FUICommandInfo> DrawLightCard;

	TSharedPtr<FUICommandInfo> ToggleAllLabels;
	TSharedPtr<FUICommandInfo> ToggleLightCardLabels;
	TSharedPtr<FUICommandInfo> ToggleLabelsVisibleInEditor;
	TSharedPtr<FUICommandInfo> ToggleLabelsHiddenInGame;

	TSharedPtr<FUICommandInfo> ToggleIconVisibility;
};
