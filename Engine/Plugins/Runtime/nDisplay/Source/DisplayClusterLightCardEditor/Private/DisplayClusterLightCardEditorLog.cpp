// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterLightCardEditorLog.h"


// Module-wide log categories
DEFINE_LOG_CATEGORY(DisplayClusterLightCardEditorLog);
