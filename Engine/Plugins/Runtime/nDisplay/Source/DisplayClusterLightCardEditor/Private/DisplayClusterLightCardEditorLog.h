// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"

// Module-wide log categories
DECLARE_LOG_CATEGORY_EXTERN(DisplayClusterLightCardEditorLog, Log, All);

DECLARE_STATS_GROUP(TEXT("nDisplay Light Card Editor"), STATGROUP_NDisplayLightCardEditor, STATCAT_Advanced)