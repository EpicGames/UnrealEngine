// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterLightCardEditorModule.h"

#include "DisplayClusterConfigurationTypes.h"

#include "DisplayClusterLightCardActor.h"
#include "DisplayClusterRootActor.h"
#include "Blueprints/DisplayClusterBlueprintLib.h"

#include "DisplayClusterLightCardEditor.h"
#include "DisplayClusterLightCardEditorCommands.h"

#include "DetailCustomizations/DisplayClusterLightCardActorDetails.h"
#include "LightCardTemplates/DisplayClusterLightCardTemplate.h"
#include "PropertyEditorModule.h"
#include "Settings/DisplayClusterLightCardEditorSettings.h"

#include "IDisplayClusterOperator.h"

#include "ISettingsModule.h"

#define REGISTER_OBJECT_LAYOUT(ObjectType, CustomizationType) { \
	const FName LayoutName = ObjectType::StaticClass()->GetFName(); \
	RegisteredClassLayoutNames.Add(LayoutName); \
	PropertyModule.RegisterCustomClassLayout(LayoutName, \
		FOnGetDetailCustomizationInstance::CreateStatic(&CustomizationType::MakeInstance)); \
}

#define LOCTEXT_NAMESPACE "DisplayClusterLightCardEditor"

void FDisplayClusterLightCardEditorModule::StartupModule()
{
	RegisterSettings();
	RegisterDetailCustomizations();
	FDisplayClusterLightCardEditorCommands::Register();

	if (FModuleManager::Get().IsModuleLoaded(IDisplayClusterOperator::ModuleName))
	{
		RegisterOperatorApp();
	}
	else
	{
		ModuleChangedHandle = FModuleManager::Get().OnModulesChanged().AddRaw(this, &FDisplayClusterLightCardEditorModule::HandleModuleChanged);
	}
}

void FDisplayClusterLightCardEditorModule::ShutdownModule()
{
	UnregisterSettings();
	UnregisterDetailCustomizations();
	UnregisterOperatorApp();

	if (ModuleChangedHandle.IsValid())
	{
		FModuleManager::Get().OnModulesChanged().Remove(ModuleChangedHandle);
	}
	
	FDisplayClusterLightCardEditorCommands::Unregister();
}

void FDisplayClusterLightCardEditorModule::ShowLabels(const FLabelArgs& InArgs)
{
	check(InArgs.RootActor != nullptr);
	
	UDisplayClusterLightCardEditorProjectSettings* ProjectSettings = GetMutableDefault<UDisplayClusterLightCardEditorProjectSettings>();
	
	ProjectSettings->Modify();
	ProjectSettings->bDisplayLightCardLabels = InArgs.bVisible;
	ProjectSettings->LightCardLabelScale = InArgs.Scale;

	TSet<ADisplayClusterLightCardActor*> RootActorLightCardActors;
	UDisplayClusterBlueprintLib::FindLightCardsForRootActor(InArgs.RootActor, RootActorLightCardActors);

	for (ADisplayClusterLightCardActor* LightCardActor : RootActorLightCardActors)
	{
		LightCardActor->Modify(false);
		LightCardActor->ShowLightCardLabel(InArgs);
	}
}

UDisplayClusterStageActorTemplate* FDisplayClusterLightCardEditorModule::GetDefaultLightCardTemplate() const
{
	const UDisplayClusterLightCardEditorProjectSettings* Settings = GetDefault<UDisplayClusterLightCardEditorProjectSettings>();
	return Settings->DefaultLightCardTemplate.LoadSynchronous();
}

void FDisplayClusterLightCardEditorModule::RegisterSettings()
{
	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
	{
		UDisplayClusterLightCardEditorProjectSettings* Settings = GetMutableDefault<UDisplayClusterLightCardEditorProjectSettings>();
		// Needs transactional for undo/redo support in the light card editor
		Settings->SetFlags(RF_Transactional);
		SettingsModule->RegisterSettings("Project", "Plugins", "nDisplayICVFXEditor",
			LOCTEXT("nDisplayICVFXEditorName", "nDisplay - ICVFX Editor"),
			LOCTEXT("nDisplayICVFXEditorDescription", "Configure settings for the nDisplay ICVFX Editor."),
			Settings);
	}
}

void FDisplayClusterLightCardEditorModule::UnregisterSettings()
{
	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
	{
		SettingsModule->UnregisterSettings("Project", "Plugins", "nDisplayLightCardEditor");
	}
}

void FDisplayClusterLightCardEditorModule::RegisterDetailCustomizations()
{
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	REGISTER_OBJECT_LAYOUT(ADisplayClusterLightCardActor, FDisplayClusterLightCardActorDetails);
}

void FDisplayClusterLightCardEditorModule::UnregisterDetailCustomizations()
{
	FPropertyEditorModule* PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor");
	if (PropertyModule)
	{
		for (const FName& ClassName : RegisteredClassLayoutNames)
		{
			PropertyModule->UnregisterCustomClassLayout(ClassName);
		}
	}

	RegisteredClassLayoutNames.Empty();
}

void FDisplayClusterLightCardEditorModule::RegisterOperatorApp()
{
	OperatorAppHandle = IDisplayClusterOperator::Get().RegisterApp(IDisplayClusterOperator::FOnGetAppInstance::CreateStatic(&FDisplayClusterLightCardEditor::MakeInstance));
}

void FDisplayClusterLightCardEditorModule::UnregisterOperatorApp()
{
	if (IDisplayClusterOperator* OperatorModule = FModuleManager::GetModulePtr<IDisplayClusterOperator>(IDisplayClusterOperator::ModuleName))
	{
		OperatorModule->UnregisterApp(OperatorAppHandle);
	}
}

void FDisplayClusterLightCardEditorModule::HandleModuleChanged(FName InModuleName, EModuleChangeReason InChangeReason)
{
	if (InModuleName == IDisplayClusterOperator::ModuleName && InChangeReason == EModuleChangeReason::ModuleLoaded)
	{
		RegisterOperatorApp();
		FModuleManager::Get().OnModulesChanged().Remove(ModuleChangedHandle);
		ModuleChangedHandle.Reset();
	}
}

IMPLEMENT_MODULE(FDisplayClusterLightCardEditorModule, DisplayClusterLightCardEditor);

#undef LOCTEXT_NAMESPACE
