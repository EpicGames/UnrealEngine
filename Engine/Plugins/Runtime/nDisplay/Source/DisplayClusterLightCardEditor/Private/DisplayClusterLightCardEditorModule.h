// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDisplayClusterLightCardEditor.h"

/**
 * Display Cluster editor module
 */
class FDisplayClusterLightCardEditorModule :
	public IDisplayClusterLightCardEditor
{
public:
	//~ IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	
	virtual void ShowLabels(const FLabelArgs& InArgs) override;
	virtual UDisplayClusterStageActorTemplate* GetDefaultLightCardTemplate() const override;
	
private:
	void RegisterSettings();
	void UnregisterSettings();

	void RegisterDetailCustomizations();
	void UnregisterDetailCustomizations();

	void RegisterOperatorApp();
	void UnregisterOperatorApp();
	
	void HandleModuleChanged(FName InModuleName, EModuleChangeReason InChangeReason);
	
private:
	/** Stores a list of class names that custom layouts have been registered for */
	TArray<FName> RegisteredClassLayoutNames;

	FDelegateHandle ModuleChangedHandle;
	FDelegateHandle OperatorAppHandle;
};
