// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

enum class EDisplayClusterLightCardEditorProxyType : uint8
{
	All,
	RootActor,
	StageActor
};