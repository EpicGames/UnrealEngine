// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterLightCardEditorStyle.h"

#include "Styling/AppStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/StyleColors.h"
#include "Styling/ToolBarStyle.h"


FDisplayClusterLightCardEditorStyle::FDisplayClusterLightCardEditorStyle(): FSlateStyleSet("DisplayClusterLightCardEditorStyle")
{
	const FVector2D Icon40x40(40.0f, 40.0f);
	const FVector2D Icon16x16(16.0f, 16.0f);

	// Set miscellaneous icons
	SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Runtime/nDisplay/Content/"));
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));

	// Core Content
	{
		Set("DisplayClusterLightCardEditor.Labels", new CORE_IMAGE_BRUSH_SVG("Starship/Common/IssueTracker", Icon16x16));
		Set("DisplayClusterLightCardEditor.IconSymbol", new CORE_IMAGE_BRUSH_SVG("Starship/Common/IssueTracker", Icon16x16));
	}

	// Content
	{
		Set("DisplayClusterLightCardEditor.LabelSymbol", new IMAGE_BRUSH_SVG("Icons/OperatorPanel/LabelSymbol", Icon16x16));

		Set("DisplayClusterLightCardEditor.UV", new IMAGE_BRUSH_SVG("Icons/LightCard/LightCardUV", Icon16x16));
		Set("DisplayClusterLightCardEditor.Dome", new IMAGE_BRUSH_SVG("Icons/OperatorPanel/Dome", Icon16x16));
		Set("DisplayClusterLightCardEditor.Orthographic", new IMAGE_BRUSH_SVG("Icons/OperatorPanel/Orthographic", Icon16x16));

		Set("DisplayClusterLightCardEditor.DrawPoly", new IMAGE_BRUSH("Icons/OperatorPanel/PolyPath_40x", Icon40x40));
		Set("DisplayClusterLightCardEditor.ActorHidden", new IMAGE_BRUSH_SVG("Icons/OperatorPanel/ActorHidden", Icon16x16));
		Set("DisplayClusterLightCardEditor.ActorNotHidden", new IMAGE_BRUSH_SVG("Icons/OperatorPanel/ActorNotHidden", Icon16x16));
		Set("DisplayClusterLightCardEditor.Template", new IMAGE_BRUSH_SVG("Icons/OperatorPanel/Template", Icon16x16));
		Set("DisplayClusterLightCardEditor.FrustumOnTop", new IMAGE_BRUSH_SVG("Icons/OperatorPanel/FrustumOnTop", Icon16x16));
		Set("DisplayClusterLightCardEditor.FrustumUnderneath", new IMAGE_BRUSH_SVG("Icons/OperatorPanel/FrustumUnderneath", Icon16x16));
		Set("DisplayClusterLightCardEditor.ViewportsFrozen", new IMAGE_BRUSH_SVG("Icons/Viewport/nDisplayFrozen_16", Icon16x16));
	}

	{
		// Build custom style by copying the base ToggleButtonCheckbox
		const FCheckBoxStyle& ToggleButtonCheckboxStyle = FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("ToggleButtonCheckbox");
		FCheckBoxStyle DrawLightcardsToggleButtonStyle = ToggleButtonCheckboxStyle;

		// Rounder corners
		constexpr float CornerRadius = 5.0f;

		// Match ViewportToolbarWarning.Raised colors (from StarshipStyle.cpp)

		FLinearColor WarningColor = FStyleColors::Warning.GetSpecifiedColor();
		const FSlateRoundedBoxBrush WarningBrush(WarningColor, CornerRadius, WarningColor, 1.0f);

		FLinearColor WarningHoveredColor = FStyleColors::Warning.GetSpecifiedColor();
		WarningHoveredColor.R = FMath::Min(1.0f, WarningHoveredColor.R * 1.5f);
		WarningHoveredColor.G = FMath::Min(1.0f, WarningHoveredColor.G * 1.5f);
		WarningHoveredColor.B = FMath::Min(1.0f, WarningHoveredColor.B * 1.5f);
		const FSlateRoundedBoxBrush WarningHoveredBrush(WarningHoveredColor, CornerRadius, WarningHoveredColor, 1.0f);

		FLinearColor WarningPressedColor = FStyleColors::Warning.GetSpecifiedColor();
		WarningPressedColor.A = .50f;
		const FSlateRoundedBoxBrush WarningPressedBrush(WarningPressedColor, CornerRadius, WarningPressedColor, 1.0f);

		// Override the checked/hover/pressed background color
		DrawLightcardsToggleButtonStyle.SetCheckedImage(WarningBrush);
		DrawLightcardsToggleButtonStyle.SetCheckedHoveredImage(WarningHoveredBrush);
		DrawLightcardsToggleButtonStyle.SetCheckedPressedImage(WarningPressedBrush);

		// Override the checked/hover/pressed foreground color
		DrawLightcardsToggleButtonStyle
			.SetCheckedForegroundColor(FStyleColors::ForegroundInverted)
			.SetCheckedHoveredForegroundColor(FStyleColors::ForegroundInverted)
			.SetCheckedPressedForegroundColor(FStyleColors::ForegroundInverted);

		Set("DisplayClusterLightCardEditor.DrawLightcardsToggleButton", DrawLightcardsToggleButtonStyle);
	}

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}
