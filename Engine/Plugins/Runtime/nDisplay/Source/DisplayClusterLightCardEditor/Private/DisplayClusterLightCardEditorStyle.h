// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateStyleRegistry.h"

#include "Styling/SlateStyle.h"


/**
 * Implements the visual style of the operator panel
 */
class FDisplayClusterLightCardEditorStyle final : public FSlateStyleSet
{
public:

	FDisplayClusterLightCardEditorStyle();

	virtual ~FDisplayClusterLightCardEditorStyle()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	}

	static FDisplayClusterLightCardEditorStyle& Get()
	{
		static FDisplayClusterLightCardEditorStyle Inst;
		return Inst;
	}
};
