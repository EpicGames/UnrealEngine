// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterLightCardEditorUtils.h"

#include "AssetRegistry/AssetData.h"
#include "DisplayClusterLightCardActor.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Engine/Blueprint.h"
#include "UObject/UObjectIterator.h"

bool UE::DisplayClusterLightCardEditorUtils::IsManagedActor(const AActor* InActor)
{
	return InActor && InActor->Implements<UDisplayClusterStageActor>();
}

bool UE::DisplayClusterLightCardEditorUtils::IsProxySelectable(const AActor* InActor)
{
	// Currently managed actors & selectable proxies are 1 - 1
	return IsManagedActor(InActor);
}

TSet<UClass*> UE::DisplayClusterLightCardEditorUtils::GetAllStageActorClasses()
{
	TSet<UClass*> OutClasses;

	auto ShouldAddNativeClass = [] (const UClass* Class) -> bool
	{
		return Class->ImplementsInterface(UDisplayClusterStageActor::StaticClass()) && Class->IsNative() &&
			!Class->HasAnyClassFlags(CLASS_Deprecated | CLASS_NewerVersionExists | CLASS_Abstract) &&
			!Class->GetName().StartsWith(TEXT("SKEL_")) && !Class->GetName().StartsWith(TEXT("REINST_"));
	};

	// First search native classes
	for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
	{
		UClass* Class = *ClassIt;

		if (ShouldAddNativeClass(Class))
		{
			OutClasses.Add(Class);
		}
	}

	// Next search blueprint classes inheriting the stage actor interface
	const IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(FName("AssetRegistry")).Get();

	TSet<FTopLevelAssetPath> InheritedInterfacePaths;
	AssetRegistry.GetDerivedClassNames({UDisplayClusterStageActor::StaticClass()->GetClassPathName()},
		{}, InheritedInterfacePaths);

	TArray<FAssetData> OutAssets;
	AssetRegistry.GetAssetsByClass(UBlueprint::StaticClass()->GetClassPathName(), OutAssets, true);

	for (const FAssetData& Asset : OutAssets)
	{
		FAssetDataTagMapSharedView::FFindTagResult Result = Asset.TagsAndValues.FindTag(TEXT("GeneratedClass"));
		if (Result.IsSet())
		{
			const FString& GeneratedClassPathPtr = Result.GetValue();
			const FTopLevelAssetPath ClassObjectPath(FPackageName::ExportTextPathToObjectPath(*GeneratedClassPathPtr));
			if (InheritedInterfacePaths.Contains(ClassObjectPath))
			{
				TSoftObjectPtr<UClass> SoftObjectPtr = TSoftObjectPtr<UClass>(FSoftObjectPath(ClassObjectPath.ToString()));
				if (UClass* Class = SoftObjectPtr.LoadSynchronous())
				{
					OutClasses.Add(Class);
				}
			}
		}
	}
	return OutClasses;
}
