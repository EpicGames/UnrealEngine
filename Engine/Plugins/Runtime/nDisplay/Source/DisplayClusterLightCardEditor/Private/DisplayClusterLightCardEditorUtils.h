// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class AActor;

namespace UE::DisplayClusterLightCardEditorUtils
{
	/** Check if the given actor is of a type that should be managed by the light card editor */
	bool IsManagedActor(const AActor* InActor);
	
	/** Check if the given actor is of a type that is selectable in the light card editor */
	bool IsProxySelectable(const AActor* InActor);

	/** Discover all classes implementing or inheriting the stage actor interface */
	TSet<UClass*> GetAllStageActorClasses();
}