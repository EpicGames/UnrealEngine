// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DisplayClusterLightCardActor.h"
#include "StageActor/DisplayClusterStageActorTemplate.h"

#include "DisplayClusterLightCardTemplate.generated.h"

/**
 * A template asset to store appearance settings from Light Card actors.
 */
UCLASS(NotBlueprintType, NotBlueprintable, NotPlaceable)
class UDisplayClusterLightCardTemplate : public UDisplayClusterStageActorTemplate
{
	GENERATED_BODY()

public:
	// UDisplayClusterStageActorTemplate
	virtual AActor* GetTemplateActor() const override { return LightCardActor; }
	// ~UDisplayClusterStageActorTemplate
	
public:
	/** The instanced template object containing user settings for the light card. */
	UPROPERTY(VisibleAnywhere, Instanced, Category = Template, NoClear, meta = (ShowOnlyInnerProperties))
	TObjectPtr<ADisplayClusterLightCardActor> LightCardActor;
};
