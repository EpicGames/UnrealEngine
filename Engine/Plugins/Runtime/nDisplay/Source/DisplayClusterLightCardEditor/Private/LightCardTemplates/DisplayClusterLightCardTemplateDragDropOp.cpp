// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterLightCardTemplateDragDropOp.h"

#define LOCTEXT_NAMESPACE "FDisplayClusterLightCardTemplateDragDropOp"

TSharedRef<FDisplayClusterLightCardTemplateDragDropOp> FDisplayClusterLightCardTemplateDragDropOp::New(
	TWeakObjectPtr<UDisplayClusterLightCardTemplate> InTemplate)
{
	TSharedRef<FDisplayClusterLightCardTemplateDragDropOp> DragDropOp = MakeShared<FDisplayClusterLightCardTemplateDragDropOp>();
	DragDropOp->LightCardTemplate = InTemplate;
	DragDropOp->MouseCursor = EMouseCursor::GrabHandClosed;

	return DragDropOp;
}

void FDisplayClusterLightCardTemplateDragDropOp::Construct()
{
	DefaultHoverText = GetHoverText();
	ResetToDefaultToolTip();
	
	FDecoratedDragDropOp::Construct();
}

void FDisplayClusterLightCardTemplateDragDropOp::SetDropAsInvalid(FText Message)
{
	bCanBeDropped = false;
	CurrentHoverText = !Message.IsEmpty() ? FText::Format(LOCTEXT("HoverText_Formatted", "{0}: {1}"), GetHoverText(), Message) : DefaultHoverText;

	CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));
}

void FDisplayClusterLightCardTemplateDragDropOp::SetDropAsValid(FText Message)
{
	bCanBeDropped = true;
	CurrentHoverText = !Message.IsEmpty() ? FText::Format(LOCTEXT("HoverText_Formatted", "{0}: {1}"), GetHoverText(), Message) : DefaultHoverText;

	CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK"));
}

#undef LOCTEXT_NAMESPACE