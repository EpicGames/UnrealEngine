// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/DragAndDrop.h"
#include "DragAndDrop/DecoratedDragDropOp.h"

class UDisplayClusterLightCardTemplate;

class FDisplayClusterLightCardTemplateDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FDisplayClusterLightCardTemplateDragDropOp, FDecoratedDragDropOp)

	FDisplayClusterLightCardTemplateDragDropOp() :
		bCanBeDropped(true)
	{ }

	static TSharedRef<FDisplayClusterLightCardTemplateDragDropOp> New(TWeakObjectPtr<UDisplayClusterLightCardTemplate> InTemplate);

	TWeakObjectPtr<UDisplayClusterLightCardTemplate> GetTemplate() const { return LightCardTemplate; }
	
	// Begin FDragDropOperation
protected:
	virtual void Construct() override;
	// End FDragDropOperation

public:
	void SetDropAsInvalid(FText Message = FText::GetEmpty());
	void SetDropAsValid(FText Message = FText::GetEmpty());
	bool CanBeDropped() const { return bCanBeDropped; }

protected:
	virtual FText GetHoverText() const { return FText::GetEmpty(); }

protected:
	bool bCanBeDropped;

private:
	TWeakObjectPtr<UDisplayClusterLightCardTemplate> LightCardTemplate;
};