// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterLightCardTemplateHelpers.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "LightCardTemplates/DisplayClusterLightCardTemplate.h"

TArray<UDisplayClusterLightCardTemplate*> UE::DisplayClusterLightCardTemplateHelpers::GetLightCardTemplates(bool bFavoritesOnly)
{
	TArray<UDisplayClusterLightCardTemplate*> OutLightCardTemplates;
	
	const IAssetRegistry& AssetRegistry = FModuleManager::GetModuleChecked<FAssetRegistryModule>(AssetRegistryConstants::ModuleName).Get();
	
	TArray<FAssetData> TemplateAssets;
	const FTopLevelAssetPath ClassPath = UDisplayClusterLightCardTemplate::StaticClass()->GetClassPathName();
	AssetRegistry.GetAssetsByClass(ClassPath, TemplateAssets, true);

	for (const FAssetData& AssetData : TemplateAssets)
	{
		UDisplayClusterLightCardTemplate* AssetObject = CastChecked<UDisplayClusterLightCardTemplate>(AssetData.GetAsset());
		if (!bFavoritesOnly || AssetObject->bIsFavorite)
		{
			OutLightCardTemplates.Add(AssetObject);
		}
	}

	return OutLightCardTemplates;
}
