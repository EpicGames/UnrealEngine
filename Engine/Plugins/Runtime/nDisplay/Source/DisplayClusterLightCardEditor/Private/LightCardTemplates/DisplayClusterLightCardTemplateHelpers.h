// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class UDisplayClusterLightCardTemplate;

namespace UE::DisplayClusterLightCardTemplateHelpers
{
	/**
	 * Load and return all light card templates
	 * 
	 * @param bFavoritesOnly only return light card templates that are marked as a user favorite
	 * @return the light card template objects, loaded from their asset
	 */
	TArray<UDisplayClusterLightCardTemplate*> GetLightCardTemplates(bool bFavoritesOnly = false);
}