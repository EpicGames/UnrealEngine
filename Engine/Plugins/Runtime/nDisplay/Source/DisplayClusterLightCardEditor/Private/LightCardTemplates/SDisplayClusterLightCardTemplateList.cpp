// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDisplayClusterLightCardTemplateList.h"

#include "AssetRegistry/AssetData.h"
#include "DisplayClusterLightCardTemplateDragDropOp.h"
#include "DisplayClusterLightCardTemplateHelpers.h"
#include "DisplayClusterLightCardEditor.h"
#include "Engine/Texture2D.h"
#include "Framework/Application/SlateApplication.h"
#include "LightCardTemplates/DisplayClusterLightCardTemplate.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "Engine/Texture.h"
#include "Styling/SlateIconFinder.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"

#define LOCTEXT_NAMESPACE "SDisplayClusterLightCardTemplateList"

namespace DisplayClusterLightCardTemplateListColumnNames
{
	const static FName LightCardTemplateName(TEXT("LightCardTemplateName"));
};

class SLightCardTemplateTreeItemRow : public SMultiColumnTableRow<TSharedPtr<SDisplayClusterLightCardTemplateList::FLightCardTemplateTreeItem>>
{
	SLATE_BEGIN_ARGS(SLightCardTemplateTreeItemRow) {}
	SLATE_END_ARGS()

public:

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable,
		const TSharedPtr<SDisplayClusterLightCardTemplateList>& InTemplateList,
		const TSharedPtr<SDisplayClusterLightCardTemplateList::FLightCardTemplateTreeItem> InLightCardTreeItem)
	{
		LightCardTreeItem = InLightCardTreeItem;
		OwningTemplateListPtr = InTemplateList;

		SMultiColumnTableRow<TSharedPtr<SDisplayClusterLightCardTemplateList::FLightCardTemplateTreeItem>>::Construct(
			SMultiColumnTableRow<TSharedPtr<SDisplayClusterLightCardTemplateList::FLightCardTemplateTreeItem>>::FArguments()
			.Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("SceneOutliner.TableViewRow"))
			.OnDragDetected(this, &SLightCardTemplateTreeItemRow::HandleDragDetected)
		, InOwnerTable);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		if (ColumnName == DisplayClusterLightCardTemplateListColumnNames::LightCardTemplateName)
		{
			return SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(6.f, 0.f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			[
				SAssignNew(FavoriteIcon, SBorder)
				.BorderImage(FAppStyle::GetBrush("NoBorder"))
				.OnMouseButtonDown(this, &SLightCardTemplateTreeItemRow::OnFavoriteMouseDown)
				[
					SNew(SImage)
					.Image(this, &SLightCardTemplateTreeItemRow::GetFavoriteIcon)
					.ToolTipText(LOCTEXT("FavoriteToolTip", "Favorite this template"))
				]
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(0, 1, 6, 1))
			.VAlign(VAlign_Center)
			[
				SNew(SBox)
				.WidthOverride(16)
				.HeightOverride(16)
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(this, &SLightCardTemplateTreeItemRow::GetLightCardIcon)
				]
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			[
				SNew(STextBlock)
				.Text(this, &SLightCardTemplateTreeItemRow::GetLightCardDisplayName)
			];
		}

		return SNullWidget::NullWidget;
	}

private:
	FText GetLightCardDisplayName() const
	{
		FString ItemName = TEXT("");
		if (LightCardTreeItem.IsValid())
		{
			const TSharedPtr<SDisplayClusterLightCardTemplateList::FLightCardTemplateTreeItem> LightCardTreeItemPin = LightCardTreeItem.Pin();
			ItemName = LightCardTreeItemPin->TemplateName.ToString();
		}

		return FText::FromString(*ItemName);
	}

	const FSlateBrush* GetLightCardIcon() const
	{
		if (LightCardTreeItem.IsValid())
		{
			if (LightCardTreeItem.Pin()->SlateBrush.IsValid())
			{
				return LightCardTreeItem.Pin()->SlateBrush.Get();
			}
			if (LightCardTreeItem.Pin()->LightCardTemplate.IsValid())
			{
				return FSlateIconFinder::FindIconBrushForClass(LightCardTreeItem.Pin()->LightCardTemplate->GetClass());
			}
		}

		return nullptr;
	}
	
	FReply HandleDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		if (OwningTemplateListPtr.IsValid() && OwningTemplateListPtr.Pin()->IsDragDropEnabled() &&
			LightCardTreeItem.IsValid() && MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
		{
			const TSharedPtr<FDragDropOperation> DragDropOp = FDisplayClusterLightCardTemplateDragDropOp::New(LightCardTreeItem.Pin()->LightCardTemplate);

			if (DragDropOp.IsValid())
			{
				return FReply::Handled().BeginDragDrop(DragDropOp.ToSharedRef());
			}
		}

		return FReply::Unhandled();
	}
	
	const FSlateBrush* GetFavoriteIcon() const
	{
		const bool bIsFavorite = (LightCardTreeItem.IsValid() && LightCardTreeItem.Pin()->LightCardTemplate.IsValid()) ? LightCardTreeItem.Pin()->LightCardTemplate->bIsFavorite : false;
        const FName FavoriteIconName = bIsFavorite ? "DetailsView.PropertyIsFavorite" : "DetailsView.PropertyIsNotFavorite";
		return FSlateIcon(FAppStyle::Get().GetStyleSetName(), FavoriteIconName).GetIcon();
	}

	FReply OnFavoriteMouseDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		if (LightCardTreeItem.IsValid() && LightCardTreeItem.Pin()->LightCardTemplate.IsValid())
		{
			LightCardTreeItem.Pin()->LightCardTemplate->bIsFavorite = !LightCardTreeItem.Pin()->LightCardTemplate->bIsFavorite;
			LightCardTreeItem.Pin()->LightCardTemplate->PostEditChange();
			LightCardTreeItem.Pin()->LightCardTemplate->SaveConfig();
			
			if (OwningTemplateListPtr.IsValid())
			{
				OwningTemplateListPtr.Pin()->ApplyFilter();
			}
		}
		return FReply::Handled();
	}

private:
	TWeakPtr<SDisplayClusterLightCardTemplateList> OwningTemplateListPtr;
	TWeakPtr<SDisplayClusterLightCardTemplateList::FLightCardTemplateTreeItem> LightCardTreeItem;
	TSharedPtr<SBorder> FavoriteIcon;
};

bool SDisplayClusterLightCardTemplateList::FLightCardTemplateTreeItem::IsFavorite() const
{
	return LightCardTemplate.IsValid() && LightCardTemplate->bIsFavorite;
}

SDisplayClusterLightCardTemplateList::~SDisplayClusterLightCardTemplateList()
{
	IAssetRegistry& AssetRegistry = FModuleManager::GetModuleChecked<FAssetRegistryModule>(AssetRegistryConstants::ModuleName).Get();
	AssetRegistry.OnAssetAdded().RemoveAll(this);
	AssetRegistry.OnAssetRemoved().RemoveAll(this);
	AssetRegistry.OnFilesLoaded().RemoveAll(this);
}

void SDisplayClusterLightCardTemplateList::Construct(const FArguments& InArgs, TSharedPtr<FDisplayClusterLightCardEditor> InLightCardEditor)
{
	LightCardEditorPtr = InLightCardEditor;

	bSpawnOnSelection = InArgs._SpawnOnSelection;

	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(AssetRegistryConstants::ModuleName).Get();
	AssetRegistry.OnAssetAdded().AddRaw(this, &SDisplayClusterLightCardTemplateList::OnAssetAddedOrRemoved);
	AssetRegistry.OnAssetRemoved().AddRaw(this, &SDisplayClusterLightCardTemplateList::OnAssetAddedOrRemoved);
	AssetRegistry.OnFilesLoaded().AddRaw(this, &SDisplayClusterLightCardTemplateList::OnAssetsLoaded);
	
	RefreshTemplateList();

	ChildSlot
	[
		SNew(SBox)
		.MinDesiredWidth(260.f)
		.Padding(FMargin(4.f, 0.f))
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.Padding(0.f, 8.f, 0.f, 5.f)
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.Padding(6.f, 0.f)
				.AutoWidth()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				[
					SNew(SImage)
					.Image(this, &SDisplayClusterLightCardTemplateList::GetFavoriteFilterIcon)
					.ToolTipText(LOCTEXT("FavoriteFilterToolTip", "Filter by favorites"))
					.OnMouseButtonDown_Lambda([this](const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
					{
						bFilterFavorites = !bFilterFavorites;
						ApplyFilter();
						return FReply::Handled();
					})
				]
				+SHorizontalBox::Slot()
				.FillWidth(1.f)
				[
					SNew(SSearchBox)
					.SelectAllTextWhenFocused(true)
					.OnTextChanged(this, &SDisplayClusterLightCardTemplateList::OnFilterTextChanged)
					.HintText(LOCTEXT("SearchBoxHint", "Search Light Card Templates..."))
				]
			]
			+SVerticalBox::Slot()
			[
				SAssignNew(LightCardTemplateTreeView, STreeView<TSharedPtr<FLightCardTemplateTreeItem>>)
				.TreeItemsSource(&FilteredLightCardTemplateTree)
				.SelectionMode(ESelectionMode::Single)
				.OnGenerateRow(this, &SDisplayClusterLightCardTemplateList::GenerateTreeItemRow)
				.OnGetChildren(this, &SDisplayClusterLightCardTemplateList::GetChildrenForTreeItem)
				.OnSelectionChanged(this, &SDisplayClusterLightCardTemplateList::OnSelectionChanged)
				.HeaderRow
				(
					SNew(SHeaderRow)
					.Visibility(InArgs._HideHeader ? EVisibility::Collapsed : EVisibility::Visible)
					+SHeaderRow::Column(DisplayClusterLightCardTemplateListColumnNames::LightCardTemplateName)
					.DefaultLabel(LOCTEXT("LightCardTemplateName", "Light Card Templates"))
					.FillWidth(0.8f)
				)
			]
		]
	];
}

void SDisplayClusterLightCardTemplateList::RefreshTemplateList()
{
	LightCardTemplateTree.Reset();

	TArray<UDisplayClusterLightCardTemplate*> LightCardTemplates =
		UE::DisplayClusterLightCardTemplateHelpers::GetLightCardTemplates();
	
	for (UDisplayClusterLightCardTemplate* Template : LightCardTemplates)
	{
		TSharedPtr<FLightCardTemplateTreeItem> TemplateTreeItem = MakeShared<FLightCardTemplateTreeItem>();
		
		TemplateTreeItem->TemplateName = Template->GetFName();
		TemplateTreeItem->LightCardTemplate = Template;

		if (Template->LightCardActor && Template->LightCardActor->Texture.Get())
		{
			TemplateTreeItem->SlateBrush = MakeShared<FSlateBrush>();
		
			TemplateTreeItem->SlateBrush->SetResourceObject(Template->LightCardActor->Texture.Get());
			TemplateTreeItem->SlateBrush->ImageSize = FVector2D(64.f, 64.f);
		}
		
		LightCardTemplateTree.Add(TemplateTreeItem);
	}

	ApplyFilter();
}

void SDisplayClusterLightCardTemplateList::ApplyFilter(const FString& InFilterText, bool bFavorite)
{
	FilterText = InFilterText;
	
	FilteredLightCardTemplateTree.Reset();

	for (TSharedPtr<FLightCardTemplateTreeItem>& LightCardTemplate : LightCardTemplateTree)
	{
		if (LightCardTemplate.IsValid() &&
			((InFilterText.IsEmpty() || LightCardTemplate->TemplateName.ToString().Contains(InFilterText)) &&
			(!bFavorite || LightCardTemplate->IsFavorite())))
		{
			FilteredLightCardTemplateTree.Add(LightCardTemplate);
		}
	}

	if (LightCardTemplateTreeView.IsValid())
	{
		LightCardTemplateTreeView->RequestTreeRefresh();
	}
}

void SDisplayClusterLightCardTemplateList::ApplyFilter()
{
	ApplyFilter(FilterText, bFilterFavorites);
}

bool SDisplayClusterLightCardTemplateList::IsDragDropEnabled() const
{
	return !bSpawnOnSelection;
}

TSharedRef<ITableRow> SDisplayClusterLightCardTemplateList::GenerateTreeItemRow(
	TSharedPtr<FLightCardTemplateTreeItem> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SLightCardTemplateTreeItemRow, OwnerTable, SharedThis(this), Item);
}

void SDisplayClusterLightCardTemplateList::GetChildrenForTreeItem(TSharedPtr<FLightCardTemplateTreeItem> InItem,
	TArray<TSharedPtr<FLightCardTemplateTreeItem>>& OutChildren)
{
}

const FSlateBrush* SDisplayClusterLightCardTemplateList::GetFavoriteFilterIcon() const
{
	const FName FavoriteIconName = bFilterFavorites ? "DetailsView.PropertyIsFavorite" : "DetailsView.PropertyIsNotFavorite";
	return FSlateIcon(FAppStyle::Get().GetStyleSetName(), FavoriteIconName).GetIcon();
}

void SDisplayClusterLightCardTemplateList::OnSelectionChanged(TSharedPtr<FLightCardTemplateTreeItem> InItem, ESelectInfo::Type SelectInfo)
{
	if (bSpawnOnSelection && (SelectInfo == ESelectInfo::OnKeyPress || SelectInfo == ESelectInfo::OnMouseClick) && InItem.IsValid() && LightCardEditorPtr.IsValid())
	{
		LightCardEditorPtr.Pin()->SpawnActor(InItem->LightCardTemplate.Get());
		// Close the menu containing this widget
		FSlateApplication::Get().DismissAllMenus();
	}
}

void SDisplayClusterLightCardTemplateList::OnFilterTextChanged(const FText& SearchText)
{
	ApplyFilter(SearchText.ToString(), bFilterFavorites);
}

void SDisplayClusterLightCardTemplateList::OnAssetAddedOrRemoved(const FAssetData& InAssetData)
{
	const IAssetRegistry& AssetRegistry = FModuleManager::GetModuleChecked<FAssetRegistryModule>(AssetRegistryConstants::ModuleName).Get();
	if (AssetRegistry.IsLoadingAssets())
	{
		// This will be triggered once assets have finished loaded.
		return;
	}
	
	if (InAssetData.IsValid() && !InAssetData.IsRedirector())
	{
		if (InAssetData.AssetClassPath == UDisplayClusterLightCardTemplate::StaticClass()->GetClassPathName())
		{
			RefreshTemplateList();
		}
	}
}

void SDisplayClusterLightCardTemplateList::OnAssetsLoaded()
{
	RefreshTemplateList();
}

#undef LOCTEXT_NAMESPACE
