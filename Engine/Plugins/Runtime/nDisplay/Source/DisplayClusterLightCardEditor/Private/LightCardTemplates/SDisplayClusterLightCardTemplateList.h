// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FDisplayClusterLightCardEditor;
class UDisplayClusterLightCardTemplate;
class ITableRow;
class STableViewBase;
template<class T>
class STreeView;

/** Displays available light card templates for users to drop into the light card editor scene */
class SDisplayClusterLightCardTemplateList : public SCompoundWidget
{
public:
	struct FLightCardTemplateTreeItem
	{
		TWeakObjectPtr<UDisplayClusterLightCardTemplate> LightCardTemplate;
		FName TemplateName;
		TSharedPtr<FSlateBrush> SlateBrush;

		FLightCardTemplateTreeItem()
		{ }

		/** If the template is a user favorite */
		bool IsFavorite() const;
	};
	
	SLATE_BEGIN_ARGS(SDisplayClusterLightCardTemplateList)
	{}
	SLATE_ARGUMENT(bool, HideHeader)
	SLATE_ARGUMENT(bool, SpawnOnSelection)
	SLATE_END_ARGS()

	virtual ~SDisplayClusterLightCardTemplateList() override;

	void Construct(const FArguments& InArgs, TSharedPtr<FDisplayClusterLightCardEditor> InLightCardEditor);

	/** Find available templates and refresh the list */
	void RefreshTemplateList();

	/** Apply a filter to the template list */
	void ApplyFilter(const FString& InFilterText, bool bFavorite);
	
	/** Reapply the current filter */
	void ApplyFilter();

	/** If drag & drop should be used */
	bool IsDragDropEnabled() const;
	
private:
	TSharedRef<ITableRow> GenerateTreeItemRow(TSharedPtr<FLightCardTemplateTreeItem> Item, const TSharedRef<STableViewBase>& OwnerTable);
	void GetChildrenForTreeItem(TSharedPtr<FLightCardTemplateTreeItem> InItem, TArray<TSharedPtr<FLightCardTemplateTreeItem>>& OutChildren);
	const FSlateBrush* GetFavoriteFilterIcon() const;
	void OnSelectionChanged(TSharedPtr<FLightCardTemplateTreeItem> InItem, ESelectInfo::Type SelectInfo);
	
	void OnFilterTextChanged(const FText& SearchText);
	
	void OnAssetAddedOrRemoved(const FAssetData& InAssetData);
	void OnAssetsLoaded();
	
private:
	/** Pointer to the light card editor that owns this widget */
	TWeakPtr<FDisplayClusterLightCardEditor> LightCardEditorPtr;

	/** A list of all light card tree items to be displayed in the tree view */
	TArray<TSharedPtr<FLightCardTemplateTreeItem>> LightCardTemplateTree;

	/** The visible list of light card tree items filtered by the user */
	TArray<TSharedPtr<FLightCardTemplateTreeItem>> FilteredLightCardTemplateTree;
	
	/** The tree view widget displaying the light card templates */
	TSharedPtr<STreeView<TSharedPtr<FLightCardTemplateTreeItem>>> LightCardTemplateTreeView;

	/** The search string text */
	FString FilterText;

	/** If the favorite filter should be applied */
	bool bFilterFavorites = false;
 
	/** If selecting a template should spawn it */
	bool bSpawnOnSelection = false;
};