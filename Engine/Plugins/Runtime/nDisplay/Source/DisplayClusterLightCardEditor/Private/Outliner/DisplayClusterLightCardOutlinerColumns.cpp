// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterLightCardOutlinerColumns.h"

#include "DisplayClusterLightCardEditorStyle.h"

#include "ActorTreeItem.h"
#include "ISceneOutliner.h"
#include "ISceneOutlinerMode.h"
#include "SortHelper.h"
#include "Widgets/Input/SCheckBox.h"

#define LOCTEXT_NAMESPACE "DisplayClusterLightCardOutlinerColumns"

/**
 * Sets the 'Hidden in Game' field. Similar to SVisibilityWidget but is implemented as a checkbox
 * and can produce an undetermined state.
 */
class SHiddenInGameVisibilityWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SHiddenInGameVisibilityWidget) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TWeakPtr<FSceneOutlinerGutter> InWeakColumn, TWeakPtr<ISceneOutliner> InWeakOutliner,
		TWeakPtr<ISceneOutlinerTreeItem> InWeakTreeItem, const STableRow<FSceneOutlinerTreeItemPtr>* InRow)
	{
		WeakColumn = InWeakColumn;
		WeakOutliner = InWeakOutliner;
		WeakTreeItem = InWeakTreeItem;
		Row = InRow;
		
		ActorHiddenBrush = FDisplayClusterLightCardEditorStyle::Get().GetBrush(TEXT("DisplayClusterLightCardEditor.ActorHidden"));
		ActorNotHiddenBrush = FDisplayClusterLightCardEditorStyle::Get().GetBrush(TEXT("DisplayClusterLightCardEditor.ActorNotHidden"));

		ChildSlot
		[
			SNew(SCheckBox)
			.Padding(FMargin(0.f))
			.Visibility(this, &SHiddenInGameVisibilityWidget::GetCheckBoxVisibility)
			.ForegroundColor(this, &SHiddenInGameVisibilityWidget::GetForegroundColor)
			.IsChecked(this, &SHiddenInGameVisibilityWidget::IsChecked)
			.OnCheckStateChanged(this, &SHiddenInGameVisibilityWidget::OnCheckBoxStateChanged)
			.BackgroundImage(FAppStyle::GetNoBrush())
			.BackgroundHoveredImage(FAppStyle::GetNoBrush())
			.BackgroundPressedImage(FAppStyle::GetNoBrush())
			.CheckedImage(ActorHiddenBrush)
			.CheckedHoveredImage(ActorHiddenBrush)
			.CheckedPressedImage(ActorHiddenBrush)
			.UncheckedImage(ActorNotHiddenBrush)
			.UncheckedHoveredImage(ActorNotHiddenBrush)
			.UncheckedPressedImage(ActorNotHiddenBrush)
			.UndeterminedImage(ActorNotHiddenBrush)
			.UndeterminedHoveredImage(ActorNotHiddenBrush)
			.UndeterminedPressedImage(ActorNotHiddenBrush)
		];
	}

	TWeakPtr<FSceneOutlinerGutter> WeakColumn;
	TWeakPtr<ISceneOutliner> WeakOutliner;
	TWeakPtr<ISceneOutlinerTreeItem> WeakTreeItem;
	const STableRow<FSceneOutlinerTreeItemPtr>* Row;

	const FSlateBrush* ActorNotHiddenBrush;
	const FSlateBrush* ActorHiddenBrush;
	
protected:

	virtual FSlateColor GetForegroundColor() const override
	{
		if (WeakOutliner.IsValid() && WeakTreeItem.IsValid())
		{
			const TSharedPtr<ISceneOutliner> Outliner = WeakOutliner.Pin();
			const TSharedPtr<ISceneOutlinerTreeItem> TreeItem = WeakTreeItem.Pin();

			const bool bIsSelected = Outliner->GetTree().IsItemSelected(TreeItem.ToSharedRef());

			// Highlights when hovered
			if (IsHovered() && !bIsSelected)
			{
				return FAppStyle::Get().GetSlateColor("Colors.ForegroundHover");
			}
		}

		return FSlateColor::UseForeground();
	}
	
	EVisibility GetCheckBoxVisibility() const
	{
		// Only display if hovered, selected, or the checkbox is explicitly set to hidden in game
		const bool bVisible = Row->IsHovered() || Row->IsSelected() || IsChecked() == ECheckBoxState::Checked;
		return bVisible ? EVisibility::Visible : EVisibility::Collapsed;
	}

	ECheckBoxState IsChecked() const
	{
		if (WeakTreeItem.IsValid())
		{
			return GetCheckStateRecursive(*WeakTreeItem.Pin());
		}

		return ECheckBoxState::Unchecked;
	}
	
	void OnCheckBoxStateChanged(ECheckBoxState NewState)
	{
		if (WeakTreeItem.IsValid())
		{
			const bool bHidden = NewState == ECheckBoxState::Checked;

			FScopedTransaction ScopedTransaction(NSLOCTEXT("DisplayClusterLightCardOutlinerHiddenInGameColumn","SetHiddenInGame", "Set Hidden In Game"));

			const STreeView<FSceneOutlinerTreeItemPtr>& Tree = WeakOutliner.Pin()->GetTree();

			// Update all selected items if this item is selected
			if (Tree.IsItemSelected(WeakTreeItem.Pin()))
			{
				for (TSharedPtr<ISceneOutlinerTreeItem>& SelectedItem : Tree.GetSelectedItems())
				{
					SetHiddenInGame(*SelectedItem, bHidden);	
				}
			}
			else
			{
				SetHiddenInGame(*WeakTreeItem.Pin(), bHidden);	
			}
		}
	}
	
	/** Determine the check state based on the given item and all of its children states */
	static ECheckBoxState GetCheckStateRecursive(const ISceneOutlinerTreeItem& Item)
	{
		TOptional<ECheckBoxState> CheckBoxState;
		
		if (Item.IsA<FActorTreeItem>())
		{
			const FActorTreeItem* ActorTreeItem = Item.CastTo<FActorTreeItem>();
			if (ActorTreeItem->Actor.IsValid())
			{
				CheckBoxState = ActorTreeItem->Actor->IsHidden() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			}
		}
		
		for (const TWeakPtr<ISceneOutlinerTreeItem>& ChildPtr : Item.GetChildren())
		{
			TSharedPtr<ISceneOutlinerTreeItem> Child = ChildPtr.Pin();
			if (Child.IsValid())
			{
				const ECheckBoxState ChildCheckBoxState = GetCheckStateRecursive(*Child);
				if (CheckBoxState.IsSet() && *CheckBoxState != ChildCheckBoxState)
				{
					CheckBoxState = ECheckBoxState::Undetermined;
					break;
				}
				
				CheckBoxState = ChildCheckBoxState;
			}
		}

		return CheckBoxState.Get(ECheckBoxState::Unchecked);
	}

	static void SetHiddenInGame(ISceneOutlinerTreeItem& Item, const bool bNewValue)
	{
		if (Item.IsValid() && Item.IsA<FActorTreeItem>())
		{
			const FActorTreeItem* ActorTreeItem = Item.CastTo<FActorTreeItem>();
			if (ActorTreeItem->Actor.IsValid())
			{
				ActorTreeItem->Actor->Modify();
				ActorTreeItem->Actor->SetActorHiddenInGame(bNewValue);
			}
		}

		for (const TWeakPtr<ISceneOutlinerTreeItem>& ChildPtr : Item.GetChildren())
		{
			TSharedPtr<ISceneOutlinerTreeItem> Child = ChildPtr.Pin();
			if (Child.IsValid())
			{
				SetHiddenInGame(*Child, bNewValue);
			}
		}
	}
};

FName FDisplayClusterLightCardOutlinerHiddenInGameColumn::GetID()
{
	static FName HiddenInGame("Hidden in Game");
	return HiddenInGame;
}

FText FDisplayClusterLightCardOutlinerHiddenInGameColumn::GetDisplayText()
{
	return LOCTEXT("HiddenInGameColumnText", "Hidden in Game");
}

SHeaderRow::FColumn::FArguments FDisplayClusterLightCardOutlinerHiddenInGameColumn::ConstructHeaderRowColumn()
{
	return SHeaderRow::Column(GetColumnID())
		.FixedWidth(24.f)
		.HAlignHeader(HAlign_Left)
		.VAlignHeader(VAlign_Center)
		.HAlignCell(HAlign_Center)
		.VAlignCell(VAlign_Center)
		.DefaultTooltip(FText::FromName(GetColumnID()))
		.HeaderContentPadding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
		[
			SNew(SImage)
			.ColorAndOpacity(FSlateColor::UseForeground())
			.Image(FDisplayClusterLightCardEditorStyle::Get().GetBrush(TEXT("DisplayClusterLightCardEditor.ActorNotHidden")))
		];
}

const TSharedRef<SWidget> FDisplayClusterLightCardOutlinerHiddenInGameColumn::ConstructRowWidget(
	FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row)
{
	return SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Center)
		[
			SNew(SHiddenInGameVisibilityWidget, SharedThis(this), WeakOutliner, TreeItem, &Row)
		];
}

void FDisplayClusterLightCardOutlinerHiddenInGameColumn::SortItems(TArray<FSceneOutlinerTreeItemPtr>& RootItems, const EColumnSortMode::Type SortMode) const
{
	FSceneOutlinerSortHelper<int32, bool>()
		/** Sort by type first */
		.Primary([this](const ISceneOutlinerTreeItem& Item){ return WeakOutliner.Pin()->GetMode()->GetTypeSortPriority(Item); }, SortMode)
		/** Then by hidden in game */
		.Secondary([](const ISceneOutlinerTreeItem& Item)
		{
			if (const FActorTreeItem* ActorTreeItem = Item.CastTo<FActorTreeItem>())
			{
				if (ActorTreeItem->Actor.IsValid())
				{
					return ActorTreeItem->Actor->IsHidden();
				}
			}

			return false;
		}, SortMode)
		.Sort(RootItems);
}

#undef LOCTEXT_NAMESPACE