// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneOutlinerFwd.h"
#include "SceneOutlinerGutter.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class ISceneOutliner;
class SWidget;
template<typename ItemType> class STableRow;

class FDisplayClusterLightCardOutlinerHiddenInGameColumn : public FSceneOutlinerGutter
{
public:
	FDisplayClusterLightCardOutlinerHiddenInGameColumn(ISceneOutliner& SceneOutliner) : FSceneOutlinerGutter(SceneOutliner) {}
	virtual ~FDisplayClusterLightCardOutlinerHiddenInGameColumn() {}
	static FName GetID();
	static FText GetDisplayText();
	
	// Begin ISceneOutlinerColumn Implementation
	virtual FName GetColumnID() override { return GetID(); }
	virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn() override;
	virtual const TSharedRef<SWidget> ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row) override;
	virtual bool SupportsSorting() const override { return true; }
	virtual void SortItems(TArray<FSceneOutlinerTreeItemPtr>& OutItems, const EColumnSortMode::Type SortMode) const override;
	// End ISceneOutlinerColumn Implementation
};