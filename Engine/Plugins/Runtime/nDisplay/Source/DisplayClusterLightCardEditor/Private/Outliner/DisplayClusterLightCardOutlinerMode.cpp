// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterLightCardOutlinerMode.h"

#include "DisplayClusterLightCardEditorLog.h"
#include "Logging/MessageLog.h"
#include "SDisplayClusterLightCardOutliner.h"

#include "ActorEditorUtils.h"
#include "ActorFolderPickingMode.h"
#include "ActorTreeItem.h"
#include "ComponentTreeItem.h"
#include "Editor.h"
#include "EditorActorFolders.h"
#include "EditorLevelUtils.h"
#include "EditorViewportCommands.h"
#include "FolderTreeItem.h"
#include "LevelTreeItem.h"
#include "SceneOutlinerMenuContext.h"
#include "ScopedTransaction.h"
#include "Selection.h"
#include "ToolMenus.h"
#include "WorldTreeItem.h"
#include "DragAndDrop/ActorDragDropGraphEdOp.h"
#include "DragAndDrop/FolderDragDropOp.h"
#include "SSocketChooser.h"
#include "HAL/PlatformApplicationMisc.h"
#include "LevelInstance/LevelInstanceInterface.h"
#include "LevelInstance/LevelInstanceSubsystem.h"

#define LOCTEXT_NAMESPACE "DisplayClusterLightCardOutlinerMode"

namespace UE::LightCardOutlinerMode
{
	static const FName DefaultContextBaseMenuName("LightCardOutliner.DefaultContextMenuBase");
	static const FName DefaultContextMenuName("LightCardOutliner.DefaultContextMenu");
}

struct FFolderPathSelector
{
	bool operator()(TWeakPtr<ISceneOutlinerTreeItem> Item, FFolder& DataOut) const
	{
		if (const FFolderTreeItem* FolderItem = Item.Pin()->CastTo<FFolderTreeItem>())
		{
			if (FolderItem->IsValid())
			{
				DataOut = FolderItem->GetFolder();
				return true;
			}
		}
		return false;
	}
};

struct FWeakActorSelector
{
	bool operator()(const TWeakPtr<ISceneOutlinerTreeItem>& Item, TWeakObjectPtr<AActor>& DataOut) const
	{
		if (TSharedPtr<ISceneOutlinerTreeItem> ItemPtr = Item.Pin())
		{
			if (FActorTreeItem* ActorItem = ItemPtr->CastTo<FActorTreeItem>())
			{
				if (ActorItem->IsValid())
				{
					DataOut = ActorItem->Actor;
					return true;
				}
			}
		}
		return false;
	}
};

struct FActorSelector
{
	bool operator()(const TWeakPtr<ISceneOutlinerTreeItem>& Item, AActor*& ActorPtrOut) const
	{
		if (const TSharedPtr<ISceneOutlinerTreeItem> ItemPtr = Item.Pin())
		{
			if (const FActorTreeItem* ActorItem = ItemPtr->CastTo<FActorTreeItem>())
			{
				if (ActorItem->IsValid())
				{
					if (AActor* Actor = ActorItem->Actor.Get())
					{
						ActorPtrOut = Actor;
						return true;
					}
				}
			}
			else if (const FComponentTreeItem* ComponentItem = ItemPtr->CastTo<FComponentTreeItem>())
			{
				if (ComponentItem->IsValid())
				{
					if (AActor* Actor = ComponentItem->Component->GetOwner())
					{
						ActorPtrOut = Actor;
						return true;
					}
				}
			}
		}

		return false;
	}
};

FDisplayClusterLightCardOutlinerMode::FDisplayClusterLightCardOutlinerMode(SSceneOutliner* InSceneOutliner,
	TWeakPtr<SDisplayClusterLightCardOutliner> InLightCardOutliner, TWeakObjectPtr<UWorld> InSpecifiedWorldToDisplay)
	: FActorMode(FActorModeParams(InSceneOutliner, InSpecifiedWorldToDisplay,
	/* bHideComponents */ true, /* bHideLevelInstanceHierarchy */ true, /* bHideUnloadedActors */ true, /* bHideEmptyFolders */ true)),
	LightCardOutliner(InLightCardOutliner)
{
	FEditorDelegates::OnEditCutActorsBegin.AddRaw(this, &FDisplayClusterLightCardOutlinerMode::OnEditCutActorsBegin);
	FEditorDelegates::OnEditCutActorsEnd.AddRaw(this, &FDisplayClusterLightCardOutlinerMode::OnEditCutActorsEnd);
	FEditorDelegates::OnEditCopyActorsBegin.AddRaw(this, &FDisplayClusterLightCardOutlinerMode::OnEditCopyActorsBegin);
	FEditorDelegates::OnEditCopyActorsEnd.AddRaw(this, &FDisplayClusterLightCardOutlinerMode::OnEditCopyActorsEnd);
	FEditorDelegates::OnEditPasteActorsBegin.AddRaw(this, &FDisplayClusterLightCardOutlinerMode::OnEditPasteActorsBegin);
	FEditorDelegates::OnEditPasteActorsEnd.AddRaw(this, &FDisplayClusterLightCardOutlinerMode::OnEditPasteActorsEnd);
	FEditorDelegates::OnDuplicateActorsBegin.AddRaw(this, &FDisplayClusterLightCardOutlinerMode::OnDuplicateActorsBegin);
	FEditorDelegates::OnDuplicateActorsEnd.AddRaw(this, &FDisplayClusterLightCardOutlinerMode::OnDuplicateActorsEnd);
	FEditorDelegates::OnDeleteActorsBegin.AddRaw(this, &FDisplayClusterLightCardOutlinerMode::OnDeleteActorsBegin);
	FEditorDelegates::OnDeleteActorsEnd.AddRaw(this, &FDisplayClusterLightCardOutlinerMode::OnDeleteActorsEnd);
}

FDisplayClusterLightCardOutlinerMode::~FDisplayClusterLightCardOutlinerMode()
{
	FEditorDelegates::OnEditCutActorsBegin.RemoveAll(this);
	FEditorDelegates::OnEditCutActorsEnd.RemoveAll(this);
	FEditorDelegates::OnEditCopyActorsBegin.RemoveAll(this);
	FEditorDelegates::OnEditCopyActorsEnd.RemoveAll(this);
	FEditorDelegates::OnEditPasteActorsBegin.RemoveAll(this);
	FEditorDelegates::OnEditPasteActorsEnd.RemoveAll(this);
	FEditorDelegates::OnDuplicateActorsBegin.RemoveAll(this);
	FEditorDelegates::OnDuplicateActorsEnd.RemoveAll(this);
	FEditorDelegates::OnDeleteActorsBegin.RemoveAll(this);
	FEditorDelegates::OnDeleteActorsEnd.RemoveAll(this);
}

void FDisplayClusterLightCardOutlinerMode::SynchronizeSelection()
{
	if (LightCardOutliner.IsValid())
	{
		// This can be called from undo/redo and the internal selection will be cleared. The cached selection
		// of the lightcard outliner will contain the last valid selection.
		LightCardOutliner.Pin()->RestoreCachedSelection();
	}
}

FCreateSceneOutlinerMode FDisplayClusterLightCardOutlinerMode::CreateFolderPickerMode(
	const FFolder::FRootObject& InRootObject) const
{
	auto MoveSelectionTo = [this, InRootObject](const FSceneOutlinerTreeItemRef& NewParent)
	{
		if (FWorldTreeItem* WorldItem = NewParent->CastTo<FWorldTreeItem>())
		{
			SceneOutliner->MoveSelectionTo(GetWorldDefaultRootFolder());
		}
		else if (FFolderTreeItem* FolderItem = NewParent->CastTo<FFolderTreeItem>())
		{
			SceneOutliner->MoveSelectionTo(FolderItem->GetFolder());
		}
		else if (FActorTreeItem* ActorItem = NewParent->CastTo<FActorTreeItem>())
		{
			if (FFolder::IsRootObjectValid(InRootObject))
			{
				SceneOutliner->MoveSelectionTo(FFolder(InRootObject));
			}
		}
		else if (FLevelTreeItem* LevelItem = NewParent->CastTo<FLevelTreeItem>())
		{
			if (FFolder::IsRootObjectValid(InRootObject))
			{
				SceneOutliner->MoveSelectionTo(FFolder(InRootObject));
			}
		}
	};

	return FCreateSceneOutlinerMode::CreateLambda([this, MoveSelectionTo, InRootObject](SSceneOutliner* Outliner)
	{
		return new FActorFolderPickingMode(Outliner, FOnSceneOutlinerItemPicked::CreateLambda(MoveSelectionTo), nullptr, InRootObject);
	});
}

TSharedPtr<SWidget> FDisplayClusterLightCardOutlinerMode::CreateContextMenu()
{
	RegisterContextMenu();

	const FSceneOutlinerItemSelection ItemSelection(SceneOutliner->GetSelection());

	USceneOutlinerMenuContext* ContextObject = NewObject<USceneOutlinerMenuContext>();
	ContextObject->SceneOutliner = StaticCastSharedRef<SSceneOutliner>(SceneOutliner->AsShared());
	ContextObject->bShowParentTree = SceneOutliner->GetSharedData().bShowParentTree;
	ContextObject->NumSelectedItems = ItemSelection.Num();
	ContextObject->NumSelectedFolders = ItemSelection.Num<FFolderTreeItem>();
	ContextObject->NumWorldsSelected = ItemSelection.Num<FWorldTreeItem>();
	ContextObject->bRepresentingGameWorld = false;
	ContextObject->bRepresentingPartitionedWorld = false;

	FToolMenuContext Context(ContextObject);

	FName MenuName = UE::LightCardOutlinerMode::DefaultContextMenuName;
	SceneOutliner->GetSharedData().ModifyContextMenu.ExecuteIfBound(MenuName, Context);

	// Build up the menu for a selection
	UToolMenus* ToolMenus = UToolMenus::Get();
	UToolMenu* Menu = ToolMenus->GenerateMenu(MenuName, Context);

	for (const FToolMenuSection& Section : Menu->Sections)
	{
		if (Section.Blocks.Num() > 0)
		{
			return ToolMenus->GenerateWidget(Menu);
		}
	}

	return nullptr;
}

void FDisplayClusterLightCardOutlinerMode::OnItemDoubleClick(FSceneOutlinerTreeItemPtr Item)
{
	if (const FActorTreeItem* ActorItem = Item->CastTo<FActorTreeItem>())
	{
		AActor* Actor = ActorItem->Actor.Get();
		check(Actor);

		// Assume the user wants this selected in the level viewport
		UpdateGlobalActorSelection();
		
		if (Item->CanInteract())
		{
			FSceneOutlinerItemSelection Selection(SceneOutliner->GetSelection());
			if (Selection.Has<FActorTreeItem>())
			{
				const bool bActiveViewportOnly = false;
				GEditor->MoveViewportCamerasToActor(Selection.GetData<AActor*>(FActorSelector()), bActiveViewportOnly);
			}
		}
		else
		{
			const bool bActiveViewportOnly = false;
			GEditor->MoveViewportCamerasToActor(*Actor, bActiveViewportOnly);
		}
	}
}

FReply FDisplayClusterLightCardOutlinerMode::OnKeyDown(const FKeyEvent& InKeyEvent)
{
	const FSceneOutlinerItemSelection& Selection = SceneOutliner->GetSelection();

	// Rename key: Rename selected actors (not rebindable, because it doesn't make much sense to bind.)
	if (InKeyEvent.GetKey() == EKeys::F2)
	{
		if (Selection.Num() == 1)
		{
			FSceneOutlinerTreeItemPtr ItemToRename = Selection.SelectedItems[0].Pin();

			if (ItemToRename.IsValid() && CanRenameItem(*ItemToRename) && ItemToRename->CanInteract())
			{
				SceneOutliner->SetPendingRenameItem(ItemToRename);
				SceneOutliner->ScrollItemIntoView(ItemToRename);
			}

			return FReply::Handled();
		}
	}

	// F5 forces a full refresh
	else if (InKeyEvent.GetKey() == EKeys::F5)
	{
		SceneOutliner->FullRefresh();
		return FReply::Handled();
	}

	// Delete key: Delete selected actors (not rebindable, because it doesn't make much sense to bind.)
	// Use Delete and Backspace instead of Platform_Delete because the LevelEditor default Edit Delete is bound to both
	else if (InKeyEvent.GetKey() == EKeys::Delete || InKeyEvent.GetKey() == EKeys::BackSpace)
	{
		if (SceneOutliner->GetSharedData().CustomDelete.IsBound())
		{
			SceneOutliner->GetSharedData().CustomDelete.Execute(Selection.SelectedItems);
		}
		else
		{
			if (RepresentingWorld.IsValid())
			{
				// Only handle deleting folders here.
				SceneOutliner->DeleteFoldersBegin();
				SceneOutliner->DeleteFoldersEnd();
			}
		}

		// Let the delete key get handled by the light card editor.
		return FReply::Unhandled();
	}

	/* Allow the user to scroll to the current selection (and expand if needed) by pressing the key bound to
	 * FEditorViewportCommands::Get().FocusViewportToSelection (Default: 'F')
	 */
	else
	{
		const FModifierKeysState ModifierKeys = FSlateApplication::Get().GetModifierKeys();
		const FInputChord CheckChord( InKeyEvent.GetKey(), EModifierKey::FromBools(ModifierKeys.IsControlDown(), ModifierKeys.IsAltDown(), ModifierKeys.IsShiftDown(), ModifierKeys.IsCommandDown()) );

		// Use the keyboard shortcut bound to 'Focus Viewport To Selection'
		if (FEditorViewportCommands::Get().FocusViewportToSelection->HasActiveChord(CheckChord))
		{
			if (Selection.Num() == 1)
			{
				FSceneOutlinerTreeItemPtr ItemToFocus = Selection.SelectedItems[0].Pin();

				if (ItemToFocus.IsValid())
				{
					SceneOutliner->ScrollItemIntoView(ItemToFocus);
				}

				// Return Unhandled here so that the level editor viewport can handle this event and focus the selected item
				return FReply::Unhandled();
			}
		}
	}

	return FReply::Unhandled();
}

bool FDisplayClusterLightCardOutlinerMode::CanDelete() const
{
	// Also handled in OnKeyDown for folders
	
	const FSceneOutlinerItemSelection ItemSelection = SceneOutliner->GetSelection();
	const uint32 NumberOfFolders = ItemSelection.Num<FFolderTreeItem>();
	return (NumberOfFolders > 0 && NumberOfFolders == ItemSelection.Num());
}

bool FDisplayClusterLightCardOutlinerMode::CanRename() const
{
	const FSceneOutlinerItemSelection ItemSelection = SceneOutliner->GetSelection();
	const uint32 NumberOfFolders = ItemSelection.Num<FFolderTreeItem>();
	return (NumberOfFolders == 1 && NumberOfFolders == ItemSelection.Num());
}

bool FDisplayClusterLightCardOutlinerMode::CanRenameItem(const ISceneOutlinerTreeItem& Item) const
{
	return (Item.IsValid() && (Item.IsA<FActorTreeItem>() || Item.IsA<FFolderTreeItem>()));
}

bool FDisplayClusterLightCardOutlinerMode::CanCut() const
{
	const FSceneOutlinerItemSelection ItemSelection = SceneOutliner->GetSelection();
	const uint32 NumberOfFolders = ItemSelection.Num<FFolderTreeItem>();
	return (NumberOfFolders > 0 && NumberOfFolders == ItemSelection.Num());
}

bool FDisplayClusterLightCardOutlinerMode::CanCopy() const
{
	const FSceneOutlinerItemSelection ItemSelection = SceneOutliner->GetSelection();
	const uint32 NumberOfFolders = ItemSelection.Num<FFolderTreeItem>();
	return (NumberOfFolders > 0 && NumberOfFolders == ItemSelection.Num());
}

bool FDisplayClusterLightCardOutlinerMode::CanPaste() const
{
	return CanPasteFoldersOnlyFromClipboard();
}

FFolder FDisplayClusterLightCardOutlinerMode::CreateNewFolder()
{
	const FScopedTransaction Transaction(LOCTEXT("UndoAction_CreateFolder", "Create Folder"));

	// FActorFolders calls will rely on GEditor selection for folder creation.
	UpdateGlobalActorSelection();
	
	TArray<FFolder> SelectedFolders = SceneOutliner->GetSelection().GetData<FFolder>(FFolderPathSelector());
	const FFolder NewFolderName = FActorFolders::Get().GetDefaultFolderForSelection(*RepresentingWorld, &SelectedFolders);
	FActorFolders::Get().CreateFolderContainingSelection(*RepresentingWorld, NewFolderName);

	return NewFolderName;
}

FFolder FDisplayClusterLightCardOutlinerMode::GetFolder(const FFolder& ParentPath, const FName& LeafName)
{
	return FActorFolders::Get().GetFolderName(*RepresentingWorld, ParentPath, LeafName);
}

bool FDisplayClusterLightCardOutlinerMode::CreateFolder(const FFolder& NewPath)
{
	return FActorFolders::Get().CreateFolder(*RepresentingWorld, NewPath);
}

bool FDisplayClusterLightCardOutlinerMode::ReparentItemToFolder(const FFolder& FolderPath, const FSceneOutlinerTreeItemPtr& Item)
{
	if (const FActorTreeItem* ActorItem = Item->CastTo<FActorTreeItem>())
	{
		// Make sure actor has the same root object before updating path
		if (ActorItem->Actor->GetFolderRootObject() == FolderPath.GetRootObject())
		{
			ActorItem->Actor->SetFolderPath_Recursively(FolderPath.GetPath());
			return true;
		}
	}
	return false;
}

TSharedPtr<FDragDropOperation> FDisplayClusterLightCardOutlinerMode::CreateDragDropOperation(
	const FPointerEvent& MouseEvent, const TArray<FSceneOutlinerTreeItemPtr>& InTreeItems) const
{
	FSceneOutlinerDragDropPayload DraggedObjects(InTreeItems);

	if (DraggedObjects.Has<FActorTreeItem>() && !DraggedObjects.Has<FFolderTreeItem>())
	{
		return FActorDragDropGraphEdOp::New(DraggedObjects.GetData<TWeakObjectPtr<AActor>>(FWeakActorSelector()));
	}

	TSharedPtr<FSceneOutlinerDragDropOp> OutlinerOp = MakeShareable(new FSceneOutlinerDragDropOp());

	if (DraggedObjects.Has<FActorTreeItem>())
	{
		TSharedPtr<FActorDragDropOp> ActorOperation = MakeShareable(new FActorDragDropGraphEdOp);
		ActorOperation->Init(DraggedObjects.GetData<TWeakObjectPtr<AActor>>(FWeakActorSelector()));
		OutlinerOp->AddSubOp(ActorOperation);
	}

	if (DraggedObjects.Has<FFolderTreeItem>())
	{
		FFolder::FRootObject CommonRootObject;
		TArray<FName> DraggedFolders;
		if (GetFolderNamesFromPayload(DraggedObjects, DraggedFolders, CommonRootObject))
		{
			TSharedPtr<FFolderDragDropOp> FolderOperation = MakeShareable(new FFolderDragDropOp);
			FolderOperation->Init(DraggedFolders, RepresentingWorld.Get(), CommonRootObject);
			OutlinerOp->AddSubOp(FolderOperation);
		}
	}
	OutlinerOp->Construct();
	return OutlinerOp;
}

bool FDisplayClusterLightCardOutlinerMode::ParseDragDrop(FSceneOutlinerDragDropPayload& OutPayload,
	const FDragDropOperation& Operation) const
{
	if (Operation.IsOfType<FSceneOutlinerDragDropOp>())
	{
		const FSceneOutlinerDragDropOp& OutlinerOp = static_cast<const FSceneOutlinerDragDropOp&>(Operation);
		if (const TSharedPtr<const FFolderDragDropOp>& FolderOp = OutlinerOp.GetSubOp<FFolderDragDropOp>())
		{
			for (const FName& Folder : FolderOp->Folders)
			{
				OutPayload.DraggedItems.Add(SceneOutliner->GetTreeItem(FFolder(FolderOp->RootObject, Folder)));
			}
		}
		if (const TSharedPtr<const FActorDragDropOp>& ActorOp = OutlinerOp.GetSubOp<FActorDragDropOp>())
		{
			for (const TWeakObjectPtr<AActor>& Actor : ActorOp->Actors)
			{
				OutPayload.DraggedItems.Add(SceneOutliner->GetTreeItem(Actor.Get()));
			}
		}
		return true;
	}
	else if (Operation.IsOfType<FActorDragDropOp>())
	{
		for (const TWeakObjectPtr<AActor>& Actor : static_cast<const FActorDragDropOp&>(Operation).Actors)
		{
			OutPayload.DraggedItems.Add(SceneOutliner->GetTreeItem(Actor.Get()));
		}
		return true;
	}

	return false;
}

FSceneOutlinerDragValidationInfo FDisplayClusterLightCardOutlinerMode::ValidateDrop(
	const ISceneOutlinerTreeItem& DropTarget, const FSceneOutlinerDragDropPayload& Payload) const
{
	if (Payload.Has<FFolderTreeItem>())
	{
		FFolder::FRootObject TargetRootObject = DropTarget.GetRootObject();
		FFolder::FRootObject CommonPayloadFoldersRootObject;
		TArray<FName> PayloadFolders;
		const bool bHasCommonRootObject = GetFolderNamesFromPayload(Payload, PayloadFolders, CommonPayloadFoldersRootObject);
		if (!bHasCommonRootObject)
		{
			return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, LOCTEXT("CantMoveFoldersWithMultipleRoots", "Cannot move folders with multiple roots"));
		}
		else if (CommonPayloadFoldersRootObject != TargetRootObject)
		{
			return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, LOCTEXT("CantChangeFoldersRoot", "Cannot change folders root"));
		}
	}

	if (const FActorTreeItem* ActorItem = DropTarget.CastTo<FActorTreeItem>())
	{
		const AActor* ActorTarget = ActorItem->Actor.Get();
		if (!ActorTarget)
		{
			return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, FText());
		}

		const ILevelInstanceInterface* LevelInstanceTarget = Cast<ILevelInstanceInterface>(ActorTarget);
		const ULevelInstanceSubsystem* LevelInstanceSubsystem = RepresentingWorld->GetSubsystem<ULevelInstanceSubsystem>();
		if (LevelInstanceTarget)
		{
			check(LevelInstanceSubsystem);
			if (!LevelInstanceTarget->IsEditing())
			{
				return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, LOCTEXT("Error_AttachToClosedLevelInstance", "Cannot attach to LevelInstance which is not being edited"));
			}
		}
		else
		{
			if (Payload.Has<FFolderTreeItem>())
			{
				return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, LOCTEXT("FoldersOnActorError", "Cannot attach folders to actors"));
			}

			if (!Payload.Has<FActorTreeItem>())
			{
				return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, FText());
			}
		}

		FText AttachErrorMsg;
		bool bCanAttach = true;
		bool bDraggedOntoAttachmentParent = true;
		const TArray<TWeakObjectPtr<AActor>>& DragActors = Payload.GetData<TWeakObjectPtr<
			AActor>>(FWeakActorSelector());
		for (const TWeakObjectPtr<AActor>& DragActorPtr : DragActors)
		{
			AActor* DragActor = DragActorPtr.Get();
			if (DragActor)
			{
				if (bCanAttach)
				{
					if (LevelInstanceSubsystem)
					{
						// Either all actors must be in a LevelInstance or none of them
						if (const ILevelInstanceInterface* ParentLevelInstance = LevelInstanceSubsystem->GetParentLevelInstance(DragActor))
						{
							if (!ParentLevelInstance->IsEditing())
							{
								return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, LOCTEXT("Error_RemoveEditingLevelInstance", "Cannot detach from a LevelInstance which is not being edited"));
							}
						}

						if (!LevelInstanceSubsystem->CanMoveActorToLevel(DragActor, &AttachErrorMsg))
						{
							bCanAttach = bDraggedOntoAttachmentParent = false;
							break;
						}
					}

					if (DragActor->IsChildActor())
					{
						AttachErrorMsg = FText::Format(LOCTEXT("Error_AttachChildActor", "Cannot move {0} as it is a child actor."), FText::FromString(DragActor->GetActorLabel()));
						bCanAttach = bDraggedOntoAttachmentParent = false;
						break;
					}
					if (!LevelInstanceTarget && !GEditor->CanParentActors(ActorTarget, DragActor, &AttachErrorMsg))
					{
						bCanAttach = false;
					}
				}

				if (DragActor->GetSceneOutlinerParent() != ActorTarget)
				{
					bDraggedOntoAttachmentParent = false;
				}
			}
		}

		const FText ActorLabel = FText::FromString(ActorTarget->GetActorLabel());
		if (bDraggedOntoAttachmentParent)
		{
			if (DragActors.Num() == 1)
			{
				return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::CompatibleDetach, ActorLabel);
			}
			else
			{
				return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::CompatibleMultipleDetach, ActorLabel);
			}
		}
		else if (bCanAttach)
		{
			if (DragActors.Num() == 1)
			{
				return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::CompatibleAttach, ActorLabel);
			}
			else
			{
				return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::CompatibleMultipleAttach, ActorLabel);
			}
		}
		else
		{
			if (DragActors.Num() == 1)
			{
				return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, AttachErrorMsg);
			}
			else
			{
				const FText ReasonText = FText::Format(LOCTEXT("DropOntoText", "{0}. {1}"), ActorLabel, AttachErrorMsg);
				return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleMultipleAttach, ReasonText);
			}
		}
	}
	else if (DropTarget.IsA<FFolderTreeItem>() || DropTarget.IsA<FWorldTreeItem>() || DropTarget.IsA<FLevelTreeItem>())
	{
		const FFolderTreeItem* FolderItem = DropTarget.CastTo<FFolderTreeItem>();
		const FWorldTreeItem* WorldItem = DropTarget.CastTo<FWorldTreeItem>();
		const FLevelTreeItem* LevelItem = DropTarget.CastTo<FLevelTreeItem>();
		// WorldTreeItem and LevelTreeItem are treated as root folders (path = none), with the difference that LevelTreeItem has a RootObject.
		const FFolder DestinationPath = FolderItem ? FolderItem->GetFolder() : (LevelItem ? FFolder(FFolder::GetOptionalFolderRootObject(LevelItem->Level.Get()).Get(FFolder::GetInvalidRootObject())) : GetWorldDefaultRootFolder());
		const FFolder::FRootObject& DestinationRootObject = DestinationPath.GetRootObject();
		ILevelInstanceInterface* LevelInstanceTarget = Cast<ILevelInstanceInterface>(DestinationPath.GetRootObjectPtr());
		if (LevelInstanceTarget && !LevelInstanceTarget->IsEditing())
		{
			return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, LOCTEXT("Error_DragInNonEditingLevelInstance", "Cannot drag into a LevelInstance which is not being edited"));
		}

		if (Payload.Has<FFolderTreeItem>())
		{
			FFolder::FRootObject CommonFolderRootObject;
			TArray<FName> DraggedFolders;
			if (GetFolderNamesFromPayload(Payload, DraggedFolders, CommonFolderRootObject))
			{
				// Iterate over all the folders that have been dragged
				for (const FName& DraggedFolder : DraggedFolders)
				{
					const FName Leaf = FEditorFolderUtils::GetLeafName(DraggedFolder);
					const FName Parent = FEditorFolderUtils::GetParentPath(DraggedFolder);

					if ((CommonFolderRootObject != DestinationRootObject) && FFolder::IsRootObjectValid(CommonFolderRootObject) && FFolder::IsRootObjectValid(DestinationRootObject))
					{
						FFormatNamedArguments Args;
						Args.Add(TEXT("SourceName"), FText::FromName(Leaf));
						FText Text = FText::Format(LOCTEXT("CantChangeFolderRoot", "Cannot change {SourceName} folder root"), Args);
						return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, Text);
					}

					if (Parent == DestinationPath.GetPath())
					{
						FFormatNamedArguments Args;
						Args.Add(TEXT("SourceName"), FText::FromName(Leaf));

						FText Text;
						if (DestinationPath.IsNone())
						{
							Text = FText::Format(LOCTEXT("FolderAlreadyAssignedRoot", "{SourceName} is already assigned to root"), Args);
						}
						else
						{
							Args.Add(TEXT("DestPath"), FText::FromName(DestinationPath.GetPath()));
							Text = FText::Format(LOCTEXT("FolderAlreadyAssigned", "{SourceName} is already assigned to {DestPath}"), Args);
						}

						return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, Text);
					}

					const FString DragFolderPath = DraggedFolder.ToString();
					const FString LeafName = Leaf.ToString();
					const FString DstFolderPath = DestinationPath.IsNone() ? FString() : DestinationPath.ToString();
					const FString NewPath = DstFolderPath / LeafName;

					if (FActorFolders::Get().ContainsFolder(*RepresentingWorld, FFolder(DestinationRootObject, FName(*NewPath))))
					{
						// The folder already exists
						FFormatNamedArguments Args;
						Args.Add(TEXT("DragName"), FText::FromString(LeafName));
						return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric,
							FText::Format(LOCTEXT("FolderAlreadyExistsRoot", "A folder called \"{DragName}\" already exists at this level"), Args));
					}
					else if (DragFolderPath == DstFolderPath || DstFolderPath.StartsWith(DragFolderPath + "/"))
					{
						// Cannot drag as a child of itself
						FFormatNamedArguments Args;
						Args.Add(TEXT("FolderPath"), FText::FromName(DraggedFolder));
						return FSceneOutlinerDragValidationInfo(
							ESceneOutlinerDropCompatibility::IncompatibleGeneric,
							FText::Format(LOCTEXT("ChildOfItself", "Cannot move \"{FolderPath}\" to be a child of itself"), Args));
					}
				}
			}
		}

		if (Payload.Has<FActorTreeItem>())
		{
			const ULevelInstanceSubsystem* LevelInstanceSubsystem = RepresentingWorld->GetSubsystem<ULevelInstanceSubsystem>();
			// Iterate over all the actors that have been dragged
			for (const TWeakObjectPtr<AActor>& WeakActor : Payload.GetData<TWeakObjectPtr<AActor>>(FWeakActorSelector()))
			{
				const AActor* Actor = WeakActor.Get();

				bool bActorContainedInLevelInstance = false;
				if (LevelInstanceSubsystem)
				{
					if (const ILevelInstanceInterface* ParentLevelInstance = LevelInstanceSubsystem->GetParentLevelInstance(Actor))
					{
						if (!ParentLevelInstance->IsEditing())
						{
							return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, LOCTEXT("Error_RemoveEditingLevelInstance", "Cannot detach from a LevelInstance which is not being edited"));
						}
						bActorContainedInLevelInstance = true;
					}

					if (const ILevelInstanceInterface* LevelInstance = Cast<ILevelInstanceInterface>(Actor))
					{
						FText Reason;
						if (!LevelInstanceSubsystem->CanMoveActorToLevel(Actor, &Reason))
						{
							return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, Reason);
						}
					}
				}

				if (Actor->IsChildActor())
				{
					return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, FText::Format(LOCTEXT("Error_AttachChildActor", "Cannot move {0} as it is a child actor."), FText::FromString(Actor->GetActorLabel())));
				}
				else if ((Actor->GetFolderRootObject() != DestinationRootObject) && FFolder::IsRootObjectValid(Actor->GetFolderRootObject()) && FFolder::IsRootObjectValid(DestinationRootObject))
				{
					FFormatNamedArguments Args;
					Args.Add(TEXT("SourceName"), FText::FromString(Actor->GetActorLabel()));
					FText Text = FText::Format(LOCTEXT("CantChangeActorRoot", "Cannot change {SourceName} folder root"), Args);
					return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, Text);
				}
				else if (Actor->GetFolder() == DestinationPath && !Actor->GetSceneOutlinerParent() && !bActorContainedInLevelInstance)
				{
					FFormatNamedArguments Args;
					Args.Add(TEXT("SourceName"), FText::FromString(Actor->GetActorLabel()));

					FText Text;
					if (DestinationPath.IsNone())
					{
						Text = FText::Format(LOCTEXT("FolderAlreadyAssignedRoot", "{SourceName} is already assigned to root"), Args);
					}
					else
					{
						Args.Add(TEXT("DestPath"), FText::FromName(DestinationPath.GetPath()));
						Text = FText::Format(LOCTEXT("FolderAlreadyAssigned", "{SourceName} is already assigned to {DestPath}"), Args);
					}

					return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, Text);
				}
			}
		}

		// Everything else is a valid operation
		if (DestinationPath.IsNone())
		{
			return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::CompatibleGeneric, LOCTEXT("MoveToRoot", "Move to root"));
		}
		else
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("DestPath"), FText::FromName(DestinationPath.GetPath()));
			return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::CompatibleGeneric, FText::Format(LOCTEXT("MoveInto", "Move into \"{DestPath}\""), Args));
		}
	}
	else if (DropTarget.IsA<FComponentTreeItem>())
	{
		// we don't allow drag and drop on components for now
		return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, FText());
	}
	return FSceneOutlinerDragValidationInfo::Invalid();
}

void FDisplayClusterLightCardOutlinerMode::OnDrop(ISceneOutlinerTreeItem& DropTarget,
	const FSceneOutlinerDragDropPayload& Payload, const FSceneOutlinerDragValidationInfo& ValidationInfo) const
{
	if (const FActorTreeItem* ActorItem = DropTarget.CastTo<FActorTreeItem>())
	{
		AActor* DropActor = ActorItem->Actor.Get();
		if (!DropActor)
		{
			return;
		}

		FMessageLog EditorErrors("EditorErrors");
		EditorErrors.NewPage(LOCTEXT("ActorAttachmentsPageLabel", "Actor attachment"));

		if (ValidationInfo.CompatibilityType == ESceneOutlinerDropCompatibility::CompatibleMultipleDetach || ValidationInfo.CompatibilityType == ESceneOutlinerDropCompatibility::CompatibleDetach)
		{
			const FScopedTransaction Transaction(LOCTEXT("UndoAction_DetachActors", "Detach actors"));

			TArray<TWeakObjectPtr<AActor>> DraggedActors = Payload.GetData<TWeakObjectPtr<AActor>>(FWeakActorSelector());
			for (const TWeakObjectPtr<AActor>& WeakActor : DraggedActors)
			{
				if (AActor* DragActor = WeakActor.Get())
				{
					// Detach from parent
					USceneComponent* RootComp = DragActor->GetRootComponent();
					if (RootComp && RootComp->GetAttachParent())
					{
						AActor* OldParent = RootComp->GetAttachParent()->GetOwner();
						// Attachment is persisted on the child so modify both actors for Undo/Redo but do not mark the Parent package dirty
						OldParent->Modify(/*bAlwaysMarkDirty=*/false);
						RootComp->DetachFromComponent(FDetachmentTransformRules::KeepWorldTransform);

						DragActor->SetFolderPath_Recursively(OldParent->GetFolderPath());
					}
				}
			}
		}
		else if (ValidationInfo.CompatibilityType == ESceneOutlinerDropCompatibility::CompatibleMultipleAttach || ValidationInfo.CompatibilityType == ESceneOutlinerDropCompatibility::CompatibleAttach)
		{
			// Show socket chooser if we have sockets to select

			if (ILevelInstanceInterface* TargetLevelInstance = Cast<ILevelInstanceInterface>(DropActor))
			{
				check(TargetLevelInstance->IsEditing());
				const FScopedTransaction Transaction(LOCTEXT("UndoAction_MoveActorsToLevelInstance", "Move actors to LevelInstance"));

				const FFolder DestinationPath = FFolder(FFolder::FRootObject(DropActor));
				auto MoveToDestination = [&DestinationPath](FFolderTreeItem& Item)
				{
					Item.MoveTo(DestinationPath);
				};
				Payload.ForEachItem<FFolderTreeItem>(MoveToDestination);

				// Since target root is directly the Level Instance, clear folder path
				TArray<AActor*> DraggedActors = Payload.GetData<AActor*>(FActorSelector());
				for (AActor*& Actor : DraggedActors)
				{
					Actor->SetFolderPath_Recursively(FName());
				}

				ULevelInstanceSubsystem* LevelInstanceSubsystem = RepresentingWorld->GetSubsystem<ULevelInstanceSubsystem>();
				check(LevelInstanceSubsystem);
				LevelInstanceSubsystem->MoveActorsTo(TargetLevelInstance, DraggedActors);
			}
			else
			{
				auto PerformAttachment = [](FName SocketName, TWeakObjectPtr<AActor> Parent, const TArray<TWeakObjectPtr<AActor>> NewAttachments)
				{
					AActor* ParentActor = Parent.Get();
					if (ParentActor)
					{
						// modify parent and child
						const FScopedTransaction Transaction(LOCTEXT("UndoAction_PerformAttachment", "Attach actors"));

						// Attach each child
						bool bAttached = false;
						for (const TWeakObjectPtr<AActor>& Child : NewAttachments)
						{
							AActor* ChildActor = Child.Get();
							if (GEditor->CanParentActors(ParentActor, ChildActor))
							{
								GEditor->ParentActors(ParentActor, ChildActor, SocketName);

								ChildActor->SetFolderPath_Recursively(ParentActor->GetFolderPath());
							}
						}
					}
				};

				TArray<TWeakObjectPtr<AActor>> DraggedActors = Payload.GetData<TWeakObjectPtr<AActor>>(FWeakActorSelector());
				USceneComponent* Component = DropActor->GetRootComponent();
				if ((Component != NULL) && (Component->HasAnySockets()))
				{
					// Create the popup
					FSlateApplication::Get().PushMenu(
						SceneOutliner->AsShared(),
						FWidgetPath(),
						SNew(SSocketChooserPopup)
						.SceneComponent(Component)
						.OnSocketChosen_Lambda(PerformAttachment, DropActor, MoveTemp(DraggedActors)),
						FSlateApplication::Get().GetCursorPos(),
						FPopupTransitionEffect(FPopupTransitionEffect::TypeInPopup)
					);
				}
				else
				{
					PerformAttachment(NAME_None, DropActor, MoveTemp(DraggedActors));
				}
			}

		}
		// Report errors
		EditorErrors.Notify(NSLOCTEXT("ActorAttachmentError", "AttachmentsFailed", "Attachments Failed!"));
	}
	else if (DropTarget.IsA<FFolderTreeItem>() || DropTarget.IsA<FWorldTreeItem>() || DropTarget.IsA<FLevelTreeItem>())
	{
		const FFolderTreeItem* FolderItem = DropTarget.CastTo<FFolderTreeItem>();
		const FWorldTreeItem* WorldItem = DropTarget.CastTo<FWorldTreeItem>();
		const FLevelTreeItem* LevelItem = DropTarget.CastTo<FLevelTreeItem>();
		// WorldTreeItem and LevelTreeItem are treated as root folders (path = none), with the difference that LevelTreeItem has a RootObject.
		const FFolder DestinationPath = FolderItem ? FolderItem->GetFolder() : (LevelItem ? FFolder(FFolder::GetOptionalFolderRootObject(LevelItem->Level.Get()).Get(FFolder::GetInvalidRootObject())) : GetWorldDefaultRootFolder());

		const FScopedTransaction Transaction(LOCTEXT("MoveOutlinerItems", "Move World Outliner Items"));

		auto MoveToDestination = [&DestinationPath](FFolderTreeItem& Item)
		{
			Item.MoveTo(DestinationPath);
		};
		Payload.ForEachItem<FFolderTreeItem>(MoveToDestination);

		// Set the folder path on all the dragged actors, and detach any that need to be moved
		if (Payload.Has<FActorTreeItem>())
		{
			TSet<const AActor*> ParentActors;
			TSet<const AActor*> ChildActors;

			TArray<AActor*> MovingActorsToValidRootObject;
			Payload.ForEachItem<FActorTreeItem>([&DestinationPath, &ParentActors, &ChildActors, &MovingActorsToValidRootObject](const FActorTreeItem& ActorItem)
			{
				AActor* Actor = ActorItem.Actor.Get();
				if (Actor)
				{
					// First mark this object as a parent, then set its children's path
					ParentActors.Add(Actor);

					const FFolder SrcFolder = Actor->GetFolder();

					// If the folder root object changes, 1st pass will put actors at root. 2nd pass will set the destination path.
					FName NewPath = (SrcFolder.GetRootObject() == DestinationPath.GetRootObject()) ? DestinationPath.GetPath() : NAME_None;

					Actor->SetFolderPath(NewPath);
					FActorEditorUtils::TraverseActorTree_ParentFirst(Actor, [&](AActor* InActor) {
						ChildActors.Add(InActor);
						InActor->SetFolderPath(NewPath);
						return true;
						}, false);

					if ((Actor->GetFolderRootObject() != DestinationPath.GetRootObject()) && SrcFolder.IsRootObjectPersistentLevel() && (DestinationPath.IsRootObjectValid() && !DestinationPath.IsRootObjectPersistentLevel()))
					{
						MovingActorsToValidRootObject.Add(Actor);
					}
				}
			});

			// Detach parent actors
			for (const AActor* Parent : ParentActors)
			{
				USceneComponent* RootComp = Parent->GetRootComponent();

				// We don't detach if it's a child of another that's been dragged
				if (RootComp && RootComp->GetAttachParent() && !ChildActors.Contains(Parent))
				{
					if (AActor* OldParentActor = RootComp->GetAttachParent()->GetOwner())
					{
						// Attachment is persisted on the child so modify both actors for Undo/Redo but do not mark the Parent package dirty
						OldParentActor->Modify(/*bAlwaysMarkDirty=*/false);
					}
					RootComp->DetachFromComponent(FDetachmentTransformRules::KeepWorldTransform);
				}
			}

			auto MoveActorsToLevel = [](const TArray<AActor*>& InActorsToMove, ULevel* InDestLevel, const FName& InDestinationPath)
			{
				// We are moving actors to another level
				const bool bWarnAboutReferences = true;
				const bool bWarnAboutRenaming = true;
				const bool bMoveAllOrFail = true;
				TArray<AActor*> MovedActors;
				if (!EditorLevelUtils::MoveActorsToLevel(InActorsToMove, InDestLevel, bWarnAboutReferences, bWarnAboutRenaming, bMoveAllOrFail, &MovedActors))
				{
					UE_LOGF(DisplayClusterLightCardEditorLog, Warning, "Failed to move actors because not all actors could be moved");
				}
				// Once moved, update actors folder path
				for (AActor* Actor : MovedActors)
				{
					Actor->SetFolderPath_Recursively(InDestinationPath);
				}
			};

			if (DestinationPath.IsRootObjectPersistentLevel())
			{
				const ULevelInstanceSubsystem* LevelInstanceSubsystem = RepresentingWorld->GetSubsystem<ULevelInstanceSubsystem>();
				check(LevelInstanceSubsystem);
				ULevel* DestinationLevel = RepresentingWorld->PersistentLevel;
				check(DestinationLevel);

				TArray<AActor*> LevelInstanceActorsToMove;
				TArray<AActor*> ActorsToMoveToPersistentLevel;
				Payload.ForEachItem<FActorTreeItem>([LevelInstanceSubsystem, &LevelInstanceActorsToMove, &ActorsToMoveToPersistentLevel](const FActorTreeItem& ActorItem)
				{
					AActor* Actor = ActorItem.Actor.Get();
					if (const ILevelInstanceInterface* ParentLevelInstance = LevelInstanceSubsystem->GetParentLevelInstance(Actor))
					{
						check(ParentLevelInstance->IsEditing());
						LevelInstanceActorsToMove.Add(Actor);
					}
					else
					{
						const FFolder ActorSrcFolder = Actor->GetFolder();
						if (ActorSrcFolder.IsRootObjectValid() && !ActorSrcFolder.IsRootObjectPersistentLevel())
						{
							ActorsToMoveToPersistentLevel.Add(Actor);
						}
					}
				});

				// We are moving actors outside of an editing level instance to a folder (or root) into the persistent level.
				if (LevelInstanceActorsToMove.Num() > 0)
				{
					TArray<AActor*> MovedActors;
					LevelInstanceSubsystem->MoveActorsToLevel(LevelInstanceActorsToMove, DestinationLevel, &MovedActors);
					// Once moved, update actors folder path
					for (AActor* Actor : MovedActors)
					{
						Actor->SetFolderPath_Recursively(DestinationPath.GetPath());
					}
				}
				if (ActorsToMoveToPersistentLevel.Num() > 0)
				{
					MoveActorsToLevel(ActorsToMoveToPersistentLevel, DestinationLevel, DestinationPath.GetPath());
				}
			}
			else if (MovingActorsToValidRootObject.Num())
			{
				if (ILevelInstanceInterface* TargetLevelInstance = Cast<ILevelInstanceInterface>(DestinationPath.GetRootObjectPtr()))
				{
					// We are moving actors inside an editing level instance
					check(TargetLevelInstance->IsEditing());

					ULevelInstanceSubsystem* LevelInstanceSubsystem = RepresentingWorld->GetSubsystem<ULevelInstanceSubsystem>();
					check(LevelInstanceSubsystem);
					TArray<AActor*> MovedActors;
					LevelInstanceSubsystem->MoveActorsTo(TargetLevelInstance, MovingActorsToValidRootObject, &MovedActors);
					// Once moved, update actors folder path
					for (AActor* Actor : MovedActors)
					{
						Actor->SetFolderPath_Recursively(DestinationPath.GetPath());
					}
				}
				else if (ULevel* DestinationLevel = Cast<ULevel>(DestinationPath.GetRootObjectPtr()))
				{
					MoveActorsToLevel(MovingActorsToValidRootObject, DestinationLevel, DestinationPath.GetPath());
				}
			}
		}
	}
}

void FDisplayClusterLightCardOutlinerMode::OnEditCutActorsBegin()
{
	SceneOutliner->CopyFoldersBegin();
	SceneOutliner->DeleteFoldersBegin();
}

void FDisplayClusterLightCardOutlinerMode::OnEditCutActorsEnd()
{
	SceneOutliner->CopyFoldersEnd();
	SceneOutliner->DeleteFoldersEnd();
}

void FDisplayClusterLightCardOutlinerMode::OnEditCopyActorsBegin()
{
	SceneOutliner->CopyFoldersBegin();
}

void FDisplayClusterLightCardOutlinerMode::OnEditCopyActorsEnd()
{
	SceneOutliner->CopyFoldersEnd();
}

void FDisplayClusterLightCardOutlinerMode::OnEditPasteActorsBegin()
{
	const TArray<FName> FolderPaths = SceneOutliner->GetClipboardPasteFolders();
	SceneOutliner->PasteFoldersBegin(FolderPaths);
}

void FDisplayClusterLightCardOutlinerMode::OnEditPasteActorsEnd()
{
	SceneOutliner->PasteFoldersEnd();
}

void FDisplayClusterLightCardOutlinerMode::OnDuplicateActorsBegin()
{
	FFolder::FRootObject CommonRootObject;
	TArray<FName> SelectedFolderPaths;
	FFolder::GetFolderPathsAndCommonRootObject(SceneOutliner->GetSelection().GetData<FFolder>(FFolderPathSelector()), SelectedFolderPaths, CommonRootObject);
	SceneOutliner->PasteFoldersBegin(SelectedFolderPaths);
}

void FDisplayClusterLightCardOutlinerMode::OnDuplicateActorsEnd()
{
	SceneOutliner->PasteFoldersEnd();
}

void FDisplayClusterLightCardOutlinerMode::OnDeleteActorsBegin()
{
	SceneOutliner->DeleteFoldersBegin();
}

void FDisplayClusterLightCardOutlinerMode::OnDeleteActorsEnd()
{
	SceneOutliner->DeleteFoldersEnd();
}

void FDisplayClusterLightCardOutlinerMode::RegisterContextMenu()
{
	UToolMenus* ToolMenus = UToolMenus::Get();

	if (!ToolMenus->IsMenuRegistered(UE::LightCardOutlinerMode::DefaultContextBaseMenuName))
	{
		UToolMenu* Menu = ToolMenus->RegisterMenu(UE::LightCardOutlinerMode::DefaultContextBaseMenuName);

		Menu->AddDynamicSection("DynamicHierarchySection", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
			{
				USceneOutlinerMenuContext* Context = InMenu->FindContext<USceneOutlinerMenuContext>();
				if (!Context || !Context->SceneOutliner.IsValid())
				{
					return;
				}

				// NOTE: the name "Section" is used in many other places
				FToolMenuSection& Section = InMenu->FindOrAddSection("Section");
				Section.Label = LOCTEXT("HierarchySectionName", "Hierarchy");

				SSceneOutliner* SceneOutliner = Context->SceneOutliner.Pin().Get();
				if (Context->bShowParentTree)
				{
					// Only show create folder if no items are selected or a folder is selected.
					// Actors will have the MoveTo option available to support folder creation.
					// This lightcard outliner implementation doesn't currently support the normal folder
					// hierarchy options like duplication.
					if (Context->NumSelectedItems == 0 || Context->NumSelectedFolders == 1)
					{
						FSceneOutlinerMenuHelper::AddMenuEntryCreateFolder(Section, *SceneOutliner);
					}
				}
			}));

		Menu->AddDynamicSection("DynamicMainSection", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
			{
				// We always create a section here, even if there is no parent so that clients can still extend the menu
				FToolMenuSection& Section = InMenu->AddSection("MainSection", LOCTEXT("OutlinerSectionName", "Outliner"));

				if (USceneOutlinerMenuContext* Context = InMenu->FindContext<USceneOutlinerMenuContext>())
				{
					// Don't add any of these menu items if we're not showing the parent tree
					// Can't move worlds or level blueprints
					if (Context->bShowParentTree && Context->NumSelectedItems > 0 && Context->NumWorldsSelected == 0 && Context->SceneOutliner.IsValid())
					{
						Section.AddSubMenu(
							"MoveActorsTo",
							LOCTEXT("MoveActorsTo", "Move To"),
							LOCTEXT("MoveActorsTo_Tooltip", "Move selection to another folder"),
							FNewToolMenuDelegate::CreateSP(Context->SceneOutliner.Pin().Get(), &SSceneOutliner::FillFoldersSubMenu));
					}

					if (Context->bShowParentTree && Context->NumSelectedItems > 0 && Context->SceneOutliner.IsValid())
					{
						// Only add the menu option to wp levels
						if (!Context->bRepresentingGameWorld && Context->bRepresentingPartitionedWorld)
						{
							// If selection contains some unpinned items, show the pin option
							// If the selection contains folders, always show the pin option
							if (Context->NumPinnedItems != Context->NumSelectedItems || Context->NumSelectedFolders > 0)
							{
								Section.AddMenuEntry(
									"PinItems",
									LOCTEXT("Pin", "Pin"),
									FText(),
									FSlateIcon(),
									FUIAction(FExecuteAction::CreateSP(Context->SceneOutliner.Pin().Get(), &SSceneOutliner::PinSelectedItems)));
							}

							// If the selection contains some pinned items, show the unpin option
							// If the selection contains folders, always show the unpin option
							if (Context->NumPinnedItems != 0 || Context->NumSelectedFolders > 0)
							{
								Section.AddMenuEntry(
									"UnpinItems",
									LOCTEXT("Unpin", "Unpin"),
									FText(),
									FSlateIcon(),
									FUIAction(FExecuteAction::CreateSP(Context->SceneOutliner.Pin().Get(), &SSceneOutliner::UnpinSelectedItems)));
							}
						}
					}

					if (Context->NumSelectedItems > 0 && Context->SceneOutliner.IsValid())
					{
						SSceneOutliner* SceneOutliner = Context->SceneOutliner.Pin().Get();
						SceneOutliner->AddSourceControlMenuOptions(InMenu);
					}
				}
			}));
	}

	if (!ToolMenus->IsMenuRegistered(UE::LightCardOutlinerMode::DefaultContextMenuName))
	{
		ToolMenus->RegisterMenu(UE::LightCardOutlinerMode::DefaultContextMenuName, UE::LightCardOutlinerMode::DefaultContextBaseMenuName);
	}
}

bool FDisplayClusterLightCardOutlinerMode::CanPasteFoldersOnlyFromClipboard() const
{
	FString PasteString;
	FPlatformApplicationMisc::ClipboardPaste(PasteString);
	return PasteString.StartsWith("BEGIN FOLDERLIST");
}

bool FDisplayClusterLightCardOutlinerMode::GetFolderNamesFromPayload(const FSceneOutlinerDragDropPayload& InPayload,
	TArray<FName>& OutFolders, FFolder::FRootObject& OutCommonRootObject) const
{
	return FFolder::GetFolderPathsAndCommonRootObject(InPayload.GetData<FFolder>(FFolderPathSelector()), OutFolders, OutCommonRootObject);
}

FFolder FDisplayClusterLightCardOutlinerMode::GetWorldDefaultRootFolder() const
{
	return FFolder::GetWorldRootFolder(RepresentingWorld.Get());
}

void FDisplayClusterLightCardOutlinerMode::UpdateGlobalActorSelection()
{
	TArray<AActor*> SelectedActors = SceneOutliner->GetSelection().GetData<AActor*>(FActorSelector());
	
	const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "SelectActors", "Select Actors"));
	GEditor->GetSelectedActors()->Modify();

	GEditor->GetSelectedActors()->BeginBatchSelectOperation();
	
	GEditor->SelectNone(false, true, true);
	
	const bool bShouldSelect = true;
	const bool bNotifyAfterSelect = false;
	const bool bSelectEvenIfHidden = true;
	for (AActor* Actor : SelectedActors)
	{
		GEditor->SelectActor(Actor, bShouldSelect, bNotifyAfterSelect, bSelectEvenIfHidden);
	}

	GEditor->GetSelectedActors()->EndBatchSelectOperation(/*bNotify*/false);
	GEditor->NoteSelectionChange();
}

#undef LOCTEXT_NAMESPACE
