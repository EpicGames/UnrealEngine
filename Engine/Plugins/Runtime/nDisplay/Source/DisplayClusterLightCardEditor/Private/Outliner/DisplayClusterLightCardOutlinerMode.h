// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActorMode.h"

class SDisplayClusterLightCardOutliner;

/**
 * Custom outliner mode for the light card editor based on FActorBrowsingMode. The primary goal is to allow basic actor
 * functionality such as selecting actors locally, creating folders, and drag & drop. Selection should occur locally
 * when possible and not update the GEditor selection. Additionally the context menu is cleaned up.
 */
class FDisplayClusterLightCardOutlinerMode final : public FActorMode
{
public:
	
	FDisplayClusterLightCardOutlinerMode(SSceneOutliner* InSceneOutliner, TWeakPtr<SDisplayClusterLightCardOutliner> InLightCardOutliner,
		TWeakObjectPtr<UWorld> InSpecifiedWorldToDisplay = nullptr);

	virtual ~FDisplayClusterLightCardOutlinerMode() override;
	
	virtual void SynchronizeSelection() override;
	virtual bool IsInteractive() const override { return true; }
	virtual FCreateSceneOutlinerMode CreateFolderPickerMode(const FFolder::FRootObject& InRootObject = FFolder::GetInvalidRootObject()) const override;
	virtual TSharedPtr<SWidget> CreateContextMenu() override;
	virtual ESelectionMode::Type GetSelectionMode() const override { return ESelectionMode::Multi; }
	virtual bool SupportsKeyboardFocus() const override { return true; }
	virtual bool ShouldShowFolders() const override { return true; }
	virtual bool SupportsCreateNewFolder() const override { return true; }
	virtual bool ShowStatusBar() const override { return true; }
	virtual bool ShowViewButton() const override { return true; }
	virtual bool ShowFilterOptions() const override { return true; }
	virtual void OnItemDoubleClick(FSceneOutlinerTreeItemPtr Item) override;
	virtual FReply OnKeyDown(const FKeyEvent& InKeyEvent) override;
	virtual bool CanDelete() const override;
	virtual bool CanRename() const override;
	virtual bool CanRenameItem(const ISceneOutlinerTreeItem& Item) const override;
	virtual bool CanCut() const override;
	virtual bool CanCopy() const override;
	virtual bool CanPaste() const override;
	virtual FFolder CreateNewFolder() override;
	virtual FFolder GetFolder(const FFolder& ParentPath, const FName& LeafName) override;
	virtual bool CreateFolder(const FFolder& NewFolder) override;
	virtual bool ReparentItemToFolder(const FFolder& FolderPath, const FSceneOutlinerTreeItemPtr& Item) override;
	virtual bool CanSupportDragAndDrop() const override { return true; }
	virtual TSharedPtr<FDragDropOperation> CreateDragDropOperation(const FPointerEvent& MouseEvent, const TArray<FSceneOutlinerTreeItemPtr>& InTreeItems) const override;
	virtual bool ParseDragDrop(FSceneOutlinerDragDropPayload& OutPayload, const FDragDropOperation& Operation) const override;
	virtual FSceneOutlinerDragValidationInfo ValidateDrop(const ISceneOutlinerTreeItem& DropTarget, const FSceneOutlinerDragDropPayload& Payload) const override;
	virtual void OnDrop(ISceneOutlinerTreeItem& DropTarget, const FSceneOutlinerDragDropPayload& Payload, const FSceneOutlinerDragValidationInfo& ValidationInfo) const override;

	/** Called by engine when edit cut actors begins */
	void OnEditCutActorsBegin();

	/** Called by engine when edit cut actors ends */
	void OnEditCutActorsEnd();

	/** Called by engine when edit copy actors begins */
	void OnEditCopyActorsBegin();

	/** Called by engine when edit copy actors ends */
	void OnEditCopyActorsEnd();

	/** Called by engine when edit paste actors begins */
	void OnEditPasteActorsBegin();

	/** Called by engine when edit paste actors ends */
	void OnEditPasteActorsEnd();

	/** Called by engine when edit duplicate actors begins */
	void OnDuplicateActorsBegin();

	/** Called by engine when edit duplicate actors ends */
	void OnDuplicateActorsEnd();

	/** Called by engine when edit delete actors begins */
	void OnDeleteActorsBegin();

	/** Called by engine when edit delete actors ends */
	void OnDeleteActorsEnd();
	
private:
	static void RegisterContextMenu();
	bool CanPasteFoldersOnlyFromClipboard() const;
	bool GetFolderNamesFromPayload(const FSceneOutlinerDragDropPayload& InPayload, TArray<FName>& OutFolders, FFolder::FRootObject& OutCommonRootObject) const;
	FFolder GetWorldDefaultRootFolder() const;

	/** Updates GEditor selection */
	void UpdateGlobalActorSelection();
	
private:
	TWeakPtr<SDisplayClusterLightCardOutliner> LightCardOutliner;
};
