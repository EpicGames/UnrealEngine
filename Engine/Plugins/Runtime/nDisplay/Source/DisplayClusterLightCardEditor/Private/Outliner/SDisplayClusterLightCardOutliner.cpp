// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDisplayClusterLightCardOutliner.h"

#include "DisplayClusterLightCardEditorCommands.h"
#include "DisplayClusterLightCardOutlinerMode.h"
#include "DisplayClusterLightCardEditor.h"
#include "DisplayClusterLightCardOutlinerColumns.h"

#include "IDisplayClusterOperator.h"
#include "IDisplayClusterOperatorViewModel.h"

#include "DisplayClusterConfigurationTypes.h"

#include "DisplayClusterLightCardActor.h"
#include "DisplayClusterRootActor.h"

#include "StageActor/IDisplayClusterStageActor.h"

#include "ActorTreeItem.h"
#include "ClassIconFinder.h"
#include "FolderTreeItem.h"
#include "SceneOutlinerMenuContext.h"
#include "SceneOutlinerModule.h"
#include "SceneOutlinerTextInfoColumn.h"
#include "Selection.h"
#include "SSceneOutliner.h"
#include "ToolMenus.h"
#include "WorldTreeItem.h"
#include "Framework/Commands/GenericCommands.h"

#define LOCTEXT_NAMESPACE "SDisplayClusterLightCardOutliner"

SDisplayClusterLightCardOutliner::~SDisplayClusterLightCardOutliner()
{
	if (SceneOutliner.IsValid() && SceneOutlinerSelectionChanged.IsValid())
	{
		SceneOutliner->GetOnItemSelectionChanged().Remove(SceneOutlinerSelectionChanged);
	}
}

void SDisplayClusterLightCardOutliner::Construct(const FArguments& InArgs, TSharedPtr<FDisplayClusterLightCardEditor> InLightCardEditor, TSharedPtr<FUICommandList> InCommandList)
{
	LightCardEditorPtr = InLightCardEditor;
	check(LightCardEditorPtr.IsValid());
	
	CommandList = InCommandList;

	RootActor = InLightCardEditor->GetActiveRootActor();
	CreateWorldOutliner();

	SelectActors(LightCardEditorPtr.Pin()->GetSelectedActorsAs<AActor>());
}

void SDisplayClusterLightCardOutliner::SetRootActor(ADisplayClusterRootActor* NewRootActor)
{
	TArray<AActor*> PreviouslySelectedActors;
	if (LightCardEditorPtr.IsValid())
	{
		PreviouslySelectedActors = LightCardEditorPtr.Pin()->GetSelectedActorsAs<AActor>();
	}

	for (TWeakObjectPtr<AActor> Actor : NonOutlinerSelectedActors)
	{
		if (Actor.IsValid())
		{
			PreviouslySelectedActors.AddUnique(Actor.Get());
		}
	}

	RootActor = NewRootActor;

	FillActorList();
	if (SceneOutliner.IsValid())
	{
		SceneOutliner->FullRefresh();
	}
	
	// Select previously selected light cards. This fixes an issue where the details panel may clear in some cases
	// and also supports maintaining the selection of shared light cards across different root actors.
	if (LightCardEditorPtr.IsValid())
	{
		LightCardEditorPtr.Pin()->SelectActors(MoveTemp(PreviouslySelectedActors));
	}
}

void SDisplayClusterLightCardOutliner::GetSelectedActors(TArray<AActor*>& OutSelectedActors) const
{
	TArray<FSceneOutlinerTreeItemPtr> SelectedOutlinerActors = SceneOutliner->GetSelectedItems();
	for (const FSceneOutlinerTreeItemPtr& SelectedLightCard : SelectedOutlinerActors)
	{
		TSharedPtr<FStageActorTreeItem> MatchingTreeItem = GetStageActorTreeItemFromOutliner(SelectedLightCard);
		if (MatchingTreeItem.IsValid() && MatchingTreeItem->Actor.IsValid())
		{
			OutSelectedActors.Add(MatchingTreeItem->Actor.Get());
		}
	}

	for (TWeakObjectPtr<AActor> Actor : NonOutlinerSelectedActors)
	{
		if (Actor.IsValid())
		{
			OutSelectedActors.Add(Actor.Get());
		}
	}
}

void SDisplayClusterLightCardOutliner::SelectActors(const TArray<AActor*>& ActorsToSelect)
{
	NonOutlinerSelectedActors.Empty();

	if (bIsOutlinerChangingSelection)
	{
		// Selection change initiated from outliner; assume the selection is already applied and non-outliner actors are deselected
		IDisplayClusterOperator::Get().GetOperatorViewModel()->OnOutlinerSelectionChanged().Broadcast(ActorsToSelect);
		return;
	}
	
	TArray<FSceneOutlinerTreeItemPtr> SelectedTreeItems;
	TArray<AActor*> SelectedActors;
	CachedOutlinerItems.Reset();
	
	for (const TSharedPtr<FStageActorTreeItem>& TreeItem : StageActorTreeItems)
	{
		if (ActorsToSelect.Contains(TreeItem->Actor))
		{
			if (FSceneOutlinerTreeItemPtr OutlinerItem = SceneOutliner->GetTreeItem(TreeItem->Actor.Get()))
			{
				SelectedTreeItems.Add(OutlinerItem);
				SelectedActors.Add(TreeItem->Actor.Get());
			}
		}
	}

	for (AActor* Actor : ActorsToSelect)
	{
		if (!SelectedActors.Contains(Actor))
		{
			NonOutlinerSelectedActors.Add(Actor);
		}
	}

	CachedOutlinerItems = SelectedTreeItems;
	SceneOutliner->SetItemSelection(SelectedTreeItems, true);

	IDisplayClusterOperator::Get().GetOperatorViewModel()->OnOutlinerSelectionChanged().Broadcast(ActorsToSelect);
}

void SDisplayClusterLightCardOutliner::RestoreCachedSelection()
{
	TArray<AActor*> ActorsToSelect;
	TArray<TSharedPtr<ISceneOutlinerTreeItem>> AdditionalTreeItemsToSelect;
	
	for (TSharedPtr<ISceneOutlinerTreeItem> CachedItem : CachedOutlinerItems)
	{
		if (CachedItem.IsValid())
		{
			if (const FActorTreeItem* ActorItem = CachedItem->CastTo<FActorTreeItem>())
			{
				if (ActorItem->Actor.IsValid())
				{
					ActorsToSelect.Add(ActorItem->Actor.Get());
				}
			}
			else
			{
				AdditionalTreeItemsToSelect.Add(CachedItem);
			}
		}
	}

	for (TWeakObjectPtr<AActor> Actor : NonOutlinerSelectedActors)
	{
		if (Actor.IsValid())
		{
			ActorsToSelect.Add(Actor.Get());
		}
	}

	if (LightCardEditorPtr.IsValid())
	{
		LightCardEditorPtr.Pin()->SelectActors(MoveTemp(ActorsToSelect));
	}

	if (AdditionalTreeItemsToSelect.Num() > 0)
	{
		SceneOutliner->AddToSelection(AdditionalTreeItemsToSelect);
	}
}

bool SDisplayClusterLightCardOutliner::CanDeleteSelectedFolder() const
{
	if (SceneOutliner.IsValid())
	{
		const FDisplayClusterLightCardOutlinerMode* Mode = static_cast<const FDisplayClusterLightCardOutlinerMode*>(SceneOutliner->GetMode());
		return Mode->CanDelete();
	}
	
	return false;
}

bool SDisplayClusterLightCardOutliner::CanCutSelectedFolder() const
{
	if (SceneOutliner.IsValid())
	{
		const FDisplayClusterLightCardOutlinerMode* Mode = static_cast<const FDisplayClusterLightCardOutlinerMode*>(SceneOutliner->GetMode());
		return Mode->CanCut();
	}

	return false;
}

bool SDisplayClusterLightCardOutliner::CanCopySelectedFolder() const
{
	if (SceneOutliner.IsValid())
	{
		const FDisplayClusterLightCardOutlinerMode* Mode = static_cast<const FDisplayClusterLightCardOutlinerMode*>(SceneOutliner->GetMode());
		return Mode->CanCopy();
	}

	return false;
}

bool SDisplayClusterLightCardOutliner::CanPasteSelectedFolder() const
{
	if (SceneOutliner.IsValid())
	{
		const FDisplayClusterLightCardOutlinerMode* Mode = static_cast<const FDisplayClusterLightCardOutlinerMode*>(SceneOutliner->GetMode());
		return Mode->CanPaste();
	}

	return false;
}

bool SDisplayClusterLightCardOutliner::CanRenameSelectedItem() const
{
	if (SceneOutliner.IsValid())
	{
		TArray<FSceneOutlinerTreeItemPtr> SelectedOutlinerItems = SceneOutliner->GetSelectedItems();
		const FDisplayClusterLightCardOutlinerMode* Mode = static_cast<const FDisplayClusterLightCardOutlinerMode*>(SceneOutliner->GetMode());
		if (SelectedOutlinerItems.Num() == 1 && SelectedOutlinerItems[0].IsValid())
		{
			return Mode->CanRenameItem(*SelectedOutlinerItems[0].Get());
		}
	}

	return false;
}

void SDisplayClusterLightCardOutliner::RenameSelectedItem()
{
	if (SceneOutliner.IsValid())
	{
		TArray<FSceneOutlinerTreeItemPtr> SelectedItems = SceneOutliner->GetSelectedItems();
		if (SelectedItems.Num() == 1)
		{
			SelectedItems[0]->RenameRequestEvent.ExecuteIfBound();
		}
	}
}

void SDisplayClusterLightCardOutliner::CreateWorldOutliner()
{
	FillActorList();
	
	auto OutlinerFilterPredicate = [this](const AActor* InActor)
	{
		return TrackedActors.Contains(InActor);
	};

	const FSceneOutlinerModule& SceneOutlinerModule = FModuleManager::Get().LoadModuleChecked<FSceneOutlinerModule>("SceneOutliner");

	if (SceneOutliner.IsValid() && SceneOutlinerSelectionChanged.IsValid())
	{
		SceneOutliner->GetOnItemSelectionChanged().Remove(SceneOutlinerSelectionChanged);
	}
	
	FSceneOutlinerInitializationOptions SceneOutlinerOptions;

	// Explicitly add in the columns we want. Otherwise all column visibility can be toggled and this outliner only needs several.
	
	SceneOutlinerOptions.ColumnMap.Add(FSceneOutlinerBuiltInColumnTypes::Gutter(),
		FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 0, FCreateSceneOutlinerColumn(), false, TOptional<float>(),
				FSceneOutlinerBuiltInColumnTypes::Gutter_Localized()));

	SceneOutlinerOptions.ColumnMap.Add(FDisplayClusterLightCardOutlinerHiddenInGameColumn::GetID(),
		FSceneOutlinerColumnInfo(FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible,
			1, FCreateSceneOutlinerColumn::CreateLambda([](ISceneOutliner& InSceneOutliner)
			{ return MakeShareable(new FDisplayClusterLightCardOutlinerHiddenInGameColumn(InSceneOutliner)); }), true,
			TOptional<float>(), FDisplayClusterLightCardOutlinerHiddenInGameColumn::GetDisplayText())));
	
	SceneOutlinerOptions.ColumnMap.Add(FSceneOutlinerBuiltInColumnTypes::Label(),
		FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 10, FCreateSceneOutlinerColumn(), false, TOptional<float>(),
				FSceneOutlinerBuiltInColumnTypes::Label_Localized()));

	// Add a custom class/type column. The default ActorInfo column returns the class FName only, where as we want to display a friendly
	// display for the class.
	{
		// Based on struct FGetInfo from SceneOutlinerActorInfoColumn
		const FGetTextForItem InternalNameInfoText = FGetTextForItem::CreateLambda([](const ISceneOutlinerTreeItem& Item) -> FString
		{
			if (const FActorTreeItem* ActorItem = Item.CastTo<FActorTreeItem>())
			{
				if (const AActor* Actor = ActorItem->Actor.Get())
				{
					const FString ClassDisplayName = Actor->GetClass()->GetDisplayNameText().ToString();
					
					if (const ADisplayClusterLightCardActor* LightCardActor = Cast<ADisplayClusterLightCardActor>(Actor))
					{
						const FString LightCardType = LightCardActor->IsLightCardFlag() ? TEXT("Flag") : ClassDisplayName;
						
						return LightCardActor->bIsUVLightCard ?
							FString::Printf(TEXT("UV %s"), *LightCardType) :
							FString::Printf(TEXT("%s"), *LightCardType);
					}
					
					return ClassDisplayName;
				}
			
				return FString();
			}
			if (Item.IsA<FFolderTreeItem>())
			{
				return LOCTEXT("FolderTypeName", "Folder").ToString();
			}
			if (Item.IsA<FWorldTreeItem>())
			{
				return LOCTEXT("WorldTypeName", "World").ToString();
			}

			return FString();
		});
		
		SceneOutlinerOptions.ColumnMap.Add(FSceneOutlinerBuiltInColumnTypes::ActorInfo(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 20,
			FCreateSceneOutlinerColumn::CreateStatic(&FTextInfoColumn::CreateTextInfoColumn, FSceneOutlinerBuiltInColumnTypes::ActorInfo(), InternalNameInfoText, FText::GetEmpty()),
			true, TOptional<float>(),
				FSceneOutlinerBuiltInColumnTypes::ActorInfo_Localized()));
	}
	
	SceneOutlinerOptions.ModifyContextMenu = FSceneOutlinerModifyContextMenu::CreateSP(this, &SDisplayClusterLightCardOutliner::RegisterContextMenu);
	SceneOutlinerOptions.OutlinerIdentifier = TEXT("LightCardEditorOutliner");
	SceneOutlinerOptions.Filters->AddFilterPredicate<FActorTreeItem>(FActorTreeItem::FFilterPredicate::CreateLambda(OutlinerFilterPredicate));

	TWeakPtr<SDisplayClusterLightCardOutliner> WeakPtrThis = SharedThis(this);
	UWorld* OutlinerWorld = RootActor.IsValid() ? RootActor->GetWorld() : nullptr;
	SceneOutlinerOptions.ModeFactory = FCreateSceneOutlinerMode::CreateLambda([WeakPtrThis, SpecifiedWorld = MakeWeakObjectPtr(OutlinerWorld)](SSceneOutliner* Outliner)
		{
			return new FDisplayClusterLightCardOutlinerMode(Outliner, WeakPtrThis, SpecifiedWorld);
		});
	
	SceneOutliner = StaticCastSharedRef<SSceneOutliner>(SceneOutlinerModule.CreateSceneOutliner(SceneOutlinerOptions));
	SceneOutlinerSelectionChanged = SceneOutliner->GetOnItemSelectionChanged().AddRaw(this, &SDisplayClusterLightCardOutliner::OnOutlinerSelectionChanged);

	ChildSlot
	[
		SceneOutliner.ToSharedRef()
	];
}

bool SDisplayClusterLightCardOutliner::FillActorList()
{
	TArray<TSharedPtr<FStageActorTreeItem>> OriginalTree = MoveTemp(StageActorTree);
	StageActorTreeItems.Empty();

	if (LightCardEditorPtr.IsValid())
	{
		TArray<AActor*> ManagedActors = LightCardEditorPtr.Pin()->FindAllManagedActors();
		StageActorTreeItems.Reserve(ManagedActors.Num());
		
		for (AActor* Actor : ManagedActors)
		{
			const TSharedPtr<FStageActorTreeItem> TreeItem = MakeShared<FStageActorTreeItem>();
			TreeItem->Actor = Actor;
			StageActorTree.Add(TreeItem);
			StageActorTreeItems.Add(TreeItem);
		}
	}

	TrackedActors.Empty(StageActorTreeItems.Num());

	for (const TSharedPtr<FStageActorTreeItem>& LightCard : StageActorTreeItems)
	{
		TrackedActors.Add(LightCard->Actor.Get(), LightCard);
	}
	
	// Check if the tree items have changed.
	{
		if (OriginalTree.Num() != StageActorTree.Num())
		{
			return true;
		}
	
		for (const TSharedPtr<FStageActorTreeItem>& OriginalTreeItem : OriginalTree)
		{
			if (!StageActorTree.ContainsByPredicate([OriginalTreeItem](const TSharedPtr<FStageActorTreeItem>& LightCardTreeItem)
			{
				return OriginalTreeItem.IsValid() && LightCardTreeItem.IsValid() &&
					OriginalTreeItem->Actor.Get() == LightCardTreeItem->Actor.Get();
			}))
			{
				return true;
			}
		}
	}

	return false;
}

void SDisplayClusterLightCardOutliner::OnOutlinerSelectionChanged(FSceneOutlinerTreeItemPtr TreeItem,
	ESelectInfo::Type Type)
{
	MostRecentSelectedItem = TreeItem;
	
	// User made a selection directly in the outliner, so override the non-outliner selection
	NonOutlinerSelectedActors.Empty();

	if (LightCardEditorPtr.IsValid())
	{
		TArray<AActor*> SelectedLightCards;
    	GetSelectedActors(SelectedLightCards);
	
		bIsOutlinerChangingSelection = true;
		LightCardEditorPtr.Pin()->SelectActors(SelectedLightCards);
		LightCardEditorPtr.Pin()->SelectActorProxies(SelectedLightCards);
		
		CachedOutlinerItems = SceneOutliner->GetSelectedItems();
		
		bIsOutlinerChangingSelection = false;
	}
}

TSharedPtr<SDisplayClusterLightCardOutliner::FStageActorTreeItem> SDisplayClusterLightCardOutliner::
GetStageActorTreeItemFromOutliner(FSceneOutlinerTreeItemPtr InOutlinerTreeItem) const
{
	if (const FActorTreeItem* ActorTreeItem = InOutlinerTreeItem->CastTo<FActorTreeItem>())
	{
		if (const TSharedPtr<FStageActorTreeItem>* MatchingLightCard = TrackedActors.Find(
			ActorTreeItem->Actor.Get()))
		{
			return *MatchingLightCard;
		}
	}

	return nullptr;
}

AActor* SDisplayClusterLightCardOutliner::GetActorFromTreeItem(FSceneOutlinerTreeItemPtr InOutlinerTreeItem) const
{
	if (InOutlinerTreeItem != nullptr)
	{
		if (const FActorTreeItem* ActorTreeItem = InOutlinerTreeItem->CastTo<FActorTreeItem>())
		{
			return ActorTreeItem->Actor.Get();
		}
	}

	return nullptr;
}

FReply SDisplayClusterLightCardOutliner::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

void SDisplayClusterLightCardOutliner::RegisterContextMenu(FName& InName, FToolMenuContext& InContext)
{
	static const FName LightCardEditorSectionName = "DynamicLightCardSection";
	
	UToolMenus* ToolMenus = UToolMenus::Get();
	if (ensure(ToolMenus->IsMenuRegistered(InName)))
	{
		UToolMenu* Menu = ToolMenus->FindMenu(InName);

		InContext.AddCleanup([Menu]()
		{
			Menu->RemoveSection(LightCardEditorSectionName);
		});

		check(LightCardEditorPtr.IsValid());
		InContext.AppendCommandList(LightCardEditorPtr.Pin()->GetCommandList());
		
		Menu->AddDynamicSection(LightCardEditorSectionName, FNewToolMenuDelegate::CreateLambda([this](UToolMenu* InMenu)
		{
			USceneOutlinerMenuContext* Context = InMenu->FindContext<USceneOutlinerMenuContext>();
			if (!Context || !Context->SceneOutliner.IsValid())
			{
				return;
			}
			
			AActor* Actor = GetActorFromTreeItem(MostRecentSelectedItem.Pin());

			if (Actor == nullptr && Context->SceneOutliner.Pin()->GetSelection().Num() == 1)
			{
				// If a delete operation was undone the most recent selection will be invalid.
				// Assume a single entry light card is correct.
				TArray<AActor*> SelectedLightCardActors;
				GetSelectedActors(SelectedLightCardActors);
				if (SelectedLightCardActors.Num() == 1)
				{
					Actor = SelectedLightCardActors[0];
				}
			}
			
			if (ADisplayClusterLightCardActor* LightCardActor = Cast<ADisplayClusterLightCardActor>(Actor))
			{
				FToolMenuSection& Section = InMenu->AddSection("LightCardDefaultSection", LOCTEXT("LightCardSectionName", "Light Cards"));

				Section.AddMenuEntry(FDisplayClusterLightCardEditorCommands::Get().RemoveLightCard);
				Section.AddMenuEntry(FDisplayClusterLightCardEditorCommands::Get().SaveLightCardTemplate);
			}
			
			// View section
			{
				FToolMenuSection& Section = InMenu->AddSection("LightCardViewSection", LOCTEXT("LightCardviewSectionName", "View"));
				Section.AddMenuEntry(FDisplayClusterLightCardEditorCommands::Get().FrameSelection);
			}

			// Edit section
			{
				FToolMenuSection& Section = InMenu->AddSection("LightCardEditSection", LOCTEXT("LightCardEditSectionName", "Edit"));

				Section.AddMenuEntry(FGenericCommands::Get().Cut);
				Section.AddMenuEntry(FGenericCommands::Get().Copy);
				Section.AddMenuEntry(FGenericCommands::Get().Paste);
				Section.AddMenuEntry(FGenericCommands::Get().Duplicate);
				Section.AddMenuEntry(FGenericCommands::Get().Delete);
				Section.AddMenuEntry(FGenericCommands::Get().Rename);
			}
		}));
	}
}

#undef LOCTEXT_NAMESPACE
