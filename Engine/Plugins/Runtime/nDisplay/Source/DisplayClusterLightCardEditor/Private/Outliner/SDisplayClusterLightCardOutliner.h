// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneOutlinerFwd.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class ADisplayClusterRootActor;
class ADisplayClusterLightCardActor;
class ITableRow;
class FExtender;
class FUICommandList;
class FDisplayClusterLightCardEditor;
class STableViewBase;

class SSceneOutliner;
template<class T>
class STreeView;

/** Displays all of the light cards associated with a particular display cluster root actor in a list view */
class SDisplayClusterLightCardOutliner : public SCompoundWidget
{
public:
	struct FStageActorTreeItem
	{
		TWeakObjectPtr<AActor> Actor;

		FStageActorTreeItem() :
			Actor(nullptr)
		{ }
	};

public:
	SLATE_BEGIN_ARGS(SDisplayClusterLightCardOutliner)
	{}
	SLATE_END_ARGS()

	virtual ~SDisplayClusterLightCardOutliner() override;

	void Construct(const FArguments& InArgs, TSharedPtr<FDisplayClusterLightCardEditor> InLightCardEditor, TSharedPtr<FUICommandList> InCommandList);

	void SetRootActor(ADisplayClusterRootActor* NewRootActor);

	const TArray<TSharedPtr<FStageActorTreeItem>>& GetStageActorTreeItems() const { return StageActorTreeItems; }

	/** Gets a list of light card actors that are currently selected in the list */
	void GetSelectedActors(TArray<AActor*>& OutSelectedActors) const;

	/** Select light cards in the outliner */
	void SelectActors(const TArray<AActor*>& ActorsToSelect);

	/** Restores the last valid cached selection to the outliner */
	void RestoreCachedSelection();

	// Outliner is responsible for renames on all items, all other edit options are for folders only
	// Otherwise the light card editor is responsible for actor edits.
	
	bool CanDeleteSelectedFolder() const;
	bool CanCutSelectedFolder() const;
	bool CanCopySelectedFolder() const;
	bool CanPasteSelectedFolder() const;

	/** If either a folder or actor should be renamed */
	bool CanRenameSelectedItem() const;
	/** Rename the selected folder or actor */
	void RenameSelectedItem();
	
private:
	/** Creates a world outliner based on the root actor world */
	void CreateWorldOutliner();
	
	/**
	 * Fill the outliner with available actors
	 * @return True if the list has been modified
	 */
	bool FillActorList();

	/** Called when the user clicks on a selection in the outliner */
	void OnOutlinerSelectionChanged(FSceneOutlinerTreeItemPtr TreeItem, ESelectInfo::Type Type);

	/** Return our stage actor tree item from the outliner tree item */
	TSharedPtr<FStageActorTreeItem> GetStageActorTreeItemFromOutliner(FSceneOutlinerTreeItemPtr InOutlinerTreeItem) const;

	/** Return an actor if the tree item is an actor tree item */
	AActor* GetActorFromTreeItem(FSceneOutlinerTreeItemPtr InOutlinerTreeItem) const;
	
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	/** Registers a context menu for use with tool menus */
	void RegisterContextMenu(FName& InName, struct FToolMenuContext& InContext);

private:
	/** Pointer to the light card editor that owns this widget */
	TWeakPtr<FDisplayClusterLightCardEditor> LightCardEditorPtr;

	/** A hierarchical list of stage actor tree items to be displayed in the tree view */
	TArray<TSharedPtr<FStageActorTreeItem>> StageActorTree;

	/** All actors currently tracked by the scene outliner */
	TMap<TObjectPtr<AActor>, TSharedPtr<FStageActorTreeItem>> TrackedActors;

	/** Cached items currently selected. Used for synchronizing with the outliner mode */
	TArray<FSceneOutlinerTreeItemPtr> CachedOutlinerItems;
	
	/** A flattened list of all the light card actors being displayed in the tree view */
	TArray<TSharedPtr<FStageActorTreeItem>> StageActorTreeItems;

	/**
	 * Additional selected actors that aren't visible in the outliner hierarchy.
	 * We store these to avoid trampling the light card editor's selection, which may include actors that aren't displayed in the outliner.
	 */
	TArray<TWeakObjectPtr<AActor>> NonOutlinerSelectedActors;

	/** The active root actor whose light cards are being displayed */
	TWeakObjectPtr<ADisplayClusterRootActor> RootActor;

	/** The scene outliner managed by this widget */
	TSharedPtr<SSceneOutliner> SceneOutliner;

	/** The most recently selected item from the outliner */
	TWeakPtr<ISceneOutlinerTreeItem> MostRecentSelectedItem;

	/** Called when the outliner makes a selection change */
	FDelegateHandle SceneOutlinerSelectionChanged;
	
	/** Extenders for menus */
	TSharedPtr<FExtender> Extenders;

	/** Mapped commands for this list */
	TSharedPtr<FUICommandList> CommandList;

	/** If the outliner is responsible for changing the actor selection */
	bool bIsOutlinerChangingSelection = false;
};