// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterLightCardEditorSettings.h"

#include "LightCardTemplates/DisplayClusterLightCardTemplate.h"

#include "DisplayClusterMeshProjectionRenderer.h"

#include "DisplayClusterLightCardActor.h"
#include "Components/DisplayClusterLabelComponent.h"

#include "Editor/UnrealEdTypes.h"
#include "Engine/Texture.h"
#include "Styling/SlateBrush.h"
#include "Styling/SlateIconFinder.h"

#define LOCTEXT_NAMESPACE "DisplayClusterLightCardEditorSettings"

UDisplayClusterLightCardEditorProjectSettings::UDisplayClusterLightCardEditorProjectSettings()
{
	LightCardTemplateDefaultPath.Path = TEXT("/Game/VP/LightCards");
	LightCardLabelScale = 1.f;
	bDisplayLightCardLabels = false;
	LightCardLabelFlags = EDisplayClusterLabelFlags::DisplayInGame | EDisplayClusterLabelFlags::DisplayInEditor;
}

const FName FDisplayClusterLightCardEditorRecentItem::Type_LightCard = "LightCard";
const FName FDisplayClusterLightCardEditorRecentItem::Type_Flag = "Flag";
const FName FDisplayClusterLightCardEditorRecentItem::Type_LightCardTemplate = "LightCardTemplate";
const FName FDisplayClusterLightCardEditorRecentItem::Type_Dynamic = "Dynamic";

FText FDisplayClusterLightCardEditorRecentItem::GetItemDisplayName() const
{
	if (ItemType == Type_LightCard)
	{
		return LOCTEXT("LightCardRecentItemName", "Light Card");
	}
	if (ItemType == Type_Flag)
	{
		return LOCTEXT("FlagRecentItemName", "Flag");
	}
	if (const UObject* Object = ObjectPath.LoadSynchronous())
	{
		if (const UClass* Class = Cast<UClass>(Object))
		{
			return Class->GetDisplayNameText();
		}
		
		FString ObjectName = Object->GetName();
		ObjectName.RemoveFromEnd(TEXT("_C"));
		return FText::FromString(MoveTemp(ObjectName));
	}

	return FText::GetEmpty();
}

const FSlateBrush* FDisplayClusterLightCardEditorRecentItem::GetSlateBrush() const
{
	if (ItemType == Type_Dynamic)
	{
		if (const UObject* Object = ObjectPath.LoadSynchronous())
		{
			if (const UClass* Class = Cast<UClass>(Object))
			{
				return FSlateIconFinder::FindIconBrushForClass(Class);
			}
		}
	}

	if (ItemType == Type_LightCard || ItemType == Type_Flag || ItemType == Type_Dynamic)
	{
		return FSlateIconFinder::FindIconBrushForClass(ADisplayClusterLightCardActor::StaticClass());
	}
	
	if (ItemType == Type_LightCardTemplate && !SlateBrush.IsValid())
	{
		if (const UDisplayClusterLightCardTemplate* Template = Cast<UDisplayClusterLightCardTemplate>(ObjectPath.LoadSynchronous()))
		{
			UTexture* Texture = Template->LightCardActor != nullptr ? Template->LightCardActor->Texture.Get() : nullptr;
			if (Texture == nullptr)
			{
				return FSlateIconFinder::FindIconBrushForClass(Template->GetClass());;
			}
			
			if (!SlateBrush.IsValid())
			{
				SlateBrush = MakeShared<FSlateBrush>();
			}
		
			SlateBrush->SetResourceObject(Texture);
			SlateBrush->ImageSize = FVector2D(16.f, 16.f);
		}
	}

	if (SlateBrush.IsValid())
	{
		return &*SlateBrush;
	}

	return nullptr;
}

UDisplayClusterLightCardEditorSettings::UDisplayClusterLightCardEditorSettings()
{
	ProjectionMode = EDisplayClusterMeshProjectionType::Azimuthal;
	RenderViewportType = ELevelViewportType::LVT_Perspective;
	
	bDisplayIcons = true;
    IconScale = 0.5f;
}

#undef LOCTEXT_NAMESPACE