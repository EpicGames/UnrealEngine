// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DisplayClusterLightCardEditorSettings.generated.h"

enum class EDisplayClusterLabelFlags : uint8;
class UDisplayClusterLightCardTemplate;
struct FSlateBrush;

/**
 * Default settings shared across users of the project
 */
UCLASS(config = Editor, defaultconfig, meta = (DisplayClusterMultiUserInclude))
class UDisplayClusterLightCardEditorProjectSettings : public UObject
{
	GENERATED_BODY()

public:
	UDisplayClusterLightCardEditorProjectSettings();

	/** The default path to save new light card templates */
	UPROPERTY(config, EditAnywhere, Category = Templates)
	FDirectoryPath LightCardTemplateDefaultPath;

	/** The default template to use when creating a new light card */
	UPROPERTY(config, EditAnywhere, Category = Templates)
	TSoftObjectPtr<UDisplayClusterLightCardTemplate> DefaultLightCardTemplate;

	/** The default template to use when creating a new flag */
	UPROPERTY(config, EditAnywhere, Category = Templates)
	TSoftObjectPtr<UDisplayClusterLightCardTemplate> DefaultFlagTemplate;
	
	/** Whether light card labels should be displayed. Handled through the light card editor */
	UPROPERTY()
	bool bDisplayLightCardLabels;
	
	/** Flags for the label */
	UPROPERTY()
	EDisplayClusterLabelFlags LightCardLabelFlags;
	
	/** The scale to use for light card labels */
	UPROPERTY(config, EditAnywhere, Category = Labels)
	float LightCardLabelScale;
};

USTRUCT()
struct FDisplayClusterLightCardEditorRecentItem
{
	GENERATED_BODY()
	
	static const FName Type_LightCard;
	static const FName Type_Flag;
	static const FName Type_LightCardTemplate;
	static const FName Type_Dynamic;

	/** Path of object to instantiate */
	UPROPERTY()
	TSoftObjectPtr<UObject> ObjectPath;
	
	/** Type of item placed */
	UPROPERTY()
	FName ItemType;

	FText GetItemDisplayName() const;
	const FSlateBrush* GetSlateBrush() const;

	bool operator==(const FDisplayClusterLightCardEditorRecentItem& Rhs) const
	{
		return ObjectPath == Rhs.ObjectPath && ItemType == Rhs.ItemType;
	}

private:
	mutable TSharedPtr<FSlateBrush> SlateBrush;
};

/**
 * Editor preferences unique to this user
 */
UCLASS(config = EditorPerProjectUserSettings)
class UDisplayClusterLightCardEditorSettings : public UObject
{
	GENERATED_BODY()

public:
	UDisplayClusterLightCardEditorSettings();

	/** Items recently placed by the user */
	UPROPERTY(config)
	TArray<FDisplayClusterLightCardEditorRecentItem> RecentlyPlacedItems;

	/** Last used projection mode user has set */
	UPROPERTY(config)
	uint8 ProjectionMode;

	/** Last used viewport type user has set */
	UPROPERTY(config)
	uint8 RenderViewportType;

	/** Display icons in the light card editor where applicable */
	UPROPERTY(config, EditAnywhere, Category = Icons)
	bool bDisplayIcons;

	/** Scale icon size where applicable */
	UPROPERTY(config, EditAnywhere, Category = Icons)
	float IconScale;
};
