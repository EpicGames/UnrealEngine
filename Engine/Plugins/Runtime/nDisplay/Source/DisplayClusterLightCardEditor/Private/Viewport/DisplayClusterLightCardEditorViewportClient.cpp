// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterLightCardEditorViewportClient.h"

#include "Application/ThrottleManager.h"
#include "DisplayClusterLightcardEditorViewport.h"
#include "DisplayClusterLightCardEditorWidget.h"
#include "LightCardTemplates/DisplayClusterLightCardTemplate.h"

#include "Components/DisplayClusterCameraComponent.h"
#include "Components/DisplayClusterScreenComponent.h"
#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterLightCardActor.h"
#include "DisplayClusterLightCardEditor.h"
#include "DisplayClusterLightCardEditorHelper.h"
#include "DisplayClusterLightCardEditorLog.h"
#include "DisplayClusterLightCardEditorUtils.h"
#include "DisplayClusterProjectionStrings.h"
#include "DisplayClusterRootActor.h"
#include "DisplayClusterRootActorContainers.h"
#include "IDisplayClusterScenePreview.h"
#include "PrimitiveDrawingUtils.h"
#include "Settings/DisplayClusterLightCardEditorSettings.h"

#include "IDisplayClusterLightCardExtenderModule.h"

#include "Render/Viewport/IDisplayClusterViewportManager.h"

#include "AudioDevice.h"
#include "CameraController.h"
#include "Components/BillboardComponent.h"
#include "Components/DirectionalLightComponent.h"
#include "Components/LineBatchComponent.h"
#include "Components/PostProcessComponent.h"
#include "Components/PrimitiveComponent.h"
#include "Components/SkyLightComponent.h"
#include "Components/WidgetComponent.h"
#include "Debug/DebugDrawService.h"
#include "Editor/UnrealEdEngine.h"
#include "EditorModeManager.h"
#include "EditorModes.h"
#include "Engine/Canvas.h"
#include "Engine/DebugDisplayProperty.h"
#include "Engine/TextureRenderTarget2D.h"
#include "EngineModule.h"
#include "EngineUtils.h"
#include "ImageUtils.h"
#include "Kismet/GameplayStatics.h"
#include "Kismet/KismetMathLibrary.h"
#include "KismetProceduralMeshLibrary.h"
#include "LegacyScreenPercentageDriver.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/ScopeLock.h"
#include "PreviewScene.h"
#include "ProceduralMeshComponent.h"
#include "RenderResource.h"
#include "ScopedTransaction.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "Slate/SceneViewport.h"
#include "TextureResource.h"
#include "UnrealEdGlobals.h"
#include "UnrealWidget.h"
#include "Components/DisplayClusterLabelComponent.h"
#include "Components/DisplayClusterStageGeometryComponent.h"
#include "Widgets/Docking/SDockTab.h"


#define LOCTEXT_NAMESPACE "DisplayClusterLightCardEditorViewportClient"


/** Console variable used to control the size of the UV light card map texture */
static TAutoConsoleVariable<bool> CVarICVFXPanelAutoPan(
	TEXT("nDisplay.panel.autopan"),
	true,
	TEXT("When true, the stage view will automatically tilt and pan on supported map projections as you drag objects near the edges of the screen."));

//////////////////////////////////////////////////////////////////////////
// FDisplayClusterLightCardEditorViewportClient

FDisplayClusterLightCardEditorViewportClient::FDisplayClusterLightCardEditorViewportClient(FPreviewScene& InPreviewScene,
	const TWeakPtr<SDisplayClusterLightCardEditorViewport>& InEditorViewportWidget)
	: FEditorViewportClient(nullptr, &InPreviewScene, InEditorViewportWidget)
	, LightCardEditorViewportPtr(InEditorViewportWidget)
{
	check(InEditorViewportWidget.IsValid());
	
	LightCardEditorPtr = InEditorViewportWidget.Pin()->GetLightCardEditor();
	check(LightCardEditorPtr.IsValid());

	IDisplayClusterScenePreview& PreviewModule = IDisplayClusterScenePreview::Get();
	PreviewRendererId = PreviewModule.CreateRenderer();
	PreviewModule.SetRendererActorSelectedDelegate(
		PreviewRendererId,
		FDisplayClusterMeshProjectionRenderer::FSelection::CreateRaw(this, &FDisplayClusterLightCardEditorViewportClient::IsActorSelected)
	);
	PreviewModule.SetRendererRenderSimpleElementsDelegate(
		PreviewRendererId,
		FDisplayClusterMeshProjectionRenderer::FSimpleElementPass::CreateRaw(this, &FDisplayClusterLightCardEditorViewportClient::Draw)
	);

	ProjectionHelper = MakeShared<FDisplayClusterLightCardEditorHelper>(PreviewRendererId);

	InputMode = EInputMode::Idle;
	DragWidgetOffset = FVector::ZeroVector;
	
	EditorWidget = MakeShared<FDisplayClusterLightCardEditorWidget>();

	// Setup defaults for the common draw helper.
	bUsesDrawHelper = false;

	EngineShowFlags.SetSelectionOutline(true);
	 
	check(Widget);
	Widget->SetSnapEnabled(true);
	
	ShowWidget(true);

	SetViewMode(VMI_Unlit);
	
	ViewportType = LVT_Perspective;
	bSetListenerPosition = false;
	bUseNumpadCameraControl = false;
	SetRealtime(true);
	SetShowStats(true);

	ResetProjectionViewConfigurations();

	//This seems to be needed to get the correct world time in the preview.
	SetIsSimulateInEditorViewport(true);

	const UDisplayClusterLightCardEditorSettings* Settings = GetDefault<UDisplayClusterLightCardEditorSettings>();
	
	SetProjectionMode(static_cast<EDisplayClusterMeshProjectionType>(Settings->ProjectionMode),
		static_cast<ELevelViewportType>(Settings->RenderViewportType));

	SetCameraSpeedSettings(FEditorViewportCameraSpeedSettings(1.0f));

	IDisplayClusterLightCardExtenderModule& LightCardExtenderModule = IDisplayClusterLightCardExtenderModule::Get();
	LightCardExtenderModule.GetOnSequencerTimeChanged().AddRaw(this, &FDisplayClusterLightCardEditorViewportClient::OnSequencerTimeChanged);
}

FDisplayClusterLightCardEditorViewportClient::~FDisplayClusterLightCardEditorViewportClient()
{
	IDisplayClusterScenePreview::Get().DestroyRenderer(PreviewRendererId);

	EndTransaction();

	IDisplayClusterLightCardExtenderModule& LightCardExtenderModule = IDisplayClusterLightCardExtenderModule::Get();
	LightCardExtenderModule.GetOnSequencerTimeChanged().RemoveAll(this);
}

FLinearColor FDisplayClusterLightCardEditorViewportClient::GetBackgroundColor() const
{
	return FLinearColor::Gray;
}

void FDisplayClusterLightCardEditorViewportClient::Tick(float DeltaSeconds)
{
	FEditorViewportClient::Tick(DeltaSeconds);

	// Camera position is locked to a specific location
	FixCameraTransform();

	CalcEditorWidgetTransform(CachedEditorWidgetWorldTransform);

	// Tick the preview scene world.
	// Allow full tick only if preview simulation is enabled and we're not currently in an active SIE or PIE session
	if (GEditor->PlayWorld == nullptr && !GEditor->bIsSimulatingInEditor)
	{
		PreviewScene->GetWorld()->Tick(IsRealtime() ? LEVELTICK_All : LEVELTICK_TimeOnly, DeltaSeconds);
	}
	else
	{
		PreviewScene->GetWorld()->Tick(IsRealtime() ? LEVELTICK_ViewportsOnly : LEVELTICK_TimeOnly, DeltaSeconds);
	}

	// EditorViewportClient sets the cursor settings based on the state of the built in FWidget, which isn't being used here, so
	// force a hardware cursor if we are dragging an actor so that the correct mouse cursor shows up
	switch (InputMode)
	{
	case EInputMode::DraggingActor:
		SetRequiredCursor(true, false);
		SetRequiredCursorOverride(true, EMouseCursor::GrabHandClosed);
		break;

	case EInputMode::DrawingLightCard:
		SetRequiredCursor(true, false);
		SetRequiredCursorOverride(true, EMouseCursor::Crosshairs);
		break;
	}

	if (DesiredLookAtLocation.IsSet())
	{
		const FRotator LookAtRotation = UKismetMathLibrary::FindLookAtRotation(GetViewLocation(), *DesiredLookAtLocation);
		const FRotator NewRotation = FMath::RInterpTo(GetViewRotation(), LookAtRotation, DeltaSeconds, DesiredLookAtSpeed);
		SetViewRotation(NewRotation);

		FSceneViewInitOptions SceneVewInitOptions;
		GetSceneViewInitOptions(SceneVewInitOptions);
		FViewMatrices ViewMatrices(SceneVewInitOptions);

		FVector ProjectedEditorWidgetPosition = ProjectWorldPosition(CachedEditorWidgetWorldTransform.GetTranslation(), ViewMatrices);

		if (NewRotation.Equals(LookAtRotation, 2.f) ||
			!IsLocationCloseToEdge(ProjectedEditorWidgetPosition))
		{
			DesiredLookAtLocation.Reset();
		}
	}

	// The root actor could be stale if it was set during a re-instancing from a compile, but the editor should have the up-to-date actor.
	if (RootActorLevelInstance.IsStale() && LightCardEditorPtr.IsValid() &&
		(LightCardEditorPtr.Pin()->GetActiveRootActor().IsValid() || LightCardEditorPtr.Pin()->GetActiveRootActor().IsExplicitlyNull()))
	{
		UpdatePreviewActor(LightCardEditorPtr.Pin()->GetActiveRootActor().Get(), true);
	}

	if (EditorWidget.IsValid())
	{
		EditorWidget->Tick(this, DeltaSeconds);
	}
}

void FDisplayClusterLightCardEditorViewportClient::Draw(FViewport* InViewport, FCanvas* Canvas)
{
	FViewport* ViewportBackup = Viewport;
	Viewport = InViewport ? InViewport : Viewport;

	UWorld* World = GetWorld();
	FGameTime Time;
	if (!World || (GetScene() != World->Scene) || UseAppTime()) 
	{
		Time = FGameTime::GetTimeSinceAppStart();
	}
	else
	{
		Time = World->GetTime();
	}

	FEngineShowFlags UseEngineShowFlags = EngineShowFlags;
	if (OverrideShowFlagsFunc)
	{
		OverrideShowFlagsFunc(UseEngineShowFlags);
	}

	// Setup a FSceneViewFamily/FSceneView for the viewport.
	FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
		Canvas->GetRenderTarget(),
		GetScene(),
		UseEngineShowFlags)
		.SetTime(Time)
		.SetRealtimeUpdate(IsRealtime() && FSlateThrottleManager::Get().IsAllowingExpensiveTasks())
	);

	ViewFamily.DebugDPIScale = GetDPIScale();
	ViewFamily.bIsHDR = Viewport->IsHDRViewport();

	ViewFamily.EngineShowFlags = UseEngineShowFlags;
	ViewFamily.EngineShowFlags.CameraInterpolation = 0;
	ViewFamily.EngineShowFlags.SetScreenPercentage(false);

	ViewFamily.ViewExtensions = GEngine->ViewExtensions->GatherActiveExtensions(FSceneViewExtensionContext(InViewport));

	for (auto ViewExt : ViewFamily.ViewExtensions)
	{
		ViewExt->SetupViewFamily(ViewFamily);
	}

	ViewFamily.ViewMode = VMI_Unlit;

	EngineShowFlagOverride(ESFIM_Editor, ViewFamily.ViewMode, ViewFamily.EngineShowFlags, false);
	EngineShowFlagOrthographicOverride(IsPerspective(), ViewFamily.EngineShowFlags);

	ViewFamily.ExposureSettings = ExposureSettings;

	// Setup the screen percentage and upscaling method for the view family.
	{
		checkf(ViewFamily.GetScreenPercentageInterface() == nullptr,
			TEXT("Some code has tried to set up an alien screen percentage driver, that could be wrong if not supported very well by the RHI."));

		if (SupportsLowDPIPreview() && IsLowDPIPreview() && ViewFamily.SupportsScreenPercentage())
		{
			ViewFamily.SecondaryViewFraction = GetDPIDerivedResolutionFraction();
		}
	}

	FSceneView* View = nullptr;

	View = CalcSceneView(&ViewFamily, INDEX_NONE);
	SetupViewForRendering(ViewFamily,*View);

	FSlateRect SafeFrame;
	View->CameraConstrainedViewRect = View->UnscaledViewRect;
	if (CalculateEditorConstrainedViewRect(SafeFrame, Viewport, Canvas->GetDPIScale()))
	{
		View->CameraConstrainedViewRect = FIntRect(SafeFrame.Left, SafeFrame.Top, SafeFrame.Right, SafeFrame.Bottom);
	}

	{
		// If a screen percentage interface was not set by one of the view extension, then set the legacy one.
		if (ViewFamily.GetScreenPercentageInterface() == nullptr)
		{
			float GlobalResolutionFraction = 1.0f;

			if (SupportsPreviewResolutionFraction() && ViewFamily.SupportsScreenPercentage())
			{
				GlobalResolutionFraction = GetDefaultPrimaryResolutionFractionTarget();

				// Force screen percentage's engine show flag to be turned on for preview screen percentage.
				ViewFamily.EngineShowFlags.ScreenPercentage = (GlobalResolutionFraction != 1.0);
			}

			// In editor viewport, we ignore r.ScreenPercentage and FPostProcessSettings::ScreenPercentage by design.
			ViewFamily.SetScreenPercentageInterface(new FLegacyScreenPercentageDriver(ViewFamily, GlobalResolutionFraction));
		}

		check(ViewFamily.GetScreenPercentageInterface() != nullptr);
	}

	Canvas->Clear(FLinearColor::Black);

	FSceneViewInitOptions SpriteSceneViewInitOptions;
	
	if (!bDisableCustomRenderer)
	{
		FDisplayClusterMeshProjectionRenderSettings RenderSettings;
		RenderSettings.RenderType = EDisplayClusterMeshProjectionOutput::Color;
		RenderSettings.EngineShowFlags = EngineShowFlags;

		ProjectionHelper->ConfigureRenderProjectionSettings(RenderSettings, GetViewTransform().GetLocation());
		RenderSettings.PrimitiveFilter.ShouldRenderPrimitiveDelegate = ProjectionHelper->CreateDefaultShouldRenderPrimitiveFilter();
		RenderSettings.PrimitiveFilter.ShouldApplyProjectionDelegate = ProjectionHelper->CreateDefaultShouldApplyProjectionToPrimitiveFilter();

		GetSceneViewInitOptions(RenderSettings.ViewInitOptions);

		SpriteSceneViewInitOptions = RenderSettings.ViewInitOptions;
		
		IDisplayClusterScenePreview::Get().Render(PreviewRendererId, RenderSettings, *Canvas);
	}
	else
	{
		if (BillboardComponentProxies.Num() > 0 || WidgetComponentProxies.Num() > 0)
		{
			GetSceneViewInitOptions(SpriteSceneViewInitOptions);
		}
		
		GetRendererModule().BeginRenderingViewFamily(Canvas, &ViewFamily);
	}

	if (BillboardComponentProxies.Num() > 0 || WidgetComponentProxies.Num() > 0)
	{
		FScopeLock Lock(&SpriteComponentCS);
		SpriteViewMatrices = FViewMatrices(SpriteSceneViewInitOptions);
	}

	if (View)
	{
		DrawCanvas(*Viewport, *View, *Canvas);
	}

	// Remove temporary debug lines.
	// Possibly a hack. Lines may get added without the scene being rendered etc.
	constexpr const UWorld::ELineBatcherType LineBatchersToFlush[] = { UWorld::ELineBatcherType::World, UWorld::ELineBatcherType::Foreground };
	World->FlushLineBatchers(LineBatchersToFlush);

	// Draw the widget.
	/*if (Widget && bShowWidget)
	{
		Widget->DrawHUD( Canvas );
	}*/

	// Axes indicators
	if (bDrawAxes && !ViewFamily.EngineShowFlags.Game && !GLevelEditorModeTools().IsViewportUIHidden() && !IsVisualizeCalibrationMaterialEnabled())
	{
		// Don't draw the usual 3D axes if the projection mode is UV
		if (ProjectionMode != EDisplayClusterMeshProjectionType::UV)
		{
			DrawAxes(Viewport, Canvas);
		}
	}

	// NOTE: DebugCanvasObject will be created by UDebugDrawService::Draw() if it doesn't already exist.
	FCanvas* DebugCanvas = Viewport->GetDebugCanvas();
	UDebugDrawService::Draw(ViewFamily.EngineShowFlags, Viewport, View, DebugCanvas);
	UCanvas* DebugCanvasObject = FindObjectChecked<UCanvas>(GetTransientPackage(),TEXT("DebugCanvasObject"));
	DebugCanvasObject->Canvas = DebugCanvas;
	DebugCanvasObject->Init( Viewport->GetSizeXY().X, Viewport->GetSizeXY().Y, View , DebugCanvas);

	// Stats display
	if( IsRealtime() && ShouldShowStats() && DebugCanvas)
	{
		const int32 XPos = 4;
		TArray< FDebugDisplayProperty > EmptyPropertyArray;
		DrawStatsHUD( World, Viewport, DebugCanvas, NULL, EmptyPropertyArray, GetViewLocation(), GetViewRotation() );
	}

	// Show warning if nDisplay real-time rendering is disabled
	if (!IDisplayClusterScenePreview::Get().IsRealTimePreviewEnabled())
	{
		const FString WarningLines[] = {
			LOCTEXT("RealTimeWarningLine1", "WARNING: Real-time rendering is disabled.").ToString(),
			LOCTEXT("RealTimeWarningLine2", "nDisplay previews and light card positions may not update correctly.").ToString(),
		};
		const int32 TextMargin = 8;

		const float RightMarginX = Viewport->GetSizeXY().X - TextMargin;
		FCanvasTextItem RealTimeWarningText(
			FVector2D(RightMarginX, Viewport->GetSizeXY().Y - TextMargin),
			FText::GetEmpty(),
			GEngine->GetSmallFont(),
			FLinearColor::Yellow
		);
		RealTimeWarningText.EnableShadow(FLinearColor::Black);

		// Draw each line, starting from the bottom edge of the viewport
		for (int32 LineIndex = sizeof(WarningLines) / sizeof(FString) - 1; LineIndex >= 0; --LineIndex)
		{
			const FString& LineString = WarningLines[LineIndex];
			const int32 TextWidth = RealTimeWarningText.Font->GetStringSize(*LineString);

			RealTimeWarningText.Text = FText::FromString(LineString);
			RealTimeWarningText.Position.X = RightMarginX - TextWidth;
			RealTimeWarningText.Position.Y -= RealTimeWarningText.Font->GetStringHeightSize(*LineString);

			Canvas->DrawItem(RealTimeWarningText);
		}
	}

	if(!IsRealtime())
	{
		// Wait for the rendering thread to finish drawing the view before returning.
		// This reduces the apparent latency of dragging the viewport around.
		FlushRenderingCommands();
	}

	Viewport = ViewportBackup;
}

void FDisplayClusterLightCardEditorViewportClient::Draw(const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	const bool bIsUVProjection = ProjectionMode == EDisplayClusterMeshProjectionType::UV;

	// Draw any sprites - this would normally go under the CPU draw which would avoid needing critical regions, but
	// we can't render the gizmo over the icons in that case. Canvas items drawn onto the viewport will draw onto the
	// back buffer after the entire custom render pipeline is drawn.
	{
		FScopeLock Lock(&SpriteComponentCS);
		for (const TWeakObjectPtr<UBillboardComponent>& BillboardComponent : BillboardComponentProxies)
		{
			if (!BillboardComponent.IsValid())
			{
				continue;
			}

			if (const IDisplayClusterStageActor* StageActor = Cast<IDisplayClusterStageActor>(BillboardComponent->GetOwner()))
			{
				if ((bIsUVProjection && !StageActor->IsUVActor()) || (!bIsUVProjection && StageActor->IsUVActor()))
				{
					continue;
				}
			}
			
			FSpriteProxy SpriteProxy = FSpriteProxy::FromBillboard(BillboardComponent.Get());
		
			FVector ProjectedLocation = ProjectWorldPosition(SpriteProxy.WorldPosition, SpriteViewMatrices);
		
			PDI->SetHitProxy(new HActor(BillboardComponent->GetOwner(), BillboardComponent.Get()));
		
			if (const FTexture* TextureResource = SpriteProxy.Sprite ? SpriteProxy.Sprite->GetResource() : nullptr)
			{
				const UDisplayClusterLightCardEditorSettings* Settings = GetDefault<UDisplayClusterLightCardEditorSettings>();

				if (Settings->bDisplayIcons)
				{
					const float UL = SpriteProxy.UL == 0.0f ? TextureResource->GetSizeX() : SpriteProxy.UL;
					const float VL = SpriteProxy.VL == 0.0f ? TextureResource->GetSizeY() : SpriteProxy.VL;
			
					// Calculate the view-dependent scaling factor.
					float ViewedSizeX = SpriteProxy.SpriteScale * UL;
					float ViewedSizeY = SpriteProxy.SpriteScale * VL;

					if (SpriteProxy.bIsScreenSizeScaled && (View->ViewMatrices.GetViewToClip().M[3][3] != 1.0f))
					{
						const float ZoomFactor = FMath::Min<float>(View->ViewMatrices.GetViewToClip().M[0][0], View->ViewMatrices.GetViewToClip().M[1][1]);
						if(ZoomFactor != 0.0f)
						{
							const float Radius = View->WorldToScreen(ProjectedLocation).W * (SpriteProxy.ScreenSize / ZoomFactor);

							if (Radius < 1.0f)
							{
								ViewedSizeX *= Radius;
								ViewedSizeY *= Radius;
							}					
						}
					}
				
					ViewedSizeX *= Settings->IconScale;
					ViewedSizeY *= Settings->IconScale;
			
					PDI->DrawSprite(ProjectedLocation, ViewedSizeX, ViewedSizeY, TextureResource, FLinearColor::White, SDPG_World,
						SpriteProxy.U, UL, SpriteProxy.V, VL, SE_BLEND_Masked, SpriteProxy.OpacityMaskRefVal);
				}
			}

			PDI->SetHitProxy(nullptr);
		}
	}

	if (!bIsUVProjection) // Labels not currently supported in UV mode
	{
		FScopeLock Lock(&SpriteComponentCS);
		for (const TWeakObjectPtr<UWidgetComponent>& WidgetComponent : WidgetComponentProxies)
		{
			if (!WidgetComponent.IsValid())
			{
				continue;
			}
			
			if (UTextureRenderTarget2D* RenderTarget = WidgetComponent->GetRenderTarget())
			{
				if (const FTextureResource* Resource = RenderTarget->GetResource())
				{
					const FVector2D Size = WidgetComponent->GetDrawSize();
					const float UL = Size.X;
					const float VL = Size.Y;

					// Labels can appear large by default, especially in dome projection mode
					const float ScaleModifier = ProjectionMode == EDisplayClusterMeshProjectionType::Azimuthal ? 0.125f : 0.25f;

					const UDisplayClusterLightCardEditorProjectSettings* Settings = GetDefault<UDisplayClusterLightCardEditorProjectSettings>();
					const float ViewedSizeX = (Settings->LightCardLabelScale * UL) * ScaleModifier;
					const float ViewedSizeY = (Settings->LightCardLabelScale * VL) * ScaleModifier;

					const FVector ProjectedLocation = ProjectWorldPosition(WidgetComponent->GetComponentLocation(), SpriteViewMatrices);
				
					PDI->SetHitProxy(new HActor(WidgetComponent->GetOwner(), WidgetComponent.Get()));
					PDI->DrawSprite(ProjectedLocation, ViewedSizeX, ViewedSizeY,
						Resource, FLinearColor::White,
						SDPG_World, 0, 0, 0, 0, SE_BLEND_Masked);
					PDI->SetHitProxy(nullptr);
				}
			}
		}
	}
		
	if (LastSelectedActor.IsValid() && LastSelectedActor->IsUVActor() == bIsUVProjection)
	{
		// Project the editor widget's world position into projection space so that it renders at the appropriate screen location.
		// This needs to be computed on the render thread using the render thread's scene view, which will be behind the game thread's scene view
		// by at least one frame
		const bool bDrawZAxis = EditorWidget->GetWidgetMode() == FDisplayClusterLightCardEditorWidget::EWidgetMode::WM_Translate &&
			EditorWidgetCoordinateSystem == FDisplayClusterLightCardEditorHelper::ECoordinateSystem::Cartesian;

		EditorWidget->SetTransform(CachedEditorWidgetWorldTransform);
		EditorWidget->SetProjectionTransform(FDisplayClusterMeshProjectionTransform(ProjectionMode, View->ViewMatrices.GetWorldToView()));
		EditorWidget->Draw(View, this, PDI, bDrawZAxis);
	}

	// Draw polygonal lightcard in progress
	if (DrawnMousePositions.Num())
	{
		const FDisplayClusterMeshProjectionTransform ProjectionTransform(ProjectionMode, View->ViewMatrices.GetWorldToView());

		const FLinearColor LineColor = FLinearColor::Green;
		constexpr float LineThickness = 5;

		// Convenience function to draw a line between two mouse points
		auto DrawLine = [&](const FIntPoint& MousePosA, const FIntPoint& MousePosB, const FLinearColor& LineColor)
		{
			FVector4 ScreenPosA = View->PixelToScreen(MousePosA.X, MousePosA.Y, 0);
			FVector4 ScreenPosB = View->PixelToScreen(MousePosB.X, MousePosB.Y, 0);

			// ScreenPos will have Z=0 here, but we need Z=1 to be in the near plane.
			ScreenPosA.Z = 1;
			ScreenPosB.Z = 1;

			FVector4 ViewPosA = View->ViewMatrices.GetClipToView().TransformFVector4(ScreenPosA);
			FVector4 ViewPosB = View->ViewMatrices.GetClipToView().TransformFVector4(ScreenPosB);

			constexpr float NearPlaneSafetyFactor = 1.01f;

			if (RenderViewportType == LVT_Perspective)
			{
				ViewPosA.W /= NearPlaneSafetyFactor;
			}
			else
			{
				ViewPosA.Z = GNearClippingPlane * NearPlaneSafetyFactor;
			}

			const FVector4 WorldPosA = View->ViewMatrices.GetViewToWorld().TransformFVector4(ViewPosA);
			const FVector4 WorldPosB = View->ViewMatrices.GetViewToWorld().TransformFVector4(ViewPosB);

			const FVector PointA = WorldPosA / WorldPosA.W;
			const FVector PointB = WorldPosB / WorldPosB.W;

			PDI->DrawLine(PointA, PointB, LineColor, ESceneDepthPriorityGroup::SDPG_Foreground, LineThickness, 0.0, true /* bScreenSpace */);
		};

		// Draw committed lines
		for (int32 PosIdx = 0; PosIdx < DrawnMousePositions.Num() - 1; PosIdx++)
		{
			DrawLine(DrawnMousePositions[PosIdx], DrawnMousePositions[PosIdx + 1], LineColor);
		}

		// Draw line to current mouse position, slightly darker as a visual queue that it is not committed yet.
		DrawLine(DrawnMousePositions[DrawnMousePositions.Num() - 1], FIntPoint(GetCachedMouseX(), GetCachedMouseY()), LineColor/2);
	}
}

FSceneView* FDisplayClusterLightCardEditorViewportClient::CalcSceneView(FSceneViewFamily* ViewFamily, const int32 StereoViewIndex)
{
	FSceneViewInitOptions ViewInitOptions;
	GetSceneViewInitOptions(ViewInitOptions);

	ViewInitOptions.ViewFamily = ViewFamily;

	TimeForForceRedraw = 0.0;

	FSceneView* View = new FSceneView(ViewInitOptions);

	View->SubduedSelectionOutlineColor = GEngine->GetSubduedSelectionOutlineColor();

	int32 FamilyIndex = ViewFamily->Views.Add(View);
	check(FamilyIndex == View->StereoViewIndex || View->StereoViewIndex == INDEX_NONE);

	View->StartFinalPostprocessSettings(View->ViewLocation);

	OverridePostProcessSettings(*View);

	View->EndFinalPostprocessSettings(ViewInitOptions);

	for (int ViewExt = 0; ViewExt < ViewFamily->ViewExtensions.Num(); ViewExt++)
	{
		ViewFamily->ViewExtensions[ViewExt]->SetupView(*ViewFamily, *View);
	}

	return View;
}

bool FDisplayClusterLightCardEditorViewportClient::InputKey(const FInputKeyEventArgs& EventArgs)
{
	switch (InputMode)
	{
	case EInputMode::DrawingLightCard:
		
		if ((EventArgs.Key == EKeys::LeftMouseButton) && (EventArgs.Event == IE_Released))
		{
			// Add a new light card polygon point

			FIntPoint MousePos;
			EventArgs.Viewport->GetMousePos(MousePos);

			if (!DrawnMousePositions.Num() || (DrawnMousePositions.Last() != MousePos))
			{
				// Append to array of points
				DrawnMousePositions.Add(MousePos);
			}
		}
		else if ((EventArgs.Key == EKeys::RightMouseButton) && (EventArgs.Event == IE_Released))
		{
			// Create the polygon light card.
			CreateDrawnLightCard(DrawnMousePositions);

			// Reset the list of polygon points used to create the new card
			DrawnMousePositions.Empty();

			// We go back to idle input mode
			InputMode = EInputMode::Idle;
		}

		// Returning here (and not calling Super::InputKey) locks the viewport so that the user does not inadvertently 
		// change the perspective while in the middle of drawing a light card, as that is not currently supported
		return true;

	default:

		if ((EventArgs.Key == EKeys::MouseScrollUp || EventArgs.Key == EKeys::MouseScrollDown) && (EventArgs.Event == IE_Pressed))
		{
			const int32 Sign = EventArgs.Key == EKeys::MouseScrollUp ? -1 : 1;
			const float CurrentFOV = GetProjectionModeFOV(ProjectionMode);
			const float NewFOV = FMath::Clamp(CurrentFOV + Sign * FOVScrollIncrement, CameraController->GetConfig().MinimumAllowedFOV, CameraController->GetConfig().MaximumAllowedFOV);

			SetProjectionModeFOV(ProjectionMode, NewFOV);
			return true;
		}

		break;
	}

	return FEditorViewportClient::InputKey(EventArgs);
}

bool FDisplayClusterLightCardEditorViewportClient::InputWidgetDelta(FViewport* InViewport,
	EAxisList::Type CurrentAxis,
	FVector& Drag,
	FRotator& Rot,
	FVector& Scale)
{
	bool bHandled = false;

	if (FEditorViewportClient::InputWidgetDelta(InViewport, CurrentAxis, Drag, Rot, Scale))
	{
		bHandled = true;
	}
	else
	{
		if (CurrentAxis != EAxisList::Type::None && SelectedActors.Num())
		{
			switch (EditorWidget->GetWidgetMode())
			{
			case FDisplayClusterLightCardEditorWidget::EWidgetMode::WM_Translate:
				if (ProjectionMode == EDisplayClusterMeshProjectionType::UV)
				{
					MoveSelectedUVActors(InViewport, CurrentAxis);
				}
				else
				{
					MoveSelectedActors(InViewport, CurrentAxis);
				}
				break;

			case FDisplayClusterLightCardEditorWidget::EWidgetMode::WM_RotateZ:
				SpinSelectedActors(InViewport);
				break;

			case FDisplayClusterLightCardEditorWidget::EWidgetMode::WM_Scale:
				ScaleSelectedActors(InViewport, CurrentAxis);
				break;
			}

			bHandled = true;
		}

		InViewport->GetMousePos(LastWidgetMousePos);
	}

	return bHandled;
}

void FDisplayClusterLightCardEditorViewportClient::TrackingStarted(
	const FInputEventState& InInputState, 
	bool bIsDraggingWidget,
	bool bNudge)
{
	if ((InputMode == EInputMode::Idle) && bIsDraggingWidget && InInputState.IsLeftMouseButtonPressed() && SelectedActors.Num())
	{
		// Start dragging actor
		InputMode = EInputMode::DraggingActor;

		GEditor->DisableDeltaModification(true);
		{
			// The pivot location won't update properly and the actor will rotate / move around the original selection origin
			// so update it here to fix that.
			GUnrealEd->UpdatePivotLocationForSelection();
			GUnrealEd->SetPivotMovedIndependently(false);
		}

		BeginTransaction(LOCTEXT("MoveLightCard", "Move Light Card"));

		DesiredLookAtLocation.Reset();

		// Compute and store the delta between the widget's origin and the place the user clicked on it,
		// in order to factor it out when transforming the selected actor
		FIntPoint MousePos;
		InInputState.GetViewport()->GetMousePos(MousePos);

		FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
			InInputState.GetViewport(),
			GetScene(),
			EngineShowFlags)
			.SetRealtimeUpdate(IsRealtime()));

		FSceneView* View = CalcSceneView(&ViewFamily);

		FVector Origin;
		FVector Direction;
		ProjectionHelper->PixelToWorld(*View, MousePos, Origin, Direction);

		if (RenderViewportType != LVT_Perspective)
		{
			// For orthogonal projections, the drag widget offset should store the origin offset instead of the direction offset,
			// since the direction offset is always the same regardless of which pixel has been clicked while the origin moves
			DragWidgetOffset = Origin - FPlane::PointPlaneProject(CachedEditorWidgetWorldTransform.GetTranslation(), FPlane(Origin, Direction));
		}
		else
		{
			DragWidgetOffset = Direction - (CachedEditorWidgetWorldTransform.GetTranslation() - Origin).GetSafeNormal();
		}

		LastWidgetMousePos = MousePos;
	}

	FEditorViewportClient::TrackingStarted(InInputState, bIsDraggingWidget, bNudge);
}

void FDisplayClusterLightCardEditorViewportClient::TrackingStopped()
{
	if (InputMode == EInputMode::DraggingActor)
	{
		InputMode = EInputMode::Idle;

		for (const FDisplayClusterWeakStageActorPtr& ActorProxy : SelectedActors)
		{
			const FActorProxy* FoundProxy = ActorProxies.FindByKey(ActorProxy.AsActor());
			if (FoundProxy && FoundProxy->Proxy == ActorProxy)
			{
				if (IDisplayClusterStageActor* StageActor = Cast<IDisplayClusterStageActor>(FoundProxy->LevelInstance.Get()))
				{
					StageActor->UpdateEditorGizmos();
				}
			}
		}

		DragWidgetOffset = FVector::ZeroVector;
		EndTransaction();

		if (SelectedActors.Num())
		{
			// Update pre-capture mouse pos to avoid it resetting back to the starting position
			Viewport->SetPreCaptureMousePosFromSlateCursor();
			GEditor->DisableDeltaModification(false);
		}
	}

	FEditorViewportClient::TrackingStopped();
}

void FDisplayClusterLightCardEditorViewportClient::CreateDrawnLightCard(const TArray<FIntPoint>& MousePositions)
{
	if (MousePositions.Num() < 3 || !Viewport)
	{
		return;
	}

	//
	// Find direction of each mouse position
	//

	FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
		Viewport,
		GetScene(),
		EngineShowFlags
	).SetRealtimeUpdate(IsRealtime()));

	FSceneView* View = CalcSceneView(&ViewFamily);
	const FVector ViewOrigin = View->ViewLocation;
	const FVector OriginOffset = ProjectionMode == EDisplayClusterMeshProjectionType::UV ? ViewOrigin : FVector::ZeroVector;

	TArray<FVector> MouseWorldDirections; // directions from view point

	FVector ScreenOrigin;
	FVector ScreenDirection;
	MouseWorldDirections.AddUninitialized(MousePositions.Num());

	for (int32 PointIdx = 0; PointIdx < MousePositions.Num(); ++PointIdx)
	{
		ProjectionHelper->CalculateOriginAndDirectionFromPixelPosition(MousePositions[PointIdx], *View, OriginOffset, ScreenOrigin, ScreenDirection);
		MouseWorldDirections[PointIdx] = ScreenDirection;
	}

	//
	// Find light card direction based on center of bounds of world positions of mouse positions
	//

	FVector LightCardDirection; // direction from origin to where pixel points to

	{
		double MinX =  DBL_MAX;
		double MinY =  DBL_MAX;
		double MinZ =  DBL_MAX;

		double MaxX = -DBL_MAX;
		double MaxY = -DBL_MAX;
		double MaxZ = -DBL_MAX;

		for (const FVector& MouseWorldDirection : MouseWorldDirections)
		{
			FVector MouseWorldPosition;
			FVector MouseRelativeNormal;

			ProjectionHelper->CalculateNormalAndPositionInDirection(ViewOrigin, MouseWorldDirection, MouseWorldPosition, MouseRelativeNormal);

			MinX = FMath::Min(MinX, MouseWorldPosition.X);
			MinY = FMath::Min(MinY, MouseWorldPosition.Y);
			MinZ = FMath::Min(MinZ, MouseWorldPosition.Z);

			MaxX = FMath::Max(MaxX, MouseWorldPosition.X);
			MaxY = FMath::Max(MaxY, MouseWorldPosition.Y);
			MaxZ = FMath::Max(MaxZ, MouseWorldPosition.Z);
		}

		const FVector MouseWorldBoundsCenter(
			(MaxX + MinX) / 2,
			(MaxY + MinY) / 2,
			(MaxZ + MinZ) / 2
		);

		LightCardDirection = (MouseWorldBoundsCenter - ViewOrigin).GetSafeNormal();

		if (LightCardDirection == FVector::ZeroVector)
		{
			UE_LOGF(DisplayClusterLightCardEditorLog, Warning, "Could not determine a LightCardDirection for the given polygon");
			return;
		}
	}

	//
	// Find world position and normal of card, and its plane axes
	// 

	FVector LightCardLocation; // Wolrd location of light card
	FVector LightCardPlaneAxisX; // Normal vector at the found enveloping surface point
	FVector LightCardPlaneAxisY;
	FVector LightCardPlaneAxisZ;

	{
		FVector LightCardRelativeNormal;

		ProjectionHelper->CalculateNormalAndPositionInDirection(ViewOrigin, LightCardDirection, LightCardLocation, LightCardRelativeNormal);

		const FMatrix RotEffector = FRotationMatrix::MakeFromX(-LightCardRelativeNormal);
		const FMatrix RotArm = ProjectionMode != EDisplayClusterMeshProjectionType::UV ? FRotationMatrix::MakeFromX(LightCardDirection) : FMatrix::Identity;
		const FRotator TotalRotator = (RotEffector * RotArm).Rotator();

		LightCardPlaneAxisX = TotalRotator.RotateVector(-FVector::XAxisVector); // Same as Normal
		LightCardPlaneAxisY = TotalRotator.RotateVector(-FVector::YAxisVector);
		LightCardPlaneAxisZ = TotalRotator.RotateVector( FVector::ZAxisVector);
	}

	//
	// Project mouse positions to light card plane.
	//

	TArray<FVector3d> ProjectedMousePositions;
	ProjectedMousePositions.Reserve(MouseWorldDirections.Num());
	const FPlane LightCardPlane = FPlane(LightCardLocation, LightCardPlaneAxisX);

	for (int32 PointIdx = 0; PointIdx < MouseWorldDirections.Num(); ++PointIdx)
	{
		// skip the point if the intersection doesn't exist, which happens when the normal and the 
		// ray direction are perpendicular (nearly zero dot product)
		if (FMath::IsNearlyZero(FVector::DotProduct(LightCardPlaneAxisX, MouseWorldDirections[PointIdx])))
		{
			continue;
		}

		ProjectedMousePositions.Add(FMath::RayPlaneIntersection(ViewOrigin, MouseWorldDirections[PointIdx], LightCardPlane));
	}

	// Convert projected mouse positions in world space to light card plane space (2d)

	TArray<FVector2d> PointsInLightCardPlane;
	PointsInLightCardPlane.Reserve(ProjectedMousePositions.Num());

	for (const FVector& ProjectedMousePosition: ProjectedMousePositions)
	{
		const FVector ProjMousePosLocal = ProjectedMousePosition - LightCardLocation; 

		PointsInLightCardPlane.Add(FVector2D(
			-FVector::DotProduct(ProjMousePosLocal, LightCardPlaneAxisY),
			 FVector::DotProduct(ProjMousePosLocal, LightCardPlaneAxisZ))
		);
	}

	// Find the spin that minimizes the area. 
	// For now, simply a fixed size linear search but a better optimization algorithm could be used.

	double Spin = 0;
	double LightCardWidth = 0;
	double LightCardHeight = 0;

	{
		double Area = DBL_MAX;
		double constexpr SpinStepSize = 15;

		for (double SpinTest = 0; SpinTest < 180; SpinTest += SpinStepSize)
		{
			double LightCardWidthTest = 0;
			double LightCardHeightTest = 0;

			const FRotator Rotator(0, -SpinTest, 0); // using yaw for spin for convenience

			for (const FVector2d& Point : PointsInLightCardPlane)
			{
				FVector RotatedPoint = Rotator.RotateVector(FVector(Point.X, Point.Y, 0));

				LightCardWidthTest = FMath::Max(LightCardWidthTest, 2 * abs(RotatedPoint.X));
				LightCardHeightTest = FMath::Max(LightCardHeightTest, 2 * abs(RotatedPoint.Y));
			}

			const double AreaTest = LightCardWidthTest * LightCardHeightTest;

			if (AreaTest < Area)
			{
				Spin = SpinTest;
				Area = AreaTest;
				LightCardWidth = LightCardWidthTest;
				LightCardHeight = LightCardHeightTest;
			}
		}
	}

	// Update the points with the rotated ones, since we're going to spin the card.
	{
		const FRotator Rotator(0, -Spin, 0);

		for (FVector2d& Point : PointsInLightCardPlane)
		{
			FVector RotatedPoint = Rotator.RotateVector(FVector(Point.X, Point.Y, 0));

			Point.X = RotatedPoint.X;
			Point.Y = RotatedPoint.Y;
		}
	}

	if (FMath::IsNearlyZero(LightCardWidth) || FMath::IsNearlyZero(LightCardHeight))
	{
		UE_LOGF(DisplayClusterLightCardEditorLog, Warning, "Could not create new light card because one or more of its dimensions was zero.");
		return;
	}

	// Create lightcard

	if (!LightCardEditorPtr.IsValid())
	{
		UE_LOGF(DisplayClusterLightCardEditorLog, Warning, "Could not create new light card because LightCardEditorPtr was not valid.");
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("AddNewLightCard", "Add New Light Card"));

	ADisplayClusterLightCardActor* LightCard = LightCardEditorPtr.Pin()->SpawnActorAs<ADisplayClusterLightCardActor>(TEXT("LightCard"));
	ProjectionHelper->VerifyAndFixActorOrigin(LightCard);

	if (!LightCard)
	{
		UE_LOGF(DisplayClusterLightCardEditorLog, Warning, "Could not create new light card because AddNewLightCard failed.");
		return;
	}

	LightCard->Spin = Spin;

	// Assign polygon mask

	LightCard->Mask = EDisplayClusterLightCardMask::Polygon;

	LightCard->Polygon.Empty(PointsInLightCardPlane.Num());

	for (const FVector2d& PlanePoint : PointsInLightCardPlane)
	{
		LightCard->Polygon.Add(FVector2D(
			PlanePoint.X / LightCardWidth  + 0.5,
			PlanePoint.Y / LightCardHeight + 0.5
		));
	}

	LightCard->UpdatePolygonTexture();
	LightCard->UpdateLightCardMaterialInstance();

	// Update scale to match the desired size
	{
		// The default card plane is a 100x100 square.
		constexpr double LightCardPlaneWidth = 100;
		constexpr double LightCardPlaneHeight = 100;

		LightCard->Scale.X = LightCardWidth / LightCardPlaneWidth;
		LightCard->Scale.Y = LightCardHeight / LightCardPlaneHeight;
	}

	// Update position
	if (ProjectionMode == EDisplayClusterMeshProjectionType::UV)
	{
		const float UVProjectionPlaneSize = ADisplayClusterLightCardActor::UVPlaneDefaultSize;

		const FVector DesiredLocation = LightCardLocation - ViewOrigin;
		const FVector2D DesiredUVLocation = FVector2D(DesiredLocation.Y / UVProjectionPlaneSize + 0.5f, 0.5f - DesiredLocation.Z / UVProjectionPlaneSize);

		LightCard->UVCoordinates = DesiredUVLocation;
	}
	else
	{
		const FDisplayClusterLightCardEditorHelper::FSphericalCoordinates LightCardCoords(LightCardLocation - ViewOrigin);
		MoveActorTo(LightCard, LightCardLocation - ViewOrigin);
	}
}

double FDisplayClusterLightCardEditorViewportClient::CalculateFinalActorDistance(double FlushDistance, double DesiredOffsetFromFlush) const
{
	double Distance = FMath::Min(FlushDistance, RootActorBoundingRadius) + DesiredOffsetFromFlush;

	return FMath::Max(Distance, 0);
}

const FDisplayClusterLightCardEditorViewportClient::FActorProxy* FDisplayClusterLightCardEditorViewportClient::FindActorProxyFromLevelInstance(
	AActor* InLevelInstance) const
{
	if (InLevelInstance == nullptr)
	{
		return nullptr;
	}
	
	return ActorProxies.FindByPredicate([InLevelInstance](const FActorProxy& Proxy)
	{
		return Proxy.LevelInstance.AsActor() == InLevelInstance;
	});
}

const FDisplayClusterLightCardEditorViewportClient::FActorProxy* FDisplayClusterLightCardEditorViewportClient::FindActorProxyFromActor(AActor* InActor) const
{
	if (InActor == nullptr)
	{
		return nullptr;
	}
	
	return ActorProxies.FindByPredicate([InActor](const FActorProxy& Proxy)
	{
		return Proxy.LevelInstance.AsActor() == InActor || Proxy.Proxy.AsActor() == InActor;
	});
}

AActor* FDisplayClusterLightCardEditorViewportClient::CreateStageActorProxy(AActor* InLevelInstance)
{
	if (!IsValid(InLevelInstance))
	{
		// Can happen if the level actor was destroyed the tick prior to proxy creation.
		return nullptr;
	}
	
	const FTransform RALevelTransformNoScale(RootActorLevelInstance->GetActorRotation(), RootActorLevelInstance->GetActorLocation(), FVector::OneVector);

	const UWorld* PreviewWorld = PreviewScene->GetWorld();
	check(PreviewWorld);
	
	// Create proxy
	FObjectDuplicationParameters DupeActorParameters(InLevelInstance, PreviewWorld->GetCurrentLevel());
	DupeActorParameters.FlagMask = RF_AllFlags & ~(RF_ArchetypeObject | RF_Transactional);
	DupeActorParameters.PortFlags = PPF_DuplicateVerbatim;

	AActor* ActorProxy = CastChecked<AActor>(StaticDuplicateObjectEx(DupeActorParameters));
	ActorProxy->SetFlags(RF_Transient); // This signals to the stage actor it is a proxy
	PreviewWorld->GetCurrentLevel()->AddLoadedActor(ActorProxy);

	const FTransform LCLevelRelativeToRALevel = ActorProxy->GetTransform().GetRelativeTransform(RALevelTransformNoScale);
	ActorProxy->SetActorTransform(LCLevelRelativeToRALevel);
	
	if (ADisplayClusterLightCardActor* LightCard = Cast<ADisplayClusterLightCardActor>(InLevelInstance))
	{
		ADisplayClusterLightCardActor* LightCardProxy = CastChecked<ADisplayClusterLightCardActor>(ActorProxy);
		LightCardProxy->PolygonMask = LightCard->PolygonMask;

		// Change mesh to proxy mesh with more vertices
		if (const UStaticMesh* LightCardMesh = LightCardProxy->GetStaticMesh())
		{
			// Only change the mesh if we are using the default one.

			const FString DefaultPlanePath = TEXT("/nDisplay/LightCard/SM_LightCardPlane.SM_LightCardPlane");
			const UStaticMesh* DefaultPlane = Cast<UStaticMesh>(FSoftObjectPath(DefaultPlanePath).TryLoad());

			if (DefaultPlane == LightCardMesh)
			{
				const FString LightCardPlanePath = TEXT("/nDisplay/LightCard/SM_LightCardPlaneSubdivided.SM_LightCardPlaneSubdivided");
				if (UStaticMesh* LightCardPlaneMesh = Cast<UStaticMesh>(FSoftObjectPath(LightCardPlanePath).TryLoad()))
				{
					LightCardProxy->SetStaticMesh(LightCardPlaneMesh);
				}
			}
		}
	}
	else
	{
		// Misc actor proxies
		TArray<UBillboardComponent*> BillboardComponents;
		ActorProxy->GetComponents<UBillboardComponent>(BillboardComponents);

		FScopeLock Lock(&SpriteComponentCS);
		BillboardComponentProxies.Append(BillboardComponents);
	}

	// Label widgets
	{
		TArray<UDisplayClusterLabelComponent*> LabelComponents;
		ActorProxy->GetComponents<UDisplayClusterLabelComponent>(LabelComponents);

		FScopeLock Lock(&SpriteComponentCS);
		
		for (UDisplayClusterLabelComponent* LabelComponent : LabelComponents)
		{
			UWidgetComponent* WidgetComponent = LabelComponent->GetWidgetComponent();
			WidgetComponent->SetTickWhenOffscreen(true);
			WidgetComponentProxies.Add(WidgetComponent);
		}
	}
	
	FActorProxy ActorProxyStruct(InLevelInstance, ActorProxy);
	ActorProxies.Add(ActorProxyStruct);
	ProjectionHelper->VerifyAndFixActorOrigin(InLevelInstance);
	ProjectionHelper->VerifyAndFixActorOrigin(ActorProxy);

	UpdateProxyTransforms(ActorProxyStruct);

	return ActorProxy;
}

void FDisplayClusterLightCardEditorViewportClient::UpdateProxyTransforms(const FActorProxy& InActorProxy)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FDisplayClusterLightCardEditorViewportClient::UpdateProxyTransforms"), STAT_UpdateProxyTransforms, STATGROUP_NDisplayLightCardEditor);
	
	if (RootActorLevelInstance.IsValid() && InActorProxy.LevelInstance.IsValid() && InActorProxy.Proxy.IsValid())
	{
		const FTransform RALevelTransformNoScale(RootActorLevelInstance->GetActorRotation(), RootActorLevelInstance->GetActorLocation(), FVector::OneVector);
		const FTransform LCLevelRelativeToRALevel = InActorProxy.LevelInstance.AsActorChecked()->GetTransform().GetRelativeTransform(RALevelTransformNoScale);
		InActorProxy.Proxy.AsActorChecked()->SetActorTransform(LCLevelRelativeToRALevel);
				
		// When dealing with light card actors the transform scale can be set separately from the stage actor 2d scale. When updating the
		// transform scale on the level instance it won't impact the positional params so the proxy transform scale won't update unless we manually set it
		InActorProxy.Proxy.AsActorChecked()->SetActorScale3D(InActorProxy.LevelInstance.AsActorChecked()->GetActorScale3D());
				
		// Need to update these manually or the proxy's position will be out of sync next update
		InActorProxy.Proxy->SetPositionalParams(InActorProxy.LevelInstance->GetPositionalParams());

		// Update UV -- only valid for UV light cards
		InActorProxy.Proxy->SetUVCoordinates(InActorProxy.LevelInstance->GetUVCoordinates());
	}
}

void FDisplayClusterLightCardEditorViewportClient::OnSequencerTimeChanged(TWeakPtr<ISequencer> InSequencer)
{
	UpdateProxyTransforms();
}

void FDisplayClusterLightCardEditorViewportClient::ProcessClick(FSceneView& View, HHitProxy* HitProxy, FKey Key, EInputEvent Event, uint32 HitX, uint32 HitY)
{
	// Don't select light cards while drawing a new light card
	if (InputMode == EInputMode::DrawingLightCard)
	{
		FEditorViewportClient::ProcessClick(View, HitProxy, Key, Event, HitX, HitY);
		return;
	}

	const bool bIsCtrlKeyDown = Viewport->KeyState(EKeys::LeftControl) || Viewport->KeyState(EKeys::RightControl);
	const bool bMultiSelect = Key == EKeys::LeftMouseButton && bIsCtrlKeyDown;
	const bool bIsRightClickSelection = Key == EKeys::RightMouseButton && !bIsCtrlKeyDown && !Viewport->KeyState(EKeys::LeftMouseButton);

	if (HitProxy)
	{
		const HHitProxy* HitProxyTest = Viewport->GetHitProxy(HitX, HitY);
		if (!ensure(HitProxyTest == HitProxy))
		{
			// Speculative work around for invalid hit proxy. The HitProxy passed to ProcessClick may have been deleted,
			// but since we're dealing with a raw ptr we can't check that. This is so we can possibly hit it and debug.
			return;
		}
		
		if (HitProxy->IsA(HActor::StaticGetType()))
		{
			HActor* ActorHitProxy = static_cast<HActor*>(HitProxy);

			// Only perform the ray trace when not in UV mode, as it doesn't make sense to ray trace against UV space
			if (ActorHitProxy->Actor == RootActorProxy.Get() && ProjectionMode != EDisplayClusterMeshProjectionType::UV)
			{
				if (ActorHitProxy->PrimComponent && ActorHitProxy->PrimComponent->IsA<UStaticMeshComponent>())
				{
					AActor* TracedActor = TraceScreenForActor(View, HitX, HitY);
					SelectActor(TracedActor, bMultiSelect);
				}
			}
			else if (ActorProxies.Contains(ActorHitProxy->Actor) && UE::DisplayClusterLightCardEditorUtils::IsProxySelectable(ActorHitProxy->Actor))
			{
				SelectActor(ActorHitProxy->Actor, bMultiSelect);
			}
			else if (!bMultiSelect)
			{
				// Unless a right click is being performed, clear the selection if no geometry was clicked
				if (!bIsRightClickSelection)
				{
					SelectActor(nullptr);
				}
			}
		}
	}
	else
	{
		// Unless a right click is being performed, clear the selection if no geometry was clicked
		if (!bIsRightClickSelection)
		{
			SelectActor(nullptr);
		}
	}
	
	PropagateActorSelection();

	if (bIsRightClickSelection)
	{
		LightCardEditorViewportPtr.Pin()->SummonContextMenu();
	}

	FEditorViewportClient::ProcessClick(View, HitProxy, Key, Event, HitX, HitY);
}

void FDisplayClusterLightCardEditorViewportClient::BeginCameraMovement(bool bHasMovement)
{
	// A little bit of hackery here. FEditorViewportClient doesn't provide many ways to change how camera movement happens (e.g. what keys or mouse movements do what),
	// but it does call BeginCameraMovement before it actually applies any of the movement stored in CameraUserImpulseData, so we can perform our own camera movement 
	// impulses here. For most projection modes, we simply zero out any camera translation impulses since the user isn't generally allowed to move the camera away from
	// the view point. The UV projection mode is the exception, where the user is allowed to pan the camera around
	if (ProjectionMode == EDisplayClusterMeshProjectionType::UV)
	{
		bool bIsLeftRightKeyPressed = false;
		bool bIsForwardBackKeyPressed = false;

		for (uint32 Index = 0; Index < static_cast<uint8>(EMultipleKeyBindingIndex::NumChords); ++Index)
		{
			EMultipleKeyBindingIndex ChordIndex = static_cast<EMultipleKeyBindingIndex>(Index);
			bIsForwardBackKeyPressed |= Viewport->KeyState(FViewportNavigationCommands::Get().Forward->GetActiveChord(ChordIndex)->Key);
			bIsForwardBackKeyPressed |= Viewport->KeyState(FViewportNavigationCommands::Get().Backward->GetActiveChord(ChordIndex)->Key);

			bIsLeftRightKeyPressed |= Viewport->KeyState(FViewportNavigationCommands::Get().Left->GetActiveChord(ChordIndex)->Key);
			bIsLeftRightKeyPressed |= Viewport->KeyState(FViewportNavigationCommands::Get().Right->GetActiveChord(ChordIndex)->Key);
		}

		bIsForwardBackKeyPressed |= Viewport->KeyState(EKeys::Up);
		bIsForwardBackKeyPressed |= Viewport->KeyState(EKeys::Down);
		bIsLeftRightKeyPressed |= Viewport->KeyState(EKeys::Left);
		bIsLeftRightKeyPressed |= Viewport->KeyState(EKeys::Right);

		// The editor viewport client is hardcoded to move left/right and forward/back using WSAD and arrows, but for UV, we want the arrow keys to 
		// control up/down instead of forward/back. If the up/down keys are pressed, reroute the forward/back impulses to the up/down impulse data
		if (bIsForwardBackKeyPressed)
		{
			CameraUserImpulseData->MoveWorldUpDownImpulse = CameraUserImpulseData->MoveForwardBackwardImpulse;
			CameraUserImpulseData->MoveForwardBackwardImpulse = 0.0f;
		}
		else if (!bIsForwardBackKeyPressed && !bIsLeftRightKeyPressed)
		{
			// Zero out all camera movement if WASD or arrow keys are not pressed, to keep the control scheme cleaner
			CameraUserImpulseData->MoveForwardBackwardImpulse = 0.0f;
			CameraUserImpulseData->MoveRightLeftImpulse = 0.0f;
			CameraUserImpulseData->MoveWorldUpDownImpulse = 0.0f;
			CameraUserImpulseData->MoveLocalUpDownImpulse = 0.0f;
		}
	}
	else
	{
		// Always zero out the camera's movement impulses so that the user cannot move the view using any keyboard chords
		CameraUserImpulseData->MoveForwardBackwardImpulse = 0.0f;
		CameraUserImpulseData->MoveRightLeftImpulse = 0.0f;
		CameraUserImpulseData->MoveWorldUpDownImpulse = 0.0f;
		CameraUserImpulseData->MoveLocalUpDownImpulse = 0.0f;
	}
}

EMouseCursor::Type FDisplayClusterLightCardEditorViewportClient::GetCursor(FViewport* InViewport, int32 X, int32 Y)
{
	EMouseCursor::Type MouseCursor = EMouseCursor::Default;

	if (RequiredCursorVisibiltyAndAppearance.bOverrideAppearance &&
		RequiredCursorVisibiltyAndAppearance.bHardwareCursorVisible)
	{
		 MouseCursor = RequiredCursorVisibiltyAndAppearance.RequiredCursor;
	}
	else if (!RequiredCursorVisibiltyAndAppearance.bHardwareCursorVisible)
	{
		MouseCursor = EMouseCursor::None;
	}
	else if (InViewport->IsCursorVisible() && !bWidgetAxisControlledByDrag)
	{
		EditorWidget->SetHighlightedAxis(EAxisList::Type::None);

		HHitProxy* HitProxy = InViewport->GetHitProxy(X,Y);
		if (HitProxy)
		{
			TrySetShouldCheckHitProxy();

			if (HitProxy->IsA(HActor::StaticGetType()))
			{
				HActor* ActorHitProxy = static_cast<HActor*>(HitProxy);
				if (ActorHitProxy->Actor == RootActorProxy.Get())
				{
					if (ActorHitProxy->PrimComponent && ActorHitProxy->PrimComponent->IsA<UStaticMeshComponent>())
					{
						FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
							InViewport,
							GetScene(),
							EngineShowFlags)
							.SetRealtimeUpdate(IsRealtime()));
						FSceneView* View = CalcSceneView(&ViewFamily);

						if (const AActor* TracedActor = TraceScreenForActor(*View, X, Y))
						{
							MouseCursor = EMouseCursor::Crosshairs;
						}
					}
				}
				else if (ActorProxies.Contains(ActorHitProxy->Actor))
				{
					MouseCursor = EMouseCursor::Crosshairs;
				}
			}
			else if (HitProxy->IsA(HWidgetAxis::StaticGetType()))
			{
				HWidgetAxis* AxisHitProxy = static_cast<HWidgetAxis*>(HitProxy);
				
				MouseCursor = AxisHitProxy->GetMouseCursor();
				EditorWidget->SetHighlightedAxis(AxisHitProxy->Axis);
			}
		}
	}

	CachedMouseX = X;
	CachedMouseY = Y;

	return MouseCursor;
}

void FDisplayClusterLightCardEditorViewportClient::DestroyDropPreviewActors()
{
	if (HasDropPreviewActors())
	{
		for (auto ActorIt = DropPreviewLightCards.CreateConstIterator(); ActorIt; ++ActorIt)
		{
			AActor* PreviewActor = (*ActorIt).AsActor();
			if (PreviewActor)
			{
				IDisplayClusterScenePreview::Get().RemoveActorFromRenderer(PreviewRendererId, PreviewActor);
				GetWorld()->DestroyActor(PreviewActor);
			}
		}
		DropPreviewLightCards.Empty();
	}
}

bool FDisplayClusterLightCardEditorViewportClient::UpdateDropPreviewActors(int32 MouseX, int32 MouseY,
                                                                           const TArray<UObject*>& DroppedObjects, bool& bOutDroppedObjectsVisible, UActorFactory* FactoryToUse)
{
	bOutDroppedObjectsVisible = false;
	if(!HasDropPreviewActors())
	{
		return false;
	}

	bNeedsRedraw = true;

	const FIntPoint NewMousePos { MouseX, MouseY };

	for (const FDisplayClusterWeakStageActorPtr& Actor : DropPreviewLightCards)
	{
		if (Actor.IsValid())
		{
			ProjectionHelper->VerifyAndFixActorOrigin(Actor);
		}
	}
	
	MoveActorsToPixel(NewMousePos, DropPreviewLightCards);
	
	return true;
}

bool FDisplayClusterLightCardEditorViewportClient::DropObjectsAtCoordinates(int32 MouseX, int32 MouseY,
                                                                            const TArray<UObject*>& DroppedObjects, TArray<AActor*>& OutNewActors, bool bOnlyDropOnTarget,
                                                                            bool bCreateDropPreview, bool bSelectActors, UActorFactory* FactoryToUse)
{
	if (!LightCardEditorPtr.IsValid() || LightCardEditorPtr.Pin()->GetActiveRootActor() == nullptr)
	{
		return false;
	}
	
	UWorld* PreviewWorld = PreviewScene->GetWorld();
	check(PreviewWorld);
	
	DestroyDropPreviewActors();
	
	Viewport->InvalidateHitProxy();

	bool bSuccess = false;

	if (bSelectActors)
	{
		SelectedActors.Empty();
	}
	
	TArray<FDisplayClusterWeakStageActorPtr> CreatedLightCards;
	CreatedLightCards.Reserve(DroppedObjects.Num());

	SelectActor(nullptr);
	PropagateActorSelection();
	
	for (UObject* DroppedObject : DroppedObjects)
	{
		if (const UDisplayClusterLightCardTemplate* Template = Cast<UDisplayClusterLightCardTemplate>(DroppedObject))
		{
			ADisplayClusterLightCardActor* LightCardActor =
				CastChecked<ADisplayClusterLightCardActor>(LightCardEditorPtr.Pin()->SpawnActor(Template,
					bCreateDropPreview ? PreviewWorld->GetCurrentLevel() : nullptr, bCreateDropPreview));

			ProjectionHelper->VerifyAndFixActorOrigin(LightCardActor);
			
			CreatedLightCards.Add(LightCardActor);
			
			if (bCreateDropPreview)
			{
				DropPreviewLightCards.Add(LightCardActor);
				
				LightCardActor->SetFlags(RF_Transient);
				IDisplayClusterScenePreview::Get().AddActorToRenderer(PreviewRendererId, LightCardActor);
			}
			else
			{
				LightCardActor->UpdatePolygonTexture();
				LightCardActor->UpdateLightCardMaterialInstance();
				if (bSelectActors)
				{
					GetOnNextSceneRefresh().AddLambda([this, LightCardActor]()
					{
						// Select on next refresh so the persistent level and corresponding proxy has spawned.
						SelectActor(LightCardActor, true);
						PropagateActorSelection();
					});
				}
			}
			
			bSuccess = true;
		}
	}

	const FIntPoint NewMousePos { MouseX, MouseY };
	MoveActorsToPixel(NewMousePos, CreatedLightCards);

	return bSuccess;
}

void FDisplayClusterLightCardEditorViewportClient::UpdatePreviewActor(ADisplayClusterRootActor* RootActor, bool bForce,
                                                                      EDisplayClusterLightCardEditorProxyType ProxyType,
                                                                      AActor* StageActor)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FDisplayClusterLightCardEditorViewportClient::UpdatePreviewActor"), STAT_UpdatePreviewActor, STATGROUP_NDisplayLightCardEditor);
	
	check(IsInGameThread());

	if (!StageActor && ((!bForce && RootActor == RootActorLevelInstance.GetEvenIfUnreachable()) ||
		(ProxyTypesRefreshing.Contains(ProxyType) || ProxyTypesRefreshing.Contains(EDisplayClusterLightCardEditorProxyType::All))))
	{
		return;
	}

	if (StageActor && ActorsRefreshing.Contains(StageActor))
	{
		return;
	}

	if (StageActor)
	{
		ActorsRefreshing.Add(StageActor);
	}
	else
	{
		ProxyTypesRefreshing.Add(ProxyType);
	}
	
	auto Finalize = [this, ProxyType, StageActor]()
	{
		Viewport->InvalidateHitProxy();
		TrySetShouldCheckHitProxy();

		if (!StageActor)
		{
			ProxyTypesRefreshing.Remove(ProxyType);
		}
		
		OnNextSceneRefreshDelegate.Broadcast();
		OnNextSceneRefreshDelegate.Clear();
	};
	
	if (RootActor == nullptr)
	{
		DestroyProxies(ProxyType);
		RootActorLevelInstance.Reset();
		
		Finalize();
	}
	else
	{
		const UWorld* PreviewWorld = PreviewScene->GetWorld();
		check(PreviewWorld);

		const TWeakObjectPtr<ADisplayClusterRootActor> RootActorPtr (RootActor);
		const TWeakPtr<FDisplayClusterLightCardEditorViewportClient> WeakPtrThis = SharedThis(this);

		TWeakObjectPtr<AActor> StageActorWeakPtr(StageActor);
		// Schedule for the next tick so CDO changes get propagated first in the event of config editor skeleton
		// regeneration & compiles. nDisplay's custom propagation may have issues if the archetype isn't correct.
		PreviewWorld->GetTimerManager().SetTimerForNextTick([=, this]()
		{
			DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FDisplayClusterLightCardEditorViewportClient::UpdatePreviewActorImpl"), STAT_UpdatePreviewActorImpl, STATGROUP_NDisplayLightCardEditor);
			
			if (!WeakPtrThis.IsValid())
			{
				return;
			}

			if (StageActorWeakPtr.IsValid())
			{
				ActorsRefreshing.Remove(StageActorWeakPtr.Get());
				DestroyProxy(StageActorWeakPtr.Get());
			}
			else
			{
				DestroyProxies(ProxyType);
			}
			
			if (!RootActorPtr.IsValid())
			{
				Finalize();
				return;
			}

			RootActorLevelInstance = RootActorPtr;

			TArray<TObjectPtr<AActor>> ActorProxiesCreated;
			
			if (ProxyType == EDisplayClusterLightCardEditorProxyType::All ||
				ProxyType == EDisplayClusterLightCardEditorProxyType::RootActor)
			{
				if (!RootActorProxy.IsValid())
				{
					FObjectDuplicationParameters DupeActorParameters(RootActorPtr.Get(), PreviewWorld->GetCurrentLevel());
					DupeActorParameters.FlagMask = RF_AllFlags & ~(RF_ArchetypeObject | RF_Transactional); // Keeps archetypes correct in config data.
					DupeActorParameters.PortFlags = PPF_DuplicateVerbatim;

					RootActorProxy = CastChecked<ADisplayClusterRootActor>(StaticDuplicateObjectEx(DupeActorParameters));
				}

				// Use root actor from scene to render
				if (IDisplayClusterViewportManager* ViewportManager = RootActorProxy.IsValid() ? RootActorProxy->GetOrCreateViewportManager() : nullptr)
				{
					// Using DCRA from the scene for rendering
					ViewportManager->GetConfiguration().SetRootActor(EDisplayClusterRootActorType::Scene | EDisplayClusterRootActorType::Configuration, RootActorPtr.Get());
				}

				PreviewWorld->GetCurrentLevel()->AddLoadedActor(RootActorProxy.Get());

				// Draw the geometry map for the proxy stage actor immediately to avoid a race condition where the geometry map could render
				// before the actor location changes propagate to its component proxies, resulting in an inaccurate proxy geometry map
				RootActorProxy->GetStageGeometryComponent()->Invalidate(true);

				// Spawned actor will take the transform values from the template, so manually reset them to zero here
				RootActorProxy->SetActorLocation(FVector::ZeroVector);
				RootActorProxy->SetActorRotation(FRotator::ZeroRotator);

				ProjectionOriginComponent = FindProjectionOriginComponent(RootActorProxy.Get());

				ProjectionHelper->SetLevelInstanceRootActor(*RootActorLevelInstance);
				ProjectionHelper->SetEditorViewportClient(AsShared());

				//@Todo: In the case when RootActorProxy is not recreated each time it is changed:
				// the DCRA  Copy cfg from RootActorLevelInstance to RootActorProxy every tick (do not recreate local DCRA)
				if (UDisplayClusterConfigurationData* ProxyConfig = RootActorProxy->GetConfigData())
				{
					// Disable lightcards so that it doesn't try to update the ones in the level instance world.
					ProxyConfig->StageSettings.Lightcard.bEnable = false;
				}

				FBox BoundingBox = RootActorProxy->GetComponentsBoundingBox();
				RootActorBoundingRadius = FMath::Max(BoundingBox.Min.Length(), BoundingBox.Max.Length());

				// Set translucency sort priority of root actor proxy primitive components so that actors that are flush with screens are rendered on top of them
				RootActorProxy->ForEachComponent<UPrimitiveComponent>(false, [](UPrimitiveComponent* InPrimitiveComponent)
				{
					InPrimitiveComponent->SetTranslucentSortPriority(-10);
				});
			}

			// Use custom settings for root actor.
			FDisplayClusterRootActorPropertyOverrides PropertyOverrides;
			{
				PropertyOverrides.bPreviewICVFXFrustums = false;
				PropertyOverrides.bEnablePreviewTechvis = false;
				PropertyOverrides.bPreviewEnableOverlayMaterial = false;
				PropertyOverrides.bFreezePreviewRender = false;

				PropertyOverrides.bPreviewEnablePostProcess = true;
				PropertyOverrides.bPreviewEnable = true;

				// Can render a DCRA preview without the HoldoutComposite plugin.
				PropertyOverrides.bPreviewEnableHoldoutComposite = false;

				PropertyOverrides.PreviewSetttingsSource = EDisplayClusterConfigurationRootActorPreviewSettingsSource::RootActor;
			}

			// Filter out any primitives hidden in game except screen components
			IDisplayClusterScenePreview::Get().SetRendererRootActor(PreviewRendererId, RootActorProxy.Get(), PropertyOverrides);

			if (ProxyType == EDisplayClusterLightCardEditorProxyType::All ||
				ProxyType == EDisplayClusterLightCardEditorProxyType::StageActor)
			{
				SelectActor(nullptr);

				if (LightCardEditorPtr.IsValid())
				{
					if (StageActorWeakPtr.IsValid())
					{
						if (AActor* ActorProxy = CreateStageActorProxy(StageActorWeakPtr.Get()))
						{
							ActorProxiesCreated.Add(ActorProxy);
						}
					}
					else
					{
						TArray<AActor*> ManagedActors = LightCardEditorPtr.Pin()->FindAllManagedActors();
						for (AActor* Actor : ManagedActors)
						{
							if (AActor* ActorProxy = CreateStageActorProxy(Actor))
							{
								ActorProxiesCreated.Add(ActorProxy);
							}
						}
					}
				}

				// Update the selected light card proxies to match the currently selected light cards in the light card list
				TArray<AActor*> CurrentlySelectedActors;
				if (LightCardEditorPtr.IsValid())
				{
					LightCardEditorPtr.Pin()->GetSelectedActors(CurrentlySelectedActors);
				}

				SelectActors(CurrentlySelectedActors);
			}

			// Make sure proxies are added to the renderer. Necessary for selections to render even if stage actors were not modified but root actor was updated
			for (const TObjectPtr<AActor>& ActorProxy : ActorProxiesCreated)
			{
				// Hack - CL 23230783 sets CCW meshes to hidden which causes problems with selection, so always add CCWs to the renderer.
				// Updating to include all stage actors now that CCRs are also supported.
				bool bIsStageActor = false;
				for (const UClass* Class = ActorProxy->GetClass(); Class && (UObject::StaticClass() != Class); Class = Class->GetSuperClass())
				{
					if (Class->ImplementsInterface(UDisplayClusterStageActor::StaticClass()))
					{
						bIsStageActor = true;
						break;
					}
				}
				
				IDisplayClusterScenePreview::Get().AddActorToRenderer(PreviewRendererId,
					ActorProxy, [this, ActorProxy, bIsStageActor](const UPrimitiveComponent* PrimitiveComponent)
				{
					// Always add the light card mesh component to the renderer's scene even if it is marked hidden in game, since UV light cards will purposefully
					// hide the light card mesh since it isn't supposed to exist in 3D space. The light card mesh will be appropriately filtered when the scene is
					// rendered based on the projection mode
					if (PrimitiveComponent->GetFName() == TEXT("LightCard") || bIsStageActor)
					{
						return true;
					}

					return !PrimitiveComponent->bHiddenInGame;
				});
			}

			Finalize();
		});
	}
}

void FDisplayClusterLightCardEditorViewportClient::UpdateProxyTransforms()
{
	if (RootActorLevelInstance.IsValid())
	{
		if (RootActorProxy.IsValid())
		{
			// Only update scale for the root actor.
			RootActorProxy->SetActorScale3D(RootActorLevelInstance->GetActorScale3D());
		}
		
		for (const FActorProxy& ActorProxy : ActorProxies)
		{
			UpdateProxyTransforms(ActorProxy);
		}
	}
}

void FDisplayClusterLightCardEditorViewportClient::UpdateProxyTransformFromLevelInstance(AActor* InLevelInstance)
{
	if (const FActorProxy* ActorProxy = FindActorProxyFromLevelInstance(InLevelInstance))
	{
		UpdateProxyTransforms(*ActorProxy);
	}
}

void FDisplayClusterLightCardEditorViewportClient::DestroyProxies(
	EDisplayClusterLightCardEditorProxyType ProxyType)
{
	FScopeLock Lock(&SpriteComponentCS);

	// Clear the primitives from the scene renderer based on the type of proxy that is being destroyed
	switch (ProxyType)
	{
	case EDisplayClusterLightCardEditorProxyType::RootActor:
		if (RootActorProxy.IsValid())
		{
			IDisplayClusterScenePreview::Get().RemoveActorFromRenderer(PreviewRendererId, RootActorProxy.Get());
		}
		break;

	case EDisplayClusterLightCardEditorProxyType::StageActor:
		for (const FActorProxy& ActorProxy : ActorProxies)
		{
			if (ActorProxy.Proxy.IsValid())
			{
				IDisplayClusterScenePreview::Get().RemoveActorFromRenderer(PreviewRendererId, ActorProxy.Proxy.AsActor());
			}
		}
		break;

	case EDisplayClusterLightCardEditorProxyType::All:
	default:
		IDisplayClusterScenePreview::Get().ClearRendererScene(PreviewRendererId);
		break;
	}

	UWorld* PreviewWorld = PreviewScene->GetWorld();
	check(PreviewWorld);
	
	if (ProxyType == EDisplayClusterLightCardEditorProxyType::All ||
		ProxyType == EDisplayClusterLightCardEditorProxyType::RootActor)
	{
		if (RootActorProxy.IsValid())
		{
			PreviewWorld->EditorDestroyActor(RootActorProxy.Get(), false);
			RootActorProxy.Reset();
		}
	}
	
	if (ProxyType == EDisplayClusterLightCardEditorProxyType::All ||
		ProxyType == EDisplayClusterLightCardEditorProxyType::StageActor)
	{
		for (const FActorProxy& ActorProxy : ActorProxies)
		{
			if (ActorProxy.Proxy.IsValid())
			{
				PreviewWorld->EditorDestroyActor(ActorProxy.Proxy.AsActor(), false);
			}
		}

		ActorProxies.Empty();
		BillboardComponentProxies.Empty();
	}
}

void FDisplayClusterLightCardEditorViewportClient::DestroyProxy(AActor* Actor)
{
	if (const FActorProxy* ActorProxy = FindActorProxyFromActor(Actor))
	{
		if (AActor* ActualProxy = ActorProxy->Proxy.AsActor())
		{
			ActorProxies.RemoveAll([ActualProxy](const FActorProxy& OtherProxy)
			{
				return OtherProxy == ActualProxy;
			});
		
			FScopeLock Lock(&SpriteComponentCS);
		
			UWorld* PreviewWorld = PreviewScene->GetWorld();
			check(PreviewWorld);
			IDisplayClusterScenePreview::Get().RemoveActorFromRenderer(PreviewRendererId, ActualProxy);
			PreviewWorld->EditorDestroyActor(ActualProxy, false);
			
			BillboardComponentProxies.RemoveAll([ActualProxy](const TWeakObjectPtr<UBillboardComponent>& BillboardComponent)
			{
				return !BillboardComponent.IsValid() || BillboardComponent->GetOwner() == ActualProxy;
			});

			WidgetComponentProxies.RemoveAll([ActualProxy](const TWeakObjectPtr<UWidgetComponent>& WidgetComponent)
			{
				return !WidgetComponent.IsValid() || WidgetComponent->GetOwner() == ActualProxy;
			});
		}
	}
}

void FDisplayClusterLightCardEditorViewportClient::SelectActors(const TArray<AActor*>& ActorsToSelect)
{
	SelectActor(nullptr);
	for (AActor* Actor : ActorsToSelect)
	{
		if (const FActorProxy* FoundProxy = ActorProxies.FindByKey(Actor))
		{
			if (FoundProxy->Proxy.IsValid())
			{
				SelectActor(FoundProxy->Proxy.AsActor(), true);
			}
		}
	}
}

bool FDisplayClusterLightCardEditorViewportClient::HasSelection() const
{
	return SelectedActors.Num() > 0;
}

void FDisplayClusterLightCardEditorViewportClient::SetEditorWidgetMode(FDisplayClusterLightCardEditorWidget::EWidgetMode InWidgetMode)
{
	// Force the coordinate system back to spherical if the widget mode is set to anything besides translate
	if (InWidgetMode != FDisplayClusterLightCardEditorWidget::EWidgetMode::WM_Translate)
	{
		EditorWidgetCoordinateSystem = FDisplayClusterLightCardEditorHelper::ECoordinateSystem::Spherical;
	}

	EditorWidget->SetWidgetMode(InWidgetMode);
}

void FDisplayClusterLightCardEditorViewportClient::SetProjectionMode(EDisplayClusterMeshProjectionType InProjectionMode, ELevelViewportType InViewportType)
{
	SaveProjectionCameraTransform();

	ProjectionMode = InProjectionMode;
	RenderViewportType = InViewportType;

	UDisplayClusterLightCardEditorSettings* Settings = GetMutableDefault<UDisplayClusterLightCardEditorSettings>();
	
	Settings->ProjectionMode = ProjectionMode;
	Settings->RenderViewportType = InViewportType;
	Settings->PostEditChange();
	Settings->SaveConfig();

	ProjectionHelper->SetProjectionMode(ProjectionMode);
	ProjectionHelper->SetIsOrthographic(InViewportType != LVT_Perspective);

	RestoreProjectionCameraTransform();

	const float WidgetScale = ProjectionMode == EDisplayClusterMeshProjectionType::Azimuthal ? 0.5f : 1.0f;
	EditorWidget->SetWidgetScale(WidgetScale);

	ProjectionOriginComponent = FindProjectionOriginComponent(RootActorProxy.Get());

	if (Viewport)
	{
		Viewport->InvalidateHitProxy();
	}

	TrySetShouldCheckHitProxy();
}

float FDisplayClusterLightCardEditorViewportClient::GetProjectionModeFOV(EDisplayClusterMeshProjectionType InProjectionMode) const
{
	int32 ProjectionModeIndex = (int32)InProjectionMode;
	if (ProjectionViewConfigurations.Num() > ProjectionModeIndex)
	{
		const bool bIsOrthographic = RenderViewportType != ELevelViewportType::LVT_Perspective;
		return bIsOrthographic
			? ProjectionViewConfigurations[ProjectionModeIndex].OrthographicFOV
			: ProjectionViewConfigurations[ProjectionModeIndex].PerspectiveFOV;
	}
	else
	{
		return ViewFOV;
	}
}

void FDisplayClusterLightCardEditorViewportClient::SetProjectionModeFOV(EDisplayClusterMeshProjectionType InProjectionMode, float NewFOV)
{
	int32 ProjectionModeIndex = (int32)InProjectionMode;
	if (ProjectionViewConfigurations.Num() > ProjectionModeIndex)
	{
		const bool bIsOrthographic = RenderViewportType != ELevelViewportType::LVT_Perspective;
		if (bIsOrthographic)
		{
			ProjectionViewConfigurations[ProjectionModeIndex].OrthographicFOV = NewFOV;
		}
		else
		{
			ProjectionViewConfigurations[ProjectionModeIndex].PerspectiveFOV = NewFOV;
		}
	}
	else
	{
		ViewFOV = NewFOV;
	}

	Viewport->InvalidateHitProxy();
	TrySetShouldCheckHitProxy();
}

void FDisplayClusterLightCardEditorViewportClient::ResetCamera(bool bLocationOnly)
{
	FVector Location = FVector::ZeroVector;
	if (ProjectionOriginComponent.IsValid())
	{
		Location = ProjectionOriginComponent->GetComponentLocation();
	}

	SetViewLocation(Location);

	if (bLocationOnly)
	{
		return;
	}
	
	SetProjectionMode(GetProjectionMode(), GetRenderViewportType());

	ResetProjectionViewConfigurations();
}

void FDisplayClusterLightCardEditorViewportClient::FrameSelection()
{
	TArray<FVector> SelectionPositions;
	FVector AveragePosition = FVector::ZeroVector;

	if (ProjectionMode == EDisplayClusterMeshProjectionType::UV)
	{
		FVector2D AverageUVCoords = FVector2D::ZeroVector;

		const float UVPlaneSize = ADisplayClusterLightCardActor::UVPlaneDefaultSize;
		const float UVPlaneDistance = ADisplayClusterLightCardActor::UVPlaneDefaultDistance;

		for (const FDisplayClusterWeakStageActorPtr& Actor : SelectedActors)
		{
			if (Actor.IsValid() && Actor->IsUVActor())
			{
				const FVector2D UVCoords = Actor->GetUVCoordinates();
				const FVector ProxyPosition = FVector(UVPlaneDistance, UVPlaneSize * (UVCoords.X - 0.5f), UVPlaneSize * (0.5f - UVCoords.Y));
				AverageUVCoords += Actor->GetUVCoordinates();
				SelectionPositions.Add(ProxyPosition);
			}
		}

		if (SelectionPositions.Num())
		{
			AverageUVCoords /= SelectionPositions.Num();
			AveragePosition = FVector(UVPlaneDistance, UVPlaneSize *(AverageUVCoords.X - 0.5f), UVPlaneSize * (0.5f - AverageUVCoords.Y));

			const FVector ViewLocation = FVector(0.0f, AveragePosition.Y, AveragePosition.Z);
			SetViewLocation(ViewLocation);
		}
	}
	else
	{
		for (const FDisplayClusterWeakStageActorPtr& Actor : SelectedActors)
		{
			if (Actor.IsValid() && !Actor->IsUVActor())
			{
				const FVector ActorPosition = Actor->GetStageActorTransform().GetLocation() - GetViewLocation();
				AveragePosition += ActorPosition;
				SelectionPositions.Add(ActorPosition);
			}
		}

		if (SelectionPositions.Num())
		{
			AveragePosition /= SelectionPositions.Num();

			SetViewRotation(AveragePosition.GetSafeNormal().ToOrientationRotator());
		}
	}

	if (SelectionPositions.Num() > 1)
	{
		// In the case where more than one actor is selected, the view's FOV may need to be increased so that all selected items are in view

		float MaxTheta = 0;
		if (RenderViewportType == ELevelViewportType::LVT_Perspective)
		{
			// For perspective projection, find the largest angle from the averaged position, and if the current FOV is smaller than that, set the FOV to be equal to that angle
			const FVector NormalizedAveragePos = AveragePosition.GetSafeNormal();
			for (const FVector& SelectionPosition : SelectionPositions)
			{
				float Theta = FMath::RadiansToDegrees(FMath::Acos(SelectionPosition.GetSafeNormal() | NormalizedAveragePos));
				MaxTheta = FMath::Clamp(MaxTheta, Theta, 90.0f);
			}
		}
		else
		{
			// For orthographic projection, rotate each selected actor position into view space, and compute the bounding box of all positions.
			// From this, compute the FOV needed to match those bounds (as per GetSceneViewInitOptions, the ortho projection size is equal
			// to 0.5 * tan(0.5 * FOV) * ViewportSize / DPI)
			const FMatrix ViewBasis = FRotationMatrix::MakeFromX(AveragePosition.GetSafeNormal());

			FBox FrameBounds(EForceInit::ForceInit);
			for (const FVector& SelectionPosition : SelectionPositions)
			{
				const FVector ViewPosition = ViewBasis.TransformVector(SelectionPosition);
				FrameBounds += ViewPosition;
			}

			const FVector2D FrameSize(FrameBounds.GetSize().Y, FrameBounds.GetSize().Z);
			const FVector2D ViewportSize = Viewport->GetSizeXY();
			const float DPIScale = GetDPIScale();

			const float ThetaX = FMath::Atan(2 * DPIScale * FrameSize.X / ViewportSize.X);
			const float ThetaY = FMath::Atan(2 * DPIScale * FrameSize.Y / ViewportSize.Y);
			MaxTheta = FMath::RadiansToDegrees(FMath::Max(ThetaX, ThetaY));
		}

		// Only change the FOV if the FOV needed to frame the selection is larger than the current FOV
		const float HalfFOV = 0.5f * GetProjectionModeFOV(ProjectionMode);
		if (MaxTheta > HalfFOV)
		{
			// Round up the new FOV to the nearest scroll increment, to give the frame a comfortable amount of extra space
			const float NewFOV = FMath::DivideAndRoundUp(2.0f * MaxTheta, FOVScrollIncrement) * FOVScrollIncrement;
			SetProjectionModeFOV(ProjectionMode, FMath::Clamp(NewFOV, CameraController->GetConfig().MinimumAllowedFOV, CameraController->GetConfig().MaximumAllowedFOV));
		}
	}
}

void FDisplayClusterLightCardEditorViewportClient::CycleCoordinateSystem()
{
	if (EditorWidgetCoordinateSystem == FDisplayClusterLightCardEditorHelper::ECoordinateSystem::Cartesian)
	{
		SetCoordinateSystem(FDisplayClusterLightCardEditorHelper::ECoordinateSystem::Spherical);
	}
	else
	{
		SetCoordinateSystem(FDisplayClusterLightCardEditorHelper::ECoordinateSystem::Cartesian);
	}
}

void FDisplayClusterLightCardEditorViewportClient::SetCoordinateSystem(FDisplayClusterLightCardEditorHelper::ECoordinateSystem NewCoordinateSystem)
{
	// The only widget mode that supports multiple coordinate systems at the moment is translation
	if (EditorWidget->GetWidgetMode() == FDisplayClusterLightCardEditorWidget::EWidgetMode::WM_Translate)
	{
		EditorWidgetCoordinateSystem = NewCoordinateSystem;
	}
}

void FDisplayClusterLightCardEditorViewportClient::MoveActorTo(const FDisplayClusterWeakStageActorPtr& Actor, const FDisplayClusterLightCardEditorHelper::FSphericalCoordinates& SphericalCoords) const
{
	ProjectionHelper->MoveActorsTo({ Actor }, SphericalCoords);
}

void FDisplayClusterLightCardEditorViewportClient::CenterActorInView(const FDisplayClusterWeakStageActorPtr& Actor)
{
	ProjectionHelper->VerifyAndFixActorOrigin(Actor);

	if (Actor->IsUVActor())
	{
		Actor->SetUVCoordinates(FVector2D(0.5, 0.5));
	}
	else
	{
		MoveActorTo(Actor, FDisplayClusterLightCardEditorHelper::FSphericalCoordinates(GetViewRotation().RotateVector(FVector::ForwardVector)));
	}

	// If this is a proxy light, propagate to its counterpart in the level.
	if (Actor->IsProxy())
	{
		PropagateActorTransform(Actor);
	}
}

void FDisplayClusterLightCardEditorViewportClient::MoveSelectedActorsToPixel(const FIntPoint& PixelPos)
{
	MoveActorsToPixel(PixelPos, SelectedActors);
}

void FDisplayClusterLightCardEditorViewportClient::MoveActorsToPixel(const FIntPoint& PixelPos, const TArray<FDisplayClusterWeakStageActorPtr>& InActors)
{
	FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
		Viewport,
		GetScene(),
		EngineShowFlags)
		.SetRealtimeUpdate(IsRealtime())
	);

	FSceneView* View = CalcSceneView(&ViewFamily);

	ProjectionHelper->MoveActorsToPixel(InActors, PixelPos, *View);

	// Update each light card with the delta coordinates; the flush constraint is applied by MoveActorTo, ensuring the light card is always flush to screens
	for (const FDisplayClusterWeakStageActorPtr& LightCard : InActors)
	{
		if (LightCard.IsValid() &&
			((LightCard->IsUVActor() && ProjectionMode == EDisplayClusterMeshProjectionType::UV) ||
			(!LightCard->IsUVActor() && ProjectionMode != EDisplayClusterMeshProjectionType::UV)))
		{
			PropagateActorTransform(LightCard);
		}
	}
}

void FDisplayClusterLightCardEditorViewportClient::BeginTransaction(const FText& Description)
{
	GEditor->BeginTransaction(Description);
}

void FDisplayClusterLightCardEditorViewportClient::EndTransaction()
{
	GEditor->EndTransaction();
}

void FDisplayClusterLightCardEditorViewportClient::GetScenePrimitiveComponents(TArray<UPrimitiveComponent*>& OutPrimitiveComponents)
{
	RootActorProxy->ForEachComponent<UPrimitiveComponent>(true, [&](UPrimitiveComponent* PrimitiveComponent)
	{
		OutPrimitiveComponents.Add(PrimitiveComponent);
	});
}

void FDisplayClusterLightCardEditorViewportClient::GetSceneViewInitOptions(FSceneViewInitOptions& OutViewInitOptions)
{
	const FViewportCameraTransform& ViewTransform = GetViewTransform();
	const FRotator& ViewRotation = ViewTransform.GetRotation();
	const FMatrix RotationMatrix = CalcViewRotationMatrix(ViewRotation);

	FSceneViewInitOptions ViewInitOptions;
	ProjectionHelper->GetSceneViewInitOptions(
		ViewInitOptions,
		GetProjectionModeFOV(ProjectionMode),
		Viewport->GetSizeXY(),
		ViewTransform.GetLocation(),
		ViewTransform.GetRotation(),
		GetDefault<ULevelEditorViewportSettings>()->AspectRatioAxisConstraint,
		GetNearClipPlane(),
		&RotationMatrix,
		GetDPIScale()
	);

	ViewInitOptions.SceneViewStateInterface = ViewState.GetReference();
	ViewInitOptions.ViewElementDrawer = this;

	ViewInitOptions.BackgroundColor = GetBackgroundColor();

	ViewInitOptions.EditorViewBitflag = (uint64)1 << ViewIndex, // send the bit for this view - each actor will check it's visibility bits against this

	ViewInitOptions.OverrideFarClippingPlaneDistance = GetFarClipPlaneOverride();
	ViewInitOptions.CursorPos = CurrentMousePos;

	OutViewInitOptions = ViewInitOptions;
}

void FDisplayClusterLightCardEditorViewportClient::GetNormalMapSceneViewInitOptions(FIntPoint NormalMapSize, float NormalMapFOV, const FVector& ViewDirection, FSceneViewInitOptions& OutViewInitOptions)
{
	FViewportCameraTransform& ViewTransform = GetViewTransform();

	OutViewInitOptions.ViewLocation = ProjectionOriginComponent.IsValid() ? ProjectionOriginComponent->GetComponentLocation() : FVector::ZeroVector;
	OutViewInitOptions.ViewRotation = ViewDirection.Rotation();
	OutViewInitOptions.ViewOrigin = OutViewInitOptions.ViewLocation;

	OutViewInitOptions.SetViewRectangle(FIntRect(0, 0, NormalMapSize.X, NormalMapSize.Y));

	AWorldSettings* WorldSettings = nullptr;
	if (GetScene() != nullptr && GetScene()->GetWorld() != nullptr)
	{
		WorldSettings = GetScene()->GetWorld()->GetWorldSettings();
	}

	if (WorldSettings != nullptr)
	{
		OutViewInitOptions.WorldToMetersScale = WorldSettings->WorldToMeters;
	}

	// Rotate view 90 degrees
	OutViewInitOptions.ViewRotationMatrix = FInverseRotationMatrix(OutViewInitOptions.ViewRotation) * FMatrix(
		FPlane(0, 0, 1, 0),
		FPlane(1, 0, 0, 0),
		FPlane(0, 1, 0, 0),
		FPlane(0, 0, 0, 1));

	const float MinZ = GetNearClipPlane();
	const float MaxZ = FMath::Max(RootActorBoundingRadius, MinZ);

	// Avoid zero ViewFOV's which cause divide by zero's in projection matrix
	const float MatrixFOV = FMath::Max(0.001f, NormalMapFOV) * (float)PI / 360.0f;

	const float XAxisMultiplier = 1.0f;
	const float YAxisMultiplier = 1.0f;

	OutViewInitOptions.ProjectionMatrix = FReversedZPerspectiveMatrix(
		MatrixFOV,
		MatrixFOV,
		XAxisMultiplier,
		YAxisMultiplier,
		MinZ,
		MaxZ);

	OutViewInitOptions.SceneViewStateInterface = ViewState.GetReference();
	OutViewInitOptions.ViewElementDrawer = this;

	OutViewInitOptions.EditorViewBitflag = (uint64)1 << ViewIndex, // send the bit for this view - each actor will check it's visibility bits against this

	OutViewInitOptions.FOV = NormalMapFOV;
	OutViewInitOptions.OverrideFarClippingPlaneDistance = GetFarClipPlaneOverride();
}

UDisplayClusterConfigurationViewport* FDisplayClusterLightCardEditorViewportClient::FindViewportForPrimitiveComponent(UPrimitiveComponent* PrimitiveComponent)
{
	if (RootActorProxy.IsValid())
	{
		const FString PrimitiveComponentName = PrimitiveComponent->GetName();
		UDisplayClusterConfigurationData* Config = RootActorProxy->GetConfigData();
		
		for (const TPair<FString, TObjectPtr<UDisplayClusterConfigurationClusterNode>>& NodePair : Config->Cluster->Nodes)
		{
			const UDisplayClusterConfigurationClusterNode* Node = NodePair.Value;
			for (const TPair<FString, TObjectPtr<UDisplayClusterConfigurationViewport>>& ViewportPair : Node->Viewports)
			{
				UDisplayClusterConfigurationViewport* CfgViewport = ViewportPair.Value;

				FString ComponentName;
				if (CfgViewport->ProjectionPolicy.Type.Equals(DisplayClusterProjectionStrings::projection::Mesh, ESearchCase::IgnoreCase)
					&& CfgViewport->ProjectionPolicy.Parameters.Contains(DisplayClusterProjectionStrings::cfg::mesh::Component))
				{
					ComponentName = CfgViewport->ProjectionPolicy.Parameters[DisplayClusterProjectionStrings::cfg::mesh::Component];
				}

				if (ComponentName == PrimitiveComponentName)
				{
					return CfgViewport;
				}
			}
		}
	}

	return nullptr;
}

USceneComponent* FDisplayClusterLightCardEditorViewportClient::FindProjectionOriginComponent(const ADisplayClusterRootActor* InRootActor) const
{
	if (!InRootActor)
	{
		return nullptr;
	}

	return InRootActor->GetCommonViewPoint();
}

bool FDisplayClusterLightCardEditorViewportClient::IsActorSelected(const AActor* Actor)
{
	return Actor->Implements<UDisplayClusterStageActor>() && SelectedActors.Contains(Actor);
}

void FDisplayClusterLightCardEditorViewportClient::SelectActor(AActor* Actor, bool bAddToSelection)
{
	TArray<AActor*> UpdatedActors;

	if (!bAddToSelection)
	{
		for (const FDisplayClusterWeakStageActorPtr& SelectedActor : SelectedActors)
		{
			if (SelectedActor.IsValid())
			{
				UpdatedActors.Add(SelectedActor.AsActor());
			}
		}

		SelectedActors.Empty();
		LastSelectedActor.Reset();
	}

	if (Actor)
	{
		SelectedActors.Add(Actor);
		UpdatedActors.Add(Actor);
		LastSelectedActor = Actor;
	}

	for (AActor* UpdatedActor : UpdatedActors)
	{
		UpdatedActor->PushSelectionToProxies();
	}
}

void FDisplayClusterLightCardEditorViewportClient::PropagateActorSelection()
{
	TArray<AActor*> SelectedLevelInstances;
	for (const FDisplayClusterWeakStageActorPtr& SelectedActor : SelectedActors)
	{
		if (const FActorProxy* FoundProxy = ActorProxies.FindByKey(SelectedActor.AsActor()))
		{
			if (FoundProxy->LevelInstance.IsValid())
			{
				SelectedLevelInstances.Add(FoundProxy->LevelInstance.AsActor());
			}
		}
	}

	LightCardEditorPtr.Pin()->SelectActors(SelectedLevelInstances);
}

void FDisplayClusterLightCardEditorViewportClient::PropagateActorTransform(const FDisplayClusterWeakStageActorPtr& ActorProxy)
{
	const FActorProxy* FoundProxy = ActorProxies.FindByKey(ActorProxy.AsActor());
	if (FoundProxy && FoundProxy->Proxy == ActorProxy && FoundProxy->LevelInstance.IsValid())
	{
		AActor* LevelInstance = FoundProxy->LevelInstance.AsActorChecked();
		LevelInstance->Modify();

		IDisplayClusterStageActor* LevelInstanceStageActor = Cast<IDisplayClusterStageActor>(LevelInstance);
		if (!ensure(LevelInstanceStageActor))
		{
			return;
		}

		IDisplayClusterStageActor::FPositionalPropertyArray ProxyPropertyPairs;
		ActorProxy->GetPositionalProperties(ProxyPropertyPairs);

		IDisplayClusterStageActor::FPositionalPropertyArray LevelInstancePropertyPairs;
		LevelInstanceStageActor->GetPositionalProperties(LevelInstancePropertyPairs);

		if (!ensure(ProxyPropertyPairs.Num() == LevelInstancePropertyPairs.Num()))
		{
			return;
		}
		
		// Set the level instance property value to our proxy property value.
		auto TryChangeProperty = [&](
			const IDisplayClusterStageActor::FPropertyPair& ProxyPropertyPair,
			const IDisplayClusterStageActor::FPropertyPair& LevelInstancePropertyPair,
			TArray<const FProperty*>& InOutChangedProperties
		) -> void
		{
			// Only change if values are different.
			FProperty* Property = ProxyPropertyPair.Value;
			if (!Property->Identical_InContainer(ProxyPropertyPair.Key, LevelInstancePropertyPair.Key))
			{
				FEditPropertyChain PropertyChain;
				PropertyChain.AddTail(Property);
				PropertyChain.SetActivePropertyNode(Property);

				const FName PositionalParamsPropertyName = LevelInstanceStageActor->GetPositionalPropertiesMemberName();
				if (!PositionalParamsPropertyName.IsNone())
				{
					if (FProperty* MemberProperty = LevelInstance->GetClass()->FindPropertyByName(PositionalParamsPropertyName))
					{
						PropertyChain.AddHead(MemberProperty);
						PropertyChain.SetActiveMemberPropertyNode(MemberProperty);
					}
				}

				// Required for sequencer to update property key frames
				FCoreUObjectDelegates::OnPreObjectPropertyChanged.Broadcast(LevelInstance, PropertyChain);
				
				Property->CopyCompleteValue_InContainer(LevelInstancePropertyPair.Key, ProxyPropertyPair.Key);

				bHasCalledPostEditChangeProperty = true;
				FPropertyChangedEvent PropertyChangedEvent(Property, EPropertyChangeType::ValueSet);
				FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(LevelInstance, PropertyChangedEvent);
				bHasCalledPostEditChangeProperty = false;
				
				InOutChangedProperties.Add(Property);
			}
		};
		
		// Here we count on the fact that the root actor proxy has zero loc/rot.
		// If that ever changes then the math below will need to be updated to use the 
		// relative loc/rot of the LC proxies wrt the root actor proxy.

		const FRotator RALevelRotation = RootActorLevelInstance.IsValid() ? RootActorLevelInstance->GetActorRotation() : FRotator::ZeroRotator;
		const FVector  RALevelLocation = RootActorLevelInstance.IsValid() ? RootActorLevelInstance->GetActorLocation() : FVector::ZeroVector;

		const FTransform RALevelTransformNoScale(RALevelRotation, RALevelLocation, FVector::OneVector);
		
		FTransform NewTransform = ActorProxy.AsActorChecked()->GetTransform() * RALevelTransformNoScale;

		FProperty* TransformProperty = LevelInstance->GetRootComponent() ?
				LevelInstance->GetRootComponent()->GetClass()->FindPropertyByName(USceneComponent::GetRelativeLocationPropertyName())
				: nullptr;

		const bool bTransformChanged = !NewTransform.Equals(LevelInstance->GetActorTransform(), 0.f);

		if (TransformProperty && bTransformChanged)
		{
			FEditPropertyChain PropertyChain;
			PropertyChain.AddTail(TransformProperty);
			PropertyChain.SetActivePropertyNode(TransformProperty);
			// Required for sequencer and CCWs.
			FCoreUObjectDelegates::OnPreObjectPropertyChanged.Broadcast(LevelInstance, PropertyChain);
		}
		
		LevelInstance->SetActorTransform(NewTransform);

		if (TransformProperty && bTransformChanged)
		{
			// Required for sequencer and CCWs.
			bHasCalledPostEditChangeProperty = true;
			FPropertyChangedEvent PropertyChangedEvent(TransformProperty, EPropertyChangeType::ValueSet);
			FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(LevelInstance, PropertyChangedEvent);
			bHasCalledPostEditChangeProperty = false;
		}
		
		TArray<const FProperty*> ChangedProperties;
		ChangedProperties.Reserve(ProxyPropertyPairs.Num());

		for (int32 PropertyIndex = 0; PropertyIndex < ProxyPropertyPairs.Num(); ++PropertyIndex)
		{
			TryChangeProperty(ProxyPropertyPairs[PropertyIndex], LevelInstancePropertyPairs[PropertyIndex], ChangedProperties);
		}
		
		// Snapshot the changed properties so multi-user can update while dragging.
		if (ChangedProperties.Num() > 0)
		{
			SnapshotTransactionBuffer(LevelInstance, MakeArrayView(ChangedProperties));
		}

		// Allows MU to receive the update in real-time.
		LevelInstance->PostEditMove(false);

		// Gizmo update can be expensive, so prevent doing it every frame while dragging
		if (InputMode != EInputMode::DraggingActor)
		{
			if (IDisplayClusterStageActor* StageActor = Cast<IDisplayClusterStageActor>(LevelInstance))
			{
				StageActor->UpdateEditorGizmos();
			}
		}
	}
}

void FDisplayClusterLightCardEditorViewportClient::MoveSelectedActors(FViewport* InViewport, EAxisList::Type CurrentAxis)
{
	FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
		InViewport,
		GetScene(),
		EngineShowFlags)
		.SetRealtimeUpdate(IsRealtime())
	);

	FSceneView* View = CalcSceneView(&ViewFamily);

	FIntPoint MousePos;
	InViewport->GetMousePos(MousePos);

	// Move the light cards
	ProjectionHelper->DragActors(SelectedActors, MousePos, *View, EditorWidgetCoordinateSystem, DragWidgetOffset, CurrentAxis, LastSelectedActor);

	// Update the level instances
	for (const FDisplayClusterWeakStageActorPtr& SelectedActor : SelectedActors)
	{
		PropagateActorTransform(SelectedActor);
	}

	FSceneViewInitOptions SceneViewInitOptions;
	GetSceneViewInitOptions(SceneViewInitOptions);
	FViewMatrices ViewMatrices(SceneViewInitOptions);
	
	FVector LightCardProjectedLocation = ProjectWorldPosition(CachedEditorWidgetWorldTransform.GetTranslation(), ViewMatrices);
	FVector2D ScreenPercentage;
	
	if (CVarICVFXPanelAutoPan.GetValueOnGameThread() && IsLocationCloseToEdge(LightCardProjectedLocation, InViewport, View, &ScreenPercentage))
	{
		DesiredLookAtSpeed = FMath::Max(ScreenPercentage.X, ScreenPercentage.Y) * MaxDesiredLookAtSpeed;
		DesiredLookAtLocation = LightCardProjectedLocation;
	}
	else
	{
		DesiredLookAtLocation.Reset();
	}
}

void FDisplayClusterLightCardEditorViewportClient::MoveSelectedUVActors(FViewport* InViewport, EAxisList::Type CurrentAxis)
{
	FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
		InViewport,
		GetScene(),
		EngineShowFlags)
		.SetRealtimeUpdate(IsRealtime()));

	FSceneView* View = CalcSceneView(&ViewFamily);

	FIntPoint MousePos;
	InViewport->GetMousePos(MousePos);

	ProjectionHelper->DragUVActors(SelectedActors, MousePos, *View, DragWidgetOffset, CurrentAxis, LastSelectedActor);

	// Update the level instances
	for (const FDisplayClusterWeakStageActorPtr& Actor : SelectedActors)
	{
		PropagateActorTransform(Actor);
	}
}

void FDisplayClusterLightCardEditorViewportClient::ScaleSelectedActors(FViewport* InViewport, EAxisList::Type CurrentAxis)
{
	if (LastSelectedActor.IsValid())
	{
		const FVector2D DeltaScale = GetActorScaleDelta(InViewport, LastSelectedActor, CurrentAxis);
		for (const FDisplayClusterWeakStageActorPtr& SelectedActor : SelectedActors)
		{
			SelectedActor->SetScale(SelectedActor->GetScale() + DeltaScale);
			PropagateActorTransform(SelectedActor);
		}
	}
}

FVector2D FDisplayClusterLightCardEditorViewportClient::GetActorScaleDelta(FViewport* InViewport, const FDisplayClusterWeakStageActorPtr& InActor, EAxisList::Type CurrentAxis)
{
	if (InActor == nullptr)
	{
		return FVector2D();
	}
	
	FIntPoint MousePos;
	InViewport->GetMousePos(MousePos);

	FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
		InViewport,
		GetScene(),
		EngineShowFlags)
		.SetRealtimeUpdate(IsRealtime()));
	FSceneView* View = CalcSceneView(&ViewFamily);

	const FVector2D DragDir = MousePos - LastWidgetMousePos;

	const FVector WorldWidgetOrigin = CachedEditorWidgetWorldTransform.GetTranslation();
	const FVector WorldWidgetXAxis = CachedEditorWidgetWorldTransform.GetRotation().RotateVector(FVector::XAxisVector);
	const FVector WorldWidgetYAxis = CachedEditorWidgetWorldTransform.GetRotation().RotateVector(FVector::YAxisVector);

	FVector2D ScreenXAxis = FVector2D::ZeroVector;
	FVector2D ScreenYAxis = FVector2D::ZeroVector;

	WorldToScreenDirection(*View, WorldWidgetOrigin, WorldWidgetXAxis, ScreenXAxis);
	WorldToScreenDirection(*View, WorldWidgetOrigin, WorldWidgetYAxis, ScreenYAxis);

	FVector2D ScaleDir = FVector2D::ZeroVector;

	if (CurrentAxis & EAxisList::Type::X)
	{
		ScaleDir += ScreenXAxis;
	}

	if (CurrentAxis & EAxisList::Type::Y)
	{
		ScaleDir += ScreenYAxis;
	}

	ScaleDir.Normalize();

	// To give the scale delta a nice feel to it when dragging, we want the distance the user is dragging the mouse to be proportional to the change in size
	// of the light card when the scale delta is applied. So, if the user moves the mouse a distance d, then the light card's bounds along the direction of scale
	// should change by 2d (one d for each side), and the new scale should be s' = (h + 2d) / h_0, where h is the current length of the side and h_0 is the unscaled
	// length of the side. Since the current scale s = h / h_0, this means that the scale delta is s' - s = 2d / h_0.

	// First, obtain the size of the unscaled light card in the direction the user is scaling. Convert to screen space as the scale and drag vectors are in screen space
	const bool bLocalSpace = true;
	const FVector LightCardSize3D = InActor->GetBoxBounds(bLocalSpace).GetSize();
	const FVector2D SizeToScale = FVector2D((CurrentAxis & EAxisList::X) * LightCardSize3D.Y, (CurrentAxis & EAxisList::Y) * LightCardSize3D.Z);
	const double DistanceFromCamera = FMath::Max(FVector::Dist(WorldWidgetOrigin, View->ViewMatrices.GetViewOrigin()), 1.0f);
	const double ScreenSize = RenderViewportType == LVT_Perspective 
		? View->ViewMatrices.GetScreenScale() * SizeToScale.Length() / DistanceFromCamera 
		: SizeToScale.Length() * View->ViewMatrices.GetViewToClip().M[0][0] * View->UnscaledViewRect.Width();

	// Compute the scale delta as s' - s = 2d / h_0
	const double ScaleMagnitude = 2.0f * (ScaleDir | DragDir) / ScreenSize;

	FVector2D ScaleDelta = FVector2D((CurrentAxis & EAxisList::X) * ScaleMagnitude, (CurrentAxis & EAxisList::Y) * ScaleMagnitude);

	// If both axes are being scaled at the same time, preserve the aspect ratio of the scale delta
	if ((CurrentAxis & EAxisList::Type::X) && (CurrentAxis & EAxisList::Type::Y))
	{
		// Ensure the signs of the deltas remain the same, and avoid potential divide by zero
		FVector2D Scale = InActor->GetScale();
		ScaleDelta.Y = ScaleDelta.X * FMath::Abs(Scale.Y) / FMath::Max(0.001, FMath::Abs(Scale.X));
	}

	return ScaleDelta;
}

void FDisplayClusterLightCardEditorViewportClient::SpinSelectedActors(FViewport* InViewport)
{
	if (LastSelectedActor.IsValid())
	{
		const double DeltaSpin = GetActorSpinDelta(InViewport);
		for (const FDisplayClusterWeakStageActorPtr& Actor : SelectedActors)
		{
			if (!Actor.IsValid())
			{
				continue;
			}

			Actor->SetSpin(Actor->GetSpin() + DeltaSpin);

			PropagateActorTransform(Actor);
		}
	}
}

double FDisplayClusterLightCardEditorViewportClient::GetActorSpinDelta(FViewport* InViewport)
{
	FIntPoint MousePos;
	InViewport->GetMousePos(MousePos);

	FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
		InViewport,
		GetScene(),
		EngineShowFlags)
		.SetRealtimeUpdate(IsRealtime()));
	FSceneView* View = CalcSceneView(&ViewFamily);

	const FVector WorldWidgetOrigin = CachedEditorWidgetWorldTransform.GetTranslation();
	const FVector WorldWidgetXAxis = CachedEditorWidgetWorldTransform.GetRotation().RotateVector(FVector::XAxisVector);
	const FVector WorldWidgetYAxis = CachedEditorWidgetWorldTransform.GetRotation().RotateVector(FVector::YAxisVector);

	FVector2D ScreenOrigin = FVector2D::ZeroVector;
	FVector2D ScreenXAxis = FVector2D::ZeroVector;
	FVector2D ScreenYAxis = FVector2D::ZeroVector;

	ProjectionHelper->WorldToPixel(*View, WorldWidgetOrigin, ScreenOrigin);
	WorldToScreenDirection(*View, WorldWidgetOrigin, WorldWidgetXAxis, ScreenXAxis);
	WorldToScreenDirection(*View, WorldWidgetOrigin, WorldWidgetYAxis, ScreenYAxis);

	FVector2D MousePosOffset = FVector2D(MousePos) - ScreenOrigin;
	FVector2D LastMousePosOffset = FVector2D(LastWidgetMousePos) - ScreenOrigin;

	double Theta = FMath::Atan2(MousePosOffset | ScreenYAxis, MousePosOffset | ScreenXAxis);
	double LastTheta = FMath::Atan2(LastMousePosOffset | ScreenYAxis, LastMousePosOffset | ScreenXAxis);

	return FMath::RadiansToDegrees(Theta - LastTheta);
}

AActor* FDisplayClusterLightCardEditorViewportClient::TraceScreenForActor(const FSceneView& View, int32 HitX, int32 HitY)
{
	UWorld* PreviewWorld = PreviewScene->GetWorld();
	check(PreviewWorld);

	FVector Origin;
	FVector Direction;
	ProjectionHelper->PixelToWorld(View, FIntPoint(HitX, HitY), Origin, Direction);

	const FVector CursorRayStart = Origin;
	const FVector CursorRayEnd = CursorRayStart + Direction * (RenderViewportType == LVT_Perspective ? HALF_WORLD_MAX : WORLD_MAX);

	FCollisionQueryParams TraceParams(SCENE_QUERY_STAT(LightCardTrace), true);

	const bool bInUVMode = ProjectionMode == EDisplayClusterMeshProjectionType::UV;
	auto IsValidActor = [&](AActor* InActor)
	{
		bool bValidForUV = true;
		if (const IDisplayClusterStageActor* StageActor = Cast<IDisplayClusterStageActor>(InActor))
		{
			// Ignore actors not supporting the current UV mode
			bValidForUV = (bInUVMode && StageActor->IsUVActor()) || (!bInUVMode && !StageActor->IsUVActor());
		}
		return bValidForUV && UE::DisplayClusterLightCardEditorUtils::IsProxySelectable(InActor) && ActorProxies.Contains(InActor);
	};
	
	FHitResult ScreenHitResult;
	if (PreviewWorld->LineTraceSingleByObjectType(ScreenHitResult, CursorRayStart, CursorRayEnd, FCollisionObjectQueryParams(FCollisionObjectQueryParams::InitType::AllObjects), TraceParams))
	{
		if (AActor* HitActor = ScreenHitResult.GetActor())
		{
			if (RootActorProxy.Get() == HitActor && ScreenHitResult.Component.IsValid())
			{
				if (UDisplayClusterConfigurationViewport* CfgViewport = FindViewportForPrimitiveComponent(ScreenHitResult.Component.Get()))
				{
					const FString ViewPointName = CfgViewport->Camera;
					UDisplayClusterCameraComponent* ViewPointComponent= nullptr;

					// If the view point name is empty, use the first found view point in the root actor
					if (ViewPointName.IsEmpty())
					{
						ViewPointComponent = RootActorProxy->GetDefaultCamera();
					}
					else
					{
						ViewPointComponent = RootActorProxy->GetComponentByName<UDisplayClusterCameraComponent>(ViewPointName);
					}

					if (ViewPointComponent)
					{
						const FVector ViewPointRayStart = ViewPointComponent->GetComponentLocation();
						const FVector ViewPointRayEnd = ViewPointRayStart + (ScreenHitResult.Location - ViewPointRayStart) * HALF_WORLD_MAX;

						TArray<FHitResult> HitResults;
						if (PreviewWorld->LineTraceMultiByObjectType(HitResults, ViewPointRayStart, ViewPointRayEnd, FCollisionObjectQueryParams(FCollisionObjectQueryParams::InitType::AllObjects), TraceParams))
						{
							for (FHitResult& HitResult : HitResults)
							{
								if (AActor* HitStageActor = HitResult.GetActor())
								{
									if (IsValidActor(HitStageActor))
									{
										return HitStageActor;
									}
								}
							}
						}
					}
				}
			}
			else if (IsValidActor(HitActor))
			{
				return HitActor;
			}
		}
	}

	return nullptr;
}

FVector FDisplayClusterLightCardEditorViewportClient::ProjectWorldPosition(const FVector& UnprojectedWorldPosition, const FViewMatrices& ViewMatrices) const
{
	FDisplayClusterMeshProjectionTransform Transform(ProjectionMode, ViewMatrices.GetWorldToView());
	return Transform.ProjectPosition(UnprojectedWorldPosition);
}

bool FDisplayClusterLightCardEditorViewportClient::WorldToScreenDirection(const FSceneView& View, const FVector& WorldPos, const FVector& WorldDirection, FVector2D& OutScreenDir)
{
	FVector2D ScreenVectorStart = FVector2D::ZeroVector;
	FVector2D ScreenVectorEnd = FVector2D::ZeroVector;

	if (ProjectionHelper->WorldToPixel(View, WorldPos, ScreenVectorStart)
		&& ProjectionHelper->WorldToPixel(View, WorldPos + WorldDirection, ScreenVectorEnd))
	{
		OutScreenDir = (ScreenVectorEnd - ScreenVectorStart).GetSafeNormal();
		return true;
	}
	else
	{
		// If either the start or end of the vector is not onscreen, translate the vector to be in front of the camera to approximate the screen space direction
		const FMatrix InvViewMatrix = View.ViewMatrices.GetViewToWorld();
		const FVector ViewLocation = InvViewMatrix.GetOrigin();
		const FVector ViewDirection = InvViewMatrix.GetUnitAxis(EAxis::Z);
		const FVector Offset = ViewDirection * (FVector::DotProduct(ViewLocation - WorldPos, ViewDirection) + 100.0f);
		const FVector AdjustedWorldPos = WorldPos + Offset;

		if (ProjectionHelper->WorldToPixel(View, AdjustedWorldPos, ScreenVectorStart)
			&& ProjectionHelper->WorldToPixel(View, AdjustedWorldPos + WorldDirection, ScreenVectorEnd))
		{
			OutScreenDir = -(ScreenVectorEnd - ScreenVectorStart).GetSafeNormal();
			return true;
		}
	}

	return false;
}

bool FDisplayClusterLightCardEditorViewportClient::CalcEditorWidgetTransform(FTransform& WidgetTransform)
{
	if (!SelectedActors.Num())
	{
		return false;
	}

	if (!LastSelectedActor.IsValid())
	{
		return false;
	}

	const FVector ActorPosition = LastSelectedActor->GetStageActorTransform(false).GetTranslation();

	WidgetTransform = FTransform(FRotator::ZeroRotator, ActorPosition, FVector::OneVector);

	FQuat WidgetOrientation;
	if (EditorWidget->GetWidgetMode() == FDisplayClusterLightCardEditorWidget::EWidgetMode::WM_Translate)
	{
		if (ProjectionMode == EDisplayClusterMeshProjectionType::UV)
		{
			WidgetOrientation = FMatrix(FVector::YAxisVector, FVector::ZAxisVector, FVector::XAxisVector, FVector::ZeroVector).ToQuat();
		}
		else
		{
			if (EditorWidgetCoordinateSystem == FDisplayClusterLightCardEditorHelper::ECoordinateSystem::Spherical)
			{
				// The translation widget should be oriented to show the x axis pointing in the longitudinal direction and the y axis pointing in the latitudinal direction
				const FVector ProjectionOrigin = ProjectionOriginComponent.IsValid() ? ProjectionOriginComponent->GetComponentLocation() : FVector::ZeroVector;
				const FVector RadialVector = (ActorPosition - ProjectionOrigin).GetSafeNormal();
				const FVector AzimuthalVector = (FVector::ZAxisVector ^ RadialVector).GetSafeNormal();
				const FVector InclinationVector = RadialVector ^ AzimuthalVector;

				WidgetOrientation = FMatrix(AzimuthalVector, InclinationVector, RadialVector, FVector::ZeroVector).ToQuat();
			}
			else if (EditorWidgetCoordinateSystem == FDisplayClusterLightCardEditorHelper::ECoordinateSystem::Cartesian)
			{
				WidgetOrientation = FMatrix(FVector::XAxisVector, FVector::YAxisVector, FVector::ZAxisVector, FVector::ZeroVector).ToQuat();
			}
		}
	}
	else
	{
		// Otherwise, orient the widget to match the light card's local orientation (spin, pitch, yaw)
		const FQuat LightCardOrientation = LastSelectedActor->GetStageActorTransform(false).GetRotation();
		const FVector Normal = LightCardOrientation.RotateVector(FVector::XAxisVector);
		const FVector Tangent = LightCardOrientation.RotateVector(FVector::YAxisVector);
		const FVector Binormal = LightCardOrientation.RotateVector(FVector::ZAxisVector);

		// Reorder the orientation basis so that the x axis and the y axis point along the light card's width and height, respectively
		WidgetOrientation = FMatrix(-Tangent, Binormal, -Normal, FVector::ZeroVector).ToQuat();
	}

	WidgetTransform.SetRotation(WidgetOrientation);

	return true;
}

bool FDisplayClusterLightCardEditorViewportClient::IsLocationCloseToEdge(const FVector& InPosition, const FViewport* InViewport,
                                                                         const FSceneView* InView, FVector2D* OutPercentageToEdge)
{
	if (InViewport == nullptr)
	{
		InViewport = Viewport;
	}

	check(InViewport);
	const FIntPoint ViewportSize = InViewport->GetSizeXY();

	FPlane Projection;
	if (InView == nullptr)
	{
		FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
		Viewport,
		GetScene(),
		EngineShowFlags)
		.SetRealtimeUpdate(IsRealtime()));
		
		InView = CalcSceneView(&ViewFamily);
		Projection = InView->Project(InPosition);

		// InView will be deleted here
	}
	else
	{
		Projection = InView->Project(InPosition);
	}
	
	if (Projection.W > 0)
	{
		const float HighThreshold = 1.f - EdgePercentageLookAtThreshold;
		
		const int32 HalfX = 0.5f * Viewport->GetSizeXY().X;
		const int32 HalfY = 0.5f * Viewport->GetSizeXY().Y;
		const int32 XPos = HalfX + (HalfX * Projection.X);
		const int32 YPos = HalfY + (HalfY * (Projection.Y * -1));
			
		auto GetPercentToEdge = [&](int32 CurrentPos, int32 MaxPos) -> float
		{
			const float Center = static_cast<float>(MaxPos) / 2.f;
			const float RelativePosition = FMath::Abs(CurrentPos - Center);
			return RelativePosition / Center;
		};
			
		const float XPercent = GetPercentToEdge(XPos, ViewportSize.X);
		const float YPercent = GetPercentToEdge(YPos, ViewportSize.Y);
			
		if (OutPercentageToEdge)
		{
			*OutPercentageToEdge = FVector2D(XPercent, YPercent);
		}
			
		return XPercent >= HighThreshold || YPercent >= HighThreshold;
	}

	return false;
}

void FDisplayClusterLightCardEditorViewportClient::FixCameraTransform()
{
	if (ProjectionMode == EDisplayClusterMeshProjectionType::UV)
	{
		// While in UV projection mode, the camera is not allowed to be rotated, but is allowed to be panned
		SetViewRotation(FVector::ForwardVector.Rotation());

		// Clamp the panning to ensure that the UV plane isn't accidentally dragged offscreen
		const float PlaneSize = 0.5f * ADisplayClusterLightCardActor::UVPlaneDefaultSize;
		const FVector ViewLocation = GetViewTransform().GetLocation();
		const FVector ClampedLocation = FVector(0.0f, FMath::Clamp(ViewLocation.Y, -PlaneSize, PlaneSize), FMath::Clamp(ViewLocation.Z, -PlaneSize, PlaneSize));

		SetViewLocation(ClampedLocation);
	}
	else
	{
		// All other projection modes besides UV are not allowed to move the camera's position, but can be rotated freely
		ResetCamera(/* bLocationOnly */ true);
	}
}

void FDisplayClusterLightCardEditorViewportClient::SaveProjectionCameraTransform()
{
	int32 ProjectionModeIndex = (int32)ProjectionMode;
	if (ProjectionViewConfigurations.Num() > ProjectionModeIndex)
	{
		FViewportCameraTransform CameraTransform;
		CameraTransform.SetRotation(GetViewRotation());

		if (ProjectionMode == EDisplayClusterMeshProjectionType::UV)
		{
			CameraTransform.SetLocation(GetViewLocation());
		}

		const bool bIsOrthographic = RenderViewportType != ELevelViewportType::LVT_Perspective;
		if (bIsOrthographic)
		{
			ProjectionViewConfigurations[ProjectionModeIndex].OrthographicTransform = CameraTransform;
		}
		else
		{
			ProjectionViewConfigurations[ProjectionModeIndex].PerspectiveTransform = CameraTransform;
		}
	}
}

void FDisplayClusterLightCardEditorViewportClient::RestoreProjectionCameraTransform()
{
	int32 ProjectionModeIndex = (int32)ProjectionMode;
	if (ProjectionViewConfigurations.Num() > ProjectionModeIndex)
	{
		const bool bIsOrthographic = RenderViewportType != ELevelViewportType::LVT_Perspective;
		const FViewportCameraTransform& CameraTransform = bIsOrthographic 
			? ProjectionViewConfigurations[ProjectionModeIndex].OrthographicTransform
			: ProjectionViewConfigurations[ProjectionModeIndex].PerspectiveTransform;

		SetViewRotation(CameraTransform.GetRotation());

		if (ProjectionMode == EDisplayClusterMeshProjectionType::UV)
		{
			SetViewLocation(CameraTransform.GetLocation());
		}
	}
}

void FDisplayClusterLightCardEditorViewportClient::ResetProjectionViewConfigurations()
{
	constexpr int32 MaxProjections = 3;
	if (ProjectionViewConfigurations.Num() < MaxProjections)
	{
		ProjectionViewConfigurations.AddDefaulted(MaxProjections - ProjectionViewConfigurations.Num());
	}

	ProjectionViewConfigurations[static_cast<int32>(EDisplayClusterMeshProjectionType::Linear)].PerspectiveTransform.SetRotation(FVector::ForwardVector.Rotation());
	ProjectionViewConfigurations[static_cast<int32>(EDisplayClusterMeshProjectionType::Linear)].OrthographicTransform.SetRotation(FVector::UpVector.Rotation());
	ProjectionViewConfigurations[static_cast<int32>(EDisplayClusterMeshProjectionType::Linear)].PerspectiveFOV = 90.0f;
	ProjectionViewConfigurations[static_cast<int32>(EDisplayClusterMeshProjectionType::Linear)].OrthographicFOV = 90.0f;

	ProjectionViewConfigurations[static_cast<int32>(EDisplayClusterMeshProjectionType::Azimuthal)].PerspectiveTransform.SetRotation(FVector::UpVector.Rotation());
	ProjectionViewConfigurations[static_cast<int32>(EDisplayClusterMeshProjectionType::Azimuthal)].PerspectiveFOV = 130.0f;

	ProjectionViewConfigurations[static_cast<int32>(EDisplayClusterMeshProjectionType::UV)].OrthographicTransform.SetRotation(FVector::ForwardVector.Rotation());
	ProjectionViewConfigurations[static_cast<int32>(EDisplayClusterMeshProjectionType::UV)].OrthographicTransform.SetLocation(FVector::ZeroVector);
	ProjectionViewConfigurations[static_cast<int32>(EDisplayClusterMeshProjectionType::UV)].OrthographicFOV = 45.0f;

	RestoreProjectionCameraTransform();
}

void FDisplayClusterLightCardEditorViewportClient::TrySetShouldCheckHitProxy()
{
	if (GetEditorViewportWidget().IsValid() && GetEditorViewportWidget()->GetSceneViewport().IsValid() &&
		((FRenderResource*)GetEditorViewportWidget()->GetSceneViewport().Get())->IsInitialized())
	{
		// If this is set before the SceneViewport is initialized it's possible the client will tick first,
		// attempt to check hit proxies, and crash because RHI is assumed to be initialized at this point.
		bShouldCheckHitProxy = true;
	}
}

void FDisplayClusterLightCardEditorViewportClient::EnterDrawingLightCardMode()
{
	if (InputMode == EInputMode::Idle)
	{
		InputMode = EInputMode::DrawingLightCard;
		SelectActor(nullptr);
	}
}

void FDisplayClusterLightCardEditorViewportClient::ExitDrawingLightCardMode()
{
	if (InputMode == EInputMode::DrawingLightCard)
	{
		InputMode = EInputMode::Idle;
		DrawnMousePositions.Empty();
	}
}

FDisplayClusterLightCardEditorViewportClient::FSpriteProxy FDisplayClusterLightCardEditorViewportClient::FSpriteProxy::
FromBillboard(const UBillboardComponent* InBillboardComponent)
{
	check(InBillboardComponent);
	return FSpriteProxy{
		InBillboardComponent->Sprite,
		InBillboardComponent->GetComponentLocation(),
		InBillboardComponent->U,
		InBillboardComponent->UL,
		InBillboardComponent->V,
		InBillboardComponent->VL,
		InBillboardComponent->ScreenSize,
		InBillboardComponent->GetComponentTransform().GetMaximumAxisScale() * InBillboardComponent->GetOwner()->SpriteScale * 0.25f,
		InBillboardComponent->OpacityMaskRefVal,
		static_cast<bool>(InBillboardComponent->bIsScreenSizeScaled)
	};
}

#undef LOCTEXT_NAMESPACE
