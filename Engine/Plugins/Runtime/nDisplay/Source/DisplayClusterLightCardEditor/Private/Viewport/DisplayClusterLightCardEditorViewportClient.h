// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SCommonEditorViewportToolbarBase.h"
#include "SEditorViewport.h"
#include "AdvancedPreviewScene.h"

#include "DisplayClusterLightCardEditorProxyType.h"
#include "DisplayClusterMeshProjectionRenderer.h"
#include "DisplayClusterLightCardActor.h"
#include "DisplayClusterLightCardEditorWidget.h"
#include "DisplayClusterLightCardEditorHelper.h"
#include "StageActor/DisplayClusterWeakStageActorPtr.h"

class UWidgetComponent;
class ADisplayClusterRootActor;
class FDisplayClusterLightCardEditor;
class SDisplayClusterLightCardEditorViewport;
class FScopedTransaction;
class UDisplayClusterConfigurationViewport;
class UProceduralMeshComponent;

/** Viewport Client for the preview viewport */
class FDisplayClusterLightCardEditorViewportClient : public FEditorViewportClient, public TSharedFromThis<FDisplayClusterLightCardEditorViewportClient>
{
	using Super = FEditorViewportClient;

public:

	/** State machine that helps keep track of the context to which user inputs apply (e.g. mouse buttons, key presses) */
	enum class EInputMode
	{
		Idle,

		/** Indicates that the user is dragging an actor in the viewport */
		DraggingActor,

		/** Indicates that the user is drawing a light card in the viewport */
		DrawingLightCard,
	};

private:
	DECLARE_MULTICAST_DELEGATE(FOnNextSceneRefresh);

public:
	FDisplayClusterLightCardEditorViewportClient(FPreviewScene& InPreviewScene,
		const TWeakPtr<SDisplayClusterLightCardEditorViewport>& InEditorViewportWidget);
	virtual ~FDisplayClusterLightCardEditorViewportClient() override;
	
	// FEditorViewportClient
	virtual FLinearColor GetBackgroundColor() const override;
	virtual void Tick(float DeltaSeconds) override;
	virtual void Draw(FViewport* InViewport, FCanvas* Canvas) override;
	virtual void Draw(const FSceneView* View, FPrimitiveDrawInterface* PDI) override;
	virtual bool CanSetWidgetMode(UE::Widget::EWidgetMode NewMode) const override { return false; }
	virtual bool CanCycleWidgetMode() const override { return false; }
	virtual UE::Widget::EWidgetMode GetWidgetMode() const override { return UE::Widget::WM_None; }
	virtual FSceneView* CalcSceneView(FSceneViewFamily* ViewFamily, const int32 StereoViewIndex = INDEX_NONE) override;
	virtual bool IsLevelEditorClient() const override { return false; }
	virtual bool InputKey(const FInputKeyEventArgs& EventArgs) override;
	virtual bool InputWidgetDelta(FViewport* InViewport, EAxisList::Type CurrentAxis, FVector& Drag, FRotator& Rot, FVector& Scale) override;
	virtual void TrackingStarted(const FInputEventState& InInputState, bool bIsDraggingWidget, bool bNudge) override;
	virtual void TrackingStopped() override;
	virtual void ProcessClick(FSceneView& View, HHitProxy* HitProxy, FKey Key, EInputEvent Event, uint32 HitX, uint32 HitY) override;
	virtual void BeginCameraMovement(bool bHasMovement) override;
	virtual EMouseCursor::Type GetCursor(FViewport* Viewport,int32 X,int32 Y) override;
	virtual ELevelViewportType GetViewportType() const override { return LVT_Perspective; }
	virtual bool HasDropPreviewActors() const override { return DropPreviewLightCards.Num() > 0; }
	virtual void DestroyDropPreviewActors() override;
	virtual bool UpdateDropPreviewActors(int32 MouseX, int32 MouseY, const TArray<UObject*>& DroppedObjects, bool& bOutDroppedObjectsVisible, UActorFactory* FactoryToUse) override;
	virtual bool DropObjectsAtCoordinates(int32 MouseX, int32 MouseY, const TArray<UObject*>& DroppedObjects, TArray<AActor*>& OutNewActors, bool bOnlyDropOnTarget, bool bCreateDropPreview, bool bSelectActors, UActorFactory* FactoryToUse) override;
	// ~FEditorViewportClient

	/** Returns a delegate that is invoked on the next scene refresh. The delegate is cleared afterwards */
	FOnNextSceneRefresh& GetOnNextSceneRefresh() { return OnNextSceneRefreshDelegate; }

	/**
	 * Update the spawned preview actor from a root actor in the level
	 * 
	 * @param RootActor The new root actor to use. Accepts nullptr
	 * @param bForce Force the update even if the RootActor hasn't changed
	 * @param ProxyType The proxy type to destroy and update
	 * @param StageActor Limit the update to a single stage actor
	 */
	void UpdatePreviewActor(ADisplayClusterRootActor* RootActor, bool bForce = false,
		EDisplayClusterLightCardEditorProxyType ProxyType = EDisplayClusterLightCardEditorProxyType::All,
		AActor* StageActor = nullptr);
	
	/** Only update required transform values of proxies */
	void UpdateProxyTransforms();

	/** Update transforms value for a proxy from a level instance */
	void UpdateProxyTransformFromLevelInstance(AActor* InLevelInstance);
	
	/** Remove proxies of the specified type */
	void DestroyProxies(EDisplayClusterLightCardEditorProxyType ProxyType);

	/** Remove a proxy that is or belongs to the given actor */
	void DestroyProxy(AActor* Actor);
	
	/** Selects the actor proxies that correspond to the specified actors */
	void SelectActors(const TArray<AActor*>& ActorsToSelect);

	/** Gets whether any items are selected in the viewport */
	bool HasSelection() const;

	FDisplayClusterLightCardEditorWidget::EWidgetMode GetEditorWidgetMode() const { return EditorWidget->GetWidgetMode(); }
	void SetEditorWidgetMode(FDisplayClusterLightCardEditorWidget::EWidgetMode InWidgetMode);

	/** Gets the current projection mode of the editor viewport */
	EDisplayClusterMeshProjectionType GetProjectionMode() const { return ProjectionMode; }

	/** Gets the current viewport type the viewport is being rendered with */
	ELevelViewportType GetRenderViewportType() const { return RenderViewportType; }

	/** Sets the projection mode and the render viewport type of the viewport */
	void SetProjectionMode(EDisplayClusterMeshProjectionType InProjectionMode, ELevelViewportType InViewportType);

	/** Gets the field of view of the specified projection mode */
	float GetProjectionModeFOV(EDisplayClusterMeshProjectionType InProjectionMode) const;

	/** Sets the field of view of the specified projection mode */
	void SetProjectionModeFOV(EDisplayClusterMeshProjectionType InProjectionMode, float NewFOV);

	/** Resets the camera to the initial rotation / position */
	void ResetCamera(bool bLocationOnly = false);

	/** Frames the currently selected items in the viewport */
	void FrameSelection();

	/** Cycles the editor widget's coordinate system between cartesian and spherical */
	void CycleCoordinateSystem();

	/** Gets the editor widget's current coordinate system */
	FDisplayClusterLightCardEditorHelper::ECoordinateSystem GetCoordinateSystem() const { return EditorWidgetCoordinateSystem; }

	/** Sets the editor widget's coordinate system to the specified coordinate system */
	void SetCoordinateSystem(FDisplayClusterLightCardEditorHelper::ECoordinateSystem NewCoordinateSystem);

	/** Moves specified card to desired coordinates. Actual radius will be based on flush constraint and Actor's RadialOffset.
	 *
	 * @param Actor The actor that we are moving
	 * @param SphericalCoords specifies desired location of actor in spherical coordinates with respect to view point.
	 * 
	*/
	void MoveActorTo(const FDisplayClusterWeakStageActorPtr& Actor, const FDisplayClusterLightCardEditorHelper::FSphericalCoordinates& SphericalCoords) const;

	/** Places the given light card in the middle of the current viewport */
	void CenterActorInView(const FDisplayClusterWeakStageActorPtr& Actor);

	/** Moves all selected light cards to the specified pixel position */
	void MoveSelectedActorsToPixel(const FIntPoint& PixelPos);

	/** Moves all given actors to the specified pixel position */
	void MoveActorsToPixel(const FIntPoint& PixelPos, const TArray<FDisplayClusterWeakStageActorPtr>& InActors);
	
	/** Returns the current input mode */
	EInputMode GetInputMode() { return InputMode; }

	/** Requests that we enter light card drawing input mode */
	void EnterDrawingLightCardMode();

	/** Requests that we exit light card drawing input mode (and go back to idle/normal) */
	void ExitDrawingLightCardMode();

	/** If the viewport client is responsible for the post edit change property call */
	bool HasCalledPostEditChangeProperty() const { return bHasCalledPostEditChangeProperty; }

private:
	/** Initiates a transaction. */
	void BeginTransaction(const FText& Description);

	/** Ends the current transaction, if one exists. */
	void EndTransaction();
	
	/** Gets a list of all primitive components to be rendered in the scene */
	void GetScenePrimitiveComponents(TArray<UPrimitiveComponent*>& OutPrimitiveComponents);

	/** Gets the scene view init options to use to create scene views for the preview scene */
	void GetSceneViewInitOptions(FSceneViewInitOptions& OutViewInitOptions);

	/** Gets the scene view init options to use when rendering the normal map cache */
	void GetNormalMapSceneViewInitOptions(FIntPoint NormalMapSize, float NormalMapFOV, const FVector& ViewDirection, FSceneViewInitOptions& OutViewInitOptions);

	/** Gets the viewport that is attached to the specified primitive component */
	UDisplayClusterConfigurationViewport* FindViewportForPrimitiveComponent(UPrimitiveComponent* PrimitiveComponent);

	/** Finds a suitable primitive component on the stage actor to use as a projection origin */
	USceneComponent* FindProjectionOriginComponent(const ADisplayClusterRootActor* InRootActor) const;

	/** Callback to check if an actor is among the list of selected actors */
	bool IsActorSelected(const AActor* Actor);

	/** Adds the specified actor the the list of selected actors */
	void SelectActor(AActor* Actor, bool bAddToSelection = false);

	/** Notifies the light card editor of the currently selected actors so that it may update other UI components to match */
	void PropagateActorSelection();

	/** Propagates the specified actor proxy's transform back to its level instance version */
	void PropagateActorTransform(const FDisplayClusterWeakStageActorPtr& ActorProxy);

	/** Moves the currently selected actors */
	void MoveSelectedActors(FViewport* InViewport, EAxisList::Type CurrentAxis);
	
	/** Moves the currently selected UV actors */
	void MoveSelectedUVActors(FViewport* InViewport, EAxisList::Type CurrentAxis);

	/** Scales the currently selected actors */
	void ScaleSelectedActors(FViewport* InViewport, EAxisList::Type CurrentAxis);

	/** Determines the appropriate scale delta needed to scale the actor */
	FVector2D GetActorScaleDelta(FViewport* InViewport, const FDisplayClusterWeakStageActorPtr& InActor, EAxisList::Type CurrentAxis);

	/** Rotates the currently selected actors around the actor's normal axis */
	void SpinSelectedActors(FViewport* InViewport);

	/** Determines the appropriate spin delta needed to rotate the light card */
	double GetActorSpinDelta(FViewport* InViewport);

	/** Traces to find the actor corresponding to a click on a stage screen */
	AActor* TraceScreenForActor(const FSceneView& View, int32 HitX, int32 HitY);

	/** Projects the specified world position to the viewport's current projection space */
	FVector ProjectWorldPosition(const FVector& UnprojectedWorldPosition, const FViewMatrices& ViewMatrices) const;

	/** Converts a direction vector from world space to screen screen space, and returns true of the direction vector is on the screen */
	bool WorldToScreenDirection(const FSceneView& View, const FVector& WorldPos, const FVector& WorldDirection, FVector2D& OutScreenDir);

	/** Calculates the world transform to render the editor widget with to align it with the selected actor */
	bool CalcEditorWidgetTransform(FTransform& WidgetTransform);

	/** Checks if the location is approaching the edge of the view space */
	bool IsLocationCloseToEdge(const FVector& InPosition, const FViewport* InViewport = nullptr, const FSceneView* InView = nullptr, FVector2D* OutPercentageToEdge = nullptr);

	/** Called each tick to ensure the viewport camera is in the appropriate place based on the projection mode */
	void FixCameraTransform();

	/** Saves the current camera transform for the current projection mode */
	void SaveProjectionCameraTransform();

	/** Restores the saved camera transform for the current projection mode */
	void RestoreProjectionCameraTransform();

	/** Resets the stored view configurations to default values for each projection mode */
	void ResetProjectionViewConfigurations();

	/** Attempt to set bShouldCheckHitProxy */
	void TrySetShouldCheckHitProxy();
	
	/** Creates a new light card using a polygon alpha mask as defined by the given mouse positions on the viewport */
	void CreateDrawnLightCard(const TArray<FIntPoint>& MousePositions);

	/** Calculates the final distance from the origin of an actor, given its flush distance and a desired offset */
	double CalculateFinalActorDistance(double FlushDistance, double DesiredOffsetFromFlush = 0.) const;

	struct FActorProxy
	{
		FDisplayClusterWeakStageActorPtr LevelInstance;
		FDisplayClusterWeakStageActorPtr Proxy;

		FActorProxy(const AActor* InLevelInstance, const AActor* InProxy)
			: LevelInstance(InLevelInstance)
			, Proxy(InProxy)
		{
		}

		bool operator==(const AActor* Actor) const { return (LevelInstance.IsValid() && LevelInstance.AsActor() == Actor) || (Proxy.IsValid() && Proxy.AsActor() == Actor); }
	};

	struct FSpriteProxy
	{
		TObjectPtr<UTexture2D> Sprite;
		FVector WorldPosition;
		float U;
		float UL;
		float V;
		float VL;
		float ScreenSize;
		float SpriteScale;
		float OpacityMaskRefVal;

		bool bIsScreenSizeScaled = false;

		/** Construct a sprite proxy from a billboard component */
		static FSpriteProxy FromBillboard(const UBillboardComponent* InBillboardComponent);
	};
	
	/** Find the actor proxy if it exists from a level instance */
	const FActorProxy* FindActorProxyFromLevelInstance(AActor* InLevelInstance) const;

	/** Find the actor proxy from either the proxy itself or the actor the proxy belongs to */
	const FActorProxy* FindActorProxyFromActor(AActor* InActor) const;

	/** Create a stage actor proxy */
	AActor* CreateStageActorProxy(AActor* InLevelInstance);
	
	/** Update the transforms for the specific proxy */
	void UpdateProxyTransforms(const FActorProxy& InActorProxy);

	/** When the time in a sequencer has changed */
	void OnSequencerTimeChanged(TWeakPtr<class ISequencer> InSequencer);
	
private:
	TWeakPtr<FSceneViewport> SceneViewportPtr;
	TWeakPtr<SDisplayClusterLightCardEditorViewport> LightCardEditorViewportPtr;
	TWeakPtr<FDisplayClusterLightCardEditor> LightCardEditorPtr;
	TWeakObjectPtr<ADisplayClusterRootActor> RootActorProxy;
	TWeakObjectPtr<ADisplayClusterRootActor> RootActorLevelInstance;
	TSet<AActor*> ActorsRefreshing;

	/** Proxy types that are currently refreshing over 1-2 frames */
	TSet<EDisplayClusterLightCardEditorProxyType> ProxyTypesRefreshing;
	
	/** The radius of the bounding sphere that entirely encapsulates the root actor */
	float RootActorBoundingRadius = 0.0f;

	/** Stores view configuration values (view transform and field of view) for the supported projection modes */
	struct FProjectionViewConfiguration
	{
	public:
		FViewportCameraTransform PerspectiveTransform = FViewportCameraTransform();
		FViewportCameraTransform OrthographicTransform = FViewportCameraTransform();
		float PerspectiveFOV = 0.0f;
		float OrthographicFOV = 0.0f;
	};

	TArray<FActorProxy> ActorProxies;
	TArray<TWeakObjectPtr<UBillboardComponent>> BillboardComponentProxies;
	TArray<TWeakObjectPtr<UWidgetComponent>> WidgetComponentProxies;
	TArray<FDisplayClusterWeakStageActorPtr> SelectedActors;

	/** Billboards and widgets are accessed from the render thread and game thread for sprite rendering */
	FCriticalSection SpriteComponentCS;

	/** View matrices set per tick if there are any sprites being rendered */
	FViewMatrices SpriteViewMatrices;

	/** The actor in the selected actor list that was selected last */
	FDisplayClusterWeakStageActorPtr LastSelectedActor;

	/** Light card preview actors being dropped on the scene */
	TArray<FDisplayClusterWeakStageActorPtr> DropPreviewLightCards;
	
	/** The index of the scene preview renderer returned from IDisplayClusterScenePreview */
	int32 PreviewRendererId = -1;

	/** Helper used to convert between screen and world coordinates for actor positions. */
	TSharedPtr<FDisplayClusterLightCardEditorHelper> ProjectionHelper;
	
	/** The LC editor widget used to manipulate actors */
	TSharedPtr<FDisplayClusterLightCardEditorWidget> EditorWidget;

	/** The cached editor widget transform in unprojected world space */
	FTransform CachedEditorWidgetWorldTransform;

	/** The current coordinate system of the editor widget */
	FDisplayClusterLightCardEditorHelper::ECoordinateSystem EditorWidgetCoordinateSystem = FDisplayClusterLightCardEditorHelper::ECoordinateSystem::Spherical;

	/** The offset between the widget's origin and the place it was clicked when a drag action was started */
	FVector DragWidgetOffset;

	/** The mouse position registered on the previous widget input pass */
	FIntPoint LastWidgetMousePos;

	/** The location the camera should be looking at */
	TOptional<FVector> DesiredLookAtLocation;

	/** The current speed to use when looking at a location */
	float DesiredLookAtSpeed = 1.f;

	/** The maximum speed to use when looking at a location */
	float MaxDesiredLookAtSpeed = 5.f;

	/** The percentage to the edge of the view which should trigger auto tilt and auto pan */
	float EdgePercentageLookAtThreshold = 0.1f;

	/** The current projection mode the 3D viewport is being displayed with */
	EDisplayClusterMeshProjectionType ProjectionMode = EDisplayClusterMeshProjectionType::Azimuthal;

	/** The viewport type (perspective or orthogonal) to use when rendering the viewport. Separate from ViewportType since ViewportType also determines input functionality */
	ELevelViewportType RenderViewportType = LVT_Perspective;

	/** The component of the root actor that is acting as the projection origin. Can be either the root component (stage origin) or a view point component */
	TWeakObjectPtr<USceneComponent> ProjectionOriginComponent;

	/** Stores each projecion mode's view configuration separately so they can be restored when the projection mode is selected  */
	TArray<FProjectionViewConfiguration> ProjectionViewConfigurations;

	/** The increment to change the FOV by when using the scroll wheel */
	float FOVScrollIncrement = 5.0f;

	/** Current input mode */
	EInputMode InputMode = EInputMode::Idle;

	/** Array of mouse positions that will be used to spawn a new light card with the shape of the drawn polygon */
	TArray<FIntPoint> DrawnMousePositions;

	/** Multicast delegate that stores callbacks to be invoked the next time the scene is refreshed */
	FOnNextSceneRefresh OnNextSceneRefreshDelegate;

	/** A flag that disables drawing with the custom projection renderer and instead renders the viewport client with the editor's default renderer */
	bool bDisableCustomRenderer = false;

	/** True if the viewport client has triggered the post edit change property call */
	bool bHasCalledPostEditChangeProperty = false;
};