// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterLightCardEditorWidget.h"

#include "DisplayClusterLightCardEditorViewportClient.h"

#include "Engine/Texture2D.h"
#include "PrimitiveDrawingUtils.h"
#include "TextureResource.h"
#include "UnrealWidget.h"

FDisplayClusterLightCardEditorWidget::FDisplayClusterLightCardEditorWidget()
	: TranslateOriginTexture(LoadObject<UTexture2D>(nullptr, TEXT("/nDisplay/Icons/S_LightCardEditorWidgetTranslate")))
	, ScaleOriginTexture(LoadObject<UTexture2D>(nullptr, TEXT("/nDisplay/Icons/S_LightCardEditorWidgetScale")))
{
}

void FDisplayClusterLightCardEditorWidget::Tick(const FDisplayClusterLightCardEditorViewportClient* ViewportClient, float InDeltaSeconds)
{
	// Cache the DPI scale from the viewport. Must be done here to avoid calling on render thread and potentially failing an assert. See UE-353611
	DPIScale = ViewportClient->GetDPIScale();
}

void FDisplayClusterLightCardEditorWidget::Draw(const FSceneView* View, const FDisplayClusterLightCardEditorViewportClient* ViewportClient, FPrimitiveDrawInterface* PDI, bool bDrawZAxis)
{
	const bool bIsOrthographic = ViewportClient->GetRenderViewportType() != LVT_Perspective;

	const float SizeScalar = GetSizeScreenScalar(View, ViewportClient);
	const float LengthScalar = GetLengthScreenScalar(View, ViewportClient, Transform.GetTranslation());
	const float OrthoScalar = bIsOrthographic ? 1.0f / View->ViewMatrices.GetViewToClip().M[0][0] : 1.0f;

	if (WidgetMode == EWidgetMode::WM_RotateZ)
	{
		DrawAxis(PDI, EAxisList::Type::Y, SizeScalar, LengthScalar, OrthoScalar);
		DrawCircle(PDI, EAxisList::Type::Z, SizeScalar, LengthScalar);
		DrawOrigin(PDI, LengthScalar, 1.f);
	}
	else
	{
		DrawAxis(PDI, EAxisList::Type::X, SizeScalar, LengthScalar, OrthoScalar);
		DrawAxis(PDI, EAxisList::Type::Y, SizeScalar, LengthScalar, OrthoScalar);

		if (bDrawZAxis)
		{
			DrawAxis(PDI, EAxisList::Type::Z, SizeScalar, LengthScalar, OrthoScalar);
		}

		// When drawing with sprites, the length scalar is needed instead of the size scalar. Orthographic scaling is already factored into the length scalar,
		// so the ortho scalar here can be 1.0
		DrawOrigin(PDI, LengthScalar, 1.0f);
	}
}

void FDisplayClusterLightCardEditorWidget::DrawAxis(FPrimitiveDrawInterface* PDI, EAxisList::Type Axis, float SizeScalar, float LengthScalar, float OrthoScalar)
{
	const FVector Origin = Transform.GetTranslation();

	const FVector GlobalAxis = GetGlobalAxis(Axis);
	const FVector TransformedAxis = Transform.TransformVector(GlobalAxis);
	const FLinearColor AxisColor = HighlightedAxis == Axis ? HighlightColor : GetAxisColor(Axis);

	PDI->SetHitProxy(new HWidgetAxis(Axis));
	{
		const FVector AxisStart = ProjectionTransform.ProjectPosition(Origin);
		const FVector AxisEnd = ProjectionTransform.ProjectPosition(Origin + TransformedAxis * AxisLength * WidgetScale * LengthScalar);

		PDI->DrawLine(AxisStart, AxisEnd, AxisColor, ESceneDepthPriorityGroup::SDPG_Foreground, AxisThickness * SizeScalar, 0.0, true);

		if (WidgetMode == EWidgetMode::WM_Scale)
		{
			PDI->DrawPoint(AxisEnd, AxisColor, AxisCapSize * SizeScalar * OrthoScalar, ESceneDepthPriorityGroup::SDPG_Foreground);
		}
	}
	PDI->SetHitProxy(nullptr);
}

void FDisplayClusterLightCardEditorWidget::DrawOrigin(FPrimitiveDrawInterface* PDI, float SizeScalar, float OrthoScalar)
{
	const FVector Origin = ProjectionTransform.ProjectPosition(Transform.GetTranslation());
	const FLinearColor Color = HighlightedAxis == EAxisList::Type::XYZ ? HighlightColor : FLinearColor::White;
	const float SpriteSize = OriginSize * SizeScalar * OrthoScalar;
	
	UTexture* OriginTexture = WidgetMode == EWidgetMode::WM_Scale ? ScaleOriginTexture : TranslateOriginTexture;
	if (FTextureResource* Texture = OriginTexture->GetResource())
	{
		PDI->SetHitProxy(new HWidgetAxis(EAxisList::Type::XYZ));
		PDI->DrawSprite(Origin,
			SpriteSize, SpriteSize,
			Texture,
			Color,
			ESceneDepthPriorityGroup::SDPG_Foreground,
			0, Texture->GetSizeX(),
			0, Texture->GetSizeY(),
			SE_BLEND_Masked);
		PDI->SetHitProxy(nullptr);
	}
}

void FDisplayClusterLightCardEditorWidget::DrawCircle(FPrimitiveDrawInterface* PDI, EAxisList::Type Axis, float SizeScalar, float LengthScalar)
{
	const float ScaledRadius = CirlceRadius * LengthScalar;
	const int32 NumSides = FMath::Clamp(ScaledRadius / 2, 24.0f, 100.0f);
	const float AngleDelta = 2.0f * UE_PI / NumSides;

	const FVector Origin = Transform.GetTranslation();

	const FVector GlobalAxis = GetGlobalAxis(Axis);
	const FVector Normal = Transform.TransformVector(GlobalAxis);
	const FVector X = GlobalAxis == FVector::ZAxisVector ? Transform.TransformVector(FVector::XAxisVector) : Transform.TransformVector(FVector::ZAxisVector);
	const FVector Y = Normal ^ X;

	EAxisList::Type RotationPlane = (EAxisList::Type)(EAxisList::Type::XYZ & ~Axis);

	const FLinearColor CircleColor = HighlightedAxis == RotationPlane ? HighlightColor : GetAxisColor(Axis);

	PDI->SetHitProxy(new HWidgetAxis(RotationPlane));
	{
		FVector Vertex = ProjectionTransform.ProjectPosition(Origin + X * ScaledRadius);
		for (int32 Index = 0; Index < NumSides; ++Index)
		{
			FVector NextVertex = ProjectionTransform.ProjectPosition(Origin + (X * FMath::Cos(AngleDelta * (Index + 1)) + Y * FMath::Sin(AngleDelta * (Index + 1))) * ScaledRadius);
			PDI->DrawLine(Vertex, NextVertex, CircleColor, ESceneDepthPriorityGroup::SDPG_Foreground, CircleThickness * SizeScalar, 0.0f, true);

			Vertex = NextVertex;
		}
	}
	PDI->SetHitProxy(nullptr);
}

FVector FDisplayClusterLightCardEditorWidget::GetGlobalAxis(EAxisList::Type Axis) const
{
	switch (Axis)
	{
	case EAxisList::Type::X:
		return FVector::UnitX();

	case EAxisList::Type::Y:
		return FVector::UnitY();
		
	case EAxisList::Type::Z:
		return FVector::UnitZ();
	}

	return FVector::ZeroVector;
}

FLinearColor FDisplayClusterLightCardEditorWidget::GetAxisColor(EAxisList::Type Axis) const
{
	switch (Axis)
	{
	case EAxisList::Type::X:
		return AxisColorX;

	case EAxisList::Type::Y:
		return AxisColorY;
		
	case EAxisList::Type::Z:
		return AxisColorZ;
	}

	return FLinearColor::Transparent;
}

float FDisplayClusterLightCardEditorWidget::GetLengthScreenScalar(const FSceneView* View, const FDisplayClusterLightCardEditorViewportClient* ViewportClient, const FVector& Origin) const
{
	// The ideal behavior for the length of the widget axes is to remain a fixed length regardless of the field of view or distance from the camera,
	// but change proportionally with the size of the viewport, so that the widget takes up the same percentage of viewport space. 
	// To accomplish this, three factors go into the size scalar: 
	// * DPIScalar ensures the length is the same, relative to viewport size, on high density monitors
	// * ResolutionScale ensures the length remains the same, relative to viewport size, regardless of the size of the viewport.
	// * ProjectionScale ensures that the length remains the same regardless of field of view or distance from the camera. For lengths specifically,
	//   the distance from the camera needs to be taken into account as well. The ViewMatrices ScreenScale contains the appropraite FOV and viewport size factors.
	// Note we use the x axis sizes to compute the scalars as the viewport will only scale smaller when the width is resized; when the 
	// height is resized, the view is clamped

	const bool bIsOrthographic = ViewportClient->GetRenderViewportType() != LVT_Perspective;
	const float ResolutionScale = View->UnconstrainedViewRect.Size().X / (DPIScale * ReferenceResolution);

	const float DistanceFromView = FMath::Max(FVector::Dist(Origin, View->ViewMatrices.GetViewOrigin()), 1.f);
	const FMatrix& ProjectionMatrix = View->ViewMatrices.GetViewToClip();
	const float ProjectionScale = bIsOrthographic ? 
		1.0 / (View->ViewMatrices.GetScreenScale() * FMath::Abs(ProjectionMatrix.M[0][0])) :
		DistanceFromView / View->ViewMatrices.GetScreenScale();

	const float FinalScalar = DPIScale * ResolutionScale * ProjectionScale;
	return FinalScalar;
}

float FDisplayClusterLightCardEditorWidget::GetSizeScreenScalar(const FSceneView* View, const FDisplayClusterLightCardEditorViewportClient* ViewportClient) const
{
	// The ideal behavior for the size of the widget is to remain a fixed size regardless of the field of view or distance from the camera,
	// but change proportionally with the size of the viewport, so that the widget takes up the same percentage of viewport space. 
	// To accomplish this, three factors go into the size scalar: 
	// * DPIScalar ensures the widget is the same size, relative to viewport size, on high density monitors
	// * ResolutionScale ensures the widget remains the same size, relative to viewport size, regardless of the size of the viewport.
	// * ProjectionScale ensures that the widget remains the same size regardless of field of view or distance from the camera
	// Note we use the x axis sizes to compute the scalars as the viewport will only scale smaller when the width is resized; when the 
	// height is resized, the view is clamped

	const bool bIsOrthographic = ViewportClient->GetRenderViewportType() != LVT_Perspective;
	const float ResolutionScale = View->UnconstrainedViewRect.Size().X / (DPIScale * ReferenceResolution);

	const FMatrix& ProjectionMatrix = View->ViewMatrices.GetViewToClip();
	const float ProjectionScale = !bIsOrthographic ? FMath::Abs(ProjectionMatrix.M[0][0]) : 1.0f;

	const float FinalScalar = DPIScale * ResolutionScale / ProjectionScale;
	return FinalScalar;
}