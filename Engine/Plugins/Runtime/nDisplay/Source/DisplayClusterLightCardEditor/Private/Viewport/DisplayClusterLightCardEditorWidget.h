// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Math/Axis.h"

#include "DisplayClusterMeshProjectionRenderer.h"

class FSceneView;
class FPrimitiveDrawInterface;
class FDisplayClusterLightCardEditorViewportClient;

/** Widget used to manipulate entities in the light card editor's viewport */
class FDisplayClusterLightCardEditorWidget
{
public:
	/** The default reference resolution that the size and length constants are defined relative to */
	constexpr static float ReferenceResolution = 1920.0f;

	/** The length of the widget's axes, in pixels */
	constexpr static float AxisLength = 100.f;

	/** The thickness of the widget's axes, in pixels */
	constexpr static float AxisThickness = 5.f;

	/** The size of the axes caps */
	constexpr static float AxisCapSize = 15.f;

	/** The size of the axes origin */
	constexpr static float OriginSize = 10.0f;

	/** The size of the rotation circle, in pixels */
	constexpr static float CirlceRadius = 100.0f;

	/** The thickness of the rotation circle, in pixels */
	constexpr static float CircleThickness = 5.f;

	constexpr static FLinearColor AxisColorX = FLinearColor(0.594f, 0.0197f, 0.0f);
	constexpr static FLinearColor AxisColorY = FLinearColor(0.1349f, 0.3959f, 0.0f);
	constexpr static FLinearColor AxisColorZ = FLinearColor(0.0251f, 0.207f, 0.85f);
	constexpr static FLinearColor HighlightColor = FLinearColor(1.0f, 1.0f, 0.0f);

	enum EWidgetMode
	{
		WM_Translate,
		WM_RotateZ,
		WM_Scale,

		WM_Max
	};

	FDisplayClusterLightCardEditorWidget();

	/** Called from the game thread, allows the widget to perform any game thread specific actions */
	void Tick(const FDisplayClusterLightCardEditorViewportClient* ViewportClient, float InDeltaSeconds);
	
	void Draw(const FSceneView* View, const FDisplayClusterLightCardEditorViewportClient* ViewportClient, FPrimitiveDrawInterface* PDI, bool bDrawZAxis);

	EWidgetMode GetWidgetMode() const { return WidgetMode; }
	void SetWidgetMode(EWidgetMode NewWidgetMode) { WidgetMode = NewWidgetMode; }

	void SetTransform(const FTransform& NewTransform) { Transform = FTransform(NewTransform); }
	void SetProjectionTransform(const FDisplayClusterMeshProjectionTransform& NewProjectionTransform) { ProjectionTransform = FDisplayClusterMeshProjectionTransform(NewProjectionTransform); }
	void SetHighlightedAxis(EAxisList::Type InAxis) { HighlightedAxis = InAxis; }
	void SetWidgetScale(float InWidgetScale) { WidgetScale = InWidgetScale; }

private:
	/** Draws the specified axis to the PDI, sizing and coloring it appropriately */
	void DrawAxis(FPrimitiveDrawInterface* PDI, EAxisList::Type Axis, float SizeScalar, float LengthScalar, float OrthoScalar);

	/** Draws the widget origin to the PDI, sizing and coloring it appropriately */
	void DrawOrigin(FPrimitiveDrawInterface* PDI, float SizeScalar, float OrthoScalar);

	void DrawCircle(FPrimitiveDrawInterface* PDI, EAxisList::Type Axis, float SizeScalar, float LengthScalar);

	/** Gets the axis vector in global coordinates for the specified axis */
	FVector GetGlobalAxis(EAxisList::Type Axis) const;

	/** Gets the color for the specified axis */
	FLinearColor GetAxisColor(EAxisList::Type Axis) const;

	/** Calculates a scalar to use to keep lengths and lines a fixed size on the screen regardless of DPI, viewport size, FOV, or distance from the view point. */
	float GetLengthScreenScalar(const FSceneView* View, const FDisplayClusterLightCardEditorViewportClient* ViewportClient, const FVector& Origin) const;

	/** Calculates a scalar to use to keep sizes a fixed size on the screen regardless of DPI, viewport size, FOV, or distance from the view point. */
	float GetSizeScreenScalar(const FSceneView* View, const FDisplayClusterLightCardEditorViewportClient* ViewportClient) const;

private:
	/** Texture to use to render the translation widget origin */
	UTexture2D* TranslateOriginTexture;

	/** Texture to use to render the scale widget origin */
	UTexture2D* ScaleOriginTexture;

	/** The current mode the widget is in */
	EWidgetMode WidgetMode = WM_Translate;

	/** The world transform to apply to the widget when rendering */
	FTransform Transform;

	/** The projection transfrom to apply to the widget when rendering */
	FDisplayClusterMeshProjectionTransform ProjectionTransform;

	/** The axis on the widget which should be highlighted */
	EAxisList::Type HighlightedAxis = EAxisList::Type::None;

	/** A scale factor to scale the widget's rendered size */
	float WidgetScale = 1.f;

	/** Cached DPI scale used to scale the widget's rendered size */
	float DPIScale = 1.f;
};