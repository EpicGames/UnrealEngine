// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterLightcardEditorViewport.h"

#include "DisplayClusterLightCardEditorCommands.h"
#include "DisplayClusterLightCardEditorStyle.h"
#include "DisplayClusterLightCardEditorUtils.h"
#include "DisplayClusterLightCardEditorViewportClient.h"
#include "DisplayClusterLightCardEditor.h"
#include "LightCardTemplates/DisplayClusterLightCardTemplate.h"
#include "LightCardTemplates/DisplayClusterLightCardTemplateDragDropOp.h"

#include "DisplayClusterConfigurationTypes.h"

#include "DisplayClusterRootActor.h"

#include "Algo/Transform.h"
#include "EditorViewportCommands.h"
#include "Internationalization/Text.h"
#include "ScopedTransaction.h"
#include "SEditorViewportToolBarButton.h"
#include "SEditorViewportToolBarMenu.h"
#include "STransformViewportToolbar.h"
#include "Framework/Commands/GenericCommands.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Kismet2/DebuggerCommands.h"
#include "Slate/SceneViewport.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateIconFinder.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "ToolMenuSection.h"
#include "ViewportToolbar/UnrealEdViewportToolbar.h"
#include "Widgets/Input/SCheckBox.h"


#define LOCTEXT_NAMESPACE "DisplayClusterLightcardEditorViewport"


class SDisplayClusterLightCardEditorViewportToolBar : public SViewportToolBar
{
public:
	SLATE_BEGIN_ARGS(SDisplayClusterLightCardEditorViewportToolBar) {}
		SLATE_ARGUMENT(TWeakPtr<SDisplayClusterLightCardEditorViewport>, EditorViewport)
	SLATE_END_ARGS()

	/** Constructs this widget with the given parameters */
	void Construct(const FArguments& InArgs)
	{
		EditorViewport = InArgs._EditorViewport;

		this->ChildSlot
			[
				BuildToolbar()
			];

		SViewportToolBar::Construct(SViewportToolBar::FArguments());
	}

	/** Constructs the unified toolbar using UToolMenus. */
	TSharedRef<SWidget> BuildToolbar()
	{
		const FName ToolbarName = FName("DisplayClusterLightCardEditor.ViewportToolbar");

		// Register the toolbar menu if it doesn't already exist.
		
		UToolMenu* ToolbarMenu = nullptr;

		if (!UToolMenus::Get()->IsMenuRegistered(ToolbarName))
		{
			ToolbarMenu = UToolMenus::Get()->RegisterMenu(ToolbarName, NAME_None, EMultiBoxType::SlimHorizontalToolBar);
		}
		else
		{
			ToolbarMenu = UToolMenus::Get()->FindMenu(ToolbarName);
		}

		ToolbarMenu->StyleName = "ViewportToolbar";

		// Left section: TRS and coordinate system (spherical/cartesian)
		{
			FToolMenuSection& LeftSection = ToolbarMenu->FindOrAddSection("Left");

			// TRS selection options, includes dropdown and 
			// Note: Not using UE::UnrealEd::CreateTransformsSubmenu because of the extra clutter of n/a options.
			LeftSection.AddEntry(CreateTransformsSubmenu());
			
			// Separator, to match level editor.
			LeftSection.AddSeparator("CoordinateSystemSeparator");

			// Coordinate system toggle button
			LeftSection.AddEntry(CreateCoordinateSystemSubMenuEntry());
		}

		// Right section: Frozen viewports | projection | Show | DrawLightCard
		{
			FToolMenuSection& RightSection = ToolbarMenu->FindOrAddSection("Right");
			RightSection.Alignment = EToolMenuSectionAlign::Last;

			// Frozen viewports Warning button
			{
				FToolMenuEntry UnfreezeAllEntry = FToolMenuEntry::InitMenuEntry(
					FName("UnfreezeAllViewports"),
					LOCTEXT("UnfreezeAllViewportsLabel", "Unfreeze All Viewports"),
					LOCTEXT("ViewportsFrozenWarningToolTip", "Outer viewports are frozen. Click to unfreeze them."),
					FSlateIcon(FDisplayClusterLightCardEditorStyle::Get().GetStyleSetName(), "DisplayClusterLightCardEditor.ViewportsFrozen"),
					FUIAction(
						FExecuteAction::CreateSP(this, &SDisplayClusterLightCardEditorViewportToolBar::UnfreezeAllViewports),
						FCanExecuteAction::CreateSP(this, &SDisplayClusterLightCardEditorViewportToolBar::AreViewportsFrozen)
					),
					EUserInterfaceActionType::Button
				);

				UnfreezeAllEntry.ToolBarData.LabelOverride = FText();
				UnfreezeAllEntry.Visibility = [this]() -> bool { return AreViewportsFrozen(); };

				UnfreezeAllEntry.SubMenuData.Style.StyleSet = &FDisplayClusterLightCardEditorStyle::Get();
				UnfreezeAllEntry.StyleNameOverride = "ViewportToolbarWarning.Raised";

				RightSection.AddEntry(UnfreezeAllEntry);
			}

			// Projection dropdown.
			{
				RightSection.AddSubMenu(
					"ProjectionView",
					TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &SDisplayClusterLightCardEditorViewportToolBar::GetProjectionMenuLabel)),
					LOCTEXT("ProjectionViewTooltip", "Select Projection/View Options"),
					FNewToolMenuDelegate::CreateSP(this, &SDisplayClusterLightCardEditorViewportToolBar::GenerateProjectionViewMenu),
					true /* bInOpenSubMenuOnClick */,
					TAttribute<FSlateIcon>::Create(TAttribute<FSlateIcon>::FGetter::CreateSP(this, &SDisplayClusterLightCardEditorViewportToolBar::GetProjectionMenuIcon))
				);
			}

			// Show (eye) dropdown.
			{
				RightSection.AddEntry(UE::UnrealEd::CreateShowSubmenu(FNewToolMenuDelegate::CreateLambda(
					[](UToolMenu* InMenu)
					{
						FToolMenuSection& SubSection = InMenu->FindOrAddSection("ShowFlags", LOCTEXT("ShowFlagsSection", "Show Flags"));

						SubSection.AddMenuEntry(
							FDisplayClusterLightCardEditorCommands::Get().ToggleAllLabels,
							TAttribute<FText>(),
							TAttribute<FText>(),
							TAttribute<FSlateIcon>(FSlateIcon(FDisplayClusterLightCardEditorStyle::Get().GetStyleSetName(), "DisplayClusterLightCardEditor.LabelSymbol"))
						);

						SubSection.AddMenuEntry(
							FDisplayClusterLightCardEditorCommands::Get().ToggleIconVisibility,
							TAttribute<FText>(),
							TAttribute<FText>(),
							TAttribute<FSlateIcon>(FSlateIcon(FDisplayClusterLightCardEditorStyle::Get().GetStyleSetName(), "DisplayClusterLightCardEditor.IconSymbol"))
						);
					})
				));
			}

			// DrawLightCard button as a widget (it is more straight forward to customize the style of a Widget than a ToolBarButton).
			{
				FToolMenuEntry DrawEntry = FToolMenuEntry::InitWidget(
					FName("DrawLightCardWidget"),
					MakeDrawLightCardWidget(),
					LOCTEXT("DrawLC", "Draw Light Card"),
					false, /* bNoIndent */
					false, /* bSearchable */
					false, /* bNoPadding */
					FText()
				);

				RightSection.AddEntry(DrawEntry);
			}
		}

		// Set up a menu context with the command list.

		FToolMenuContext Context;

		if (EditorViewport.IsValid())
		{
			Context.AppendCommandList(EditorViewport.Pin()->GetCommandList());
		}

		return UToolMenus::Get()->GenerateWidget(ToolbarName, Context);
	}

	/** Creates TRS menu. Slimmed down version of UE::UnrealEd::CreateTransformsSubmenu */
	FToolMenuEntry CreateTransformsSubmenu()
	{
		FToolMenuEntry Entry = FToolMenuEntry::InitSubMenu(
			"Transform",
			LOCTEXT("TransformsSubmenuLabel", "Transform"),
			LOCTEXT("TransformsSubmenuTooltip", "Viewport-related transforms tools"),
			FNewToolMenuDelegate::CreateLambda(
				[](UToolMenu* Submenu) -> void
				{
					{
						FToolMenuSection& TransformToolsSection =
							Submenu->FindOrAddSection("TransformTools", LOCTEXT("TransformToolsLabel", "Transform Tools"));

						FToolMenuEntry TranslateMode =
							FToolMenuEntry::InitMenuEntry(FEditorViewportCommands::Get().TranslateMode);
						TranslateMode.SetShowInToolbarTopLevel(true);
						TranslateMode.ToolBarData.StyleNameOverride = "ViewportToolbar.TransformTools";
						TransformToolsSection.AddEntry(TranslateMode);

						FToolMenuEntry RotateMode = FToolMenuEntry::InitMenuEntry(FEditorViewportCommands::Get().RotateMode);
						RotateMode.SetShowInToolbarTopLevel(true);
						RotateMode.ToolBarData.StyleNameOverride = "ViewportToolbar.TransformTools";
						TransformToolsSection.AddEntry(RotateMode);

						FToolMenuEntry ScaleMode = FToolMenuEntry::InitMenuEntry(FEditorViewportCommands::Get().ScaleMode);
						ScaleMode.SetShowInToolbarTopLevel(true);
						ScaleMode.ToolBarData.StyleNameOverride = "ViewportToolbar.TransformTools";
						TransformToolsSection.AddEntry(ScaleMode);
					}
				}
			)
		);

		Entry.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.SelectMode");
		Entry.ToolBarData.LabelOverride = FText();
		Entry.ToolBarData.ResizeParams.ClippingPriority = 1000;

		return Entry;
	}

	/** Make the DrawLightCard toggle button widget. */
	TSharedRef<SWidget> MakeDrawLightCardWidget()
	{
		const TSharedPtr<const FUICommandList> Commands = EditorViewport.Pin()->GetCommandList();
		const TSharedPtr<const FUICommandInfo> DrawCmd = FDisplayClusterLightCardEditorCommands::Get().DrawLightCard;

		return SNew(SCheckBox)
			.Style(&FDisplayClusterLightCardEditorStyle::Get().GetWidgetStyle<FCheckBoxStyle>("DisplayClusterLightCardEditor.DrawLightcardsToggleButton"))
			.Padding(FMargin(2))
			.IsChecked_Lambda([Commands, DrawCmd]() -> ECheckBoxState
				{
					if (Commands.IsValid() && DrawCmd.IsValid())
					{
						if (const FUIAction* Action = Commands->GetActionForCommand(DrawCmd))
						{
							return Action->GetCheckState();
						}
					}
					return ECheckBoxState::Unchecked;
				})
			.OnCheckStateChanged_Lambda([Commands, DrawCmd](ECheckBoxState)
				{
					if (Commands.IsValid() && DrawCmd.IsValid())
					{
						if (const FUIAction* Action = Commands->GetActionForCommand(DrawCmd))
						{
							Action->Execute();
						}
					}
				})
			.ToolTipText(DrawCmd->GetDescription())
			[
				SNew(SBorder)
					.Padding(FMargin(4)) // Pad to make the icon not fully fill the button area.
					.BorderImage(FCoreStyle::Get().GetBrush("NoBorder"))
					[
						SNew(SBox)
							.WidthOverride(16) // Constrain the size to match the rest of the toolbar.
							.HeightOverride(16)
							[
								SNew(SImage)
									.Image(FDisplayClusterLightCardEditorStyle::Get().GetBrush("DisplayClusterLightCardEditor.DrawPoly"))
									// Tint the icon according to its state.
									.ColorAndOpacity_Lambda([Commands, DrawCmd]() -> FSlateColor 
										{
											if (Commands.IsValid() && DrawCmd.IsValid())
											{
												if (const FUIAction* Action = Commands->GetActionForCommand(DrawCmd))
												{
													return Action->GetCheckState() == ECheckBoxState::Checked
														? FSlateColor(FLinearColor::Black)
														: FSlateColor(FLinearColor::White);
												}
											}
											return FSlateColor(FLinearColor::White);
										})
							]
					]
			];
	}

	/** Returns the icon for the current projection mode. */
	FSlateIcon GetProjectionMenuIcon() const
	{
		if (EditorViewport.IsValid())
		{
			TSharedRef<FDisplayClusterLightCardEditorViewportClient> ViewportClient = EditorViewport.Pin()->GetLightCardEditorViewportClient();

			switch (ViewportClient->GetProjectionMode())
			{
			case EDisplayClusterMeshProjectionType::Linear:
				if (ViewportClient->GetRenderViewportType() == ELevelViewportType::LVT_Perspective)
				{
					return FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorViewport.Perspective");
				}
				else
				{
					return FSlateIcon(FDisplayClusterLightCardEditorStyle::Get().GetStyleSetName(), "DisplayClusterLightCardEditor.Orthographic");
				}

			case EDisplayClusterMeshProjectionType::Azimuthal:
				return FSlateIcon(FDisplayClusterLightCardEditorStyle::Get().GetStyleSetName(), "DisplayClusterLightCardEditor.Dome");

			case EDisplayClusterMeshProjectionType::UV:
				return FSlateIcon(FDisplayClusterLightCardEditorStyle::Get().GetStyleSetName(), "DisplayClusterLightCardEditor.UV");
			}
		}

		// Fallback icon
		return FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorViewport.Perspective");
	}

	/** Creates the menu entry for the coordinate system (spherical/cartesian). Doesn't refer to global/local */
	FToolMenuEntry CreateCoordinateSystemSubMenuEntry()
	{
		FToolMenuEntry CoordinateSystemSubmenu = FToolMenuEntry::InitSubMenu(
			"CoordinateSystem",
			LOCTEXT("CoordinateSystemLabel", "Coordinate System"),
			LOCTEXT("CoordinateSystemTooltip", "Select between coordinate systems"),
			FNewToolMenuDelegate::CreateLambda(
				[](UToolMenu* InSubmenu)
				{
					FToolMenuSection& UnnamedSection = InSubmenu->FindOrAddSection(NAME_None);

					UnnamedSection.AddMenuEntry(FDisplayClusterLightCardEditorCommands::Get().SphericalCoordinateSystem);
					UnnamedSection.AddMenuEntry(FDisplayClusterLightCardEditorCommands::Get().CartesianCoordinateSystem);
				}
			)
		);

		// Set the icon
		{
			TWeakPtr<SDisplayClusterLightCardEditorViewportToolBar> WeakSelf = SharedThis(this);
			CoordinateSystemSubmenu.Icon = TAttribute<FSlateIcon>::CreateLambda(
				[WeakSelf]() -> FSlateIcon
				{
					TSharedPtr<SDisplayClusterLightCardEditorViewportToolBar> PinnedSelf = WeakSelf.Pin();

					if (PinnedSelf.IsValid()
						&& PinnedSelf->EditorViewport.IsValid()
						&& PinnedSelf->EditorViewport.Pin()->GetLightCardEditorViewportClient()->GetCoordinateSystem() == FDisplayClusterLightCardEditorHelper::ECoordinateSystem::Cartesian)
					{
						return FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("EditorViewport.RelativeCoordinateSystem_World"));
					}

					return FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.Transform"));
				}
			);
		}

		// Tooltip to show the hotkeys for cycling.
		{
			CoordinateSystemSubmenu.ToolTip = TAttribute<FText>::CreateLambda(
				[]() -> FText
				{
					const FInputChord& PrimaryChord = *FDisplayClusterLightCardEditorCommands::Get().CycleEditorWidgetCoordinateSystem->GetActiveChord(EMultipleKeyBindingIndex::Primary);
					const FInputChord& SecondaryChord = *FDisplayClusterLightCardEditorCommands::Get().CycleEditorWidgetCoordinateSystem->GetActiveChord(EMultipleKeyBindingIndex::Secondary);

					if (PrimaryChord.IsValidChord() && SecondaryChord.IsValidChord())
					{
						FFormatNamedArguments Args;
						Args.Add(TEXT("PrimaryChord"), PrimaryChord.GetInputText());
						Args.Add(TEXT("SecondaryChord"), SecondaryChord.GetInputText());
						return FText::Format(
							LOCTEXT("CoordinateSystemTooltipWithBothChords", "Select between coordinate systems. \n{PrimaryChord} or {SecondaryChord} to cycle between them."),
							Args
						);
					}
					else if (PrimaryChord.IsValidChord() || SecondaryChord.IsValidChord())
					{
						FText ChordText = PrimaryChord.IsValidChord() ? PrimaryChord.GetInputText() : SecondaryChord.GetInputText();
						return FText::Format(
							LOCTEXT("CoordinateSystemTooltipSingleChord", "Select between coordinate systems. \n{0} to cycle between them."),
							ChordText
						);
					}
					return LOCTEXT("CoordinateSystemTooltipNoChords", "Select between coordinate systems");
				}
			);
		}

		// Action override so that clicking the button executes the cycle command.
		FToolUIAction CycleCoordSystemAction;
		{
			CycleCoordSystemAction.ExecuteAction = FToolMenuExecuteAction::CreateLambda(
				[](const FToolMenuContext& InContext)
				{
					if (const FUIAction* Action = InContext.GetActionForCommand(FDisplayClusterLightCardEditorCommands::Get().CycleEditorWidgetCoordinateSystem))
					{
						Action->Execute();
					}
				}
			);
		}

		CoordinateSystemSubmenu.ToolBarData.LabelOverride = FText::GetEmpty();
		CoordinateSystemSubmenu.ToolBarData.ActionOverride = CycleCoordSystemAction;
		CoordinateSystemSubmenu.SetShowInToolbarTopLevel(true);

		return CoordinateSystemSubmenu;
	}

	/** Combined Projection and View Menu */
	void GenerateProjectionViewMenu(UToolMenu* InMenu)
	{
		// Projection section.
		{
			FToolMenuSection& ProjectionSection = InMenu->AddSection("Projection", LOCTEXT("ProjectionMenuHeader", "Projection"));

			// Perspective
			ProjectionSection.AddMenuEntry(
				FDisplayClusterLightCardEditorCommands::Get().PerspectiveProjection,
				TAttribute<FText>(),
				TAttribute<FText>(),
				FSlateIcon(FAppStyle::Get().GetStyleSetName(), "EditorViewport.Perspective")
			);

			// Orthographic
			ProjectionSection.AddMenuEntry(
				FDisplayClusterLightCardEditorCommands::Get().OrthographicProjection,
				TAttribute<FText>(),
				TAttribute<FText>(),
				FSlateIcon(FDisplayClusterLightCardEditorStyle::Get().GetStyleSetName(), "DisplayClusterLightCardEditor.Orthographic")
			);

			// Azimuthal
			ProjectionSection.AddMenuEntry(
				FDisplayClusterLightCardEditorCommands::Get().AzimuthalProjection,
				TAttribute<FText>(),
				TAttribute<FText>(),
				FSlateIcon(FDisplayClusterLightCardEditorStyle::Get().GetStyleSetName(), "DisplayClusterLightCardEditor.Dome")
			);

			// UV
			ProjectionSection.AddMenuEntry(
				FDisplayClusterLightCardEditorCommands::Get().UVProjection,
				TAttribute<FText>(),
				TAttribute<FText>(),
				FSlateIcon(FDisplayClusterLightCardEditorStyle::Get().GetStyleSetName(), "DisplayClusterLightCardEditor.UV")
			);
		}

		// View Orientation section.
		{
			FToolMenuSection& ViewSection = InMenu->AddSection("ViewOrientation", LOCTEXT("ViewOrientationMenuHeader", "View Orientation"));

			ViewSection.AddMenuEntry(
				FDisplayClusterLightCardEditorCommands::Get().ViewOrientationTop,
				TAttribute<FText>(),
				TAttribute<FText>(),
				TAttribute<FSlateIcon>(FSlateIcon(FAppStyle::Get().GetStyleSetName(), "EditorViewport.Top"))
			);

			ViewSection.AddMenuEntry(
				FDisplayClusterLightCardEditorCommands::Get().ViewOrientationBottom,
				TAttribute<FText>(),
				TAttribute<FText>(),
				TAttribute<FSlateIcon>(FSlateIcon(FAppStyle::Get().GetStyleSetName(), "EditorViewport.Bottom"))
			);

			ViewSection.AddMenuEntry(
				FDisplayClusterLightCardEditorCommands::Get().ViewOrientationLeft,
				TAttribute<FText>(),
				TAttribute<FText>(),
				TAttribute<FSlateIcon>(FSlateIcon(FAppStyle::Get().GetStyleSetName(), "EditorViewport.Left"))
			);

			ViewSection.AddMenuEntry(
				FDisplayClusterLightCardEditorCommands::Get().ViewOrientationRight,
				TAttribute<FText>(),
				TAttribute<FText>(),
				TAttribute<FSlateIcon>(FSlateIcon(FAppStyle::Get().GetStyleSetName(), "EditorViewport.Right"))
			);

			ViewSection.AddMenuEntry(
				FDisplayClusterLightCardEditorCommands::Get().ViewOrientationFront,
				TAttribute<FText>(),
				TAttribute<FText>(),
				TAttribute<FSlateIcon>(FSlateIcon(FAppStyle::Get().GetStyleSetName(), "EditorViewport.Front"))
			);

			ViewSection.AddMenuEntry(
				FDisplayClusterLightCardEditorCommands::Get().ViewOrientationBack,
				TAttribute<FText>(),
				TAttribute<FText>(),
				TAttribute<FSlateIcon>(FSlateIcon(FAppStyle::Get().GetStyleSetName(), "EditorViewport.Back"))
			);
		}

		// Additional View Options section.
		{
			FToolMenuSection& OptionsSection = InMenu->AddSection("ViewOptions", LOCTEXT("ViewOptionsMenuHeader", "View Options"));

			OptionsSection.AddMenuEntry(
				FDisplayClusterLightCardEditorCommands::Get().ResetCamera,
				TAttribute<FText>(),
				TAttribute<FText>(),
				FSlateIcon(FAppStyle::Get().GetStyleSetName(), "StaticMeshEditor.ResetCamera")
			);
		}
	}

	/** Returns the label for the Projection/View dropdown based on the current projection mode. */
	FText GetProjectionMenuLabel() const
	{
		FText Label = LOCTEXT("ProjectionMenuTitle_Default", "Projection");

		if (EditorViewport.IsValid())
		{
			TSharedRef<FDisplayClusterLightCardEditorViewportClient> ViewportClient = EditorViewport.Pin()->GetLightCardEditorViewportClient();
			switch (ViewportClient->GetProjectionMode())
			{
			case EDisplayClusterMeshProjectionType::Linear:
				Label = (ViewportClient->GetRenderViewportType() == ELevelViewportType::LVT_Perspective)
					? LOCTEXT("ProjectionMenuTitle_Perspective", "Perspective")
					: LOCTEXT("ProjectionMenuTitle_Orthographic", "Orthographic");
				break;

			case EDisplayClusterMeshProjectionType::Azimuthal:
				Label = LOCTEXT("ProjectionMenuTitle_Azimuthal", "Dome");
				break;

			case EDisplayClusterMeshProjectionType::UV:
				Label = LOCTEXT("ProjectionMenuTitle_UV", "UV");
				break;
			}
		}

		return Label;
	}

	void UnfreezeAllViewports()
	{
		if (EditorViewport.IsValid())
		{
			const TWeakObjectPtr<ADisplayClusterRootActor> RootActor = EditorViewport.Pin()->GetRootActor();
			if (RootActor.IsValid())
			{
				FScopedTransaction Transaction(LOCTEXT("UnfreezeViewports", "Unfreeze viewports"));
				RootActor->SetFreezeOuterViewports(false);
			}
		}
	}

	bool AreViewportsFrozen() const
	{
		if (EditorViewport.IsValid())
		{
			const TWeakObjectPtr<ADisplayClusterRootActor> RootActor = EditorViewport.Pin()->GetRootActor();
			if (RootActor.IsValid())
			{
				const UDisplayClusterConfigurationData* ConfigData = RootActor->GetConfigData();
				if (ConfigData && ConfigData->StageSettings.bFreezeRenderOuterViewports)
				{
					return true;
				}
			}
		}
		return false;
	}

private:
	/** Reference to the parent viewport */
	TWeakPtr<SDisplayClusterLightCardEditorViewport> EditorViewport;
};

const FVector SDisplayClusterLightCardEditorViewport::ViewDirectionTop = FVector(0.0f, 0.0f, 1.0f);
const FVector SDisplayClusterLightCardEditorViewport::ViewDirectionBottom = FVector(0.0f, 0.0f, -1.0f);
const FVector SDisplayClusterLightCardEditorViewport::ViewDirectionLeft = FVector(0.0f, -1.0f, 0.0f);
const FVector SDisplayClusterLightCardEditorViewport::ViewDirectionRight = FVector(0.0f, 1.0f, 0.0f);
const FVector SDisplayClusterLightCardEditorViewport::ViewDirectionFront = FVector(1.0f, 0.0f, 0.0f);
const FVector SDisplayClusterLightCardEditorViewport::ViewDirectionBack = FVector(-1.0f, 0.0f, 0.0f);

void SDisplayClusterLightCardEditorViewport::Construct(const FArguments& InArgs, TSharedPtr<FDisplayClusterLightCardEditor> InLightCardEditor, TSharedPtr<class FUICommandList> InCommandList)
{
	LightCardEditorPtr = InLightCardEditor;

	PreviewScene = MakeShared<FPreviewScene>(FPreviewScene::ConstructionValues());

	SEditorViewport::Construct(SEditorViewport::FArguments());

	if (InCommandList.IsValid())
	{
		CommandList->Append(InCommandList.ToSharedRef());
	}

	SetRootActor(LightCardEditorPtr.Pin()->GetActiveRootActor().Get());
}

SDisplayClusterLightCardEditorViewport::~SDisplayClusterLightCardEditorViewport()
{
	if (PreviewScene.IsValid())
	{
		if (UWorld* PreviewWorld = PreviewScene->GetWorld())
		{
			PreviewWorld->DestroyWorld(true);
			PreviewWorld->MarkObjectsPendingKill();
			PreviewWorld->MarkAsGarbage();
		}
		PreviewScene.Reset();
	}
}

TSharedRef<SEditorViewport> SDisplayClusterLightCardEditorViewport::GetViewportWidget()
{
	return SharedThis(this);
}

TSharedPtr<FExtender> SDisplayClusterLightCardEditorViewport::GetExtenders() const
{
	TSharedPtr<FExtender> Result(MakeShareable(new FExtender));
	return Result;
}

void SDisplayClusterLightCardEditorViewport::OnFloatingButtonClicked()
{
}

FReply SDisplayClusterLightCardEditorViewport::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	const FVector2D MousePos = FSlateApplication::Get().GetCursorPos();
	const float DPIScale = FPlatformApplicationMisc::GetDPIScaleFactorAtPoint(MousePos.X, MousePos.Y);
	PasteHerePos = MyGeometry.AbsoluteToLocal(MousePos) * DPIScale;

	return SEditorViewport::OnKeyDown(MyGeometry, InKeyEvent);
}

void SDisplayClusterLightCardEditorViewport::SetRootActor(ADisplayClusterRootActor* NewRootActor)
{
	if (ViewportClient.IsValid())
	{
		ViewportClient->UpdatePreviewActor(NewRootActor);
	}
}

TWeakObjectPtr<ADisplayClusterRootActor> SDisplayClusterLightCardEditorViewport::GetRootActor() const
{
	if (LightCardEditorPtr.IsValid())
	{
		return LightCardEditorPtr.Pin()->GetActiveRootActor();
	}
	return nullptr;
}

void SDisplayClusterLightCardEditorViewport::SummonContextMenu()
{
	const FVector2D MousePos = FSlateApplication::Get().GetCursorPos();
	const float DPIScale = FPlatformApplicationMisc::GetDPIScaleFactorAtPoint(MousePos.X, MousePos.Y);
	PasteHerePos = GetTickSpaceGeometry().AbsoluteToLocal(FSlateApplication::Get().GetCursorPos()) * DPIScale;

	TSharedRef<SWidget> MenuContents = MakeContextMenu();
	FSlateApplication::Get().PushMenu(
		AsShared(),
		FWidgetPath(),
		MenuContents,
		MousePos,
		FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));
}

TSharedRef<FEditorViewportClient> SDisplayClusterLightCardEditorViewport::MakeEditorViewportClient()
{
	check(PreviewScene.IsValid());

	ViewportClient = MakeShareable(new FDisplayClusterLightCardEditorViewportClient(*PreviewScene.Get(), SharedThis(this)));

	if (LightCardEditorPtr.IsValid())
	{
		ViewportClient->UpdatePreviewActor(LightCardEditorPtr.Pin()->GetActiveRootActor().Get());
	}

	return ViewportClient.ToSharedRef();
}

TSharedPtr<SWidget> SDisplayClusterLightCardEditorViewport::BuildViewportToolbar()
{
	return SNew(SDisplayClusterLightCardEditorViewportToolBar)
		.EditorViewport(SharedThis(this))
		.IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute());
}

void SDisplayClusterLightCardEditorViewport::PopulateViewportOverlays(TSharedRef<SOverlay> Overlay)
{
	SEditorViewport::PopulateViewportOverlays(Overlay);
}

void SDisplayClusterLightCardEditorViewport::BindCommands()
{
	SEditorViewport::BindCommands();

	{
		const FEditorViewportCommands& Commands = FEditorViewportCommands::Get();

		CommandList->MapAction(
			Commands.TranslateMode,
			FExecuteAction::CreateSP(this, &SDisplayClusterLightCardEditorViewport::SetEditorWidgetMode, FDisplayClusterLightCardEditorWidget::EWidgetMode::WM_Translate),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &SDisplayClusterLightCardEditorViewport::IsEditorWidgetModeSelected, FDisplayClusterLightCardEditorWidget::EWidgetMode::WM_Translate)
		);

		CommandList->MapAction(
			Commands.RotateMode,
			FExecuteAction::CreateSP(this, &SDisplayClusterLightCardEditorViewport::SetEditorWidgetMode, FDisplayClusterLightCardEditorWidget::EWidgetMode::WM_RotateZ),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &SDisplayClusterLightCardEditorViewport::IsEditorWidgetModeSelected, FDisplayClusterLightCardEditorWidget::EWidgetMode::WM_RotateZ)
		);

		CommandList->MapAction(
			Commands.ScaleMode,
			FExecuteAction::CreateSP(this, &SDisplayClusterLightCardEditorViewport::SetEditorWidgetMode, FDisplayClusterLightCardEditorWidget::EWidgetMode::WM_Scale),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &SDisplayClusterLightCardEditorViewport::IsEditorWidgetModeSelected, FDisplayClusterLightCardEditorWidget::EWidgetMode::WM_Scale)
		);

		CommandList->MapAction(
			Commands.CycleTransformGizmos,
			FExecuteAction::CreateSP(this, &SDisplayClusterLightCardEditorViewport::CycleEditorWidgetMode),
			FCanExecuteAction()
		);

		CommandList->UnmapAction(Commands.Top);
		CommandList->UnmapAction(Commands.Bottom);
		CommandList->UnmapAction(Commands.Left);
		CommandList->UnmapAction(Commands.Right);
		CommandList->UnmapAction(Commands.Front);
		CommandList->UnmapAction(Commands.Back);
		CommandList->UnmapAction(Commands.FocusViewportToSelection);
		CommandList->UnmapAction(Commands.FocusAllViewportsToSelection);
	}

	{
		const FDisplayClusterLightCardEditorCommands& Commands = FDisplayClusterLightCardEditorCommands::Get();

		CommandList->MapAction(
			Commands.PerspectiveProjection,
			FExecuteAction::CreateSP(this, &SDisplayClusterLightCardEditorViewport::SetProjectionMode, EDisplayClusterMeshProjectionType::Linear, ELevelViewportType::LVT_Perspective),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &SDisplayClusterLightCardEditorViewport::IsProjectionModeSelected, EDisplayClusterMeshProjectionType::Linear, ELevelViewportType::LVT_Perspective)
		);

		CommandList->MapAction(
			Commands.OrthographicProjection,
			FExecuteAction::CreateSP(this, &SDisplayClusterLightCardEditorViewport::SetProjectionMode, EDisplayClusterMeshProjectionType::Linear, ELevelViewportType::LVT_OrthoFreelook),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &SDisplayClusterLightCardEditorViewport::IsProjectionModeSelected, EDisplayClusterMeshProjectionType::Linear, ELevelViewportType::LVT_OrthoFreelook)
		);

		CommandList->MapAction(
			Commands.AzimuthalProjection,
			FExecuteAction::CreateSP(this, &SDisplayClusterLightCardEditorViewport::SetProjectionMode, EDisplayClusterMeshProjectionType::Azimuthal, ELevelViewportType::LVT_Perspective),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &SDisplayClusterLightCardEditorViewport::IsProjectionModeSelected, EDisplayClusterMeshProjectionType::Azimuthal, ELevelViewportType::LVT_Perspective)
		);

		CommandList->MapAction(
			Commands.UVProjection,
			FExecuteAction::CreateSP(this, &SDisplayClusterLightCardEditorViewport::SetProjectionMode, EDisplayClusterMeshProjectionType::UV, ELevelViewportType::LVT_OrthoFreelook),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &SDisplayClusterLightCardEditorViewport::IsProjectionModeSelected, EDisplayClusterMeshProjectionType::UV, ELevelViewportType::LVT_OrthoFreelook)
		);

		CommandList->MapAction(
			Commands.ViewOrientationTop,
			FExecuteAction::CreateSP(this, &SDisplayClusterLightCardEditorViewport::SetViewDirection, ViewDirectionTop)
		);

		CommandList->MapAction(
			Commands.ViewOrientationBottom,
			FExecuteAction::CreateSP(this, &SDisplayClusterLightCardEditorViewport::SetViewDirection, ViewDirectionBottom)
		);

		CommandList->MapAction(
			Commands.ViewOrientationLeft,
			FExecuteAction::CreateSP(this, &SDisplayClusterLightCardEditorViewport::SetViewDirection, ViewDirectionLeft)
		);

		CommandList->MapAction(
			Commands.ViewOrientationRight,
			FExecuteAction::CreateSP(this, &SDisplayClusterLightCardEditorViewport::SetViewDirection, ViewDirectionRight)
		);

		CommandList->MapAction(
			Commands.ViewOrientationFront,
			FExecuteAction::CreateSP(this, &SDisplayClusterLightCardEditorViewport::SetViewDirection, ViewDirectionFront)
		);

		CommandList->MapAction(
			Commands.ViewOrientationBack,
			FExecuteAction::CreateSP(this, &SDisplayClusterLightCardEditorViewport::SetViewDirection, ViewDirectionBack)
		);

		CommandList->MapAction(
			Commands.ResetCamera,
			FExecuteAction::CreateSP(ViewportClient.Get(), &FDisplayClusterLightCardEditorViewportClient::ResetCamera, false)
		);

		CommandList->MapAction(
			Commands.FrameSelection,
			FExecuteAction::CreateSP(ViewportClient.Get(), &FDisplayClusterLightCardEditorViewportClient::FrameSelection),
			FCanExecuteAction::CreateSP(ViewportClient.Get(), &FDisplayClusterLightCardEditorViewportClient::HasSelection)
		);

		CommandList->MapAction(
			Commands.CycleEditorWidgetCoordinateSystem,
			FExecuteAction::CreateSP(ViewportClient.Get(), &FDisplayClusterLightCardEditorViewportClient::CycleCoordinateSystem)
		);

		CommandList->MapAction(
			Commands.SphericalCoordinateSystem,
			FExecuteAction::CreateSP(ViewportClient.Get(), &FDisplayClusterLightCardEditorViewportClient::SetCoordinateSystem, FDisplayClusterLightCardEditorHelper::ECoordinateSystem::Spherical),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([this]() { return ViewportClient->GetCoordinateSystem() == FDisplayClusterLightCardEditorHelper::ECoordinateSystem::Spherical; })
		);

		CommandList->MapAction(
			Commands.CartesianCoordinateSystem,
			FExecuteAction::CreateSP(ViewportClient.Get(), &FDisplayClusterLightCardEditorViewportClient::SetCoordinateSystem, FDisplayClusterLightCardEditorHelper::ECoordinateSystem::Cartesian),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([this]() { return ViewportClient->GetCoordinateSystem() == FDisplayClusterLightCardEditorHelper::ECoordinateSystem::Cartesian; })
		);

		CommandList->MapAction(
			FDisplayClusterLightCardEditorCommands::Get().DrawLightCard,
			FExecuteAction::CreateSP(this, &SDisplayClusterLightCardEditorViewport::DrawLightCard),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &SDisplayClusterLightCardEditorViewport::IsDrawingLightCard)
		);

		CommandList->MapAction(
			Commands.PasteHere,
			FExecuteAction::CreateSP(this, &SDisplayClusterLightCardEditorViewport::PasteLightCardsHere),
			FCanExecuteAction::CreateSP(this, &SDisplayClusterLightCardEditorViewport::CanPasteLightCardsHere)
		);

		CommandList->MapAction(
			Commands.ToggleAllLabels,
			FExecuteAction::CreateSP(this, &SDisplayClusterLightCardEditorViewport::ToggleLabels),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &SDisplayClusterLightCardEditorViewport::AreLabelsToggled)
		);

		CommandList->MapAction(
			Commands.ToggleIconVisibility,
			FExecuteAction::CreateSP(this, &SDisplayClusterLightCardEditorViewport::ToggleIcons),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &SDisplayClusterLightCardEditorViewport::AreIconsToggled)
		);
	}
}

FReply SDisplayClusterLightCardEditorViewport::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	const TSharedPtr<FDisplayClusterLightCardTemplateDragDropOp> TemplateDragDropOp = DragDropEvent.GetOperationAs<FDisplayClusterLightCardTemplateDragDropOp>();
	if (TemplateDragDropOp.IsValid() && LightCardEditorPtr.IsValid() &&
		LightCardEditorPtr.Pin()->GetActiveRootActor().IsValid() && TemplateDragDropOp->GetTemplate().IsValid())
	{
		TemplateDragDropOp->SetDropAsValid(FText::Format(LOCTEXT("TemplateDragDropOp_LightCardTemplate", "Spawn light card from template {0}"),
			FText::FromString(TemplateDragDropOp->GetTemplate()->GetName())));

		FIntPoint ViewportOrigin, ViewportSize;
		ViewportClient->GetViewportDimensions(ViewportOrigin, ViewportSize);
		const FVector2D MousePos = MyGeometry.AbsoluteToLocal(DragDropEvent.GetScreenSpacePosition()) * MyGeometry.Scale - ViewportOrigin;

		const TArray<UObject*> DroppedObjects;
		bool bDroppedObjectsVisible = true;
		ViewportClient->UpdateDropPreviewActors(MousePos.X, MousePos.Y, DroppedObjects, bDroppedObjectsVisible, nullptr);

		return FReply::Handled();
	}

	return SEditorViewport::OnDragOver(MyGeometry, DragDropEvent);
}

void SDisplayClusterLightCardEditorViewport::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	const TSharedPtr<FDisplayClusterLightCardTemplateDragDropOp> TemplateDragDropOp = DragDropEvent.GetOperationAs<FDisplayClusterLightCardTemplateDragDropOp>();
	if (TemplateDragDropOp.IsValid() && LightCardEditorPtr.IsValid())
	{
		FIntPoint ViewportOrigin, ViewportSize;
		ViewportClient->GetViewportDimensions(ViewportOrigin, ViewportSize);

		const FVector2D MousePos = MyGeometry.AbsoluteToLocal(DragDropEvent.GetScreenSpacePosition()) * MyGeometry.Scale - ViewportOrigin;

		const TWeakObjectPtr<UDisplayClusterLightCardTemplate> LightCardTemplate = TemplateDragDropOp->GetTemplate();

		if (LightCardTemplate.IsValid())
		{
			const TArray<UObject*> DroppedObjects{ LightCardTemplate.Get() };

			TArray<AActor*> TemporaryActors;

			const bool bIsPreview = true;
			ViewportClient->DropObjectsAtCoordinates(MousePos.X, MousePos.Y, DroppedObjects, TemporaryActors, false, bIsPreview, false, nullptr);

			return;
		}
	}

	SEditorViewport::OnDragEnter(MyGeometry, DragDropEvent);
}

void SDisplayClusterLightCardEditorViewport::OnDragLeave(const FDragDropEvent& DragDropEvent)
{
	ViewportClient->DestroyDropPreviewActors();

	const TSharedPtr<FDisplayClusterLightCardTemplateDragDropOp> TemplateDragDropOp = DragDropEvent.GetOperationAs<FDisplayClusterLightCardTemplateDragDropOp>();
	if (TemplateDragDropOp.IsValid())
	{
		TemplateDragDropOp->SetDropAsInvalid();
		return;
	}

	SEditorViewport::OnDragLeave(DragDropEvent);
}

FReply SDisplayClusterLightCardEditorViewport::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	const TSharedPtr<FDisplayClusterLightCardTemplateDragDropOp> TemplateDragDropOp = DragDropEvent.GetOperationAs<FDisplayClusterLightCardTemplateDragDropOp>();
	if (TemplateDragDropOp.IsValid() && TemplateDragDropOp->CanBeDropped() && LightCardEditorPtr.IsValid())
	{
		FIntPoint ViewportOrigin, ViewportSize;
		ViewportClient->GetViewportDimensions(ViewportOrigin, ViewportSize);

		const FVector2D MousePos = MyGeometry.AbsoluteToLocal(DragDropEvent.GetScreenSpacePosition()) * MyGeometry.Scale - ViewportOrigin;

		const TWeakObjectPtr<UDisplayClusterLightCardTemplate> LightCardTemplate = TemplateDragDropOp->GetTemplate();

		if (LightCardTemplate.IsValid())
		{
			const TArray<UObject*> DroppedObjects{ LightCardTemplate.Get() };

			TArray<AActor*> TemporaryActors;
			const bool bSelectActor = true;

			const FScopedTransaction Transaction(LOCTEXT("CreateLightCardFromTemplate", "Create Light Card from Template"));
			ViewportClient->DropObjectsAtCoordinates(MousePos.X, MousePos.Y, DroppedObjects, TemporaryActors, false, false, bSelectActor, nullptr);

			return FReply::Handled();
		}
	}

	return SEditorViewport::OnDrop(MyGeometry, DragDropEvent);
}

TSharedRef<SWidget> SDisplayClusterLightCardEditorViewport::MakeContextMenu()
{
	const bool bInShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bInShouldCloseWindowAfterMenuSelection, CommandList);

	MenuBuilder.BeginSection("Actors", LOCTEXT("ActorsSection", "Actors"));
	{
		MenuBuilder.AddSubMenu(
			LOCTEXT("PlaceActorsSubMenuLabel", "Place Actor"),
			LOCTEXT("PlaceActorsSubMenuToolTip", "Add new actors to the stage"),
			FNewMenuDelegate::CreateSP(this, &SDisplayClusterLightCardEditorViewport::MakePlaceActorsSubMenu)
		);

		MenuBuilder.AddMenuEntry(FDisplayClusterLightCardEditorCommands::Get().RemoveLightCard);
		MenuBuilder.AddMenuEntry(FDisplayClusterLightCardEditorCommands::Get().SaveLightCardTemplate);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("View", LOCTEXT("ViewSection", "View"));
	{
		MenuBuilder.AddMenuEntry(FDisplayClusterLightCardEditorCommands::Get().FrameSelection);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Edit", LOCTEXT("EditSection", "Edit"));
	{
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Cut);
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Copy);
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Paste);
		MenuBuilder.AddMenuEntry(FDisplayClusterLightCardEditorCommands::Get().PasteHere);
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Duplicate);
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SDisplayClusterLightCardEditorViewport::MakePlaceActorsSubMenu(FMenuBuilder& MenuBuilder)
{
	FSlateIcon LightCardIcon = FSlateIconFinder::FindIconForClass(ADisplayClusterLightCardActor::StaticClass());
	FSlateIcon FlagIcon = FSlateIconFinder::FindIcon(TEXT("ClassIcon.DisplayClusterLightCardActor.Flag"));
	FSlateIcon UVLightCardIcon = FSlateIconFinder::FindIcon(TEXT("ClassIcon.DisplayClusterLightCardActor.UVLightCard"));

	bool bIsUVMode = false;
	if (ViewportClient.IsValid())
	{
		bIsUVMode = ViewportClient->GetProjectionMode() == EDisplayClusterMeshProjectionType::UV;
	}

	MenuBuilder.AddMenuEntry(
		FDisplayClusterLightCardEditorCommands::Get().AddNewFlag->GetLabel(),
		FDisplayClusterLightCardEditorCommands::Get().AddNewFlag->GetDescription(),
		FlagIcon,
		FUIAction(FExecuteAction::CreateSP(this, &SDisplayClusterLightCardEditorViewport::AddFlagHere),
			FCanExecuteAction::CreateSP(this, &SDisplayClusterLightCardEditorViewport::CanPlaceActorHere))
	);

	MenuBuilder.AddMenuEntry(
		FDisplayClusterLightCardEditorCommands::Get().AddNewLightCard->GetLabel(),
		FDisplayClusterLightCardEditorCommands::Get().AddNewLightCard->GetDescription(),
		bIsUVMode ? UVLightCardIcon : LightCardIcon,
		FUIAction(FExecuteAction::CreateSP(this, &SDisplayClusterLightCardEditorViewport::AddLightCardHere),
			FCanExecuteAction::CreateSP(this, &SDisplayClusterLightCardEditorViewport::CanPlaceActorHere))
	);

	TSet<UClass*> StageActorClasses = UE::DisplayClusterLightCardEditorUtils::GetAllStageActorClasses();
	for (UClass* Class : StageActorClasses)
	{
		if (Class == ADisplayClusterLightCardActor::StaticClass())
		{
			continue;
		}

		FText Label = Class->GetDisplayNameText();
		FSlateIcon StageActorIcon = FSlateIconFinder::FindIconForClass(Class);
		MenuBuilder.AddMenuEntry(
			Label,
			LOCTEXT("AddStageActorHeader", "Add a stage actor to the scene"),
			StageActorIcon,
			FUIAction(FExecuteAction::CreateSP(this, &SDisplayClusterLightCardEditorViewport::AddStageActorHere, Class),
				FCanExecuteAction::CreateSP(this, &SDisplayClusterLightCardEditorViewport::CanPlaceActorHere, Class))
		);
	}
}

void SDisplayClusterLightCardEditorViewport::SetEditorWidgetMode(FDisplayClusterLightCardEditorWidget::EWidgetMode InWidgetMode)
{
	if (ViewportClient.IsValid())
	{
		ViewportClient->SetEditorWidgetMode(InWidgetMode);
	}
}

bool SDisplayClusterLightCardEditorViewport::IsEditorWidgetModeSelected(FDisplayClusterLightCardEditorWidget::EWidgetMode InWidgetMode) const
{
	if (ViewportClient.IsValid())
	{
		return ViewportClient->GetEditorWidgetMode() == InWidgetMode;
	}

	return false;
}

void SDisplayClusterLightCardEditorViewport::DrawLightCard()
{
	if (!ViewportClient.IsValid())
	{
		return;
	}

	if (IsDrawingLightCard())
	{
		ViewportClient->ExitDrawingLightCardMode();
	}
	else
	{
		ViewportClient->EnterDrawingLightCardMode();
	}
}

void SDisplayClusterLightCardEditorViewport::CycleEditorWidgetMode()
{
	int32 WidgetModeAsInt = ViewportClient->GetEditorWidgetMode();
	WidgetModeAsInt = (WidgetModeAsInt + 1) % FDisplayClusterLightCardEditorWidget::EWidgetMode::WM_Max;
	SetEditorWidgetMode((FDisplayClusterLightCardEditorWidget::EWidgetMode)WidgetModeAsInt);
}

void SDisplayClusterLightCardEditorViewport::SetProjectionMode(EDisplayClusterMeshProjectionType InProjectionMode, ELevelViewportType InViewportType)
{
	if (ViewportClient.IsValid())
	{
		ViewportClient->SetProjectionMode(InProjectionMode, InViewportType);
	}
}

bool SDisplayClusterLightCardEditorViewport::IsProjectionModeSelected(EDisplayClusterMeshProjectionType InProjectionMode, ELevelViewportType ViewportType) const
{
	if (ViewportClient.IsValid())
	{
		return ViewportClient->GetProjectionMode() == InProjectionMode && ViewportClient->GetRenderViewportType() == ViewportType;
	}

	return false;
}

void SDisplayClusterLightCardEditorViewport::SetViewDirection(FVector InViewDirection)
{
	if (ViewportClient.IsValid())
	{
		ViewportClient->SetViewRotation(InViewDirection.Rotation());
	}
}

bool SDisplayClusterLightCardEditorViewport::IsDrawingLightCard() const
{
	if (ViewportClient.IsValid())
	{
		return ViewportClient->GetInputMode() == FDisplayClusterLightCardEditorViewportClient::EInputMode::DrawingLightCard;
	}

	return false;
}

void SDisplayClusterLightCardEditorViewport::PasteLightCardsHere()
{
	if (LightCardEditorPtr.IsValid())
	{
		const TSharedPtr<FScopedTransaction> Transaction = MakeShared<FScopedTransaction>(LOCTEXT("PasteActorsHereTransactionMessage", "Paste Actors Here"));
		TArray<AActor*> PastedActors = LightCardEditorPtr.Pin()->PasteActors();
		if (ViewportClient.IsValid() && PastedActors.Num() > 0)
		{
			AddActorHereTransactions.Add(Transaction);
			ViewportClient->GetOnNextSceneRefresh().AddLambda([this, Transaction, PastedActors = MoveTemp(PastedActors)]()
				{
					TArray<FDisplayClusterWeakStageActorPtr> PastedStageActors;
					Algo::TransformIf(PastedActors, PastedStageActors, [](const AActor* InItem)
						{
							return IsValid(InItem);
						},
						[](const AActor* InItem) -> FDisplayClusterWeakStageActorPtr
						{
							return InItem;
						});
					ViewportClient->MoveActorsToPixel(FIntPoint(PasteHerePos.X, PasteHerePos.Y), PastedStageActors);
					AddActorHereTransactions.Remove(Transaction);
				});
		}
	}
}

bool SDisplayClusterLightCardEditorViewport::CanPasteLightCardsHere() const
{
	if (LightCardEditorPtr.IsValid())
	{
		return LightCardEditorPtr.Pin()->CanPasteActors();
	}

	return false;
}

void SDisplayClusterLightCardEditorViewport::AddLightCardHere()
{
	if (LightCardEditorPtr.IsValid())
	{
		const TSharedPtr<FScopedTransaction> Transaction = MakeShared<FScopedTransaction>(LOCTEXT("AddLightCardHereTransactionMessage", "Add Light Card Here"));
		ADisplayClusterLightCardActor* NewLightCard = LightCardEditorPtr.Pin()->AddNewLightCard();
		if (ViewportClient.IsValid() && NewLightCard)
		{
			AddActorHereTransactions.Add(Transaction);
			ViewportClient->GetOnNextSceneRefresh().AddLambda([this, Transaction, NewLightCard]()
				{
					ViewportClient->MoveActorsToPixel(FIntPoint(PasteHerePos.X, PasteHerePos.Y), { NewLightCard });
					AddActorHereTransactions.Remove(Transaction);
				});
		}
	}
}

void SDisplayClusterLightCardEditorViewport::AddFlagHere()
{
	if (LightCardEditorPtr.IsValid())
	{
		const TSharedPtr<FScopedTransaction> Transaction = MakeShared<FScopedTransaction>(LOCTEXT("AddFlagHereTransactionMessage", "Add Flag Here"));
		ADisplayClusterLightCardActor* NewFlag = LightCardEditorPtr.Pin()->AddNewFlag();
		if (ViewportClient.IsValid() && NewFlag)
		{
			AddActorHereTransactions.Add(Transaction);
			ViewportClient->GetOnNextSceneRefresh().AddLambda([this, Transaction, NewFlag]()
				{
					ViewportClient->MoveActorsToPixel(FIntPoint(PasteHerePos.X, PasteHerePos.Y), { NewFlag });
					AddActorHereTransactions.Remove(Transaction);
				});
		}
	}
}

void SDisplayClusterLightCardEditorViewport::AddStageActorHere(UClass* InClass)
{
	if (LightCardEditorPtr.IsValid())
	{
		const TSharedPtr<FScopedTransaction> Transaction = MakeShared<FScopedTransaction>(LOCTEXT("AddStageActorHereTransactionMessage", "Add Stage Actor Here"));
		AActor* NewActor = LightCardEditorPtr.Pin()->AddNewDynamic(InClass);
		if (ViewportClient.IsValid() && NewActor)
		{
			AddActorHereTransactions.Add(Transaction);
			ViewportClient->GetOnNextSceneRefresh().AddLambda([this, Transaction, NewActor]()
				{
					ViewportClient->MoveActorsToPixel(FIntPoint(PasteHerePos.X, PasteHerePos.Y), { NewActor });
					AddActorHereTransactions.Remove(Transaction);
				});
		}
	}
}

bool SDisplayClusterLightCardEditorViewport::CanPlaceActorHere() const
{
	return CanPlaceActorHere(nullptr);
}

bool SDisplayClusterLightCardEditorViewport::CanPlaceActorHere(UClass* Class) const
{
	return LightCardEditorPtr.IsValid() && LightCardEditorPtr.Pin()->CanAddNewActor(Class);
}

void SDisplayClusterLightCardEditorViewport::ToggleLabels()
{
	if (LightCardEditorPtr.IsValid())
	{
		return LightCardEditorPtr.Pin()->ToggleLightCardLabels();
	}
}

bool SDisplayClusterLightCardEditorViewport::AreLabelsToggled() const
{
	return LightCardEditorPtr.IsValid() && LightCardEditorPtr.Pin()->ShouldShowLightCardLabels();
}

void SDisplayClusterLightCardEditorViewport::ToggleIcons()
{
	if (LightCardEditorPtr.IsValid())
	{
		LightCardEditorPtr.Pin()->ShowIcons(!AreIconsToggled());
	}
}

bool SDisplayClusterLightCardEditorViewport::AreIconsToggled() const
{
	return LightCardEditorPtr.IsValid() && LightCardEditorPtr.Pin()->ShouldShowIcons();
}

#undef LOCTEXT_NAMESPACE
