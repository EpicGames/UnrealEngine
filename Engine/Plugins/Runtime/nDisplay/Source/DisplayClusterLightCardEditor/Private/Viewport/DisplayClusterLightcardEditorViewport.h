// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AdvancedPreviewScene.h"
#include "SCommonEditorViewportToolbarBase.h"
#include "SEditorViewport.h"

#include "DisplayClusterMeshProjectionRenderer.h"
#include "DisplayClusterLightCardEditorWidget.h"

class FDisplayClusterLightCardEditor;
class ADisplayClusterRootActor;
class FDisplayClusterLightCardEditorViewportClient;
class FScopedTransaction;

/**
 * Slate widget which renders our view client.
 */
class SDisplayClusterLightCardEditorViewport : public SEditorViewport, public ICommonEditorViewportToolbarInfoProvider
{
public:
	static const FVector ViewDirectionTop;
	static const FVector ViewDirectionBottom;
	static const FVector ViewDirectionLeft;
	static const FVector ViewDirectionRight;
	static const FVector ViewDirectionFront;
	static const FVector ViewDirectionBack;

public:
	SLATE_BEGIN_ARGS(SDisplayClusterLightCardEditorViewport) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<FDisplayClusterLightCardEditor> InLightCardEditor, TSharedPtr<class FUICommandList> InCommandList);
	~SDisplayClusterLightCardEditorViewport();

	// ICommonEditorViewportToolbarInfoProvider
	virtual TSharedRef<SEditorViewport> GetViewportWidget() override;
	virtual TSharedPtr<FExtender> GetExtenders() const override;
	virtual void OnFloatingButtonClicked() override;
	// ~ICommonEditorViewportToolbarInfoProvider

	// SWidget interface
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	// ~SWidget

	TWeakPtr<FDisplayClusterLightCardEditor> GetLightCardEditor() { return LightCardEditorPtr;}

	void SetRootActor(ADisplayClusterRootActor* NewRootActor);

	/** Return the active root actor for this viewport. */
	TWeakObjectPtr<ADisplayClusterRootActor> GetRootActor() const;
	
	TSharedRef<FDisplayClusterLightCardEditorViewportClient> GetLightCardEditorViewportClient() const { return ViewportClient.ToSharedRef(); }

	/** Summons the context menu within this viewport at the mouse's current position */
	void SummonContextMenu();

private:
	// SEditorViewport
	virtual TSharedRef<FEditorViewportClient> MakeEditorViewportClient() override;
	virtual TSharedPtr<SWidget> BuildViewportToolbar() override;
	virtual void PopulateViewportOverlays(TSharedRef<SOverlay> Overlay) override;
	virtual void BindCommands() override;
	virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	// ~SEditorViewport

	/** Create the popup context menu */
	TSharedRef<SWidget> MakeContextMenu();

	/** Create the submenu for placing actors */
	void MakePlaceActorsSubMenu(FMenuBuilder& MenuBuilder);

	/** Sets the widget mode for the editor viewport to the specified mode */
	void SetEditorWidgetMode(FDisplayClusterLightCardEditorWidget::EWidgetMode InWidgetMode);

	/** Returns true if the widget mode of the editor viewport matches the specified mode */
	bool IsEditorWidgetModeSelected(FDisplayClusterLightCardEditorWidget::EWidgetMode InWidgetMode) const;

	/** Cycles the widget mode of the editor viewport */
	void CycleEditorWidgetMode();

	/** Sets the projection mode and viewport type of the editor viewport */
	void SetProjectionMode(EDisplayClusterMeshProjectionType InProjectionMode, ELevelViewportType InViewportType);

	/** Returns true if the editor viewport's projection mode and viewport type match the specified parameters */
	bool IsProjectionModeSelected(EDisplayClusterMeshProjectionType InProjectionMode, ELevelViewportType InViewportType) const;

	/** Sets the direction of the editor viewport */
	void SetViewDirection(FVector InViewDirection);

	/** Handles the Draw LC button being pressed. */
	void DrawLightCard();

	/** Returns true if we are in light card drawing mode */
	bool IsDrawingLightCard() const;

	/** Pastes any light cards in the clipboard and moves them to the cached mouse position */
	void PasteLightCardsHere();

	/** Determines if there are any light cards in the clipboard that can be pasted */
	bool CanPasteLightCardsHere() const;

	/** Adds a new light card and moves it to the cached mouse position */
	void AddLightCardHere();

	/** Adds a new flag and moves it to the cached mouse position */
	void AddFlagHere();

	/** Adds a new stage actor of the specified class and moves it to the cached mouse position */
	void AddStageActorHere(UClass* InClass);

	/** If placing actors is allowed */
	bool CanPlaceActorHere() const;
	bool CanPlaceActorHere(UClass* Class) const;

	/** Toggles light card labels on or off */
	void ToggleLabels();

	/** If light card labels are currently toggled */
	bool AreLabelsToggled() const;

	/** Toggles all icons on or off */
	void ToggleIcons();

	/** If icons are currently toggled */
	bool AreIconsToggled() const;
	
private:
	/** The preview scene to use by the viewport client to manage the preview 3D world */
	TSharedPtr<FPreviewScene> PreviewScene;

	/** Open transactions when using an 'Add Actor Here' method since they occur over multiple frames */
	TSet<TSharedPtr<FScopedTransaction>> AddActorHereTransactions;
	
	/** Level viewport client */
	TSharedPtr<FDisplayClusterLightCardEditorViewportClient> ViewportClient;
	TWeakPtr<FDisplayClusterLightCardEditor> LightCardEditorPtr;

	/** The cached mouse position used for the Paste Here command */
	FVector2D PasteHerePos;
};