// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/DisplayClusterLabelComponent.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class UDisplayClusterStageActorTemplate;

/**
 * Display Cluster Light Card Editor module interface
 */
class IDisplayClusterLightCardEditor : public IModuleInterface
{
public:
	static constexpr const TCHAR* ModuleName = TEXT("DisplayClusterLightCardEditor");

public:
	virtual ~IDisplayClusterLightCardEditor() = default;

	/**
	* Singleton-like access to this module's interface.  This is just for convenience!
	* Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	*
	* @return Returns singleton instance, loading the module on demand if needed
	*/
	static inline IDisplayClusterLightCardEditor& Get()
	{
		return FModuleManager::GetModuleChecked<IDisplayClusterLightCardEditor>(ModuleName);
	}

	/**
	* Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	*
	* @return True if the module is loaded and ready to use
	*/
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded(ModuleName);
	}

	struct FLabelArgs : FDisplayClusterLabelConfiguration
	{
		// TODO: Specify type (Light Card vs CCR)
	};
	
	/**
	 * Show the light card labels for the given root actor and save current settings.
	 *
	 * @param InArgs The arguments for displaying labels.
	 */
	virtual void ShowLabels(const FLabelArgs& InArgs) = 0;

	/**
	 * Get the default template to use when creating a new light card
	 */
	virtual UDisplayClusterStageActorTemplate* GetDefaultLightCardTemplate() const = 0;
};