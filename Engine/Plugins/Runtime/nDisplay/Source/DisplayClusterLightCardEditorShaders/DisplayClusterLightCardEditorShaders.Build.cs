// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class DisplayClusterLightCardEditorShaders : ModuleRules
{
	public DisplayClusterLightCardEditorShaders(ReadOnlyTargetRules ROTargetRules) : base(ROTargetRules)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"RenderCore",
				"Renderer",
				"RHI",
				"Projects"
			});

		PrivateIncludePaths.AddRange(
			new string[] {
			});

		ShortName = "DCLCEShaders";
	}
}
