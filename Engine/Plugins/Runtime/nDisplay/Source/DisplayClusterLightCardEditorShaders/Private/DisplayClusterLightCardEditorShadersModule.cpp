// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterLightCardEditorShadersModule.h"

#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "ShaderCore.h"

#define NDISPLAY_SHADERS_MAP TEXT("/Plugin/nDisplay")

void FDisplayClusterLightCardEditorShadersModule::StartupModule()
{
	if (!AllShaderSourceDirectoryMappings().Contains(NDISPLAY_SHADERS_MAP))
	{
		FString PluginShaderDir = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("nDisplay"))->GetBaseDir(), TEXT("Shaders"));
		AddShaderSourceDirectoryMapping(NDISPLAY_SHADERS_MAP, PluginShaderDir);
	}
}

void FDisplayClusterLightCardEditorShadersModule::ShutdownModule()
{
}

IMPLEMENT_MODULE(FDisplayClusterLightCardEditorShadersModule, DisplayClusterLightCardEditorShaders);
