// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDisplayClusterLightCardEditorShaders.h"
#include "UObject/WeakObjectPtr.h"

/**
 * Display Cluster editor module
 */
class FDisplayClusterLightCardEditorShadersModule :
	public IDisplayClusterLightCardEditorShaders
{
public:
	//~ IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};
