// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterMeshProjectionRenderer.h"

#include "PrimitiveSceneInfo.h"
#include "PrimitiveSceneProxy.h"
#include "MeshPassProcessor.h"
#include "MeshMaterialShader.h"
#include "MeshPassProcessor.inl"
#include "Shader.h"
#include "CanvasTypes.h"
#include "EngineModule.h"
#include "SceneManagement.h"
#include "SceneRendererInterface.h"
#include "SceneViewExtension.h"
#include "ScreenPass.h"
#include "Components/PrimitiveComponent.h"
#include "InstanceCulling/InstanceCullingContext.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "ScreenPass.h"
#include "ShaderCompilerCore.h"

namespace UE::DisplayClusterMeshProjectionRenderer
{
	/**
	* Whether this primitive component should be rendered in this view.
	* Each 'View' belongs to a 'ViewFamily' that belongs to the 'World'.
	* UE can only render one world for one "View".
	*/
	static inline bool ShouldRenderPrimitiveComponentForView(const UPrimitiveComponent* InPrimitiveComponent, const FSceneView* InView)
	{
		const FSceneViewFamily* ViewFamily = InView ? InView->Family : nullptr;
		if (const FSceneInterface* SceneInterface = ViewFamily ? ViewFamily->Scene : nullptr)
		{
			// Get primitive info from the scene by primitive ID.
			if (FPrimitiveSceneInfo* PrimitiveSceneInfo = SceneInterface->GetPrimitiveSceneInfo(InPrimitiveComponent->GetPrimitiveSceneId()))
			{
				// The scene primitive identifier is simply a numeric value that can be the same in multiple worlds.
				// To check if the primitive component is the same instance, let's compare references to FPrimitiveSceneProxy.
				if (PrimitiveSceneInfo->Proxy == InPrimitiveComponent->GetSceneProxy())
				{
					return true;
				}
			}

			// This is a component belonging to a different world; skip it for the current rendering pass.
			return false;
		}

		return true;
	}
};

//////////////////////////////////////////////////////////////////////////
// Mesh Pass Processor

BEGIN_SHADER_PARAMETER_STRUCT(FMeshProjectionPassParameters, )
	SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FInstanceCullingGlobalUniforms, InstanceCulling)
	SHADER_PARAMETER_STRUCT_INCLUDE(FInstanceCullingDrawParams, InstanceCullingDrawParams)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

class FMeshProjectionShaderElementData : public FMeshMaterialShaderElementData
{
public:
	FDisplayClusterMeshProjectionTypeSettings ProjectionTypeSettings;
	FMatrix44f NormalCorrectionMatrix;
};

/** Base class for all mesh pass processors used by the projection renderer.
 * 
 *  NOTE: Due to C++ dependent naming, any class derived from this base class must access this class's
 *  member variables and functions through this-> or FMeshProjectionPassProcessorBase<...>::, to ensure
 *  that the compiler can properly locate the variables and functions once the class templates have been generated
 */
template<typename VertexType, typename PixelType, typename ShaderElementDataType = FMeshProjectionShaderElementData>
class FMeshProjectionPassProcessorBase : public FMeshPassProcessor
{
public:
	FMeshProjectionPassProcessorBase(const FScene* InScene,
		const FSceneView* InView,
		FMeshPassDrawListContext* InDrawListContext,
		const FDisplayClusterMeshProjectionRenderSettings& InRenderSettings)
		: FMeshPassProcessor(EMeshPass::Num, InScene, GMaxRHIFeatureLevel, InView, InDrawListContext)
		, ProjectionTypeSettings(InRenderSettings.ProjectionTypeSettings)
		, NormalCorrectionMatrix(InRenderSettings.NormalCorrectionMatrix)
		, StencilValue(0)
	{
		DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<>::GetRHI());
		DrawRenderState.SetBlendState(TStaticBlendState<>::GetRHI());
	}

	virtual void AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId = -1) override final
	{
		const FMaterial* Material;
		const FMaterialRenderProxy* MaterialRenderProxy;
		GetMeshBatchMaterial(MeshBatch, Material, MaterialRenderProxy);

		check(Material);
		check(MaterialRenderProxy);

		if (!CanDrawMeshBatch(MeshBatch, PrimitiveSceneProxy, Material))
		{
			return;
		}

		const FVertexFactory* VertexFactory = MeshBatch.VertexFactory;

		TMeshProcessorShaders<VertexType, PixelType> PassShaders;

		FMaterialShaderTypes ShaderTypes;
		ShaderTypes.AddShaderType<VertexType>();
		ShaderTypes.AddShaderType<PixelType>();

		FMaterialShaders Shaders;
		if (!Material->TryGetShaders(ShaderTypes, VertexFactory->GetType(), Shaders))
		{
			return;
		}

		if (!Shaders.TryGetVertexShader(PassShaders.VertexShader) || !Shaders.TryGetPixelShader(PassShaders.PixelShader))
		{
			// Always check if shaders are available on the current platform and hardware
			return;
		}

		FMeshDrawingPolicyOverrideSettings OverrideSettings = GetMeshOverrideSettings(MeshBatch);
		ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(*Material, OverrideSettings);
		ERasterizerCullMode MeshCullMode = ComputeMeshCullMode(*Material, OverrideSettings);

		ShaderElementDataType ShaderElementData = CreateShaderElementData(MeshBatch, PrimitiveSceneProxy, StaticMeshId);
		FMeshDrawCommandSortKey SortKey = CreateMeshSortKey(MeshBatch, PrimitiveSceneProxy, *Material, PassShaders.VertexShader.GetShader(), PassShaders.PixelShader.GetShader());

		DrawRenderState.SetStencilRef(StencilValue);

		BuildMeshDrawCommands(
			MeshBatch,
			BatchElementMask,
			PrimitiveSceneProxy,
			*MaterialRenderProxy,
			*Material,
			DrawRenderState,
			PassShaders,
			MeshFillMode,
			MeshCullMode,
			SortKey,
			EMeshPassFeatures::Default,
			ShaderElementData);
	}

	void SetStencilValue(uint32 InStencilValue)
	{
		StencilValue = InStencilValue;
	}

protected:
	virtual bool CanDrawMeshBatch(const FMeshBatch& RESTRICT MeshBatch, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, const FMaterial* Material)
	{
		return true;
	}

	virtual void GetMeshBatchMaterial(const FMeshBatch& RESTRICT MeshBatch, const FMaterial*& OutMaterial, const FMaterialRenderProxy*& OutMaterialRenderProxy)
	{
		const FMaterialRenderProxy* FallbackMaterialRenderProxy = nullptr;
		OutMaterial = &MeshBatch.MaterialRenderProxy->GetMaterialWithFallback(FeatureLevel, FallbackMaterialRenderProxy);
		OutMaterialRenderProxy = FallbackMaterialRenderProxy ? FallbackMaterialRenderProxy : MeshBatch.MaterialRenderProxy;
	}

	virtual FMeshPassProcessor::FMeshDrawingPolicyOverrideSettings GetMeshOverrideSettings(const FMeshBatch& RESTRICT MeshBatch)
	{
		return ComputeMeshOverrideSettings(MeshBatch);
	}

	virtual ShaderElementDataType CreateShaderElementData(const FMeshBatch& RESTRICT MeshBatch, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId)
	{
		ShaderElementDataType ShaderElementData;
		ShaderElementData.InitializeMeshMaterialData(ViewIfDynamicMeshCommand, PrimitiveSceneProxy, MeshBatch, StaticMeshId, false);
		ShaderElementData.ProjectionTypeSettings = ProjectionTypeSettings;
		ShaderElementData.NormalCorrectionMatrix = NormalCorrectionMatrix;

		return ShaderElementData;
	}

	virtual FMeshDrawCommandSortKey CreateMeshSortKey(const FMeshBatch& RESTRICT MeshBatch,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		const FMaterial& Material,
		const FMeshMaterialShader* VertexShader,
		const FMeshMaterialShader* PixelShader)
	{
		return CalculateMeshStaticSortKey(VertexShader, PixelShader);
	}

protected:
	FMeshPassProcessorRenderState DrawRenderState;
	FDisplayClusterMeshProjectionTypeSettings ProjectionTypeSettings;
	FMatrix44f NormalCorrectionMatrix;
	uint32 StencilValue;
};


//////////////////////////////////////////////////////////////////////////
// Base Render Pass

template<EDisplayClusterMeshProjectionType ProjectionType>
class FMeshProjectionVS : public FMeshMaterialShader
{
public:
	DECLARE_SHADER_TYPE(FMeshProjectionVS, MeshMaterial);

	FMeshProjectionVS() { }
	FMeshProjectionVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FMeshMaterialShader(Initializer)
	{
		PassUniformBuffer.Bind(Initializer.ParameterMap, FSceneTextureUniformParameters::FTypeInfo::GetStructMetadata()->GetShaderVariableName());
		UVProjectionIndex.Bind(Initializer.ParameterMap, TEXT("ProjectionParameters_UVProjectionIndex"), SPF_Optional);
		UVProjectionPlaneSize.Bind(Initializer.ParameterMap, TEXT("ProjectionParameters_UVProjectionPlaneSize"), SPF_Optional);
		UVProjectionPlaneDistance.Bind(Initializer.ParameterMap, TEXT("ProjectionParameters_UVProjectionPlaneDistance"), SPF_Optional);
		UVProjectionPlaneOffset.Bind(Initializer.ParameterMap, TEXT("ProjectionParameters_UVProjectionPlaneOffset"), SPF_Optional);
	}

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		return Parameters.VertexFactoryType == FindVertexFactoryType(TEXT("FLocalVertexFactory"));
	}

	void GetShaderBindings(
		const FScene* Scene,
		ERHIFeatureLevel::Type FeatureLevel,
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material,
		const FMeshProjectionShaderElementData& ShaderElementData,
		FMeshDrawSingleShaderBindings& ShaderBindings) const
	{
		FMeshMaterialShader::GetShaderBindings(Scene, FeatureLevel, PrimitiveSceneProxy, MaterialRenderProxy, Material, ShaderElementData, ShaderBindings);

		ShaderBindings.Add(UVProjectionIndex, ShaderElementData.ProjectionTypeSettings.UVProjectionIndex);
		ShaderBindings.Add(UVProjectionPlaneSize, ShaderElementData.ProjectionTypeSettings.UVProjectionPlaneSize);
		ShaderBindings.Add(UVProjectionPlaneDistance, ShaderElementData.ProjectionTypeSettings.UVProjectionPlaneDistance);
		ShaderBindings.Add(UVProjectionPlaneOffset, FVector3f(ShaderElementData.ProjectionTypeSettings.UVProjectionPlaneOffset));
	}

private:
	LAYOUT_FIELD(FShaderParameter, UVProjectionIndex);
	LAYOUT_FIELD(FShaderParameter, UVProjectionPlaneSize);
	LAYOUT_FIELD(FShaderParameter, UVProjectionPlaneDistance);
	LAYOUT_FIELD(FShaderParameter, UVProjectionPlaneOffset);
};

using FMeshPerspectiveProjectionVS = FMeshProjectionVS<EDisplayClusterMeshProjectionType::Linear>;
using FMeshAzimuthalProjectionVS = FMeshProjectionVS<EDisplayClusterMeshProjectionType::Azimuthal>;
using FMeshUVProjectionVS = FMeshProjectionVS<EDisplayClusterMeshProjectionType::UV>;

IMPLEMENT_MATERIAL_SHADER_TYPE(template<>, FMeshPerspectiveProjectionVS, TEXT("/Plugin/nDisplay/Private/MeshProjectionShaders.usf"), TEXT("PerspectiveVS"), SF_Vertex);
IMPLEMENT_MATERIAL_SHADER_TYPE(template<>, FMeshAzimuthalProjectionVS, TEXT("/Plugin/nDisplay/Private/MeshProjectionShaders.usf"), TEXT("AzimuthalVS"), SF_Vertex);
IMPLEMENT_MATERIAL_SHADER_TYPE(template<>, FMeshUVProjectionVS, TEXT("/Plugin/nDisplay/Private/MeshProjectionShaders.usf"), TEXT("UVProjectionVS"), SF_Vertex);

template<EDisplayClusterMeshProjectionOutput OutputType>
class FMeshProjectionPS : public FMeshMaterialShader
{
public:
	DECLARE_SHADER_TYPE(FMeshProjectionPS, MeshMaterial);

	FMeshProjectionPS() { }
	FMeshProjectionPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FMeshMaterialShader(Initializer)
	{
		PassUniformBuffer.Bind(Initializer.ParameterMap, FSceneTextureUniformParameters::FTypeInfo::GetStructMetadata()->GetShaderVariableName());
		NormalCorrectionMatrix.Bind(Initializer.ParameterMap, TEXT("NormalCorrectionMatrix"), SPF_Optional);
	}

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		return Parameters.VertexFactoryType == FindVertexFactoryType(TEXT("FLocalVertexFactory"));
	}

	void GetShaderBindings(
		const FScene* Scene,
		ERHIFeatureLevel::Type FeatureLevel,
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material,
		const FMeshProjectionShaderElementData& ShaderElementData,
		FMeshDrawSingleShaderBindings& ShaderBindings) const
	{
		FMeshMaterialShader::GetShaderBindings(Scene, FeatureLevel, PrimitiveSceneProxy, MaterialRenderProxy, Material, ShaderElementData, ShaderBindings);

		ShaderBindings.Add(NormalCorrectionMatrix, ShaderElementData.NormalCorrectionMatrix);
	}

private:
	LAYOUT_FIELD(FShaderParameter, NormalCorrectionMatrix);
};

using FMeshProjectionColorPS = FMeshProjectionPS<EDisplayClusterMeshProjectionOutput::Color>;
using FMeshProjectionNormalPS = FMeshProjectionPS<EDisplayClusterMeshProjectionOutput::Normals>;

IMPLEMENT_MATERIAL_SHADER_TYPE(template<>, FMeshProjectionColorPS, TEXT("/Plugin/nDisplay/Private/MeshProjectionShaders.usf"), TEXT("MainPS"), SF_Pixel);
IMPLEMENT_MATERIAL_SHADER_TYPE(template<>, FMeshProjectionNormalPS, TEXT("/Plugin/nDisplay/Private/MeshProjectionShaders.usf"), TEXT("NormalPS"), SF_Pixel);

FMeshProjectionPassParameters* CreateProjectionPassParameters(FRDGBuilder& GraphBuilder, const FSceneView* View, const FDisplayClusterMeshProjectionRenderSettings& RenderSettings)
{
	FMeshProjectionPassParameters* PassParameters = GraphBuilder.AllocParameters<FMeshProjectionPassParameters>();
	PassParameters->View = View->ViewUniformBuffer;
	PassParameters->Scene = GetSceneUniformBufferRef(GraphBuilder, *View);
	PassParameters->InstanceCulling = FInstanceCullingContext::CreateDummyInstanceCullingUniformBuffer(GraphBuilder);

	return PassParameters;
}

template<EDisplayClusterMeshProjectionType ProjectionType, EDisplayClusterMeshProjectionOutput OutputType = EDisplayClusterMeshProjectionOutput::Color>
class FMeshProjectionPassProcessor : public FMeshProjectionPassProcessorBase<FMeshProjectionVS<ProjectionType>, FMeshProjectionPS<OutputType>>
{
	using Super = FMeshProjectionPassProcessorBase<FMeshProjectionVS<ProjectionType>, FMeshProjectionPS<OutputType>>;

public:
	FMeshProjectionPassProcessor(const FScene* InScene,
		const FSceneView* InView,
		FMeshPassDrawListContext* InDrawListContext,
		const FDisplayClusterMeshProjectionRenderSettings& InRenderSettings,
		bool bIsTranslucencyPass = false)
		: Super(InScene, InView, InDrawListContext, InRenderSettings)
		, bTranslucencyPass(bIsTranslucencyPass)
		, bIgnoreTranslucency(false)
	{
		if (bTranslucencyPass)
		{
			this->DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<false, CF_DepthNearOrEqual>::GetRHI());
			this->DrawRenderState.SetBlendState(TStaticBlendState<CW_RGBA, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One>::GetRHI());
		}
		else
		{
			this->DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<true, CF_DepthNearOrEqual, true, CF_Always, SO_Keep, SO_Keep, SO_Replace>::GetRHI());
			this->DrawRenderState.SetBlendState(TStaticBlendStateWriteMask<CW_RGBA, CW_RGBA, CW_RGBA, CW_RGBA>::GetRHI());
		}
	}

	void SetIgnoreTranslucency(bool bInIgnoreTranslucency)
	{
		bIgnoreTranslucency = bInIgnoreTranslucency;
	}

protected:
	virtual bool CanDrawMeshBatch(const FMeshBatch& RESTRICT MeshBatch, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, const FMaterial* Material) override
	{
		const bool bIsTranslucent = IsTranslucentBlendMode(*Material);

		if (bTranslucencyPass)
		{
			return bIsTranslucent;
		}
		else
		{
			return bIgnoreTranslucency || !bIsTranslucent;
		}
	}

	virtual FMeshPassProcessor::FMeshDrawingPolicyOverrideSettings GetMeshOverrideSettings(const FMeshBatch& RESTRICT MeshBatch) override
	{
		FMeshPassProcessor::FMeshDrawingPolicyOverrideSettings OverrideSettings = FMeshPassProcessor::ComputeMeshOverrideSettings(MeshBatch);

		// Don't cull backfacing geometry if the projection mode is UV, since there is no guarantee that the UV coordinates are in the correct winding order
		if (ProjectionType == EDisplayClusterMeshProjectionType::UV)
		{
			OverrideSettings.MeshOverrideFlags |= EDrawingPolicyOverrideFlags::TwoSided;
		}

		return OverrideSettings;
	}

	virtual FMeshDrawCommandSortKey CreateMeshSortKey(const FMeshBatch& RESTRICT MeshBatch,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		const FMaterial& Material,
		const FMeshMaterialShader* VertexShader,
		const FMeshMaterialShader* PixelShader) override
	{
		FMeshDrawCommandSortKey SortKey = FMeshDrawCommandSortKey::Default;

		if (bTranslucencyPass)
		{
			uint16 SortKeyPriority = 0;
			float Distance = 0.0f;

			if (PrimitiveSceneProxy)
			{
				const FPrimitiveSceneInfo* PrimitiveSceneInfo = PrimitiveSceneProxy->GetPrimitiveSceneInfo();
				SortKeyPriority = (uint16)((int32)PrimitiveSceneInfo->Proxy->GetTranslucencySortPriority() - (int32)SHRT_MIN);

				// Use the standard sort by distance method for translucent objects
				const float DistanceOffset = PrimitiveSceneInfo->Proxy->GetTranslucencySortDistanceOffset();
				const FVector BoundsOrigin = PrimitiveSceneProxy->GetBounds().Origin;
				const FVector ViewOrigin = this->ViewIfDynamicMeshCommand->ViewMatrices.GetViewOrigin();
				Distance = (BoundsOrigin - ViewOrigin).Size() + DistanceOffset;
			}

			SortKey.Translucent.MeshIdInPrimitive = MeshBatch.MeshIdInPrimitive;
			SortKey.Translucent.Priority = SortKeyPriority;
			SortKey.Translucent.Distance = (uint32)~BitInvertIfNegativeFloat(*(uint32*)&Distance);
		}
		else
		{
			SortKey.BasePass.VertexShaderHash = (VertexShader ? VertexShader->GetSortKey() : 0) & 0xFFFF;
			SortKey.BasePass.PixelShaderHash = PixelShader ? PixelShader->GetSortKey() : 0;
			SortKey.BasePass.Masked = IsMaskedBlendMode(Material) ? 1 : 0;
		}

		return SortKey;
	}

private:
	/** Inverts the bits of the floating point number if that number is negative */
	uint32 BitInvertIfNegativeFloat(uint32 FloatBit)
	{
		unsigned Mask = -int32(FloatBit >> 31) | 0x80000000;
		return FloatBit ^ Mask;
	}

private:
	bool bTranslucencyPass;
	bool bIgnoreTranslucency;
};


//////////////////////////////////////////////////////////////////////////
// Hit Proxy Render Pass

class FMeshProjectionHitProxyShaderElementData : public FMeshProjectionShaderElementData
{
public:
	FHitProxyId BatchHitProxyId;
};

class FMeshProjectionHitProxyPS : public FMeshMaterialShader
{
	DECLARE_SHADER_TYPE(FMeshProjectionHitProxyPS, MeshMaterial);

public:
	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		// Only compile the hit proxy shader on editor platforms
		return EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData)
			// and only compile for default materials or materials that are masked.
			&&
			(  Parameters.MaterialParameters.bIsSpecialEngineMaterial
			|| !Parameters.MaterialParameters.bWritesEveryPixel
			|| Parameters.MaterialParameters.bMaterialMayModifyMeshPosition
			|| Parameters.MaterialParameters.bIsTwoSided);
	}

	FMeshProjectionHitProxyPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer):
		FMeshMaterialShader(Initializer)
	{
		HitProxyId.Bind(Initializer.ParameterMap, TEXT("HitProxyId"), SPF_Optional);
	}

	FMeshProjectionHitProxyPS() {}

	void GetShaderBindings(
		const FScene* Scene,
		ERHIFeatureLevel::Type FeatureLevel,
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material,
		const FMeshProjectionHitProxyShaderElementData& ShaderElementData,
		FMeshDrawSingleShaderBindings& ShaderBindings) const
	{
		FMeshMaterialShader::GetShaderBindings(Scene, FeatureLevel, PrimitiveSceneProxy, MaterialRenderProxy, Material, ShaderElementData, ShaderBindings);

		ShaderBindings.Add(HitProxyId, ShaderElementData.BatchHitProxyId.GetColor().ReinterpretAsLinear());
	}

private:
	LAYOUT_FIELD(FShaderParameter, HitProxyId)
};

IMPLEMENT_MATERIAL_SHADER_TYPE(, FMeshProjectionHitProxyPS, TEXT("/Plugin/nDisplay/Private/MeshProjectionHitProxy.usf"), TEXT("Main") ,SF_Pixel);

template<EDisplayClusterMeshProjectionType ProjectionType> 
class FMeshProjectionHitProxyPassProcessor : public FMeshProjectionPassProcessorBase<FMeshProjectionVS<ProjectionType>, FMeshProjectionHitProxyPS, FMeshProjectionHitProxyShaderElementData>
{
	using Super = FMeshProjectionPassProcessorBase<FMeshProjectionVS<ProjectionType>, FMeshProjectionHitProxyPS, FMeshProjectionHitProxyShaderElementData>;

public:
	FMeshProjectionHitProxyPassProcessor(const FScene* InScene,
		const FSceneView* InView,
		FMeshPassDrawListContext* InDrawListContext,
		const FDisplayClusterMeshProjectionRenderSettings& InRenderSettings)
		: Super(InScene, InView, InDrawListContext, InRenderSettings)
	{
	}

protected:
	virtual bool CanDrawMeshBatch(const FMeshBatch& RESTRICT MeshBatch, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, const FMaterial* Material) override
	{
		const bool bDrawMeshBatch = MeshBatch.bUseForMaterial
			&& MeshBatch.BatchHitProxyId != FHitProxyId::InvisibleHitProxyId
			&& MeshBatch.bSelectable
			&& PrimitiveSceneProxy
			&& PrimitiveSceneProxy->IsSelectable();

		return bDrawMeshBatch;
	}

	virtual void GetMeshBatchMaterial(const FMeshBatch& RESTRICT MeshBatch, const FMaterial*& OutMaterial, const FMaterialRenderProxy*& OutMaterialRenderProxy) override
	{
		OutMaterialRenderProxy = MeshBatch.MaterialRenderProxy;
		OutMaterial = OutMaterialRenderProxy->GetMaterialNoFallback(this->FeatureLevel);
		if (OutMaterial && OutMaterial->GetRenderingThreadShaderMap())
		{
			if (OutMaterial->WritesEveryPixel() && !OutMaterial->IsTwoSided() && !OutMaterial->MaterialModifiesMeshPosition_RenderThread())
			{
				// Default material doesn't handle masked, and doesn't have the correct bIsTwoSided setting.
				OutMaterialRenderProxy = UMaterial::GetDefaultMaterial(MD_Surface)->GetRenderProxy();
				OutMaterial = OutMaterialRenderProxy->GetMaterialNoFallback(this->FeatureLevel);
			}
		}
		else
		{
			OutMaterialRenderProxy = UMaterial::GetDefaultMaterial(MD_Surface)->GetRenderProxy();
			OutMaterial = OutMaterialRenderProxy->GetMaterialNoFallback(this->FeatureLevel);
		}
	}

	virtual FMeshProjectionHitProxyShaderElementData CreateShaderElementData(const FMeshBatch& RESTRICT MeshBatch, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId) override
	{
		FMeshProjectionHitProxyShaderElementData ShaderElementData;
		ShaderElementData.InitializeMeshMaterialData(this->ViewIfDynamicMeshCommand, PrimitiveSceneProxy, MeshBatch, StaticMeshId, false);
		ShaderElementData.ProjectionTypeSettings = this->ProjectionTypeSettings;
		ShaderElementData.BatchHitProxyId = MeshBatch.BatchHitProxyId;

		return ShaderElementData;
	}
};

//////////////////////////////////////////////////////////////////////////
// Normals Render Pass

class FMeshProjectionNormalsCreateRWTexturesCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FMeshProjectionNormalsCreateRWTexturesCS);
	SHADER_USE_PARAMETER_STRUCT(FMeshProjectionNormalsCreateRWTexturesCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneColor)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneDepth)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, SceneStencil)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, RWColorOut)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float>, RWDepthOut)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<uint>, RWStencilOut)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<uint>, RWBlurStencilOut)
		SHADER_PARAMETER(FMatrix44f, NormalCorrectionMatrix)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
	}
};

IMPLEMENT_GLOBAL_SHADER(FMeshProjectionNormalsCreateRWTexturesCS, "/Plugin/nDisplay/Private/MeshProjectionNormalSmoothing.usf", "CreateRWTexturesCS", SF_Compute);

#define FILTER_KERNEL_SIZE 25

enum EMeshProjectionFilterType
{
	DepthDilate,
	Dilate,
	Blur
};

BEGIN_SHADER_PARAMETER_STRUCT(FMeshProjectionNormalsFilterParameters, )
	SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
	SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Input)
	SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, SceneStencil)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, RWColorIn)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, RWColorOut)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float>, RWDepthIn)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float>, RWDepthOut)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<uint>, RWStencilIn)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<uint>, RWStencilOut)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<uint>, RWBlurStencilIn)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<uint>, RWBlurStencilOut)
	SHADER_PARAMETER_SCALAR_ARRAY(float, SpatialKernel, [FILTER_KERNEL_SIZE])
	SHADER_PARAMETER_SCALAR_ARRAY(float, InteriorSpatialKernel, [FILTER_KERNEL_SIZE])
	SHADER_PARAMETER(FVector2f, SampleDirection)
	SHADER_PARAMETER(FVector2f, SampleOffsetScale)
	SHADER_PARAMETER(uint32, bCopyToBlurStencil)
END_SHADER_PARAMETER_STRUCT()

template<EMeshProjectionFilterType FilterType>
class FMeshProjectionNormalsFilterCS : public FGlobalShader
{
public:
	using FParameters = FMeshProjectionNormalsFilterParameters;

	DECLARE_GLOBAL_SHADER(FMeshProjectionNormalsFilterCS);

	FMeshProjectionNormalsFilterCS() { }
	FMeshProjectionNormalsFilterCS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{
		BindForLegacyShaderParameters<FParameters>(this, Initializer.PermutationId, Initializer.ParameterMap, true);
	}

	static inline const FShaderParametersMetadata* GetRootParametersMetadata() { return FParameters::FTypeInfo::GetStructMetadata(); }

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
	}
};

using FMeshProjectionNormalsDepthDilationCS = FMeshProjectionNormalsFilterCS<EMeshProjectionFilterType::DepthDilate>;
using FMeshProjectionNormalsDilationCS = FMeshProjectionNormalsFilterCS<EMeshProjectionFilterType::Dilate>;
using FMeshProjectionNormalsBlurCS = FMeshProjectionNormalsFilterCS<EMeshProjectionFilterType::Blur>;

IMPLEMENT_SHADER_TYPE(template<>, FMeshProjectionNormalsDepthDilationCS, TEXT("/Plugin/nDisplay/Private/MeshProjectionNormalSmoothing.usf"), TEXT("DepthDilationCS"), SF_Compute);
IMPLEMENT_SHADER_TYPE(template<>, FMeshProjectionNormalsDilationCS, TEXT("/Plugin/nDisplay/Private/MeshProjectionNormalSmoothing.usf"), TEXT("DilationCS"), SF_Compute);
IMPLEMENT_SHADER_TYPE(template<>, FMeshProjectionNormalsBlurCS, TEXT("/Plugin/nDisplay/Private/MeshProjectionNormalSmoothing.usf"), TEXT("BlurCS"), SF_Compute);

class FMeshProjectionNormalsOutputPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FMeshProjectionNormalsOutputPS);
	SHADER_USE_PARAMETER_STRUCT(FMeshProjectionNormalsOutputPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Input)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, RWColorIn)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float>, RWDepthIn)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
	}
};

IMPLEMENT_GLOBAL_SHADER(FMeshProjectionNormalsOutputPS, "/Plugin/nDisplay/Private/MeshProjectionNormalSmoothing.usf", "OutputNormalMapPS", SF_Pixel);

//////////////////////////////////////////////////////////////////////////
// Selection Outline Render Pass

class FMeshProjectionSelectionOutlinePS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FMeshProjectionSelectionOutlinePS);
	SHADER_USE_PARAMETER_STRUCT(FMeshProjectionSelectionOutlinePS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Color)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Depth)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ColorTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, ColorSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, DepthTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, DepthSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, EditorPrimitivesDepth)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, EditorPrimitivesStencil)
		SHADER_PARAMETER(FVector3f, OutlineColor)
		SHADER_PARAMETER(float, SelectionHighlightIntensity)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		// Compile the selection outline shader only on editor platforms
		return EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);
	}
};

IMPLEMENT_GLOBAL_SHADER(FMeshProjectionSelectionOutlinePS, "/Plugin/nDisplay/Private/MeshProjectionSelectionOutline.usf", "Main", SF_Pixel);

template<EDisplayClusterMeshProjectionType ProjectionType> 
class FMeshProjectionSelectionPassProcessor : public FMeshProjectionPassProcessorBase<FMeshProjectionVS<ProjectionType>, FMeshProjectionColorPS>
{
	using Super = FMeshProjectionPassProcessorBase<FMeshProjectionVS<ProjectionType>, FMeshProjectionColorPS>;

public:
	FMeshProjectionSelectionPassProcessor(const FScene* InScene,
		const FSceneView* InView,
		FMeshPassDrawListContext* InDrawListContext,
		const FDisplayClusterMeshProjectionRenderSettings& InRenderSettings)
		: Super(InScene, InView, InDrawListContext, InRenderSettings)
	{
		this->DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<true, CF_DepthNearOrEqual, true, CF_Always, SO_Keep, SO_Keep, SO_Replace>::GetRHI());
		this->DrawRenderState.SetBlendState(TStaticBlendStateWriteMask<CW_NONE, CW_NONE, CW_NONE, CW_NONE>::GetRHI());

		this->StencilValue = 1;
	}

protected:
	virtual bool CanDrawMeshBatch(const FMeshBatch& RESTRICT MeshBatch, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, const FMaterial* Material) override
	{
		const bool bDrawMeshBatch = MeshBatch.bUseForMaterial
			&& MeshBatch.bUseSelectionOutline
			&& PrimitiveSceneProxy
			&& PrimitiveSceneProxy->WantsSelectionOutline()
			&& (PrimitiveSceneProxy->IsSelected() || PrimitiveSceneProxy->IsHovered());

		return bDrawMeshBatch;
	}

	virtual void GetMeshBatchMaterial(const FMeshBatch& RESTRICT MeshBatch, const FMaterial*& OutMaterial, const FMaterialRenderProxy*& OutMaterialRenderProxy) override
	{
		OutMaterialRenderProxy = MeshBatch.MaterialRenderProxy;
		OutMaterial = OutMaterialRenderProxy->GetMaterialNoFallback(this->FeatureLevel);
		if (OutMaterial && OutMaterial->GetRenderingThreadShaderMap())
		{
			if (OutMaterial->WritesEveryPixel() && !OutMaterial->IsTwoSided() && !OutMaterial->MaterialModifiesMeshPosition_RenderThread())
			{
				// Default material doesn't handle masked, and doesn't have the correct bIsTwoSided setting.
				OutMaterialRenderProxy = UMaterial::GetDefaultMaterial(MD_Surface)->GetRenderProxy();
				OutMaterial = OutMaterialRenderProxy->GetMaterialNoFallback(this->FeatureLevel);
			}
		}
		else
		{
			OutMaterialRenderProxy = UMaterial::GetDefaultMaterial(MD_Surface)->GetRenderProxy();
			OutMaterial = OutMaterialRenderProxy->GetMaterialNoFallback(this->FeatureLevel);
		}
	}

	virtual FMeshPassProcessor::FMeshDrawingPolicyOverrideSettings GetMeshOverrideSettings(const FMeshBatch& RESTRICT MeshBatch) override
	{
		FMeshPassProcessor::FMeshDrawingPolicyOverrideSettings OverrideSettings = FMeshPassProcessor::ComputeMeshOverrideSettings(MeshBatch);
		OverrideSettings.MeshOverrideFlags |= EDrawingPolicyOverrideFlags::TwoSided;

		return OverrideSettings;
	}
};

//////////////////////////////////////////////////////////////////////////
// FDisplayClusterMeshProjectionRenderer

/** A version of FSimpleElementCollector that has a hit proxy consumer, allowing support for adding hit proxies to the rendered elements */
class FMeshProjectionElementCollector : public FSimpleElementCollector
{
public:
	FMeshProjectionElementCollector(FHitProxyConsumer* InHitProxyConsumer)
		: FSimpleElementCollector()
		, HitProxyConsumer(InHitProxyConsumer)
	{ }

	virtual void SetHitProxy(HHitProxy* HitProxy) override
	{
		FSimpleElementCollector::SetHitProxy(HitProxy);

		if (HitProxyConsumer && HitProxy)
		{
			HitProxyConsumer->AddHitProxy(HitProxy);
		}
	}

private:
	FHitProxyConsumer* HitProxyConsumer;
};

bool FDisplayClusterMeshProjectionPrimitiveFilter::ShouldRenderPrimitive(const UPrimitiveComponent* InPrimitiveComponent) const
{
	if (ShouldRenderPrimitiveDelegate.IsBound())
	{
		return ShouldRenderPrimitiveDelegate.Execute(InPrimitiveComponent);
	}

	return true;
}

bool FDisplayClusterMeshProjectionPrimitiveFilter::ShouldApplyProjection(const UPrimitiveComponent* InPrimitiveComponent) const
{
	if (ShouldApplyProjectionDelegate.IsBound())
	{
		return ShouldApplyProjectionDelegate.Execute(InPrimitiveComponent);
	}

	return true;
}

FVector FDisplayClusterMeshProjectionTransform::ProjectPosition(const FVector& WorldPosition) const
{
	FVector ProjectedPosition(WorldPosition);

	if (Projection != EDisplayClusterMeshProjectionType::Linear)
	{
		const FVector ViewPos = ViewMatrix.TransformPosition(WorldPosition);
		const FVector ProjectedViewPos = FDisplayClusterMeshProjectionRenderer::ProjectViewPosition(ViewPos, Projection);
		ProjectedPosition = InvViewMatrix.TransformPosition(ProjectedViewPos);
	}

	return ProjectedPosition;
}

FVector FDisplayClusterMeshProjectionTransform::UnprojectPosition(const FVector& ProjectedPosition) const
{
	FVector UnprojectedPosition(ProjectedPosition);

	if (Projection != EDisplayClusterMeshProjectionType::Linear)
	{
		const FVector ViewPos = ViewMatrix.TransformPosition(ProjectedPosition);
		const FVector ProjectedViewPos = FDisplayClusterMeshProjectionRenderer::UnprojectViewPosition(ViewPos, Projection);
		UnprojectedPosition = InvViewMatrix.TransformPosition(ProjectedViewPos);
	}

	return UnprojectedPosition;
}

FDisplayClusterMeshProjectionRenderer::~FDisplayClusterMeshProjectionRenderer()
{
#if WITH_EDITOR
	for (TWeakObjectPtr<UPrimitiveComponent> PrimitiveComponent : PrimitiveComponents)
	{
		if (!PrimitiveComponent.IsValid())
		{
			continue;
		}

		// Unbind this since it will still have a raw pointer to this renderer
		if (PrimitiveComponent->SelectionOverrideDelegate.IsBoundToObject(this))
		{
			PrimitiveComponent->SelectionOverrideDelegate.Unbind();
		}
	}
#endif
}

FVector FDisplayClusterMeshProjectionRenderer::ProjectViewPosition(const FVector& ViewPosition, EDisplayClusterMeshProjectionType  ProjectionType)
{
	FVector ProjectedViewPosition(ViewPosition);

	if (ProjectionType == EDisplayClusterMeshProjectionType::Azimuthal)
	{
		const float Rho = ViewPosition.Length();
		const FVector UnitViewPos = ViewPosition.GetSafeNormal();
		const FVector2D PolarCoords = FVector2D(FMath::Acos(UnitViewPos.Z), FMath::Atan2(UnitViewPos.Y, UnitViewPos.X));
		const FVector PlanePos = FVector(PolarCoords.X * FMath::Cos(PolarCoords.Y), PolarCoords.X * FMath::Sin(PolarCoords.Y), 1);

		ProjectedViewPosition = PlanePos.GetSafeNormal() * Rho;
	}

	return ProjectedViewPosition;
}

FVector FDisplayClusterMeshProjectionRenderer::UnprojectViewPosition(const FVector& ProjectedViewPosition, EDisplayClusterMeshProjectionType  ProjectionType)
{
	FVector ViewPosition(ProjectedViewPosition);

	if (ProjectionType == EDisplayClusterMeshProjectionType::Azimuthal)
	{
		const float Rho = ProjectedViewPosition.Length();
		const FVector UnitViewPos = ProjectedViewPosition.GetSafeNormal();

		const FVector PlanePos = UnitViewPos / UnitViewPos.Z;
		const FVector2D PolarCoords = FVector2D(FMath::Sqrt(PlanePos.X * PlanePos.X + PlanePos.Y * PlanePos.Y), FMath::Atan2(PlanePos.Y, PlanePos.X));
		ViewPosition = FVector(FMath::Sin(PolarCoords.X) * FMath::Cos(PolarCoords.Y), FMath::Sin(PolarCoords.X) * FMath::Sin(PolarCoords.Y), FMath::Cos(PolarCoords.X)) * Rho;
	}

	return ViewPosition;
}

void FDisplayClusterMeshProjectionRenderer::AddActor(AActor* Actor)
{
	AddActor(Actor, [](const UPrimitiveComponent* PrimitiveComponent)
	{
		return !PrimitiveComponent->bHiddenInGame;
	});
}

void FDisplayClusterMeshProjectionRenderer::AddActor(AActor* Actor, const TFunctionRef<bool(const UPrimitiveComponent*)>& PrimitiveFilter)
{
	Actor->ForEachComponent<UPrimitiveComponent>(true, [&](UPrimitiveComponent* PrimitiveComponent)
	{
		if (PrimitiveFilter(PrimitiveComponent))
		{
			PrimitiveComponents.Add(PrimitiveComponent);

#if WITH_EDITOR
			// If the ActorSelectedDelegate is bound, that indicates that the renderer will handle rendering the selection outline itself, and the component's selection override delegate
			// can be rebound; otherwise, don't touch the delegate as it may be being used by an editor viewport (such as the nDisplay config editor)
			if (ActorSelectedDelegate.IsBound())
			{
				PrimitiveComponent->SelectionOverrideDelegate = UPrimitiveComponent::FSelectionOverride::CreateRaw(this, &FDisplayClusterMeshProjectionRenderer::IsPrimitiveComponentSelected);
			}
#endif
		}
	});
}

void FDisplayClusterMeshProjectionRenderer::RemoveActor(AActor* Actor)
{
	TArray<TWeakObjectPtr<UPrimitiveComponent>> ComponentsToRemove = PrimitiveComponents.FilterByPredicate([Actor](const TWeakObjectPtr<UPrimitiveComponent>& PrimitiveComponent)
	{
		return !PrimitiveComponent.IsValid() || PrimitiveComponent->GetOwner() == Actor;
	});

	for (const TWeakObjectPtr<UPrimitiveComponent>& PrimitiveComponent : ComponentsToRemove)
	{
#if WITH_EDITOR
		if (PrimitiveComponent.IsValid() && PrimitiveComponent->SelectionOverrideDelegate.IsBoundToObject(this))
		{
			PrimitiveComponent->SelectionOverrideDelegate.Unbind();
		}
#endif

		PrimitiveComponents.Remove(PrimitiveComponent);
	}
}

void FDisplayClusterMeshProjectionRenderer::ClearScene()
{
	for (const TWeakObjectPtr<UPrimitiveComponent>& PrimitiveComponent : PrimitiveComponents)
	{
#if WITH_EDITOR
		if (PrimitiveComponent.IsValid() && PrimitiveComponent->SelectionOverrideDelegate.IsBoundToObject(this))
		{
			PrimitiveComponent->SelectionOverrideDelegate.Unbind();
		}
#endif
	}

	PrimitiveComponents.Empty();
}

void FDisplayClusterMeshProjectionRenderer::Render(FCanvas* Canvas, FSceneInterface* Scene, const FDisplayClusterMeshProjectionRenderSettings& RenderSettings)
{
	Canvas->Flush_GameThread();
	FRenderTarget* RenderTarget = Canvas->GetRenderTarget();
	const bool bIsHitTesting = Canvas->IsHitTesting();
	FHitProxyConsumer* HitProxyConsumer = Canvas->GetHitProxyConsumer();

	UE::RenderCommandPipe::FSyncScope SyncScope;

	ENQUEUE_RENDER_COMMAND(FDrawProjectedMeshes)(
		[RenderTarget, Scene, RenderSettings, bIsHitTesting, HitProxyConsumer, this](FRHICommandListImmediate& RHICmdList)
		{
			FRDGBuilder GraphBuilder(RHICmdList);

			FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
				RenderTarget,
				Scene,
				RenderSettings.EngineShowFlags)
				.SetTime(FGameTime::GetTimeSinceAppStart()));

			if (Scene)
			{
				Scene->IncrementFrameNumber();
				ViewFamily.FrameNumber = Scene->GetFrameNumber();
			}
			else
			{
				ViewFamily.FrameNumber = GFrameNumber;
			}

			ViewFamily.EngineShowFlags.SetHitProxies(bIsHitTesting);

			FScenePrimitiveRenderingContextScopeHelper ScenePrimitiveRenderingContextScopeHelper(GetRendererModule().BeginScenePrimitiveRendering(GraphBuilder, &ViewFamily));

			FSceneViewInitOptions NewInitOptions(RenderSettings.ViewInitOptions);
			NewInitOptions.ViewFamily = &ViewFamily;

			GetRendererModule().CreateAndInitSingleView(RHICmdList, &ViewFamily, &NewInitOptions);
			const FSceneView* View = ViewFamily.Views[0];

			FRDGTextureRef OutputTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(RenderTarget->GetRenderTargetTexture(), TEXT("ViewRenderTarget")));
			FRenderTargetBinding OutputRenderTargetBinding(OutputTexture, ERenderTargetLoadAction::ELoad);
			FMeshProjectionElementCollector ElementCollector(HitProxyConsumer);

			switch (RenderSettings.RenderType)
			{
			case EDisplayClusterMeshProjectionOutput::Color:
				if (ViewFamily.EngineShowFlags.HitProxies)
				{
					RenderHitProxyOutput(GraphBuilder, View, RenderSettings, OutputRenderTargetBinding, HitProxyConsumer);
				}
				else
				{
					RenderColorOutput(GraphBuilder, View, RenderSettings, OutputRenderTargetBinding);
				}

				if (RenderSimpleElementsDelegate.IsBound())
				{
					RenderSimpleElementsDelegate.Execute(View, &ElementCollector);
					AddSimpleElementPass(GraphBuilder, View, OutputRenderTargetBinding, ElementCollector);
				}
				break;

			case EDisplayClusterMeshProjectionOutput::Normals:
				RenderNormalsOutput(GraphBuilder, View, RenderSettings, OutputRenderTargetBinding);
				break;
			}

			GraphBuilder.Execute();
		});
}

void FDisplayClusterMeshProjectionRenderer::RenderColorOutput(FRDGBuilder& GraphBuilder,
	const FSceneView* View,
	const FDisplayClusterMeshProjectionRenderSettings& RenderSettings,
	FRenderTargetBinding& OutputRenderTargetBinding)
{
	FRDGTextureRef ColorTexture = GraphBuilder.CreateTexture(OutputRenderTargetBinding.GetTexture()->Desc, TEXT("DisplayClusterMeshProjection.ColorTexture"));

	const FRDGTextureDesc DepthDesc = FRDGTextureDesc::Create2D(OutputRenderTargetBinding.GetTexture()->Desc.Extent, PF_DepthStencil, FClearValueBinding::DepthFar, TexCreate_DepthStencilTargetable | TexCreate_ShaderResource);
	FRDGTextureRef DepthTexture = GraphBuilder.CreateTexture(DepthDesc, TEXT("DisplayClusterMeshProjection.DepthTexture"));

	FRenderTargetBinding ColorRenderTargetBinding(ColorTexture, ERenderTargetLoadAction::EClear);
	FDepthStencilBinding DepthStencilBinding(DepthTexture, ERenderTargetLoadAction::EClear, ERenderTargetLoadAction::ENoAction, FExclusiveDepthStencil::DepthWrite_StencilWrite);

	AddBaseRenderPass(GraphBuilder, View, RenderSettings, ColorRenderTargetBinding, DepthStencilBinding);

	ColorRenderTargetBinding.SetLoadAction(ERenderTargetLoadAction::ELoad);
	DepthStencilBinding.SetDepthLoadAction(ERenderTargetLoadAction::ELoad);

	AddTranslucencyRenderPass(GraphBuilder, View, RenderSettings, ColorRenderTargetBinding, DepthStencilBinding);

#if WITH_EDITOR
	if (RenderSettings.EngineShowFlags.SelectionOutline)
	{
		const FRDGTextureDesc SelectionDepthDesc = FRDGTextureDesc::Create2D(OutputRenderTargetBinding.GetTexture()->Desc.Extent, PF_DepthStencil, FClearValueBinding::DepthFar, TexCreate_DepthStencilTargetable | TexCreate_ShaderResource);
		FRDGTextureRef SelectionDepthTexture = GraphBuilder.CreateTexture(SelectionDepthDesc, TEXT("DisplayClusterMeshProjection.SelectionDepthTexture"));
		FDepthStencilBinding SelectionDepthStencilBinding(SelectionDepthTexture, ERenderTargetLoadAction::EClear, ERenderTargetLoadAction::EClear, FExclusiveDepthStencil::DepthWrite_StencilWrite);

		AddSelectionDepthRenderPass(GraphBuilder, View, RenderSettings, SelectionDepthStencilBinding);
		AddSelectionOutlineScreenPass(GraphBuilder, View, OutputRenderTargetBinding, ColorTexture, DepthTexture, SelectionDepthTexture);
	}
	else
#endif
	// Copy the scene color to the output render target
	{
		FCopyRectPS::FParameters* ScreenPassParameters = GraphBuilder.AllocParameters<FCopyRectPS::FParameters>();
		ScreenPassParameters->InputTexture = ColorTexture;
		ScreenPassParameters->InputSampler = TStaticSamplerState<>::GetRHI();
		ScreenPassParameters->RenderTargets[0] = OutputRenderTargetBinding;

		FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(View->FeatureLevel);
		TShaderMapRef<FScreenPassVS> ScreenPassVS(GlobalShaderMap);
		TShaderMapRef<FCopyRectPS> CopyPixelShader(GlobalShaderMap);
		if (!ScreenPassVS.IsValid() || !CopyPixelShader.IsValid())
		{
			// Always check if shaders are available on the current platform and hardware
			return;
		}

		FRHIBlendState* DefaultBlendState = FScreenPassPipelineState::FDefaultBlendState::GetRHI();
		const FScreenPassTextureViewport RegionViewport(OutputRenderTargetBinding.GetTexture());

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("MeshProjectionRenderer::CopyColorTexture"),
			ScreenPassParameters,
			ERDGPassFlags::Raster,
			[View, ScreenPassVS, CopyPixelShader, RegionViewport, ScreenPassParameters, DefaultBlendState](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			DrawScreenPass(
				RHICmdList,
				*View,
				RegionViewport,
				RegionViewport,
				FScreenPassPipelineState(ScreenPassVS, CopyPixelShader, DefaultBlendState),
				EScreenPassDrawFlags::None,
				[&](FRHICommandList&)
			{
				SetShaderParameters(RHICmdList, CopyPixelShader, CopyPixelShader.GetPixelShader(), *ScreenPassParameters);
			});
		});
	}
}

void FDisplayClusterMeshProjectionRenderer::RenderHitProxyOutput(FRDGBuilder& GraphBuilder,
	const FSceneView* View,
	const FDisplayClusterMeshProjectionRenderSettings& RenderSettings,
	FRenderTargetBinding& OutputRenderTargetBinding,
	FHitProxyConsumer* HitProxyConsumer)
{
	FRDGTextureDesc Desc(FRDGTextureDesc::Create2D(OutputRenderTargetBinding.GetTexture()->Desc.Extent, PF_B8G8R8A8, FClearValueBinding::Black, TexCreate_RenderTargetable | TexCreate_ShaderResource));
	FRDGTextureRef HitProxyTexture = GraphBuilder.CreateTexture(Desc, TEXT("DisplayClusterMeshProjection.HitProxyTexture"));

	const FRDGTextureDesc DepthDesc = FRDGTextureDesc::Create2D(OutputRenderTargetBinding.GetTexture()->Desc.Extent, PF_DepthStencil, FClearValueBinding::DepthFar, TexCreate_DepthStencilTargetable | TexCreate_ShaderResource);
	FRDGTextureRef HitProxyDepthTexture = GraphBuilder.CreateTexture(DepthDesc, TEXT("DisplayClusterMeshProjection.HitProxyDepthTexture"));

	FRenderTargetBinding HitProxyRenderTargetBinding(HitProxyTexture, ERenderTargetLoadAction::EClear);
	FDepthStencilBinding HitProxyDepthStencilBinding(HitProxyDepthTexture, ERenderTargetLoadAction::EClear, ERenderTargetLoadAction::EClear, FExclusiveDepthStencil::DepthWrite_StencilWrite);

	AddHitProxyRenderPass(GraphBuilder, View, RenderSettings, HitProxyRenderTargetBinding, HitProxyDepthStencilBinding);

	// Copy the hit proxy buffer to the viewport family's render target
	{
		FCopyRectPS::FParameters* ScreenPassParameters = GraphBuilder.AllocParameters<FCopyRectPS::FParameters>();
		ScreenPassParameters->InputTexture = HitProxyTexture;
		ScreenPassParameters->InputSampler = TStaticSamplerState<>::GetRHI();
		ScreenPassParameters->RenderTargets[0] = OutputRenderTargetBinding;

		FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(View->FeatureLevel);
		TShaderMapRef<FScreenPassVS> ScreenPassVS(GlobalShaderMap);
		TShaderMapRef<FCopyRectPS> CopyPixelShader(GlobalShaderMap);
		if (!ScreenPassVS.IsValid() || !CopyPixelShader.IsValid())
		{
			// Always check if shaders are available on the current platform and hardware
			return;
		}

		FRHIBlendState* DefaultBlendState = FScreenPassPipelineState::FDefaultBlendState::GetRHI();
		const FScreenPassTextureViewport RegionViewport(OutputRenderTargetBinding.GetTexture());

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("MeshProjectionRenderer::CopyHitProxyTexture"),
			ScreenPassParameters,
			ERDGPassFlags::Raster,
			[View, ScreenPassVS, CopyPixelShader, RegionViewport, ScreenPassParameters, DefaultBlendState](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			DrawScreenPass(
				RHICmdList,
				*View,
				RegionViewport,
				RegionViewport,
				FScreenPassPipelineState(ScreenPassVS, CopyPixelShader, DefaultBlendState),
				EScreenPassDrawFlags::None,
				[&](FRHICommandList&)
			{
				SetShaderParameters(RHICmdList, CopyPixelShader, CopyPixelShader.GetPixelShader(), *ScreenPassParameters);
			});
		});
	}
}

void FDisplayClusterMeshProjectionRenderer::RenderNormalsOutput(FRDGBuilder& GraphBuilder,
	const FSceneView* View,
	const FDisplayClusterMeshProjectionRenderSettings& RenderSettings,
	FRenderTargetBinding& OutputRenderTargetBinding)
{
	const FRDGTextureDesc NormalsDesc(FRDGTextureDesc::Create2D(OutputRenderTargetBinding.GetTexture()->Desc.Extent, OutputRenderTargetBinding.GetTexture()->Desc.Format, FClearValueBinding::Black, TexCreate_RenderTargetable | TexCreate_ShaderResource));
	FRDGTextureRef NormalsTexture = GraphBuilder.CreateTexture(NormalsDesc, TEXT("DisplayClusterMeshProjection.NormalsTexture"));

	const FRDGTextureDesc DepthDesc = FRDGTextureDesc::Create2D(OutputRenderTargetBinding.GetTexture()->Desc.Extent, PF_DepthStencil, FClearValueBinding::DepthFar, TexCreate_DepthStencilTargetable | TexCreate_ShaderResource);
	FRDGTextureRef NormalsDepthTexture = GraphBuilder.CreateTexture(DepthDesc, TEXT("DisplayClusterMeshProjection.NormalsDepthTexture"));

	FRenderTargetBinding NormalsRenderTargetBinding(NormalsTexture, ERenderTargetLoadAction::EClear);
	FDepthStencilBinding NormalsDepthStencilBinding(NormalsDepthTexture, ERenderTargetLoadAction::EClear, ERenderTargetLoadAction::EClear, FExclusiveDepthStencil::DepthWrite_StencilWrite);

	AddNormalsRenderPass(GraphBuilder, View, RenderSettings, NormalsRenderTargetBinding, NormalsDepthStencilBinding);
	AddNormalsFilterPass(GraphBuilder, View, OutputRenderTargetBinding, NormalsTexture, NormalsDepthTexture, RenderSettings.NormalCorrectionMatrix);
}

// Helper macros that generate appropriate switch cases for each projection type the mesh projection renderer supports, allowing new projection types to be easily added
#define PROJECTION_TYPE_CASE(FuncName, ProjectionType, ...) case ProjectionType: \
	FuncName<ProjectionType>(__VA_ARGS__); \
	break;

#define SWITCH_ON_PROJECTION_TYPE(FuncName, ProjectionType, ...) switch (ProjectionType) \
	{ \
	PROJECTION_TYPE_CASE(FuncName, EDisplayClusterMeshProjectionType::Azimuthal, __VA_ARGS__) \
	PROJECTION_TYPE_CASE(FuncName, EDisplayClusterMeshProjectionType::Linear, __VA_ARGS__) \
	PROJECTION_TYPE_CASE(FuncName, EDisplayClusterMeshProjectionType::UV, __VA_ARGS__) \
	}

void FDisplayClusterMeshProjectionRenderer::AddBaseRenderPass(FRDGBuilder& GraphBuilder,
	const FSceneView* View,
	const FDisplayClusterMeshProjectionRenderSettings& RenderSettings,
	FRenderTargetBinding& OutputRenderTargetBinding,
	FDepthStencilBinding& OutputDepthStencilBinding)
{
	FMeshProjectionPassParameters* MeshPassParameters = CreateProjectionPassParameters(GraphBuilder, View, RenderSettings);
	MeshPassParameters->RenderTargets[0] = OutputRenderTargetBinding;
	MeshPassParameters->RenderTargets.DepthStencil = OutputDepthStencilBinding;

	GraphBuilder.AddPass(RDG_EVENT_NAME("MeshProjectionRenderer::Base"),
		MeshPassParameters,
		ERDGPassFlags::Raster | ERDGPassFlags::NeverCull,
		[View, &RenderSettings, this](FRHICommandList& RHICmdList)
		{
			FIntRect ViewRect = View->UnscaledViewRect;
			RHICmdList.SetViewport(ViewRect.Min.X, ViewRect.Min.Y, 0.0f, ViewRect.Max.X, ViewRect.Max.Y, 1.0f);

			SWITCH_ON_PROJECTION_TYPE(RenderPrimitives_RenderThread, RenderSettings.ProjectionType, View, RHICmdList, RenderSettings, false);
		});
}

void FDisplayClusterMeshProjectionRenderer::AddTranslucencyRenderPass(FRDGBuilder& GraphBuilder,
	const FSceneView* View,
	const FDisplayClusterMeshProjectionRenderSettings& RenderSettings,
	FRenderTargetBinding& OutputRenderTargetBinding,
	FDepthStencilBinding& OutputDepthStencilBinding)
{
	FMeshProjectionPassParameters* MeshPassParameters = CreateProjectionPassParameters(GraphBuilder, View, RenderSettings);
	MeshPassParameters->RenderTargets[0] = OutputRenderTargetBinding;
	MeshPassParameters->RenderTargets.DepthStencil = OutputDepthStencilBinding;

	GraphBuilder.AddPass(RDG_EVENT_NAME("MeshProjectionRenderer::Translucency"),
		MeshPassParameters,
		ERDGPassFlags::Raster | ERDGPassFlags::NeverCull,
		[View, &RenderSettings, this](FRHICommandList& RHICmdList)
		{
			FIntRect ViewRect = View->UnscaledViewRect;
			RHICmdList.SetViewport(ViewRect.Min.X, ViewRect.Min.Y, 0.0f, ViewRect.Max.X, ViewRect.Max.Y, 1.0f);

			SWITCH_ON_PROJECTION_TYPE(RenderPrimitives_RenderThread, RenderSettings.ProjectionType, View, RHICmdList, RenderSettings, true);
		});
}

void FDisplayClusterMeshProjectionRenderer::AddHitProxyRenderPass(FRDGBuilder& GraphBuilder,
	const FSceneView* View,
	const FDisplayClusterMeshProjectionRenderSettings& RenderSettings,
	FRenderTargetBinding& OutputRenderTargetBinding,
	FDepthStencilBinding& OutputDepthStencilBinding)
{
	FMeshProjectionPassParameters* MeshPassParameters = CreateProjectionPassParameters(GraphBuilder, View, RenderSettings);
	MeshPassParameters->RenderTargets[0] = OutputRenderTargetBinding;
	MeshPassParameters->RenderTargets.DepthStencil = OutputDepthStencilBinding;

	GraphBuilder.AddPass(RDG_EVENT_NAME("MeshProjectionRenderer::HitProxies"),
		MeshPassParameters,
		ERDGPassFlags::Raster | ERDGPassFlags::NeverCull,
		[View, &RenderSettings, this](FRHICommandList& RHICmdList)
		{
			FIntRect ViewRect = View->UnscaledViewRect;
			RHICmdList.SetViewport(ViewRect.Min.X, ViewRect.Min.Y, 0.0f, ViewRect.Max.X, ViewRect.Max.Y, 1.0f);

			SWITCH_ON_PROJECTION_TYPE(RenderHitProxies_RenderThread, RenderSettings.ProjectionType, View, RHICmdList, RenderSettings);
		});
}

void FDisplayClusterMeshProjectionRenderer::AddNormalsRenderPass(FRDGBuilder& GraphBuilder,
	const FSceneView* View,
	const FDisplayClusterMeshProjectionRenderSettings& RenderSettings,
	FRenderTargetBinding& OutputRenderTargetBinding,
	FDepthStencilBinding& OutputDepthStencilBinding)
{
	FMeshProjectionPassParameters* MeshPassParameters = CreateProjectionPassParameters(GraphBuilder, View, RenderSettings);
	MeshPassParameters->RenderTargets[0] = OutputRenderTargetBinding;
	MeshPassParameters->RenderTargets.DepthStencil = OutputDepthStencilBinding;

	GraphBuilder.AddPass(RDG_EVENT_NAME("MeshProjectionRenderer::Normals"),
		MeshPassParameters,
		ERDGPassFlags::Raster | ERDGPassFlags::NeverCull,
		[View, &RenderSettings, this](FRHICommandList& RHICmdList)
		{
			FIntRect ViewRect = View->UnscaledViewRect;
			RHICmdList.SetViewport(ViewRect.Min.X, ViewRect.Min.Y, 0.0f, ViewRect.Max.X, ViewRect.Max.Y, 1.0f);

			SWITCH_ON_PROJECTION_TYPE(RenderNormals_RenderThread, RenderSettings.ProjectionType, View, RHICmdList, RenderSettings);
		});
}

void GaussianSpatialFilter(float Sigma, TArray<float>& OutKernel)
{
	constexpr uint32 KernelSize = FILTER_KERNEL_SIZE;
	constexpr uint32 KernelRadius = (KernelSize - 1) / 2;

	OutKernel.Init(0.0, KernelSize);

	const float InvSigma = 1.f / Sigma;

	float Sum = 0.0f;
	for (int Index = 0; Index < KernelSize; ++Index)
	{
		int32 X = Index - KernelRadius;

		OutKernel[Index] = 0.39894 * FMath::Exp(-0.5f * (X * X) * InvSigma * InvSigma) * InvSigma;
		Sum += OutKernel[Index];
	}

	// Normalize the kernel so that there is no change in intensity of the filtered elements
	for (int Index = 0; Index < KernelSize; ++Index)
	{
		OutKernel[Index] /= Sum;
	}
}

/** Structure that holds two textures, one for input and one for output */
struct FTextureUAVWithIntermediate
{
	FRDGTextureRef PrimaryTexture = nullptr;
	FRDGTextureUAVRef PrimaryUAV = nullptr;

	FRDGTextureRef IntermediateTexture = nullptr;
	FRDGTextureUAVRef IntermediateUAV = nullptr;

	void CreateTexture(FRDGBuilder& GraphBuilder, const FRDGTextureDesc& TextureDesc, const TCHAR* PrimaryTextureName, const TCHAR* IntermediateTextureName)
	{
		PrimaryTexture = GraphBuilder.CreateTexture(TextureDesc, PrimaryTextureName);
		IntermediateTexture = GraphBuilder.CreateTexture(TextureDesc, IntermediateTextureName);
	}
	
	void CreateUAV(FRDGBuilder& GraphBuilder)
	{
		PrimaryUAV = GraphBuilder.CreateUAV(PrimaryTexture);
		IntermediateUAV = GraphBuilder.CreateUAV(IntermediateTexture);
	}
};

#define TEXTURE_NAME(TextureName) TEXT(#TextureName)
#define TEXTURE_NAME_PAIR(TextureName) TEXTURE_NAME(DisplayClusterMeshProjection.TextureName.Primary), TEXTURE_NAME(DisplayClusterMeshProjection.TextureName.Intermediate)

template<EMeshProjectionFilterType FilterType>
void AddSeparableFilterPass(FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* GlobalShaderMap,
	const FSceneView* View,
	FRDGTexture* SceneDepth,
	const FTextureUAVWithIntermediate& InColorTextures,
	const FTextureUAVWithIntermediate& InDepthTextures,
	const FTextureUAVWithIntermediate& InStencilTextures,
	const FTextureUAVWithIntermediate& InFinalStencilTextures,
	bool bCopyToFinalStencil = false)
{
	TArray<float> SpatialKernel;
	TArray<float> InteriorSpatialKernel;

	GaussianSpatialFilter(100.0f, SpatialKernel);
	GaussianSpatialFilter(1.0f, InteriorSpatialKernel);

	const FScreenPassTextureViewport InputViewport(InColorTextures.PrimaryUAV->Desc.Texture);

	FMeshProjectionNormalsFilterParameters* HorizontalPassParameters = GraphBuilder.AllocParameters<FMeshProjectionNormalsFilterParameters>();
	HorizontalPassParameters->View = View->ViewUniformBuffer;
	HorizontalPassParameters->Input = GetScreenPassTextureViewportParameters(InputViewport);
	HorizontalPassParameters->SceneStencil = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateWithPixelFormat(SceneDepth, PF_X24_G8));
	HorizontalPassParameters->RWColorIn = InColorTextures.PrimaryUAV;
	HorizontalPassParameters->RWColorOut = InColorTextures.IntermediateUAV;
	HorizontalPassParameters->RWDepthIn = InDepthTextures.PrimaryUAV;
	HorizontalPassParameters->RWDepthOut = InDepthTextures.IntermediateUAV;
	HorizontalPassParameters->RWStencilIn = InStencilTextures.PrimaryUAV;
	HorizontalPassParameters->RWStencilOut = InStencilTextures.IntermediateUAV;
	HorizontalPassParameters->RWBlurStencilIn = InFinalStencilTextures.PrimaryUAV;
	HorizontalPassParameters->RWBlurStencilOut = InFinalStencilTextures.IntermediateUAV;
	HorizontalPassParameters->SampleDirection = FVector2f(1.0f, 0.0f);
	HorizontalPassParameters->SampleOffsetScale = FVector2f(1.f);
	HorizontalPassParameters->bCopyToBlurStencil = bCopyToFinalStencil;

	for (int Index = 0; Index < FILTER_KERNEL_SIZE; ++Index)
	{
		GET_SCALAR_ARRAY_ELEMENT(HorizontalPassParameters->SpatialKernel, Index) = SpatialKernel[Index];
		GET_SCALAR_ARRAY_ELEMENT(HorizontalPassParameters->InteriorSpatialKernel, Index) = InteriorSpatialKernel[Index];
	}

	TShaderMapRef<FMeshProjectionNormalsFilterCS<FilterType>> ComputeShader(GlobalShaderMap);
	if (!ComputeShader.IsValid())
	{
		// Always check if shaders are available on the current platform and hardware
		return;
	}

	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("MeshProjectionRenderer::NormalsFilterHorizontal"),
		ComputeShader,
		HorizontalPassParameters,
		FComputeShaderUtils::GetGroupCount(InputViewport.Extent, FIntPoint(8, 8)));

	FMeshProjectionNormalsFilterParameters* VerticalPassParameters = GraphBuilder.AllocParameters<FMeshProjectionNormalsFilterParameters>();
	VerticalPassParameters->View = View->ViewUniformBuffer;
	VerticalPassParameters->Input = GetScreenPassTextureViewportParameters(InputViewport);
	VerticalPassParameters->SceneStencil = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateWithPixelFormat(SceneDepth, PF_X24_G8));
	VerticalPassParameters->RWColorIn = InColorTextures.IntermediateUAV;
	VerticalPassParameters->RWColorOut = InColorTextures.PrimaryUAV;
	VerticalPassParameters->RWDepthIn = InDepthTextures.IntermediateUAV;
	VerticalPassParameters->RWDepthOut = InDepthTextures.PrimaryUAV;
	VerticalPassParameters->RWStencilIn = InStencilTextures.IntermediateUAV;
	VerticalPassParameters->RWStencilOut = InStencilTextures.PrimaryUAV;
	VerticalPassParameters->RWBlurStencilIn = InFinalStencilTextures.IntermediateUAV;
	VerticalPassParameters->RWBlurStencilOut = InFinalStencilTextures.PrimaryUAV;
	VerticalPassParameters->SampleDirection = FVector2f(0.0f, 1.0f);
	VerticalPassParameters->SampleOffsetScale = FVector2f(1.f);
	VerticalPassParameters->bCopyToBlurStencil = bCopyToFinalStencil;

	for (int Index = 0; Index < FILTER_KERNEL_SIZE; ++Index)
	{
		GET_SCALAR_ARRAY_ELEMENT(VerticalPassParameters->SpatialKernel, Index) = SpatialKernel[Index];
		GET_SCALAR_ARRAY_ELEMENT(VerticalPassParameters->InteriorSpatialKernel, Index) = InteriorSpatialKernel[Index];
	}

	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("MeshProjectionRenderer::NormalsFilterVertical"),
		ComputeShader,
		VerticalPassParameters,
		FComputeShaderUtils::GetGroupCount(InputViewport.Extent, FIntPoint(8, 8)));
}

void FDisplayClusterMeshProjectionRenderer::AddNormalsFilterPass(FRDGBuilder& GraphBuilder,
	const FSceneView* View,
	FRenderTargetBinding& OutputRenderTargetBinding,
	FRDGTexture* SceneColor,
	FRDGTexture* SceneDepth,
	const FMatrix44f& NormalCorrectionMatrix)
{
	// The normal map filter pass is made up of several separate passes:
	// * First, the scene color, depth, and stencil textures must be copied into UAV read-write textures for the compute shaders to operate on. This pass also
	//   converts the normal vectors from world space to radial space, so that the filtering operates on the relative normals instead of the absolute normals
	// * Second, the depth and normals are dilated so they bleed into empty space. There is a separate interior depth dilate pass which dilates closer overlapping
	//   screen depths into further away screens, so that when blurred, there is a continuous depth change between overlapping screens
	// * Third, the normals and depths are blurred using a Gaussian blur. There are two separate gaussian spatial filters used, one used for blurring empty space regions
	//   and one used for blurring the screens. The empty space filter has a large sigma, allowing a large blur, while the interior filter has a smaller sigma, so that
	//   the blurring doesn't cause too much feature loss
	// * Finally, the RW textures are rendered back to a render target. Here, the normals are placed into the RGB components, while the depth is placed in the A component

	const FRDGTextureDesc RWColorDesc(FRDGTextureDesc::Create2D(SceneColor->Desc.Extent, PF_B8G8R8A8, FClearValueBinding::Black, TexCreate_ShaderResource | TexCreate_UAV));
	FTextureUAVWithIntermediate RWColor;
	RWColor.CreateTexture(GraphBuilder, RWColorDesc, TEXTURE_NAME_PAIR(NormalsRWTexture));
	RWColor.CreateUAV(GraphBuilder);

	const FRDGTextureDesc RWDepthDesc = FRDGTextureDesc::Create2D(SceneDepth->Desc.Extent, PF_G16, FClearValueBinding::White, TexCreate_UAV | TexCreate_ShaderResource);
	FTextureUAVWithIntermediate RWDepth;
	RWDepth.CreateTexture(GraphBuilder, RWDepthDesc, TEXTURE_NAME_PAIR(NormalsRWDepth));
	RWDepth.CreateUAV(GraphBuilder);

	const FRDGTextureDesc RWStencilDesc = FRDGTextureDesc::Create2D(SceneDepth->Desc.Extent, PF_R32_UINT, FClearValueBinding::Black, TexCreate_UAV | TexCreate_ShaderResource);
	FTextureUAVWithIntermediate RWStencil;
	RWStencil.CreateTexture(GraphBuilder, RWStencilDesc, TEXTURE_NAME_PAIR(NormalsRWStencil));
	RWStencil.CreateUAV(GraphBuilder);

	FTextureUAVWithIntermediate RWBlurStencil;
	RWBlurStencil.CreateTexture(GraphBuilder, RWStencilDesc, TEXTURE_NAME_PAIR(NormalsRWBlurStencil));
	RWBlurStencil.CreateUAV(GraphBuilder);

	FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(View->FeatureLevel);

	// Copy the scene color and depth to RW buffers
	{
		FMeshProjectionNormalsCreateRWTexturesCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FMeshProjectionNormalsCreateRWTexturesCS::FParameters>();
		PassParameters->View = View->ViewUniformBuffer;
		PassParameters->SceneColor = SceneColor;
		PassParameters->SceneDepth = SceneDepth;
		PassParameters->SceneStencil = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateWithPixelFormat(SceneDepth, PF_X24_G8));
		PassParameters->RWColorOut = RWColor.PrimaryUAV;
		PassParameters->RWDepthOut = RWDepth.PrimaryUAV;
		PassParameters->RWStencilOut = RWStencil.PrimaryUAV;
		PassParameters->RWBlurStencilOut = RWBlurStencil.PrimaryUAV;
		PassParameters->NormalCorrectionMatrix = NormalCorrectionMatrix;

		TShaderMapRef<FMeshProjectionNormalsCreateRWTexturesCS> ComputeShader(GlobalShaderMap);
		if (!ComputeShader.IsValid())
		{
			// Always check if shaders are available on the current platform and hardware
			return;
		}

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("MeshProjectionRenderer::CreateRWTextures"),
			ComputeShader,
			PassParameters,
			FComputeShaderUtils::GetGroupCount(SceneColor->Desc.Extent, FIntPoint(8, 8)));
	}

	static constexpr int32 DepthDilatePasses = 1;
	static constexpr int32 DilatePasses = 8;
	static constexpr int32 BlurPasses = 32;

	// Filter the RW buffers, dilating and blurring them appropriately
	for (int32 Index = 0; Index < DepthDilatePasses; Index++)
	{
		AddSeparableFilterPass<EMeshProjectionFilterType::DepthDilate>(GraphBuilder, GlobalShaderMap, View, SceneDepth, RWColor, RWDepth, RWStencil, RWBlurStencil);
	}

	for (int32 Index = 0; Index < DilatePasses; ++Index)
	{
		// Copy the second iteration of the dilated stencil to the blur stencil so that the region using the smaller
		// interior blur kernel extends a little beyond the stage's geometry bounds
		const bool bCopyToBlurStencil = Index == 1;
		AddSeparableFilterPass<EMeshProjectionFilterType::Dilate>(GraphBuilder, GlobalShaderMap, View, SceneDepth, RWColor, RWDepth, RWStencil, RWBlurStencil, bCopyToBlurStencil);
	}

	for (int32 Index = 0; Index < BlurPasses; Index++)
	{
		AddSeparableFilterPass<EMeshProjectionFilterType::Blur>(GraphBuilder, GlobalShaderMap, View, SceneDepth, RWColor, RWDepth, RWStencil, RWBlurStencil);
	}

	// Copy the RW buffers to the output render target
	{
		const FScreenPassTextureViewport InputViewport(RWColor.PrimaryTexture);
		const FScreenPassTextureViewport OutputViewport(OutputRenderTargetBinding.GetTexture());

		FMeshProjectionNormalsOutputPS::FParameters* ScreenPassParameters = GraphBuilder.AllocParameters<FMeshProjectionNormalsOutputPS::FParameters>();
		ScreenPassParameters->Input = GetScreenPassTextureViewportParameters(InputViewport);
		ScreenPassParameters->RWColorIn = RWColor.PrimaryUAV;
		ScreenPassParameters->RWDepthIn = RWDepth.PrimaryUAV;
		ScreenPassParameters->RenderTargets[0] = OutputRenderTargetBinding;

		TShaderMapRef<FScreenPassVS> ScreenPassVS(GlobalShaderMap);
		TShaderMapRef<FMeshProjectionNormalsOutputPS> OutputNormalsPS(GlobalShaderMap);
		if (!ScreenPassVS.IsValid() || !OutputNormalsPS.IsValid())
		{
			// Always check if shaders are available on the current platform and hardware
			return;
		}

		FRHIBlendState* DefaultBlendState = FScreenPassPipelineState::FDefaultBlendState::GetRHI();

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("MeshProjectionRenderer::OutputNormals"),
			ScreenPassParameters,
			ERDGPassFlags::Raster,
			[View, ScreenPassVS, OutputNormalsPS, InputViewport, OutputViewport, ScreenPassParameters, DefaultBlendState](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			DrawScreenPass(
				RHICmdList,
				*View,
				InputViewport,
				OutputViewport,
				FScreenPassPipelineState(ScreenPassVS, OutputNormalsPS, DefaultBlendState),
				EScreenPassDrawFlags::None,
				[&](FRHICommandList&)
			{
				SetShaderParameters(RHICmdList, OutputNormalsPS, OutputNormalsPS.GetPixelShader(), *ScreenPassParameters);
			});
		});
	}
}

#if WITH_EDITOR
void FDisplayClusterMeshProjectionRenderer::AddSelectionDepthRenderPass(FRDGBuilder& GraphBuilder, 
	const FSceneView* View,
	const FDisplayClusterMeshProjectionRenderSettings& RenderSettings,
	FDepthStencilBinding& OutputDepthStencilBinding)
{
	FMeshProjectionPassParameters* SelectionPassParameters = CreateProjectionPassParameters(GraphBuilder, View, RenderSettings);
	SelectionPassParameters->RenderTargets.DepthStencil = OutputDepthStencilBinding;

	GraphBuilder.AddPass(RDG_EVENT_NAME("MeshProjectionRenderer::SelectionDepth"),
		SelectionPassParameters,
		ERDGPassFlags::Raster | ERDGPassFlags::NeverCull,
		[View, &RenderSettings, this](FRHICommandList& RHICmdList)
	{
		FIntRect ViewRect = View->UnscaledViewRect;
		RHICmdList.SetViewport(ViewRect.Min.X, ViewRect.Min.Y, 0.0f, ViewRect.Max.X, ViewRect.Max.Y, 1.0f);

		SWITCH_ON_PROJECTION_TYPE(RenderSelection_RenderThread, RenderSettings.ProjectionType, View, RHICmdList, RenderSettings);
	});
}

void FDisplayClusterMeshProjectionRenderer::AddSelectionOutlineScreenPass(FRDGBuilder& GraphBuilder,
	const FSceneView* View,
	FRenderTargetBinding& OutputRenderTargetBinding,
	FRDGTexture* SceneColor,
	FRDGTexture* SceneDepth,
	FRDGTexture* SelectionDepth)
{
	const FScreenPassTextureViewport OutputViewport(OutputRenderTargetBinding.GetTexture());
	const FScreenPassTextureViewport ColorViewport(SceneColor);
	const FScreenPassTextureViewport DepthViewport(SceneDepth);

	FRHISamplerState* PointClampSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	FMeshProjectionSelectionOutlinePS::FParameters* ScreenPassParameters = GraphBuilder.AllocParameters<FMeshProjectionSelectionOutlinePS::FParameters>();
	ScreenPassParameters->RenderTargets[0] = OutputRenderTargetBinding;
	ScreenPassParameters->View = View->ViewUniformBuffer;
	ScreenPassParameters->Color = GetScreenPassTextureViewportParameters(ColorViewport);
	ScreenPassParameters->Depth = GetScreenPassTextureViewportParameters(DepthViewport);
	ScreenPassParameters->ColorTexture = SceneColor;
	ScreenPassParameters->ColorSampler = PointClampSampler;
	ScreenPassParameters->DepthTexture = SceneDepth;
	ScreenPassParameters->DepthSampler = PointClampSampler;
	ScreenPassParameters->EditorPrimitivesDepth = SelectionDepth;
	ScreenPassParameters->EditorPrimitivesStencil = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateWithPixelFormat(SelectionDepth, PF_X24_G8));
	ScreenPassParameters->OutlineColor = FVector3f(View->SelectionOutlineColor);
	ScreenPassParameters->SelectionHighlightIntensity = GEngine->SelectionHighlightIntensity;

	FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(View->FeatureLevel);

	TShaderMapRef<FScreenPassVS> ScreenPassVS(GlobalShaderMap);
	TShaderMapRef<FMeshProjectionSelectionOutlinePS> SelectionOutlinePS(GlobalShaderMap);
	if (!ScreenPassVS.IsValid() || !SelectionOutlinePS.IsValid())
	{
		// Always check if shaders are available on the current platform and hardware
		return;
	}

	FRHIBlendState* DefaultBlendState = FScreenPassPipelineState::FDefaultBlendState::GetRHI();

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("MeshProjectionRenderer::SelectionScreen"),
		ScreenPassParameters,
		ERDGPassFlags::Raster,
		[View, ScreenPassVS, SelectionOutlinePS, OutputViewport, ScreenPassParameters, DefaultBlendState](FRDGAsyncTask, FRHICommandList& RHICmdList)
	{
		DrawScreenPass(
			RHICmdList,
			*View,
			OutputViewport,
			OutputViewport,
			FScreenPassPipelineState(ScreenPassVS, SelectionOutlinePS, DefaultBlendState),
			EScreenPassDrawFlags::None,
			[&](FRHICommandList&)
		{
			SetShaderParameters(RHICmdList, SelectionOutlinePS, SelectionOutlinePS.GetPixelShader(), *ScreenPassParameters);
		});
	});
}
#endif

void FDisplayClusterMeshProjectionRenderer::AddSimpleElementPass(FRDGBuilder& GraphBuilder,
	const FSceneView* View,
	FRenderTargetBinding& OutputRenderTargetBinding,
	FSimpleElementCollector& ElementCollector)
{
	FMeshProjectionPassParameters* PassParameters = GraphBuilder.AllocParameters<FMeshProjectionPassParameters>();
	PassParameters->View = View->ViewUniformBuffer;
	PassParameters->InstanceCulling = FInstanceCullingContext::CreateDummyInstanceCullingUniformBuffer(GraphBuilder);
	PassParameters->RenderTargets[0] = OutputRenderTargetBinding;

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("MeshProjectionRenderer::SimpleElements"),
		PassParameters,
		ERDGPassFlags::Raster,
		[this, View,  &ElementCollector](FRHICommandListImmediate& RHICmdList)
		{
			FIntRect ViewRect = View->UnscaledViewRect;
			RHICmdList.SetViewport(ViewRect.Min.X, ViewRect.Min.Y, 0.0f, ViewRect.Max.X, ViewRect.Max.Y, 1.0f);

			FMeshPassProcessorRenderState DrawRenderState;
			DrawRenderState.SetDepthStencilAccess(FExclusiveDepthStencil::DepthRead_StencilWrite);
			DrawRenderState.SetBlendState(TStaticBlendStateWriteMask<CW_RGBA, CW_RGBA, CW_RGBA, CW_RGBA>::GetRHI());
			DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<false, CF_DepthNearOrEqual>::GetRHI());

			ElementCollector.DrawBatchedElements(RHICmdList, DrawRenderState, *View, EBlendModeFilter::OpaqueAndMasked, ESceneDepthPriorityGroup::SDPG_World);
			ElementCollector.DrawBatchedElements(RHICmdList, DrawRenderState, *View, EBlendModeFilter::OpaqueAndMasked, ESceneDepthPriorityGroup::SDPG_Foreground);
		}
	);
}

template<EDisplayClusterMeshProjectionType ProjectionType>
void FDisplayClusterMeshProjectionRenderer::RenderPrimitives_RenderThread(const FSceneView* View,
	FRHICommandList& RHICmdList,
	const FDisplayClusterMeshProjectionRenderSettings& RenderSettings,
	bool bTranslucencyPass)
{
	DrawDynamicMeshPass(*View, RHICmdList, [View, &RenderSettings, bTranslucencyPass, this](FDynamicPassMeshDrawListContext* DynamicMeshPassContext)
	{
		TArray<FSceneProxyElement> PrimitiveSceneProxies;
		GetSceneProxies(PrimitiveSceneProxies, RenderSettings, [View](const UPrimitiveComponent* InPrimitiveComponent)
			{
				return UE::DisplayClusterMeshProjectionRenderer::ShouldRenderPrimitiveComponentForView(InPrimitiveComponent, View);
			});

		if (PrimitiveSceneProxies.IsEmpty())
		{
			// Skip if there is nothing to render.
			return;
		}

		FMeshProjectionPassProcessor<ProjectionType> ProjectionMeshProcessor(nullptr, View, DynamicMeshPassContext, RenderSettings, bTranslucencyPass);
		FMeshProjectionPassProcessor<EDisplayClusterMeshProjectionType::Linear> LinearMeshProcessor(nullptr, View, DynamicMeshPassContext, RenderSettings, bTranslucencyPass);

		for (const FSceneProxyElement& PrimitiveProxyElement : PrimitiveSceneProxies)
		{
			FPrimitiveSceneProxy* PrimitiveProxy = PrimitiveProxyElement.PrimitiveSceneProxy;
			if (PrimitiveProxy == nullptr || PrimitiveProxy->GetPrimitiveSceneInfo() == nullptr)
			{
				continue;
			}

			if (const FMeshBatch* MeshBatch = PrimitiveProxy->GetPrimitiveSceneInfo()->GetMeshBatch(PrimitiveProxy->GetPrimitiveSceneInfo()->StaticMeshes.Num() - 1))
			{
				MeshBatch->MaterialRenderProxy->UpdateUniformExpressionCacheIfNeeded(View->GetFeatureLevel());

				const uint64 BatchElementMask = ~0ull;

				if (PrimitiveProxyElement.bApplyProjection)
				{
					ProjectionMeshProcessor.AddMeshBatch(*MeshBatch, BatchElementMask, PrimitiveProxy);
				}
				else
				{
					LinearMeshProcessor.AddMeshBatch(*MeshBatch, BatchElementMask, PrimitiveProxy);
				}
			}
		}
	});
}

template<EDisplayClusterMeshProjectionType ProjectionType>
void FDisplayClusterMeshProjectionRenderer::RenderHitProxies_RenderThread(const FSceneView* View,
	FRHICommandList& RHICmdList,
	const FDisplayClusterMeshProjectionRenderSettings& RenderSettings)
{
	DrawDynamicMeshPass(*View, RHICmdList, [View, &RenderSettings, this](FDynamicPassMeshDrawListContext* DynamicMeshPassContext)
	{
		TArray<FSceneProxyElement> PrimitiveSceneProxies;
		GetSceneProxies(PrimitiveSceneProxies, RenderSettings, [View](const UPrimitiveComponent* InPrimitiveComponent)
			{
				return UE::DisplayClusterMeshProjectionRenderer::ShouldRenderPrimitiveComponentForView(InPrimitiveComponent, View);
			});

		if (PrimitiveSceneProxies.IsEmpty())
		{
			// Skip if there is nothing to render.
			return;
		}

		FMeshProjectionHitProxyPassProcessor<ProjectionType> ProjectionMeshProcessor(nullptr, View, DynamicMeshPassContext, RenderSettings);
		FMeshProjectionHitProxyPassProcessor<EDisplayClusterMeshProjectionType::Linear> LinearMeshProcessor(nullptr, View, DynamicMeshPassContext, RenderSettings);

		for (const FSceneProxyElement& PrimitiveProxyElement : PrimitiveSceneProxies)
		{
			FPrimitiveSceneProxy* PrimitiveProxy = PrimitiveProxyElement.PrimitiveSceneProxy;
			if (const FMeshBatch* MeshBatch = PrimitiveProxy->GetPrimitiveSceneInfo()->GetMeshBatch(PrimitiveProxy->GetPrimitiveSceneInfo()->StaticMeshes.Num() - 1))
			{
				MeshBatch->MaterialRenderProxy->UpdateUniformExpressionCacheIfNeeded(View->GetFeatureLevel());

				const uint64 BatchElementMask = ~0ull;

				if (PrimitiveProxyElement.bApplyProjection)
				{
					ProjectionMeshProcessor.AddMeshBatch(*MeshBatch, BatchElementMask, PrimitiveProxy);
				}
				else
				{
					LinearMeshProcessor.AddMeshBatch(*MeshBatch, BatchElementMask, PrimitiveProxy);
				}
			}
		}
	});
}

template<EDisplayClusterMeshProjectionType ProjectionType>
void FDisplayClusterMeshProjectionRenderer::RenderNormals_RenderThread(const FSceneView* View,
	FRHICommandList& RHICmdList,
	const FDisplayClusterMeshProjectionRenderSettings& RenderSettings)
{
	DrawDynamicMeshPass(*View, RHICmdList, [View, &RenderSettings, this](FDynamicPassMeshDrawListContext* DynamicMeshPassContext)
	{
		TArray<FSceneProxyElement> PrimitiveSceneProxies;
		GetSceneProxies(PrimitiveSceneProxies, RenderSettings, [View](const UPrimitiveComponent* InPrimitiveComponent)
			{
				return UE::DisplayClusterMeshProjectionRenderer::ShouldRenderPrimitiveComponentForView(InPrimitiveComponent, View);
			});

		if (PrimitiveSceneProxies.IsEmpty())
		{
			// Skip if there is nothing to render.
			return;
		}

		FMeshProjectionPassProcessor<ProjectionType, EDisplayClusterMeshProjectionOutput::Normals> ProjectionMeshProcessor(nullptr, View, DynamicMeshPassContext, RenderSettings);
		ProjectionMeshProcessor.SetStencilValue(1);
		ProjectionMeshProcessor.SetIgnoreTranslucency(true);

		FMeshProjectionPassProcessor<EDisplayClusterMeshProjectionType::Linear, EDisplayClusterMeshProjectionOutput::Normals> LinearMeshProcessor(nullptr, View, DynamicMeshPassContext, RenderSettings);
		LinearMeshProcessor.SetStencilValue(1);
		LinearMeshProcessor.SetIgnoreTranslucency(true);

		for (const FSceneProxyElement& PrimitiveProxyElement : PrimitiveSceneProxies)
		{
			FPrimitiveSceneProxy* PrimitiveProxy = PrimitiveProxyElement.PrimitiveSceneProxy;
			if (const FMeshBatch* MeshBatch = PrimitiveProxy->GetPrimitiveSceneInfo()->GetMeshBatch(PrimitiveProxy->GetPrimitiveSceneInfo()->StaticMeshes.Num() - 1))
			{
				MeshBatch->MaterialRenderProxy->UpdateUniformExpressionCacheIfNeeded(View->GetFeatureLevel());

				const uint64 BatchElementMask = ~0ull;

				if (PrimitiveProxyElement.bApplyProjection)
				{
					ProjectionMeshProcessor.AddMeshBatch(*MeshBatch, BatchElementMask, PrimitiveProxy);
				}
				else
				{
					LinearMeshProcessor.AddMeshBatch(*MeshBatch, BatchElementMask, PrimitiveProxy);
				}
			}
		}
	});
}

#if WITH_EDITOR
template<EDisplayClusterMeshProjectionType ProjectionType>
void FDisplayClusterMeshProjectionRenderer::RenderSelection_RenderThread(const FSceneView* View,
	FRHICommandList& RHICmdList,
	const FDisplayClusterMeshProjectionRenderSettings& RenderSettings)
{
	DrawDynamicMeshPass(*View, RHICmdList, [View, &RenderSettings, this](FDynamicPassMeshDrawListContext* DynamicMeshPassContext)
	{
		TArray<FSceneProxyElement> PrimitiveSceneProxies;
		GetSceneProxies(PrimitiveSceneProxies, RenderSettings, [View](const UPrimitiveComponent* InPrimitiveComponent)
		{
			if (!UE::DisplayClusterMeshProjectionRenderer::ShouldRenderPrimitiveComponentForView(InPrimitiveComponent, View))
			{
				return false;
			}

			return InPrimitiveComponent->SceneProxy->IsSelected();
		});

		if (PrimitiveSceneProxies.IsEmpty())
		{
			// Skip if there is nothing to render.
			return;
		}

		FMeshProjectionSelectionPassProcessor<ProjectionType> ProjectionMeshProcessor(nullptr, View, DynamicMeshPassContext, RenderSettings);
		FMeshProjectionSelectionPassProcessor<EDisplayClusterMeshProjectionType::Linear> LinearMeshProcessor(nullptr, View, DynamicMeshPassContext, RenderSettings);

		for (const FSceneProxyElement& PrimitiveProxyElement : PrimitiveSceneProxies)
		{
			FPrimitiveSceneProxy* PrimitiveProxy = PrimitiveProxyElement.PrimitiveSceneProxy;
			if (const FMeshBatch* MeshBatch = PrimitiveProxy->GetPrimitiveSceneInfo()->GetMeshBatch(PrimitiveProxy->GetPrimitiveSceneInfo()->StaticMeshes.Num() - 1))
			{
				MeshBatch->MaterialRenderProxy->UpdateUniformExpressionCacheIfNeeded(View->GetFeatureLevel());

				const uint64 BatchElementMask = ~0ull;

				if (PrimitiveProxyElement.bApplyProjection)
				{
					ProjectionMeshProcessor.AddMeshBatch(*MeshBatch, BatchElementMask, PrimitiveProxy);
				}
				else
				{
					LinearMeshProcessor.AddMeshBatch(*MeshBatch, BatchElementMask, PrimitiveProxy);
				}
			}
		}
	});
}
#endif

bool FDisplayClusterMeshProjectionRenderer::IsPrimitiveComponentSelected(const UPrimitiveComponent* InPrimitiveComponent)
{
	if (ActorSelectedDelegate.IsBound())
	{
		return ActorSelectedDelegate.Execute(InPrimitiveComponent->GetOwner());
	}

	return false;
}

void FDisplayClusterMeshProjectionRenderer::GetSceneProxies(TArray<FSceneProxyElement>& OutSceneProxyElements,
	const FDisplayClusterMeshProjectionRenderSettings& RenderSettings,
	const TFunctionRef<bool(const UPrimitiveComponent*)> PrimitiveFilter)
{
	for (const TWeakObjectPtr<UPrimitiveComponent>& PrimitiveComponent : PrimitiveComponents)
	{
		if (PrimitiveComponent.IsValid() && PrimitiveComponent->SceneProxy && !PrimitiveComponent->bVisibleInSceneCaptureOnly)
		{
			if (PrimitiveFilter(PrimitiveComponent.Get()) && RenderSettings.PrimitiveFilter.ShouldRenderPrimitive(PrimitiveComponent.Get()))
			{
				FSceneProxyElement SceneProxyElement;
				SceneProxyElement.PrimitiveSceneProxy = PrimitiveComponent->SceneProxy;
				SceneProxyElement.bApplyProjection = RenderSettings.PrimitiveFilter.ShouldApplyProjection(PrimitiveComponent.Get());

				OutSceneProxyElements.Add(SceneProxyElement);
			}
		}
	}
}

void FDisplayClusterMeshProjectionRenderer::RenderScenes(FCanvas* Canvas, const TArray<FSceneInterface*>& InScenes, const FDisplayClusterMeshProjectionRenderSettings& RenderSettings)
{
	Canvas->Flush_GameThread();

	// This code is based on the existing FDisplayClusterMeshProjectionRenderer::Render() function.

	FRenderTarget* RenderTarget = Canvas->GetRenderTarget();
	const bool bIsHitTesting = Canvas->IsHitTesting();
	FHitProxyConsumer* HitProxyConsumer = Canvas->GetHitProxyConsumer();

	UE::RenderCommandPipe::FSyncScope SyncScope;

	ENQUEUE_RENDER_COMMAND(FDrawProjectedMeshes)(
		[RenderTarget, Scenes = MoveTempIfPossible(InScenes), RenderSettings, bIsHitTesting, HitProxyConsumer, this](FRHICommandListImmediate& RHICmdList)
		{
			FRDGBuilder GraphBuilder(RHICmdList);

			TArray<TSharedRef<FSceneViewFamilyContext>> ViewFamilies;
			TArray<TSharedRef<FScenePrimitiveRenderingContextScopeHelper>> ScopeHelpers;
			TArray<const FSceneView*> SceneViews;
			if (Scenes.Num() > 0)
			{
				// Initial fix to a crash due to the GraphBuilder's FRDGBlackboard getting two instances of the same struct types.
				// Only the first scene (PreviewWorld->Scene) will be rendered, which implies that the light cards / stage actors 
				// won't be visible outside the stage meshes. This code might need to be updated to that composites.
				FSceneInterface* Scene = Scenes[0];
				ViewFamilies.Add(
					MakeShared<FSceneViewFamilyContext>(FSceneViewFamily::ConstructionValues(
						RenderTarget,
						Scene,
						RenderSettings.EngineShowFlags)
						.SetTime(FGameTime::GetTimeSinceAppStart()))
				);

				FSceneViewFamilyContext& ViewFamily = ViewFamilies.Last().Get();

				if (Scene)
				{
					Scene->IncrementFrameNumber();
					ViewFamily.FrameNumber = Scene->GetFrameNumber();
				}
				else
				{
					ViewFamily.FrameNumber = GFrameNumber;
				}

				ViewFamily.EngineShowFlags.SetHitProxies(bIsHitTesting);

				ScopeHelpers.Add(MakeShared<FScenePrimitiveRenderingContextScopeHelper>(GetRendererModule().BeginScenePrimitiveRendering(GraphBuilder, &ViewFamily)));

				FSceneViewInitOptions NewInitOptions(RenderSettings.ViewInitOptions);
				NewInitOptions.ViewFamily = &ViewFamily;

				GetRendererModule().CreateAndInitSingleView(RHICmdList, &ViewFamily, &NewInitOptions);
				SceneViews.Add(ViewFamily.Views[0]);
			}

			FRDGTextureRef OutputTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(RenderTarget->GetRenderTargetTexture(), TEXT("ViewRenderTarget")));
			FRenderTargetBinding OutputRenderTargetBinding(OutputTexture, ERenderTargetLoadAction::ELoad);
			FMeshProjectionElementCollector ElementCollector(HitProxyConsumer);

			switch (RenderSettings.RenderType)
			{
			case EDisplayClusterMeshProjectionOutput::Color:
				if (ViewFamilies[0]->EngineShowFlags.HitProxies)
				{
					RenderHitProxyOutput(GraphBuilder, SceneViews[0], RenderSettings, OutputRenderTargetBinding, HitProxyConsumer);
				}
				else
				{
					RenderColorOutputs(GraphBuilder, SceneViews, RenderSettings, OutputRenderTargetBinding);
				}

				if (RenderSimpleElementsDelegate.IsBound())
				{
					RenderSimpleElementsDelegate.Execute(SceneViews[0], &ElementCollector);
					AddSimpleElementPass(GraphBuilder, SceneViews[0], OutputRenderTargetBinding, ElementCollector);
				}
				break;

			case EDisplayClusterMeshProjectionOutput::Normals:
				RenderNormalsOutput(GraphBuilder, SceneViews[0], RenderSettings, OutputRenderTargetBinding);
				break;
			}

			GraphBuilder.Execute();
		});
}

void FDisplayClusterMeshProjectionRenderer::RenderColorOutputs(FRDGBuilder& GraphBuilder,
	const TArray<const FSceneView*>& Views,
	const FDisplayClusterMeshProjectionRenderSettings& RenderSettings,
	FRenderTargetBinding& OutputRenderTargetBinding)
{
	FRDGTextureRef ColorTexture = GraphBuilder.CreateTexture(OutputRenderTargetBinding.GetTexture()->Desc, TEXT("DisplayClusterMeshProjection.ColorTexture"));

	const FRDGTextureDesc DepthDesc = FRDGTextureDesc::Create2D(OutputRenderTargetBinding.GetTexture()->Desc.Extent, PF_DepthStencil, FClearValueBinding::DepthFar, TexCreate_DepthStencilTargetable | TexCreate_ShaderResource);
	FRDGTextureRef DepthTexture = GraphBuilder.CreateTexture(DepthDesc, TEXT("DisplayClusterMeshProjection.DepthTexture"));

	FRenderTargetBinding ColorRenderTargetBinding(ColorTexture, ERenderTargetLoadAction::EClear);
	FDepthStencilBinding DepthStencilBinding(DepthTexture, ERenderTargetLoadAction::EClear, ERenderTargetLoadAction::ENoAction, FExclusiveDepthStencil::DepthWrite_StencilWrite);

	for (const FSceneView* ViewIt : Views)
	{
		AddBaseRenderPass(GraphBuilder, ViewIt, RenderSettings, ColorRenderTargetBinding, DepthStencilBinding);

		ColorRenderTargetBinding.SetLoadAction(ERenderTargetLoadAction::ELoad);
		DepthStencilBinding.SetDepthLoadAction(ERenderTargetLoadAction::ELoad);
	}

	for (const FSceneView* ViewIt : Views)
	{
		AddTranslucencyRenderPass(GraphBuilder, ViewIt, RenderSettings, ColorRenderTargetBinding, DepthStencilBinding);
	}

#if WITH_EDITOR
	if (RenderSettings.EngineShowFlags.SelectionOutline)
	{
		const FRDGTextureDesc SelectionDepthDesc = FRDGTextureDesc::Create2D(OutputRenderTargetBinding.GetTexture()->Desc.Extent, PF_DepthStencil, FClearValueBinding::DepthFar, TexCreate_DepthStencilTargetable | TexCreate_ShaderResource);
		FRDGTextureRef SelectionDepthTexture = GraphBuilder.CreateTexture(SelectionDepthDesc, TEXT("DisplayClusterMeshProjection.SelectionDepthTexture"));
		FDepthStencilBinding SelectionDepthStencilBinding(SelectionDepthTexture, ERenderTargetLoadAction::EClear, ERenderTargetLoadAction::EClear, FExclusiveDepthStencil::DepthWrite_StencilWrite);

		for (const FSceneView* ViewIt : Views)
		{
			AddSelectionDepthRenderPass(GraphBuilder, ViewIt, RenderSettings, SelectionDepthStencilBinding);
		}

		for (const FSceneView* ViewIt : Views)
		{
			AddSelectionOutlineScreenPass(GraphBuilder, ViewIt, OutputRenderTargetBinding, ColorTexture, DepthTexture, SelectionDepthTexture);
		}
	}
	else
#endif
		// Copy the scene color to the output render target
	{
		FCopyRectPS::FParameters* ScreenPassParameters = GraphBuilder.AllocParameters<FCopyRectPS::FParameters>();
		ScreenPassParameters->InputTexture = ColorTexture;
		ScreenPassParameters->InputSampler = TStaticSamplerState<>::GetRHI();
		ScreenPassParameters->RenderTargets[0] = OutputRenderTargetBinding;

		FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(Views[0]->FeatureLevel);
		TShaderMapRef<FScreenPassVS> ScreenPassVS(GlobalShaderMap);
		TShaderMapRef<FCopyRectPS> CopyPixelShader(GlobalShaderMap);
		if (!ScreenPassVS.IsValid() || !CopyPixelShader.IsValid())
		{
			// Always check if shaders are available on the current platform and hardware
			return;
		}

		FRHIBlendState* DefaultBlendState = FScreenPassPipelineState::FDefaultBlendState::GetRHI();
		const FScreenPassTextureViewport RegionViewport(OutputRenderTargetBinding.GetTexture());

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("MeshProjectionRenderer::CopyColorTexture"),
			ScreenPassParameters,
			ERDGPassFlags::Raster,
			[View = Views[0], ScreenPassVS, CopyPixelShader, RegionViewport, ScreenPassParameters, DefaultBlendState](FRDGAsyncTask, FRHICommandList& RHICmdList)
			{
				DrawScreenPass(
					RHICmdList,
					*View,
					RegionViewport,
					RegionViewport,
					FScreenPassPipelineState(ScreenPassVS, CopyPixelShader, DefaultBlendState),
					EScreenPassDrawFlags::None,
					[&](FRHICommandList&)
					{
						SetShaderParameters(RHICmdList, CopyPixelShader, CopyPixelShader.GetPixelShader(), *ScreenPassParameters);
					});
			});
	}
}

// Explicit template specializations for each projection type
#define PROJECTION_TYPE_TEMPLATE_SPECIALIZATION(FuncName, ProjectionType, ...) template void FDisplayClusterMeshProjectionRenderer::FuncName<ProjectionType>(__VA_ARGS__);
#define CREATE_PROJECTION_TYPE_TEMPLATE_SPECIALIZATIONS(FuncName, ...) \
	PROJECTION_TYPE_TEMPLATE_SPECIALIZATION(FuncName, EDisplayClusterMeshProjectionType::Linear, __VA_ARGS__) \
	PROJECTION_TYPE_TEMPLATE_SPECIALIZATION(FuncName, EDisplayClusterMeshProjectionType::Azimuthal, __VA_ARGS__) \
	PROJECTION_TYPE_TEMPLATE_SPECIALIZATION(FuncName, EDisplayClusterMeshProjectionType::UV, __VA_ARGS__)

CREATE_PROJECTION_TYPE_TEMPLATE_SPECIALIZATIONS(RenderPrimitives_RenderThread, const FSceneView* View, FRHICommandList& RHICmdList, const FDisplayClusterMeshProjectionRenderSettings& RenderSettings, bool bTranslucencyPass)
CREATE_PROJECTION_TYPE_TEMPLATE_SPECIALIZATIONS(RenderHitProxies_RenderThread, const FSceneView* View, FRHICommandList& RHICmdList, const FDisplayClusterMeshProjectionRenderSettings& RenderSettings)
CREATE_PROJECTION_TYPE_TEMPLATE_SPECIALIZATIONS(RenderNormals_RenderThread, const FSceneView* View, FRHICommandList& RHICmdList, const FDisplayClusterMeshProjectionRenderSettings& RenderSettings)

#if WITH_EDITOR
CREATE_PROJECTION_TYPE_TEMPLATE_SPECIALIZATIONS(RenderSelection_RenderThread, const FSceneView* View, FRHICommandList& RHICmdList, const FDisplayClusterMeshProjectionRenderSettings& RenderSettings)
#endif