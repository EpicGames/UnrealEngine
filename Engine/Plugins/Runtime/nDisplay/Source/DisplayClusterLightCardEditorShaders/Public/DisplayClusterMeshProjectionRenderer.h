// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneView.h"

class AActor;
class UPrimitiveComponent;
class FHitProxyConsumer;
class FSceneView;
class FViewInfo;
class FPrimitiveDrawInterface;
class FPrimitiveSceneProxy;
class FRDGBuilder;
class FSimpleElementCollector;
class FMeshProjectionPassParameters;
class FCanvas;
struct FEngineShowFlags;

/** Indicates which kind of projection is used by the renderer */
enum EDisplayClusterMeshProjectionType
{
	/** Default linear projection */
	Linear,

	/** Non-linear spherical projection based on the azimuthal equidistant map projection */
	Azimuthal,

	/** Projection that positions vertices based on their UV coordinates */
	UV
};

/** Indicates the quantity that is output to the canvas by the renderer */
enum EDisplayClusterMeshProjectionOutput
{
	/** Outputs the emissive color of the rendered primitives */
	Color,

	/** Outputs the normals and depth of the rendered primitives */
	Normals
};

/** A filter that allows specific primitive components to be filtered from a render pass */
class FDisplayClusterMeshProjectionPrimitiveFilter
{
public:
	DECLARE_DELEGATE_RetVal_OneParam(bool, FPrimitiveFilter, const UPrimitiveComponent*);


	/** A delegate that returns true if the primitive component should be included in the render pass */
	FPrimitiveFilter ShouldRenderPrimitiveDelegate;

	/** A delegate that returns true if the primitive component should be rendered using the render pass's projection type */
	FPrimitiveFilter ShouldApplyProjectionDelegate;

	/** Gets whether a primitive component should be filtered out of the render pass or not */
	DISPLAYCLUSTERLIGHTCARDEDITORSHADERS_API bool ShouldRenderPrimitive(const UPrimitiveComponent* InPrimitiveComponent) const;

	/** Gets whether a primitive component should be rendered using the current projection type or not */
	DISPLAYCLUSTERLIGHTCARDEDITORSHADERS_API bool ShouldApplyProjection(const UPrimitiveComponent* InPrimitiveComponent) const;
};

/** Settings for specific mesh projection types  */
struct FDisplayClusterMeshProjectionTypeSettings
{
	/** The index of the UV to use when performing a UV projection */
	uint32 UVProjectionIndex = 0;

	/** The size of the plane the UVs are projected to, in view space */
	float UVProjectionPlaneSize = 100.0f;

	/** The distance from the view point of the plane the UVs are projected to, in view space */
	float UVProjectionPlaneDistance = 100.0f;

	/** A translation offset for the UV projection plane */
	FVector UVProjectionPlaneOffset = FVector::ZeroVector;
};

/** Settings for producing a single render. */
struct FDisplayClusterMeshProjectionRenderSettings
{
	/** Camera setup options for the render. */
	FSceneViewInitOptions ViewInitOptions;

	/**
	 * Flags controlling renderer features to enable/disable for this preview.
	 * Only flags supported by FDisplayClusterMeshProjectionRenderer will have an impact on the render output.
	 */
	FEngineShowFlags EngineShowFlags = FEngineShowFlags(ESFIM_Editor);

	/** Type of projection to use for the renderer. */
	EDisplayClusterMeshProjectionType ProjectionType = EDisplayClusterMeshProjectionType::Azimuthal;

	/** Settings used by the projection type when rendering */
	FDisplayClusterMeshProjectionTypeSettings ProjectionTypeSettings;

	/** The output type to use for the renderer. */
	EDisplayClusterMeshProjectionOutput RenderType = EDisplayClusterMeshProjectionOutput::Color;

	/** A matrix used to rotate the normals in order to account for the root actor's rotation. */
	FMatrix44f NormalCorrectionMatrix = FMatrix44f::Identity;

	/**
	 * Optional filter to prevent specific primitives from being rendered.
	 * This only applies when RenderType is ERenderType::Normals.
	 */
	FDisplayClusterMeshProjectionPrimitiveFilter PrimitiveFilter;
};

/** A transform that can be passed around to project and unprojection positions for a specific projection type */
class FDisplayClusterMeshProjectionTransform
{
public:
	FDisplayClusterMeshProjectionTransform()
		: FDisplayClusterMeshProjectionTransform(EDisplayClusterMeshProjectionType::Linear, FMatrix::Identity)
	{ }

	FDisplayClusterMeshProjectionTransform(EDisplayClusterMeshProjectionType InProjection, const FMatrix& InViewMatrix)
		: Projection(InProjection)
		, ViewMatrix(InViewMatrix)
		, InvViewMatrix(InViewMatrix.Inverse())
	{ }

	DISPLAYCLUSTERLIGHTCARDEDITORSHADERS_API FVector ProjectPosition(const FVector& WorldPosition) const;
	DISPLAYCLUSTERLIGHTCARDEDITORSHADERS_API FVector UnprojectPosition(const FVector& ProjectedPosition) const;

private:
	EDisplayClusterMeshProjectionType Projection;
	FMatrix ViewMatrix;
	FMatrix InvViewMatrix;
};

/** A renderer that projects meshes to screen space using non-linear projection methods */
class FDisplayClusterMeshProjectionRenderer
{
public:
	/** Clean up any references to the renderer. */
	DISPLAYCLUSTERLIGHTCARDEDITORSHADERS_API ~FDisplayClusterMeshProjectionRenderer();

	/** Projects a position in view coordinates into the projected view space of the specified projection type */
	DISPLAYCLUSTERLIGHTCARDEDITORSHADERS_API static FVector ProjectViewPosition(const FVector& ViewPosition, EDisplayClusterMeshProjectionType  ProjectionType);

	/** Projects a position in the projected view space of the specified projection type to ordinary view coordinates */
	DISPLAYCLUSTERLIGHTCARDEDITORSHADERS_API static FVector UnprojectViewPosition(const FVector& ProjectedViewPosition, EDisplayClusterMeshProjectionType  ProjectionType);

	/** Adds an actor's primitive components to the list of primitives to render */
	DISPLAYCLUSTERLIGHTCARDEDITORSHADERS_API void AddActor(AActor* Actor);

	/** Adds an actor's primitive components to the list of primitives to render, filtering which primitive components get rendered using the specified callback */
	DISPLAYCLUSTERLIGHTCARDEDITORSHADERS_API void AddActor(AActor* Actor, const TFunctionRef<bool(const UPrimitiveComponent*)>& PrimitiveFilter);

	/** Removes an actor's primitive components from the list of primitives to render */
	DISPLAYCLUSTERLIGHTCARDEDITORSHADERS_API void RemoveActor(AActor* Actor);

	/** Clears the list of primitives to render */
	DISPLAYCLUSTERLIGHTCARDEDITORSHADERS_API void ClearScene();

	/**
	* Renders its list of primitive components to the specified canvas using the desired projection type. Can be called from either the game thread or the rendering thread.
	*/
	DISPLAYCLUSTERLIGHTCARDEDITORSHADERS_API void Render(FCanvas* Canvas, FSceneInterface* Scene, const FDisplayClusterMeshProjectionRenderSettings& RenderSettings);

	/**
	* Renders its list of primitive components to the specified canvas using the desired projection type. Can be called from either the game thread or the rendering thread.
	* Render scene by scene
	*/
	DISPLAYCLUSTERLIGHTCARDEDITORSHADERS_API void RenderScenes(FCanvas* Canvas, const TArray<FSceneInterface*>& Scenes, const FDisplayClusterMeshProjectionRenderSettings& RenderSettings);

private:
	/** Constructs the necessary render passes for the default output of the rendered primitives */
	void RenderColorOutput(FRDGBuilder& GraphBuilder,
		const FSceneView* View,
		const FDisplayClusterMeshProjectionRenderSettings& RenderSettings,
		FRenderTargetBinding& OutputRenderTargetBinding);

	/** Constructs the necessary render passes for the default output of the rendered primitives */
	void RenderColorOutputs(FRDGBuilder& GraphBuilder,
		const TArray<const FSceneView*>& Views,
		const FDisplayClusterMeshProjectionRenderSettings& RenderSettings,
		FRenderTargetBinding& OutputRenderTargetBinding);

	/** Constructs the necessary render passes for the hit proxy output of the rendered primitives */
	void RenderHitProxyOutput(FRDGBuilder& GraphBuilder,
		const FSceneView* View,
		const FDisplayClusterMeshProjectionRenderSettings& RenderSettings,
		FRenderTargetBinding& OutputRenderTargetBinding,
		FHitProxyConsumer* HitProxyConsumer);

	/** Constructs the necessary render passes for the normals/depth output of the rendered primitives */
	void RenderNormalsOutput(FRDGBuilder& GraphBuilder,
		const FSceneView* View,
		const FDisplayClusterMeshProjectionRenderSettings& RenderSettings,
		FRenderTargetBinding& OutputRenderTargetBinding);

	/** Adds a pass to perform the base render for the mesh projection. */
	void AddBaseRenderPass(FRDGBuilder& GraphBuilder, 
		const FSceneView* View,
		const FDisplayClusterMeshProjectionRenderSettings& RenderSettings,
		FRenderTargetBinding& OutputRenderTargetBinding,
		FDepthStencilBinding& OutputDepthStencilBinding);

	/** Adds a pass to perform the translucency render for the mesh projection. */
	void AddTranslucencyRenderPass(FRDGBuilder& GraphBuilder, 
		const FSceneView* View,
		const FDisplayClusterMeshProjectionRenderSettings& RenderSettings,
		FRenderTargetBinding& OutputRenderTargetBinding,
		FDepthStencilBinding& OutputDepthStencilBinding);

	/** Adds a pass to perform the hit proxy render for the mesh projection */
	void AddHitProxyRenderPass(FRDGBuilder& GraphBuilder, 
		const FSceneView* View,
		const FDisplayClusterMeshProjectionRenderSettings& RenderSettings,
		FRenderTargetBinding& OutputRenderTargetBinding,
		FDepthStencilBinding& OutputDepthStencilBinding);

	/** Adds a pass to perform a render of the primitives' normals for the mesh projection */
	void AddNormalsRenderPass(FRDGBuilder& GraphBuilder,
		const FSceneView* View,
		const FDisplayClusterMeshProjectionRenderSettings& RenderSettings,
		FRenderTargetBinding& OutputRenderTargetBinding,
		FDepthStencilBinding& OutputDepthStencilBinding);

	/** Adds a pass to filter the output normal map for the mesh projection, dilating and blurring it
	 * to obtain a continuous normal map from primitives into empty space */
	void AddNormalsFilterPass(FRDGBuilder& GraphBuilder,
		const FSceneView* View,
		FRenderTargetBinding& OutputRenderTargetBinding,
		FRDGTexture* SceneColor,
		FRDGTexture* SceneDepth,
		const FMatrix44f& NormalCorrectionMatrix);

#if WITH_EDITOR
	/** Adds a pass to perform the depth render for any selected primitives for the mesh projection */
	void AddSelectionDepthRenderPass(FRDGBuilder& GraphBuilder, 
		const FSceneView* View,
		const FDisplayClusterMeshProjectionRenderSettings& RenderSettings,
		FDepthStencilBinding& OutputDepthStencilBinding);

	/** Adds a pass to perform the post process selection outline for the mesh projection */
	void AddSelectionOutlineScreenPass(FRDGBuilder& GraphBuilder,
		const FSceneView* View,
		FRenderTargetBinding& OutputRenderTargetBinding,
		FRDGTexture* SceneColor,
		FRDGTexture* SceneDepth,
		FRDGTexture* SelectionDepth);
#endif

	/** Adds a pass that renders any elements added to the PDI through the renderer's RenderSimpleElements callback */
	void AddSimpleElementPass(FRDGBuilder& GraphBuilder,
		const FSceneView* View,
		FRenderTargetBinding& OutputRenderTargetBinding,
		FSimpleElementCollector& ElementCollector);

	/** Renders the list of primitive components using the appropriate mesh pass processor given by the template parameter */
	template<EDisplayClusterMeshProjectionType ProjectionType>
	void RenderPrimitives_RenderThread(const FSceneView* View,
		FRHICommandList& RHICmdList,
		const FDisplayClusterMeshProjectionRenderSettings& RenderSettings,
		bool bTranslucencyPass);

	/** Renders the hit proxies of the list of primitive components using the appropriate mesh pass processor given by the template parameter */
	template<EDisplayClusterMeshProjectionType ProjectionType>
	void RenderHitProxies_RenderThread(const FSceneView* View,
		FRHICommandList& RHICmdList,
		const FDisplayClusterMeshProjectionRenderSettings& RenderSettings);

	/** Renders the normals of the list of primitive components using the appropriate mesh pass processor given by the template parameter */
	template<EDisplayClusterMeshProjectionType ProjectionType>
	void RenderNormals_RenderThread(const FSceneView* View,
		FRHICommandList& RHICmdList,
		const FDisplayClusterMeshProjectionRenderSettings& RenderSettings);

#if WITH_EDITOR
	/** Renders the list of primitive components using the appropriate mesh pass processor given by the template parameter */
	template<EDisplayClusterMeshProjectionType ProjectionType>
	void RenderSelection_RenderThread(const FSceneView* View,
		FRHICommandList& RHICmdList,
		const FDisplayClusterMeshProjectionRenderSettings& RenderSettings);
#endif


	/** Callback used to determine if a primitive component should be rendered with a selection outline */
	bool IsPrimitiveComponentSelected(const UPrimitiveComponent* InPrimitiveComponent);

	/** Data struct that contains the scene proxy and the render configuration needed to render the scene's primitive components */
	struct FSceneProxyElement
	{
		FPrimitiveSceneProxy* PrimitiveSceneProxy;
		bool bApplyProjection;
	};

	/** Collects all valid scene proxies to be rendered from the scene's primitive componets */
	void GetSceneProxies(TArray<FSceneProxyElement>& OutSceneProxyElements,
		const FDisplayClusterMeshProjectionRenderSettings& RenderSettings,
		const TFunctionRef<bool(const UPrimitiveComponent*)> PrimitiveFilter = [](const UPrimitiveComponent*) { return true; });

public:
	/** Delegate to determine if an actor should be rendered as selected */
	DECLARE_DELEGATE_RetVal_OneParam(bool, FSelection, const AActor*);
	FSelection ActorSelectedDelegate;

	/** Delegate raised during a render pass allowing simple elements ro be rendered to the viewport after meshes are projected to it */
	DECLARE_DELEGATE_TwoParams(FSimpleElementPass, const FSceneView*, FPrimitiveDrawInterface*);
	FSimpleElementPass RenderSimpleElementsDelegate;

private:
	/** The list of primitive components to render */
	TArray<TWeakObjectPtr<UPrimitiveComponent>> PrimitiveComponents;
};