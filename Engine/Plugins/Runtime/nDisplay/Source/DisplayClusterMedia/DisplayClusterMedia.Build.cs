// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DisplayClusterMedia : ModuleRules
{
	public DisplayClusterMedia(ReadOnlyTargetRules ROTargetRules) : base(ROTargetRules)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"DisplayClusterConfiguration",
				"Media",
				"MediaAssets",
				"MediaIOCore",
			});

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"DisplayCluster",
				"DisplayClusterShaders",
				"Engine",
				"OpenColorIO",
				"Renderer",
				"RenderCore",
				"RHI",
			});

		if (Target.bBuildEditor == true)
		{
			PrivateDependencyModuleNames.Add("UnrealEd");
		}

		if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
		{
			AddEngineThirdPartyPrivateStaticDependencies(Target, "DX12");
			PrivateDependencyModuleNames.Add("D3D12RHI");
		}
	}
}
