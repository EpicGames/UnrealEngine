// Copyright Epic Games, Inc. All Rights Reserved.

#include "Capture/DisplayClusterMediaCaptureBase.h"

#include "DisplayClusterMediaHelpers.h"
#include "DisplayClusterMediaLog.h"
#include "DisplayClusterConfigurationTypes_Media.h"
#include "DisplayClusterConfigurationTypes_MediaSync.h"

#include "IDisplayCluster.h"
#include "IDisplayClusterCallbacks.h"
#include "IDisplayClusterShaders.h"

#include "MediaCapture.h"
#include "MediaOutput.h"

#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "RHICommandList.h"
#include "RHIUtilities.h"

#include "ShaderParameters/DisplayClusterShaderParameters_Media.h"

#include "UObject/UObjectGlobals.h"
#include "UObject/Package.h"


FDisplayClusterMediaCaptureBase::FDisplayClusterMediaCaptureBase(
	const FString& InMediaId,
	const FString& InClusterNodeId,
	UMediaOutput* InMediaOutput,
	UDisplayClusterMediaOutputSynchronizationPolicy* InSyncPolicy
)
	: FDisplayClusterMediaBase(InMediaId, InClusterNodeId)
	, SyncPolicy(InSyncPolicy)
{
	checkSlow(InMediaOutput);

	// Here we expect to receive
	// {0, 0}, if no capture size requested
	// {X > 0, Y > 0}, if explicit capture size is specified
	CustomResolution = InMediaOutput->GetRequestedSize();

	MediaOutput = DuplicateObject(InMediaOutput, GetTransientPackage());
	checkSlow(MediaOutput);

	IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPostTick().AddRaw(this, &FDisplayClusterMediaCaptureBase::OnPostClusterTick);
}


FDisplayClusterMediaCaptureBase::~FDisplayClusterMediaCaptureBase()
{
	IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPostTick().RemoveAll(this);
}

void FDisplayClusterMediaCaptureBase::AddReferencedObjects(FReferenceCollector& Collector)
{
	if (MediaOutput)
	{
		Collector.AddReferencedObject(MediaOutput);
	}

	if (MediaCapture)
	{
		Collector.AddReferencedObject(MediaCapture);
	}

	if (SyncPolicy)
	{
		Collector.AddReferencedObject(SyncPolicy);
	}
}

bool FDisplayClusterMediaCaptureBase::StartCapture()
{
	if (MediaOutput && !MediaCapture)
	{
		MediaCapture = MediaOutput->CreateMediaCapture();
		if (IsValid(MediaCapture))
		{
			MediaCapture->SetMediaOutput(MediaOutput);

			// Initialize and start capture synchronization
			if (IsValid(SyncPolicy))
			{
				SyncPolicyHandler = SyncPolicy->GetHandler();
				if (SyncPolicyHandler)
				{
					if (SyncPolicyHandler->IsCaptureTypeSupported(MediaCapture))
					{
						if (SyncPolicyHandler->StartSynchronization(MediaCapture, GetMediaId()))
						{
							UE_LOGF(LogDisplayClusterMedia, Log, "MediaCapture '%ls' started synchronization type '%ls'.", *GetMediaId(), *SyncPolicy->GetName());
						}
						else
						{
							UE_LOGF(LogDisplayClusterMedia, Warning, "MediaCapture '%ls': couldn't start synchronization.", *GetMediaId());
						}
					}
					else
					{
						UE_LOGF(LogDisplayClusterMedia, Warning, "MediaCapture '%ls' is not compatible with media SyncPolicy '%ls'.", *GetMediaId(), *SyncPolicy->GetName());
					}
				}
				else
				{
					UE_LOGF(LogDisplayClusterMedia, Warning, "Could not create media sync policy handler from '%ls'.", *SyncPolicy->GetName());
				}
			}

			bWasCaptureStarted = StartMediaCapture();
			return bWasCaptureStarted;
		}
		else
		{
			UE_LOGF(LogDisplayClusterMedia, Warning, "Failed to create MediaCapture '%ls' from MediaOutput", *GetMediaId());
		}
	}

	return false;
}

void FDisplayClusterMediaCaptureBase::StopCapture()
{
	// Stop synchronization
	if (SyncPolicyHandler)
	{
		SyncPolicyHandler->StopSynchronization();
	}

	// Stop capture
	if (MediaCapture)
	{
		MediaCapture->StopCapture(false);
		MediaCapture = nullptr;
		bWasCaptureStarted = false;
	}
}

void FDisplayClusterMediaCaptureBase::ExportMediaData_RenderThread(FRDGBuilder& GraphBuilder, const FMediaOutputTextureInfo& TextureInfo)
{
	// Check if MediaCapture is available
	if (!MediaCapture)
	{
		UE_LOGF(LogDisplayClusterMedia, Verbose, "MediaCapture '%ls' is null, unable to export on RT frame %llu", *GetMediaId(), GFrameCounterRenderThread);
		return;
	}

	// Check if request data is valid
	if (!IsValidRequestData(TextureInfo))
	{
		UE_LOGF(LogDisplayClusterMedia, Warning, "MediaCapture '%ls': no capture performed on RT frame %llu", *GetMediaId(), GFrameCounterRenderThread);
		return;
	}

	MediaCapture->SetValidSourceGPUMask(GraphBuilder.RHICmdList.GetGPUMask());

	{
		const FIntPoint& SrcTextureSize = TextureInfo.Texture->Desc.Extent;
		const FIntPoint  SrcRegionSize = TextureInfo.Region.Size();

		LastSrcRegionSize = FIntSize(SrcRegionSize);

		UE_LOGF(LogDisplayClusterMedia, VeryVerbose, "MediaCapture '%ls': Requested texture export [size=%dx%d, rect=%dx%d, format=%u] on RT frame '%llu'...",
			*GetMediaId(), SrcTextureSize.X, SrcTextureSize.Y, SrcRegionSize.X, SrcRegionSize.Y, (uint32)TextureInfo.Texture->Desc.Format, GFrameCounterRenderThread);
	}

	bool bCaptureSucceeded = false;

	// Is PQ-encoding required?
	constexpr bool bConsideringLateOCIOEnabled = true;
	const bool bLateOCIOWithPQTransfer = IsTransferPQ(bConsideringLateOCIOEnabled);

	// When PQ encoding is required, we have to add a separate PQ-encoding pass
	if (bLateOCIOWithPQTransfer)
	{
		// Allocate intermediate PQ texture of PF_A2B10G10R10 pixel format
		FRDGTextureDesc TexturePQDesc = FRDGTextureDesc::Create2D(TextureInfo.Region.Size(), PF_A2B10G10R10, FClearValueBinding::Black, TexCreate_ShaderResource | TexCreate_RenderTargetable);
		FRDGTextureRef TexturePQ = GraphBuilder.CreateTexture(TexturePQDesc, TEXT("DC.MediaTexturePQ"));

		// Add PQ-encoding pass
		FDisplayClusterShaderParameters_MediaPQ Parameters{ TextureInfo.Texture, TextureInfo.Region, TexturePQ, { {0, 0}, TexturePQ->Desc.Extent } };
		IDisplayClusterShaders::Get().AddLinearToPQPass(GraphBuilder, Parameters);

		UE_LOGF(LogDisplayClusterMedia, VeryVerbose, "MediaCapture '%ls': PQ exporting TexSize[%ls], TexRect[%ls], TexFormat[%u] on RT frame '%llu'...",
			*GetMediaId(),
			*Parameters.OutputTexture->Desc.Extent.ToString(),
			*Parameters.OutputRect.ToString(),
			(uint32)Parameters.OutputTexture->Desc.Format,
			GFrameCounterRenderThread);

		// Pass the texture to the capture device
		bCaptureSucceeded = MediaCapture->TryCaptureImmediate_RenderThread(GraphBuilder, Parameters.OutputTexture, Parameters.OutputRect);
	}
	else
	{
		UE_LOGF(LogDisplayClusterMedia, VeryVerbose, "MediaCapture '%ls': Direct exporting TexSize[%ls], TexRect[%ls], TexFormat[%u] on RT frame '%llu'...",
			*GetMediaId(),
			*TextureInfo.Texture->Desc.Extent.ToString(),
			*TextureInfo.Region.ToString(),
			(uint32)TextureInfo.Texture->Desc.Format,
			GFrameCounterRenderThread);

		// Direct capture
		bCaptureSucceeded = MediaCapture->TryCaptureImmediate_RenderThread(GraphBuilder, TextureInfo.Texture, TextureInfo.Region);
	}

	if(!bCaptureSucceeded)
	{
		UE_LOGF(LogDisplayClusterMedia, Warning, "MediaCapture '%ls': failed to capture resource", *GetMediaId());
	}
}

bool FDisplayClusterMediaCaptureBase::IsValidRequestData(const FMediaOutputTextureInfo& TextureInfo) const
{
	// Check if source texture is valid
	if (!TextureInfo.Texture)
	{
		UE_LOGF(LogDisplayClusterMedia, Warning, "MediaCapture '%ls': invalid source texture on RT frame %llu", *GetMediaId(), GFrameCounterRenderThread);
		return false;
	}

	// Check if region matches the texture
	const FIntPoint RegionSize = TextureInfo.Region.Size();
	const bool bCorrectRegion =
		TextureInfo.Region.Min.X >= 0 &&
		TextureInfo.Region.Min.Y >= 0 &&
		RegionSize.X > 0 &&
		RegionSize.Y > 0 &&
		RegionSize.X <= TextureInfo.Texture->Desc.Extent.X &&
		RegionSize.Y <= TextureInfo.Texture->Desc.Extent.Y;

	if (!bCorrectRegion)
	{
		UE_LOGF(LogDisplayClusterMedia, Warning, "MediaCapture '%ls': invalid source region on RT frame %llu", *GetMediaId(), GFrameCounterRenderThread);
		return false;
	}

	return true;
}

void FDisplayClusterMediaCaptureBase::OnPostClusterTick()
{
	if (MediaCapture)
	{
		EMediaCaptureState MediaCaptureState = MediaCapture->GetState();

		// If we're capturing but the desired capture resolution does not match the texture being captured,
		// restart the capture with the updated size.

		if (MediaCaptureState == EMediaCaptureState::Capturing)
		{
			const FIntPoint LastSrcRegionIntPoint = LastSrcRegionSize.load().ToIntPoint();
			const FIntPoint DesiredSize = MediaCapture->GetDesiredSize();
			const bool bHasExportedAlready = (LastSrcRegionIntPoint != FIntPoint::ZeroValue);
			const bool bDesiredSizeMatchesLastRegion = (DesiredSize == LastSrcRegionIntPoint);
			const bool bDesiredSizeMatchesCustomRegion = (CustomResolution.X > 0 && CustomResolution.Y > 0 && DesiredSize == CustomResolution);

			// We don't restart if we haven't exported any textures yet (indicated by zero size LastSrcRegion)
			// to avoid constant media restarts since in such case media is set to use GetCaptureSize() != (0.0).
			// Once an export happens, any media restart will use LastSrcRegion which should not trigger a restart
			// when texture exports are suspended, since that does not cause a mismatch. 
			// This is preferred to setting LastSrcRegion to GetCaptureSize() because that would not reflect the actual
			// last captured size.

			if (bHasExportedAlready && !(bDesiredSizeMatchesLastRegion || bDesiredSizeMatchesCustomRegion))
			{
				UE_LOGF(LogDisplayClusterMedia, Log, "Stopping MediaCapture '%ls' because its DesiredSize (%d, %d) doesn't match the captured texture size (%d, %d)", 
					*GetMediaId(), DesiredSize.X, DesiredSize.Y, LastSrcRegionIntPoint.X, LastSrcRegionIntPoint.Y);

				MediaCapture->StopCapture(false /* bAllowPendingFrameToBeProcess */);
				MediaCaptureState = MediaCapture->GetState(); // Re-sample state to restart the media capture right away
			}
		}

		const bool bMediaCaptureNeedsRestart = (MediaCaptureState == EMediaCaptureState::Error) || (MediaCaptureState == EMediaCaptureState::Stopped);

		if (!bWasCaptureStarted || bMediaCaptureNeedsRestart)
		{
			constexpr double Interval = 1.0;
			const double CurrentTime = FPlatformTime::Seconds();

			if (CurrentTime - LastRestartTimestamp > Interval)
			{
				UE_LOGF(LogDisplayClusterMedia, Log, "MediaCapture '%ls' is in error or stopped, restarting it.", *GetMediaId());

				bWasCaptureStarted = StartMediaCapture();
				LastRestartTimestamp = CurrentTime;
			}
		}
	}
}

bool FDisplayClusterMediaCaptureBase::StartMediaCapture()
{
	FRHICaptureResourceDescription Descriptor;

	if (LastSrcRegionSize.load().ToIntPoint() == FIntPoint::ZeroValue)
	{
		Descriptor.ResourceSize = GetCaptureSize();
	}
	else
	{
		Descriptor.ResourceSize = LastSrcRegionSize.load().ToIntPoint();
	}
	
	if (Descriptor.ResourceSize == FIntPoint::ZeroValue)
	{
		return false;
	}

	FMediaCaptureOptions MediaCaptureOptions;
	MediaCaptureOptions.NumberOfFramesToCapture = -1;
	MediaCaptureOptions.bAutoRestartOnSourceSizeChange = false; // true won't work due to MediaCapture auto-changing crop mode to custom when capture region is specified.
	MediaCaptureOptions.bSkipFrameWhenRunningExpensiveTasks = false;
	MediaCaptureOptions.OverrunAction = EMediaCaptureOverrunAction::Flush;

	// When custom resolution is required, we should always scale the output to the required size
	if (CustomResolution.X > 0 && CustomResolution.Y > 0)
	{
		Descriptor.ResourceSize = CustomResolution;
		MediaCaptureOptions.Crop = EMediaCaptureCroppingType::None;
		MediaCaptureOptions.ResizeMethod = EMediaCaptureResizeMethod::ResizeInRenderPass;
	}

	const bool bCaptureStarted = MediaCapture->CaptureRHITexture(Descriptor, MediaCaptureOptions);

	if (bCaptureStarted)
	{
		UE_LOGF(LogDisplayClusterMedia, Log, "Started media capture: '%ls' (%d x %d)",
			*GetMediaId(), Descriptor.ResourceSize.X, Descriptor.ResourceSize.Y);
	}
	else
	{
		UE_LOGF(LogDisplayClusterMedia, Warning, "Couldn't start media capture '%ls' (%d x %d)", 
			*GetMediaId(), Descriptor.ResourceSize.X, Descriptor.ResourceSize.Y);
	}

	return bCaptureStarted;
}
