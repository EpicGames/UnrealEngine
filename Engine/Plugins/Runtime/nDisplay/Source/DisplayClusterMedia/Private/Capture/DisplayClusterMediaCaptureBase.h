// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Common/DisplayClusterMediaBase.h"
#include "UObject/GCObject.h"

#include "RenderGraphFwd.h"

#include "RHI.h"
#include "RHIResources.h"

class UMediaCapture;
class UMediaOutput;
class UDisplayClusterMediaOutputSynchronizationPolicy;
class IDisplayClusterMediaOutputSynchronizationPolicyHandler;


/**
 * Base media capture adapter class
 */
class FDisplayClusterMediaCaptureBase
	: public FDisplayClusterMediaBase
	, public FGCObject
{
public:

	FDisplayClusterMediaCaptureBase(
		const FString& MediaId,
		const FString& ClusterNodeId,
		UMediaOutput* MediaOutput,
		UDisplayClusterMediaOutputSynchronizationPolicy* SyncPolicy
	);

	virtual ~FDisplayClusterMediaCaptureBase();

public:

	//~ Begin FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FDisplayClusterMediaCaptureBase");
	}
	//~ End FGCObject interface

public:

	/** Start capturing */
	virtual bool StartCapture();

	/** Stop capturing */
	virtual void StopCapture();

	/** Returns current media capture device */
	UMediaCapture* GetMediaCapture() const
	{
		return MediaCapture;
	}

protected:

	/** Media capture data */
	struct FMediaOutputTextureInfo
	{
		/** Texture to capture by a media capture device */
		FRDGTextureRef Texture = nullptr;

		/** Subregion to capture */
		FIntRect Region = { FIntPoint::ZeroValue, FIntPoint::ZeroValue };
	};

protected:

	/** PostClusterTick event handler. It's used to restart capturing if needed */
	void OnPostClusterTick();

	/** Re-starts media capturing after failure */
	bool StartMediaCapture();

	/** Passes capture data request to the capture device */
	void ExportMediaData_RenderThread(FRDGBuilder& GraphBuilder, const FMediaOutputTextureInfo& TextureInfo);

	/** Returns capture size (main thread) */
	virtual FIntPoint GetCaptureSize() const = 0;

private:

	/** Validate if capture request data is valid */
	bool IsValidRequestData(const FMediaOutputTextureInfo& TextureInfo) const;

private:

	/** Trivial version of FIntPoint so that it can be std::atomic */
	struct FIntSize
	{
		int32 X = 0;
		int32 Y = 0;

		FIntSize(int32 InX, int32 InY) : X(InX), Y(InY) {}
		FIntSize(const FIntPoint& IntPoint) : X(IntPoint.X), Y(IntPoint.Y) {}

		FIntPoint ToIntPoint()
		{
			return FIntPoint(X, Y);
		}
	};

private:

	//~ Begin GC by AddReferencedObjects
	TObjectPtr<UMediaOutput>  MediaOutput;
	TObjectPtr<UMediaCapture> MediaCapture;
	TObjectPtr<UDisplayClusterMediaOutputSynchronizationPolicy> SyncPolicy;
	//~ End GC by AddReferencedObjects

	/** Custom resolution to use on capture side */
	FIntPoint CustomResolution = FIntPoint::ZeroValue;

	/** Used to restart media capture in the case it falls in error */
	bool bWasCaptureStarted = false;

	/** Used to control the rate at which we try to restart the capture */
	double LastRestartTimestamp = 0;

	/** Last region size of the texture being exported.Used to restart the capture when in error. */
	std::atomic<FIntSize> LastSrcRegionSize { FIntSize(0,0) };

	/** Sync policy handler to deal with synchronization logic */
	TSharedPtr<IDisplayClusterMediaOutputSynchronizationPolicyHandler> SyncPolicyHandler;
};
