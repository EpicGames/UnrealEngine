// Copyright Epic Games, Inc. All Rights Reserved.

#include "Capture/DisplayClusterMediaCaptureCameraBase.h"


FDisplayClusterMediaCaptureCameraBase::FDisplayClusterMediaCaptureCameraBase(
	const FString& InMediaId,
	const FString& InClusterNodeId,
	const FString& InCameraId,
	const FString& InCameraViewportId,
	UMediaOutput* InMediaOutput,
	UDisplayClusterMediaOutputSynchronizationPolicy* InSyncPolicy
)
	: FDisplayClusterMediaCaptureViewportBase(InMediaId, InClusterNodeId, InCameraViewportId, InMediaOutput, InSyncPolicy)
	, FDisplayClusterMediaCameraCommon(InCameraId)
{
}


void FDisplayClusterMediaCaptureCameraBase::UpdateLateOCIOState(const IDisplayClusterViewport* Viewport)
{
	FLateOCIOData NewLateOCIOConfiguration;

	// Get current late OCIO parameters from the camera component
	FDisplayClusterMediaCameraCommon::GetLateOCIOParameters(NewLateOCIOConfiguration.bLateOCIO, NewLateOCIOConfiguration.bTransferPQ);

	// And store/update for current frame
	SetLateOCIO(NewLateOCIOConfiguration);
}

void FDisplayClusterMediaCaptureCameraBase::UpdateMediaPassthrough(const IDisplayClusterViewport* Viewport)
{
	// No implementation for camera so far. It's unlikely we'd ever need media passthrough for ICVFX cameras.
}
