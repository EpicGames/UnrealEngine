// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Common/DisplayClusterMediaCameraCommon.h"
#include "Capture/DisplayClusterMediaCaptureViewportBase.h"


/**
 * Base camera capture adapter
 */
class FDisplayClusterMediaCaptureCameraBase
	: public FDisplayClusterMediaCaptureViewportBase
	, public FDisplayClusterMediaCameraCommon
{
public:

	FDisplayClusterMediaCaptureCameraBase(
		const FString& MediaId,
		const FString& ClusterNodeId,
		const FString& CameraId,
		const FString& CameraViewportId,
		UMediaOutput* MediaOutput,
		UDisplayClusterMediaOutputSynchronizationPolicy* SyncPolicy
	);

protected:

	/** Updates late OCIO state */
	virtual void UpdateLateOCIOState(const IDisplayClusterViewport* Viewport) override;

	/** Checks whether current frame should use media passthrough */
	virtual void UpdateMediaPassthrough(const IDisplayClusterViewport* Viewport) override;
};
