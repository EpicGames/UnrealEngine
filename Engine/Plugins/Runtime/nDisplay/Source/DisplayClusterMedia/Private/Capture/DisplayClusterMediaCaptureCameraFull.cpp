// Copyright Epic Games, Inc. All Rights Reserved.

#include "Capture/DisplayClusterMediaCaptureCameraFull.h"

#include "Components/DisplayClusterICVFXCameraComponent.h"

#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterMediaHelpers.h"


FDisplayClusterMediaCaptureCameraFull::FDisplayClusterMediaCaptureCameraFull(
	const FString& InMediaId,
	const FString& InClusterNodeId,
	const FString& InCameraId,
	UMediaOutput* InMediaOutput,
	UDisplayClusterMediaOutputSynchronizationPolicy* InSyncPolicy
)
	: FDisplayClusterMediaCaptureCameraBase(
		InMediaId,
		InClusterNodeId,
		InCameraId,
		DisplayClusterMediaHelpers::GenerateICVFXViewportName(InClusterNodeId, InCameraId),
		InMediaOutput,
		InSyncPolicy)
{
}

bool FDisplayClusterMediaCaptureCameraFull::GetCaptureSizeFromConfig(FIntPoint& OutSize) const
{
	const UDisplayClusterICVFXCameraComponent* const ICVFXCameraComponent = GetCameraComponent();
	if (!ICVFXCameraComponent)
	{
		return false;
	}

	const FDisplayClusterConfigurationICVFX_CameraSettings& CameraSettings = ICVFXCameraComponent->GetCameraSettingsICVFX();

	if (CameraSettings.CustomFrustum.bEnable)
	{
		OutSize = CameraSettings.CustomFrustum.EstimatedOverscanResolution;
	}
	else
	{
		OutSize = CameraSettings.CustomFrustum.InnerFrustumResolution;
	}

	return true;
}
