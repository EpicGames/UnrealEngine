// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Capture/DisplayClusterMediaCaptureCameraBase.h"


/**
 * Camera capture adapter (full frame)
 */
class FDisplayClusterMediaCaptureCameraFull
	: public FDisplayClusterMediaCaptureCameraBase
{
public:

	FDisplayClusterMediaCaptureCameraFull(
		const FString& MediaId,
		const FString& ClusterNodeId,
		const FString& CameraId,
		UMediaOutput* MediaOutput,
		UDisplayClusterMediaOutputSynchronizationPolicy* SyncPolicy
	);

public:

	//~ Begin FDisplayClusterMediaCaptureViewport
	virtual bool GetCaptureSizeFromConfig(FIntPoint& OutSize) const override;
	//~ End FDisplayClusterMediaCaptureViewport
};
