// Copyright Epic Games, Inc. All Rights Reserved.

#include "Capture/DisplayClusterMediaCaptureCameraTile.h"

#include "DisplayClusterMediaHelpers.h"


FDisplayClusterMediaCaptureCameraTile::FDisplayClusterMediaCaptureCameraTile(
	const FString& InMediaId,
	const FString& InClusterNodeId,
	const FString& InCameraId,
	const FIntPoint& InTilePosition,
	UMediaOutput* InMediaOutput,
	UDisplayClusterMediaOutputSynchronizationPolicy* InSyncPolicy
)
	: FDisplayClusterMediaCaptureCameraBase(
		InMediaId,
		InClusterNodeId,
		InCameraId,
		DisplayClusterMediaHelpers::GenerateICVFXTileViewportName(InClusterNodeId, InCameraId, InTilePosition),
		InMediaOutput,
		InSyncPolicy)
{
}


bool FDisplayClusterMediaCaptureCameraTile::GetCaptureSizeFromConfig(FIntPoint& OutSize) const
{
	// The same workaround as the viewport tile has

	OutSize = { 64, 64 };

	return true;
}
