// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Capture/DisplayClusterMediaCaptureCameraBase.h"


/**
 * Camera capture adapter (tile)
 */
class FDisplayClusterMediaCaptureCameraTile
	: public FDisplayClusterMediaCaptureCameraBase
{
public:

	FDisplayClusterMediaCaptureCameraTile(
		const FString& MediaId,
		const FString& ClusterNodeId,
		const FString& CameraId,
		const FIntPoint& TilePosition,
		UMediaOutput* MediaOutput,
		UDisplayClusterMediaOutputSynchronizationPolicy* SyncPolicy
	);

protected:

	//~ Begin FDisplayClusterMediaCaptureViewport
	virtual bool GetCaptureSizeFromConfig(FIntPoint& OutSize) const override;
	//~ End FDisplayClusterMediaCaptureViewport
};
