// Copyright Epic Games, Inc. All Rights Reserved.

#include "Capture/DisplayClusterMediaCaptureNodeBase.h"

#include "Config/IDisplayClusterConfigManager.h"
#include "Engine/Engine.h"

#include "DisplayClusterConfigurationTypes.h"

#include "IDisplayCluster.h"
#include "IDisplayClusterCallbacks.h"

#include "RHICommandList.h"
#include "RHIResources.h"


FDisplayClusterMediaCaptureNodeBase::FDisplayClusterMediaCaptureNodeBase(
	const FString& InMediaId,
	const FString& InClusterNodeId,
	UMediaOutput* InMediaOutput,
	UDisplayClusterMediaOutputSynchronizationPolicy* SyncPolicy
)
	: FDisplayClusterMediaCaptureBase(InMediaId, InClusterNodeId, InMediaOutput, SyncPolicy)
{
}


bool FDisplayClusterMediaCaptureNodeBase::StartCapture()
{
	// If capturing initialized and started successfully, subscribe for rendering callbacks
	if (FDisplayClusterMediaCaptureBase::StartCapture())
	{
		IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPostBackbufferUpdated_RenderThread().AddRaw(this, &FDisplayClusterMediaCaptureNodeBase::OnPostBackbufferUpdated_RenderThread);
		return true;
	}

	return false;
}

void FDisplayClusterMediaCaptureNodeBase::StopCapture()
{
	// Stop rendering notifications
	IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPostBackbufferUpdated_RenderThread().RemoveAll(this);

	// Stop capturing
	FDisplayClusterMediaCaptureBase::StopCapture();
}

void FDisplayClusterMediaCaptureNodeBase::OnPostBackbufferUpdated_RenderThread(FRHICommandListImmediate& RHICmdList, FViewport* Viewport)
{
	// Pass to the children
	ProcessPostBackbufferUpdated_RenderThread(RHICmdList, Viewport);
}
