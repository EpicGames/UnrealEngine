// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Capture/DisplayClusterMediaCaptureBase.h"

class FRHICommandListImmediate;
class FViewport;
class UDisplayClusterMediaOutputSynchronizationPolicy;
class UMediaOutput;


/**
 * Node backbuffer media capture base class
 */
class FDisplayClusterMediaCaptureNodeBase
	: public FDisplayClusterMediaCaptureBase
{
public:

	/** Constructor */
	FDisplayClusterMediaCaptureNodeBase(
		const FString& MediaId,
		const FString& ClusterNodeId,
		UMediaOutput* MediaOutput,
		UDisplayClusterMediaOutputSynchronizationPolicy* SyncPolicy
	);

public:

	/** Start backbuffer capture */
	virtual bool StartCapture() override;

	/** Stop backbuffer capture */
	virtual void StopCapture() override;

protected:

	/** PostBackbufferUpdated implementation on the children side */
	virtual void ProcessPostBackbufferUpdated_RenderThread(FRHICommandListImmediate& RHICmdList, FViewport* Viewport) = 0;

private:

	/** Main PostBackbufferUpdated event handler. It's passed as virtual ProcessPostBackbufferUpdated_RenderThread call to the children. */
	void OnPostBackbufferUpdated_RenderThread(FRHICommandListImmediate& RHICmdList, FViewport* Viewport);
};
