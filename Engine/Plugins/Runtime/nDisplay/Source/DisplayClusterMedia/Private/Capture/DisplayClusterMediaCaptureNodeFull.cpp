// Copyright Epic Games, Inc. All Rights Reserved.

#include "Capture/DisplayClusterMediaCaptureNodeFull.h"

#include "DisplayClusterMediaLog.h"

#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "RHICommandList.h"
#include "RHIResources.h"
#include "UnrealClient.h"

#include "Engine/Engine.h"
#include "Engine/GameViewportClient.h"


FDisplayClusterMediaCaptureNodeFull::FDisplayClusterMediaCaptureNodeFull(
	const FString& InMediaId,
	const FString& InClusterNodeId,
	UMediaOutput* InMediaOutput,
	UDisplayClusterMediaOutputSynchronizationPolicy* SyncPolicy
)
	: FDisplayClusterMediaCaptureNodeBase(InMediaId, InClusterNodeId, InMediaOutput, SyncPolicy)
{
}

FIntPoint FDisplayClusterMediaCaptureNodeFull::GetCaptureSize() const
{
	// Return backbuffer runtime size
	if (GEngine && GEngine->GameViewport && GEngine->GameViewport->Viewport)
	{
		const FIntPoint Size = GEngine->GameViewport->Viewport->GetSizeXY();

		UE_LOGF(LogDisplayClusterMedia, Log, "'%ls' capture size is [%d, %d]", *GetMediaId(), Size.X, Size.Y);

		return Size;
	}

	UE_LOGF(LogDisplayClusterMedia, Warning, "'%ls' couldn't get viewport size", *GetMediaId());

	return FIntPoint();
}

void FDisplayClusterMediaCaptureNodeFull::ProcessPostBackbufferUpdated_RenderThread(FRHICommandListImmediate& RHICmdList, FViewport* Viewport)
{
	if (!Viewport)
	{
		UE_LOGF(LogDisplayClusterMedia, Warning, "'%ls' capture failed, got invalid viewport", *GetMediaId());
		return;
	}

	if (FRHITexture* const BackbufferTexture = Viewport->GetRenderTargetTexture())
	{
		FRDGBuilder GraphBuilder(RHICmdList);

		// Prepare capture request data
		FRDGTextureRef BackbufferTextureRef = RegisterExternalTexture(GraphBuilder, BackbufferTexture, TEXT("DCMediaOutBackbufferTex"));
		const FIntRect TextureRegion = { FIntPoint::ZeroValue, BackbufferTextureRef->Desc.Extent };
		const FMediaOutputTextureInfo TextureInfo{ BackbufferTextureRef, TextureRegion };

		UE_LOGF(LogDisplayClusterMedia, VeryVerbose, "'%ls' capturing backbuffer of size %dx%d",
			*GetMediaId(), BackbufferTextureRef->Desc.Extent.X, BackbufferTextureRef->Desc.Extent.Y);

		// Capture backbuffer
		ExportMediaData_RenderThread(GraphBuilder, TextureInfo);

		GraphBuilder.Execute();
	}
}
