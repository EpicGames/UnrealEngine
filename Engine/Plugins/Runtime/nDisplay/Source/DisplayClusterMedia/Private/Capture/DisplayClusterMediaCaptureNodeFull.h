// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Capture/DisplayClusterMediaCaptureNodeBase.h"

class FRHICommandListImmediate;
class FViewport;
class UDisplayClusterMediaOutputSynchronizationPolicy;
class UMediaOutput;


/**
 * Node backbuffer media capture (whole buffer)
 */
class FDisplayClusterMediaCaptureNodeFull
	: public FDisplayClusterMediaCaptureNodeBase
{
public:

	/** Constructor */
	FDisplayClusterMediaCaptureNodeFull(
		const FString& MediaId,
		const FString& ClusterNodeId,
		UMediaOutput* MediaOutput,
		UDisplayClusterMediaOutputSynchronizationPolicy* SyncPolicy = nullptr
	);

protected:

	/** Returns backbuffer size */
	virtual FIntPoint GetCaptureSize() const override;

	/** PostBackbufferUpdated handler */
	virtual void ProcessPostBackbufferUpdated_RenderThread(FRHICommandListImmediate& RHICmdList, FViewport* Viewport) override;
};
