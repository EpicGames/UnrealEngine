// Copyright Epic Games, Inc. All Rights Reserved.

#include "Capture/DisplayClusterMediaCaptureNodeTile.h"

#include "DisplayClusterMediaHelpers.h"
#include "DisplayClusterMediaLog.h"

#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "RHICommandList.h"
#include "RHIResources.h"
#include "UnrealClient.h"

#include "Engine/Engine.h"
#include "Engine/GameViewportClient.h"


FDisplayClusterMediaCaptureNodeTile::FDisplayClusterMediaCaptureNodeTile(
	const FString& InMediaId,
	const FString& InClusterNodeId,
	const FIntPoint& InTileLayout,
	const FIntPoint& InTilePosition,
	UMediaOutput* InMediaOutput,
	UDisplayClusterMediaOutputSynchronizationPolicy* SyncPolicy
)
	: FDisplayClusterMediaCaptureNodeBase(InMediaId, InClusterNodeId, InMediaOutput, SyncPolicy)
	, bValidTileSettings(
		DisplayClusterMediaHelpers::IsValidLayout(InTileLayout, GetMaxTileLayout()) &&   // Valid layout
		DisplayClusterMediaHelpers::IsValidTileCoordinate(InTilePosition, InTileLayout)) // Valid coordinates
	, bEndingX(InTilePosition.X == InTileLayout.X - 1)
	, bEndingY(InTilePosition.Y == InTileLayout.Y - 1)
	, TileLayout(InTileLayout)
	, TilePosition(InTilePosition)
{
	checkSlow(bValidTileSettings);
}

bool FDisplayClusterMediaCaptureNodeTile::StartCapture()
{
	// Don't start capture if there is something invalid
	if (bValidTileSettings)
	{
		return FDisplayClusterMediaCaptureNodeBase::StartCapture();
	}

	return false;
}

FIntPoint FDisplayClusterMediaCaptureNodeTile::GetCaptureSize() const
{
	// Return backbuffer runtime size
	if (!(GEngine && GEngine->GameViewport && GEngine->GameViewport->Viewport))
	{
		UE_LOGF(LogDisplayClusterMedia, Warning, "'%ls' couldn't get viewport size", *GetMediaId());
		return { };
	}

	const FIntPoint GameViewportSize = GEngine->GameViewport->Viewport->GetSizeXY();

	// Non-ending tile width/height for reference
	const int32 NonEndingTileWidth  = GameViewportSize.X / TileLayout.X;
	const int32 NonEndingTileHeight = GameViewportSize.Y / TileLayout.Y;

	// Ending tile width/height for reference
	const int32 EndingTileWidth  = (GameViewportSize.X % TileLayout.X == 0 ? NonEndingTileWidth  : GameViewportSize.X % TileLayout.X);
	const int32 EndingTileHeight = (GameViewportSize.Y % TileLayout.Y == 0 ? NonEndingTileHeight : GameViewportSize.Y % TileLayout.Y);

	// This tile size
	const int32 ThisTileWidth  = bEndingX ? EndingTileWidth  : NonEndingTileWidth;
	const int32 ThisTileHeight = bEndingY ? EndingTileHeight : NonEndingTileHeight;

	UE_LOGF(LogDisplayClusterMedia, Log, "'%ls' capture size is [%d, %d]", *GetMediaId(), ThisTileWidth, ThisTileHeight);

	return FIntPoint{ ThisTileWidth, ThisTileHeight };
}

void FDisplayClusterMediaCaptureNodeTile::ProcessPostBackbufferUpdated_RenderThread(FRHICommandListImmediate& RHICmdList, FViewport* Viewport)
{
	if (!Viewport)
	{
		UE_LOGF(LogDisplayClusterMedia, Warning, "'%ls' couldn't capture backbuffer tile [%d:%d], nullptr viewport", *GetMediaId(), TilePosition.X, TilePosition.Y);
		return;
	}

	if (!bValidTileSettings)
	{
		UE_LOGF(LogDisplayClusterMedia, Warning, "'%ls' couldn't capture backbuffer tile [%d:%d], layout [%d:%d] - invalid tile settings", *GetMediaId(), TilePosition.X, TilePosition.Y, TileLayout.X, TileLayout.Y);
		return;
	}

	if (FRHITexture* const BackbufferTexture = Viewport->GetRenderTargetTexture())
	{
		FRDGBuilder GraphBuilder(RHICmdList);

		FRDGTextureRef BackbufferTextureRef = RegisterExternalTexture(GraphBuilder, BackbufferTexture, TEXT("DCMediaOutBackbufferTex"));

		// Tile width/height (non-edge case)
		const int32 TileWidth  = BackbufferTextureRef->Desc.Extent.X / TileLayout.X;
		const int32 TileHeight = BackbufferTextureRef->Desc.Extent.Y / TileLayout.Y;

		// Top-left of the tile sub-region
		const FIntPoint RectStart{
			TilePosition.X * TileWidth,
			TilePosition.Y * TileHeight
		};

		// Bottom-right of the tile sub-region. Ending tiles might have a little bit more pixels then TileWidth or TileHeight.
		const FIntPoint RectEnd{
			bEndingX ? BackbufferTextureRef->Desc.Extent.X : (TilePosition.X + 1) * TileWidth,
			bEndingY ? BackbufferTextureRef->Desc.Extent.Y : (TilePosition.Y + 1) * TileHeight
		};

		// Final tile sub-region
		const FIntRect TextureRegion = { RectStart, RectEnd };

		UE_LOGF(LogDisplayClusterMedia, VeryVerbose, "'%ls' capturing backbuffer tile [%d:%d], region [%d:%d - %d:%d] of size [%dx%d]",
			*GetMediaId(), TilePosition.X, TilePosition.Y,
			RectStart.X, RectStart.Y, RectEnd.X, RectEnd.Y,
			BackbufferTextureRef->Desc.Extent.X, BackbufferTextureRef->Desc.Extent.Y);

		// Capture
		const FMediaOutputTextureInfo TextureInfo{ BackbufferTextureRef, TextureRegion };
		ExportMediaData_RenderThread(GraphBuilder, TextureInfo);

		GraphBuilder.Execute();
	}
}
