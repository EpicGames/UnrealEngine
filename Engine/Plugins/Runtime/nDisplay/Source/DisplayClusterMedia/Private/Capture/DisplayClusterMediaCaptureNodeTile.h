// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Capture/DisplayClusterMediaCaptureNodeBase.h"

class FRHICommandListImmediate;
class FViewport;
class UDisplayClusterMediaOutputSynchronizationPolicy;
class UMediaOutput;


/**
 * Node backbuffer media capture (tile)
 */
class FDisplayClusterMediaCaptureNodeTile
	: public FDisplayClusterMediaCaptureNodeBase
{
public:

	/** Returns maximum layout allowed for backbuffer capture */
	static const FIntPoint GetMaxTileLayout()
	{
		return FIntPoint{ 4, 4 };
	}

public:

	/** Constructor */
	FDisplayClusterMediaCaptureNodeTile(
		const FString& MediaId,
		const FString& ClusterNodeId,
		const FIntPoint& TileLayout,
		const FIntPoint& TilePosition,
		UMediaOutput* MediaOutput,
		UDisplayClusterMediaOutputSynchronizationPolicy* SyncPolicy = nullptr
	);

public:

	/** Start backbuffer capture */
	virtual bool StartCapture() override;

protected:

	/** Returns tile size */
	virtual FIntPoint GetCaptureSize() const override;

	/** PostBackbufferUpdated handler */
	virtual void ProcessPostBackbufferUpdated_RenderThread(FRHICommandListImmediate& RHICmdList, FViewport* Viewport) override;

private:

	/** Pre-computed optimization flag to avoid repetitive tile settings validation */
	const bool bValidTileSettings;

	/** Pre-computed optimization flag to know if it's the last tile in a raw */
	const bool bEndingX;

	/** Pre-computed optimization flag to know if it's the last tile in a column */
	const bool bEndingY;

	/** Output tile layout */
	const FIntPoint TileLayout;

	/** This tile XY coordinate */
	const FIntPoint TilePosition;
};
