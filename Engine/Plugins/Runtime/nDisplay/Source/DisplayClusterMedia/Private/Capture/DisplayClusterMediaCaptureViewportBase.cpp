// Copyright Epic Games, Inc. All Rights Reserved.

#include "Capture/DisplayClusterMediaCaptureViewportBase.h"

#include "Game/IDisplayClusterGameManager.h"
#include "PostProcess/PostProcessMaterialInputs.h"

#include "Render/IDisplayClusterRenderManager.h"
#include "Render/Viewport/IDisplayClusterViewport.h"
#include "Render/Viewport/IDisplayClusterViewportManager.h"
#include "Render/Viewport/IDisplayClusterViewportManagerProxy.h"
#include "Render/Viewport/IDisplayClusterViewportProxy.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_Context.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_RenderSettings.h"

#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterMediaLog.h"
#include "DisplayClusterRootActor.h"

#include "IDisplayCluster.h"
#include "IDisplayClusterCallbacks.h"

#include "RHICommandList.h"
#include "RHIResources.h"


FDisplayClusterMediaCaptureViewportBase::FDisplayClusterMediaCaptureViewportBase(
	const FString& InMediaId,
	const FString& InClusterNodeId,
	const FString& InViewportId,
	UMediaOutput* InMediaOutput,
	UDisplayClusterMediaOutputSynchronizationPolicy* SyncPolicy
)
	: FDisplayClusterMediaCaptureBase(InMediaId, InClusterNodeId, InMediaOutput, SyncPolicy)
	, ReferencedViewportId(InViewportId)
{
}


bool FDisplayClusterMediaCaptureViewportBase::StartCapture()
{
	// Media state update callback
	IDisplayCluster::Get().GetCallbacks().OnDisplayClusterUpdateViewportMediaState().AddRaw(this, &FDisplayClusterMediaCaptureViewportBase::OnUpdateViewportMediaState);

	// PostTonemap callback for the late OCIO path
	IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPostTonemapPass_RenderThread().AddRaw(this, &FDisplayClusterMediaCaptureViewportBase::OnPostTonemapPass_RenderThread);

	// PostRenderViewFamily for a regular capture path
	IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPostRenderViewFamily_RenderThread().AddRaw(this, &FDisplayClusterMediaCaptureViewportBase::OnPostRenderViewFamily_RenderThread);

	// Passthrough media capture
	IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPassthroughMediaCapture_RenderThread().AddRaw(this, &FDisplayClusterMediaCaptureViewportBase::OnPassthroughMediaCapture_RenderThread);

	// PostResolveOverridden capture
	IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPostResolveOverridden_RenderThread().AddRaw(this, &FDisplayClusterMediaCaptureViewportBase::OnPostResolveOverridden_RenderThread);

	// Start capture
	const bool bStarted = FDisplayClusterMediaCaptureBase::StartCapture();

	return bStarted;
}

void FDisplayClusterMediaCaptureViewportBase::StopCapture()
{
	// Unsubscribe from external events/callbacks
	IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPostRenderViewFamily_RenderThread().RemoveAll(this);
	IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPostTonemapPass_RenderThread().RemoveAll(this);
	IDisplayCluster::Get().GetCallbacks().OnDisplayClusterUpdateViewportMediaState().RemoveAll(this);
	IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPassthroughMediaCapture_RenderThread().RemoveAll(this);
	IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPostResolveOverridden_RenderThread().RemoveAll(this);

	// Stop capturing
	FDisplayClusterMediaCaptureBase::StopCapture();
}

void FDisplayClusterMediaCaptureViewportBase::OnUpdateViewportMediaState(IDisplayClusterViewport* InViewport, EDisplayClusterViewportMediaState& InOutMediaState)
{
	// Set capture flag for the matching viewport
	if (InViewport && InViewport->GetId().Equals(GetViewportId(), ESearchCase::IgnoreCase))
	{
		// Raise flags that this viewport will be captured by media.
		InOutMediaState |= EDisplayClusterViewportMediaState::Capture;

		// Update late OCIO state on current frame
		UpdateLateOCIOState(InViewport);

		// Late OCIO flag
		if (IsLateOCIO())
		{
			InOutMediaState |= EDisplayClusterViewportMediaState::CaptureLateOCIO;
		}

		// Update media passthrough for this frame
		UpdateMediaPassthrough(InViewport);
	}
}

FIntPoint FDisplayClusterMediaCaptureViewportBase::GetCaptureSize() const
{
	return GetViewportSize();
}

FIntPoint FDisplayClusterMediaCaptureViewportBase::GetViewportSize() const
{
	FIntPoint CaptureSize{ FIntPoint::ZeroValue };

	if (GetCaptureSizeFromGameProxy(CaptureSize))
	{
		UE_LOGF(LogDisplayClusterMedia, Verbose, "'%ls' acquired capture size from game proxy [%d, %d]", *GetMediaId(), CaptureSize.X, CaptureSize.Y);
	}
	else if (GetCaptureSizeFromConfig(CaptureSize))
	{
		UE_LOGF(LogDisplayClusterMedia, Verbose, "'%ls' acquired capture size from config [%d, %d]", *GetMediaId(), CaptureSize.X, CaptureSize.Y);
	}
	else
	{
		UE_LOGF(LogDisplayClusterMedia, Verbose, "'%ls' couldn't acquire capture size", *GetMediaId());
	}

	return CaptureSize;
}


bool FDisplayClusterMediaCaptureViewportBase::GetCaptureSizeFromGameProxy(FIntPoint& OutSize) const
{
	// We need to get actual texture size for the viewport
	if (const IDisplayClusterRenderManager* const RenderMgr = IDisplayCluster::Get().GetRenderMgr())
	{
		if (const IDisplayClusterViewportManager* const ViewportMgr = RenderMgr->GetViewportManager())
		{
			if (const IDisplayClusterViewport* const Viewport = ViewportMgr->FindViewport(GetViewportId()))
			{
				const TArray<FDisplayClusterViewport_Context>& Contexts = Viewport->GetContexts();
				if (Contexts.Num() > 0)
				{
					OutSize = Contexts[0].RenderTargetRect.Size();
					return true;
				}
			}
		}
	}

	return false;
}

void FDisplayClusterMediaCaptureViewportBase::UpdateLateOCIOState(const IDisplayClusterViewport* Viewport)
{
	// So far, regular viewports dont't support late OCIO feature.
}

void FDisplayClusterMediaCaptureViewportBase::HandleLateOCIOChanged(const FLateOCIOData& NewLateOCIOConfiguration)
{
	// Restart capture
	StopCapture();
	StartCapture();
}

void FDisplayClusterMediaCaptureViewportBase::UpdateMediaPassthrough(const IDisplayClusterViewport* Viewport)
{
	bool bNewPassthroughState = false;

	// Media passthough is used when the same viewport has both media input and output configured. Let's see if this is the case.
	if (const ADisplayClusterRootActor* const RootActor = IDisplayCluster::Get().GetGameMgr()->GetRootActor())
	{
		if (const UDisplayClusterConfigurationData* const ConfigData = RootActor->GetConfigData())
		{
			if (const UDisplayClusterConfigurationViewport* const ViewportCfg = ConfigData->GetViewport(GetClusterNodeId(), GetViewportId()))
			{
				const FDisplayClusterConfigurationMediaViewport& MediaSettings = ViewportCfg->RenderSettings.Media;
				if (MediaSettings.bEnable)
				{
					bNewPassthroughState = MediaSettings.IsMediaInputAssigned() && MediaSettings.IsMediaOutputAssigned();
				}
			}
		}
	}

	// Pass to the render thread
	ENQUEUE_RENDER_COMMAND(DCMediaCaptureUpdatePassthrough)(
		[This = AsShared(), bSetVal = bNewPassthroughState](FRHICommandListImmediate& RHICmdList)
		{
			StaticCastSharedRef<FDisplayClusterMediaCaptureViewportBase>(This)->bUseMediaPassthrough_RT = bSetVal;
		});
}

void FDisplayClusterMediaCaptureViewportBase::OnPostTonemapPass_RenderThread(FRDGBuilder& GraphBuilder, const IDisplayClusterViewportProxy* ViewportProxy, const FSceneView& View, const FPostProcessMaterialInputs& Inputs, const uint32 ContextNum)
{
	checkSlow(ViewportProxy);

	// If late OCIO is not used currently, we have to ignore this PostTonemap callback.
	if (!IsLateOCIO())
	{
		return;
	}

	// Media subsystem does not support stereo, therefore we process context 0 only
	if (ContextNum != 0)
	{
		return;
	}

	// Check if proxy object is valid
	if (!ViewportProxy)
	{
		return;
	}

	// Make sure this is our viewport
	const bool bMatchingViewport = ViewportProxy->GetId().Equals(GetViewportId(), ESearchCase::IgnoreCase);
	if (!bMatchingViewport)
	{
		return;
	}

	// Get current SceneColor texture
	const FScreenPassTexture& SceneColor = FScreenPassTexture::CopyFromSlice(GraphBuilder, Inputs.GetInput(EPostProcessMaterialInput::SceneColor));

	// Pass it to the media capture pipeline
	if (SceneColor.IsValid())
	{
		FMediaOutputTextureInfo TextureInfo{ SceneColor.Texture, SceneColor.ViewRect };
		ExportMediaData_RenderThread(GraphBuilder, TextureInfo);
	}
}

void FDisplayClusterMediaCaptureViewportBase::OnPostRenderViewFamily_RenderThread(FRDGBuilder& GraphBuilder, const FSceneViewFamily& ViewFamily, const IDisplayClusterViewportProxy* ViewportProxy)
{
	checkSlow(ViewportProxy);

	// If late OCIO is used currently, we have to ignore this PostRenderViewFamily callback.
	if (IsLateOCIO())
	{
		return;
	}

	// Otherwise, find our viewport and export its texture
	if (ViewportProxy && ViewportProxy->GetId().Equals(GetViewportId(), ESearchCase::IgnoreCase))
	{
		TArray<FRHITexture*> Textures;
		TArray<FIntRect>     Regions;

		// Get RHI texture and pass it to the media capture pipeline
		if (ViewportProxy->GetResourcesWithRects_RenderThread(EDisplayClusterViewportResourceType::InternalRenderTargetEntireRectResource, Textures, Regions))
		{
			if (Textures.Num() > 0 && Regions.Num() > 0 && Textures[0])
			{
				FRDGTextureRef SrcTextureRef = RegisterExternalTexture(GraphBuilder, Textures[0], TEXT("DCMediaOutViewportTex"));

				FMediaOutputTextureInfo TextureInfo{ SrcTextureRef, Regions[0] };
				ExportMediaData_RenderThread(GraphBuilder, TextureInfo);
			}
		}
	}
}

void FDisplayClusterMediaCaptureViewportBase::OnPassthroughMediaCapture_RenderThread(FRHICommandListImmediate& RHICmdList, const IDisplayClusterViewportManagerProxy* ViewportManagerProxy)
{
	checkSlow(ViewportManagerProxy);

	// Nothing to do if media passthrough is not used
	if (!bUseMediaPassthrough_RT)
	{
		return;
	}

	// Otherwise, find our viewport and export its texture
	if (ViewportManagerProxy)
	{
		if (IDisplayClusterViewportProxy* ViewportProxy = ViewportManagerProxy->FindViewport_RenderThread(ReferencedViewportId))
		{
			TArray<FRHITexture*> Textures;
			TArray<FIntRect>     Regions;

			// Get RHI texture and pass it to the media capture pipeline
			if (ViewportProxy->GetResourcesWithRects_RenderThread(EDisplayClusterViewportResourceType::InternalRenderTargetResource, Textures, Regions))
			{
				if (Textures.Num() > 0 && Regions.Num() > 0 && Textures[0])
				{
					FRDGBuilder GraphBuilder(RHICmdList);

					FRDGTextureRef SrcTextureRef = RegisterExternalTexture(GraphBuilder, Textures[0], TEXT("DCMediaOutViewportTex"));

					FMediaOutputTextureInfo TextureInfo{ SrcTextureRef, Regions[0] };
					ExportMediaData_RenderThread(GraphBuilder, TextureInfo);

					GraphBuilder.Execute();
				}
			}
		}
	}
}

void FDisplayClusterMediaCaptureViewportBase::OnPostResolveOverridden_RenderThread(FRHICommandListImmediate& RHICmdList, const IDisplayClusterViewportProxy* ViewportProxy)
{
	checkSlow(ViewportProxy);

	if (!ViewportProxy)
	{
		return;
	}

	const FString CallbackViewportId = ViewportProxy->GetId();
	const FString RequestedViewportId = GetViewportId();
	if (!CallbackViewportId.Equals(RequestedViewportId, ESearchCase::IgnoreCase))
	{
		return;
	}

	TArray<FRHITexture*> Textures;
	TArray<FIntRect>     Regions;

	// Get RHI texture and pass it to the media capture pipeline
	if (ViewportProxy->GetResourcesWithRects_RenderThread(EDisplayClusterViewportResourceType::AfterWarpBlendTargetableResource, Textures, Regions))
	{
		if (Textures.Num() > 0 && Regions.Num() > 0 && Textures[0])
		{
			FRDGBuilder GraphBuilder(RHICmdList);

			FRDGTextureRef SrcTextureRef = RegisterExternalTexture(GraphBuilder, Textures[0], TEXT("DCMediaOutViewportTex"));

			FMediaOutputTextureInfo TextureInfo{ SrcTextureRef, Regions[0] };
			ExportMediaData_RenderThread(GraphBuilder, TextureInfo);

			GraphBuilder.Execute();
		}
	}
}
