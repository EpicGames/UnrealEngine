// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Capture/DisplayClusterMediaCaptureBase.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_Enums.h"

class FRDGBuilder;
class FSceneView;
class FSceneViewFamily;
class IDisplayClusterViewport;
class IDisplayClusterViewportManagerProxy;
class IDisplayClusterViewportProxy;
struct FPostProcessMaterialInputs;


/**
 * Base viewport capture adapter
 */
class FDisplayClusterMediaCaptureViewportBase
	: public FDisplayClusterMediaCaptureBase
{
public:

	FDisplayClusterMediaCaptureViewportBase(
		const FString& MediaId,
		const FString& ClusterNodeId,
		const FString& ViewportId,
		UMediaOutput* MediaOutput,
		UDisplayClusterMediaOutputSynchronizationPolicy* SyncPolicy
	);

public:

	/** Start capturing */
	virtual bool StartCapture() override;

	/** Stop capturing */
	virtual void StopCapture() override;

	/** Returns viewport ID that is configured for capture */
	const FString& GetViewportId() const
	{
		return ReferencedViewportId;
	}

	/** Returns texture size of a viewport assigned to capture (main thread) */
	virtual FIntPoint GetCaptureSize() const override;

	/** Ask children about capture size based on the configuration data */
	virtual bool GetCaptureSizeFromConfig(FIntPoint& OutSize) const = 0;

	/** Provides texture size from a game proxy (if available) */
	bool GetCaptureSizeFromGameProxy(FIntPoint& OutSize) const;

protected:

	/** Updates late OCIO state */
	virtual void UpdateLateOCIOState(const IDisplayClusterViewport* Viewport);

	/** Process late OCIO state updates */
	virtual void HandleLateOCIOChanged(const FLateOCIOData& NewLateOCIOConfiguration) override;

	/** Checks whether current frame should use media passthrough */
	virtual void UpdateMediaPassthrough(const IDisplayClusterViewport* Viewport);

private:

	/** UpdateViewportMediaState callback to configure media state for a viewoprt */
	void OnUpdateViewportMediaState(IDisplayClusterViewport* InViewport, EDisplayClusterViewportMediaState& InOutMediaState);

	/** PostRenderViewFamily callback handler where data is captured (no late OCIO) */
	void OnPostRenderViewFamily_RenderThread(FRDGBuilder& GraphBuilder, const FSceneViewFamily& ViewFamily, const IDisplayClusterViewportProxy* ViewportProxy);

	/** PostTonemapPass callback handler (late OCIO) */
	void OnPostTonemapPass_RenderThread(FRDGBuilder& GraphBuilder, const IDisplayClusterViewportProxy* ViewportProxy, const FSceneView& View, const FPostProcessMaterialInputs& Inputs, const uint32 ContextNum);

	/** PassthroughMediaCapture callback handler is used to capture in the passthrough scenarios */
	void OnPassthroughMediaCapture_RenderThread(FRHICommandListImmediate& RHICmdList, const IDisplayClusterViewportManagerProxy* ViewportManagerProxy);

	/** PostResolveOverridden callback for the viewports that grab textures from somewhere and don't emit PostRender events */
	void OnPostResolveOverridden_RenderThread(FRHICommandListImmediate& RHICmdList, const IDisplayClusterViewportProxy* ViewportProxy);

private:

	/** Returns size of the viewport bound to this media */
	virtual FIntPoint GetViewportSize() const;

private:

	/** Viewport ID assigned to capture */
	const FString ReferencedViewportId;

	/** Whether passthrough capture is needed */
	bool bUseMediaPassthrough_RT = false;
};
