// Copyright Epic Games, Inc. All Rights Reserved.

#include "Capture/DisplayClusterMediaCaptureViewportFull.h"

#include "DisplayClusterConfigurationTypes_Viewport.h"
#include "DisplayClusterRootActor.h"

#include "IDisplayCluster.h"

#include "Game/IDisplayClusterGameManager.h"


FDisplayClusterMediaCaptureViewportFull::FDisplayClusterMediaCaptureViewportFull(
	const FString& InMediaId,
	const FString& InClusterNodeId,
	const FString& InViewportId,
	UMediaOutput* InMediaOutput,
	UDisplayClusterMediaOutputSynchronizationPolicy* SyncPolicy
)
	: FDisplayClusterMediaCaptureViewportBase(InMediaId, InClusterNodeId, InViewportId, InMediaOutput, SyncPolicy)
{
}


bool FDisplayClusterMediaCaptureViewportFull::GetCaptureSizeFromConfig(FIntPoint& OutSize) const
{
	if (const ADisplayClusterRootActor* const ActiveRootActor = IDisplayCluster::Get().GetGameMgr()->GetRootActor())
	{
		if (const UDisplayClusterConfigurationData* const ConfigData = ActiveRootActor->GetConfigData())
		{
			const FString& NodeId     = GetClusterNodeId();
			const FString& ViewportId = GetViewportId();
			if (const UDisplayClusterConfigurationViewport* const ViewportCfg = ConfigData->GetViewport(NodeId, ViewportId))
			{
				const FIntRect ViewportRect = ViewportCfg->Region.ToRect();
				OutSize = FIntPoint(ViewportRect.Width(), ViewportRect.Height());

				return true;
			}
		}
	}

	return false;
}
