// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Capture/DisplayClusterMediaCaptureViewportBase.h"


/**
 * Viewport capture adapter (full frame)
 */
class FDisplayClusterMediaCaptureViewportFull
	: public FDisplayClusterMediaCaptureViewportBase
{
public:

	FDisplayClusterMediaCaptureViewportFull(
		const FString& MediaId,
		const FString& ClusterNodeId,
		const FString& ViewportId,
		UMediaOutput* MediaOutput,
		UDisplayClusterMediaOutputSynchronizationPolicy* SyncPolicy
	);

public:

	/** Provides default capture size from config */
	virtual bool GetCaptureSizeFromConfig(FIntPoint& OutSize) const;
};
