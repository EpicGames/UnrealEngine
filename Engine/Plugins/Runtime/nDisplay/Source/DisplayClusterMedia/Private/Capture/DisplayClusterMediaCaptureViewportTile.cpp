// Copyright Epic Games, Inc. All Rights Reserved.

#include "Capture/DisplayClusterMediaCaptureViewportTile.h"

#include "DisplayClusterMediaHelpers.h"


FDisplayClusterMediaCaptureViewportTile::FDisplayClusterMediaCaptureViewportTile(
	const FString& InMediaId,
	const FString& InClusterNodeId,
	const FString& InViewportId,
	const FIntPoint& InTilePosition,
	UMediaOutput* InMediaOutput,
	UDisplayClusterMediaOutputSynchronizationPolicy* SyncPolicy
)
	: FDisplayClusterMediaCaptureViewportBase(
		InMediaId,
		InClusterNodeId,
		DisplayClusterMediaHelpers::GenerateTileViewportName(InViewportId, InTilePosition),
		InMediaOutput,
		SyncPolicy)
{
}

bool FDisplayClusterMediaCaptureViewportTile::GetCaptureSizeFromConfig(FIntPoint& OutSize) const
{
	// The upper level logic always tries to get actual capture size from the game proxy. If the game proxy
	// is not available (camera/viewport is disabled or deactivated on start and therefore not being rendered),
	// it tries to acquire capture size from config. Being here means exactly this case.
	//
	// The problem is the tiles can't know their size until they start to render. There is a bunch of intermediate
	// logic that affects the final size of the owning camera/viewport texture such as "adopt resolution",
	// "screen percentage", "overscan", etc.
	//
	// But we need to provide something to start capture successfully. As a workaround, we return capture size
	// of 64x64. It looks valid and small enough to initialize media capture pipeline with a little
	// resource allocation. When tile rendering is started (e.g. camera/viewport is activated), its texture
	// will have final size available in the game proxy. Most likely it will be different so the media capture
	// pipeline will reconfigure for the new size.
	//
	// If game proxy is available, the capture size will be acquired by the higher level in
	// FDisplayClusterMediaCaptureViewport::GetCaptureSizeFromGameProxy()
	//
	// This trick looks kind of legal as we allow to change let's say ICVFX camera texture size in runtime.
	// The media pipeline is able to recognize the size changes and reconfigure.

	OutSize = { 64, 64 };

	return true;
}
