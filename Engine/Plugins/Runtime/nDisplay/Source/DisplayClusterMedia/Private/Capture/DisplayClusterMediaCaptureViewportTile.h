// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Capture/DisplayClusterMediaCaptureViewportBase.h"


/**
 * Viewport capture adapter (tile)
 */
class FDisplayClusterMediaCaptureViewportTile
	: public FDisplayClusterMediaCaptureViewportBase
{
public:

	FDisplayClusterMediaCaptureViewportTile(
		const FString& MediaId,
		const FString& ClusterNodeId,
		const FString& ViewportId,
		const FIntPoint& TilePosition,
		UMediaOutput* MediaOutput,
		UDisplayClusterMediaOutputSynchronizationPolicy* SyncPolicy
	);

protected:

	//~ Begin FDisplayClusterMediaCaptureViewport
	virtual bool GetCaptureSizeFromConfig(FIntPoint& OutSize) const override;
	//~ End FDisplayClusterMediaCaptureViewport
};
