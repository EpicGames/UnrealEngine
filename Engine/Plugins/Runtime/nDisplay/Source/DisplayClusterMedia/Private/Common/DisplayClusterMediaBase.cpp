// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterMediaBase.h"
#include "RenderingThread.h"


bool FDisplayClusterMediaBase::IsLateOCIO() const
{
	if (IsInGameThread())
	{
		return LateOCIOConfiguration.bLateOCIO;
	}
	else if (IsInRenderingThread())
	{
		return LateOCIOConfiguration_RT.bLateOCIO;
	}

	return false;
}

bool FDisplayClusterMediaBase::IsTransferPQ(bool bConsideringLateOCIOState) const
{
	if (IsInGameThread())
	{
		return bConsideringLateOCIOState ?
			LateOCIOConfiguration.bLateOCIO && LateOCIOConfiguration.bTransferPQ :
			LateOCIOConfiguration.bTransferPQ;
	}
	else if (IsInRenderingThread())
	{
		return bConsideringLateOCIOState ?
			LateOCIOConfiguration_RT.bLateOCIO && LateOCIOConfiguration_RT.bTransferPQ :
			LateOCIOConfiguration_RT.bTransferPQ;
	}

	return false;
}

void FDisplayClusterMediaBase::SetLateOCIO(const FLateOCIOData& NewLateOCIOConfiguration)
{
	if (IsInGameThread())
	{
		// Let children know the OCIO parameters have changed
		if (NewLateOCIOConfiguration != LateOCIOConfiguration)
		{
			HandleLateOCIOChanged(NewLateOCIOConfiguration);
		}

		// Update configuration
		LateOCIOConfiguration = NewLateOCIOConfiguration;

		// And pass to the render thread
		ENQUEUE_RENDER_COMMAND(DCMediaUpdateOCIOState)(
			[This = AsShared(), SetNewLateOCIO = LateOCIOConfiguration](FRHICommandListImmediate& RHICmdList)
			{
				This->LateOCIOConfiguration_RT = SetNewLateOCIO;
			});
	}
}
