// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "OpenColorIOColorSpace.h"


/**
 * Base media adapter class
 */
class FDisplayClusterMediaBase
	: public TSharedFromThis<FDisplayClusterMediaBase>
{
public:

	FDisplayClusterMediaBase(const FString& InMediaId, const FString& InClusterNodeId)
		: MediaId(InMediaId)
		, ClusterNodeId(InClusterNodeId)
	{ }

	virtual ~FDisplayClusterMediaBase() = default;

public:

	/** Returns ID of this media adapter */
	const FString& GetMediaId() const
	{
		return MediaId;
	}

	/** Returns current cluster node ID */
	const FString& GetClusterNodeId() const
	{
		return ClusterNodeId;
	}

	/** Whether late OCIO is active on current frame */
	bool IsLateOCIO() const;

	/**
	 * Is PQ transfer required?
	 * 
	 * @param bConsideringLateOCIOState - Should the OCIO on/off state be logically AND-ed with the PQ on/off state
	 */
	bool IsTransferPQ(bool bConsideringLateOCIOState = true) const;

protected:

	/** Auxiliary structure to keep late OCIO parameters */
	struct FLateOCIOData
	{
		/** Late OCIO enabled/disabled flag */
		bool bLateOCIO = false;

		/** PQ transfer enabled/disabled */
		bool bTransferPQ = false;

		/** Comparison operator */
		bool operator==(const FLateOCIOData& Other)
		{
			const bool bSameEnabledState = (this->bLateOCIO == Other.bLateOCIO);
			const bool bSamePQ = (this->bTransferPQ == Other.bTransferPQ);

			const bool bEqual = (bSameEnabledState && bSamePQ);

			return bEqual;
		}
	};

	/** Set late OCIO configuration by children */
	void SetLateOCIO(const FLateOCIOData& NewLateOCIOConfiguration);

	/** Let children handle the OCIO changes */
	virtual void HandleLateOCIOChanged(const FLateOCIOData& NewLateOCIOConfiguration)
	{ }

private:

	/** ID of this media adapter */
	const FString MediaId;

	/** Cluster node ID we're running on */
	const FString ClusterNodeId;

private:

	/** Late OCIO configuration on current frame */
	FLateOCIOData LateOCIOConfiguration;

	/** Late OCIO configuration on current frame (render thread) */
	FLateOCIOData LateOCIOConfiguration_RT;
};
