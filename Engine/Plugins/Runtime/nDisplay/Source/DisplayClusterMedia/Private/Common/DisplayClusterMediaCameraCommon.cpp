// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common/DisplayClusterMediaCameraCommon.h"

#include "Components/DisplayClusterICVFXCameraComponent.h"
#include "Game/IDisplayClusterGameManager.h"

#include "DisplayClusterRootActor.h"
#include "IDisplayCluster.h"


FDisplayClusterMediaCameraCommon::FDisplayClusterMediaCameraCommon(const FString& InCameraId)
	: CameraId(InCameraId)
{
}

UDisplayClusterICVFXCameraComponent* FDisplayClusterMediaCameraCommon::GetCameraComponent() const
{
	if (const ADisplayClusterRootActor* const DCRA = IDisplayCluster::Get().GetGameMgr()->GetRootActor())
	{
		if (UDisplayClusterICVFXCameraComponent* ICVFXCameraComponent = DCRA->GetComponentByName<UDisplayClusterICVFXCameraComponent>(CameraId))
		{
			return IsValid(ICVFXCameraComponent) ? ICVFXCameraComponent : nullptr;
		}
	}

	return nullptr;
}

void FDisplayClusterMediaCameraCommon::GetLateOCIOParameters(bool& bOutLateOCIOEnabled, bool& bOutTransferPQ) const
{
	if (const UDisplayClusterICVFXCameraComponent* const ICVFXCameraComponent = GetCameraComponent())
	{
		// Check if the main OCIO switch is on
		const bool bEnabledOCIO = ICVFXCameraComponent->CameraSettings.CameraOCIO.AllNodesOCIOConfiguration.bIsEnabled;
		// Check if media is enabled on this camera
		const bool bEnabledMedia = ICVFXCameraComponent->CameraSettings.RenderSettings.Media.bEnable;
		// Check if late OCIO is required by current per-node OCIO configuration
		const bool bValidPerNodeOCIO = HasAnyValidPerNodeOCIOConfiguration();

		// Late OCIO
		bOutLateOCIOEnabled = bEnabledOCIO && bEnabledMedia && bValidPerNodeOCIO;
		// Always PQ
		bOutTransferPQ = true;
	}
	else
	{
		// Reset if we get here
		bOutLateOCIOEnabled = false;
		bOutTransferPQ = true;
	}
}

bool FDisplayClusterMediaCameraCommon::HasAnyValidPerNodeOCIOConfiguration() const
{
	if (const UDisplayClusterICVFXCameraComponent* const ICVFXCamera = GetCameraComponent())
	{
		// See if there is at least one valid configuration
		const bool bHasValidPerNodeOCIO = ICVFXCamera->GetCameraSettingsICVFX().CameraOCIO.PerNodeOCIOProfiles.ContainsByPredicate(
			[](const FDisplayClusterConfigurationOCIOProfile& Item)
			{
				return Item.bIsEnabled && Item.ColorConfiguration.IsValid() && !Item.ApplyOCIOToObjects.IsEmpty();
			});

		return bHasValidPerNodeOCIO;
	}

	return false;
}
