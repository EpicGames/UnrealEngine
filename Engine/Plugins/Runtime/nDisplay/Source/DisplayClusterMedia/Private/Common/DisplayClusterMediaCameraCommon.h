// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class UDisplayClusterICVFXCameraComponent;


/**
 * Common camera logic for media
 */
class FDisplayClusterMediaCameraCommon
{
public:

	FDisplayClusterMediaCameraCommon(const FString& CameraId);
	virtual ~FDisplayClusterMediaCameraCommon() = default;

protected:

	/** Finds an ICVFX camera component by name, otherwise nullptr */
	UDisplayClusterICVFXCameraComponent* GetCameraComponent() const;

	/** Returns late OCIO parameters of the current camera */
	void GetLateOCIOParameters(bool& bOutLateOCIOEnabled, bool& bOutTransferPQ) const;

private:

	/** Checks whether any valid per-node OCIO configuration is set */
	bool HasAnyValidPerNodeOCIOConfiguration() const;

private:

	/** Holds ICVFX camera component name associated with this media adapter */
	const FString CameraId;
};
