// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterMediaCVars.h"


TAutoConsoleVariable<bool> CVarMediaEnabled(
	TEXT("nDisplay.Media.Enabled"),
	true,
	TEXT("nDisplay media subsystem\n")
	TEXT("0 : Disabled\n")
	TEXT("1 : Enabled\n"),
	ECVF_ReadOnly
);
