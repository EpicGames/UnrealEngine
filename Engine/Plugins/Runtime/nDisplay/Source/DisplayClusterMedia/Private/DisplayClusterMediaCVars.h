// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/IConsoleManager.h"

/**
 * Enable/disable media.
 * 
 * May be useful if media needs to be disabled without doing any changes to the nDisplay configuration asset.
 */
extern TAutoConsoleVariable<bool> CVarMediaEnabled;
