// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterMediaHelpers.h"

#include "CommonRenderResources.h"
#include "PixelShaderUtils.h"
#include "ScreenRendering.h"
#include "PostProcess/DrawRectangle.h"


namespace DisplayClusterMediaHelpers
{
	namespace MediaId
	{
		// Generate media ID for a specific entity
		FString GenerateMediaId(EMediaDeviceType DeviceType, EMediaOwnerType OwnerType, const FString& NodeId, const FString& DCRAName, const FString& OwnerName, uint8 Index, const FIntPoint* InTilePos)
		{
			if (DeviceType == EMediaDeviceType::Input)
			{
				switch (OwnerType)
				{
				case EMediaOwnerType::Backbuffer:
					return FString::Printf(TEXT("%s_%s_backbuffer_input"), *NodeId, *DCRAName);

				case EMediaOwnerType::Viewport:
					return InTilePos
						? FString::Printf(TEXT("%s_%s_%s_viewport_tile_%d_%d_input"), *NodeId, *DCRAName, *OwnerName, InTilePos->X, InTilePos->Y)
						: FString::Printf(TEXT("%s_%s_%s_viewport_input"), *NodeId, *DCRAName, *OwnerName);

				case EMediaOwnerType::ICVFXCamera:
					return InTilePos
						? FString::Printf(TEXT("%s_%s_%s_icvfx_tile_%d_%d_input"), *NodeId, *DCRAName, *OwnerName, InTilePos->X, InTilePos->Y)
						: FString::Printf(TEXT("%s_%s_%s_icvfx_input"), *NodeId, *DCRAName, *OwnerName);

				default:
					checkNoEntry();
				}
			}
			else if (DeviceType == EMediaDeviceType::Output)
			{
				switch (OwnerType)
				{
				case EMediaOwnerType::Backbuffer:
					return InTilePos
						? FString::Printf(TEXT("%s_%s_backbuffer_tile_%d_%d_capture_%u"), *NodeId, *DCRAName, InTilePos->X, InTilePos->Y, Index)
						: FString::Printf(TEXT("%s_%s_backbuffer_capture_%u"), *NodeId, *DCRAName, Index);

				case EMediaOwnerType::Viewport:
					return InTilePos
						? FString::Printf(TEXT("%s_%s_%s_viewport_tile_%d_%d_capture_%u"), *NodeId, *DCRAName, *OwnerName, InTilePos->X, InTilePos->Y, Index)
						: FString::Printf(TEXT("%s_%s_%s_viewport_capture_%u"), *NodeId, *DCRAName, *OwnerName, Index);

				case EMediaOwnerType::ICVFXCamera:
					return InTilePos
						? FString::Printf(TEXT("%s_%s_%s_icvfx_tile_%d_%d_capture_%u"), *NodeId, *DCRAName, *OwnerName, InTilePos->X, InTilePos->Y, Index)
						: FString::Printf(TEXT("%s_%s_%s_icvfx_capture_%u"), *NodeId, *DCRAName, *OwnerName, Index);;

				default:
					checkNoEntry();
				}
			}

			// Should never get here
			return FString();
		}
	}

	//@todo This needs to be exposed from the DisplayCluster core module after its refactoring
	FString GenerateICVFXViewportName(const FString& ClusterNodeId, const FString& ICVFXCameraName)
	{
		return FString::Printf(TEXT("%s_icvfx_%s_incamera"), *ClusterNodeId, *ICVFXCameraName);
	}

	//@todo This needs to be exposed from the DisplayCluster core module after its refactoring
	FString GenerateTileViewportName(const FString& InViewportId, const FIntPoint& InTilePos)
	{
		return FString::Printf(TEXT("%s_tile_%d_%d"), *InViewportId, InTilePos.X, InTilePos.Y);
	}

	FString GenerateICVFXTileViewportName(const FString& ClusterNodeId, const FString& ICVFXCameraName, const FIntPoint& InTilePos)
	{
		const FString BaseCameraViewportId = GenerateICVFXViewportName(ClusterNodeId, ICVFXCameraName);
		const FString CameraTileViewportId = GenerateTileViewportName(BaseCameraViewportId, InTilePos);

		return CameraTileViewportId;
	}

	template<class TScreenPixelShader>
	void ResampleTextureImpl_RenderThread(FRHICommandListImmediate& RHICmdList, FRHITexture* SrcTexture, FRHITexture* DstTexture, const FIntRect& SrcRect, const FIntRect& DstRect)
	{
		FRHIRenderPassInfo RPInfo(DstTexture, ERenderTargetActions::Load_Store);
		RHICmdList.Transition(FRHITransitionInfo(DstTexture, ERHIAccess::Unknown, ERHIAccess::RTV));

		RHICmdList.BeginRenderPass(RPInfo, TEXT("DisaplyClusterMedia_ResampleTexture"));
		{
			FIntVector SrcSizeXYZ = SrcTexture->GetSizeXYZ();
			FIntVector DstSizeXYZ = DstTexture->GetSizeXYZ();

			FIntPoint SrcSize(SrcSizeXYZ.X, SrcSizeXYZ.Y);
			FIntPoint DstSize(DstSizeXYZ.X, DstSizeXYZ.Y);

			RHICmdList.SetViewport(0.f, 0.f, 0.0f, DstSize.X, DstSize.Y, 1.0f);

			FGraphicsPipelineStateInitializer GraphicsPSOInit;
			RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

			GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
			GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();


			FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
			TShaderMapRef<FScreenVS> VertexShader(ShaderMap);
			TShaderMapRef<TScreenPixelShader> PixelShader(ShaderMap);
			if (!VertexShader.IsValid() || !PixelShader.IsValid())
			{
				// Always check if shaders are available on the current platform and hardware
				return;
			}

			GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
			GraphicsPSOInit.PrimitiveType = PT_TriangleList;

			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

			FRHISamplerState* SamplerState = (SrcRect.Size() != DstRect.Size()) ? TStaticSamplerState<SF_Bilinear>::GetRHI() : TStaticSamplerState<SF_Point>::GetRHI();

			SetShaderParametersLegacyPS(RHICmdList, PixelShader, SamplerState, SrcTexture);

			// Set up vertex uniform parameters for scaling and biasing the rectangle.
			// Note: Use DrawRectangle in the vertex shader to calculate the correct vertex position and uv.

			UE::Renderer::PostProcess::DrawRectangle(
				RHICmdList, VertexShader,
				DstRect.Min.X, DstRect.Min.Y,
				DstRect.Size().X, DstRect.Size().Y,
				SrcRect.Min.X, SrcRect.Min.Y,
				SrcRect.Size().X, SrcRect.Size().Y,
				DstSize, SrcSize
			);
		}

		RHICmdList.EndRenderPass();
		RHICmdList.Transition(FRHITransitionInfo(DstTexture, ERHIAccess::Unknown, ERHIAccess::SRVMask));
	}

	void ResampleTexture_RenderThread(FRHICommandListImmediate& RHICmdList, FRHITexture* SrcTexture, FRHITexture* DstTexture, const FIntRect& SrcRect, const FIntRect& DstRect)
	{
		check(SrcTexture);
		check(DstTexture);

		const bool bSrcSrgb = EnumHasAnyFlags(SrcTexture->GetFlags(), TexCreate_SRGB);
		const bool bDstSrgb = EnumHasAnyFlags(DstTexture->GetFlags(), TexCreate_SRGB);

		// We only need to convert from sRGB encoding to linear if the source is sRGB encoded and the destination is not.
		if (bSrcSrgb && !bDstSrgb)
		{
			ResampleTextureImpl_RenderThread<FScreenPSsRGBSource>(RHICmdList, SrcTexture, DstTexture, SrcRect, DstRect);
		}
		else
		{
			ResampleTextureImpl_RenderThread<FScreenPS>(RHICmdList, SrcTexture, DstTexture, SrcRect, DstRect);
		}
	}

	bool IsValidLayout(const FIntPoint& TileLayout, const FIntPoint& MaxLayout)
	{
		const bool bValidX  = TileLayout.X > 0 && TileLayout.X <= MaxLayout.X;
		const bool bValidY  = TileLayout.Y > 0 && TileLayout.Y <= MaxLayout.Y;
		const bool bNot1by1 = TileLayout.X > 1 || TileLayout.Y > 1;

		const bool bIsValid = bValidX && bValidY && bNot1by1;

		return bIsValid;
	}

	bool IsValidTileCoordinate(const FIntPoint& TilePosition, const FIntPoint& TileLayout)
	{
		const bool bValidX = TilePosition.X >= 0 && TilePosition.X < TileLayout.X;
		const bool bValidY = TilePosition.Y >= 0 && TilePosition.Y < TileLayout.Y;

		const bool bIsValid = bValidX && bValidY;

		return bIsValid;
	}
}
