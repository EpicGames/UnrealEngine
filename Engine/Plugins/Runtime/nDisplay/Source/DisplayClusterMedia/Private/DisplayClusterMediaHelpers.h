// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FRHICommandListImmediate;
class FRHITexture;


namespace DisplayClusterMediaHelpers
{
	namespace MediaId
	{
		/** Helper for media ID generation */
		enum class EMediaDeviceType : uint8
		{
			Input,
			Output
		};

		/** Helper for media ID generation */
		enum class EMediaOwnerType : uint8
		{
			Backbuffer,
			Viewport,
			ICVFXCamera
		};

		/** Generate media ID for a specific entity */
		FString GenerateMediaId(EMediaDeviceType DeviceType, EMediaOwnerType OwnerType, const FString& NodeId, const FString& DCRAName, const FString& OwnerName, uint8 Index, const FIntPoint* InTilePos = nullptr);
	}

	/** Generates internal ICVFX viewport IDs */
	FString GenerateICVFXViewportName(const FString& ClusterNodeId, const FString& ICVFXCameraName);

	/** Generates viewport tile IDs */
	FString GenerateTileViewportName(const FString& InViewportId, const FIntPoint& InTilePos);

	/** Generates viewport tile IDs for an ICVFX camera */
	FString GenerateICVFXTileViewportName(const FString& ClusterNodeId, const FString& ICVFXCameraName, const FIntPoint& InTilePos);

	/** Copy and resize an RHI texture */
	void ResampleTexture_RenderThread(FRHICommandListImmediate& RHICmdList, FRHITexture* SrcTexture, FRHITexture* DstTexture, const FIntRect& SrcRect, const FIntRect& DstRect);

	/** Checks whether a tile layout is valid */
	bool IsValidLayout(const FIntPoint& TileLayout, const FIntPoint& MaxLayout);

	/** Checks the correctness of the coordinates of the tile in the given layout */
	bool IsValidTileCoordinate(const FIntPoint& TilePosition, const FIntPoint& TileLayout);
}
