// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterMediaLog.h"


// Module-wide log categories
DEFINE_LOG_CATEGORY(LogDisplayClusterMedia);
DEFINE_LOG_CATEGORY(LogDisplayClusterMediaSync);
