// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterMediaModule.h"

#include "DisplayClusterMediaCVars.h"
#include "DisplayClusterMediaLog.h"
#include "DisplayClusterMediaHelpers.h"

#include "IDisplayCluster.h"
#include "IDisplayClusterCallbacks.h"
#include "DisplayClusterEnums.h"
#include "DisplayClusterRootActor.h"
#include "DisplayClusterConfigurationTypes.h"

#include "Components/DisplayClusterICVFXCameraComponent.h"
#include "Config/IDisplayClusterConfigManager.h"
#include "Cluster/IDisplayClusterClusterManager.h"
#include "Game/IDisplayClusterGameManager.h"

#include "Capture/DisplayClusterMediaCaptureCameraFull.h"
#include "Capture/DisplayClusterMediaCaptureCameraTile.h"
#include "Capture/DisplayClusterMediaCaptureNodeFull.h"
#include "Capture/DisplayClusterMediaCaptureNodeTile.h"
#include "Capture/DisplayClusterMediaCaptureViewportFull.h"
#include "Capture/DisplayClusterMediaCaptureViewportTile.h"
#include "Input/DisplayClusterMediaInputCameraFull.h"
#include "Input/DisplayClusterMediaInputCameraTile.h"
#include "Input/DisplayClusterMediaInputViewportFull.h"
#include "Input/DisplayClusterMediaInputViewportTile.h"

#include "IMediaModule.h"
#include "Misc/CoreDelegates.h"


FDisplayClusterMediaModule::FDisplayClusterMediaModule()
	: FrameQueue(MakeShared<FDisplayClusterFrameQueue>())
{
}

void FDisplayClusterMediaModule::StartupModule()
{
	UE_LOGF(LogDisplayClusterMedia, Log, "Starting module 'DisplayClusterMedia'...");

	IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPreSubmitViewFamilies().AddRaw(this, &FDisplayClusterMediaModule::OnPreSubmitViewFamilies);
	FCoreDelegates::OnEnginePreExit.AddRaw(this, &FDisplayClusterMediaModule::OnEnginePreExit);
}

void FDisplayClusterMediaModule::ShutdownModule()
{
	UE_LOGF(LogDisplayClusterMedia, Log, "Shutting down module 'DisplayClusterMedia'...");

	// We should already be unsubscribed from it but do it in case the callback has never been called.
	IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPreSubmitViewFamilies().RemoveAll(this);

	FCoreDelegates::OnEnginePreExit.RemoveAll(this);
}

void FDisplayClusterMediaModule::OnPreSubmitViewFamilies(TArray<FSceneViewFamilyContext*>&)
{
	// Unsubscribe after first call. Currently, media initialization is a one time procedure. No need to receive any further callbacks.
	IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPreSubmitViewFamilies().RemoveAll(this);

	InitializeMedia();

	StartCapture();
	PlayMedia();
}

void FDisplayClusterMediaModule::OnEnginePreExit()
{
	ReleaseMedia();
}

void FDisplayClusterMediaModule::InitializeMedia()
{
	// Runtime only for now
	if (IDisplayCluster::Get().GetOperationMode() != EDisplayClusterOperationMode::Cluster)
	{
		UE_LOGF(LogDisplayClusterMedia, Warning, "DisplayClusterMedia is available in 'cluster' operation mode only");
		return;
	}

	// Check if media enabled
	if (!CVarMediaEnabled.GetValueOnGameThread())
	{
		UE_LOGF(LogDisplayClusterMedia, Log, "nDisplay media subsytem is disabled by a cvar");
		return;
	}

	// Instantiate latency queue
	FrameQueue->Init();

	// Parse DCRA configuration and initialize media
	if (const ADisplayClusterRootActor* const RootActor = IDisplayCluster::Get().GetGameMgr()->GetRootActor())
	{
		const FString ClusterNodeId = IDisplayCluster::Get().GetClusterMgr()->GetNodeId();
		const FString RootActorName = RootActor->GetName();

		if (const UDisplayClusterConfigurationClusterNode* const ClusterNode = RootActor->GetConfigData()->Cluster->GetNode(ClusterNodeId))
		{
			// Node backbuffer media setup
			{
				InitializeBackbufferFullFrameOutput(ClusterNode, RootActorName, ClusterNodeId);
				InitializeBackbufferUniformTilesOutput(ClusterNode, RootActorName, ClusterNodeId);
			}

			// Viewports media setup
			for (const TPair<FString, TObjectPtr<UDisplayClusterConfigurationViewport>>& ViewportIt : ClusterNode->Viewports)
			{
				InitializeViewportInput(ViewportIt.Value, ViewportIt.Key, RootActorName, ClusterNodeId);
				InitializeViewportOutput(ViewportIt.Value, ViewportIt.Key, RootActorName, ClusterNodeId);
			}

			// ICVFX media setup
			{
				// Get all ICVFX camera components
				TArray<UDisplayClusterICVFXCameraComponent*> ICVFXCameraComponents;
				RootActor->GetComponents(ICVFXCameraComponents);

				for (const UDisplayClusterICVFXCameraComponent* const ICVFXCameraComponent : ICVFXCameraComponents)
				{
					const FDisplayClusterConfigurationMediaICVFX& MediaSettings = ICVFXCameraComponent->CameraSettings.RenderSettings.Media;

					// Full frame
					if (MediaSettings.SplitType == EDisplayClusterConfigurationMediaSplitType::FullFrame)
					{
						InitializeICVFXCameraFullFrameInput(ICVFXCameraComponent, RootActorName, ClusterNodeId);
						InitializeICVFXCameraFullFrameOutput(ICVFXCameraComponent, RootActorName, ClusterNodeId);
					}
					// Uniform tiles
					else if (MediaSettings.SplitType == EDisplayClusterConfigurationMediaSplitType::UniformTiles)
					{
						InitializeICVFXCameraUniformTilesInput(ICVFXCameraComponent, RootActorName, ClusterNodeId);
						InitializeICVFXCameraUniformTilesOutput(ICVFXCameraComponent, RootActorName, ClusterNodeId);
					}
				}
			}
		}
	}
}

void FDisplayClusterMediaModule::ReleaseMedia()
{
	StopCapture();
	StopMedia();

	CaptureDevices.Reset();
	InputDevices.Reset();

	FrameQueue->Release();
}

void FDisplayClusterMediaModule::StartCapture()
{
	// Start all capture devices
	for (TPair<FString, TSharedPtr<FDisplayClusterMediaCaptureBase>>& CaptureDevice : CaptureDevices)
	{
		CaptureDevice.Value->StartCapture();
	}
}

void FDisplayClusterMediaModule::StopCapture()
{
	// Stop all capture devices
	for (TPair<FString, TSharedPtr<FDisplayClusterMediaCaptureBase>>& CaptureDevice : CaptureDevices)
	{
		CaptureDevice.Value->StopCapture();
	}
}

void FDisplayClusterMediaModule::PlayMedia()
{
	// Start all input devices
	for (TPair<FString, TSharedPtr<FDisplayClusterMediaInputBase>>& InputDevice : InputDevices)
	{
		InputDevice.Value->Play();
	}
}

void FDisplayClusterMediaModule::StopMedia()
{
	// Stop all input devices
	for (TPair<FString, TSharedPtr<FDisplayClusterMediaInputBase>>& InputDevice : InputDevices)
	{
		InputDevice.Value->Stop();
	}
}

void FDisplayClusterMediaModule::InitializeBackbufferFullFrameOutput(const UDisplayClusterConfigurationClusterNode* ClusterNode, const FString& RootActorName, const FString& ClusterNodeId)
{
	if (IsValid(ClusterNode))
	{
		const FDisplayClusterConfigurationMediaNodeBackbuffer& MediaSettings = ClusterNode->MediaSettings;

		if (MediaSettings.bEnable)
		{
			uint8 CaptureIdx = 0;
			for (const FDisplayClusterConfigurationMediaOutput& MediaOutputItem : MediaSettings.MediaOutputs)
			{
				if (IsValid(MediaOutputItem.MediaOutput))
				{
					const FString MediaCaptureId = DisplayClusterMediaHelpers::MediaId::GenerateMediaId(
						DisplayClusterMediaHelpers::MediaId::EMediaDeviceType::Output,
						DisplayClusterMediaHelpers::MediaId::EMediaOwnerType::Backbuffer,
						ClusterNodeId, RootActorName, FString(), CaptureIdx);

					UE_LOGF(LogDisplayClusterMedia, Log, "Initializing backbuffer media capture [%u]: '%ls'", CaptureIdx, *MediaCaptureId);

					TSharedPtr<FDisplayClusterMediaCaptureNodeFull> NewNodeCapture = MakeShared<FDisplayClusterMediaCaptureNodeFull>(
						MediaCaptureId,
						ClusterNodeId,
						MediaOutputItem.MediaOutput,
						MediaOutputItem.OutputSyncPolicy);

					CaptureDevices.Emplace(MediaCaptureId, MoveTemp(NewNodeCapture));

					++CaptureIdx;
				}
			}
		}
	}
}

void FDisplayClusterMediaModule::InitializeBackbufferUniformTilesOutput(const UDisplayClusterConfigurationClusterNode* ClusterNode, const FString& RootActorName, const FString& ClusterNodeId)
{
	if (IsValid(ClusterNode))
	{
		const FDisplayClusterConfigurationMediaNodeBackbuffer& MediaSettings = ClusterNode->MediaSettings;

		if (MediaSettings.bEnable)
		{
			const FIntPoint MaxLayout = FDisplayClusterMediaCaptureNodeTile::GetMaxTileLayout();
			const bool bLayoutIsValid = DisplayClusterMediaHelpers::IsValidLayout(MediaSettings.TiledSplitLayout, MaxLayout);

			// Validate tile layout
			if (!bLayoutIsValid)
			{
				if (MediaSettings.TiledMediaOutputs.Num() > 0)
				{
					UE_LOGF(LogDisplayClusterMedia, Warning, "Invalid layout [%dx%d] was requested for backbuffer capture. Max layout is [%dx%d].",
						MediaSettings.TiledSplitLayout.X, MediaSettings.TiledSplitLayout.Y, MaxLayout.X, MaxLayout.Y);
				}

				return;
			}

			uint8 CaptureIdx = 0;
			for (const FDisplayClusterConfigurationMediaUniformTileOutput& MediaOutputTile : MediaSettings.TiledMediaOutputs)
			{
				if (IsValid(MediaOutputTile.MediaOutput))
				{
					const FString MediaCaptureId = DisplayClusterMediaHelpers::MediaId::GenerateMediaId(
						DisplayClusterMediaHelpers::MediaId::EMediaDeviceType::Output,
						DisplayClusterMediaHelpers::MediaId::EMediaOwnerType::Backbuffer,
						ClusterNodeId, RootActorName, FString(),
						CaptureIdx, &MediaOutputTile.Position);

					UE_LOGF(LogDisplayClusterMedia, Log, "Initializing backbuffer media capture tile [%u]:'%d,%d': '%ls'",
						CaptureIdx, MediaOutputTile.Position.X, MediaOutputTile.Position.Y, *MediaCaptureId);

					TSharedPtr<FDisplayClusterMediaCaptureNodeTile> NewNodeCapture = MakeShared<FDisplayClusterMediaCaptureNodeTile>(
						MediaCaptureId, ClusterNodeId,
						MediaSettings.TiledSplitLayout,
						MediaOutputTile.Position,
						MediaOutputTile.MediaOutput,
						MediaOutputTile.OutputSyncPolicy);

					CaptureDevices.Emplace(MediaCaptureId, NewNodeCapture);

					++CaptureIdx;
				}
			}
		}
	}
}

void FDisplayClusterMediaModule::InitializeViewportInput(const UDisplayClusterConfigurationViewport* Viewport, const FString& ViewportId, const FString& RootActorName, const FString& ClusterNodeId)
{
	if (IsValid(Viewport))
	{
		const FDisplayClusterConfigurationMediaViewport& MediaSettings = Viewport->RenderSettings.Media;

		if (MediaSettings.bEnable)
		{
			if (MediaSettings.IsMediaInputAssigned())
			{
				const FString MediaInputId = DisplayClusterMediaHelpers::MediaId::GenerateMediaId(
					DisplayClusterMediaHelpers::MediaId::EMediaDeviceType::Input,
					DisplayClusterMediaHelpers::MediaId::EMediaOwnerType::Viewport,
					ClusterNodeId, RootActorName, ViewportId, 0);

				UE_LOGF(LogDisplayClusterMedia, Log, "Initializing viewport media input '%ls' for viewport '%ls'", *MediaInputId, *ViewportId);

				TSharedPtr<FDisplayClusterMediaInputViewportFull> NewViewportInput = MakeShared<FDisplayClusterMediaInputViewportFull>(
					MediaInputId,
					ClusterNodeId,
					ViewportId,
					MediaSettings.MediaInput.MediaSource);

				InputDevices.Emplace(MediaInputId, MoveTemp(NewViewportInput));
			}
		}
	}
}

void FDisplayClusterMediaModule::InitializeViewportOutput(const UDisplayClusterConfigurationViewport* Viewport, const FString& ViewportId, const FString& RootActorName, const FString& ClusterNodeId)
{
	if (IsValid(Viewport))
	{
		const FDisplayClusterConfigurationMediaViewport& MediaSettings = Viewport->RenderSettings.Media;

		if (MediaSettings.bEnable)
		{
			// Media capture
			uint8 CaptureIdx = 0;
			for (const FDisplayClusterConfigurationMediaOutput& MediaOutputItem : MediaSettings.MediaOutputs)
			{
				if (IsValid(MediaOutputItem.MediaOutput))
				{
					const FString MediaCaptureId = DisplayClusterMediaHelpers::MediaId::GenerateMediaId(
						DisplayClusterMediaHelpers::MediaId::EMediaDeviceType::Output,
						DisplayClusterMediaHelpers::MediaId::EMediaOwnerType::Viewport,
						ClusterNodeId, RootActorName, ViewportId, CaptureIdx);

					UE_LOGF(LogDisplayClusterMedia, Log, "Initializing viewport capture [%u]: '%ls' for viewport '%ls'", CaptureIdx, *MediaCaptureId, *ViewportId);

					TSharedPtr<FDisplayClusterMediaCaptureViewportFull> NewViewportCapture = MakeShared<FDisplayClusterMediaCaptureViewportFull>(
						MediaCaptureId,
						ClusterNodeId,
						ViewportId,
						MediaOutputItem.MediaOutput,
						MediaOutputItem.OutputSyncPolicy);

					CaptureDevices.Emplace(MediaCaptureId, MoveTemp(NewViewportCapture));

					++CaptureIdx;
				}
			}
		}
	}
}

void FDisplayClusterMediaModule::InitializeICVFXCameraFullFrameInput(const UDisplayClusterICVFXCameraComponent* ICVFXCameraComponent, const FString& RootActorName, const FString& ClusterNodeId)
{
	if (IsValid(ICVFXCameraComponent))
	{
		const FDisplayClusterConfigurationMediaICVFX& MediaSettings = ICVFXCameraComponent->CameraSettings.RenderSettings.Media;

		if (MediaSettings.bEnable && MediaSettings.SplitType == EDisplayClusterConfigurationMediaSplitType::FullFrame)
		{
			if (UMediaSource* MediaSource = MediaSettings.GetMediaSource(ClusterNodeId))
			{
				const FString ICVFXCameraName = ICVFXCameraComponent->GetName();

				const FString MediaInputId = DisplayClusterMediaHelpers::MediaId::GenerateMediaId(
					DisplayClusterMediaHelpers::MediaId::EMediaDeviceType::Input,
					DisplayClusterMediaHelpers::MediaId::EMediaOwnerType::ICVFXCamera,
					ClusterNodeId, RootActorName, ICVFXCameraName, 0);

				UE_LOGF(LogDisplayClusterMedia, Log, "Initializing ICVFX media input '%ls' for camera '%ls'", *MediaInputId, *ICVFXCameraName);

				TSharedPtr<FDisplayClusterMediaInputCameraFull> NewICVFXInput = MakeShared<FDisplayClusterMediaInputCameraFull>(
					MediaInputId,
					ClusterNodeId,
					ICVFXCameraName,
					MediaSource);

				InputDevices.Emplace(MediaInputId, MoveTemp(NewICVFXInput));
			}
		}
	}
}

void FDisplayClusterMediaModule::InitializeICVFXCameraFullFrameOutput(const UDisplayClusterICVFXCameraComponent* ICVFXCameraComponent, const FString& RootActorName, const FString& ClusterNodeId)
{
	if (IsValid(ICVFXCameraComponent))
	{
		const FDisplayClusterConfigurationMediaICVFX& MediaSettings = ICVFXCameraComponent->CameraSettings.RenderSettings.Media;

		if (MediaSettings.bEnable && MediaSettings.SplitType == EDisplayClusterConfigurationMediaSplitType::FullFrame)
		{
			const FString ICVFXCameraName = ICVFXCameraComponent->GetName();

			// Media capture (full frame)
			const TArray<FDisplayClusterConfigurationMediaOutputGroup> MediaOutputItems = MediaSettings.GetMediaOutputGroups(ClusterNodeId);
			uint8 CaptureIdx = 0;
			for (const FDisplayClusterConfigurationMediaOutputGroup& MediaOutputItem : MediaOutputItems)
			{
				if (IsValid(MediaOutputItem.MediaOutput))
				{
					const FString MediaCaptureId = DisplayClusterMediaHelpers::MediaId::GenerateMediaId(
						DisplayClusterMediaHelpers::MediaId::EMediaDeviceType::Output,
						DisplayClusterMediaHelpers::MediaId::EMediaOwnerType::ICVFXCamera,
						ClusterNodeId, RootActorName, ICVFXCameraName, CaptureIdx);

					UE_LOGF(LogDisplayClusterMedia, Log, "Initializing ICVFX capture [%u]: '%ls' for camera '%ls'", CaptureIdx, *MediaCaptureId, *ICVFXCameraName);

					TSharedPtr<FDisplayClusterMediaCaptureCameraFull> NewICVFXCapture = MakeShared<FDisplayClusterMediaCaptureCameraFull>(
						MediaCaptureId,
						ClusterNodeId,
						ICVFXCameraName,
						MediaOutputItem.MediaOutput,
						MediaOutputItem.OutputSyncPolicy);

					CaptureDevices.Emplace(MediaCaptureId, MoveTemp(NewICVFXCapture));

					++CaptureIdx;
				}
			}
		}
	}
}

void FDisplayClusterMediaModule::InitializeICVFXCameraUniformTilesInput(const UDisplayClusterICVFXCameraComponent* ICVFXCameraComponent, const FString& RootActorName, const FString& ClusterNodeId)
{
	if (IsValid(ICVFXCameraComponent))
	{
		const FDisplayClusterConfigurationMediaICVFX& MediaSettings = ICVFXCameraComponent->CameraSettings.RenderSettings.Media;

		if (MediaSettings.bEnable && MediaSettings.SplitType == EDisplayClusterConfigurationMediaSplitType::UniformTiles)
		{
			// Find corresponsing media group
			TArray<FDisplayClusterConfigurationMediaUniformTileInput> MediaInputTiles;
			const bool bMediaInputGroupFound = MediaSettings.GetMediaInputTiles(ClusterNodeId, MediaInputTiles);

			if (bMediaInputGroupFound)
			{
				const FString ICVFXCameraName = ICVFXCameraComponent->GetName();

				uint8 Index = 0;
				for (const FDisplayClusterConfigurationMediaUniformTileInput& MediaInputTile : MediaInputTiles)
				{
					if (IsValid(MediaInputTile.MediaSource))
					{
						const FString MediaInputId = DisplayClusterMediaHelpers::MediaId::GenerateMediaId(
							DisplayClusterMediaHelpers::MediaId::EMediaDeviceType::Input,
							DisplayClusterMediaHelpers::MediaId::EMediaOwnerType::ICVFXCamera,
							ClusterNodeId, RootActorName, ICVFXCameraName,
							static_cast<uint8>(Index), &MediaInputTile.Position);

						UE_LOGF(LogDisplayClusterMedia, Log, "Initializing ICVFX media input '%ls' for camera '%ls' tile '%d,%d'",
							*MediaInputId, *ICVFXCameraName, MediaInputTile.Position.X, MediaInputTile.Position.Y);

						TSharedPtr<FDisplayClusterMediaInputCameraTile> NewICVFXTileInput = MakeShared<FDisplayClusterMediaInputCameraTile>(
							MediaInputId,
							ClusterNodeId,
							ICVFXCameraName,
							MediaInputTile.Position,
							MediaInputTile.MediaSource);

						InputDevices.Emplace(MediaInputId, MoveTemp(NewICVFXTileInput));

						++Index;
					}
				}
			}
		}
	}
}

void FDisplayClusterMediaModule::InitializeICVFXCameraUniformTilesOutput(const UDisplayClusterICVFXCameraComponent* ICVFXCameraComponent, const FString& RootActorName, const FString& ClusterNodeId)
{
	if (IsValid(ICVFXCameraComponent))
	{
		const FDisplayClusterConfigurationMediaICVFX& MediaSettings = ICVFXCameraComponent->CameraSettings.RenderSettings.Media;

		if (MediaSettings.bEnable && MediaSettings.SplitType == EDisplayClusterConfigurationMediaSplitType::UniformTiles)
		{
			// Find corresponsing media group
			TArray<FDisplayClusterConfigurationMediaUniformTileOutput> MediaOutputTiles;
			const bool bMediaOutputGroupFound = MediaSettings.GetMediaOutputTiles(ClusterNodeId, MediaOutputTiles);

			if (bMediaOutputGroupFound)
			{
				const FString ICVFXCameraName = ICVFXCameraComponent->GetName();

				uint8 Index = 0;
				for (const FDisplayClusterConfigurationMediaUniformTileOutput& MediaOutputTile : MediaOutputTiles)
				{
					if (IsValid(MediaOutputTile.MediaOutput))
					{
						const FString MediaOutputId = DisplayClusterMediaHelpers::MediaId::GenerateMediaId(
							DisplayClusterMediaHelpers::MediaId::EMediaDeviceType::Output,
							DisplayClusterMediaHelpers::MediaId::EMediaOwnerType::ICVFXCamera,
							ClusterNodeId, RootActorName, ICVFXCameraName,
							static_cast<uint8>(Index), &MediaOutputTile.Position);

						UE_LOGF(LogDisplayClusterMedia, Log, "Initializing ICVFX media output '%ls' for camera '%ls' tile '%d,%d'",
							*MediaOutputId, *ICVFXCameraName, MediaOutputTile.Position.X, MediaOutputTile.Position.Y);

						TSharedPtr<FDisplayClusterMediaCaptureCameraTile> NewICVFXTileOutput = MakeShared<FDisplayClusterMediaCaptureCameraTile>(
							MediaOutputId,
							ClusterNodeId,
							ICVFXCameraName,
							MediaOutputTile.Position,
							MediaOutputTile.MediaOutput,
							MediaOutputTile.OutputSyncPolicy);

						CaptureDevices.Emplace(MediaOutputId, MoveTemp(NewICVFXTileOutput));

						++Index;
					}
				}
			}
		}
	}
}

IMPLEMENT_MODULE(FDisplayClusterMediaModule, DisplayClusterMedia);
