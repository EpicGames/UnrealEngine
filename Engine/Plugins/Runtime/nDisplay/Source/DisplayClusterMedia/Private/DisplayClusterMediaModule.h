// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

#include "Synchronization/LatencyQueue/DisplayClusterFrameQueue.h"

class FDisplayClusterMediaCaptureBase;
class FDisplayClusterMediaInputBase;
class FSceneViewFamilyContext;
class UDisplayClusterConfigurationClusterNode;
class UDisplayClusterConfigurationViewport;
class UDisplayClusterICVFXCameraComponent;


/**
 * Media module
 */
class FDisplayClusterMediaModule :
	public IModuleInterface
{
public:

	FDisplayClusterMediaModule();
	~FDisplayClusterMediaModule() = default;

public:

	//~ Begin IModuleInterface Implementation
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface Implementation

protected:

	/** Initialize all media internals (from instance) */
	void InitializeMedia();

	/** Release all media internals */
	void ReleaseMedia();

	/** Start media capture (all sources) */
	void StartCapture();

	/** Stop media capture (all sources) */
	void StopCapture();

	/** Start playing media (all inputs) */
	void PlayMedia();

	/** Stop playing media (all inputs) */
	void StopMedia();

private:

	/** Backbuffer output initializer (full frame) */
	void InitializeBackbufferFullFrameOutput(const UDisplayClusterConfigurationClusterNode* ClusterNode, const FString& RootActorName, const FString& ClusterNodeId);

	/** Backbuffer output initializer (uniform tiles) */
	void InitializeBackbufferUniformTilesOutput(const UDisplayClusterConfigurationClusterNode* ClusterNode, const FString& RootActorName, const FString& ClusterNodeId);

	/** Viewport input initializer */
	void InitializeViewportInput(const UDisplayClusterConfigurationViewport* Viewport, const FString& ViewportId, const FString& RootActorName, const FString& ClusterNodeId);

	/** Viewport output initializer */
	void InitializeViewportOutput(const UDisplayClusterConfigurationViewport* Viewport, const FString& ViewportId, const FString& RootActorName, const FString& ClusterNodeId);

	/** ICVFX camera input initializer (full frame) */
	void InitializeICVFXCameraFullFrameInput(const UDisplayClusterICVFXCameraComponent* ICVFXCameraComponent, const FString& RootActorName, const FString& ClusterNodeId);

	/** ICVFX camera output initializer (full frame) */
	void InitializeICVFXCameraFullFrameOutput(const UDisplayClusterICVFXCameraComponent* ICVFXCameraComponent, const FString& RootActorName, const FString& ClusterNodeId);

	/** ICVFX camera input initializer (uniform tiles) */
	void InitializeICVFXCameraUniformTilesInput(const UDisplayClusterICVFXCameraComponent* ICVFXCameraComponent, const FString& RootActorName, const FString& ClusterNodeId);

	/** ICVFX camera output initializer (uniform tiles) */
	void InitializeICVFXCameraUniformTilesOutput(const UDisplayClusterICVFXCameraComponent* ICVFXCameraComponent, const FString& RootActorName, const FString& ClusterNodeId);

private:

	/** PreSubmitViewFamilies event handler. It's used to initialize media on start. */
	void OnPreSubmitViewFamilies(TArray<FSceneViewFamilyContext*>&);

	/** EnginePreExit event handler */
	void OnEnginePreExit();

private:

	/** Active media capture devices */
	TMap<FString, TSharedPtr<FDisplayClusterMediaCaptureBase>> CaptureDevices;

	/** Active media input devices */
	TMap<FString, TSharedPtr<FDisplayClusterMediaInputBase>> InputDevices;

private:

	/** Latency queue */
	TSharedRef<FDisplayClusterFrameQueue> FrameQueue;
};
