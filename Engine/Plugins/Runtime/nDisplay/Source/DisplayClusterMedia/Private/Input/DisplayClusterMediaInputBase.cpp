// Copyright Epic Games, Inc. All Rights Reserved.

#include "Input/DisplayClusterMediaInputBase.h"

#include "DisplayClusterMediaHelpers.h"
#include "DisplayClusterMediaLog.h"

#include "IDisplayClusterShaders.h"

#include "MediaPlayer.h"
#include "MediaSource.h"
#include "MediaTexture.h"
#include "TextureResource.h"
#include "RHIUtilities.h"

#include "OpenColorIOColorSpace.h"
#include "OpenColorIORendering.h"

#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "ScreenPass.h"

#include "Engine/Engine.h"

#include "ShaderParameters/DisplayClusterShaderParameters_Media.h"

#include "UObject/UObjectGlobals.h"
#include "UObject/Package.h"


TAutoConsoleVariable<bool> CVarTempRivermaxCropWorkaround(
	TEXT("nDisplay.Media.Rivermax.CropWorkaround"),
	true,
	TEXT("nDisplay workaround for Rivermax input\n")
	TEXT("0 : Disabled\n")
	TEXT("1 : Enabled\n"),
	ECVF_RenderThreadSafe
);

// Due to various alignment requirements the texture coming from Rivermax may be padded.
// This CVar controls the maximum number of pixels removed in case the received resolution
// doesn't match expected. #500 roughly corresponds to max number of pixels that can fit into UDP packet.
TAutoConsoleVariable<int32> CVarTempRivermaxExtraPixelsThreshold(
	TEXT("nDisplay.Media.Rivermax.ExtraPixelsThreshold"),
	500,
	TEXT("This sets a threshold for how many pixels will be cropped from the right side of the Rivermax stream \
		if the stream was padded. \n"),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarTempRivermaxExtraPixelsRemove(
	TEXT("nDisplay.Media.Rivermax.ExtraPixelsRemove"),
	0,
	TEXT("nDisplay workaround for Rivermax input\n"),
	ECVF_RenderThreadSafe
);

FDisplayClusterMediaInputBase::FDisplayClusterMediaInputBase(
	const FString& InMediaId,
	const FString& InClusterNodeId,
	UMediaSource* InMediaSource
)
	: FDisplayClusterMediaBase(InMediaId, InClusterNodeId)
{
	checkSlow(InMediaSource);
	MediaSource = DuplicateObject(InMediaSource, GetTransientPackage());
	checkSlow(MediaSource);

	// Instantiate media player
	MediaPlayer = NewObject<UMediaPlayer>();
	if (MediaPlayer)
	{
		MediaPlayer->SetLooping(false);
		MediaPlayer->PlayOnOpen = false;

		// Instantiate media texture
		MediaTexture = NewObject<UMediaTexture>();
		if (MediaTexture)
		{
			MediaTexture->NewStyleOutput = true;
			MediaTexture->SetRenderMode(UMediaTexture::ERenderMode::JustInTime);
			MediaTexture->SetMediaPlayer(MediaPlayer);
			MediaTexture->UpdateResource();
		}
	}
}


void FDisplayClusterMediaInputBase::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(MediaSource);
	Collector.AddReferencedObject(MediaPlayer);
	Collector.AddReferencedObject(MediaTexture);
}

bool FDisplayClusterMediaInputBase::Play()
{
	if (MediaSource && MediaPlayer && MediaTexture)
	{
		MediaPlayer->PlayOnOpen = true;
		MediaPlayer->OnMediaEvent().AddRaw(this, &FDisplayClusterMediaInputBase::OnMediaEvent);

		bWasPlayerStarted = MediaPlayer->OpenSource(MediaSource);

		static FName RiverMaxPlayerName(TEXT("RivermaxMedia"));
		bRunningRivermaxMedia = MediaPlayer->GetPlayerName() == RiverMaxPlayerName;

		return bWasPlayerStarted;
	}

	return false;
}

void FDisplayClusterMediaInputBase::Stop()
{
	if (MediaPlayer)
	{
		bWasPlayerStarted = false;
		MediaPlayer->Close();
		MediaPlayer->OnMediaEvent().RemoveAll(this);
	}

	bRunningRivermaxMedia = false;
}

void FDisplayClusterMediaInputBase::OverrideTextureRegions_RenderThread(FIntRect& InOutSrcRect, FIntRect& InOutDstRect) const
{
	const FIntPoint SrcSize = InOutSrcRect.Size();
	const FIntPoint DstSize = InOutDstRect.Size();
	if (SrcSize == DstSize)
	{
		return;
	}

	// [Workaround]
	// Based on the discussion, it looks like the problem is the incoming 2110 textures
	// may have up to CVarTempRivermaxExtraPixelsThreshold extra pixels.
	// If this is the only difference, we just copy the required subregion.
	if (bRunningRivermaxMedia && CVarTempRivermaxCropWorkaround.GetValueOnRenderThread())
	{
		const int32 ExtraPixelsThreshold = CVarTempRivermaxExtraPixelsThreshold.GetValueOnRenderThread();

		// Crop if required
		if (SrcSize.Y == DstSize.Y
			&& SrcSize.X >= DstSize.X
			&& (SrcSize.X - DstSize.X) <= ExtraPixelsThreshold)
		{
			// Use Dest size
			InOutSrcRect.Max.X = InOutSrcRect.Min.X + DstSize.X;

			return;
		}

		// By default we always remove extra pixels from the right side.
		const int32 ExtraPixelsRemove = CVarTempRivermaxExtraPixelsRemove.GetValueOnRenderThread();

		InOutSrcRect.Max.X -= ExtraPixelsRemove;
	}
}

void FDisplayClusterMediaInputBase::ImportMediaData_RenderThread(FRHICommandListImmediate& RHICmdList, const FMediaInputTextureInfo& TextureInfo)
{
	UE_LOGF(LogDisplayClusterMedia, Verbose, "MediaInput '%ls': importing texture on RT frame '%llu'...", *GetMediaId(), GFrameCounterRenderThread);

	// Render media texture
	MediaTexture->JustInTimeRender();

	FRHITexture* SrcTexture = MediaTexture->GetResource() ? MediaTexture->GetResource()->GetTextureRHI().GetReference() : nullptr;
	FRHITexture* const DstTexture = TextureInfo.Texture;

	if (!SrcTexture || !DstTexture)
	{
		UE_LOGF(LogDisplayClusterMedia, Warning, "MediaInput '%ls': wrong texture on RT frame '%llu'...", *GetMediaId(), GFrameCounterRenderThread);
		return;
	}

	// [Temp workaround]
	// There is an extra pixel issue in Rivermax. Allow to work around it.
	FIntRect SrcRect{ { 0, 0 }, SrcTexture->GetDesc().Extent };
	FIntRect DstRect{ TextureInfo.Region };
	OverrideTextureRegions_RenderThread(SrcRect, DstRect);

	// Process import
	if (IsLateOCIO())
	{
		ImportMediaDataOCIO_RenderThread(RHICmdList, SrcTexture, SrcRect, DstTexture, DstRect, TextureInfo.OCIOPassResources);
	}
	else
	{
		ImportMediaDataDirect_RenderThread(RHICmdList, SrcTexture, SrcRect, DstTexture, DstRect);
	}
}

void FDisplayClusterMediaInputBase::ImportMediaDataDirect_RenderThread(FRHICommandListImmediate& RHICmdList, FRHITexture* SrcTexture, const FIntRect& SrcRect, FRHITexture* DstTexture, const FIntRect& DstRect)
{
	if (!SrcTexture || !DstTexture)
	{
		return;
	}

	const bool bSrcSrgb = EnumHasAnyFlags(SrcTexture->GetFlags(), TexCreate_SRGB);
	const bool bDstSrgb = EnumHasAnyFlags(DstTexture->GetFlags(), TexCreate_SRGB);

	const bool bSameSrgb = (bSrcSrgb == bDstSrgb);
	const bool bSameFormat = (SrcTexture->GetDesc().Format == DstTexture->GetDesc().Format);
	const bool bSameSize = (SrcRect.Size() == DstRect.Size());

	const bool bCanCopy = (bSameFormat && bSameSize && bSameSrgb);

	// Based on the texture properties, copy it directly or resample
	if (bCanCopy)
	{
		FRHICopyTextureInfo CopyInfo;
		CopyInfo.SourcePosition = FIntVector(SrcRect.Min.X, SrcRect.Min.Y, 0);
		CopyInfo.DestPosition = FIntVector(DstRect.Min.X, DstRect.Min.Y, 0);
		CopyInfo.Size = FIntVector(DstRect.Size().X, DstRect.Size().Y, 1);

		TransitionAndCopyTexture(RHICmdList, SrcTexture, DstTexture, CopyInfo);
	}
	else
	{
		DisplayClusterMediaHelpers::ResampleTexture_RenderThread(RHICmdList, SrcTexture, DstTexture, SrcRect, DstRect);
	}
}

void FDisplayClusterMediaInputBase::ImportMediaDataOCIO_RenderThread(FRHICommandListImmediate& RHICmdList, FRHITexture* InSrcTexture, const FIntRect& InSrcRect, FRHITexture* InDstTexture, const FIntRect& InDstRect, const FOpenColorIORenderPassResources& OCIOResources)
{
	FRDGBuilder GraphBuilder(RHICmdList);

	// Register RHI textures for further processing
	FRDGTextureRef SrcTexture = RegisterExternalTexture(GraphBuilder, InSrcTexture, TEXT("DC.MediaTextureSrc"));
	FRDGTextureRef DstTexture = RegisterExternalTexture(GraphBuilder, InDstTexture, TEXT("DC.MediaTextureDst"));

	// A helper reference to the actual input texture
	FRDGTextureRef InputTexture = SrcTexture;

	// Is PQ-decode pass required?
	if (IsTransferPQ())
	{
		// An intermediate texture to process PQ decoding and OCIO in linear space
		FRDGTextureDesc InterimTextureDesc = FRDGTextureDesc::Create2D(InDstRect.Size(), PF_FloatRGBA, FClearValueBinding::Black, TexCreate_ShaderResource | TexCreate_RenderTargetable);
		FRDGTextureRef TextureLinear = GraphBuilder.CreateTexture(InterimTextureDesc, TEXT("DC.MediaTextureTempLinear"));

		// Add PQ-decode pass
		FDisplayClusterShaderParameters_MediaPQ Parameters{ SrcTexture, InSrcRect, TextureLinear, FIntRect{ FIntPoint::ZeroValue, InDstRect.Size() }};
		IDisplayClusterShaders::Get().AddPQToLinearPass(GraphBuilder, Parameters);

		// Use this texture in the OCIO pass
		InputTexture = TextureLinear;
	}

	// Now apply OCIO and store to the destination
	{
		FOpenColorIORendering::AddPass_RenderThread(
			GraphBuilder,
			FScreenPassViewInfo(),
			GEngine->GetDefaultWorldFeatureLevel(),
			FScreenPassTexture(InputTexture, InSrcRect),
			FScreenPassRenderTarget(DstTexture, InDstRect, ERenderTargetLoadAction::EClear),
			OCIOResources,
			1.0f,
			EOpenColorIOTransformAlpha::None
		);
	}

	GraphBuilder.Execute();
}

void FDisplayClusterMediaInputBase::OnMediaEvent(EMediaEvent MediaEvent)
{
	switch (MediaEvent)
	{
	/** The player started connecting to the media source. */
	case EMediaEvent::MediaConnecting:
		UE_LOGF(LogDisplayClusterMedia, Log, "Media event for '%ls': Connection", *GetMediaId());
		break;

	/** A new media source has been opened. */
	case EMediaEvent::MediaOpened:
		UE_LOGF(LogDisplayClusterMedia, Log, "Media event for '%ls': Opened", *GetMediaId());
		break;

	/** The current media source has been closed. */
	case EMediaEvent::MediaClosed:
		UE_LOGF(LogDisplayClusterMedia, Log, "Media event for '%ls': Closed", *GetMediaId());
		OnPlayerClosed();
		break;
		
	/** A media source failed to open. */
	case EMediaEvent::MediaOpenFailed:
		UE_LOGF(LogDisplayClusterMedia, Log, "Media event for '%ls': OpenFailed", *GetMediaId());
		break;

	default:
		UE_LOGF(LogDisplayClusterMedia, Log, "Media event for '%ls': %d", *GetMediaId(), static_cast<int32>(MediaEvent));
		break;
	}
}

bool FDisplayClusterMediaInputBase::StartPlayer()
{
	const bool bIsPlaying = MediaPlayer->OpenSource(MediaSource);
	if (bIsPlaying)
	{
		UE_LOGF(LogDisplayClusterMedia, Log, "Started playing media: %ls", *GetMediaId());
	}
	else
	{
		UE_LOGF(LogDisplayClusterMedia, Warning, "Couldn't start playing media: %ls", *GetMediaId());
	}

	return bIsPlaying;
}

void FDisplayClusterMediaInputBase::OnPlayerClosed()
{
	if (MediaPlayer && bWasPlayerStarted)
	{
		constexpr double Interval = 1.0;
		const double CurrentTime = FPlatformTime::Seconds();
		if (CurrentTime - LastRestartTimestamp > Interval)
		{
			UE_LOGF(LogDisplayClusterMedia, Log, "MediaPlayer '%ls' is in error, restarting it.", *GetMediaId());

			StartPlayer();
			LastRestartTimestamp = CurrentTime;
		}
	}
}
