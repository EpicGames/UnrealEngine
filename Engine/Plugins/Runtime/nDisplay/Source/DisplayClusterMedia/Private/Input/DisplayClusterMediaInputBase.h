// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Common/DisplayClusterMediaBase.h"
#include "UObject/GCObject.h"

#include "IMediaEventSink.h"
#include "OpenColorIORendering.h"

class FRHICommandListImmediate;
class FRHITexture;

class UMediaSource;
class UMediaPlayer;
class UMediaTexture;


/**
 * Base media input adapter class
 */
class FDisplayClusterMediaInputBase
	: public FDisplayClusterMediaBase
	, public FGCObject
{
public:

	FDisplayClusterMediaInputBase(
		const FString& MediaId,
		const FString& ClusterNodeId,
		UMediaSource* MediaSource
	);

public:

	//~ Begin FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FDisplayClusterMediaInputBase");
	}
	//~ End FGCObject interface

public:

	/** Start playback */
	virtual bool Play();

	/** Stop playback */
	virtual void Stop();

public:

	/** Returns current media source */
	UMediaSource* GetMediaSource() const
	{
		return MediaSource;
	}

	/** Returns current media player */
	UMediaPlayer* GetMediaPlayer() const
	{
		return MediaPlayer;
	}

	/** Returns current media texture */
	UMediaTexture* GetMediaTexture() const
	{
		return MediaTexture;
	}

protected:

	/** Media playback data */
	struct FMediaInputTextureInfo
	{
		/** Target texture for media input */
		FRHITexture* Texture = nullptr;

		/** Target subregion */
		FIntRect Region = { FIntPoint::ZeroValue, FIntPoint::ZeroValue };

		/** OpenColorIO render pass parameters */
		FOpenColorIORenderPassResources OCIOPassResources;
	};

protected:

	/** Imports texture from a media source */
	void ImportMediaData_RenderThread(FRHICommandListImmediate& RHICmdList, const FMediaInputTextureInfo& TextureInfo);

private:

	/** Import implementation for non-OCIO path */
	void ImportMediaDataDirect_RenderThread(FRHICommandListImmediate& RHICmdList, FRHITexture* SrcTexture, const FIntRect& SrcRect, FRHITexture* DstTexture, const FIntRect& DstRect);

	/** Import implementation for OCIO path */
	void ImportMediaDataOCIO_RenderThread(FRHICommandListImmediate& RHICmdList, FRHITexture* SrcTexture, const FIntRect& SrcRect, FRHITexture* DstTexture, const FIntRect& DstRect, const FOpenColorIORenderPassResources& OCIOResources);

private:

	/** Media events root handler */
	void OnMediaEvent(EMediaEvent MediaEvent);

	/** Start playback. Used to restart playback after failure */
	bool StartPlayer();

	/** Media event handler. Called when media source is closed. */
	void OnPlayerClosed();

	/** [TEMP] A temporary workaround to cut off extra pixels for Rivermax input streams */
	void OverrideTextureRegions_RenderThread(FIntRect& InOutSrcRect, FIntRect& InOutDstRect) const;

private:

	//~ Begin GC by AddReferencedObjects
	TObjectPtr<UMediaSource>  MediaSource;
	TObjectPtr<UMediaPlayer>  MediaPlayer;
	TObjectPtr<UMediaTexture> MediaTexture;
	//~ End GC by AddReferencedObjects

	/** Used to restart media player in the case it falls in error */
	bool bWasPlayerStarted = false;

	/** Used to control the rate at which we try to restart the player */
	double LastRestartTimestamp = 0;

	/** [Temp workaround] Whether current media is Rivermax */
	bool bRunningRivermaxMedia = false;
};
