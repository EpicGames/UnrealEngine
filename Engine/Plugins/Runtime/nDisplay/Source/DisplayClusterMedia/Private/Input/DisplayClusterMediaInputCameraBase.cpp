// Copyright Epic Games, Inc. All Rights Reserved.

#include "Input/DisplayClusterMediaInputCameraBase.h"

#include "Components/DisplayClusterICVFXCameraComponent.h"
#include "Game/IDisplayClusterGameManager.h"
#include "Render/Viewport/IDisplayClusterViewport.h"

#include "DisplayClusterRootActor.h"
#include "IDisplayCluster.h"

#include "DisplayClusterMediaLog.h"


FDisplayClusterMediaInputCameraBase::FDisplayClusterMediaInputCameraBase(
	const FString& InMediaId,
	const FString& InClusterNodeId,
	const FString& InCameraId,
	const FString& InCameraViewportId,
	UMediaSource* InMediaSource
)
	: FDisplayClusterMediaInputViewportBase(InMediaId, InClusterNodeId, InCameraViewportId, InMediaSource)
	, FDisplayClusterMediaCameraCommon(InCameraId)
{
}

void FDisplayClusterMediaInputCameraBase::UpdateLateOCIOState(const IDisplayClusterViewport* Viewport)
{
	FLateOCIOData NewLateOCIOConfiguration;

	// Get current late OCIO parameters from the camera component
	FDisplayClusterMediaCameraCommon::GetLateOCIOParameters(NewLateOCIOConfiguration.bLateOCIO, NewLateOCIOConfiguration.bTransferPQ);

	// And store/update for current frame
	SetLateOCIO(NewLateOCIOConfiguration);
}
