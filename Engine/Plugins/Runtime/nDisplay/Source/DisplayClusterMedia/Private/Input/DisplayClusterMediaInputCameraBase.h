// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Common/DisplayClusterMediaCameraCommon.h"
#include "Input/DisplayClusterMediaInputViewportBase.h"

class UDisplayClusterICVFXCameraComponent;


/**
 * Base camera input adapter
 */
class FDisplayClusterMediaInputCameraBase
	: public FDisplayClusterMediaInputViewportBase
	, public FDisplayClusterMediaCameraCommon
{
public:

	FDisplayClusterMediaInputCameraBase(
		const FString& MediaId,
		const FString& ClusterNodeId,
		const FString& CameraId,
		const FString& CameraViewportId,
		UMediaSource* MediaSource
	);

protected:

	/** Updates late OCIO state */
	virtual void UpdateLateOCIOState(const IDisplayClusterViewport* Viewport) override;
};
