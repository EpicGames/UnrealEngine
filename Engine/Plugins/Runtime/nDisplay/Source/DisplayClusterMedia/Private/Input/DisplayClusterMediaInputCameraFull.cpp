// Copyright Epic Games, Inc. All Rights Reserved.

#include "Input/DisplayClusterMediaInputCameraFull.h"

#include "DisplayClusterMediaHelpers.h"


FDisplayClusterMediaInputCameraFull::FDisplayClusterMediaInputCameraFull(
	const FString& InMediaId,
	const FString& InClusterNodeId,
	const FString& InCameraId,
	UMediaSource* InMediaSource
)
	: FDisplayClusterMediaInputCameraBase(
		InMediaId,
		InClusterNodeId,
		InCameraId,
		DisplayClusterMediaHelpers::GenerateICVFXViewportName(InClusterNodeId, InCameraId),
		InMediaSource)
{
}
