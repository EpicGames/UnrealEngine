// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/DisplayClusterMediaInputCameraBase.h"


/**
 * Camera input adapter (full frame)
 */
class FDisplayClusterMediaInputCameraFull
	: public FDisplayClusterMediaInputCameraBase
{
public:

	FDisplayClusterMediaInputCameraFull(
		const FString& MediaId,
		const FString& ClusterNodeId,
		const FString& CameraId,
		UMediaSource* MediaSource
	);
};
