// Copyright Epic Games, Inc. All Rights Reserved.

#include "Input/DisplayClusterMediaInputCameraTile.h"

#include "DisplayClusterMediaHelpers.h"


FDisplayClusterMediaInputCameraTile::FDisplayClusterMediaInputCameraTile(
	const FString& InMediaId,
	const FString& InClusterNodeId,
	const FString& InCameraId,
	const FIntPoint& InTilePosition,
	UMediaSource* InMediaSource
)
	: FDisplayClusterMediaInputCameraBase(
		InMediaId,
		InClusterNodeId,
		InCameraId,
		DisplayClusterMediaHelpers::GenerateICVFXTileViewportName(InClusterNodeId, InCameraId, InTilePosition),
		InMediaSource)
{
}
