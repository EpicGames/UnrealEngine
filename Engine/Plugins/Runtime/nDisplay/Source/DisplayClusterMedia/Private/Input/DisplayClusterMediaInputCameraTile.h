// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/DisplayClusterMediaInputCameraBase.h"


/**
 * Camera input adapter (tile)
 */
class FDisplayClusterMediaInputCameraTile
	: public FDisplayClusterMediaInputCameraBase
{
public:

	FDisplayClusterMediaInputCameraTile(
		const FString& MediaId,
		const FString& InClusterNodeId,
		const FString& CameraId,
		const FIntPoint& TilePosition,
		UMediaSource* MediaSource
	);
};
