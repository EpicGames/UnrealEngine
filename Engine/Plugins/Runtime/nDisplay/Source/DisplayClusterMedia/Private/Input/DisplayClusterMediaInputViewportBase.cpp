// Copyright Epic Games, Inc. All Rights Reserved.

#include "Input/DisplayClusterMediaInputViewportBase.h"

#include "Engine/Engine.h"
#include "Engine/World.h"

#include "Render/IDisplayClusterRenderManager.h"
#include "Render/Viewport/IDisplayClusterViewportManager.h"
#include "Render/Viewport/IDisplayClusterViewportManagerProxy.h"
#include "Render/Viewport/IDisplayClusterViewport.h"

#include "IDisplayCluster.h"
#include "IDisplayClusterCallbacks.h"

#include "OpenColorIOColorSpace.h"
#include "RenderGraphUtils.h"

#include "RHICommandList.h"
#include "RHIFeatureLevel.h"
#include "RHIResources.h"


FDisplayClusterMediaInputViewportBase::FDisplayClusterMediaInputViewportBase(
	const FString& InMediaId,
	const FString& InClusterNodeId,
	const FString& InViewportId,
	UMediaSource* InMediaSource
)
	: FDisplayClusterMediaInputBase(InMediaId, InClusterNodeId, InMediaSource)
	, ViewportId(InViewportId)
{
}


bool FDisplayClusterMediaInputViewportBase::Play()
{
	// If playback has started successfully, subscribe for rendering callbacks
	if (FDisplayClusterMediaInputBase::Play())
	{
		IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPreSubmitViewFamilies().AddRaw(this, &FDisplayClusterMediaInputViewportBase::OnPreSubmitViewFamilies);
		IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPostCrossGpuTransfer_RenderThread().AddRaw(this, &FDisplayClusterMediaInputViewportBase::OnPostCrossGpuTransfer_RenderThread);
		IDisplayCluster::Get().GetCallbacks().OnDisplayClusterUpdateViewportMediaState().AddRaw(this, &FDisplayClusterMediaInputViewportBase::OnUpdateViewportMediaState);

		return true;
	}

	return false;
}

void FDisplayClusterMediaInputViewportBase::Stop()
{
	// Unsubscribe from external events/callbacks
	IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPreSubmitViewFamilies().RemoveAll(this);
	IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPostCrossGpuTransfer_RenderThread().RemoveAll(this);
	IDisplayCluster::Get().GetCallbacks().OnDisplayClusterUpdateViewportMediaState().RemoveAll(this);

	// Stop playing
	FDisplayClusterMediaInputBase::Stop();
}

void FDisplayClusterMediaInputViewportBase::UpdateLateOCIOState(const IDisplayClusterViewport* Viewport)
{
	// So far, regular viewports dont't support late OCIO feature. But it can be overrided down the hierarchy.
}

void FDisplayClusterMediaInputViewportBase::HandleLateOCIOChanged(const FLateOCIOData& NewLateOCIOConfiguration)
{
	// Restart playback
	Stop();
	Play();
}

void FDisplayClusterMediaInputViewportBase::OnPreSubmitViewFamilies(TArray<FSceneViewFamilyContext*>&)
{
	// Get OCIO settings if there are any
	if (const IDisplayClusterViewportManager* const ViewportMgr = IDisplayCluster::Get().GetRenderMgr()->GetViewportManager())
	{
		if (const IDisplayClusterViewport* const Viewport = ViewportMgr->FindViewport(ViewportId))
		{
			// Get OCIO settings assigned to this viewport
			FOpenColorIOColorConversionSettings OCIOConversionSettings;
			Viewport->GetOCIOConversionSettings(OCIOConversionSettings);

			const UWorld* const CurrentWorld = ViewportMgr->GetConfiguration().GetCurrentWorld();

			const ERHIFeatureLevel::Type FeatureLevel = (CurrentWorld ?
				CurrentWorld->GetFeatureLevel() :
				GEngine->GetDefaultWorldFeatureLevel());

			// Get OCIO render pass resources
			FOpenColorIORenderPassResources OCIOPassResources = FOpenColorIORendering::GetRenderPassResources(OCIOConversionSettings, FeatureLevel);

			// And push it to the rendering thread
			ENQUEUE_RENDER_COMMAND(DCMediaInputUpdateOCIOResources)(
				[This = AsShared(), InOCIOPassResources = MoveTemp(OCIOPassResources)](FRHICommandListImmediate& RHICmdList)
				{
					StaticCastSharedRef<FDisplayClusterMediaInputViewportBase>(This)->OCIOPassResources_RT = InOCIOPassResources;
				}
			);
		}
	}
}

void FDisplayClusterMediaInputViewportBase::OnUpdateViewportMediaState(IDisplayClusterViewport* InViewport, EDisplayClusterViewportMediaState& InOutMediaState)
{
	// Note: Media currently supports only one DCRA.
	// In the future, after the media redesign, the DCRA name will also need to be checked here.
	if (InViewport && InViewport->GetId().Equals(GetViewportId(), ESearchCase::IgnoreCase))
	{
		// Raise flags that this viewport texture will be overridden by media.
		InOutMediaState |= EDisplayClusterViewportMediaState::Input;

		// Update late OCIO state on current frame
		UpdateLateOCIOState(InViewport);

		// Late OCIO flag
		if (IsLateOCIO())
		{
			InOutMediaState |= EDisplayClusterViewportMediaState::InputLateOCIO;
		}
	}
}

void FDisplayClusterMediaInputViewportBase::OnPostCrossGpuTransfer_RenderThread(FRHICommandListImmediate& RHICmdList, const IDisplayClusterViewportManagerProxy* ViewportManagerProxy, FViewport* Viewport)
{
	checkSlow(ViewportManagerProxy);

	if (const IDisplayClusterViewportProxy* const PlaybackViewport = ViewportManagerProxy->FindViewport_RenderThread(GetViewportId()))
	{
		const bool bShouldImportMedia = !PlaybackViewport->GetPostRenderSettings_RenderThread().Replace.IsEnabled();

		if (bShouldImportMedia)
		{
			TArray<FRHITexture*> Textures;
			TArray<FIntRect>     Regions;

			// Proceed with a proper texture resource
			if (PlaybackViewport->GetResourcesWithRects_RenderThread(EDisplayClusterViewportResourceType::InternalRenderTargetEntireRectResource, Textures, Regions))
			{
				if (Textures.Num() > 0 && Regions.Num() > 0 && Textures[0])
				{
					// Prepare request data
					FMediaInputTextureInfo TextureInfo{ Textures[0], Regions[0], MoveTemp(OCIOPassResources_RT) };

					// Import texture from media input
					ImportMediaData_RenderThread(RHICmdList, TextureInfo);
				}
			}
		}
	}
}
