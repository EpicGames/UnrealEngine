// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/DisplayClusterMediaInputBase.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_Enums.h"

#include "OpenColorIORendering.h"

class FRHICommandListImmediate;
class FViewport;
class FSceneViewFamilyContext;
class IDisplayClusterViewport;
class IDisplayClusterViewportManagerProxy;


/**
 * Base viewport input adapter
 */
class FDisplayClusterMediaInputViewportBase
	: public FDisplayClusterMediaInputBase
{
public:
	FDisplayClusterMediaInputViewportBase(
		const FString& MediaId,
		const FString& ClusterNodeId,
		const FString& ViewportId,
		UMediaSource* MediaSource
	);

public:
	/** Start playback */
	virtual bool Play() override;

	/** Stop playback */
	virtual void Stop() override;

	/** Returns viewport ID bound for playback */
	const FString& GetViewportId() const
	{
		return ViewportId;
	}

protected:

	/** Updates late OCIO state */
	virtual void UpdateLateOCIOState(const IDisplayClusterViewport* Viewport);

	/** Process late OCIO state updates */
	virtual void HandleLateOCIOChanged(const FLateOCIOData& NewLateOCIOConfiguration) override;

private:

	/** PreSubmitViewFamilies event handler. It's used to initialize media on start. */
	void OnPreSubmitViewFamilies(TArray<FSceneViewFamilyContext*>&);

	/** PostCrossGpuTransfer callback handler where media data is pushed into nDisplay internal buffers */
	void OnPostCrossGpuTransfer_RenderThread(FRHICommandListImmediate& RHICmdList, const IDisplayClusterViewportManagerProxy* ViewportManagerProxy, FViewport* Viewport);

	/** UpdateViewportMediaState callback to configure media state for a viewoprt */
	void OnUpdateViewportMediaState(IDisplayClusterViewport* InViewport, EDisplayClusterViewportMediaState& InOutMediaState);

private:

	/** Viewport ID assigned for this media input */
	const FString ViewportId;

	/** OCIO conversion pass resources (render thread data) */
	FOpenColorIORenderPassResources OCIOPassResources_RT;
};
