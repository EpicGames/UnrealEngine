// Copyright Epic Games, Inc. All Rights Reserved.

#include "Input/DisplayClusterMediaInputViewportFull.h"


FDisplayClusterMediaInputViewportFull::FDisplayClusterMediaInputViewportFull(
	const FString& InMediaId,
	const FString& InClusterNodeId,
	const FString& InViewportId,
	UMediaSource* InMediaSource
)
	: FDisplayClusterMediaInputViewportBase(InMediaId, InClusterNodeId, InViewportId, InMediaSource)
{
}
