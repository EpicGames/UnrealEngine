// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/DisplayClusterMediaInputViewportBase.h"


/**
 * Viewport input adapter (full frame)
 */
class FDisplayClusterMediaInputViewportFull
	: public FDisplayClusterMediaInputViewportBase
{
public:
	FDisplayClusterMediaInputViewportFull(
		const FString& MediaId,
		const FString& ClusterNodeId,
		const FString& ViewportId,
		UMediaSource* MediaSource
	);
};
