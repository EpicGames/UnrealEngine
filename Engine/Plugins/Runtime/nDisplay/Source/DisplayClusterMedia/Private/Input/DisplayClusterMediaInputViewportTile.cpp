// Copyright Epic Games, Inc. All Rights Reserved.

#include "Input/DisplayClusterMediaInputViewportTile.h"

#include "DisplayClusterMediaHelpers.h"


FDisplayClusterMediaInputViewportTile::FDisplayClusterMediaInputViewportTile(
	const FString& InMediaId,
	const FString& InClusterNodeId,
	const FString& InViewportId,
	const FIntPoint& InTilePosition,
	UMediaSource* InMediaSource
)
	: FDisplayClusterMediaInputViewportBase(
		InMediaId,
		InClusterNodeId,
		DisplayClusterMediaHelpers::GenerateTileViewportName(InViewportId, InTilePosition),
		InMediaSource)
{
}
