// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/DisplayClusterMediaInputViewportBase.h"


/**
 * Viewport input adapter (tile)
 */
class FDisplayClusterMediaInputViewportTile
	: public FDisplayClusterMediaInputViewportBase
{
public:
	FDisplayClusterMediaInputViewportTile(
		const FString& MediaId,
		const FString& ClusterNodeId,
		const FString& ViewportId,
		const FIntPoint& TilePosition,
		UMediaSource* MediaSource
	);
};
