// Copyright Epic Games, Inc. All Rights Reserved.

#include "Synchronization/LatencyQueue/DisplayClusterFrameQueue.h"

#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterMediaLog.h"
#include "DisplayClusterRootActor.h"

#include "IDisplayCluster.h"
#include "IDisplayClusterCallbacks.h"
#include "Game/IDisplayClusterGameManager.h"
#include "Render/IDisplayClusterRenderManager.h"
#include "Render/Viewport/IDisplayClusterViewportManager.h"
#include "Render/Viewport/IDisplayClusterViewportManagerProxy.h"
#include "Render/Viewport/IDisplayClusterViewportProxy.h"
#include "ShaderParameters/DisplayClusterShaderParameters_ICVFX.h"
#include "ShaderParameters/DisplayClusterShaderParameters_WarpBlend.h"

#include "Engine/Engine.h"
#include "Engine/GameViewportClient.h"


void FDisplayClusterFrameQueue::Init()
{
	// Subscribe for events
	if (IDisplayCluster::Get().GetOperationMode() == EDisplayClusterOperationMode::Cluster)
	{
		if (GEngine && GEngine->GameViewport)
		{
			GEngine->GameViewport->OnBeginDraw().AddRaw(this, &FDisplayClusterFrameQueue::HandleBeginDraw);
			GEngine->GameViewport->OnEndDraw().AddRaw(this, &FDisplayClusterFrameQueue::HandleEndDraw);
		}

		IDisplayCluster::Get().GetCallbacks().OnDisplayClusterProcessLatency_RenderThread().AddRaw(this, &FDisplayClusterFrameQueue::HandleProcessLatency_RenderThread);
		IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPreProcessIcvfx_RenderThread().AddRaw(this, &FDisplayClusterFrameQueue::HandlePreProcessIcvfx_RenderThread);
	}
}

void FDisplayClusterFrameQueue::Release()
{
	// Unsubscribe from events
	if (IDisplayCluster::Get().GetOperationMode() == EDisplayClusterOperationMode::Cluster)
	{
		if (GEngine && GEngine->GameViewport)
		{
			GEngine->GameViewport->OnBeginDraw().RemoveAll(this);
			GEngine->GameViewport->OnEndDraw().RemoveAll(this);
		}

		IDisplayCluster::Get().GetCallbacks().OnDisplayClusterProcessLatency_RenderThread().RemoveAll(this);
		IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPreProcessIcvfx_RenderThread().RemoveAll(this);
	}

	// We need to properly release render thread data
	ENQUEUE_RENDER_COMMAND(FDisplayClusterFrameQueue_Release)(
		[This = AsShared()](FRHICommandListImmediate& RHICmdList)
		{
			This->CleanQueue_RenderThread();
		});
}

void FDisplayClusterFrameQueue::HandleBeginDraw()
{
	// Update latency value before rendering new frame
	if (ADisplayClusterRootActor* RootActor = IDisplayCluster::Get().GetGameMgr()->GetRootActor())
	{
		if (const UDisplayClusterConfigurationData* const ConfigData = RootActor->GetConfigData())
		{
			// Get latency from config
			const int32 NewLatency = FMath::Clamp(ConfigData->MediaSettings.Latency, 0, 10);
			UE_LOGF(LogDisplayClusterMedia, VeryVerbose, "New latency is %u", NewLatency);

			// Update latency data on RT
			ENQUEUE_RENDER_COMMAND(FDisplayClusterFrameQueue_SetLatency)(
				[This = AsShared(), NewLatency](FRHICommandListImmediate& RHICmdList)
				{
					This->SetLatency_RenderThread(NewLatency);
				});
		}
	}

	// Process pending frames adding
	ENQUEUE_RENDER_COMMAND(FDisplayClusterFrameQueue_ProcessAddingPendingFrames)(
		[This = AsShared()](FRHICommandListImmediate& RHICmdList)
		{
			This->ProcessAddingPendingFrames_RenderThread();
		});
}

void FDisplayClusterFrameQueue::HandleEndDraw()
{
	// We need to update head and tail indices before rendering next frame
	ENQUEUE_RENDER_COMMAND(FDisplayClusterFrameQueue_StepQueueIndices)(
		[This = AsShared()](FRHICommandListImmediate& RHICmdList)
		{
			This->StepQueueIndices_RenderThread();
		});
}

void FDisplayClusterFrameQueue::HandleProcessLatency_RenderThread(FRHICommandListImmediate& RHICmdList, const IDisplayClusterViewportManagerProxy* ViewportManagerProxy, FViewport* Viewport)
{
	if (Frames.Num() > 0)
	{
		// At this point, all the views for current frame have been rendered already
		if (IDisplayClusterRenderManager* RenderMgr = IDisplayCluster::Get().GetRenderMgr())
		{
			if (IDisplayClusterViewportManager* ViewportMgr = RenderMgr->GetViewportManager())
			{
				if (IDisplayClusterViewportManagerProxy* ViewportMgrProxy = ViewportMgr->GetProxy())
				{
					TArrayView<TSharedPtr<IDisplayClusterViewportProxy, ESPMode::ThreadSafe>> ViewportProxies = ViewportMgrProxy->GetViewports_RenderThread();
					for (TSharedPtr<IDisplayClusterViewportProxy, ESPMode::ThreadSafe>& ViewportProxyIt : ViewportProxies)
					{
						// Handle textures of all viewports that don't have any media input attached. Those don't get rendered.
						if (ViewportProxyIt.IsValid() && !(ViewportProxyIt->GetRenderSettings_RenderThread().HasAnyMediaStates(EDisplayClusterViewportMediaState::Input)))
						{
							// Get viewport texture
							TArray<FRHITexture*> Textures;
							ViewportProxyIt->GetResources_RenderThread(EDisplayClusterViewportResourceType::InternalRenderTargetEntireRectResource, Textures);

							// Save current, and replace with an older
							if (Textures.Num() > 0)
							{
								const FString ViewportId(ViewportProxyIt->GetId());

								// Enqueue this view (head)
								Frames[IdxHead].SaveView(RHICmdList, ViewportId, Textures[0]);
								// Dequeue from the past (tail)
								Frames[IdxTail].LoadView(RHICmdList, ViewportId, Textures[0]);
							}
						}
					}
				}
			}
		}
	}
}

void FDisplayClusterFrameQueue::HandlePreProcessIcvfx_RenderThread(FRHICommandListImmediate& RHICmdList, const IDisplayClusterViewportProxy* ViewportProxy, FDisplayClusterShaderParameters_WarpBlend& WarpBlendParameters, FDisplayClusterShaderParameters_ICVFX& ICVFXParameters)
{
	// Substitute shader data
	if (Frames.Num() > 0)
	{
		const FString ViewportId(ViewportProxy->GetId());

		Frames[IdxHead].SaveData(RHICmdList, ViewportId, WarpBlendParameters, ICVFXParameters);
		Frames[IdxTail].LoadData(RHICmdList, ViewportId, WarpBlendParameters, ICVFXParameters);
	}
}

void FDisplayClusterFrameQueue::SetLatency_RenderThread(int32 NewLatency)
{
	// Nothing to do if requested latency is the same as previously (no changes since last frame)
	if (LastRequestedLatency != NewLatency && NewLatency >= 0)
	{
		// Check if latency just was disabled
		if (NewLatency == 0)
		{
			// Clear the queue
			CleanQueue_RenderThread();
		}
		// Check if new latency is higher
		else if (NewLatency > LastRequestedLatency)
		{
			FramesPendingToAdd = (NewLatency + 1) - Frames.Num();
		}
		// Check if new latency is lower
		else if (NewLatency < LastRequestedLatency)
		{
			// Depending on the current queue size, we need to find out if the queue needs to be resized
			const int32 TargetQueueSize = NewLatency + 1;

			// Yeah, we need to remove unnecessary elements from the queue
			if (TargetQueueSize < Frames.Num())
			{
				RemoveFromQueue_RenderThread(Frames.Num() - TargetQueueSize);
				FramesPendingToAdd = 0;
			}
			// No need to remove anything from the queue, just recalculate new amount of pending frames
			else
			{
				FramesPendingToAdd = TargetQueueSize - Frames.Num();
			}
		}

		// Finally, remember latency requested
		LastRequestedLatency = NewLatency;
	}
}

void FDisplayClusterFrameQueue::ProcessAddingPendingFrames_RenderThread()
{
	if (FramesPendingToAdd > 0)
	{
		// Increase queue size step by step, 1 item per frame. This allows to avoid
		// situations where we have empty queue items.
		FramesAdded = 1;
		AddToQueue_RenderThread(FramesAdded);
	}
	else
	{
		// Queue length remains unchanged
		FramesAdded = 0;
	}
}

void FDisplayClusterFrameQueue::AddToQueue_RenderThread(int32 FramesAmount)
{
	checkSlow(FramesAmount > 0);

	// Duplicate the head frame to increase the queue
	for (int32 Idx = 0; Idx < FramesAmount; ++Idx)
	{
		// In case the queue is empty, just instantiate new item
		if (Frames.IsEmpty())
		{
			Frames.AddDefaulted();

			// Reset indices
			IdxHead = 0;
			IdxTail = 0;
		}
		// Otherwise duplicate head item to prevent 'black' frames in the future
		else
		{
			// For some reason, the following doesn't work (there will be check fail inside Insert)
			// >>>> Frames.Insert(Frames[IdxHead], IdxHead)
			//
			// To achieve the same avoiding unnecessary data copying, I use the approach where data copying
			// is performed in copy constructor, and the instance is pushed to the container using move constructor.
			FDisplayClusterFrameQueueItem ClonedItem(Frames[IdxHead]);
			Frames.Insert(MoveTemp(ClonedItem), IdxHead);

			// Recalculate tail index
			if (IdxTail > IdxHead)
			{
				++IdxTail;
			}
		}
	}
}

void FDisplayClusterFrameQueue::RemoveFromQueue_RenderThread(int32 FramesAmout)
{
	checkSlow(FramesAmout < Frames.Num());

	// We need to remove specified amount of frames starting from the tail
	for (int32 Idx = 0; Idx < FramesAmout; ++Idx)
	{
		// Remove the item
		Frames.RemoveAt(IdxTail);

		// Recalculate head index
		if (IdxHead > IdxTail)
		{
			--IdxHead;
		}

		// Recalculate tail index
		if (IdxTail >= Frames.Num())
		{
			IdxTail = 0;
		}
	}
}

void FDisplayClusterFrameQueue::CleanQueue_RenderThread()
{
	// Clean up the container
	Frames.Empty();

	// Reset internals
	IdxHead = 0;
	IdxTail = 0;
	FramesPendingToAdd = 0;
}

void FDisplayClusterFrameQueue::StepQueueIndices_RenderThread()
{
	// Nothing to do if latency is zero
	if (Frames.IsEmpty())
	{
		UE_LOGF(LogDisplayClusterMedia, VeryVerbose, "Queue indices weren't updated, latency=0");
		return;
	}

	// Head index
	if (++IdxHead >= Frames.Num())
	{
		IdxHead = 0;
	}

	// Tail index doesn't move while we the queue being increased
	if (FramesAdded == 0)
	{
		if (++IdxTail >= Frames.Num())
		{
			IdxTail = 0;
		}
	}
	// Otherwise update pending amount
	else
	{
		FramesPendingToAdd -= FramesAdded;
		checkSlow(FramesPendingToAdd >= 0);
	}
}
