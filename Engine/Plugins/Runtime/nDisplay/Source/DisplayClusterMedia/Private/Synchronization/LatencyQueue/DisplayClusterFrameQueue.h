// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Synchronization/LatencyQueue/DisplayClusterFrameQueueItem.h"

class FDisplayClusterMediaInputViewport;
class FDisplayClusterShaderParameters_ICVFX;
class FViewport;
class IDisplayClusterViewportManagerProxy;
class IDisplayClusterViewportProxy;
class RHICmdList;
struct FDisplayClusterShaderParameters_WarpBlend;

/**
 * Latency queue class
 *
 * Implements artificial latency feature. It's responsible for caching visual and shaders data to be used later.
 *
 * All the data that comes out of the rendering pipeline for current frame is cached to the HEAD of the queue. The final
 * output is built from the TAIL view data. Every queue item contains all necessary information for every nD viewport.
 *
 * Note: this queue needs to be an intermediate part of the nD rendering pipeline. It should be moved there
 * at some point during refactoring.
 */
class FDisplayClusterFrameQueue
	: public TSharedFromThis<FDisplayClusterFrameQueue>
{
public:
	// Initializes latency feature
	void Init();
	// Releases latency feature
	void Release();

private:
	// Handles BeginDraw event
	void HandleBeginDraw();
	// Handles EndDraw event
	void HandleEndDraw();
	// Handles latency processing event
	void HandleProcessLatency_RenderThread(FRHICommandListImmediate& RHICmdList, const IDisplayClusterViewportManagerProxy* ViewportManagerProxy, FViewport* Viewport);
	// Handles ICVFX pre-processing event
	void HandlePreProcessIcvfx_RenderThread(FRHICommandListImmediate& RHICmdList, const IDisplayClusterViewportProxy* ViewportProxy, FDisplayClusterShaderParameters_WarpBlend& WarpBlendParameters, FDisplayClusterShaderParameters_ICVFX& ICVFXParameters);

private:
	// Changes current latency
	void SetLatency_RenderThread(int32 NewLatency);
	// Auxiliary method to process queue resize
	void ProcessAddingPendingFrames_RenderThread();
	// Auxiliary method to increase queue length by amount of frames
	void AddToQueue_RenderThread(int32 FramesAmount);
	// Auxiliary method to decrease queue length by amount of frames
	void RemoveFromQueue_RenderThread(int32 FramesAmout);
	// Cleans the queue container
	void CleanQueue_RenderThread();
	// Updates head and tail indices every frame
	void StepQueueIndices_RenderThread();

private:
	// Frames buffer
	TArray<FDisplayClusterFrameQueueItem> Frames;

	// Latency value requested last time
	int32 LastRequestedLatency = 0;

	// Amount of frames pending to add to the latency queue
	int32 FramesPendingToAdd = 0;
	// Amount of frames that were added at current frame
	int32 FramesAdded = 0;

	// Head queue index
	int32 IdxHead = 0;
	// Tail queue index
	int32 IdxTail = 0;
};
