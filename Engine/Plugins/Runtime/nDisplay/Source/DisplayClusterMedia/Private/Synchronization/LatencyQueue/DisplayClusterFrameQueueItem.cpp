// Copyright Epic Games, Inc. All Rights Reserved.

#include "Synchronization/LatencyQueue/DisplayClusterFrameQueueItem.h"

#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterMediaHelpers.h"
#include "DisplayClusterMediaLog.h"
#include "DisplayClusterRootActor.h"

#include "IDisplayCluster.h"
#include "Render/IDisplayClusterRenderManager.h"
#include "Render/Viewport/IDisplayClusterViewportManager.h"
#include "Render/Viewport/IDisplayClusterViewportManagerProxy.h"

#include "RHICommandList.h"
#include "RHIUtilities.h"


FDisplayClusterFrameQueueItem::FDisplayClusterFrameQueueItem(const FDisplayClusterFrameQueueItem& Other)
{
	for (const TPair<FString, FDisplayClusterFrameQueueItemView>& View : Other.Views)
	{
		FDisplayClusterFrameQueueItemView& NewItem = this->Views.Emplace(View.Key, View.Value);
		if (View.Value.Texture.IsValid())
		{
			FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();
			NewItem.Texture = CreateTexture(RHICmdList, View.Value.Texture);
			TransitionAndCopyTexture(RHICmdList, View.Value.Texture.GetReference(), NewItem.Texture, FRHICopyTextureInfo());
		}
	}
}

void FDisplayClusterFrameQueueItem::SaveView(FRHICommandListImmediate& RHICmdList, const FString& ViewportId, FRHITexture* Texture)
{
	checkSlow(Texture);

	// Find proper view and copy texture data
	FDisplayClusterFrameQueueItemView* View = Views.Find(ViewportId);
	if (!View)
	{
		View = &Views.Emplace(ViewportId);
	}

	// Create texture if not yet available. Or re-create if the source texture has been updated (re-sized, other format, etc.)
	if (!View->Texture.IsValid() ||
		Texture->GetFormat() != View->Texture.GetReference()->GetFormat() ||
		Texture->GetDesc().Extent != View->Texture.GetReference()->GetDesc().Extent)
	{
		View->Texture = CreateTexture(RHICmdList, Texture);
		check(View->Texture.IsValid());
	}

	// Copy texture data
	if (View->Texture.IsValid())
	{
		TransitionAndCopyTexture(RHICmdList, Texture, View->Texture.GetReference(), FRHICopyTextureInfo());
	}
}

void FDisplayClusterFrameQueueItem::LoadView(FRHICommandListImmediate& RHICmdList, const FString& ViewportId, FRHITexture* Texture)
{
	checkSlow(Texture);

	// Find proper view and copy texture data
	const FDisplayClusterFrameQueueItemView* const View = Views.Find(ViewportId);
	if (View && View->Texture.IsValid())
	{
		if (View->Texture->GetDesc().Extent != Texture->GetDesc().Extent 
			||	View->Texture->GetFormat() != Texture->GetFormat())
		{
			DisplayClusterMediaHelpers::ResampleTexture_RenderThread
			(
				RHICmdList,
				View->Texture.GetReference(),
				Texture,
				{ FIntPoint::ZeroValue, View->Texture->GetDesc().Extent },
				{ FIntPoint::ZeroValue, Texture->GetDesc().Extent }
			);
		}
		else
		{
			TransitionAndCopyTexture(RHICmdList, View->Texture.GetReference(), Texture, FRHICopyTextureInfo());
		}
	}
}

void FDisplayClusterFrameQueueItem::SaveData(FRHICommandListImmediate& RHICmdList, const FString& ViewportId, FDisplayClusterShaderParameters_WarpBlend& WarpBlendParameters, FDisplayClusterShaderParameters_ICVFX& ICVFXParameters)
{
	if (FDisplayClusterFrameQueueItemView* View = Views.Find(ViewportId))
	{
		// Warp/blend - cherry pick necessary parameters only
		View->WarpBlendData.bRenderAlphaChannel = WarpBlendParameters.bRenderAlphaChannel;
		View->WarpBlendData.Context = WarpBlendParameters.Context;

		// ICVFX - cherry pick necessary parameters only
		View->IcvfxData.Cameras = ICVFXParameters.Cameras;
	}
}

void FDisplayClusterFrameQueueItem::LoadData(FRHICommandListImmediate& RHICmdList, const FString& ViewportId, FDisplayClusterShaderParameters_WarpBlend& WarpBlendParameters, FDisplayClusterShaderParameters_ICVFX& ICVFXParameters)
{
	if (const FDisplayClusterFrameQueueItemView* const View = Views.Find(ViewportId))
	{
		// Warp/blend - cherry pick necessary parameters only
		WarpBlendParameters.bRenderAlphaChannel = View->WarpBlendData.bRenderAlphaChannel;
		WarpBlendParameters.Context = View->WarpBlendData.Context;

		// ICVFX - cherry pick necessary parameters only
		for (const FDisplayClusterShaderParameters_ICVFX::FCameraSettings& CameraSettings : View->IcvfxData.Cameras)
		{
			// Look up for proper camera settings object. Here we use ViewportId as a key to compare.
			FDisplayClusterShaderParameters_ICVFX::FCameraSettings* TargetCameraSettings = ICVFXParameters.Cameras.FindByPredicate([CameraSettings](const FDisplayClusterShaderParameters_ICVFX::FCameraSettings& Item)
				{
					return Item.Resource.ViewportId.Equals(CameraSettings.Resource.ViewportId, ESearchCase::IgnoreCase);
				});

			if (TargetCameraSettings)
			{
				// Do not copy render resources (because they are not copied in the source code either.)
				const bool bIncludeResources = false;

				// Updating the camera settings is done with a new function that copies all settings
				TargetCameraSettings->SetCameraSettings(CameraSettings, bIncludeResources);
			}
		}
	}
}

FTextureRHIRef FDisplayClusterFrameQueueItem::CreateTexture(FRHICommandListBase& RHICmdList, FRHITexture* ReferenceTexture)
{
	if (ReferenceTexture)
	{
		// Use original format and size
		const int32 SizeX = ReferenceTexture->GetDesc().Extent.X;
		const int32 SizeY = ReferenceTexture->GetDesc().Extent.Y;
		const EPixelFormat Format = ReferenceTexture->GetFormat();

		// Preserve sRGB flag from the reference
		const ETextureCreateFlags RefFlags = ReferenceTexture->GetFlags();
		const bool bIsSRGB = EnumHasAnyFlags(RefFlags, ETextureCreateFlags::SRGB);
		const ETextureCreateFlags NewSRGBFlags = (bIsSRGB ? ETextureCreateFlags::SRGB : ETextureCreateFlags::None);
		const ETextureCreateFlags NewFlags = NewSRGBFlags | ETextureCreateFlags::ShaderResource | ETextureCreateFlags::MultiGPUGraphIgnore;

		// Prepare description
		FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create2D(TEXT("DisplayClusterFrameQueueCacheTexture"), SizeX, SizeY, Format)
			.SetClearValue(FClearValueBinding::Black)
			.SetNumMips(1)
			.SetFlags(NewFlags)
			.SetInitialState(ERHIAccess::SRVMask);

		// Create texture
		return RHICmdList.CreateTexture(Desc);
	}

	return nullptr;
}
