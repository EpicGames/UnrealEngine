// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "ShaderParameters/DisplayClusterShaderParameters_ICVFX.h"
#include "ShaderParameters/DisplayClusterShaderParameters_WarpBlend.h"

class FRHICommandListImmediate;
class FRHITexture;


/**
 * Viewport data cache to be used in the latency queue
 */
struct FDisplayClusterFrameQueueItemView
{
	/** Cached view texture of a viewoprt */
	FTextureRHIRef Texture = nullptr;

	/** Cached warp/blend shader data */
	FDisplayClusterShaderParameters_WarpBlend WarpBlendData;

	/** Cached ICVFX shader data */
	FDisplayClusterShaderParameters_ICVFX IcvfxData;
};


/**
 * Latency queue item class
 *
 * It's responsible for caching all required data for single frame.
 * 
 * Note: this one is render thread only. It's never used on the main thread or any other thread.
 */
class FDisplayClusterFrameQueueItem
{
public:
	FDisplayClusterFrameQueueItem() = default;
	FDisplayClusterFrameQueueItem(const FDisplayClusterFrameQueueItem& Other);
	FDisplayClusterFrameQueueItem(FDisplayClusterFrameQueueItem&& Other) = default;

	~FDisplayClusterFrameQueueItem() = default;

public:
	// Save texture of a viewport
	void SaveView(FRHICommandListImmediate& RHICmdList, const FString& ViewportId, FRHITexture* Texture);
	// Load texture of a viewport
	void LoadView(FRHICommandListImmediate& RHICmdList, const FString& ViewportId, FRHITexture* Texture);

	// Save shader data of a viewport
	void SaveData(FRHICommandListImmediate& RHICmdList, const FString& ViewportId, FDisplayClusterShaderParameters_WarpBlend& WarpBlendParameters, FDisplayClusterShaderParameters_ICVFX& ICVFXParameters);
	// Load shader data of a viewport
	void LoadData(FRHICommandListImmediate& RHICmdList, const FString& ViewportId, FDisplayClusterShaderParameters_WarpBlend& WarpBlendParameters, FDisplayClusterShaderParameters_ICVFX& ICVFXParameters);

private:
	// Auxiliary function that creates a new texture based on the reference (same format, size)
	FTextureRHIRef CreateTexture(FRHICommandListBase& RHICmdList, FRHITexture* ReferenceTexture);

private:
	// Available views (viewports)
	TMap<FString, FDisplayClusterFrameQueueItemView> Views;
};
