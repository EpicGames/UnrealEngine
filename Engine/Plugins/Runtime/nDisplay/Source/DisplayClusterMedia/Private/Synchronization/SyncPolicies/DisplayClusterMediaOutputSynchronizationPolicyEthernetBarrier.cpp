// Copyright Epic Games, Inc. All Rights Reserved.

#include "Synchronization/DisplayClusterMediaOutputSynchronizationPolicyEthernetBarrier.h"


FDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierHandler::FDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierHandler(UDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrier* InPolicyObject)
	: Super(InPolicyObject)
{

}

TSubclassOf<UDisplayClusterMediaOutputSynchronizationPolicy> FDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierHandler::GetPolicyClass() const
{
	return UDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrier::StaticClass();
}

void FDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierHandler::Synchronize()
{
	// Just sync on the barrier
	SyncThreadOnBarrier();
}

TSharedPtr<IDisplayClusterMediaOutputSynchronizationPolicyHandler> UDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrier::GetHandler()
{
	if (!Handler)
	{
		Handler = MakeShared<FDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierHandler>(this);
	}

	return Handler;
}
