// Copyright Epic Games, Inc. All Rights Reserved.

#include "Synchronization/DisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierBase.h"

#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterEnums.h"
#include "DisplayClusterMediaHelpers.h"
#include "DisplayClusterMediaLog.h"
#include "DisplayClusterRootActor.h"

#include "IDisplayCluster.h"
#include "MediaCapture.h"

#include "Cluster/IDisplayClusterClusterManager.h"
#include "Cluster/IDisplayClusterGenericBarriersClient.h"
#include "Components/DisplayClusterICVFXCameraComponent.h"
#include "Game/IDisplayClusterGameManager.h"
#include "Templates/SharedPointer.h"


FDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierBaseHandler::FDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierBaseHandler(UDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierBase* InPolicyObject)
	: BarrierTimeoutMs(InPolicyObject->BarrierTimeoutMs)
{

}

bool FDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierBaseHandler::StartSynchronization(UMediaCapture* MediaCapture, const FString& MediaId)
{
	// Cluster mode only
	if (IDisplayCluster::Get().GetOperationMode() != EDisplayClusterOperationMode::Cluster)
	{
		UE_LOGF(LogDisplayClusterMediaSync, Warning, "'%ls': Media synchronization is available in cluster mode only", *MediaId);
		return false;
	}

	// Nothing to do if already running
	if (bIsRunning)
	{
		UE_LOGF(LogDisplayClusterMediaSync, Warning, "'%ls': Synchronization is on already", *MediaId);
		return true;
	}

	if (!MediaCapture)
	{
		UE_LOGF(LogDisplayClusterMediaSync, Warning, "'%ls': Invalid capture device (nullptr)", *MediaId);
		return false;
	}

	if (!IsCaptureTypeSupported(MediaCapture))
	{
		UE_LOGF(LogDisplayClusterMediaSync, Warning, "'%ls': Synchronization of media capture '%ls' is not supported by this sync policy", *MediaId, *MediaCapture->GetName());
		return false;
	}

	// Store capture device
	CapturingDevice = MediaCapture;
	MediaDeviceId   = MediaId;

	// Initialize dynamic barrier first
	if (!InitializeBarrier(MediaId))
	{
		UE_LOGF(LogDisplayClusterMediaSync, Warning, "'%ls': Couldn't initialize barrier client", *MediaId);
		return false;
	}

	CapturingDevice->OnOutputSynchronization.BindSP(this, &FDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierBaseHandler::ProcessMediaSynchronizationCallback);

	// Update state
	bIsRunning = true;

	return true;
}

void FDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierBaseHandler::StopSynchronization()
{
	if (bIsRunning)
	{
		// Don't reference capture device
		if (CapturingDevice)
		{
			CapturingDevice->OnOutputSynchronization.Unbind();
			CapturingDevice = nullptr;
		}

		// Release barrier client
		ReleaseBarrier();

		// Update state
		bIsRunning = false;
	}
}

bool FDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierBaseHandler::IsRunning()
{
	return bIsRunning;
}

bool FDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierBaseHandler::IsCaptureTypeSupported(UMediaCapture* MediaCapture) const
{
	return true;
}

FString FDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierBaseHandler::GetMediaDeviceId() const
{
	return MediaDeviceId;
}

TSharedPtr<IDisplayClusterGenericBarriersClient> FDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierBaseHandler::GetBarrierClient()
{
	return EthernetBarrierClient;
}

const FString& FDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierBaseHandler::GetBarrierId() const
{
	return BarrierId;
}

const FString& FDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierBaseHandler::GetThreadMarker() const
{
	return ThreadMarker;
}

void FDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierBaseHandler::SyncThreadOnBarrier()
{
	// Sync on the barrier if everything is good
	if (bIsRunning && EthernetBarrierClient)
	{
		UE_LOGF(LogDisplayClusterMediaSync, Verbose, "'%ls': Synchronizing caller '%ls' at the barrier '%ls'", *GetMediaDeviceId(), *ThreadMarker, *BarrierId);
		EthernetBarrierClient->Synchronize(BarrierId, ThreadMarker);
	}
}

bool FDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierBaseHandler::InitializeBarrier(const FString& MediaId)
{
	if (MediaId.IsEmpty())
	{
		UE_LOGF(LogDisplayClusterMediaSync, Warning, "'%ls': Wrong MediaId", *MediaId);
		return false;
	}

	// Instantiate barrier client
	EthernetBarrierClient = IDisplayCluster::Get().GetClusterMgr()->CreateGenericBarriersClient();
	if (!EthernetBarrierClient)
	{
		UE_LOGF(LogDisplayClusterMediaSync, Warning, "'%ls': Couldn't get generic barriers API", *MediaId);
		return false;
	}

	BarrierId     = GenerateBarrierName();
	ThreadMarker  = MediaId;

	// Sync callers
	TMap<FString, TSet<FString>> NodeToSyncCallers;
	GenerateSyncCallersMapping(NodeToSyncCallers);

	// Create sync barrier
	if (!EthernetBarrierClient->CreateBarrier(BarrierId, NodeToSyncCallers, BarrierTimeoutMs))
	{
		UE_LOGF(LogDisplayClusterMediaSync, Warning, "'%ls': Couldn't create barrier '%ls'", *MediaId, *BarrierId);
		EthernetBarrierClient.Reset();
		return false;
	}

	return true;
}

void FDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierBaseHandler::ReleaseBarrier()
{
	if (EthernetBarrierClient)
	{
		if (!BarrierId.IsEmpty())
		{
			EthernetBarrierClient->ReleaseBarrier(BarrierId);
		}

		EthernetBarrierClient.Reset();
	}
}

FString FDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierBaseHandler::GenerateBarrierName() const
{
	// Currently we don't have any synchronization groups. This means all the sync policy instances of the same
	// class use the same barrier. If we want to introduce sync groups in the future, the barrier ID should
	// take that group ID/number into account, and encode it into the barrier name.
	//
	// For example, we want two sets of capture devices to run with different output framerate. In this case, we would
	// need to split those sets into different sync groups.
	//
	// However! All media captures are locked to UE rendering pipeline. This means all the captures will run
	// with the same framerate. Therefore we don't need any sync groups so far.
	return GetPolicyClass()->GetName();
}

void FDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierBaseHandler::GenerateSyncCallersMapping(TMap<FString, TSet<FString>>& OutNodeToSyncCallers) const
{
	OutNodeToSyncCallers.Empty();

	UE_LOGF(LogDisplayClusterMediaSync, Verbose, "'%ls': Generating sync caller mappings for barrier '%ls'...", *GetMediaDeviceId(), *BarrierId);

	// Get active DCRA
	if (const ADisplayClusterRootActor* const RootActor = IDisplayCluster::Get().GetGameMgr()->GetRootActor())
	{
		// Get config data
		if (const UDisplayClusterConfigurationData* const CfgData = RootActor->GetConfigData())
		{
			// Iterate over cluster nodes
			for (const TPair<FString, TObjectPtr<UDisplayClusterConfigurationClusterNode>>& NodeIt : CfgData->Cluster->Nodes)
			{
				/////////////////////
				// Backbuffer capture
				{
					const FDisplayClusterConfigurationMediaNodeBackbuffer& MediaSettings = NodeIt.Value->MediaSettings;

					if (MediaSettings.bEnable)
					{
						// Iterate over full frame outputs
						{
							uint8 CaptureIdx = 0;
							for (const FDisplayClusterConfigurationMediaOutput& MediaOutputItem : MediaSettings.MediaOutputs)
							{
								if (IsValid(MediaOutputItem.MediaOutput) && IsValid(MediaOutputItem.OutputSyncPolicy))
								{
									// Pick the same sync policy only
									if (MediaOutputItem.OutputSyncPolicy->GetClass() == GetPolicyClass())
									{
										const FString BackbufferCaptureId = DisplayClusterMediaHelpers::MediaId::GenerateMediaId(
											DisplayClusterMediaHelpers::MediaId::EMediaDeviceType::Output,
											DisplayClusterMediaHelpers::MediaId::EMediaOwnerType::Backbuffer,
											NodeIt.Key, RootActor->GetName(), FString(), CaptureIdx++);

										OutNodeToSyncCallers.FindOrAdd(NodeIt.Key).Add(BackbufferCaptureId);
									}
								}
							}
						}

						// Iterate over tiles
						{
							if (MediaSettings.TiledSplitLayout.X > 1 || MediaSettings.TiledSplitLayout.Y > 1)
							{
								uint8 CaptureIdx = 0;
								for (const FDisplayClusterConfigurationMediaUniformTileOutput& OutputTile : MediaSettings.TiledMediaOutputs)
								{
									if (IsValid(OutputTile.MediaOutput) && IsValid(OutputTile.OutputSyncPolicy))
									{
										// Pick the same sync policy only
										if (OutputTile.OutputSyncPolicy->GetClass() == GetPolicyClass())
										{
											const FString BackbufferCaptureId = DisplayClusterMediaHelpers::MediaId::GenerateMediaId(
												DisplayClusterMediaHelpers::MediaId::EMediaDeviceType::Output,
												DisplayClusterMediaHelpers::MediaId::EMediaOwnerType::Backbuffer,
												NodeIt.Key, RootActor->GetName(), FString(), CaptureIdx++, &OutputTile.Position);

											OutNodeToSyncCallers.FindOrAdd(NodeIt.Key).Add(BackbufferCaptureId);
										}
									}
								}
							}
						}
					}
				}

				///////////////////
				// Viewport capture
				{
					// Iterate over viewports
					for (const TPair<FString, TObjectPtr<UDisplayClusterConfigurationViewport>>& ViewportIt : NodeIt.Value->Viewports)
					{
						const FDisplayClusterConfigurationMediaViewport& MediaSettings = ViewportIt.Value->RenderSettings.Media;

						if (MediaSettings.bEnable)
						{
							uint8 CaptureIdx = 0;
							for (const FDisplayClusterConfigurationMediaOutput& MediaOutputItem : MediaSettings.MediaOutputs)
							{
								if (IsValid(MediaOutputItem.MediaOutput) && IsValid(MediaOutputItem.OutputSyncPolicy))
								{
									// Pick the same sync policy only
									if (MediaOutputItem.OutputSyncPolicy->GetClass() == GetPolicyClass())
									{
										const FString ViewportCaptureId = DisplayClusterMediaHelpers::MediaId::GenerateMediaId(
											DisplayClusterMediaHelpers::MediaId::EMediaDeviceType::Output,
											DisplayClusterMediaHelpers::MediaId::EMediaOwnerType::Viewport,
											NodeIt.Key, RootActor->GetName(), ViewportIt.Key, CaptureIdx++);

										OutNodeToSyncCallers.FindOrAdd(NodeIt.Key).Add(ViewportCaptureId);
									}
								}
							}
						}
					}
				}
			}
		}

		////////////////
		// ICVFX capture
		{
			// Get all ICVFX camera components
			TArray<UDisplayClusterICVFXCameraComponent*> ICVFXCameraComponents;
			RootActor->GetComponents(ICVFXCameraComponents);

			// Iterate over ICVFX cameras
			for (UDisplayClusterICVFXCameraComponent* const ICVFXCameraComponent : ICVFXCameraComponents)
			{
				const FDisplayClusterConfigurationMediaICVFX& MediaSettings = ICVFXCameraComponent->CameraSettings.RenderSettings.Media;

				if (MediaSettings.bEnable)
				{
					// Full-frame camera capture
					if (MediaSettings.SplitType == EDisplayClusterConfigurationMediaSplitType::FullFrame)
					{
						uint8 CaptureIdx = 0;
						for (const FDisplayClusterConfigurationMediaOutputGroup& MediaOutputGroup : MediaSettings.MediaOutputGroups)
						{
							// Pick the same sync policy only
							if (IsValid(MediaOutputGroup.MediaOutput) && IsValid(MediaOutputGroup.OutputSyncPolicy))
							{
								if (MediaOutputGroup.OutputSyncPolicy->GetClass() == GetPolicyClass())
								{
									for (const FString& NodeId : MediaOutputGroup.ClusterNodes.ItemNames)
									{
										const FString ICVFXCaptureId = DisplayClusterMediaHelpers::MediaId::GenerateMediaId(
											DisplayClusterMediaHelpers::MediaId::EMediaDeviceType::Output,
											DisplayClusterMediaHelpers::MediaId::EMediaOwnerType::ICVFXCamera,
											NodeId, RootActor->GetName(), ICVFXCameraComponent->GetName(), CaptureIdx++);

										OutNodeToSyncCallers.FindOrAdd(NodeId).Add(ICVFXCaptureId);
									}
								}
							}
						}
					}
					else if (MediaSettings.SplitType == EDisplayClusterConfigurationMediaSplitType::UniformTiles)
					{
						uint8 CaptureIdx = 0;
						for (const FDisplayClusterConfigurationMediaTiledOutputGroup& OutputGroup : MediaSettings.TiledMediaOutputGroups)
						{
							for (const FDisplayClusterConfigurationMediaUniformTileOutput& OutputTile : OutputGroup.Tiles)
							{
								// Pick the same sync policy only
								if (IsValid(OutputTile.MediaOutput) && IsValid(OutputTile.OutputSyncPolicy))
								{
									if (OutputTile.OutputSyncPolicy->GetClass() == GetPolicyClass())
									{
										for (const FString& NodeId : OutputGroup.ClusterNodes.ItemNames)
										{
											const FString ICVFXCaptureId = DisplayClusterMediaHelpers::MediaId::GenerateMediaId(
												DisplayClusterMediaHelpers::MediaId::EMediaDeviceType::Output,
												DisplayClusterMediaHelpers::MediaId::EMediaOwnerType::ICVFXCamera,
												NodeId, RootActor->GetName(), ICVFXCameraComponent->GetName(), CaptureIdx++, &OutputTile.Position);

											OutNodeToSyncCallers.FindOrAdd(NodeId).Add(ICVFXCaptureId);
										}
									}
								}
							}
						}
					}
				}
			}
		}
	}

	LogSyncCallersMapping(OutNodeToSyncCallers);
}

void FDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierBaseHandler::LogSyncCallersMapping(const TMap<FString, TSet<FString>>& NodeToSyncCallers) const
{
	if (UE_GET_LOG_VERBOSITY(LogDisplayClusterMediaSync) >= ELogVerbosity::Type::Verbose)
	{
		FString LogMsg;
		LogMsg.Reserve(2048);

		LogMsg = FString::Printf(TEXT("'%s': Generated the following NodeToCallers mapping:\n"), *GetMediaDeviceId());
		for (const TPair<FString, TSet<FString>>& NodeMapping : NodeToSyncCallers)
		{
			LogMsg += FString::Printf(TEXT(" [%s] "), *NodeMapping.Key);
			for (const FString& CallerId : NodeMapping.Value)
			{
				LogMsg += FString::Printf(TEXT("%s, "), *CallerId);
			}
			LogMsg += TEXT("\n");
		}

		UE_LOGF(LogDisplayClusterMediaSync, Verbose, "%ls", *LogMsg);
	}
}

void FDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierBaseHandler::ProcessMediaSynchronizationCallback()
{
	UE_LOGF(LogDisplayClusterMediaSync, VeryVerbose, "'%ls': Synchronizing capture...", *GetMediaDeviceId());

	// Pass to the policy implementations
	Synchronize();
}
