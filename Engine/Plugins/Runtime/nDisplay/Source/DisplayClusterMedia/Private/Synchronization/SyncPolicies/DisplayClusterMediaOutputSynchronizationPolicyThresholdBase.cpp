// Copyright Epic Games, Inc. All Rights Reserved.

#include "Synchronization/DisplayClusterMediaOutputSynchronizationPolicyThresholdBase.h"

#include "DisplayClusterMediaLog.h"


FDisplayClusterMediaOutputSynchronizationPolicyThresholdBaseHandler::FDisplayClusterMediaOutputSynchronizationPolicyThresholdBaseHandler(UDisplayClusterMediaOutputSynchronizationPolicyThresholdBase* InPolicyObject)
	: Super(InPolicyObject)
	, MarginMs(InPolicyObject->MarginMs)
{

}

void FDisplayClusterMediaOutputSynchronizationPolicyThresholdBaseHandler::Synchronize()
{
	// Ask the sync implementation about how much time we have before next synchronization timepoint
	const double TimeLeftSeconds = GetTimeBeforeNextSyncPoint();
	// Convert to seconds
	const double MarginSeconds = double(MarginMs) / 1000;

	// In case we're unsafe, skip the upcoming sync timepoint
	if (TimeLeftSeconds < MarginSeconds)
	{
		// Sleep for a bit longer to skip the alignment timepoint
		const float SleepTime = TimeLeftSeconds * 1.01f;

		UE_LOGF(LogDisplayClusterMediaSync, VeryVerbose, "'%ls': TimeLeft(%lf) < Margin(%lf) --> Sleeping for %lf...",
			*GetMediaDeviceId(), TimeLeftSeconds, MarginSeconds, SleepTime);

		FPlatformProcess::SleepNoStats(SleepTime);
	}

	// Now make sure all the outputs have skipped the sync point, then continue output
	SyncThreadOnBarrier();

	// Leaving this function means the capture thread can continue sending the frame out
}
