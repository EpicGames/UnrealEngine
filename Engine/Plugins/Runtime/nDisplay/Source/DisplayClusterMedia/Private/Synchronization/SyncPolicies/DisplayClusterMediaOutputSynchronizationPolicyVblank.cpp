// Copyright Epic Games, Inc. All Rights Reserved.

#include "Synchronization/DisplayClusterMediaOutputSynchronizationPolicyVblank.h"

#include "IDisplayCluster.h"
#include "Render/IDisplayClusterRenderManager.h"
#include "Render/Monitoring/IDisplayClusterVblankMonitor.h"

#include "DisplayClusterMediaLog.h"


FDisplayClusterMediaOutputSynchronizationPolicyVblankHandler::FDisplayClusterMediaOutputSynchronizationPolicyVblankHandler(UDisplayClusterMediaOutputSynchronizationPolicyVblank* InPolicyObject)
	: Super(InPolicyObject)
{

}

TSubclassOf<UDisplayClusterMediaOutputSynchronizationPolicy> FDisplayClusterMediaOutputSynchronizationPolicyVblankHandler::GetPolicyClass() const
{
	return UDisplayClusterMediaOutputSynchronizationPolicyVblank::StaticClass();
}

bool FDisplayClusterMediaOutputSynchronizationPolicyVblankHandler::StartSynchronization(UMediaCapture* MediaCapture, const FString& MediaId)
{
	// Get monitor interface if not yet available
	if (!VblankMonitor)
	{
		if (IDisplayClusterRenderManager* RenderMgr = IDisplayCluster::Get().GetRenderMgr())
		{
			VblankMonitor = RenderMgr->GetVblankMonitor();
		}
	}

	// Start monitoring if not running already
	if (!VblankMonitor->IsMonitoring())
	{
		VblankMonitor->StartMonitoring();
	}

	return Super::StartSynchronization(MediaCapture, MediaId);
}

double FDisplayClusterMediaOutputSynchronizationPolicyVblankHandler::GetTimeBeforeNextSyncPoint()
{
	// Compute time remaining
	if(VblankMonitor->IsVblankTimeDataAvailable())
	{
		const double TimeRemaining = VblankMonitor->GetNextVblankTime() - FPlatformTime::Seconds();
		UE_LOGF(LogDisplayClusterMediaSync, VeryVerbose, "V-blank media sync: time before V-blank - %lf", TimeRemaining);
		return TimeRemaining;
	}

	// Return 1sec (a huge duration) to prevent calling thread blocking
	return 1;
}

TSharedPtr<IDisplayClusterMediaOutputSynchronizationPolicyHandler> UDisplayClusterMediaOutputSynchronizationPolicyVblank::GetHandler()
{
	if (!Handler)
	{
		Handler = MakeShared<FDisplayClusterMediaOutputSynchronizationPolicyVblankHandler>(this);
	}

	return Handler;
}
