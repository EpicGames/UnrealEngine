// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Synchronization/DisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierBase.h"

#include "DisplayClusterMediaOutputSynchronizationPolicyEthernetBarrier.generated.h"

/*
 * Synchronization logic handler class for UDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrier.
 */
class DISPLAYCLUSTERMEDIA_API FDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierHandler
	: public FDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierBaseHandler
{
	using Super = FDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierBaseHandler;

public:

	FDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierHandler(UDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrier* InPolicyObject);

	//~ Begin IDisplayClusterMediaOutputSynchronizationPolicyHandler interface
	virtual TSubclassOf<UDisplayClusterMediaOutputSynchronizationPolicy> GetPolicyClass() const override;
	//~ End IDisplayClusterMediaOutputSynchronizationPolicyHandler interface

protected:
	//~ Begin FDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierBaseHandler interface
	virtual void Synchronize() override;
	//~ End FDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierBaseHandler interface
};

/*
 * EthernetBarrier media synchronization policy implementation
 */
UCLASS(editinlinenew, Blueprintable, meta = (DisplayName = "Ethernet Barrier"))
class DISPLAYCLUSTERMEDIA_API UDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrier
	: public UDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierBase
{
	GENERATED_BODY()

protected:
	virtual TSharedPtr<IDisplayClusterMediaOutputSynchronizationPolicyHandler> GetHandler() override;

protected:
	TSharedPtr<IDisplayClusterMediaOutputSynchronizationPolicyHandler> Handler;
};
