// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DisplayClusterConfigurationTypes_MediaSync.h"

#include "DisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierBase.generated.h"

class IDisplayClusterGenericBarriersClient;


/*
 * Synchronization logic handler class for UDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierBase.
 */
class DISPLAYCLUSTERMEDIA_API FDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierBaseHandler
								: public IDisplayClusterMediaOutputSynchronizationPolicyHandler
{
public:

	FDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierBaseHandler(UDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierBase* InPolicyObject);

public:

	//~ Begin IDisplayClusterMediaOutputSynchronizationPolicyHandler interface
	virtual bool StartSynchronization(UMediaCapture* MediaCapture, const FString& MediaId) override;
	virtual void StopSynchronization() override;
	virtual bool IsRunning() override final;
	virtual bool IsCaptureTypeSupported(UMediaCapture* MediaCapture) const override;
	//~ End IDisplayClusterMediaOutputSynchronizationPolicyHandler interface

public:

	/** Children implement their own sync approaches. */
	virtual void Synchronize() = 0;

protected:
	
	/** Initializes dynamic barrier on the primary node. */
	virtual bool InitializeBarrier(const FString& SyncInstanceId);
	
	/** Synchronizes calling thread at the barrier. */
	void SyncThreadOnBarrier();

	/** Returns media device ID being synchronized */
	FString GetMediaDeviceId() const;

	/** Returns barrier client created for this sync policy */
	UE_DEPRECATED(5.6, "This method has been deprecated. Please use another 'GetBarrierClient' that returns TSharedPtr.")
	IDisplayClusterGenericBarriersClient* const GetBarrierClient() const
	{
		return EthernetBarrierClient.Get();
	}

	/** Returns barrier client created for this sync policy */
	TSharedPtr<IDisplayClusterGenericBarriersClient> GetBarrierClient();

	/** Get Barrier ID for this sync policy */
	const FString& GetBarrierId() const;

	/** Get thread marker for this sync policy */
	const FString& GetThreadMarker() const;

private:

	/** Releases dynamic barrier on the primary node. */
	void ReleaseBarrier();

	/** Generates name of the dynamic barrier. */
	FString GenerateBarrierName() const;

	/** Generates per-node sync callers mapping. */
	void GenerateSyncCallersMapping(TMap<FString, TSet<FString>>& OutMarkers) const;

	/** Auxiliary function to log custom barrier callers mapping. */
	void LogSyncCallersMapping(const TMap<FString, TSet<FString>>& OutNodeToSyncCallers) const;

	UE_DEPRECATED(5.6, "This method has been deprecated. Please use 'GenerateSyncCallersMapping'.")
	void GenerateListOfThreadMarkers(TArray<FString>& OutMarkers) const
	{ }

	/** Handles media capture sync callbacks. */
	void ProcessMediaSynchronizationCallback();

protected:

	/** Capture device being used. */
	TObjectPtr<UMediaCapture> CapturingDevice;

	/** Barrier timeout (ms) */
	int32 BarrierTimeoutMs = 3000;

private:
	/** Is synchronization currently active. */
	bool bIsRunning = false;

	/** ID of media device being synchronized. */
	FString MediaDeviceId;

	/** Unique barrier name to use. */
	FString BarrierId;

	/** Unique thread (caller) marker to be used on the barrier. */
	FString ThreadMarker;

	/** Generic barriers API. */
	TSharedPtr<IDisplayClusterGenericBarriersClient> EthernetBarrierClient;
};

/*
 * Base class for Ethernet barrier based media synchronization policies.
 * 
 * It encapsulates network barriers related settings.
 */
UCLASS(Abstract, editinlinenew, BlueprintType, hidecategories = (Object))
class DISPLAYCLUSTERMEDIA_API UDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierBase
	: public UDisplayClusterMediaOutputSynchronizationPolicy
{
	GENERATED_BODY()

public:

	/** Barrier timeout (ms) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (DisplayName = "Barrier Timeout (ms)", ClampMin = "1", ClampMax = "10000", UIMin = "1", UIMax = "10000"))
	int32 BarrierTimeoutMs = 3000;
};
