// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Synchronization/DisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierBase.h"

#include "DisplayClusterMediaOutputSynchronizationPolicyThresholdBase.generated.h"

/*
 * Synchronization logic handler class for UDisplayClusterMediaOutputSynchronizationPolicyThresholdBase.
 */
class DISPLAYCLUSTERMEDIA_API FDisplayClusterMediaOutputSynchronizationPolicyThresholdBaseHandler
	: public FDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierBaseHandler
{
	using Super = FDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierBaseHandler;
public:
	
	FDisplayClusterMediaOutputSynchronizationPolicyThresholdBaseHandler(UDisplayClusterMediaOutputSynchronizationPolicyThresholdBase* InPolicyObject);

protected:
	//~ Begin FDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierBaseHandler interface
	virtual void Synchronize() override;
	//~ End FDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierBaseHandler interface

protected:
	/** Returns amount of time before next synchronization point. */
	virtual double GetTimeBeforeNextSyncPoint() = 0;

protected:
	/** Synchronization margin (ms) */
	int32 MarginMs = 5;
};

/*
 * Base class for threshold based media synchronization policies.
 * 
 * Basically it uses the same approach that we use in 'Ethernet' sync policy where v-blanks are used as the timepoints.
 */
UCLASS(Abstract, editinlinenew, BlueprintType, hidecategories = (Object))
class DISPLAYCLUSTERMEDIA_API UDisplayClusterMediaOutputSynchronizationPolicyThresholdBase
	: public UDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierBase
{
	GENERATED_BODY()

public:
	/** Synchronization margin (ms) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (DisplayName = "Margin (ms)", ClampMin = "1", ClampMax = "20", UIMin = "1", UIMax = "20"))
	int32 MarginMs = 5;
};
