// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Synchronization/DisplayClusterMediaOutputSynchronizationPolicyThresholdBase.h"

#include "DisplayClusterMediaOutputSynchronizationPolicyVblank.generated.h"

class IDisplayClusterVblankMonitor;

/*
 * Synchronization logic handler class for UDisplayClusterMediaOutputSynchronizationPolicyVblank.
 */
class DISPLAYCLUSTERMEDIA_API FDisplayClusterMediaOutputSynchronizationPolicyVblankHandler
	: public FDisplayClusterMediaOutputSynchronizationPolicyThresholdBaseHandler
{
	using Super = FDisplayClusterMediaOutputSynchronizationPolicyThresholdBaseHandler;

public:

	FDisplayClusterMediaOutputSynchronizationPolicyVblankHandler(UDisplayClusterMediaOutputSynchronizationPolicyVblank* InPolicyObject);

	//~ Begin IDisplayClusterMediaOutputSynchronizationPolicyHandler interface
	virtual TSubclassOf<UDisplayClusterMediaOutputSynchronizationPolicy> GetPolicyClass() const override;
	virtual bool StartSynchronization(UMediaCapture* MediaCapture, const FString& MediaId) override;
	//~ End IDisplayClusterMediaOutputSynchronizationPolicyHandler interface

protected:
	/** Returns amount of time before next synchronization point. */
	virtual double GetTimeBeforeNextSyncPoint() override;
	
private:
	// V-blank monitor
	TSharedPtr<IDisplayClusterVblankMonitor> VblankMonitor;
};

/*
 * Vblank media synchronization policy config
 */
UCLASS(editinlinenew, Blueprintable, meta = (DisplayName = "V-blank"))
class DISPLAYCLUSTERMEDIA_API UDisplayClusterMediaOutputSynchronizationPolicyVblank
	: public UDisplayClusterMediaOutputSynchronizationPolicyThresholdBase
{
	GENERATED_BODY()

public:
	virtual TSharedPtr<IDisplayClusterMediaOutputSynchronizationPolicyHandler> GetHandler() override;

protected:
	TSharedPtr<IDisplayClusterMediaOutputSynchronizationPolicyHandler> Handler;
};
