// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class DisplayClusterMediaEditor : ModuleRules
{
	public DisplayClusterMediaEditor(ReadOnlyTargetRules ROTargetRules) : base(ROTargetRules)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"AssetDefinition",
				"Core",
				"CoreUObject",
				"DisplayClusterMedia",
				"Engine",
				"MediaIOCore",
				"MediaIOEditor",
				"SharedMemoryMedia",
				"UnrealEd",
			});
	}
}
