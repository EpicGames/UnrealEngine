// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AssetDefinition/AssetDefinition_MediaOutput.h"
#include "SharedMemoryMediaOutput.h"
#include "AssetDefinition_SharedMemoryMediaOutput.generated.h"

UCLASS()
class UAssetDefinition_SharedMemoryMediaOutput : public UAssetDefinition_MediaOutput
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_SharedMemoryMediaOutput", "Shared Memory Media Output"); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return USharedMemoryMediaOutput::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Media, NSLOCTEXT("AssetDefinition", "AssetDefinition_SharedMemoryMediaOutputSubMenu", "Media Sources + Outputs"), ECategoryMenuType::Section) };
		return Categories;
	}
	// UAssetDefinition End
};
