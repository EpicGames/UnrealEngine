// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterMediaOutputSynchronizationPolicyFactories.h"

#include "AssetTypeCategories.h"

#include "Synchronization/DisplayClusterMediaOutputSynchronizationPolicyEthernetBarrier.h"
#include "Synchronization/DisplayClusterMediaOutputSynchronizationPolicyVblank.h"


/**
 * UDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrier
 */
UDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierFactory::UDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrier::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

UObject* UDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrier>(InParent, InClass, InName, Flags);
}

uint32 UDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierFactory::GetMenuCategories() const
{
	return EAssetTypeCategories::None;
}

bool UDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierFactory::ShouldShowInNewMenu() const
{
	return true;
}


/**
 * UDisplayClusterMediaOutputSynchronizationPolicyVblank
 */
UDisplayClusterMediaOutputSynchronizationPolicyVblankFactory::UDisplayClusterMediaOutputSynchronizationPolicyVblankFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UDisplayClusterMediaOutputSynchronizationPolicyVblank::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

UObject* UDisplayClusterMediaOutputSynchronizationPolicyVblankFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UDisplayClusterMediaOutputSynchronizationPolicyVblank>(InParent, InClass, InName, Flags);
}

uint32 UDisplayClusterMediaOutputSynchronizationPolicyVblankFactory::GetMenuCategories() const
{
	return EAssetTypeCategories::None;
}

bool UDisplayClusterMediaOutputSynchronizationPolicyVblankFactory::ShouldShowInNewMenu() const
{
	return true;
}
