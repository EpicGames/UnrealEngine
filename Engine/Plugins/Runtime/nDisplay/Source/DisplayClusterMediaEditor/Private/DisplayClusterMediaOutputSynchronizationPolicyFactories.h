// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"

#include "DisplayClusterMediaOutputSynchronizationPolicyFactories.generated.h"


/**
 * Factory for UDisplayClusterMediaSynchronizationPolicyEthernetBarrier objects.
 */
UCLASS(hidecategories=Object)
class UDisplayClusterMediaOutputSynchronizationPolicyEthernetBarrierFactory
	: public UFactory
{
	GENERATED_UCLASS_BODY()

public:
	//~ UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual uint32 GetMenuCategories() const override;
	virtual bool ShouldShowInNewMenu() const override;
};


/**
 * Factory for UDisplayClusterMediaSynchronizationPolicyVblank objects.
 */
UCLASS(hidecategories = Object)
class UDisplayClusterMediaOutputSynchronizationPolicyVblankFactory
	: public UFactory
{
	GENERATED_UCLASS_BODY()

public:
	//~ UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual uint32 GetMenuCategories() const override;
	virtual bool ShouldShowInNewMenu() const override;
};
