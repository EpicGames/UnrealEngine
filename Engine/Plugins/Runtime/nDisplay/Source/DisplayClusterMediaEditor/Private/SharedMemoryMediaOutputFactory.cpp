// Copyright Epic Games, Inc. All Rights Reserved.

#include "SharedMemoryMediaOutputFactory.h"

#include "AssetTypeCategories.h"
#include "SharedMemoryMediaOutput.h"
#include "EngineAnalytics.h"


/* USharedMemoryMediaOutputFactoryNew structors
 *****************************************************************************/

USharedMemoryMediaOutputFactory::USharedMemoryMediaOutputFactory()
{
	SupportedClass = USharedMemoryMediaOutput::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}


/* UFactory overrides
 *****************************************************************************/

/**
 * @EventName MediaFramework.SharedMemoryMediaOutputCreated
 * @Trigger Triggered when an SharedMemory media output asset is created.
 * @Type Client
 * @Owner MediaIO Team
 */
UObject* USharedMemoryMediaOutputFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("MediaFramework.SharedMemoryMediaOutputCreated"));
	}
	
	return NewObject<USharedMemoryMediaOutput>(InParent, InClass, InName, Flags);
}


uint32 USharedMemoryMediaOutputFactory::GetMenuCategories() const
{
	return EAssetTypeCategories::Media;
}


bool USharedMemoryMediaOutputFactory::ShouldShowInNewMenu() const
{
	return true;
}
