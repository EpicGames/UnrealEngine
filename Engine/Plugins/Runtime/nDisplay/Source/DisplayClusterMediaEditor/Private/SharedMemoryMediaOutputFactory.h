// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "UObject/ObjectMacros.h"

#include "SharedMemoryMediaOutputFactory.generated.h"


/**
 * Implements a factory for USharedMemoryMediaOutput objects.
 */
UCLASS(hidecategories=Object)
class USharedMemoryMediaOutputFactory : public UFactory
{
	GENERATED_BODY()

public:

	USharedMemoryMediaOutputFactory();

	//~ UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual uint32 GetMenuCategories() const override;
	virtual bool ShouldShowInNewMenu() const override;
};
