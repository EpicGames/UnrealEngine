// Copyright Epic Games, Inc. All Rights Reserved.

#include "SharedMemoryMediaSourceFactory.h"

#include "AssetTypeCategories.h"
#include "EngineAnalytics.h"
#include "SharedMemoryMediaSource.h"


/* USharedMemoryMediaSourceFactory structors
 *****************************************************************************/

USharedMemoryMediaSourceFactory::USharedMemoryMediaSourceFactory()
{
	SupportedClass = USharedMemoryMediaSource::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}


/* UFactory overrides
 *****************************************************************************/

 /**
 * @EventName MediaFramework.SharedMemoryMediaSourceCreated
 * @Trigger Triggered when a SharedMemory media source asset is created.
 * @Type Client
 * @Owner MediaIO Team
 */
UObject* USharedMemoryMediaSourceFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("MediaFramework.SharedMemoryMediaSourceCreated"));
	}

	return NewObject<USharedMemoryMediaSource>(InParent, InClass, InName, Flags);
}


uint32 USharedMemoryMediaSourceFactory::GetMenuCategories() const
{
	return EAssetTypeCategories::Media;
}


bool USharedMemoryMediaSourceFactory::ShouldShowInNewMenu() const
{
	return true;
}
