// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "SharedMemoryMediaSourceFactory.generated.h"


/**
 * Implements a factory for USharedMemoryMediaSource objects.
 */
UCLASS(hidecategories=Object)
class USharedMemoryMediaSourceFactory
	: public UFactory
{
	GENERATED_BODY()

public:

	USharedMemoryMediaSourceFactory();

	//~ UFactory Interface

	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual uint32 GetMenuCategories() const override;
	virtual bool ShouldShowInNewMenu() const override;
};
