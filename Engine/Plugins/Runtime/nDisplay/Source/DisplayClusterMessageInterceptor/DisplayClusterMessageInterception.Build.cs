// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class DisplayClusterMessageInterception : ModuleRules
{
	public DisplayClusterMessageInterception(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"DisplayCluster",
				"Engine",
				"Messaging"
			});
		if (Target.Type == TargetType.Editor || Target.Type == TargetType.Program)
		{
			PrivateDefinitions.Add("WITH_CONCERT=1");
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Concert",
					"ConcertSyncClient",
					"ConcertSyncCore",
					"ConcertTransport"
				});
		}
	}
}
