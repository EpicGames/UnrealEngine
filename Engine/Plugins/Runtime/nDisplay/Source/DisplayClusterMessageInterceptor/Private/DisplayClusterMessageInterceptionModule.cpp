// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#include "Cluster/DisplayClusterClusterEvent.h"
#include "Cluster/IDisplayClusterClusterManager.h"
#include "DisplayClusterGameEngine.h"
#include "DisplayClusterMessageInterceptor.h"
#include "DisplayClusterMessageInterceptionSettings.h"
#include "Engine/Engine.h"
#include "HAL/IConsoleManager.h"
#include "Templates/Atomic.h"
#include "IDisplayCluster.h"
#include "IDisplayClusterCallbacks.h"
#include "IMessageBus.h"
#include "IMessagingModule.h"

#if defined(WITH_CONCERT)
#include "IConcertClient.h"
#include "IConcertSyncClient.h"
#include "IConcertSession.h"
#include "IConcertClientWorkspace.h"
#include "IConcertSyncClientModule.h"
#endif

#if WITH_EDITOR
	#include "ISettingsModule.h"
	#include "ISettingsSection.h"
#endif

#include <atomic>

#define LOCTEXT_NAMESPACE "DisplayClusterInterception"


namespace DisplayClusterInterceptionModuleUtils
{
	static const FString EventSetup = TEXT("nDCISetup");
	static const FString EventSync = TEXT("nDCIMUSync");
	static const FString PackageSync = TEXT("nDCIPackageSync");
	static const FString EventParameterSettings = TEXT("Settings");
}

class FSetBoolOnPreTick
{
public:
	operator bool() const
	{
		FScopeLock Lock(&InternalsCS);
		return bValue;
	}

	FSetBoolOnPreTick& operator=(bool bInValue)
	{
		FScopeLock Lock(&InternalsCS);
		if (bInValue != bValue)
		{
			bPendingValue = bInValue;
		}
		return *this;
	}

	void ResetWithValue(bool bInValue)
	{
		FScopeLock Lock(&InternalsCS);

		bValue = bInValue;
		bPendingValue.Reset();
	}

	void UpdateOnPreTick()
	{
		FScopeLock Lock(&InternalsCS);

		if (bPendingValue.IsSet())
		{
			bValue = *bPendingValue;
			bPendingValue.Reset();
		}
	}

private:
	bool bValue = false;
	TOptional<bool> bPendingValue = {};
	mutable FCriticalSection InternalsCS;
};
/**
 * Display Cluster Message Interceptor module
 * Intercept a specified set of message bus messages that are received across all the display nodes
 * to process them in sync across the cluster.
 */
class FDisplayClusterMessageInterceptionModule : public IModuleInterface
{
public:
	virtual void StartupModule() override
	{
		if (IDisplayCluster::IsAvailable())
		{
			// Register for Cluster StartSession callback so everything is setup before launching interception
			IDisplayCluster::Get().GetCallbacks().OnDisplayClusterStartSession().AddRaw(this, &FDisplayClusterMessageInterceptionModule::OnDisplayClusterStartSession);
			IDisplayCluster::Get().GetCallbacks().OnDisplayClusterStartScene().AddRaw(this, &FDisplayClusterMessageInterceptionModule::OnNewSceneEvent);
		}

		// Setup console command to start/stop interception
		StartMessageSyncCommand = MakeUnique<FAutoConsoleCommand>(
			TEXT("nDisplay.MessageBusSync.Start"),
			TEXT("Start MessageBus syncing"),
			FConsoleCommandDelegate::CreateRaw(this, &FDisplayClusterMessageInterceptionModule::StartInterception)
			);
		StopMessageSyncCommand = MakeUnique<FAutoConsoleCommand>(
			TEXT("nDisplay.MessageBusSync.Stop"),
			TEXT("Stop MessageBus syncing"),
			FConsoleCommandDelegate::CreateRaw(this, &FDisplayClusterMessageInterceptionModule::StopInterception));

		ResetMultiUserSyncCommand = MakeUnique<FAutoConsoleCommand>(
			TEXT("nDisplay.MessageBusSync.ResetSyncFlags"),
			TEXT("Reset interceptor sync flags for Multi-user session connections."),
			FConsoleCommandDelegate::CreateRaw(this, &FDisplayClusterMessageInterceptionModule::ResetSyncFlags)
			);


#if WITH_EDITOR
		if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
		{
			SettingsModule->RegisterSettings(
				"Project", "Plugins", "nDisplay Message Interception",
				LOCTEXT("InterceptionSettingsName", "nDisplay Message Interception"),
				LOCTEXT("InterceptionSettingsDescription", "Configure nDisplay Message Interception."),
				GetMutableDefault<UDisplayClusterMessageInterceptionSettings>()
			);
		}
#endif
	}

	virtual void ShutdownModule() override
	{
#if WITH_EDITOR
		if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
		{
			SettingsModule->UnregisterSettings("Project", "Plugins", "nDisplay Message Interception");
		}
#endif

#if defined(WITH_CONCERT)
		if (IConcertSyncClientModule::IsAvailable())
		{
			if (TSharedPtr<IConcertSyncClient> ConcertSyncClient = IConcertSyncClientModule::Get().GetClient(TEXT("MultiUser")))
			{
				IConcertClientRef ConcertClient = ConcertSyncClient->GetConcertClient();
				ConcertClient->OnSessionStartup().RemoveAll(this);
				ConcertClient->OnSessionShutdown().RemoveAll(this);
				ConcertClient->OnSessionConnectionChanged().RemoveAll(this);
			}
		}
#endif

		if (IDisplayCluster::IsAvailable())
		{
			// Unregister cluster event listening
			IDisplayClusterClusterManager* ClusterManager = IDisplayCluster::Get().GetClusterMgr();
			if (ClusterManager && ListenerDelegate.IsBound())
			{
				ClusterManager->RemoveClusterEventJsonListener(ListenerDelegate);
				ListenerDelegate.Unbind();
			}

			// Unregister cluster session events
			IDisplayCluster::Get().GetCallbacks().OnDisplayClusterStartSession().RemoveAll(this);
			IDisplayCluster::Get().GetCallbacks().OnDisplayClusterStartScene().RemoveAll(this);
			IDisplayCluster::Get().GetCallbacks().OnDisplayClusterEndSession().RemoveAll(this);
			IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPreTick().RemoveAll(this);
			IDisplayCluster::Get().GetCallbacks().OnDisplayClusterFailoverNodeDown().RemoveAll(this);
		}
		Interceptor.Reset();
		StartMessageSyncCommand.Reset();
		StopMessageSyncCommand.Reset();
		ResetMultiUserSyncCommand.Reset();
	}

private:
	void ResetSyncFlags()
	{
		// Reset the was ever disconnected flag to allow sync to happen again on when joining a session. 
		bWasEverDisconnected = false;
	}
		
	void SetupForMultiUser()
	{
#if defined(WITH_CONCERT)
		if (TSharedPtr<IConcertSyncClient> ConcertSyncClient = IConcertSyncClientModule::Get().GetClient(TEXT("MultiUser")))
		{
			TSharedPtr<IConcertClientWorkspace> Workspace = ConcertSyncClient->GetWorkspace();
			IConcertClientRef ConcertClient = ConcertSyncClient->GetConcertClient();

			ConcertClient->OnSessionStartup().AddRaw(this, &FDisplayClusterMessageInterceptionModule::OnMultiUserStartup);
			ConcertClient->OnSessionShutdown().AddRaw(this, &FDisplayClusterMessageInterceptionModule::OnMultiUserShutdown);
			ConcertClient->OnSessionConnectionChanged().AddRaw(this, &FDisplayClusterMessageInterceptionModule::OnSessionConnectionChanged);

			if (Workspace.IsValid())
			{
				Workspace->RemoveWorkspaceFinalizeDelegate(TEXT("nDisplay Interceptor"));
				Workspace->OnWorkspaceSynchronized().RemoveAll(this);
			}
		}
		else
		{
			UE_LOGF(LogDisplayClusterInterception, Display, "No multi-user detected. Not intercepting initial activity sync.");
		}
#else
		UE_LOGF(LogDisplayClusterInterception, Display, "No multi-user available.");
#endif
	}
	void OnNewSceneEvent()
	{
		IDisplayClusterClusterManager* ClusterManager = IDisplayCluster::Get().GetClusterMgr();
		if (bStartInterceptionRequested && !bSettingsSynchronizationDone && ClusterManager && ListenerDelegate.IsBound())
		{
			// If we receive a NewScene event then our previous sync event was lost due to flushing of
			// the event queues during EndScene / NewScene.  Re-establish the event here:
			ResendSyncEvent(ClusterManager);
		}
		UE_LOGF(LogDisplayClusterInterception, Display, "New scene event");
	}

	void ResendSyncEvent(IDisplayClusterClusterManager* ClusterManager)
	{
		// Primary node will send out its interceptor settings to the cluster so everyone uses the same things
		if (ClusterManager->IsPrimary())
		{
			FString ExportedSettings;
			const UDisplayClusterMessageInterceptionSettings* CurrentSettings = GetDefault<UDisplayClusterMessageInterceptionSettings>();
			FMessageInterceptionSettings::StaticStruct()->ExportText(ExportedSettings, &CurrentSettings->InterceptionSettings, nullptr, nullptr, PPF_None, nullptr);

			FDisplayClusterClusterEventJson SettingsEvent;
			SettingsEvent.Category = DisplayClusterInterceptionModuleUtils::EventSetup;
			SettingsEvent.Name = ClusterManager->GetNodeId();
			SettingsEvent.bIsSystemEvent = true;
			SettingsEvent.Parameters.FindOrAdd(DisplayClusterInterceptionModuleUtils::EventParameterSettings) = MoveTemp(ExportedSettings);
			const bool bPrimaryOnly = true;
			ClusterManager->EmitClusterEventJson(SettingsEvent, bPrimaryOnly);
		}
	}

	void PackageSyncEvent(int64 ActivityId)
	{
		if (!CanFinalizeWorkspaceSync())
		{
			return;
		}

		UE_LOGF(LogDisplayClusterInterception, Display, "Sending package sync event.");
		IDisplayClusterClusterManager* ClusterManager = IDisplayCluster::Get().GetClusterMgr();
		check(ClusterManager != nullptr);

		bAllowPackages = false;

		FDisplayClusterClusterEventJson SyncMessagesEvent;
		SyncMessagesEvent.Category = DisplayClusterInterceptionModuleUtils::PackageSync;
		SyncMessagesEvent.Type = LexToString(ActivityId);
		SyncMessagesEvent.Name = ClusterManager->GetNodeId();	// which node got the message
		SyncMessagesEvent.bIsSystemEvent = true;				// nDisplay internal event
		SyncMessagesEvent.bShouldDiscardOnRepeat = false;		// Don' discard the events with the same cat/type/name
		const bool bPrimaryOnly = false;							// All nodes are broadcasting events to synchronize them across cluster
		ClusterManager->EmitClusterEventJson(SyncMessagesEvent, bPrimaryOnly);
	}

	void WorkspaceSyncEvent()
	{
		if (bWasEverDisconnected)
		{
			return;
		}

		UE_LOGF(LogDisplayClusterInterception, Display, "Sending activity sync event.");
		IDisplayClusterClusterManager* ClusterManager = IDisplayCluster::Get().GetClusterMgr();
		check(ClusterManager != nullptr);

		FDisplayClusterClusterEventJson SyncMessagesEvent;
		SyncMessagesEvent.Category = DisplayClusterInterceptionModuleUtils::EventSync;
		SyncMessagesEvent.Type = TEXT("WorkspaceSync");			// Required by nDisplay or message is discarded.
		SyncMessagesEvent.Name = ClusterManager->GetNodeId();	// which node got the message
		SyncMessagesEvent.bIsSystemEvent = true;				// nDisplay internal event
		SyncMessagesEvent.bShouldDiscardOnRepeat = false;		// Don' discard the events with the same cat/type/name
		const bool bPrimaryOnly = false;							// All nodes are broadcasting events to synchronize them across cluster
		ClusterManager->EmitClusterEventJson(SyncMessagesEvent, bPrimaryOnly);
	}

	bool CanFinalizeWorkspaceSync() const
	{
		return CanFinalizeWorkspace || bWasEverDisconnected;
	}

	bool CanProcessPendingPackages() const
	{
		return bAllowPackages;
	}

#if defined(WITH_CONCERT)
	void OnSessionConnectionChanged(IConcertClientSession& InSession, EConcertConnectionStatus ConnectionStatus)
	{
		TSharedPtr<IConcertSyncClient> ConcertSyncClient = IConcertSyncClientModule::Get().GetClient(TEXT("MultiUser"));
		check(ConcertSyncClient.IsValid());

		TSharedPtr<IConcertClientWorkspace> Workspace = ConcertSyncClient->GetWorkspace();
		if (ConnectionStatus == EConcertConnectionStatus::Connected)
		{
			ResetFinalizeSync();
			Workspace->OnWorkspaceSynchronized().AddRaw(this, &FDisplayClusterMessageInterceptionModule::WorkspaceSyncEvent);
			Workspace->OnActivityAddedOrUpdated().AddRaw(this, &FDisplayClusterMessageInterceptionModule::ActivityUpdated);
			Workspace->AddWorkspaceFinalizeDelegate(TEXT("nDisplay Interceptor"),
													FCanFinalizeWorkspaceDelegate::CreateRaw(
														this, &FDisplayClusterMessageInterceptionModule::CanFinalizeWorkspaceSync));
			Workspace->AddWorkspaceCanProcessPackagesDelegate(TEXT("nDisplay Interceptor"),
															  FCanProcessPendingPackages::CreateRaw(
																  this, &FDisplayClusterMessageInterceptionModule::CanProcessPendingPackages));
		}
		else if (ConnectionStatus == EConcertConnectionStatus::Disconnected)
		{
			// Once we disconnect from MU it is not possible to coordinate an activity sync any longer because other
			// nodes may be still connected and not expected to re-connect. In this case we never reenter into an
			// activity sync event.
			//
			bWasEverDisconnected = true;
			CanFinalizeWorkspace = true;
			bAllowPackages = true;

			UE_LOGF(LogDisplayClusterInterception, Display, "Disconnecting from session.");
			Workspace->RemoveWorkspaceFinalizeDelegate(TEXT("nDisplay Interceptor"));
			Workspace->RemoveWorkspaceCanProcessPackagesDelegate(TEXT("nDisplay Interceptor"));
			Workspace->OnWorkspaceSynchronized().RemoveAll(this);
			Workspace->OnActivityAddedOrUpdated().RemoveAll(this);
		}
	}

	void ActivityUpdated(const FConcertClientInfo& InClientInfo, const FConcertSyncActivity& InActivity, const FStructOnScope& /*unused*/)
	{
		if (InActivity.EventType == EConcertSyncActivityEventType::Package)
		{
			PackageSyncEvent(InActivity.ActivityId);
		}
	}

	void OnMultiUserStartup(TSharedRef<IConcertClientSession> InSession)
	{
		if (InSession->GetConnectionStatus() == EConcertConnectionStatus::Connected)
		{
			OnSessionConnectionChanged(*InSession, EConcertConnectionStatus::Connected);
		}
	}

	void OnMultiUserShutdown(TSharedRef<IConcertClientSession> InSession)
	{
		CanFinalizeWorkspace = true;
	}
#endif

	void OnDisplayClusterStartSession()
	{
		if (IDisplayCluster::IsAvailable() && IDisplayCluster::Get().GetOperationMode() == EDisplayClusterOperationMode::Cluster)
		{
			// Create the message interceptor only when we're in cluster mode
			Interceptor = MakeShared<FDisplayClusterMessageInterceptor, ESPMode::ThreadSafe>();

			// Register cluster event listeners
			IDisplayClusterClusterManager* ClusterManager = IDisplayCluster::Get().GetClusterMgr();
			if (ClusterManager && !ListenerDelegate.IsBound())
			{
				ListenerDelegate = FOnClusterEventJsonListener::CreateRaw(this, &FDisplayClusterMessageInterceptionModule::HandleClusterEvent);
				ClusterManager->AddClusterEventJsonListener(ListenerDelegate);
				ResendSyncEvent(ClusterManager);
			}

			//Start with interception enabled
			bStartInterceptionRequested = true;

			// Register cluster session events
			IDisplayCluster::Get().GetCallbacks().OnDisplayClusterEndSession().AddRaw(this, &FDisplayClusterMessageInterceptionModule::StopInterception);
			IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPreTick().AddRaw(this, &FDisplayClusterMessageInterceptionModule::HandleClusterPreTick);
			IDisplayCluster::Get().GetCallbacks().OnDisplayClusterFailoverNodeDown().AddRaw(this, &FDisplayClusterMessageInterceptionModule::HandleClusterNodeFailed);

			SetupForMultiUser();
		}
	}

	void HandleClusterEvent(const FDisplayClusterClusterEventJson& InEvent)
	{
		if (InEvent.Category == DisplayClusterInterceptionModuleUtils::EventSetup)
		{
			HandleMessageInterceptorSetupEvent(InEvent);
		}
		else if (InEvent.Category == DisplayClusterInterceptionModuleUtils::EventSync)
		{
			HandleWorkspaceSyncEvent(InEvent);
		}
		else if (InEvent.Category == DisplayClusterInterceptionModuleUtils::PackageSync)
		{
			HandlePackageEvent(InEvent);
		}
		else
		{
			//All events except our settings synchronization one are passed to the interceptor
			if (Interceptor)
			{
				Interceptor->HandleClusterEvent(InEvent);
			}
		}

	}

	void HandleClusterPreTick()
	{
		// StartInterception will be handled once settings synchronization is completed to ensure the same behavior across the cluster
		if (bStartInterceptionRequested)
		{
			IDisplayClusterClusterManager* ClusterManager = IDisplayCluster::Get().GetClusterMgr();
			if (Interceptor && ClusterManager)
			{
				if (bSettingsSynchronizationDone)
				{
					UE_LOGF(LogDisplayClusterInterception, Display, "Sync received! Starting interception!");
					bStartInterceptionRequested = false;
					Interceptor->Start(IMessagingModule::Get().GetDefaultBus().ToSharedRef());
				}
			}
		}

		if (Interceptor)
		{
			Interceptor->SyncMessages();
		}
		CanFinalizeWorkspace.UpdateOnPreTick();
	}

	void StartInterception()
	{
		bStartInterceptionRequested = true;
	}

	void StopInterception()
	{
		if (Interceptor)
		{
			Interceptor->Stop();
		}
	}

	void HandleClusterNodeFailed(const FString& NodeId)
	{
		UE_LOGF(LogDisplayClusterInterception, Log, "Handling '%ls' node failure...", *NodeId);

		HandleWorkspaceNodeFailure(NodeId);
		HandlePackageNodeFailure(NodeId);

		if (Interceptor)
		{
			Interceptor->HandleClusterNodeFailure(NodeId);
		}
	}

	void HandlePackageEvent(const FDisplayClusterClusterEventJson& InEvent)
	{
		UE_LOGF(LogDisplayClusterInterception, Display, "Handle multi-user package event sync with id %ls -> %ls.", *InEvent.Type, *InEvent.Name);

		FScopeLock Lock(&PackageActivitiesCS);

		TSet<FString>& NodesReceivedPackage = PackageActivities.FindOrAdd(InEvent.Type);
		NodesReceivedPackage.Add(InEvent.Name);

		IDisplayClusterClusterManager* ClusterManager = IDisplayCluster::Get().GetClusterMgr();
		if (NodesReceivedPackage.Num() >= (int32)ClusterManager->GetNodesAmount())
		{
			UE_LOGF(LogDisplayClusterInterception, Display, "All nodes have received package event %ls.", *InEvent.Type);
			PackageActivities.Remove(InEvent.Type);
			if (PackageActivities.Num() == 0)
			{
				UE_LOGF(LogDisplayClusterInterception, Display, "Releasing the package lock.");
				bAllowPackages = true;
			}
		}
	}


	void HandlePackageNodeFailure(const FString& NodeId)
	{
		FScopeLock Lock(&PackageActivitiesCS);

		TSet<FString> ActivitiesToClean;

		const int32 NodesAmount = (int32)IDisplayCluster::Get().GetClusterMgr()->GetNodesAmount();

		// Remove this node from the awaiting list of all activities
		for (TPair<FString, TSet<FString>>& PackageActivity : PackageActivities)
		{
			if (PackageActivity.Value.Remove(NodeId) > 0)
			{
				if (PackageActivity.Value.Num() >= NodesAmount)
				{
					UE_LOGF(LogDisplayClusterInterception, Display, "All nodes have received package event %ls.", *PackageActivity.Key);
					ActivitiesToClean.Add(PackageActivity.Key);
				}
			}
		}

		// Forget activities that have been received already by all nodes
		for (const FString& ActivityId : ActivitiesToClean)
		{
			PackageActivities.Remove(ActivityId);
		}

		// Release lock
		if (PackageActivities.Num() == 0)
		{
			UE_LOGF(LogDisplayClusterInterception, Display, "Releasing the package lock.");
			bAllowPackages = true;
		}
	}

	void HandleWorkspaceSyncEvent(const FDisplayClusterClusterEventJson& InEvent)
	{
		UE_LOGF(LogDisplayClusterInterception, Display, "Handle multi-user workspace sync -> %ls.", *InEvent.Name);

		IDisplayClusterClusterManager* ClusterManager = IDisplayCluster::Get().GetClusterMgr();

		FScopeLock Lock(&NodesReadyCS);

		NodesReady.Add(InEvent.Name);
		if (NodesReady.Num() >= (int32)ClusterManager->GetNodesAmount())
		{
			UE_LOGF(LogDisplayClusterInterception, Display, "Allowing multi-user workspace sync on next pre-tick.");
			CanFinalizeWorkspace = true;
		}
	}

	void HandleWorkspaceNodeFailure(const FString& NodeId)
	{
		FScopeLock Lock(&NodesReadyCS);

		if (NodesReady.Remove(NodeId) > 0)
		{
			if (NodesReady.Num() >= (int32)IDisplayCluster::Get().GetClusterMgr()->GetNodesAmount())
			{
				UE_LOGF(LogDisplayClusterInterception, Display, "Allowing multi-user workspace sync on next pre-tick.");
				CanFinalizeWorkspace = true;
			}
		}
	}

	void HandleMessageInterceptorSetupEvent(const FDisplayClusterClusterEventJson& InEvent)
	{
		if (Interceptor)
		{
			const FString& ExportedSettings = InEvent.Parameters.FindChecked(DisplayClusterInterceptionModuleUtils::EventParameterSettings);
			FMessageInterceptionSettings::StaticStruct()->ImportText(*ExportedSettings, &SynchronizedSettings, nullptr, EPropertyPortFlags::PPF_None, GLog, FMessageInterceptionSettings::StaticStruct()->GetName());

			IDisplayClusterClusterManager* ClusterManager = IDisplayCluster::Get().GetClusterMgr();
			if (ClusterManager)
			{
				Interceptor->Setup(ClusterManager, SynchronizedSettings);
				bSettingsSynchronizationDone = true;

				UE_LOGF(LogDisplayClusterInterception, Display, "Node '%ls' received synchronization settings event", *ClusterManager->GetNodeId());
			}
		}
	}

private:
	void ResetFinalizeSync()
	{
		UE_LOGF(LogDisplayClusterInterception, Display, "Temporarily disabling multi-user workspace sync.");
		CanFinalizeWorkspace.ResetWithValue(false);

		// Note, we intentially do not reset NodesReady here because of the timing of joining the MU session thereby causing the node
		// to forever not receive updates.
	}

	bool bWasEverDisconnected = false;
	FSetBoolOnPreTick CanFinalizeWorkspace;

	TSet<FString> NodesReady;
	FCriticalSection NodesReadyCS;

	std::atomic<bool> bAllowPackages = true;
	TMap<FString, TSet<FString>> PackageActivities;
	FCriticalSection PackageActivitiesCS;

	/** MessageBus interceptor */
	TSharedPtr<FDisplayClusterMessageInterceptor, ESPMode::ThreadSafe> Interceptor;

	/** Settings to be used synchronized around the cluster */
	FMessageInterceptionSettings SynchronizedSettings;

	/** Cluster event listener delegate */
	FOnClusterEventJsonListener ListenerDelegate;

	/** Handles to console commands that we manage. */
	TUniquePtr<FAutoConsoleCommand> StartMessageSyncCommand;
	TUniquePtr<FAutoConsoleCommand> StopMessageSyncCommand;
	TUniquePtr<FAutoConsoleCommand> ResetMultiUserSyncCommand;

	/** Request to start interception. Cached to be done once synchronization is done. */
	bool bStartInterceptionRequested;

	/** Flag to check if synchronization was done */
	bool bSettingsSynchronizationDone = false;
};

#undef LOCTEXT_NAMESPACE


IMPLEMENT_MODULE(FDisplayClusterMessageInterceptionModule, DisplayClusterMessageInterception);

