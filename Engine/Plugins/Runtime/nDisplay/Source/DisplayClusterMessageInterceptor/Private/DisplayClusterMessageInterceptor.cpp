// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterMessageInterceptor.h"

#include "Cluster/DisplayClusterClusterEvent.h"
#include "Misc/ScopeLock.h"
#include "IMessageContext.h"
#include "IMessageBus.h"
#include "Cluster/IDisplayClusterClusterManager.h"


DEFINE_LOG_CATEGORY(LogDisplayClusterInterception);


namespace DisplayClusterMessageInterceptorUtils
{
	static const FInterceptedMessageDescriptor MultiUserInterception(TArray<FTopLevelAssetPath>({ FTopLevelAssetPath(TEXT("/Script/Concert"), TEXT("ConcertSession_CustomEvent"))}), TEXT("ConcertMessageId"));
}

FDisplayClusterMessageInterceptor::FDisplayClusterMessageInterceptor()
	: bIsIntercepting(false)
	, InterceptorId(FGuid::NewGuid())
	, Address(FMessageAddress::NewAddress())
	, ClusterManager(nullptr)
{}

void FDisplayClusterMessageInterceptor::Setup(IDisplayClusterClusterManager* InClusterManager, const FMessageInterceptionSettings& InInterceptionSettings)
{
	ClusterManager = InClusterManager;
	InterceptionSettings = InInterceptionSettings;

	//Setup desired messages to be intercepted
	if (InterceptionSettings.bInterceptMultiUserMessages)
	{
		InterceptedMessages.Emplace(DisplayClusterMessageInterceptorUtils::MultiUserInterception);
	}
}

void FDisplayClusterMessageInterceptor::Purge()
{
	TArray<TSharedPtr<IMessageContext, ESPMode::ThreadSafe>> ContextToForward;
	{
		FScopeLock Lock(&ContextQueueCS);
		for (const auto& ContextPair : ContextMap)
		{
			ContextToForward.Add(ContextPair.Value.ContextPtr);
		}
		ContextMap.Empty();
	}
	if (InterceptedBus)
	{
		for (const auto& Context : ContextToForward)
		{
			InterceptedBus->Forward(Context.ToSharedRef(), Context->GetRecipients(), FTimespan::Zero(), AsShared());
		}
	}
}

void FDisplayClusterMessageInterceptor::Start(TSharedPtr<IMessageBus, ESPMode::ThreadSafe> InBus)
{
	InterceptedBus = InBus;

	if (!bIsIntercepting && InterceptionSettings.bIsEnabled && InterceptedBus)
	{
		UE_LOGF(LogDisplayClusterInterception, Display, "Starting interception of bus messages");

		for (const FInterceptedMessageDescriptor& Descriptor : InterceptedMessages)
		{
			const FString AnnotationString = Descriptor.Annotation.ToString();
			for (const FTopLevelAssetPath& MessageType : Descriptor.MessageTypes)
			{
				InterceptedBus->Intercept(AsShared(), MessageType);
				UE_LOGF(LogDisplayClusterInterception, Display, "Intercepting message type: '%ls' with annotation '%ls'", *MessageType.ToString(), *AnnotationString);
			}

		}

		bIsIntercepting = true;
	}
}

void FDisplayClusterMessageInterceptor::Stop()
{
	if (bIsIntercepting && InterceptedBus)
	{
		InterceptedBus->Unintercept(AsShared(), IMessageBus::PATHNAME_All);
		bIsIntercepting = false;
		Purge();
		UE_LOGF(LogDisplayClusterInterception, Display, "Stopping interception of bus messages.");
	}

	InterceptedBus.Reset();
}

void FDisplayClusterMessageInterceptor::SyncMessages()
{
	if (ClusterManager)
	{
		TArray<FString> MessageIds;
		{
			FScopeLock Lock(&ContextQueueCS);
			ContextMap.GenerateKeyArray(MessageIds);
		}
		FDisplayClusterClusterEventJson SyncMessagesEvent;
		SyncMessagesEvent.Category = TEXT("nDCI");				// message bus sync message
		SyncMessagesEvent.Name = ClusterManager->GetNodeId();	// which node got the message
		SyncMessagesEvent.bIsSystemEvent = true;				// nDisplay internal event
		SyncMessagesEvent.bShouldDiscardOnRepeat = false;		// Don' discard the events with the same cat/type/name
		for (const FString& MessageId : MessageIds)
		{
			SyncMessagesEvent.Type = MessageId;					// the actually message id we received
			const bool bPrimaryOnly = false; //All nodes are broadcasting events to synchronize them across cluster
			ClusterManager->EmitClusterEventJson(SyncMessagesEvent, bPrimaryOnly);
			UE_LOGF(LogDisplayClusterInterception, VeryVerbose, "Emitting cluster event for message %ls on frame %lld", *MessageId, GFrameCounter);
		}
	}

	// remove out of date messages that are not marked reliable
	const FTimespan MessageTimeoutSpan = FTimespan::FromSeconds(InterceptionSettings.TimeoutSeconds);
	const FDateTime UtcNow = FDateTime::UtcNow();

	TArray<TSharedPtr<IMessageContext, ESPMode::ThreadSafe>> ContextToForward;
	{
		FScopeLock Lock(&ContextQueueCS);
		for (auto It = ContextMap.CreateIterator(); It; ++It)
		{
			if (It.Value().ContextPtr->GetTimeForwarded() + MessageTimeoutSpan <= UtcNow)
			{
				if (!EnumHasAnyFlags(It.Value().ContextPtr->GetFlags(), EMessageFlags::Reliable))
				{
					UE_LOGF(LogDisplayClusterInterception, VeryVerbose, "Discarding unreliable message '%ls' left intercepted for more than %0.1f seconds", *It.Key(), InterceptionSettings.TimeoutSeconds);
					It.RemoveCurrent();
				}
				else
				{
					UE_LOGF(LogDisplayClusterInterception, Warning, "Forcing dispatching of reliable message '%ls' left intercepted for more than %0.1f seconds", *It.Key(), InterceptionSettings.TimeoutSeconds);

					// Force treatment if not synced after a certain amount of time and the message is reliable
					ContextToForward.Add(It.Value().ContextPtr);
					It.RemoveCurrent();
				}
			}
		}
	}
	if (InterceptedBus)
	{
		for (const auto& Context : ContextToForward)
		{
			InterceptedBus->Forward(Context.ToSharedRef(), Context->GetRecipients(), FTimespan::Zero(), AsShared());
		}
	}
}

void FDisplayClusterMessageInterceptor::HandleClusterEvent(const FDisplayClusterClusterEventJson& InEvent)
{
	TArray<TSharedPtr<IMessageContext, ESPMode::ThreadSafe>> ContextToForward;
	if (InEvent.Category == TEXT("nDCI") && ClusterManager)
	{
		FScopeLock Lock(&ContextQueueCS);
		if (FContextSync* ContextSync = ContextMap.Find(InEvent.Type))
		{
			ContextSync->NodesReceived.Add(InEvent.Name);
			if (ContextSync->NodesReceived.Num() >= (int32)ClusterManager->GetNodesAmount())
			{
				UE_LOGF(LogDisplayClusterInterception, VeryVerbose, "Fowarding message for message id %ls on frame %lld", *InEvent.Type, GFrameCounter);
				ContextToForward.Add(ContextSync->ContextPtr);
				ContextMap.Remove(InEvent.Type);
			}
		}
	}
	if (InterceptedBus)
	{
		for (const auto& Context : ContextToForward)
		{
			InterceptedBus->Forward(Context.ToSharedRef(), Context->GetRecipients(), FTimespan::Zero(), AsShared());
		}
	}
}

void FDisplayClusterMessageInterceptor::HandleClusterNodeFailure(const FString& NodeId)
{
	TArray<TSharedPtr<IMessageContext, ESPMode::ThreadSafe>> ContextToForward;

	if (!ClusterManager)
	{
		return;
	}

	{
		FScopeLock Lock(&ContextQueueCS);

		const uint32 NodesInCluster = ClusterManager->GetNodesAmount();

		// Will contain the message IDs that we can forward to process and forget about
		TSet<FString> ContextsToClear;

		// Remove this node ID from all the message contexts
		for (TPair<FString, FContextSync>& MessageContext : ContextMap)
		{
			MessageContext.Value.NodesReceived.Remove(NodeId);

			// If this node ID was the only one missing, then forward this message to process
			const uint32 NodesReceivedNum = static_cast<uint32>(MessageContext.Value.NodesReceived.Num());
			if (NodesReceivedNum >= NodesInCluster)
			{
				UE_LOGF(LogDisplayClusterInterception, VeryVerbose, "Fowarding message for message id %ls on frame %lld", *MessageContext.Key, GFrameCounter);
				ContextToForward.Add(MessageContext.Value.ContextPtr);
				ContextsToClear.Add(MessageContext.Key);
			}
		}

		// Now remove the contexts we don't need anymore
		for (const FString& MessageId : ContextsToClear)
		{
			ContextMap.Remove(MessageId);
		}
	}

	// Forward all selected messages to process
	if (InterceptedBus)
	{
		for (const auto& Context : ContextToForward)
		{
			InterceptedBus->Forward(Context.ToSharedRef(), Context->GetRecipients(), FTimespan::Zero(), AsShared());
		}
	}
}

FName FDisplayClusterMessageInterceptor::GetDebugName() const
{
	return FName("DisplayClusterInterceptor");
}

const FGuid& FDisplayClusterMessageInterceptor::GetInterceptorId() const
{
	return InterceptorId;
}

bool FDisplayClusterMessageInterceptor::InterceptMessage(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	// we do not intercept forwarded message, they will be either coming off of the network or being forwarded by ourselves
	if (!bIsIntercepting || !Context->IsForwarded())
	{
		return false;
	}

	if (Context->GetForwarder() != Address)
	{
		const FTopLevelAssetPath MessageType = Context->GetMessageTypePathName();
		const FInterceptedMessageDescriptor* Descriptor = InterceptedMessages.FindByPredicate([MessageType](const FInterceptedMessageDescriptor& Other) { return Other.MessageTypes.Contains(MessageType); });
		if (Descriptor)
		{
			const FString* MessageId = Context->GetAnnotations().Find(Descriptor->Annotation);
			if (MessageId)
			{
				UE_LOGF(LogDisplayClusterInterception, VeryVerbose, "Intercepted message '%ls'", **MessageId);

				FScopeLock Lock(&ContextQueueCS);
				ContextMap.Add(*MessageId, FContextSync(Context));
				return true;
			}
		}
	}

	return false;
}

FMessageAddress FDisplayClusterMessageInterceptor::GetSenderAddress()
{
	return Address;
}

void FDisplayClusterMessageInterceptor::NotifyMessageError(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context, const FString& Error)
{
	// deprecated
}
