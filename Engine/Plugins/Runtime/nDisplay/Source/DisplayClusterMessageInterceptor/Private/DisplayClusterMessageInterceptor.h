// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IMessageInterceptor.h"
#include "IMessageSender.h"
#include "DisplayClusterMessageInterceptionSettings.h"

class IMessageBus;
class IDisplayClusterClusterManager;
struct FDisplayClusterClusterEventJson;


DECLARE_LOG_CATEGORY_EXTERN(LogDisplayClusterInterception, Log, All);


struct FInterceptedMessageDescriptor
{
	FInterceptedMessageDescriptor(TArray<FTopLevelAssetPath>&& InMessageTypes, FName InAnnotation)
		: MessageTypes(MoveTemp(InMessageTypes))
		, Annotation(InAnnotation)
	{}

	TArray<FTopLevelAssetPath> MessageTypes;
	FName Annotation;
};

/**
 * Display Cluster Message Interceptor 
 * Intercept message bus messages to process them in sync across the display cluster
 */
class FDisplayClusterMessageInterceptor : public TSharedFromThis<FDisplayClusterMessageInterceptor, ESPMode::ThreadSafe>
	, public IMessageInterceptor
	, public IMessageSender
{
public:
	FDisplayClusterMessageInterceptor();

	void Setup(IDisplayClusterClusterManager* InClusterManager, const FMessageInterceptionSettings& InInterceptionSettings);
	
	void Start(TSharedPtr<IMessageBus, ESPMode::ThreadSafe> InBus);
	void Stop();
	
	void SyncMessages();
	void HandleClusterEvent(const FDisplayClusterClusterEventJson& InEvent);
	void HandleClusterNodeFailure(const FString& NodeId);

	// IMessageInterceptor interface
	virtual FName GetDebugName() const override;
	virtual const FGuid& GetInterceptorId() const override;
	virtual bool InterceptMessage(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context) override;

	// IMessageSender interface
	virtual FMessageAddress GetSenderAddress() override;
	virtual void NotifyMessageError(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context, const FString& Error) override;
private:
	/** Purge the message map and force their forwarding. */
	void Purge();

	/** Currently intercepting message bus. */
	bool bIsIntercepting;

	/** InterceptorId needed by message bus. */
	FGuid InterceptorId;

	/** Sender address needed by message bus. */
	FMessageAddress Address;

	/** The bus this interceptor is intercepting messages on. */
	TSharedPtr<IMessageBus, ESPMode::ThreadSafe> InterceptedBus;

	/** The cluster manager to sync messages across. */
	IDisplayClusterClusterManager* ClusterManager;

	/** Critical section for the intercepted messages. */
	FCriticalSection ContextQueueCS;

	/** Settings to be used for interception */
	FMessageInterceptionSettings InterceptionSettings;

	/** Description of messages we intercept */
	TArray<FInterceptedMessageDescriptor> InterceptedMessages;

	/** Holds intercepted messages and their current sync state across the cluster. */
	struct FContextSync
	{
		FContextSync(TSharedPtr<IMessageContext, ESPMode::ThreadSafe> InContext)
			: ContextPtr(InContext)
		{}

		TSet<FString> NodesReceived;
		TSharedPtr<IMessageContext, ESPMode::ThreadSafe> ContextPtr;
	};

	/** The intercepted message map (MessageId-to-MessageContext). */
	TMap<FString, FContextSync> ContextMap;
};

