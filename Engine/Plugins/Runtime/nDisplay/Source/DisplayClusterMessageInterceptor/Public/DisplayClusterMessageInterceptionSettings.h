// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "DisplayClusterMessageInterceptionSettings.generated.h"


USTRUCT()
struct FMessageInterceptionSettings
{
	GENERATED_BODY()

	/** 
	 * By default, interception is enabled and multi user interception is active
	 * Old config properties are not redirectable so by default, new system will be enabled
	 * fixing any discrepencies across the system with settings synchronization.
	 */

	/** Indicates if message interception is enabled. */
	UPROPERTY(config, EditAnywhere, Category = "Interception Settings")
	bool bIsEnabled = true;

	/** Indicates whether messages from multi user are intercepted. */
	UPROPERTY(config, EditAnywhere, Category = "Interception Settings", meta=(EditCondition = "bIsEnabled"))
	bool bInterceptMultiUserMessages = true;
	
	/** Maximum seconds to keep intercepted message to be synchronized across cluster. */
	UPROPERTY(config, EditAnywhere, AdvancedDisplay, Category = "Interception Settings")
	float TimeoutSeconds = 1.0f;
};


UCLASS(config=Engine)
class DISPLAYCLUSTERMESSAGEINTERCEPTION_API UDisplayClusterMessageInterceptionSettings : public UObject
{
	GENERATED_BODY()

public:

	/**
	 * Settings related to interception. 
	 * @note Settings from primary node will be synchronized across the cluster
	 */
	UPROPERTY(config, EditAnywhere, Category = "Interception Settings", meta = (ShowOnlyInnerProperties))
	FMessageInterceptionSettings InterceptionSettings;
};