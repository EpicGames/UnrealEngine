// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DisplayClusterMonitor : ModuleRules
{
	public DisplayClusterMonitor(ReadOnlyTargetRules ROTargetRules) : base(ROTargetRules)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"DisplayCluster",
				"DisplayClusterConfiguration",
				"DisplayClusterShaders",
				"Engine",
				"MediaIOCore",
				"NDIMedia",
				"RenderCore",
				"RHI",
			});

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Messaging",
				"MessagingCommon",
			});

		if (Target.bBuildEditor == true)
		{
			PrivateDependencyModuleNames.Add("UnrealEd");
		}
	}
}
