// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"

// Module-wide log categories
DECLARE_LOG_CATEGORY_EXTERN(LogDisplayClusterMonitor,                Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogDisplayClusterMonitorMsg,             Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogDisplayClusterMonitorProviderMedia,   Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogDisplayClusterMonitorObservableMedia, Log, All);
