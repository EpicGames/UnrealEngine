// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterMonitorMessenger.h"

#include "Algo/Transform.h"
#include "Cluster/IDisplayClusterClusterManager.h"
#include "Config/IDisplayClusterConfigManager.h"

#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterEnums.h"
#include "DisplayClusterMonitorLog.h"
#include "DisplayClusterMonitorSettings.h"
#include "IDisplayCluster.h"
#include "IMessageContext.h"
#include "MessageEndpointBuilder.h"


namespace UE::nDisplay::Monitor
{
	// Residence is bound to a cluster node, therefore to the process where
	// it runs. Being static, it can be re-used by multiple messenger instances.
	TOptional<FDCMData_ResidenceDescriptor> FDCMessenger::ThisResidence;


	FDCMessenger::~FDCMessenger()
	{
		if (TickerHandle.IsValid())
		{
			FTSTicker::RemoveTicker(TickerHandle);
		}
	}

	bool FDCMessenger::Start(const FString& InEndpointName, const TSet<EDCMessengerRole>& InEndpointRoles)
	{
		// Nothing to do if running already
		if (bIsRunning)
		{
			return true;
		}

		// At least one role needs to be specified
		if (InEndpointRoles.IsEmpty())
		{
			UE_LOGF(LogDisplayClusterMonitorMsg, Error, "Messenger '%ls' could not start. No roles specified.", *InEndpointName);
			return false;
		}

		// Build the message endpoint
		MessageEndpoint = FMessageEndpoint::Builder(*InEndpointName)
			.ReceivingOnThread(ENamedThreads::Type::GameThread)
			.Handling<FDCMMessage_DiscoveryRequest>(this, &FDCMessenger::OnDiscoveryRequest)
			.Handling<FDCMMessage_DiscoveryResponse>(this, &FDCMessenger::OnDiscoveryResponse)
			.Handling<FDCMMessage_Heartbeat>(this, &FDCMessenger::OnHeartbeat)
			.Handling<FDCMMessage_Shutdown>(this, &FDCMessenger::OnShutdown)
			.Handling<FDCMMessage_ExecuteConsoleCommand>(this, &FDCMessenger::OnExecuteConsoleCommand)
			.WithCatchall(this, &FDCMessenger::OnUserMessages);

		if (!MessageEndpoint.IsValid())
		{
			UE_LOGF(LogDisplayClusterMonitorMsg, Error, "Could not build '%ls' message endpoint.", *InEndpointName);
			return false;
		}

		bIsRunning = true;

		// Prepare endpoint information for this entity
		ThisEndpoint = FDCEndpoint
		{
			.Residence = GetResidenceDescriptor(),
			.Endpoint  = FDCMData_EndpointDescriptor
				{
					.Id    = FGuid::NewGuid(),
					.Name  = InEndpointName,
					.Roles = InEndpointRoles,
				},
			.Address = MessageEndpoint->GetAddress(),
		};

		// Subscribe to broadcast message types
		MessageEndpoint->Subscribe<FDCMMessage_DiscoveryRequest>();
		MessageEndpoint->Subscribe<FDCMMessage_Heartbeat>();
		MessageEndpoint->Subscribe<FDCMMessage_Shutdown>();

		UE_LOGF(LogDisplayClusterMonitorMsg, Log, "Messenger '%ls' has started at address %ls",
			*InEndpointName, *MessageEndpoint->GetAddress().ToString());

		// Reset last pulse time
		LastHeartbeatPulseTime = FPlatformTime::Seconds();

		// Start periodic Ticks to help managing some internal states (e.g. timeouts)
		TickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FDCMessenger::Tick), 0.0f);

		// Notify everyone we're up
		BroadcastJoin();

		return true;
	}

	void FDCMessenger::Stop(const FString& InReason)
	{
		// Nothing to do if not running
		if (!bIsRunning)
		{
			return;
		}

		// No more Tick() callbacks
		if (TickerHandle.IsValid())
		{
			FTSTicker::RemoveTicker(TickerHandle);
		}

		// Let everyone now we're leaving
		BroadcastLeave(InReason);

		// Cancel all our subscriptions
		if (MessageEndpoint.IsValid())
		{
			MessageEndpoint->Disable();
			MessageEndpoint->Unsubscribe<FDCMMessage_DiscoveryRequest>();
			MessageEndpoint->Unsubscribe<FDCMMessage_Heartbeat>();
			MessageEndpoint->Unsubscribe<FDCMMessage_Shutdown>();
			MessageEndpoint.Reset();
		}

		UE_LOGF(LogDisplayClusterMonitorMsg, Log, "Messenger '%ls' has stopped", *ThisEndpoint.Endpoint.Name);

		// Clear running flag
		bIsRunning = false;

		// And release internals
		UserMessageEvents.Empty();
		DiscoveredEndpoints.Empty();
		ThisEndpoint = { };
	}

	FMessageAddress FDCMessenger::GetAddress() const
	{
		if (MessageEndpoint && MessageEndpoint->IsConnected())
		{
			return MessageEndpoint->GetAddress();
		}

		return FMessageAddress();
	}

	FMessageAddress FDCMessenger::GetAddress(const FGuid& InClusterId, const FGuid& InNodeId) const
	{
		const FDCEndpoint* FoundEP = DiscoveredEndpoints.FindByPredicate([&InClusterId, &InNodeId](const FDCEndpoint& EP)
			{
				return EP.Residence.ClusterId == InClusterId && EP.Residence.NodeId == InNodeId;
			});

		return FoundEP ? FoundEP->Address : FMessageAddress();
	}

	const FDCEndpoint& FDCMessenger::GetEndpointInfo() const
	{
		return ThisEndpoint;
	}

	TConstArrayView<FDCEndpoint> FDCMessenger::GetDiscoveredEndpoints() const
	{
		return DiscoveredEndpoints;
	}

	void FDCMessenger::SendImpl(const TArray<FMessageAddress>& InRecipients, void* InPayload, UScriptStruct* InType)
	{
		// Validate input
		if (!MessageEndpoint || !InPayload || !IsValid(InType) || InRecipients.IsEmpty())
		{
			return;
		}

		// Conditional detailed logs
		if(UE_GET_LOG_VERBOSITY(LogDisplayClusterMonitorMsg) >= ELogVerbosity::VeryVerbose)
		{
			for (const FMessageAddress& Recipient : InRecipients)
			{
				UE_LOGF(LogDisplayClusterMonitorMsg, VeryVerbose, "MSG: sending to %ls: '%ls'", *InType->GetName(), *Recipient.ToString());
			}
		}

		// Send message
		constexpr EMessageFlags Flags = EMessageFlags::Reliable;
		MessageEndpoint->Send(InPayload, InType, Flags, nullptr, InRecipients, FTimespan::Zero(), FDateTime::MaxValue());
	}

	void FDCMessenger::SendToRoles(const TSet<EDCMessengerRole>& InEndpointTypes, void* InPayload, UScriptStruct* InType)
	{
		// Validate input
		if (!MessageEndpoint || !InPayload || !IsValid(InType) || InEndpointTypes.IsEmpty())
		{
			return;
		}

		// Here we'll put all the recipients based on the roles set
		TArray<FMessageAddress> Recipients;
		Recipients.Reserve(DiscoveredEndpoints.Num());

		// Filter recipient addresses
		Algo::TransformIf(DiscoveredEndpoints, Recipients,
			[&InEndpointTypes](const FDCEndpoint& EP) { return EP.Endpoint.Roles.Intersect(InEndpointTypes).Num() > 0; },
			[&InEndpointTypes](const FDCEndpoint& EP) { return EP.Address; });

		// Conditional detailed logs
		if (UE_GET_LOG_VERBOSITY(LogDisplayClusterMonitorMsg) >= ELogVerbosity::VeryVerbose)
		{
			for (const FMessageAddress& Recipient : Recipients)
			{
				UE_LOGF(LogDisplayClusterMonitorMsg, VeryVerbose, "MSG: sending to %ls: '%ls'", *InType->GetName(), *Recipient.ToString());
			}
		}

		// Send message
		constexpr EMessageFlags Flags = EMessageFlags::Reliable;
		MessageEndpoint->Send(InPayload, InType, Flags, nullptr, Recipients, FTimespan::Zero(), FDateTime::MaxValue());
	}

	void FDCMessenger::BroadcastImpl(void* InPayload, UScriptStruct* InType)
	{
		if (!MessageEndpoint || !InPayload || !IsValid(InType))
		{
			return;
		}

		MessageEndpoint->Publish(InPayload, InType, EMessageScope::All, FTimespan::Zero(), FDateTime::MaxValue());
	}

	bool FDCMessenger::Tick(float InDeltaTime)
	{
		TickTask_PulseHearbeat(InDeltaTime);
		TickTask_CheckInactiveEndpoints(InDeltaTime);

		return true;
	}

	void FDCMessenger::TickTask_PulseHearbeat(float InDeltaTime)
	{
		const double CurrentTime = FPlatformTime::Seconds();
		const double TimePassed = CurrentTime - LastHeartbeatPulseTime;

		// Get heartbeat interval from project settings
		const double HeartbeatInterval = GetDefault<UDisplayClusterMonitorSettings>()->HeartbeatInterval;

		// Send heartbeat according to time interval
		if (TimePassed > HeartbeatInterval)
		{
			LastHeartbeatPulseTime = CurrentTime;
			BroadcastHeartbeat();
		}
	}

	void FDCMessenger::TickTask_CheckInactiveEndpoints(float InDeltaTime)
	{
		// Get unresponsive time threshold from project settings
		const double UnresponsiveTimeThreshold = GetDefault<UDisplayClusterMonitorSettings>()->UnresponsiveTimeThreshold;
		const double TimeNow = GetLastActivityTime();

		// Check every known endpoint
		for (auto It = DiscoveredEndpoints.CreateIterator(); It; ++It)
		{
			const double TimeSinceLastActivity =  TimeNow - It->LastActivityTime;
			if (TimeSinceLastActivity > UnresponsiveTimeThreshold)
			{
				// Report timeout
				OnEndpointTimeout.Broadcast(*It);
				// And forget it
				It.RemoveCurrent();
			}
		}
	}

	FGuid FDCMessenger::MakeDeterministicClusterGuid(const FString& InPrimaryNodeId, const FString& InPrimaryNodeAddr, uint16 InPrimaryExclusiveCommPort)
	{
		// Generate unique string based cluster indentificator
		const FString CombinedStrId = FString::Printf(TEXT("%ls@%ls:%u"),
			*InPrimaryNodeId.ToLower().TrimStartAndEnd(),
			*InPrimaryNodeAddr.ToLower().TrimStartAndEnd(),
			InPrimaryExclusiveCommPort);

		const FGuid Guid = FGuid::NewDeterministicGuid(CombinedStrId);

		return Guid;
	}

	FDCMData_ResidenceDescriptor FDCMessenger::GetResidenceDescriptor() const
	{
		// Create new residence descriptor once
		if (!FDCMessenger::ThisResidence.IsSet())
		{
			const EDisplayClusterOperationMode OpMode = IDisplayCluster::Get().GetOperationMode();
			const IDisplayClusterClusterManager* const ClusterMgr = IDisplayCluster::Get().GetClusterMgr();
			const IDisplayClusterConfigManager* const ConfigMgr = IDisplayCluster::Get().GetConfigMgr();
			if (OpMode != EDisplayClusterOperationMode::Cluster || !ClusterMgr || !ConfigMgr)
			{
				return { };
			}

			const UDisplayClusterConfigurationData* const Config = ConfigMgr->GetConfig();
			if (!IsValid(Config))
			{
				return { };
			}

			const UDisplayClusterConfigurationClusterNode* const PrimaryNode = Config->GetPrimaryNode();
			if (!IsValid(PrimaryNode))
			{
				return { };
			}

			// Create new descriptor
			FDCMessenger::ThisResidence = FDCMData_ResidenceDescriptor
			{
				.ClusterId    = MakeDeterministicClusterGuid(Config->Cluster->PrimaryNode.Id, PrimaryNode->Host, Config->Cluster->PrimaryNode.Ports.ClusterSync),
				.NodeId       = FGuid::NewGuid(),
				.NodeName     = ClusterMgr->GetNodeId(),
				.Hostname     = FPlatformProcess::ComputerName(),
				.bIsPrimary   = ClusterMgr->IsPrimary(),
				.bIsOffscreen = FParse::Param(FCommandLine::Get(), TEXT("RenderOffscreen")),
			};
		}

		return FDCMessenger::ThisResidence.GetValue();
	}

	void FDCMessenger::BroadcastJoin()
	{
		UE_LOGF(LogDisplayClusterMonitorMsg, Verbose, "Broadcasting discovery request...");

		Broadcast(
			FDCMMessage_DiscoveryRequest
			{
				.Residence = ThisEndpoint.Residence,
				.Endpoint  = ThisEndpoint.Endpoint,
			});
	}

	void FDCMessenger::BroadcastHeartbeat()
	{
		++HeartbeatPulseCounter;
		UE_LOGF(LogDisplayClusterMonitorMsg, Verbose, "Broadcasting heartbeat pulse %u...", HeartbeatPulseCounter);

		Broadcast(
			FDCMMessage_Heartbeat
			{
			});
	}

	void FDCMessenger::BroadcastLeave(const FString& InShutdownReason)
	{
		UE_LOGF(LogDisplayClusterMonitorMsg, Verbose, "Broadcasting leave notification...");

		Broadcast(
			FDCMMessage_Shutdown
			{
				.Reason = InShutdownReason
			});
	}

	void FDCMessenger::OnDiscoveryRequest(const FDCMMessage_DiscoveryRequest& InMessage, const TSharedRef<IMessageContext>& InContext)
	{
		// Ignore self
		if (InContext->GetSender() == MessageEndpoint->GetAddress())
		{
			return;
		}

		// Make sure it has not been registered yet
		{
			// Seach for the same MessageBus address
			const bool bAlreadyExists = DiscoveredEndpoints.ContainsByPredicate([Addr = InContext->GetSender()](const FDCEndpoint& Item)
				{
					return Item.Address == Addr;
				});

			// Nothing to do if registered already
			if (bAlreadyExists)
			{
				return;
			}
		}

		UE_LOGF(LogDisplayClusterMonitorMsg, Verbose, "MSG: received DiscoveryRequest from %ls @ %ls",
			*InMessage.Endpoint.Name, *InContext->GetSender().ToString());

		// Store information about new ednpoint
		const FDCEndpoint& EndpointJoinedRef = DiscoveredEndpoints.Add_GetRef(
			{
				.Residence = InMessage.Residence,
				.Endpoint  = InMessage.Endpoint,
				.Address   = InContext->GetSender(),
				.LastActivityTime = GetLastActivityTime(),
			});

		// Also send back information about this endpoint so the original sender will also be informed
		Send({ InContext->GetSender() },
			FDCMMessage_DiscoveryResponse
			{
				.Residence = ThisEndpoint.Residence,
				.Endpoint  = ThisEndpoint.Endpoint,
			});

		// Let users know there is a new endpoint
		OnEndpointJoined.Broadcast(EndpointJoinedRef);
	}

	void FDCMessenger::OnDiscoveryResponse(const FDCMMessage_DiscoveryResponse& InMessage, const TSharedRef<IMessageContext>& InContext)
	{
		// Ignore self
		if (InContext->GetSender() == MessageEndpoint->GetAddress())
		{
			return;
		}

		// Make sure it has not been registered yet
		{
			// Seach for the same MessageBus address
			const bool bAlreadyExists = DiscoveredEndpoints.ContainsByPredicate([Addr = InContext->GetSender()](const FDCEndpoint& Item)
				{
					return Item.Address == Addr;
				});

			// Nothing to do if registered already
			if (bAlreadyExists)
			{
				return;
			}
		}

		UE_LOGF(LogDisplayClusterMonitorMsg, Verbose, "MSG: received DiscoveryResponse from %ls @ %ls",
			*InMessage.Endpoint.Name, *InContext->GetSender().ToString());

		UE_LOGF(LogDisplayClusterMonitorMsg, Log, "MSG: Discovered endpoint: name=%ls, addr=%ls", *InMessage.Endpoint.Name, *InContext->GetSender().ToString());

		// Store endpoint data
		const FDCEndpoint& EndpointJoinedRef = DiscoveredEndpoints.Add_GetRef(
			{
				.Residence = InMessage.Residence,
				.Endpoint  = InMessage.Endpoint,
				.Address   = InContext->GetSender(),
				.LastActivityTime = GetLastActivityTime(),
			});

		// Let users know there is a new endpoint
		OnEndpointJoined.Broadcast(EndpointJoinedRef);
	}

	void FDCMessenger::OnHeartbeat(const FDCMMessage_Heartbeat& InMessage, const TSharedRef<IMessageContext>& InContext)
	{
		// Ignore self
		if (InContext->GetSender() == MessageEndpoint->GetAddress())
		{
			return;
		}

		UE_LOGF(LogDisplayClusterMonitorMsg, VeryVerbose, "MSG: received Heartbeat from %ls", *InContext->GetSender().ToString());

		// Update last activity
		const FMessageAddress& SenderAddress = InContext->GetSender();
		UpdateLastActivityTime(SenderAddress);
	}

	void FDCMessenger::OnShutdown(const FDCMMessage_Shutdown& InMessage, const TSharedRef<IMessageContext>& InContext)
	{
		// Ignore self
		if (InContext->GetSender() == MessageEndpoint->GetAddress())
		{
			return;
		}

		UE_LOGF(LogDisplayClusterMonitorMsg, Verbose, "Got shutdown notification: addr=%ls", *InContext->GetSender().ToString());

		// Get sender's index
		const FMessageAddress& SenderAddress = InContext->GetSender();
		const int32 EndpointIdx = GetEndpointIdx(SenderAddress);
		if (EndpointIdx == INDEX_NONE)
		{
			return;
		}

		// Remove it before notifying the listeners
		FDCEndpoint EndpointLeft = MoveTemp(DiscoveredEndpoints[EndpointIdx]);
		DiscoveredEndpoints.RemoveAt(EndpointIdx);

		// Finally, notify the listeners
		OnEndpointLeft.Broadcast(EndpointLeft, InMessage.Reason);
	}

	void FDCMessenger::OnExecuteConsoleCommand(const FDCMMessage_ExecuteConsoleCommand& InMessage, const TSharedRef<IMessageContext>& InContext)
	{
		// Ignore completely if no handler specified
		if (!OnConsoleCommand.IsBound())
		{
			return;
		}

		// Accept requests from known endpoints only
		const FDCEndpoint* const RequestorEndpoint = GetEndpoint(InContext->GetSender());
		if (!RequestorEndpoint)
		{
			return;
		}

		// Forward request
		OnConsoleCommand.Execute(*RequestorEndpoint, InMessage.ExecutorName, InMessage.Command);
	}

	void FDCMessenger::OnUserMessages(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
	{
		// Ignore self
		if (InContext->GetSender() == MessageEndpoint->GetAddress())
		{
			return;
		}

		// Verify message validity
		if (!InContext->IsValid())
		{
			return;
		}

		// Update last activity
		const FMessageAddress& SenderAddress = InContext->GetSender();
		UpdateLastActivityTime(SenderAddress);

		// Get sender's endpoint
		const FDCEndpoint* const Endpoint = GetEndpoint(SenderAddress);
		if (!Endpoint)
		{
			return;
		}

		// Message type
		const UScriptStruct* const MessageTypeInfo = InContext->GetMessageTypeInfo().Get();
		if (!MessageTypeInfo)
		{
			return;
		}

		// Check if supported message type
		const bool bIsUserMessage = MessageTypeInfo->IsChildOf(FDCMMessage::StaticStruct());
		if (!bIsUserMessage)
		{
			return;
		}

		// The actual message
		const FDCMMessage* const Message = reinterpret_cast<const FDCMMessage*>(InContext->GetMessage());
		check(Message);

		UE_LOGF(LogDisplayClusterMonitorMsg, Verbose, "MSG: received user message '%ls' from '%ls'", *MessageTypeInfo->GetName() , *Endpoint->Endpoint.Name);

		// Let the subsribers process this message
		CallUserFunction(MessageTypeInfo, *Endpoint, *Message);
	}
}
