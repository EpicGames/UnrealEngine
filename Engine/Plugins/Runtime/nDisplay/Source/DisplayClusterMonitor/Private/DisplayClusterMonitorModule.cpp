// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterMonitorModule.h"
#include "DisplayClusterMonitorProviderMedia.h"
#include "IDisplayCluster.h"
#include "IDisplayClusterCallbacks.h"


void FDisplayClusterMonitorModule::StartupModule()
{
	if (IDisplayCluster::IsAvailable())
	{
		IDisplayCluster::Get().GetCallbacks().OnDisplayClusterStartSession().AddRaw(this, &FDisplayClusterMonitorModule::OnDisplayClusterSessionStart);
		IDisplayCluster::Get().GetCallbacks().OnDisplayClusterEndSession().AddRaw(this, &FDisplayClusterMonitorModule::OnDisplayClusterSessionEnd);
	}
}

void FDisplayClusterMonitorModule::ShutdownModule()
{
	if (IDisplayCluster::IsAvailable())
	{
		IDisplayCluster::Get().GetCallbacks().OnDisplayClusterStartSession().RemoveAll(this);
		IDisplayCluster::Get().GetCallbacks().OnDisplayClusterEndSession().RemoveAll(this);
	}

	// Providers should have been stopped already, but let's be safe on that
	StopProviders();
}

void FDisplayClusterMonitorModule::StartProviders()
{
	if (!ProviderMedia)
	{
		ProviderMedia = MakeUnique<FDCMProviderMedia>();
		ProviderMedia->Start();
	}
}

void FDisplayClusterMonitorModule::StopProviders()
{
	if (ProviderMedia)
	{
		ProviderMedia->Stop();
		ProviderMedia.Reset();
	}
}

void FDisplayClusterMonitorModule::OnDisplayClusterSessionStart()
{
	// Export observables in cluster mode only
	const bool bRunningClusterMode = (IDisplayCluster::Get().GetOperationMode() == EDisplayClusterOperationMode::Cluster);
	if (!bRunningClusterMode)
	{
		return;
	}

	// Instantiate and start local observable providers
	StartProviders();
}

void FDisplayClusterMonitorModule::OnDisplayClusterSessionEnd()
{
	// Stop all active providers
	StopProviders();
}

IMPLEMENT_MODULE(FDisplayClusterMonitorModule, DisplayClusterMonitor);
