// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

namespace UE::nDisplay::Monitor
{
	class FDisplayClusterMonitorProviderMedia;
}


/**
 * DisplayClusterMonitor module implementation
 */
class FDisplayClusterMonitorModule :
	public IModuleInterface
{
	using FDCMProviderMedia = UE::nDisplay::Monitor::FDisplayClusterMonitorProviderMedia;

public:

	FDisplayClusterMonitorModule() = default;
	virtual ~FDisplayClusterMonitorModule() override = default;

public:

	//~ Begin IModuleInterface Implementation
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface Implementation

private:

	/** Starts all observable providers that are managed by this module */
	void StartProviders();

	/** Stops all observable providers that are managed by this module */
	void StopProviders();

private:

	/** Handles nDisplay session start event to initialize per-session internals */
	void OnDisplayClusterSessionStart();

	/** Handles nDisplay session end event to release per-session internals */
	void OnDisplayClusterSessionEnd();

private:

	/** Media observables provider */
	TUniquePtr<FDCMProviderMedia> ProviderMedia;
};
