// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterMonitorProviderBase.h"


namespace UE::nDisplay::Monitor
{

	FDisplayClusterMonitorProviderBase::FDisplayClusterMonitorProviderBase()
		: Messenger(MakeUnique<FDCMessenger>())
	{
	}

	bool FDisplayClusterMonitorProviderBase::Start()
	{
		if (!Messenger.IsValid())
		{
			return false;
		}

		// Instantiate and start messenger for this provider
		const FString EndpointName = GetMessengerName();
		const TSet<EDCMessengerRole> EndpointRoles = { EDCMessengerRole::ObservablesProvider };
		Messenger->Start(EndpointName, EndpointRoles);

		const bool bIsMessengerRunning = Messenger->IsRunning();
		return bIsMessengerRunning;
	}

	void FDisplayClusterMonitorProviderBase::Stop()
	{
		if (Messenger)
		{
			Messenger->Stop();
			Messenger.Reset();
		}
	}
}
