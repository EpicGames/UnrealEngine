// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterMonitorProviderMedia.h"

#include "Engine/Engine.h"
#include "Engine/GameViewportClient.h"
#include "MediaObservables/MediaObservableBackbuffer.h"
#include "MediaObservables/MediaObservablePostRender.h"
#include "MediaObservables/MediaObservableUI.h"
#include "Misc/DisplayClusterConsoleExec.h"
#include "Misc/Guid.h"
#include "Render/GUILayer/IDisplayClusterGUILayerController.h"
#include "Render/IDisplayClusterRenderManager.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_Enums.h"
#include "Render/Viewport/IDisplayClusterViewport.h"
#include "Render/Viewport/IDisplayClusterViewportManager.h"
#include "Slate/SceneViewport.h"

#include "DisplayClusterEnums.h"
#include "DisplayClusterMonitorLog.h"
#include "DisplayClusterMonitorMessenger.h"
#include "DisplayClusterMonitorTypes.h"
#include "IDisplayCluster.h"
#include "IDisplayClusterCallbacks.h"


namespace UE::nDisplay::Monitor
{
	bool FDisplayClusterMonitorProviderMedia::Start()
	{
		UE_LOGF(LogDisplayClusterMonitorProviderMedia, Log, "Starting media observables provider...");

		FCoreDelegates::OnEndFrame.AddRaw(this, &FDisplayClusterMonitorProviderMedia::OnEndFrame);

		Messenger->OnEndpointLeft.AddRaw(this, &FDisplayClusterMonitorProviderMedia::OnEndpointLeft);
		Messenger->OnEndpointTimeout.AddRaw(this, &FDisplayClusterMonitorProviderMedia::OnEndpointTimeout);

		Messenger->OnConsoleCommand.BindRaw(this, &FDisplayClusterMonitorProviderMedia::OnConsoleCommand);

		Messenger->OnMessage<FDCMMessage_NodeObservablesRequest>().AddRaw(this, &FDisplayClusterMonitorProviderMedia::OnNodeObservablesRequest);
		Messenger->OnMessage<FDCMMessage_StartSessionRequest>().AddRaw(this, &FDisplayClusterMonitorProviderMedia::OnStartSessionRequest);
		Messenger->OnMessage<FDCMMessage_StopSessionRequest>().AddRaw(this, &FDisplayClusterMonitorProviderMedia::OnStopSessionRequest);
		Messenger->OnMessage<FDCMMessage_ObservableControlRequest>().AddRaw(this, &FDisplayClusterMonitorProviderMedia::OnObservableControlRequest);

		return Super::Start();
	}

	void FDisplayClusterMonitorProviderMedia::Stop()
	{
		UE_LOGF(LogDisplayClusterMonitorProviderMedia, Log, "Shutting down media observables provider...");

		FCoreDelegates::OnEndFrame.RemoveAll(this);

		Messenger->OnEndpointLeft.RemoveAll(this);
		Messenger->OnEndpointTimeout.RemoveAll(this);

		Messenger->OnConsoleCommand.Unbind();

		Messenger->OnMessage<FDCMMessage_NodeObservablesRequest>().RemoveAll(this);
		Messenger->OnMessage<FDCMMessage_StartSessionRequest>().RemoveAll(this);
		Messenger->OnMessage<FDCMMessage_StopSessionRequest>().RemoveAll(this);
		Messenger->OnMessage<FDCMMessage_ObservableControlRequest>().RemoveAll(this);

		// Terminate active sessions
		StopActiveSessions();

		Super::Stop();
	}

	TSharedPtr<IMediaObservable> FDisplayClusterMonitorProviderMedia::CreateObservable(EDCObservableType InType, const FSourceInfo& SourceInfo)
	{
		// Create observable instance depending on the requested type
		switch (InType)
		{
		case EDCObservableType::Backbuffer:
			return MakeShared<FMediaObservableBackbuffer>(SourceInfo.Id, SourceInfo.DisplayName, SourceInfo.ResourceName);

		case EDCObservableType::UI:
			return MakeShared<FMediaObservableUI>(SourceInfo.Id, SourceInfo.DisplayName, SourceInfo.ResourceName);

		case EDCObservableType::Viewport:
		case EDCObservableType::ICVFXCamera:
		case EDCObservableType::ICVFXCameraTile:
			return MakeShared<FMediaObservablePostRender>(SourceInfo.Id, SourceInfo.DisplayName, SourceInfo.ResourceName);

		default:
			UE_LOGF(LogDisplayClusterMonitorProviderMedia, Warning, "'%ls' type has not been implemented yet",
				*StaticEnum<EDCObservableType>()->GetNameStringByValue(static_cast<int64>(InType)));
		}

		return nullptr;
	}

	void FDisplayClusterMonitorProviderMedia::OnNodeObservablesRequest(const FDCEndpoint& InEndpoint, const FDCMMessage_NodeObservablesRequest& InMessage)
	{
		// Return the most recent observables evaluation
		Messenger->Send({ InEndpoint.Address },
			FDCMMessage_NodeObservablesResponse
			{
				.Observables = LastObservablesInfo,
			});
	}

	void FDisplayClusterMonitorProviderMedia::OnStartSessionRequest(const FDCEndpoint& InEndpoint, const FDCMMessage_StartSessionRequest& InMessage)
	{
		UE_LOGF(LogDisplayClusterMonitorProviderMedia, Log, "Got StartSession request for observable '%ls'", *InMessage.ObservableId.ToString());

		// Pre-configure the error response
		const FDCMMessage_StartSessionResponse ResponseError
		{
			.ObservableId = InMessage.ObservableId,
			.Result       = EDCRequestResult::Fail,
		};

		// Get session map for this endpoint
		TMap<FGuid, TSharedPtr<IMediaObservable>>& MonitorSessions = ActiveSessions.FindOrAdd(InEndpoint.Address);

		// Make sure the requested observable is not running already
		if (MonitorSessions.Contains(InMessage.ObservableId))
		{
			UE_LOGF(LogDisplayClusterMonitorProviderMedia, Warning, "Observable '%ls' is already running", *InMessage.ObservableId.ToString());
			Messenger->Send({ InEndpoint.Address }, ResponseError);
			return;
		}

		// Helper function that finds the requested observable, and creates new session for it
		auto CreateObservableImpl = [this](const FGuid& InObservableId) -> TSharedPtr<IMediaObservable>
			{
				// Search for the requested observable source, and instantiate the observable session
				for (const auto& [Type, SourceMap] : ActiveSources)
				{
					for (const auto& [SourceName, SourceInfo] : SourceMap)
					{
						if (SourceInfo.Id == InObservableId)
						{
							return CreateObservable(Type, SourceInfo);
						}
					}
				}

				return nullptr;
			};

		// Create observation session
		TSharedPtr<IMediaObservable> Observable = CreateObservableImpl(InMessage.ObservableId);
		if (!Observable.IsValid())
		{
			UE_LOGF(LogDisplayClusterMonitorProviderMedia, Error, "Observable '%ls' not found, or could not start a session", *InMessage.ObservableId.ToString());
			Messenger->Send({ InEndpoint.Address }, ResponseError);
			return;
		}

		// Register new observation session
		MonitorSessions.Emplace(InMessage.ObservableId, Observable);

		// Respond Ok
		Messenger->Send({ InEndpoint.Address },
			FDCMMessage_StartSessionResponse
			{
				.ObservableId = InMessage.ObservableId,
				.Result       = EDCRequestResult::Ok,
			});
	}

	void FDisplayClusterMonitorProviderMedia::OnStopSessionRequest(const FDCEndpoint& InEndpoint, const FDCMMessage_StopSessionRequest& InMessage)
	{
		UE_LOGF(LogDisplayClusterMonitorProviderMedia, Log, "Got StopSession request for observable '%ls'", *InMessage.ObservableId.ToString());

		// Pre-configure the error response
		const FDCMMessage_StopSessionResponse ResponseError
		{
			.ObservableId = InMessage.ObservableId,
			.Result       = EDCRequestResult::Fail,
		};

		// Get session map for this endpoint
		TMap<FGuid, TSharedPtr<IMediaObservable>>* MonitorSessions = ActiveSessions.Find(InEndpoint.Address);
		if (!MonitorSessions)
		{
			Messenger->Send({ InEndpoint.Address }, ResponseError);
			return;
		}

		// See if requested session exists. If no, send the error response.
		TSharedPtr<IMediaObservable>* FoundSession = MonitorSessions->Find(InMessage.ObservableId);
		if (!FoundSession)
		{
			UE_LOGF(LogDisplayClusterMonitorProviderMedia, Warning, "Session '%ls' not found", *InMessage.ObservableId.ToString());
			Messenger->Send({ InEndpoint.Address }, ResponseError);
			return;
		}

		// Make sure the session instance is valid
		if (!FoundSession->IsValid())
		{
			UE_LOGF(LogDisplayClusterMonitorProviderMedia, Warning, "Session '%ls' has invalid state", *InMessage.ObservableId.ToString());
			Messenger->Send({ InEndpoint.Address }, ResponseError);
			return;
		}

		// Stop this session
		(*FoundSession)->StopCapture();

		// Remove from the active sessions list
		MonitorSessions->Remove(InMessage.ObservableId);
		if (MonitorSessions->IsEmpty())
		{
			// Also forget the requestor if no more active sessions left
			ActiveSessions.Remove(InEndpoint.Address);
		}

		// Respond Ok
		Messenger->Send({ InEndpoint.Address },
			FDCMMessage_StopSessionResponse
			{
				.ObservableId = InMessage.ObservableId,
				.Result       = EDCRequestResult::Ok,
			});
	}

	void FDisplayClusterMonitorProviderMedia::OnObservableControlRequest(const FDCEndpoint& InEndpoint, const FDCMMessage_ObservableControlRequest& InMessage)
	{
		UE_LOGF(LogDisplayClusterMonitorProviderMedia, Log, "Got CTRL request for observable '%ls'", *InMessage.ObservableId.ToString());

		// Pre-configure the error response
		const FDCMMessage_ObservableControlResponse ResponseError
		{
			.ObservableId = InMessage.ObservableId,
			.Result       = EDCRequestResult::Fail,
		};

		// Get session map for this endpoint
		TMap<FGuid, TSharedPtr<IMediaObservable>>& MonitorSessions = ActiveSessions.FindOrAdd(InEndpoint.Address);

		// Find requested session
		TSharedPtr<IMediaObservable>* FoundSession = MonitorSessions.Find(InMessage.ObservableId);
		if (!FoundSession || !FoundSession->IsValid())
		{
			UE_LOGF(LogDisplayClusterMonitorProviderMedia, Error, "Observable '%ls' has no active session", *InMessage.ObservableId.ToString());
			Messenger->Send({ InEndpoint.Address }, ResponseError);
			return;
		}

		// Forward requested command to the actual session
		bool bResult = false;
		switch (InMessage.Command)
		{
		case EDCControlCommand::Play:
			bResult = (*FoundSession)->StartCapture();
			break;

		case EDCControlCommand::Pause:
			bResult = true;
			break;

		case EDCControlCommand::Stop:
			(*FoundSession)->StopCapture();
			bResult = true;
			break;
		}

		UE_LOGF(LogDisplayClusterMonitorProviderMedia, Log, "CTRL request '%ls' for observable '%ls'/'%ls' %ls",
			*StaticEnum<EDCControlCommand>()->GetNameStringByValue(static_cast<int64>(InMessage.Command)),
			*InMessage.ObservableId.ToString(),
			*(*FoundSession)->GetName(),
			bResult ? TEXT("was handled successfully") : TEXT("failed!")
		);

		// Respond with the command result
		Messenger->Send({ InEndpoint.Address },
			FDCMMessage_ObservableControlResponse
			{
				.ObservableId = InMessage.ObservableId,
				.Result       = (bResult ? EDCRequestResult::Ok : EDCRequestResult::Fail),
			});
	}

	void FDisplayClusterMonitorProviderMedia::OnEndpointLeft(const FDCEndpoint& InEndpoint, const FString& InReason)
	{
		UE_LOGF(LogDisplayClusterMonitorProviderMedia, Log, "Client '%ls@%ls' has left: %ls",
			*InEndpoint.Endpoint.Name, *InEndpoint.Address.ToString(), *InReason);

		DeleteClientData(InEndpoint.Address);
	}

	void FDisplayClusterMonitorProviderMedia::OnEndpointTimeout(const FDCEndpoint& InEndpoint)
	{
		UE_LOGF(LogDisplayClusterMonitorProviderMedia, Log, "Client '%ls@%ls' has timed out",
			*InEndpoint.Endpoint.Name, *InEndpoint.Address.ToString());

		DeleteClientData(InEndpoint.Address);
	}

	void FDisplayClusterMonitorProviderMedia::OnEndFrame()
	{
		// Log information about active sessions
		if (UE_GET_LOG_VERBOSITY(LogDisplayClusterMonitorProviderMedia) >= ELogVerbosity::VeryVerbose)
		{
			int32 SessionsNum = 0;
			for (const auto& [Monitor, SessionMap] : ActiveSessions)
			{
				const int32 ClientSessions = SessionMap.Num();
				SessionsNum += ClientSessions;

				UE_LOGF(LogDisplayClusterMonitorProviderMedia, VeryVerbose, "Client %ls sessions: %d", *Monitor.ToString(), ClientSessions);
			}

			UE_LOGF(LogDisplayClusterMonitorProviderMedia, VeryVerbose, "Active sessions: %d", SessionsNum);
		}

		// Get render manager. It provides access to the viewport manager.
		const IDisplayClusterRenderManager* const RenderMgr = IDisplayCluster::Get().GetRenderMgr();
		if (!RenderMgr)
		{
			return;
		}

		// Get viewport manager. It provides access to the data we need.
		const IDisplayClusterViewportManager* const ViewportMgr = RenderMgr->GetViewportManager();
		if (!ViewportMgr)
		{
			return;
		}

		// Evaluate every viewport that is will be rendered
		const TArrayView<TSharedPtr<IDisplayClusterViewport>> Viewports = ViewportMgr->GetCurrentRenderFrameViewports();
		for (const TSharedPtr<IDisplayClusterViewport>& Viewport : Viewports)
		{
			// Ignore invalid references if there are any
			if (!Viewport.IsValid())
			{
				continue;
			}

			// At least one context must be there
			if (Viewport->GetContexts().Num() < 1)
			{
				continue;
			}

			// Skip disabled ones
			if (Viewport->GetContexts()[0].bDisableRender)
			{
				continue;
			}

			// Type based evaluation
			const EDisplayClusterViewportFlags Flags = Viewport->GetViewportFlags();
			switch (Flags)
			{
			case EDisplayClusterViewportFlags::None:
				EvaluateViewport(Viewport);
				break;

			case EDisplayClusterViewportFlags::ICVFX_InnerFrustum:
				EvaluateICVFXCamera(Viewport);
				break;

			case EDisplayClusterViewportFlags::ICVFX_InnerFrustum | EDisplayClusterViewportFlags::TileViewport:
				EvaluateICVFXCameraTile(Viewport);
				break;

			default:
				break;
			}
		}

		// Separate evaluation for the GUI layer
		EvaluateUI();

		// Separate evaluation for the backbuffer
		EvaluateBackbuffer();

		// Finally, propagate the updates
		PropagateSourceInfo();
	}

	void FDisplayClusterMonitorProviderMedia::OnConsoleCommand(const FDCEndpoint& InSenderEndpoint, const FString& InExecutorName, const FString& InCommand)
	{
		FDisplayClusterConsoleExec::Exec(InExecutorName, InCommand);
	}

	void FDisplayClusterMonitorProviderMedia::EvaluateViewport(const TSharedPtr<IDisplayClusterViewport>& InViewport)
	{
		const FString   SourceName  = InViewport->GetId();
		const FIntPoint CurrentSize = InViewport->GetContexts()[0].ContextSize;

		// Ignore invalid resource size
		if (CurrentSize.GetMin() < 1)
		{
			return;
		}

		// See if this viewport has been registered already
		TMap<FString, FSourceInfo>& Sources = ActiveSources.FindOrAdd(EDCObservableType::Viewport);
		FSourceInfo* FoundItem = Sources.Find(SourceName);

		// If not yet exists, create a new one
		if (!FoundItem)
		{
			Sources.Emplace(SourceName,
				FSourceInfo
				{
					.Id           = FGuid::NewGuid(),
					.DisplayName  = InViewport->GetBaseId(),
					.ResourceName = SourceName,
					.Size         = CurrentSize,
					.Status       = ESourceStatus::New,
				});

			return;
		}

		// See if it has changed since last evaluation
		if (CurrentSize == FoundItem->Size)
		{
			FoundItem->Status = ESourceStatus::Unchanged;
			return;
		}
		else
		{
			FoundItem->Size   = CurrentSize;
			FoundItem->Status = ESourceStatus::Updated;
			return;
		}
	}

	void FDisplayClusterMonitorProviderMedia::EvaluateICVFXCamera(const TSharedPtr<IDisplayClusterViewport>& InViewport)
	{
		if (InViewport->GetRenderSettings().TileSettings.GetType() != EDisplayClusterViewportTileType::None)
		{
			return;
		}

		const FString   SourceName  = InViewport->GetId();
		const FIntPoint CurrentSize = InViewport->GetContexts()[0].ContextSize;

		// Ignore invalid resource size
		if (CurrentSize.GetMin() < 1)
		{
			return;
		}

		// See if this camera has been registered already
		TMap<FString, FSourceInfo>& Sources = ActiveSources.FindOrAdd(EDCObservableType::ICVFXCamera);
		FSourceInfo* FoundItem = Sources.Find(SourceName);

		// If not yet exists, create a new one
		if (!FoundItem)
		{
			Sources.Emplace(SourceName, FSourceInfo
				{
					.Id           = FGuid::NewGuid(),
					.DisplayName  = InViewport->GetBaseId(),
					.ResourceName = SourceName,
					.Size         = CurrentSize,
					.Status       = ESourceStatus::New,
				});

			return;
		}

		// See if it has changed since last evaluation
		if (CurrentSize == FoundItem->Size)
		{
			FoundItem->Status = ESourceStatus::Unchanged;
			return;
		}
		else
		{
			FoundItem->Size   = CurrentSize;
			FoundItem->Status = ESourceStatus::Updated;
			return;
		}
	}

	void FDisplayClusterMonitorProviderMedia::EvaluateICVFXCameraTile(const TSharedPtr<IDisplayClusterViewport>& InViewport)
	{
		if (InViewport->GetRenderSettings().TileSettings.GetType() != EDisplayClusterViewportTileType::Tile)
		{
			return;
		}

		const FString   SourceName  = InViewport->GetId();
		const FIntPoint CurrentSize = InViewport->GetContexts()[0].ContextSize;

		// Ignore invalid resource size
		if (CurrentSize.GetMin() < 1)
		{
			return;
		}

		// See if this camera tile has been registered already
		TMap<FString, FSourceInfo>& Sources = ActiveSources.FindOrAdd(EDCObservableType::ICVFXCameraTile);
		FSourceInfo* FoundItem = Sources.Find(SourceName);

		// If not yet exists, create a new one
		if (!FoundItem)
		{
			const FIntPoint& TilePos = InViewport->GetRenderSettings().TileSettings.GetPos();

			Sources.Emplace(SourceName, FSourceInfo
				{
					.Id           = FGuid::NewGuid(),
					.DisplayName  = FString::Printf(TEXT("Tile [%d:%d]"), TilePos.X, TilePos.Y),
					.ResourceName = SourceName,
					.Size         = CurrentSize,
					.ParentName   = InViewport->GetBaseId(),
					.TilePos      = TilePos,
					.Status       = ESourceStatus::New,
				});

			return;
		}

		// See if it has changed since last evaluation
		if (CurrentSize == FoundItem->Size)
		{
			FoundItem->Status = ESourceStatus::Unchanged;
			return;
		}
		else
		{
			FoundItem->Size   = CurrentSize;
			FoundItem->Status = ESourceStatus::Updated;
			return;
		}
	}

	void FDisplayClusterMonitorProviderMedia::EvaluateUI()
	{
		// Get nDispplay render manager
		const IDisplayClusterRenderManager* const RenderMgr = IDisplayCluster::Get().GetRenderMgr();
		if (!RenderMgr)
		{
			return;
		}

		// Make sure the GUI controller is active 
		IDisplayClusterGUILayerController& GuiCtrl = RenderMgr->GetGuiLayerController();
		if (!GuiCtrl.IsEnabled())
		{
			return;
		}

		// Get GUI texture size
		static const FString SourceName = TEXT("UI");
		const FIntPoint CurrentSize = GuiCtrl.GetGuiLayerTextureSize();

		// Ignore invalid resource size
		if (CurrentSize.GetMin() < 1)
		{
			return;
		}

		TMap<FString, FSourceInfo>& Sources = ActiveSources.FindOrAdd(EDCObservableType::UI);
		FSourceInfo* FoundItem = Sources.Find(SourceName);

		// If not yet exists, create a new one
		if (!FoundItem)
		{
			Sources.Emplace(SourceName, FSourceInfo
				{
					.Id           = FGuid::NewGuid(),
					.DisplayName  = SourceName,
					.Size         = CurrentSize,
					.Status       = ESourceStatus::New,
				});

			return;
		}

		// See if it has changed since last evaluation
		if (CurrentSize == FoundItem->Size)
		{
			FoundItem->Status = ESourceStatus::Unchanged;
			return;
		}
		else
		{
			FoundItem->Size   = CurrentSize;
			FoundItem->Status = ESourceStatus::Updated;
			return;
		}
	}

	void FDisplayClusterMonitorProviderMedia::EvaluateBackbuffer()
	{
		if (!GEngine || !GEngine->GameViewport || !GEngine->GameViewport->Viewport)
		{
			return;
		}

		// Get backbuffer texture size
		static const FString SourceName = TEXT("Backbuffer");
		const FIntPoint CurrentSize = GEngine->GameViewport->Viewport->GetSizeXY();

		// Ignore invalid resource size
		if (CurrentSize.GetMin() < 1)
		{
			return;
		}

		TMap<FString, FSourceInfo>& Sources = ActiveSources.FindOrAdd(EDCObservableType::Backbuffer);
		FSourceInfo* FoundItem = Sources.Find(SourceName);

		// If not yet exists, create a new one
		if (!FoundItem)
		{
			Sources.Emplace(SourceName,
				FSourceInfo
				{
					.Id           = FGuid::NewGuid(),
					.DisplayName  = SourceName,
					.ResourceName = SourceName,
					.Size         = CurrentSize,
					.Status       = ESourceStatus::New,
				});

			return;
		}

		// See if it has changed since last evaluation
		if (CurrentSize == FoundItem->Size)
		{
			FoundItem->Status = ESourceStatus::Unchanged;
			return;
		}
		else
		{
			FoundItem->Size   = CurrentSize;
			FoundItem->Status = ESourceStatus::Updated;
			return;
		}
	}

	void FDisplayClusterMonitorProviderMedia::PropagateSourceInfo()
	{
		FDCMData_NodeObservables NodeObservablesData;

		// Iterate through all type groups
		for (auto TypeIt = ActiveSources.CreateIterator(); TypeIt; ++TypeIt)
		{
			// Itereate through every source of current type
			for (auto SrcIt = TypeIt.Value().CreateIterator(); SrcIt; ++SrcIt)
			{
				// Generate observable descriptor
				FDCMData_ObservableInfo SourceInfo
				{
					.Type       = TypeIt.Key(),
					.Id         = SrcIt.Value().Id,
					.Name       = SrcIt.Value().DisplayName,
					.Resolution = SrcIt.Value().Size,
					.ParentName = SrcIt.Value().ParentName.Get(FString()),
					.TilePos    = SrcIt.Value().TilePos.Get(FDCMData_ObservableInfo::InvalidTilePos)
				};

				// Based on the status, put in a proper basket
				switch (SrcIt.Value().Status)
				{
				case ESourceStatus::Unchanged:
					NodeObservablesData.ObservablesUnchanged.Add(MoveTemp(SourceInfo));
					break;

				case ESourceStatus::Updated:
					NodeObservablesData.ObservablesUpdated.Add(MoveTemp(SourceInfo));
					break;

				case ESourceStatus::New:
					NodeObservablesData.ObservablesAdded.Add(MoveTemp(SourceInfo));
					break;

				case ESourceStatus::Removed:
					NodeObservablesData.ObservablesRemoved.Add(MoveTemp(SourceInfo));
					break;

				default:
					// Other sources have not been changed, so we ignore them here
					break;
				}

				// Remove all sources already marked as 'Removed', and mark the remaining ones as 'Removed'.
				// They will be refreshed during the next evaluation cycle, or remain 'Removed' and be deleted.
				if (SrcIt.Value().Status == ESourceStatus::Removed)
				{
					SrcIt.RemoveCurrent();
				}
				else
				{
					SrcIt.Value().Status = ESourceStatus::Removed;
				}
			}
		}

		// See if there is anything to propagate. Send an update only when something has changed (ignore unchanged).
		const bool bPropagationRequired = (
			NodeObservablesData.ObservablesAdded.Num() > 0 ||
			NodeObservablesData.ObservablesRemoved.Num() > 0 ||
			NodeObservablesData.ObservablesUpdated.Num() > 0);

		// If there are any updates...
		if (bPropagationRequired)
		{
			UE_LOGF(LogDisplayClusterMonitorProviderMedia, Verbose, "Sending media observables update: added=%d, removed=%d, updated=%d, unchanged=%d",
				NodeObservablesData.ObservablesAdded.Num(),
				NodeObservablesData.ObservablesRemoved.Num(),
				NodeObservablesData.ObservablesUpdated.Num(),
				NodeObservablesData.ObservablesUnchanged.Num());

			// ... remember this most recent data
			LastObservablesInfo = NodeObservablesData;

			// and share it to anyone interested
			const TSet<EDCMessengerRole> RecipientRoles = { EDCMessengerRole::Monitor };
			Messenger->SendToRoles(RecipientRoles,
				FDCMMessage_NodeObservablesNotification
				{
					.Observables = MoveTemp(NodeObservablesData),
				});
		}
	}

	void FDisplayClusterMonitorProviderMedia::DeleteClientData(const FMessageAddress& ClientAddress)
	{
		// Stop all sessions that this endpoint owns
		if (TMap<FGuid, TSharedPtr<IMediaObservable>>* ClientSessions = ActiveSessions.Find(ClientAddress))
		{
			// Stop all sessions
			for (auto& [Id, Session] : *ClientSessions)
			{
				Session->StopCapture();
			}

			// And remove this client's data
			ActiveSessions.Remove(ClientAddress);
		}
	}

	void FDisplayClusterMonitorProviderMedia::StopActiveSessions()
	{
		// Stop all sessions
		for (auto& [MonitorAddr, MonitorSessions] : ActiveSessions)
		{
			for (auto& [Id, Session] : MonitorSessions)
			{
				Session->StopCapture();
			}
		}

		// And release resources
		ActiveSessions.Empty();
	}
}
