// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DisplayClusterMonitorProviderBase.h"
#include "DisplayClusterMonitorTypes.h"

class IDisplayClusterViewport;
struct FGuid;
struct FMessageAddress;


namespace UE::nDisplay::Monitor
{
	class IMediaObservable;

	/**
	 * Media observables provider
	 */
	class FDisplayClusterMonitorProviderMedia
		: public FDisplayClusterMonitorProviderBase
	{
		using Super = FDisplayClusterMonitorProviderBase;

	public:

		FDisplayClusterMonitorProviderMedia() = default;
		virtual ~FDisplayClusterMonitorProviderMedia() override = default;

	public:

		//~ Begin FDisplayClusterMonitorProviderBase
		virtual bool Start() override;
		virtual void Stop() override;

		virtual FString GetMessengerName() const override
		{
			return TEXT("MediaObservablesMessenger");
		}
		//~ End FDisplayClusterMonitorProviderBase

	private:

		/**
		 * Observable source status
		 */
		enum class ESourceStatus : uint8
		{
			/** New source since last evaluation */
			New,
			/** Removed source */
			Removed,
			/** Unchanged since last evaluation */
			Unchanged,
			/** The source has been updated since last evaluation */
			Updated
		};

		/**
		 * Observable source data
		 */
		struct FSourceInfo
		{
			/** Observable GUID */
			FGuid Id;
			/** Display name to use in UI */
			FString DisplayName;
			/** Internal viewport name */
			FString ResourceName;
			/** Observable texture size */
			FIntPoint Size;
			/** Parent name (optional, tiles only) */
			TOptional<FString> ParentName;
			/** Tile position (optional, tiles only) */
			TOptional<FIntPoint> TilePos;
			/** Current evaluation status, updates every frame */
			ESourceStatus Status = ESourceStatus::New;
		};

	private:

		/** Factory method to create an observable of specific type */
		TSharedPtr<IMediaObservable> CreateObservable(EDCObservableType InType, const FSourceInfo& SourceInfo);

	private:

		/** Handles remote requests to provide the list of available observables */
		void OnNodeObservablesRequest(const FDCEndpoint& InEndpoint, const FDCMMessage_NodeObservablesRequest& InMessage);

		/** Handles remote requests to start observation sessions */
		void OnStartSessionRequest(const FDCEndpoint& InEndpoint, const FDCMMessage_StartSessionRequest& InMessage);

		/** Handles remote requests to stop observation sessions */
		void OnStopSessionRequest(const FDCEndpoint& InEndpoint, const FDCMMessage_StopSessionRequest& InMessage);

		/** Handles session control commands */
		void OnObservableControlRequest(const FDCEndpoint& InEndpoint, const FDCMMessage_ObservableControlRequest& InMessage);

		/** Handles an endpoint leaving the cluster */
		void OnEndpointLeft(const FDCEndpoint& InEndpoint, const FString& InReason);

		/** Handles an endpoint timeout */
		void OnEndpointTimeout(const FDCEndpoint& InEndpoint);

		/** Handles end frame (game thread) */
		void OnEndFrame();

		/** Handles remote console commands */
		void OnConsoleCommand(const FDCEndpoint& InSenderEndpoint, const FString& InExecutorName, const FString& InCommand);

	private:

		/** Evaluates regular viewports */
		void EvaluateViewport(const TSharedPtr<IDisplayClusterViewport>& InViewport);

		/** Evaluates ICVFX cameras */
		void EvaluateICVFXCamera(const TSharedPtr<IDisplayClusterViewport>& InViewport);

		/** Evaluates ICVFX camera tiles */
		void EvaluateICVFXCameraTile(const TSharedPtr<IDisplayClusterViewport>& InViewport);

		/** Evaluates the GUI layer */
		void EvaluateUI();

		/** Evaluates the backbuffer */
		void EvaluateBackbuffer();

		/** Propaages data to the monitors */
		void PropagateSourceInfo();

		/** Removes all client's data from internal storage */
		void DeleteClientData(const FMessageAddress& ClientAddress);

		/** Terminates all active sessions */
		void StopActiveSessions();

	private:

		/** Currently available observables */
		TMap<EDCObservableType, TMap<FString, FSourceInfo>> ActiveSources;

		/** Per-monitor active observation sessions map <MonitorAddr, <ObservableId, ObservableImpl>> */
		TMap<FMessageAddress, TMap<FGuid, TSharedPtr<IMediaObservable>>> ActiveSessions;

		/** The most recent and actual observables evaluation */
		FDCMData_NodeObservables LastObservablesInfo;
	};
}
