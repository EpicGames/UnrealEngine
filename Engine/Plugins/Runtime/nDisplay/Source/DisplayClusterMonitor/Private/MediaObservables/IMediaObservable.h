// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

struct FGuid;

namespace UE::nDisplay::Monitor
{
	/**
	 * Media observable session interface
	 * 
	 * Declares the API of all media observables on the provider side
	 */
	class IMediaObservable
	{
	public:

		virtual ~IMediaObservable() = default;

	public:

		/** Returns ID of this observable entity */
		virtual FGuid GetId() const = 0;

		/** Returns display name of this observable entity */
		virtual FString GetName() const = 0;

		/** Returns internal name of the viewport resource */
		virtual FString GetResourceId() const = 0;

		/** Start resource capturing */
		virtual bool StartCapture() = 0;

		/** Stop resource capturing */
		virtual void StopCapture() = 0;
	};
}
