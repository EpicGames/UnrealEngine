// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaObservables/MediaObservableBackbuffer.h"

#include "Engine/Engine.h"
#include "Engine/GameViewportClient.h"

#include "DisplayClusterMonitorLog.h"
#include "IDisplayCluster.h"
#include "IDisplayClusterCallbacks.h"
#include "NDIMediaOutput.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "RHICommandList.h"
#include "RHIResources.h"
#include "UnrealClient.h"


namespace UE::nDisplay::Monitor
{
	FMediaObservableBackbuffer::FMediaObservableBackbuffer(const FGuid& InObservableId, const FString& InObservableName, const FString& InResourceId)
		: Super(InObservableId, InObservableName, InResourceId)
	{
	}

	bool FMediaObservableBackbuffer::StartCapture()
	{
		const bool bStarted = Super::StartCapture();

		if (bStarted)
		{
			IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPostBackbufferUpdated_RenderThread().AddRaw(this, &FMediaObservableBackbuffer::OnPostBackbufferUpdated_RenderThread);
		}

		return bStarted;
	}

	void FMediaObservableBackbuffer::StopCapture()
	{
		IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPostBackbufferUpdated_RenderThread().RemoveAll(this);
		Super::StopCapture();
	}

	void FMediaObservableBackbuffer::OnPostBackbufferUpdated_RenderThread(FRHICommandListImmediate& RHICmdList, FViewport* Viewport)
	{
		if (!Viewport)
		{
			return;
		}

		// Get backbuffer texture
		if (FRHITexture* const BackbufferTexture = Viewport->GetRenderTargetTexture())
		{
			FRDGBuilder GraphBuilder(RHICmdList);

			// Prepare capture request data
			FRDGTextureRef BackbufferTextureRef = RegisterExternalTexture(GraphBuilder, BackbufferTexture, TEXT("DCObservableBackbuffer"));
			const FIntRect TextureRegion = { FIntPoint::ZeroValue, BackbufferTextureRef->Desc.Extent };
			const FMediaOutputTextureInfo TextureInfo{ BackbufferTextureRef, TextureRegion };

			UE_LOGF(LogDisplayClusterMonitorObservableMedia, VeryVerbose, "'%ls' capturing backbuffer of size %dx%d",
				*GetName(), BackbufferTextureRef->Desc.Extent.X, BackbufferTextureRef->Desc.Extent.Y);

			// Capture the backbuffer
			ExportMediaData_RenderThread(GraphBuilder, TextureInfo);

			GraphBuilder.Execute();
		}
	}

	FIntPoint FMediaObservableBackbuffer::GetCaptureSize() const
	{
		// Return the actual backbuffer size
		if (GEngine && GEngine->GameViewport && GEngine->GameViewport->Viewport)
		{
			const FIntPoint Size = GEngine->GameViewport->Viewport->GetSizeXY();
			return Size;
		}

		return FIntPoint::ZeroValue;
	}
}
