// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MediaObservables/MediaObservableBase.h"

class FRHICommandListImmediate;
class FViewport;


namespace UE::nDisplay::Monitor
{
	/**
	 * Backbuffer media observable
	 * 
	 * Responsible for streaming out the backbuffer texture
	 */
	class FMediaObservableBackbuffer
		: public FMediaObservableBase
	{
		using Super = FMediaObservableBase;

	public:

		FMediaObservableBackbuffer(const FGuid& InObservableId, const FString& InObservableName, const FString& InResourceId);
		virtual ~FMediaObservableBackbuffer() override = default;

	public:

		//~ Begin IMediaObservableBase interface
		virtual bool StartCapture() override;
		virtual void StopCapture() override;
		//~ End IMediaObservableBase interface

	private:

		/** Returns backbuffer size */
		virtual FIntPoint GetCaptureSize() const override;

		/**
		 * Handles the backbuffer updates. When it's called, we can safely grab
		 * the current frame output and stream it out.
		 */
		void OnPostBackbufferUpdated_RenderThread(FRHICommandListImmediate& RHICmdList, FViewport* Viewport);
	};
}
