// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaObservables/MediaObservableBase.h"

#include "UObject/UObjectGlobals.h"

#include "DisplayClusterMonitorLog.h"
#include "IDisplayCluster.h"
#include "IDisplayClusterCallbacks.h"
#include "MediaCapture.h"
#include "NDIMediaOutput.h"

#include "RenderGraphResources.h"


namespace UE::nDisplay::Monitor
{
	FMediaObservableBase::FMediaObservableBase(const FGuid& InObservableId, const FString& InObservableName, const FString& InResourceId)
		: ObservableId(InObservableId)
		, ObservableName(InObservableName)
		, ResourceId(InResourceId)
		, MediaOutput(NewObject<UNDIMediaOutput>())
	{
		IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPostTick().AddRaw(this, &FMediaObservableBase::OnPostClusterTick);
	}

	FMediaObservableBase::~FMediaObservableBase()
	{
		IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPostTick().RemoveAll(this);
	}

	void FMediaObservableBase::AddReferencedObjects(FReferenceCollector& Collector)
	{
		Collector.AddReferencedObject(MediaOutput);
		Collector.AddReferencedObject(MediaCapture);
	}

	bool FMediaObservableBase::StartCapture()
	{
		// Check if capturing already
		if (MediaCapture && bWasCaptureStarted)
		{
			return true;
		}

		// Configure media source name
		MediaOutput->SourceName = GenerateMediaSourceName();

		// Create media capture
		MediaCapture = MediaOutput->CreateMediaCapture();
		if (!IsValid(MediaCapture))
		{
			UE_LOGF(LogDisplayClusterMonitorObservableMedia, Error, "Failed to create MediaCapture '%ls'", *GetName());
			return false;
		}

		// Start cpaturing
		MediaCapture->SetMediaOutput(MediaOutput);
		bWasCaptureStarted = StartMediaCapture();

		return bWasCaptureStarted;
	}

	void FMediaObservableBase::StopCapture()
	{
		// Stop capture
		if (MediaCapture)
		{
			MediaCapture->StopCapture(false);
			MediaCapture = nullptr;
			bWasCaptureStarted = false;
		}
	}

	void FMediaObservableBase::OnPostClusterTick()
	{
		if (MediaCapture)
		{
			EMediaCaptureState MediaCaptureState = MediaCapture->GetState();

			// If we're capturing but the desired capture resolution does not match the texture being captured,
			// restart the capture with the updated size.

			if (MediaCaptureState == EMediaCaptureState::Capturing)
			{
				const FIntPoint LastSrcRegionIntPoint = LastSrcRegionSize.load().ToIntPoint();
				const FIntPoint DesiredSize = MediaCapture->GetDesiredSize();
				const bool bHasExportedAlready = (LastSrcRegionIntPoint != FIntPoint::ZeroValue);
				const bool bDesiredSizeMatchesLastRegion = (DesiredSize == LastSrcRegionIntPoint);
				const bool bDesiredSizeMatchesCustomRegion = (CustomResolution.X > 0 && CustomResolution.Y > 0 && DesiredSize == CustomResolution);

				// We don't restart if we haven't exported any textures yet (indicated by zero size LastSrcRegion)
				// to avoid constant media restarts since in such case media is set to use GetCaptureSize() != (0.0).
				// Once an export happens, any media restart will use LastSrcRegion which should not trigger a restart
				// when texture exports are suspended, since that does not cause a mismatch. 
				// This is preferred to setting LastSrcRegion to GetCaptureSize() because that would not reflect the actual
				// last captured size.

				if (bHasExportedAlready && !(bDesiredSizeMatchesLastRegion || bDesiredSizeMatchesCustomRegion))
				{
					UE_LOGF(LogDisplayClusterMonitorObservableMedia, Log, "Stopping MediaCapture '%ls' because its DesiredSize (%d, %d) doesn't match the captured texture size (%d, %d)",
						*GetName(), DesiredSize.X, DesiredSize.Y, LastSrcRegionIntPoint.X, LastSrcRegionIntPoint.Y);

					MediaCapture->StopCapture(false /* bAllowPendingFrameToBeProcess */);
					MediaCaptureState = MediaCapture->GetState(); // Re-sample state to restart the media capture right away
				}
			}

			const bool bMediaCaptureNeedsRestart = (MediaCaptureState == EMediaCaptureState::Error) || (MediaCaptureState == EMediaCaptureState::Stopped);

			if (!bWasCaptureStarted || bMediaCaptureNeedsRestart)
			{
				constexpr double Interval = 1.0;
				const double CurrentTime = FPlatformTime::Seconds();

				if (CurrentTime - LastRestartTimestamp > Interval)
				{
					UE_LOGF(LogDisplayClusterMonitorObservableMedia, Log, "MediaCapture '%ls' is in error or stopped, restarting it.", *GetName());

					bWasCaptureStarted = StartMediaCapture();
					LastRestartTimestamp = CurrentTime;
				}
			}
		}
	}

	bool FMediaObservableBase::StartMediaCapture()
	{
		FRHICaptureResourceDescription Descriptor;

		if (LastSrcRegionSize.load().ToIntPoint() == FIntPoint::ZeroValue)
		{
			Descriptor.ResourceSize = GetCaptureSize();
		}
		else
		{
			Descriptor.ResourceSize = LastSrcRegionSize.load().ToIntPoint();
		}

		if (Descriptor.ResourceSize == FIntPoint::ZeroValue)
		{
			return false;
		}

		FMediaCaptureOptions MediaCaptureOptions;
		MediaCaptureOptions.NumberOfFramesToCapture = -1;
		MediaCaptureOptions.bAutoRestartOnSourceSizeChange = false; // true won't work due to MediaCapture auto-changing crop mode to custom when capture region is specified.
		MediaCaptureOptions.bSkipFrameWhenRunningExpensiveTasks = false;
		MediaCaptureOptions.OverrunAction = EMediaCaptureOverrunAction::Flush;

		// When custom resolution is required, we should always scale the output to the required size
		if (CustomResolution.X > 0 && CustomResolution.Y > 0)
		{
			Descriptor.ResourceSize = CustomResolution;
			MediaCaptureOptions.Crop = EMediaCaptureCroppingType::None;
			MediaCaptureOptions.ResizeMethod = EMediaCaptureResizeMethod::ResizeInRenderPass;
		}

		const bool bCaptureStarted = MediaCapture->CaptureRHITexture(Descriptor, MediaCaptureOptions);

		if (bCaptureStarted)
		{
			UE_LOGF(LogDisplayClusterMonitorObservableMedia, Log, "Started media capture: '%ls' (%d x %d)",
				*GetName(), Descriptor.ResourceSize.X, Descriptor.ResourceSize.Y);
		}
		else
		{
			UE_LOGF(LogDisplayClusterMonitorObservableMedia, Warning, "Couldn't start media capture '%ls' (%d x %d)",
				*GetName(), Descriptor.ResourceSize.X, Descriptor.ResourceSize.Y);
		}

		return bCaptureStarted;
	}

	FString FMediaObservableBase::GenerateMediaSourceName() const
	{
		// Get observable name as a foundation
		FString MediaSourceName = GetName();

		// Replace invalid characters
		MediaSourceName.ReplaceInline(TEXT(":"), TEXT(" "));

		return MediaSourceName;
	}

	void FMediaObservableBase::ExportMediaData_RenderThread(FRDGBuilder& GraphBuilder, const FMediaOutputTextureInfo& TextureInfo)
	{
		// Check if MediaCapture is available
		if (!IsValid(MediaCapture))
		{
			return;
		}

		// Check if request data is valid
		if (!TextureInfo.IsValidData())
		{
			UE_LOGF(LogDisplayClusterMonitorObservableMedia, Warning, "MediaObservable '%ls': wrong capture data on RT frame %llu", *GetName(), GFrameCounterRenderThread);
			return;
		}

		// Track capture size
		{
			const FIntPoint& SrcTextureSize = TextureInfo.Texture->Desc.Extent;
			const FIntPoint  SrcRegionSize = TextureInfo.Region.Size();

			LastSrcRegionSize = FIntSize(SrcRegionSize);

			UE_LOGF(LogDisplayClusterMonitorObservableMedia, VeryVerbose, "MediaObservable '%ls': Requested texture export [size=%dx%d, rect=%dx%d, format=%u] on RT frame '%llu'...",
				*GetName(), SrcTextureSize.X, SrcTextureSize.Y, SrcRegionSize.X, SrcRegionSize.Y, (uint32)TextureInfo.Texture->Desc.Format, GFrameCounterRenderThread);
		}

		// Capture...
		const bool bCaptureSucceeded = MediaCapture->TryCaptureImmediate_RenderThread(GraphBuilder, TextureInfo.Texture, TextureInfo.Region);
		if (!bCaptureSucceeded)
		{
			UE_LOGF(LogDisplayClusterMonitorObservableMedia, Warning, "MediaCapture '%ls': failed to capture", *GetName());
		}
	}

	bool FMediaObservableBase::FMediaOutputTextureInfo::IsValidData() const
	{
		// Check if source texture is valid
		if (!Texture)
		{
			return false;
		}

		// Check if region matches the texture
		const FIntPoint RegionSize = Region.Size();
		const bool bCorrectRegion =
			Region.Min.GetMin() >= 0 &&
			RegionSize.GetMin() > 0 &&
			(Region.Min.X + RegionSize.X) <= Texture->Desc.Extent.X &&
			(Region.Min.Y + RegionSize.Y) <= Texture->Desc.Extent.Y;

		return bCorrectRegion;
	}
}
