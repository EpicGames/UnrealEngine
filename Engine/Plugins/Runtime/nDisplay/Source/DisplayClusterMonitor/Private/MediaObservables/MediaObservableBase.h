// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MediaObservables/IMediaObservable.h"
#include "UObject/GCObject.h"

#include <atomic>
#include "RenderGraphFwd.h"

class UMediaCapture;
class UNDIMediaOutput;


namespace UE::nDisplay::Monitor
{
	/**
	 * Abstract media observable class
	 * 
	 * It provides the common media capture functionality, and is used
	 * as a base one for all media observable implementations.
	 */
	class FMediaObservableBase
		: public IMediaObservable
		, public FGCObject
	{
	public:

		FMediaObservableBase(const FGuid& InObservableId, const FString& InObservableName, const FString& InResourceId);
		virtual ~FMediaObservableBase();

	public:

		//~ Begin IMediaObservableBase interface

		virtual FGuid GetId() const override final
		{
			return ObservableId;
		}

		virtual FString GetName() const override final
		{
			return ObservableName;
		}

		virtual FString GetResourceId() const override final
		{
			return ResourceId;
		}

		virtual bool StartCapture() override;
		virtual void StopCapture() override;

		//~ End IMediaObservableBase interface

	public:

		//~ Begin FGCObject interface
		virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
		virtual FString GetReferencerName() const override
		{
			return TEXT("FMediaObservableBase");
		}
		//~ End FGCObject interface

	protected:

		/** Media capture data */
		struct FMediaOutputTextureInfo
		{
			/** Texture to capture by a media capture device */
			FRDGTextureRef Texture = nullptr;

			/** Subregion to capture */
			FIntRect Region = { FIntPoint::ZeroValue, FIntPoint::ZeroValue };

			/** Validates if capture request data is valid */
			bool IsValidData() const;
		};

	protected:

		/** PostClusterTick event handler. It's used to restart capturing if needed */
		void OnPostClusterTick();

		/** Re-starts media capturing after failure */
		bool StartMediaCapture();

		/** Generates kind of unique media ID to use as media source name */
		virtual FString GenerateMediaSourceName() const;

		/** Passes capture data request to the capture device */
		void ExportMediaData_RenderThread(FRDGBuilder& GraphBuilder, const FMediaOutputTextureInfo& TextureInfo);

		/** Returns capture size */
		virtual FIntPoint GetCaptureSize() const = 0;

		/** Returns current media output */
		UNDIMediaOutput* GetMediaOutput() const
		{
			return MediaOutput;
		}

	private:

		/**
		 * Trivial version of FIntPoint so that it can be std::atomic
		 */
		struct FIntSize
		{
			int32 X = 0;
			int32 Y = 0;

			FIntSize(int32 InX, int32 InY) : X(InX), Y(InY) {}
			FIntSize(const FIntPoint& IntPoint) : X(IntPoint.X), Y(IntPoint.Y) {}

			FIntPoint ToIntPoint()
			{
				return FIntPoint(X, Y);
			}
		};

	private:

		/** Observable GUID */
		const FGuid ObservableId;

		/** Observable name */
		const FString ObservableName;

		/** Internal resource name for this observable */
		const FString ResourceId;

		/** NDI media output used for streaming data out */
		TObjectPtr<UNDIMediaOutput>  MediaOutput;

		/** Media capture used for this observable */
		TObjectPtr<UMediaCapture> MediaCapture;

		/** Custom resolution to use on capture side */
		FIntPoint CustomResolution = FIntPoint::ZeroValue;

		/** Used to restart media capture in the case it falls in error */
		bool bWasCaptureStarted = false;

		/** Used to control the rate at which we try to restart the capture */
		double LastRestartTimestamp = 0;

		/** Last region size of the texture being exported.Used to restart the capture when in error. */
		std::atomic<FIntSize> LastSrcRegionSize{ FIntSize(0, 0) };
	};
}
