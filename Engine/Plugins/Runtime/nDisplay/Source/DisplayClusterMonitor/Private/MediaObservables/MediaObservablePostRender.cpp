// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaObservables/MediaObservablePostRender.h"

#include "Render/IDisplayClusterRenderManager.h"
#include "Render/Projection/IDisplayClusterProjectionPolicy.h"
#include "Render/Viewport/IDisplayClusterViewport.h"
#include "Render/Viewport/IDisplayClusterViewportManager.h"
#include "Render/Viewport/IDisplayClusterViewportProxy.h"

#include "ShaderParameters/DisplayClusterShaderParameters_TransformTexture.h"

#include "DisplayClusterMonitorLog.h"
#include "IDisplayCluster.h"
#include "IDisplayClusterCallbacks.h"
#include "IDisplayClusterShaders.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"


namespace UE::nDisplay::Monitor
{
	FMediaObservablePostRender::FMediaObservablePostRender(const FGuid& InObservableId, const FString& InObservableName, const FString& InResourceId)
		: FMediaObservableBase(InObservableId, InObservableName, InResourceId)
	{ }

	bool FMediaObservablePostRender::StartCapture()
	{
		// Start capture
		const bool bStarted = Super::StartCapture();
		if (bStarted)
		{
			IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPostRenderViewFamily_RenderThread().AddRaw(this, &FMediaObservablePostRender::OnPostRenderViewFamily_RenderThread);
		}

		return bStarted;
	}

	void FMediaObservablePostRender::StopCapture()
	{
		// Unsubscribe from external events/callbacks
		IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPostRenderViewFamily_RenderThread().RemoveAll(this);

		// Stop capturing
		Super::StopCapture();
	}

	FIntPoint FMediaObservablePostRender::GetCaptureSize() const
	{
		// We need to get actual texture size of an internal viewport resource
		if (const IDisplayClusterRenderManager* const RenderMgr = IDisplayCluster::Get().GetRenderMgr())
		{
			if (const IDisplayClusterViewportManager* const ViewportMgr = RenderMgr->GetViewportManager())
			{
				if (const IDisplayClusterViewport* const Viewport = ViewportMgr->FindViewport(GetResourceId()))
				{
					const TArray<FDisplayClusterViewport_Context>& Contexts = Viewport->GetContexts();
					if (Contexts.Num() > 0)
					{
						return Contexts[0].ContextSize;
					}
				}
			}
		}

		return FIntPoint::ZeroValue;
	}

	void FMediaObservablePostRender::OnPostRenderViewFamily_RenderThread(FRDGBuilder& GraphBuilder, const FSceneViewFamily& /*ViewFamily*/, const IDisplayClusterViewportProxy* ViewportProxy)
	{
		checkSlow(ViewportProxy);

		if (ViewportProxy && ViewportProxy->GetId().Equals(GetResourceId(), ESearchCase::IgnoreCase))
		{
			TArray<FRHITexture*> Textures;
			TArray<FIntRect>     Regions;

			// Get RHI texture and pass it to the media capture pipeline
			if (ViewportProxy->GetResourcesWithRects_RenderThread(EDisplayClusterViewportResourceType::InternalRenderTargetEntireRectResource, Textures, Regions))
			{
				if (Textures.Num() > 0 && Regions.Num() > 0 && Textures[0])
				{
					// Is frustum currently rotated?
					bool bFrustumRotated = false;
					if (const TSharedPtr<IDisplayClusterProjectionPolicy> ProjectionPolicy = ViewportProxy->GetProjectionPolicy_RenderThread())
					{
						bFrustumRotated = ProjectionPolicy->IsFrustumRotatedToFitContextSize_RenderThread(ViewportProxy, 0);
					}

					// If the frustum is rotated, undo the rotation before capturing
					if (bFrustumRotated)
					{
						// Prepare request data
						FDisplayClusterShaderParameters_TransformTexture TransformParams;
						TransformParams.InputTexture      = RegisterExternalTexture(GraphBuilder, Textures[0], TEXT("nD.ViewportToRotate"));
						TransformParams.InputRegion       = Regions[0];
						TransformParams.TranformationType = FDisplayClusterShaderParameters_TransformTexture::ETranformation::Rotation_270; // 90 ccw

						// Rotate the texture back. We don't provide any output textures, it will be created automatically
						IDisplayClusterShaders::Get().AddTransformTexturePass(GraphBuilder, TransformParams);

						// Now capture the un-rotated texture
						if (TransformParams.OutputTexture)
						{
							ExportMediaData_RenderThread(
								GraphBuilder,
								FMediaOutputTextureInfo
								{
									.Texture = TransformParams.OutputTexture,
									.Region  = { FIntPoint::ZeroValue, TransformParams.OutputTexture->Desc.Extent}, // capture whole output
								});
						}
					}
					// Otherwise, capture it as is
					else
					{
						FRDGTextureRef SrcTextureRef = RegisterExternalTexture(GraphBuilder, Textures[0], TEXT("DCObservablePostRender"));
						FMediaOutputTextureInfo TextureInfo{ SrcTextureRef, Regions[0] };
						ExportMediaData_RenderThread(GraphBuilder, TextureInfo);
					}
				}
			}
		}
	}
}
