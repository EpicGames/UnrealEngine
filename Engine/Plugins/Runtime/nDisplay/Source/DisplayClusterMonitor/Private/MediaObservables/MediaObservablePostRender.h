// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MediaObservables/MediaObservableBase.h"

class FRDGBuilder;
class FSceneViewFamily;
class IDisplayClusterViewportProxy;


namespace UE::nDisplay::Monitor
{
	/**
	 * PostRender media observable
	 * 
	 * In nDisplay, regular viewports, ICVFX cameras or camera tiles are
	 * rendered in a normal way. They all get PostRenderViewFamily callback
	 * where we can grab the render output. This observable is responisble
	 * exactly for that. It captures the corresponding media data right
	 * after it's rendered, and streams out then.
	 */
	class FMediaObservablePostRender
		: public FMediaObservableBase
	{
		using Super = FMediaObservableBase;

	public:

		FMediaObservablePostRender(const FGuid& InObservableId, const FString& InObservableName, const FString& InResourceId);
		virtual ~FMediaObservablePostRender() override = default;

	public:

		//~ Begin IMediaObservableBase interface
		virtual bool StartCapture() override;
		virtual void StopCapture() override;
		//~ End IMediaObservableBase interface

	protected:

		//~ Begin FMediaObservableBase interface

		/** Returns texture size of a viewport assigned to capture (main thread) */
		virtual FIntPoint GetCaptureSize() const override;

		//~ End FMediaObservableBase interface

	private:

		/** PostRenderViewFamily callback handler where data is captured right after it's rendered */
		void OnPostRenderViewFamily_RenderThread(FRDGBuilder& GraphBuilder, const FSceneViewFamily& ViewFamily, const IDisplayClusterViewportProxy* ViewportProxy);
	};
}
