// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaObservables/MediaObservableUI.h"

#include "Render/IDisplayClusterRenderManager.h"
#include "Render/GUILayer/IDisplayClusterGUILayerController.h"

#include "DisplayClusterMonitorLog.h"
#include "IDisplayCluster.h"
#include "IDisplayClusterCallbacks.h"

#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"


namespace UE::nDisplay::Monitor
{
	FMediaObservableUI::FMediaObservableUI(const FGuid& InObservableId, const FString& InObservableName, const FString& InResourceId)
		: FMediaObservableBase(InObservableId, InObservableName, InResourceId)
	{
	}

	bool FMediaObservableUI::StartCapture()
	{
		const bool bStarted = Super::StartCapture();

		if (bStarted)
		{
			IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPostBackbufferUpdated_RenderThread().AddRaw(this, &FMediaObservableUI::HandlePostBackbufferUpdated_RenderThread);
		}

		return bStarted;
	}

	void FMediaObservableUI::StopCapture()
	{
		IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPostBackbufferUpdated_RenderThread().RemoveAll(this);
		Super::StopCapture();
	}


	FIntPoint FMediaObservableUI::GetCaptureSize() const
	{
		// Render manager provides access to the GUI layer controller
		const IDisplayClusterRenderManager* const RenderMgr = IDisplayCluster::Get().GetRenderMgr();
		if (!RenderMgr)
		{
			return FIntPoint::ZeroValue;
		}

		// Get GUI texture size
		IDisplayClusterGUILayerController& GuiGtrl = RenderMgr->GetGuiLayerController();
		const FIntPoint GuiTextureSize = GuiGtrl.GetGuiLayerTextureSize();
		return GuiTextureSize;
	}

	void FMediaObservableUI::HandlePostBackbufferUpdated_RenderThread(FRHICommandListImmediate& RHICmdList, FViewport* Viewport)
	{
		if (!Viewport)
		{
			return;
		}

		const IDisplayClusterRenderManager* const RenderMgr = IDisplayCluster::Get().GetRenderMgr();
		if (!RenderMgr)
		{
			return;
		}

		// Get the GUI layer controller
		IDisplayClusterGUILayerController& GuiGtrl = RenderMgr->GetGuiLayerController();
		if (FTextureRHIRef GuiTexture = GuiGtrl.GetGuiLayerTexture_RenderThread())
		{
			FRDGBuilder GraphBuilder(RHICmdList);

			// Prepare capture request data
			FRDGTextureRef GuiTextureRef = RegisterExternalTexture(GraphBuilder, GuiTexture, TEXT("DCObservableUI"));
			const FIntRect TextureRegion = { FIntPoint::ZeroValue, GuiTextureRef->Desc.Extent };
			const FMediaOutputTextureInfo TextureInfo{ GuiTextureRef, TextureRegion };

			UE_LOGF(LogDisplayClusterMonitorObservableMedia, VeryVerbose, "'%ls' capturing GUI layer texture of size %dx%d",
				*GetName(), GuiTextureRef->Desc.Extent.X, GuiTextureRef->Desc.Extent.Y);

			// Capture the UI
			ExportMediaData_RenderThread(GraphBuilder, TextureInfo);

			GraphBuilder.Execute();
		}
	}
}
