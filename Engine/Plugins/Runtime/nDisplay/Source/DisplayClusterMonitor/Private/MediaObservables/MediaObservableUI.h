// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MediaObservables/MediaObservableBase.h"

class FRHICommandListImmediate;
class FViewport;


namespace UE::nDisplay::Monitor
{
	/**
	 * GUI layer media observable
	 * 
	 * nDisplay has internal solution that allows to draw the overlay UI
	 * into a separate texture, then blend it on top of the other viewports.
	 * This observable is responsible for straming out the GUI texture only.
	 */
	class FMediaObservableUI
		: public FMediaObservableBase
	{
		using Super = FMediaObservableBase;

	public:

		FMediaObservableUI(const FGuid& InObservableId, const FString& InObservableName, const FString& InResourceId);
		virtual ~FMediaObservableUI() override = default;

	public:

		//~ Begin IMediaObservableBase interface
		virtual bool StartCapture() override;
		virtual void StopCapture() override;
		//~ End IMediaObservableBase interface

	private:

		/** Returns UI layer size */
		virtual FIntPoint GetCaptureSize() const override;

		/**
		 * It's called, when the backbuffer is already updated for current frame.
		 * This also means the GUI texture is ready as well so we can safely grab it.
		 */
		void HandlePostBackbufferUpdated_RenderThread(FRHICommandListImmediate& RHICmdList, FViewport* Viewport);
	};
}
