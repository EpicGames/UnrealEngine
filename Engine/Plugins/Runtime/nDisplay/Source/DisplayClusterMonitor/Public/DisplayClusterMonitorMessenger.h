// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DisplayClusterMonitorTypes.h"
#include "MessageEndpoint.h"
#include "Containers/Ticker.h"
#include "Delegates/DelegateCombinations.h"
#include "Misc/App.h"
#include "Templates/Function.h"
#include "UObject/Class.h"

class IMessageContext;


namespace UE::nDisplay::Monitor
{
	/**
	 * Cluster monitor messenger
	 * 
	 * It's a wrapper around the Message Bus message endpoint that encapsulates some basic functionality
	 * like disovery, timeout monitor, callbacks. etc.
	 */
	class DISPLAYCLUSTERMONITOR_API FDCMessenger
	{
	public:

		FDCMessenger() = default;
		~FDCMessenger();

	public:

		/**
		 * Start messenger with given name and roles
		 * 
		 * @param InEndpointName  - Entity name
		 * @param InEndpointRoles - Set of roles for this entity
		 * 
		 * @return - True if started successfully
		 */
		bool Start(const FString& InEndpointName, const TSet<EDCMessengerRole>& InEndpointRoles);

		/**
		 * Stop messenger
		 * 
		 * @param InReason - (optional) The reason for shutting down this entity
		 */
		void Stop(const FString& InReason = FString());

		/**
		 * Whether the messenger is currently runing
		 *
		 * @return - True if currently running
		 */
		bool IsRunning() const
		{
			return bIsRunning;
		}

		/**
		 * Returns MessageBus address of this messenger
		 *
		 * @return - MessageBus address
		 */
		FMessageAddress GetAddress() const;

		/**
		 * Returns MessageBus address of a spcific residence
		 *
		 * @param ClusterId - GUID of a cluster
		 * @param NodeId    - GUID of a cluster node
		 * 
		 * @return - MessageBus address
		 */
		FMessageAddress GetAddress(const FGuid& InClusterId, const FGuid& InNodeId) const;

		/**
		 * Returns this messenger's endpoint info
		 *
		 * @return - Endpoint information
		 */
		const FDCEndpoint& GetEndpointInfo() const;

		/**
		 * Returns a list of endpoints that have been discovered
		 *
		 * @return - Discovered endpoints
		 */
		TConstArrayView<FDCEndpoint> GetDiscoveredEndpoints() const;

	public:

		/**
		 * Send message to a list of recipients
		 *
		 * @param InRecipients - An array of message destinations
		 * @param InMessage    - A message to send
		 */
		template <typename MessageType>
		requires std::derived_from<MessageType, FDCMMessageBase>
		void Send(const TArray<FMessageAddress>& InRecipients, const MessageType& InMessage)
		{
			SendImpl(
				InRecipients,
				FMessageEndpoint::MakeMessage<MessageType>(InMessage),
				MessageType::StaticStruct());
		}

		/**
		 * Send message to a list of recipients (move semantics)
		 *
		 * @param InRecipients - An array of message destinations
		 * @param InMessage    - A message to send
		 */
		template <typename MessageType>
		requires std::derived_from<MessageType, FDCMMessageBase>
		void Send(const TArray<FMessageAddress>& InRecipients, MessageType&& InMessage)
		{
			SendImpl(
				InRecipients,
				FMessageEndpoint::MakeMessage<MessageType>(Forward<MessageType>(InMessage)),
				MessageType::StaticStruct()
			);
		}

		/**
		 * Send message to everyone who has any of specified messenger roles
		 *
		 * @param InEndpointTypes - A set of roles that should receive this message
		 * @param InMessage       - A message to send
		 */
		template <typename MessageType>
		requires std::derived_from<MessageType, FDCMMessageBase>
		void SendToRoles(const TSet<EDCMessengerRole>& InEndpointTypes, MessageType&& InMessage)
		{
			SendToRoles(InEndpointTypes, FMessageEndpoint::MakeMessage<MessageType>(InMessage), MessageType::StaticStruct());
		}

		/**
		 * Broadcast message
		 * 
		 * @note Every cluster monitor messenger will receive it.
		 *
		 * @param InMessage       - A message to send
		 */
		template <typename MessageType>
		requires std::derived_from<MessageType, FDCMMessageBase>
		void Broadcast(MessageType&& InMessage)
		{
			BroadcastImpl(FMessageEndpoint::MakeMessage<MessageType>(InMessage), MessageType::StaticStruct());
		}

	public:

		/** Called every time a new endpoint is discovered */
		DECLARE_MULTICAST_DELEGATE_OneParam(FOnEndpointJoined, const FDCEndpoint& /* Endpoint */);
		FOnEndpointJoined OnEndpointJoined;

		/** Called every time a known endpoint is considered unresponsive */
		DECLARE_MULTICAST_DELEGATE_OneParam(FOnEndpointTimeout, const FDCEndpoint& /* Endpoint */);
		FOnEndpointTimeout OnEndpointTimeout;

		/** Called every time a known endpoint reports it's leaving */
		DECLARE_MULTICAST_DELEGATE_TwoParams(FOnEndpointLeft, const FDCEndpoint& /* Endpoint */, const FString& /* Reason */);
		FOnEndpointLeft OnEndpointLeft;

		/** Called every time a cluster monitor requests local execution of a console command */
		DECLARE_DELEGATE_ThreeParams(FOnConsoleCommand, const FDCEndpoint& /* SenderEndpoint */, const FString& /* ExecutorName */, const FString& /* Command */)
		FOnConsoleCommand OnConsoleCommand;

	public:

		/**
		 * Auxiliary non-templated class to generalize user delegates
		 */
		struct FCustomMessageEventBase
		{
			virtual ~FCustomMessageEventBase() = default;

			/** Non-templated (generalized) delegate call */
			virtual void Broadcast(const FDCEndpoint& Endpoint, const FDCMMessage& Msg) = 0;
		};

		/**
		 * Universal user delegates wrapper
		 */
		template<typename MessageType>
		requires std::derived_from<MessageType, FDCMMessage>
		struct FCustomMessageEvent : public FCustomMessageEventBase
		{
			/** Actual delegate */
			DECLARE_MULTICAST_DELEGATE_TwoParams(FDelegate, const FDCEndpoint&, const MessageType&);
			FDelegate Delegate;

			/** Callback impplementation */
			virtual void Broadcast(const FDCEndpoint& Endpoint, const FDCMMessage& Msg) override
			{
				const MessageType* TypedMsg = static_cast<const MessageType*>(&Msg);
				Delegate.Broadcast(Endpoint, *TypedMsg);
			}
		};

		/**
		 * Generic access to the user delegates
		 * 
		 * Usage:
		 * OnMessage<FDCMMessage_Child1>().AddRaw(...);
		 * OnMessage<FDCMMessage_Child2>().AddRaw(...);
		 * OnMessage<FDCMMessage_Child3>().AddRaw(...);
		 */
		template<typename MessageType>
		requires std::derived_from<MessageType, FDCMMessage>
		FCustomMessageEvent<MessageType>::FDelegate& OnMessage()
		{
			// Find or create a delegate instance for this message type
			UScriptStruct* Key = MessageType::StaticStruct();
			TSharedPtr<FCustomMessageEventBase>& BaseDelegatePtr = UserMessageEvents.FindOrAdd(Key);

			// Not found, so create a new one
			if (!BaseDelegatePtr)
			{
				BaseDelegatePtr = MakeShared<FCustomMessageEvent<MessageType>>();
			}

			// And return the actual delegate instance
			return static_cast<FCustomMessageEvent<MessageType>*>(BaseDelegatePtr.Get())->Delegate;
		}

	private:

		/** It's an auxiliary function that executes user delegate based on the message type */
		void CallUserFunction(const UScriptStruct* MessageTypeInfo, const FDCEndpoint& Endpoint, const FDCMMessage& Message)
		{
			// Validate input
			if (!MessageTypeInfo || !MessageTypeInfo->IsChildOf(FDCMMessage::StaticStruct()))
			{
				return;
			}

			// Execute the delegate
			if (TSharedPtr<FCustomMessageEventBase>* Base = UserMessageEvents.Find(MessageTypeInfo))
			{
				Base->Get()->Broadcast(Endpoint, Message); // non-templated call
			}
		}

	private:

		/** Private implementation of the public templated "Send" function */
		void SendImpl(const TArray<FMessageAddress>& InRecipients, void* InPayload, UScriptStruct* InType);

		/** Private implementation of the public templated "SendToRoles" function */
		void SendToRoles(const TSet<EDCMessengerRole>& InEndpointTypes, void* InPayload, UScriptStruct* InType);

		/** Private implementation of the public templated "Broadcast" function */
		void BroadcastImpl(void* InPayload, UScriptStruct* InType);

	private:

		/** Our custom Tick() handler */
		bool Tick(float InDeltaTime);

		/** Broadcasts heartbeat pulse */
		void TickTask_PulseHearbeat(float InDeltaTime);

		/** Checks if there are any unresponsive endpoints among the known ones */
		void TickTask_CheckInactiveEndpoints(float InDeltaTime);

	private:

		/** Auxiliary function that searches for an endpoint that has the specified MessageBus address */
		FDCEndpoint* GetEndpoint(const FMessageAddress& InAddress)
		{
			return DiscoveredEndpoints.FindByPredicate([&InAddress](const FDCEndpoint& EP) { return EP.Address == InAddress; });
		}

		/** Auxiliary function that finds the index of an endpoint in the known endpoints array by its MessageBus address */
		int32 GetEndpointIdx(const FMessageAddress& InAddress) const
		{
			return DiscoveredEndpoints.IndexOfByPredicate([&InAddress](const FDCEndpoint& EP) { return EP.Address == InAddress; });
		}

		/**
		 * Auxiliary function that generates the cluster GUID deterministically. The generated GUID
		 * is expected to be the same on every cluster node.
		 */
		static FGuid MakeDeterministicClusterGuid(const FString& InPrimaryNodeId, const FString& InPrimaryNodeAddr, uint16 InPrimaryExclusiveCommPort);

		/**
		 * Returns residence discriptor of this messenger
		 */
		FDCMData_ResidenceDescriptor GetResidenceDescriptor() const;

	private:

		/** Notify everyone this messenger is online */
		void BroadcastJoin();

		/** Broadcast hearbeat pulse */
		void BroadcastHeartbeat();

		/** Notify everyone this messenger is shutting down */
		void BroadcastLeave(const FString& InShutdownReason);

	private:

		/** Handles discovery requests */
		void OnDiscoveryRequest(const FDCMMessage_DiscoveryRequest& InMessage, const TSharedRef<IMessageContext>& InContext);

		/** Handles discovery responses */
		void OnDiscoveryResponse(const FDCMMessage_DiscoveryResponse& InMessage, const TSharedRef<IMessageContext>& InContext);

		/** Handles heartbeat notifications */
		void OnHeartbeat(const FDCMMessage_Heartbeat& InMessage, const TSharedRef<IMessageContext>& InContext);

		/** Handles endpoints shutdown */
		void OnShutdown(const FDCMMessage_Shutdown& InMessage, const TSharedRef<IMessageContext>& InContext);

		/** Executes console commands locally */
		void OnExecuteConsoleCommand(const FDCMMessage_ExecuteConsoleCommand& InMessage, const TSharedRef<IMessageContext>& InContext);

		/** Handles all user messages */
		void OnUserMessages(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);

	private:

		/** Helper to update the last activity time of an endpoint with the specified address */
		void UpdateLastActivityTime(const FMessageAddress& InAddress)
		{
			if (FDCEndpoint* Endpoint = GetEndpoint(InAddress))
			{
				UpdateLastActivityTime(*Endpoint);
			}
		}

		/** Helper to update the last activity time of the specified endpoint */
		void UpdateLastActivityTime(FDCEndpoint& InEndpoint)
		{
			InEndpoint.LastActivityTime = GetLastActivityTime();
		}

		/** Activity time provider */
		static double GetLastActivityTime()
		{
			return FApp::GetCurrentTime();
		}

	private:

		/** Whether this messenenger is currently active */
		bool bIsRunning = false;

		/** Endpoint information of this messenger instance */
		FDCEndpoint ThisEndpoint;

		/** Residence of this UE process */
		static TOptional<FDCMData_ResidenceDescriptor> ThisResidence;

		/** MessageBus message endpoint */
		TSharedPtr<FMessageEndpoint> MessageEndpoint;

		/** All discovered and currently online endpoints */
		TArray<FDCEndpoint> DiscoveredEndpoints;

		/** Message type bound user delegates */
		TMap<UScriptStruct*, TSharedPtr<FCustomMessageEventBase>> UserMessageEvents;

		/** Custom Tick() delegate handle */
		FTSTicker::FDelegateHandle TickerHandle;

		/** Last time we sent hearbeat pulse */
		double LastHeartbeatPulseTime = 0.f;

		/** Heartbeat pulse counter. Increments every time with the pulse. */
		uint32 HeartbeatPulseCounter = 0;
	};
}
