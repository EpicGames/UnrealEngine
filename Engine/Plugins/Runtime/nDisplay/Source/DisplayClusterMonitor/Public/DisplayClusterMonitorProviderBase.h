// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DisplayClusterMonitorTypes.h"
#include "DisplayClusterMonitorMessenger.h"


namespace UE::nDisplay::Monitor
{
	class FDCMessenger;

	/**
	 * Base class for cluster observable providers
	 */
	class DISPLAYCLUSTERMONITOR_API FDisplayClusterMonitorProviderBase
	{
	public:

		FDisplayClusterMonitorProviderBase();
		virtual ~FDisplayClusterMonitorProviderBase() = default;

	public:

		/** Start provider activity */
		virtual bool Start();

		/** Stop provider activity */
		virtual void Stop();

		/**
		 * Get messenger name to use during messenger initialization
		 * 
		 * @return - Messenger name
		 */
		virtual FString GetMessengerName() const = 0;

	protected:

		/** The Message Bus messenger */
		TUniquePtr<FDCMessenger> Messenger;
	};
}
