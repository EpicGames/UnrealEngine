// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"

#include "DisplayClusterMonitorSettings.generated.h"


/**
 * Cluster monitor settings
 * 
 * Contains both editor only and runtime cluster monitor settings
 */
UCLASS(Config = Game, DefaultConfig)
class DISPLAYCLUSTERMONITOR_API UDisplayClusterMonitorSettings
	: public UObject
{
	GENERATED_BODY()

public:

	/**
	 * Heartbeat pulse interval for the messenger. Used to detect unresponsive or crashed nodes.
	 */
	UPROPERTY(Config, EditAnywhere, Category = "Runtime", meta = (ClampMin = 0.5f, UIMin = 0.5f, ClampMax = 10.f, UIMax = 10.f))
	float HeartbeatInterval = 3.f;

	/**
	 * Maximum time, in seconds, without receiving any messages, before an endpoint is considered unresponsive.
	 */
	UPROPERTY(Config, EditAnywhere, Category = "Runtime", meta = (ClampMin = 10.f, UIMin = 10.f, ClampMax = 30.f, UIMax = 30.f))
	float UnresponsiveTimeThreshold = 15.f;


#if WITH_EDITORONLY_DATA

	/**
	 * When enabled, new observable viewports appear at the very beginning of the viewport
	 * layout. Otherwise, they're added to the end.
	 */
	UPROPERTY(Config, EditAnywhere, Category = "UI")
	bool bAddNewObservablesToFront = true;

	/**
	* Determines the number of viewports displayed per row
	*/
	UPROPERTY(Config, EditAnywhere, Category = "UI", meta = (UIMin = 1, ClampMin = 1, UIMax = 8, ClampMax = 8))
	int32 ViewportsInRow = 2;

	/**
	 * Whether viewports have fixed height
	 */
	UPROPERTY(Config, EditAnywhere, Category = "UI")
	bool bUseFixedViewportHeight = true;

	/**
	 * The height of viewports when fixed height is used
	 */
	UPROPERTY(Config, EditAnywhere, Category = "UI")
	float FixedViewportHeight = 500;

#endif // WITH_EDITORONLY_DATA
};
