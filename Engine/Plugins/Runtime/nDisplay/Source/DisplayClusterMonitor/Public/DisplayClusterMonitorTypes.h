// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IMessageContext.h"
#include "Misc/Guid.h"
#include "UObject/ObjectMacros.h"

#include "DisplayClusterMonitorTypes.generated.h"


/**
 * Roles used in the cluster monitor communication network
 */
UENUM(BlueprintType)
enum class EDCMessengerRole : uint8
{
	/** A provider of observables data */
	ObservablesProvider,
	/** A cluster monitor */
	Monitor,
};


/**
 * Types of resources that can be observed from outside.
 */
UENUM(BlueprintType)
enum class EDCObservableType : uint8
{
	/** Invalid/uninitialized type for validation */
	None,
	/** Node backbuffer */
	Backbuffer,
	/** UI only */
	UI,
	/** Outer viewport */
	Viewport,
	/** Inner camera viewport */
	ICVFXCamera,
	/** Inner camera tile */
	ICVFXCameraTile,
};


/**
 * Observable control command types
 */
UENUM(BlueprintType)
enum class EDCControlCommand : uint8
{
	None,
	Play,
	Pause,
	Stop,
};


/**
 * Request operation result types
 */
UENUM(BlueprintType)
enum class EDCRequestResult : uint8
{
	NoResult,
	Ok,
	Fail,
};


/**
 * Residence of an observable
 * Contains information on a cluster node, and the machine where it's running on
 */
USTRUCT(BlueprintType)
struct DISPLAYCLUSTERMONITOR_API FDCMData_ResidenceDescriptor
{
	GENERATED_BODY()

public:

	/** GUID of a cluster that this cluster node belongs to */
	UPROPERTY()
	FGuid ClusterId;

	/** GUID of a cluster node */
	UPROPERTY()
	FGuid NodeId;

	/** Cluster node ID */
	UPROPERTY()
	FString NodeName;

	/** Hostname of a machine where this cluster node is running */
	UPROPERTY()
	FString Hostname;

	/** Whether this cluster node is primary */
	UPROPERTY()
	bool bIsPrimary = false;

	/** Whether this cluster node is running offscreen */
	UPROPERTY()
	bool bIsOffscreen = false;
};


/**
 * Descriptor of a cluster monitor endpoint
 */
USTRUCT(BlueprintType)
struct DISPLAYCLUSTERMONITOR_API FDCMData_EndpointDescriptor
{
	GENERATED_BODY()

public:

	/** GUID of this endpoint */
	UPROPERTY()
	FGuid Id;

	/** Name of this endpoint */
	UPROPERTY()
	FString Name;

	/** Set of roles that this endpoint is responsible for */
	UPROPERTY()
	TSet<EDCMessengerRole> Roles;
};


/**
 * Endpoint information
 */
USTRUCT(BlueprintType)
struct DISPLAYCLUSTERMONITOR_API FDCEndpoint
{
	GENERATED_BODY()

public:

	/** Endpoint residence */
	UPROPERTY()
	FDCMData_ResidenceDescriptor Residence;

	/** Endpoint descriptor */
	UPROPERTY()
	FDCMData_EndpointDescriptor Endpoint;

	/** Endpoint address (local data only) */
	FMessageAddress Address;

	/** Shows how long ago this endpoint was last active (local data only) */
	double LastActivityTime = 0;
};


/**
 * Observable resource information
 */
USTRUCT()
struct DISPLAYCLUSTERMONITOR_API FDCMData_ObservableInfo
{
	GENERATED_BODY()

	inline static const FIntPoint InvalidTilePos = { INDEX_NONE, INDEX_NONE };

public:

	/** Observable type */
	UPROPERTY()
	EDCObservableType Type = EDCObservableType::None;

	/** Observable GUID */
	UPROPERTY()
	FGuid Id;

	/** Observable name */
	UPROPERTY()
	FString Name;

	/** Observable resolution */
	UPROPERTY()
	FIntPoint Resolution = FIntPoint::ZeroValue;

	/** Observable parent name (optional, used for tiles only) */
	UPROPERTY()
	FString ParentName;

	/** Observable tile position (optional, used for tiles only) */
	UPROPERTY()
	FIntPoint TilePos = InvalidTilePos;

public:

	/** Validates the major data */
	bool IsValid() const
	{
		return Type != EDCObservableType::None && Id.IsValid() && !Name.IsEmpty() && Resolution.GetMin() > 0;
	}
};


/**
 * Cluster node information
 */
USTRUCT()
struct DISPLAYCLUSTERMONITOR_API FDCMData_NodeObservables
{
	GENERATED_BODY()

public:

	/** New sources */
	UPROPERTY()
	TArray<FDCMData_ObservableInfo> ObservablesAdded;

	/** Sources that have been modified */
	UPROPERTY()
	TArray<FDCMData_ObservableInfo> ObservablesUpdated;

	/** Sources not available anymore */
	UPROPERTY()
	TArray<FDCMData_ObservableInfo> ObservablesRemoved;

	/** Sources that keep running unchanged since last evaluation */
	UPROPERTY()
	TArray<FDCMData_ObservableInfo> ObservablesUnchanged;
};


/**
 * Base message type for any cluster monitor communication messages
 */
USTRUCT(BlueprintType)
struct DISPLAYCLUSTERMONITOR_API FDCMMessageBase
{
	GENERATED_BODY()
};


/**
 * Discovery request
 */
USTRUCT(BlueprintType)
struct DISPLAYCLUSTERMONITOR_API FDCMMessage_DiscoveryRequest
	: public FDCMMessageBase
{
	GENERATED_BODY()

	/** Residence of the endpoint that is sending this discovery request */
	UPROPERTY()
	FDCMData_ResidenceDescriptor Residence;

	/** Endpoint information of the sender */
	UPROPERTY()
	FDCMData_EndpointDescriptor Endpoint;
};


/**
 * Discovery response
 */
USTRUCT(BlueprintType)
struct DISPLAYCLUSTERMONITOR_API FDCMMessage_DiscoveryResponse
	: public FDCMMessageBase
{
	GENERATED_BODY()

	/** Residence of the endpoint that is sending this response */
	UPROPERTY()
	FDCMData_ResidenceDescriptor Residence;

	/** Endpoint information of the responder */
	UPROPERTY()
	FDCMData_EndpointDescriptor Endpoint;
};


/**
 * Heartbeat pulse message
 */
USTRUCT(BlueprintType)
struct DISPLAYCLUSTERMONITOR_API FDCMMessage_Heartbeat
	: public FDCMMessageBase
{
	GENERATED_BODY()
};


/**
 * Shutdown notification
 */
USTRUCT(BlueprintType)
struct DISPLAYCLUSTERMONITOR_API FDCMMessage_Shutdown
	: public FDCMMessageBase
{
	GENERATED_BODY()

	/** Shutdown reason info (optional) */
	UPROPERTY()
	FString Reason;
};


/**
 *
 */
USTRUCT(BlueprintType)
struct DISPLAYCLUSTERMONITOR_API FDCMMessage_ExecuteConsoleCommand
	: public FDCMMessageBase
{
	GENERATED_BODY()

	/** Command executor to address the command to */
	UPROPERTY()
	FString ExecutorName;

	/** Console command to run on the receiver side */
	UPROPERTY()
	FString Command;
};



/**
 * Base message type for any custom messages
 */
USTRUCT(BlueprintType)
struct DISPLAYCLUSTERMONITOR_API FDCMMessage
	: public FDCMMessageBase
{
	GENERATED_BODY()
};


/**
 * Request to get information aobut all available observables
 */
USTRUCT(BlueprintType)
struct DISPLAYCLUSTERMONITOR_API FDCMMessage_NodeObservablesRequest
	: public FDCMMessage
{
	GENERATED_BODY()
};


/**
 * Response with the information about all available observables
 */
USTRUCT(BlueprintType)
struct DISPLAYCLUSTERMONITOR_API FDCMMessage_NodeObservablesResponse
	: public FDCMMessage
{
	GENERATED_BODY()

	/** Observables information */
	UPROPERTY()
	FDCMData_NodeObservables Observables;
};


/**
 * Broadcast notification with the information about all available observables
 */
USTRUCT(BlueprintType)
struct DISPLAYCLUSTERMONITOR_API FDCMMessage_NodeObservablesNotification
	: public FDCMMessage
{
	GENERATED_BODY()

	/** Observables information */
	UPROPERTY()
	FDCMData_NodeObservables Observables;
};


/**
 * Request to start new observation session
 */
USTRUCT(BlueprintType)
struct DISPLAYCLUSTERMONITOR_API FDCMMessage_StartSessionRequest
	: public FDCMMessage
{
	GENERATED_BODY()

	/** Observable GUID */
	UPROPERTY()
	FGuid ObservableId;
};


/**
 * Observation start response/result
 */
USTRUCT(BlueprintType)
struct DISPLAYCLUSTERMONITOR_API FDCMMessage_StartSessionResponse
	: public FDCMMessage
{
	GENERATED_BODY()

	/** Observable GUID */
	UPROPERTY()
	FGuid ObservableId;

	/** Request result */
	UPROPERTY()
	EDCRequestResult Result = EDCRequestResult::NoResult;
};


/**
 * Request to stop an observation session
 */
USTRUCT(BlueprintType)
struct DISPLAYCLUSTERMONITOR_API FDCMMessage_StopSessionRequest
	: public FDCMMessage
{
	GENERATED_BODY()

	/** Observable GUID */
	UPROPERTY()
	FGuid ObservableId;
};


/**
 * Observation stop response/result
 */
USTRUCT(BlueprintType)
struct DISPLAYCLUSTERMONITOR_API FDCMMessage_StopSessionResponse
	: public FDCMMessage
{
	GENERATED_BODY()

	/** Observable GUID */
	UPROPERTY()
	FGuid ObservableId;

	/** Request result */
	UPROPERTY()
	EDCRequestResult Result = EDCRequestResult::NoResult;
};


/**
 * Observation session control command request
 */
USTRUCT(BlueprintType)
struct DISPLAYCLUSTERMONITOR_API FDCMMessage_ObservableControlRequest
	: public FDCMMessage
{
	GENERATED_BODY()

	/** Observable GUID */
	UPROPERTY()
	FGuid ObservableId;

	/** Control command */
	UPROPERTY()
	EDCControlCommand Command = EDCControlCommand::None;
};


/**
 * Observation session control response
 */
USTRUCT(BlueprintType)
struct DISPLAYCLUSTERMONITOR_API FDCMMessage_ObservableControlResponse
	: public FDCMMessage
{
	GENERATED_BODY()

	/** Observable GUID */
	UPROPERTY()
	FGuid ObservableId;

	/** Control command */
	UPROPERTY()
	EDCControlCommand Command = EDCControlCommand::None;

	/** Request result */
	UPROPERTY()
	EDCRequestResult Result = EDCRequestResult::NoResult;
};
