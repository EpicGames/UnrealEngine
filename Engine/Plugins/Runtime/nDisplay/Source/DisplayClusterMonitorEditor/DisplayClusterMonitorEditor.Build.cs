// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class DisplayClusterMonitorEditor : ModuleRules
	{
		public DisplayClusterMonitorEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"DisplayClusterMedia",
					"DisplayClusterMonitor",
					"EditorStyle",
					"EditorWidgets",
					"InputCore",
					"LevelEditor",
					"Media",
					"MediaAssets",
					"MediaPlayerEditor",
					"NDIMedia",
					"OutputLog",
					"Projects",
					"Slate",
					"SlateCore",
					"ToolMenus",
					"ToolWidgets",
					"UnrealEd",
					"WorkspaceMenuStructure",
				});
		}
	}
}
