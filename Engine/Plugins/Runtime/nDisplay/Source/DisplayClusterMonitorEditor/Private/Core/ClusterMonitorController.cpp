// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/ClusterMonitorController.h"

#include "Containers/Ticker.h"
#include "Core/ClusterObservable.h"
#include "Core/ClusterResidence.h"
#include "DCMonitorEditorLog.h"
#include "DisplayClusterMonitorMessenger.h"
#include "Math/UnrealMathUtility.h"
#include "UObject/StructOnScope.h"


FClusterMonitorController::FClusterMonitorController()
	: Messenger(MakeShared<FDCMessenger>())
{
}

bool FClusterMonitorController::StartCommunication()
{
	// Subsribe for endpoints discovery events
	Messenger->OnEndpointJoined.AddRaw(this,  &FClusterMonitorController::OnEndpointJoined);
	Messenger->OnEndpointLeft.AddRaw(this,    &FClusterMonitorController::OnEndpointLeft);
	Messenger->OnEndpointTimeout.AddRaw(this, &FClusterMonitorController::OnEndpointTimeout);

	// Subsribe for monitor events
	Messenger->OnMessage<FDCMMessage_NodeObservablesResponse>().AddSP(this, &FClusterMonitorController::OnObservablesInfoResponse);
	Messenger->OnMessage<FDCMMessage_NodeObservablesNotification>().AddSP(this, &FClusterMonitorController::OnObservablesInfoNotification);

	// Start messenger as a monitor
	const FString MessengerName = TEXT("DCMonitor");
	const bool bStarted = Messenger->Start(MessengerName, { EDCMessengerRole::Monitor });

	UE_LOGF(LogClusterMonitorEditor, Log, "Cluster monitor messenger '%ls' %ls",
		*MessengerName, bStarted ? TEXT("has started") : TEXT("could not start"));

	return bStarted;
}

void FClusterMonitorController::StopCommunication()
{
	// Stop messenger
	Messenger->Stop();

	// Unsubscribe from endpoint discovery events
	Messenger->OnEndpointJoined.RemoveAll(this);
	Messenger->OnEndpointLeft.RemoveAll(this);
	Messenger->OnEndpointTimeout.RemoveAll(this);

	// Unsubscribe from monitor events
	Messenger->OnMessage<FDCMMessage_NodeObservablesResponse>().RemoveAll(this);
	Messenger->OnMessage<FDCMMessage_NodeObservablesNotification>().RemoveAll(this);

	UE_LOGF(LogClusterMonitorEditor, Log, "Cluster monitor messenger has stopped");
}

TSharedPtr<IClusterObservable> FClusterMonitorController::GetObservable(const FGuid& InObservableGuid) const
{
	const TSharedRef<IClusterObservable>* FoundItem = Observables.Find(InObservableGuid);
	return FoundItem ? FoundItem->ToSharedPtr() : nullptr;
}

TSharedPtr<IClusterResidence> FClusterMonitorController::GetResidence(const FGuid& InObservableGuid) const
{
	const TSharedPtr<IClusterObservable> Observable = GetObservable(InObservableGuid);
	return Observable.IsValid() ? Observable->GetResidence().ToSharedPtr() : nullptr;
}

int32 FClusterMonitorController::GetObservablesNum() const
{
	return Observables.Num();
}

int32 FClusterMonitorController::GetActiveSessionsNum() const
{
	return ActiveSessions.Num();
}

int32 FClusterMonitorController::GetUnresponsiveNodesNum() const
{
	return UnresponsiveObservables.Num();
}

void FClusterMonitorController::Rescan()
{
	// Stop all active sessions
	RequestAllSessionsStop();

	// Stop messaging
	StopCommunication();

	// Remove all observables
	TSet<FGuid> ObservableIds;
	Observables.GetKeys(ObservableIds);
	for (const FGuid& ObservableId : ObservableIds)
	{
		RemoveObservable(ObservableId);
	}

	// Reset internal data
	Clusters.Empty();
	Residences.Empty();
	ObservableIds.Empty();
	ActiveSessions.Empty();
	UnresponsiveObservables.Empty();

	// Start messaging and discovery again after a small delay.
	// The delay is required to let Slate tick, and therefore
	// handle the updates above on GUI side (widgets).
	constexpr float RestartCommunicationDelay = 0.5f;
	FTSTicker::GetCoreTicker().AddTicker(
		FTickerDelegate::CreateLambda([This = AsShared().ToWeakPtr()](float)
			{
				TSharedPtr<FClusterMonitorController> PinnedThis = This.Pin();
				if (PinnedThis.IsValid())
				{
					PinnedThis->StartCommunication();
				}

				return false;
			}),
		RestartCommunicationDelay);
}

void FClusterMonitorController::ClearUnresponsiveEndpoints()
{
	// Forget all unresponsive observables
	for (auto It = UnresponsiveObservables.CreateIterator(); It; ++It)
	{
		// Stop session if stil alive
		RequestSessionStop(It.Key());

		// Forget about this observable
		Observables.Remove(It.Key());
		OnObservableLeft().Broadcast(It.Value(), TEXT("CleanUnresponsiveRequest"));
		It.RemoveCurrent();
	}
}

void FClusterMonitorController::RequestSessionStart(const FGuid& InObservableGuid)
{
	// Check if requested observable has not started a session already
	if (ActiveSessions.Contains(InObservableGuid))
	{
		return;
	}

	// Get the requested observable
	TSharedPtr<IClusterObservable> Observable = GetObservable(InObservableGuid);
	if (!Observable.IsValid())
	{
		return;
	}

	TSharedRef<IClusterObservable> ObservableRef = Observable.ToSharedRef();

	// Session pre-start notification
	SessionStartRequestedEvent.Broadcast(ObservableRef);

	// Remember this session
	ActiveSessions.Add(Observable->GetId(), ObservableRef);

	// Start session
	ObservableRef->StartSession();

	// Session started notification
	SessionStartedEvent.Broadcast(ObservableRef);
}

void FClusterMonitorController::RequestSessionStop(const FGuid& InObservableGuid)
{
	// Check if requested observable is actually running a session
	if (!ActiveSessions.Contains(InObservableGuid))
	{
		return;
	}

	// Get the requested observable
	TSharedPtr<IClusterObservable> Observable = GetObservable(InObservableGuid);
	if (!Observable.IsValid())
	{
		return;
	}

	TSharedRef<IClusterObservable> ObservableRef = Observable.ToSharedRef();

	// Session pre-stop notification
	SessionStopRequestedEvent.Broadcast(ObservableRef);

	// Remove from active sessions
	ActiveSessions.Remove(Observable->GetId());

	// Stop session
	Observable->StopSession();

	// Session stopped notification
	SessionStoppedEvent.Broadcast(Observable.ToSharedRef());
}

void FClusterMonitorController::RequestAllSessionsStop()
{
	TArray<FGuid> ActiveSessionGuids;
	ActiveSessions.GenerateKeyArray(ActiveSessionGuids);
	
	// Iterate and stop every active session
	for (const FGuid& ObservableId : ActiveSessionGuids)
	{
		RequestSessionStop(ObservableId);
	}
}

TSharedRef<IClusterResidence> FClusterMonitorController::FindOrCreateNodeResidence(const FDCMData_ResidenceDescriptor& InResidence)
{
	// If already exists, return it
	TSharedRef<IClusterResidence>* FoundExisting = Residences.Find(InResidence.NodeId);
	if (FoundExisting)
	{
		return *FoundExisting;
	}

	// Otherwise, create a new one. By default, mark it as 'Online'
	TSharedRef<IClusterResidence>& NewResidence = Residences.Emplace(InResidence.NodeId, MakeShared<FClusterResidence>(InResidence));
	NewResidence->SetConnectionState(IClusterResidence::EConnectionState::Online);

	// Store new entity
	Clusters.Add(NewResidence->GetClusterId());

	return NewResidence;
}

void FClusterMonitorController::RemoveNodeResidence(const FDCMData_ResidenceDescriptor& InResidence)
{
	Residences.Remove(InResidence.NodeId);
}

TSharedRef<IClusterObservable> FClusterMonitorController::CreateObservable(const FDCMData_ResidenceDescriptor& InResidence, const FDCMData_ObservableInfo& InObservableInfo)
{
	TSharedRef<IClusterResidence> Residence = FindOrCreateNodeResidence(InResidence);
	return CreateObservable(Residence, InObservableInfo);
}

TSharedRef<IClusterObservable> FClusterMonitorController::CreateObservable(const TSharedRef<IClusterResidence>& InResidence, const FDCMData_ObservableInfo& InObservableInfo)
{
	TSharedRef<IClusterObservable>& NewObservable = Observables.Emplace(InObservableInfo.Id, MakeShared<FClusterObservable>(InResidence, InObservableInfo, AsWeak()));
	OnObservableJoined().Broadcast(NewObservable);
	return NewObservable;
}

void FClusterMonitorController::RemoveObservable(const FGuid& InObservableId)
{
	// Make sure the requested observable exists
	TSharedRef<IClusterObservable>* ObservableToDeletePtr = Observables.Find(InObservableId);
	if (!ObservableToDeletePtr)
	{
		return;
	}

	TSharedRef<IClusterObservable> ObservableToDelete = *ObservableToDeletePtr;

	// Make sure there are no active sessions remain
	if (ObservableToDelete->IsSessionRunning())
	{
		ObservableToDelete->StopSession();
	}

	// Remove it from the internal data
	Observables.Remove(InObservableId);
	ActiveSessions.Remove(InObservableId);
	UnresponsiveObservables.Remove(InObservableId);

	// Notify the listeners
	OnObservableLeft().Broadcast(ObservableToDelete, FString());
}

void FClusterMonitorController::UpdateObsrevable(const FDCMData_ResidenceDescriptor& InResidence, const FDCMData_ObservableInfo& InObservableInfo)
{
	// Make sure the requested observable exists
	TSharedRef<IClusterObservable>* ObservableToUpdate = Observables.Find(InObservableInfo.Id);
	if (!ObservableToUpdate)
	{
		CreateObservable(InResidence, InObservableInfo);
		return;
	}

	// Test if update is required
	const bool bHasAnyUpdates = (*ObservableToUpdate)->HasAnyUpdates(InObservableInfo);
	if (bHasAnyUpdates)
	{
		// Update the obsevable entity
		(*ObservableToUpdate)->Update(InObservableInfo);
		// Fire the update notification
		OnObservableUpdated().Broadcast(*ObservableToUpdate);
	}
}

void FClusterMonitorController::CheckObservable(const FDCMData_ResidenceDescriptor& InResidence, const FDCMData_ObservableInfo& InObservableInfo)
{
	// Create if not exists
	if (!Observables.Contains(InObservableInfo.Id))
	{
		CreateObservable(InResidence, InObservableInfo);
	}
}

void FClusterMonitorController::OnEndpointJoined(const FDCEndpoint& InEndpoint)
{
	// Being a monitor, we only interested in the provder endpoints
	if (!InEndpoint.Endpoint.Roles.Contains(EDCMessengerRole::ObservablesProvider))
	{
		return;
	}

	UE_LOGF(LogClusterMonitorEditor, Log, "Endpoint joined: name=%ls, addr=%ls", *InEndpoint.Endpoint.Name, *InEndpoint.Address.ToString());

	// Create residency entity
	FindOrCreateNodeResidence(InEndpoint.Residence);

	// Also ask for the list of available observables
	Messenger->Send({ InEndpoint.Address }, FDCMMessage_NodeObservablesRequest{ });
}

void FClusterMonitorController::OnEndpointLeft(const FDCEndpoint& InEndpoint, const FString& InReason)
{
	// Make sure we're tracking this endpoint
	if (!InEndpoint.Endpoint.Roles.Contains(EDCMessengerRole::ObservablesProvider))
	{
		return;
	}

	UE_LOGF(LogClusterMonitorEditor, Log, "Endpoint left: name=%ls, addr=%ls", *InEndpoint.Endpoint.Name, *InEndpoint.Address.ToString());

	// Remove all observables bound to the endpoint that has left
	for (auto It = Observables.CreateIterator(); It; ++It)
	{
		if (It.Value()->GetResidence()->GetNodeId() == InEndpoint.Residence.NodeId)
		{
			const TSharedRef<IClusterObservable> ObservableLeft = It.Value();

			// Stop session if it's running currently
			RequestSessionStop(ObservableLeft->GetId());
			// Remove this observable
			It.RemoveCurrent();
			// Fire observable left notification
			OnObservableLeft().Broadcast(ObservableLeft, InReason);
		}
	}

	// Also, remove this residence
	RemoveNodeResidence(InEndpoint.Residence);
}

void FClusterMonitorController::OnEndpointTimeout(const FDCEndpoint& InEndpoint)
{
	// Make sure we're tracking this endpoint
	if (!InEndpoint.Endpoint.Roles.Contains(EDCMessengerRole::ObservablesProvider))
	{
		return;
	}

	// Update the residence status
	const TSharedRef<IClusterResidence> Residence = FindOrCreateNodeResidence(InEndpoint.Residence);
	Residence->SetConnectionState(IClusterResidence::EConnectionState::Timeout);

	UE_LOGF(LogClusterMonitorEditor, Log, "Endpoint timeout: name=%ls, addr=%ls", *InEndpoint.Endpoint.Name, *InEndpoint.Address.ToString());

	// Iterate through all observables of this endpoint
	for (auto It = Observables.CreateIterator(); It; ++It)
	{
		if (It.Value()->GetResidence()->GetNodeId() == InEndpoint.Residence.NodeId)
		{
			// Stop session if it's running currently
			TSharedRef<IClusterObservable> Observable = It.Value();
			RequestSessionStop(Observable->GetId());

			// Remember this as unresponsive
			UnresponsiveObservables.Emplace(Observable->GetId(), Observable);

			// Fire a notification
			OnObservableTimeout().Broadcast(It.Value());
		}
	}
}

void FClusterMonitorController::HandleObservablesInfo(const FDCEndpoint& InEndpoint, const FDCMData_NodeObservables& InNodeObservables)
{
	// Remove observables that are reported as 'Removed'
	for (const FDCMData_ObservableInfo& ObservableInfo : InNodeObservables.ObservablesRemoved)
	{
		RemoveObservable(ObservableInfo.Id);
	}

	// Update observables that are reported as 'Updated'
	for (const FDCMData_ObservableInfo& ObservableInfo : InNodeObservables.ObservablesUpdated)
	{
		UpdateObsrevable(InEndpoint.Residence, ObservableInfo);
	}

	// Create new observables
	for (const FDCMData_ObservableInfo& ObservableInfo : InNodeObservables.ObservablesAdded)
	{
		CreateObservable(InEndpoint.Residence, ObservableInfo);
	}

	// Check if these observables exist. If no, create corresponding entities.
	for (const FDCMData_ObservableInfo& ObservableInfo : InNodeObservables.ObservablesUnchanged)
	{
		CheckObservable(InEndpoint.Residence, ObservableInfo);
	}
}

void FClusterMonitorController::OnObservablesInfoResponse(const FDCEndpoint& InEndpoint, const FDCMMessage_NodeObservablesResponse& InResponseMsg)
{
	UE_LOGF(LogClusterMonitorEditor, Verbose, "Got data response from: name=%ls, addr=%ls", *InEndpoint.Endpoint.Name, *InEndpoint.Address.ToString());
	HandleObservablesInfo(InEndpoint, InResponseMsg.Observables);
}

void FClusterMonitorController::OnObservablesInfoNotification(const FDCEndpoint& InEndpoint, const FDCMMessage_NodeObservablesNotification& InResponseMsg)
{
	UE_LOGF(LogClusterMonitorEditor, Verbose, "Got data notification from: name=%ls, addr=%ls", *InEndpoint.Endpoint.Name, *InEndpoint.Address.ToString());
	HandleObservablesInfo(InEndpoint, InResponseMsg.Observables);
}
