// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/IClusterMonitorController.h"
#include "DisplayClusterMonitorTypes.h"

class IClusterObservable;
class IClusterResidence;
struct FGuid;

namespace UE::nDisplay::Monitor
{
	class FDCMessenger;
}


/**
 * Cluster monitor controller implementation
 * 
 * Responsible for acquiring data from outside (observables),
 * and propagate it among internal consumers (e.g. GUI panels).
 * It also provides the API for obsrvation session management.
 */
class FClusterMonitorController
	: public IClusterMonitorController
	, public TSharedFromThis<FClusterMonitorController>
{
	using FDCMessenger = UE::nDisplay::Monitor::FDCMessenger;

public:

	FClusterMonitorController();
	virtual ~FClusterMonitorController() override = default;


	//~ Begin IClusterMonitorController

public:

	virtual TSharedRef<UE::nDisplay::Monitor::FDCMessenger> GetMessenger() override
	{
		return Messenger;
	}

	virtual bool StartCommunication() override;
	virtual void StopCommunication() override;

public:

	virtual TSharedPtr<IClusterObservable> GetObservable(const FGuid& InObservableGuid) const override;
	virtual TSharedPtr<IClusterResidence> GetResidence(const FGuid& InObservableGuid) const override;

	virtual int32 GetObservablesNum() const override;
	virtual int32 GetActiveSessionsNum() const override;
	virtual int32 GetUnresponsiveNodesNum() const override;

public:

	virtual void Rescan() override;
	virtual void ClearUnresponsiveEndpoints() override;

	virtual void RequestSessionStart(const FGuid& InObservableGuid) override;
	virtual void RequestSessionStop(const FGuid& InObservableGuid) override;
	virtual void RequestAllSessionsStop() override;

public:

	virtual FObservableJoinedEvent& OnObservableJoined() override
	{
		return ObservableJoinedEvent;
	}

	virtual FObservableUpdatedEvent& OnObservableUpdated() override
	{
		return ObservableUpdatedEvent;
	}

	virtual FObservableLeftEvent& OnObservableLeft() override
	{
		return ObservableLeftEvent;
	}

	virtual FObservableTimeoutEvent& OnObservableTimeout() override
	{
		return ObservableTimeoutEvent;
	}

public:

	virtual FSessionStartRequestedEvent& OnSessionStartRequested() override
	{
		return SessionStartRequestedEvent;
	}

	virtual FSessionStopRequestedEvent& OnSessionStopRequested() override
	{
		return SessionStopRequestedEvent;
	}

	virtual FSessionStartedEvent& OnSessionStarted() override
	{
		return SessionStartedEvent;
	}

	virtual FSessionStoppedEvent& OnSessionStopped() override
	{
		return SessionStoppedEvent;
	}

	//~ End IClusterMonitorController

private:

	/** Finds existing, or creates a new residence entity for local tracking based on the residence descriptor */
	TSharedRef<IClusterResidence> FindOrCreateNodeResidence(const FDCMData_ResidenceDescriptor& InResidence);

	/** Remove specified residence */
	void RemoveNodeResidence(const FDCMData_ResidenceDescriptor& InResidence);

	/** Creates new observable entity with the residence descriptor */
	TSharedRef<IClusterObservable> CreateObservable(const FDCMData_ResidenceDescriptor& InResidence, const FDCMData_ObservableInfo& InObservableInfo);

	/** Creates new observable entity with an existing residence entity */
	TSharedRef<IClusterObservable> CreateObservable(const TSharedRef<IClusterResidence>& InResidence, const FDCMData_ObservableInfo& InObservableInfo);

	/** Removes specified observable from internal data */
	void RemoveObservable(const FGuid& ObservableId);

	/** Updates the requested observable entity from the original source */
	void UpdateObsrevable(const FDCMData_ResidenceDescriptor& InResidence, const FDCMData_ObservableInfo& InObservableInfo);

	/** Checks if an observable entity exists for a given source data. If not, creates it. */
	void CheckObservable(const FDCMData_ResidenceDescriptor& InResidence, const FDCMData_ObservableInfo& InObservableInfo);

private:

	/** Handles notification that a new endpoint was discovered */
	void OnEndpointJoined(const FDCEndpoint& InEndpoint);

	/** Handles notification that an endpoint was shut down */
	void OnEndpointLeft(const FDCEndpoint& InEndpoint, const FString& InReason);

	/** Handles notification that specified endpoint is now considered unresponsive (timeout) */
	void OnEndpointTimeout(const FDCEndpoint& InEndpoint);

private:

	/** Processes observable information that we get from various providers */
	void HandleObservablesInfo(const FDCEndpoint& InEndpoint, const FDCMData_NodeObservables& InNodeObservables);

	/** Handles responses to requests for node observables info */
	void OnObservablesInfoResponse(const FDCEndpoint& InEndpoint, const FDCMMessage_NodeObservablesResponse& InResponseMsg);

	/** Handles notifications about changes in node observables */
	void OnObservablesInfoNotification(const FDCEndpoint& InEndpoint, const FDCMMessage_NodeObservablesNotification& InResponseMsg);

private:

	/** Discovered cluster GUIDs */
	TSet<FGuid> Clusters;

	/** Discovered cluster node GUIDs(residences) */
	TMap<FGuid, TSharedRef<IClusterResidence>> Residences;

	/** Discovered observable entities */
	TMap<FGuid, TSharedRef<IClusterObservable>> Observables;

	/** Observable entities that are considered unresponsive */
	TMap<FGuid, TSharedRef<IClusterObservable>> UnresponsiveObservables;

	/** Observable entities that have active seessions */
	TMap<FGuid, TSharedRef<IClusterObservable>> ActiveSessions;

	/** MessageBus messenger */
	TSharedRef<FDCMessenger> Messenger;

private:

	/** Observable event: New observable has been discovered */
	FObservableJoinedEvent ObservableJoinedEvent;
	/** Observable event: An observable has been updated */
	FObservableUpdatedEvent ObservableUpdatedEvent;
	/** Observable event: An observable has left */
	FObservableLeftEvent ObservableLeftEvent;
	/** Observable event: An observable is considered unresponsive */
	FObservableTimeoutEvent ObservableTimeoutEvent;

private:

	/** Session event: New session start requested */
	FSessionStartRequestedEvent SessionStartRequestedEvent;
	/** Session event: Session started */
	FSessionStartedEvent SessionStartedEvent;
	/** Session event: Session stop requested */
	FSessionStopRequestedEvent SessionStopRequestedEvent;
	/** Session event: Session stopped */
	FSessionStoppedEvent SessionStoppedEvent;
};
