// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/ClusterObservable.h"
#include "Core/IClusterMonitorController.h"
#include "Core/IClusterResidence.h"
#include "UObject/Class.h"

#include "DCMonitorEditorLog.h"
#include "DCMonitorEditorStyle.h"
#include "DisplayClusterMonitorMessenger.h"
#include "DisplayClusterMonitorSettings.h"
#include "DisplayClusterMonitorTypes.h"
#include "IMediaEventSink.h"
#include "MediaPlayer.h"
#include "MediaTexture.h"
#include "NDIMediaSource.h"


FClusterObservable::FClusterObservable(
	const TSharedRef<IClusterResidence>& InResidence,
	const FDCMData_ObservableInfo& InObservableInfo,
	TWeakPtr<IClusterMonitorController> InController
)
	: Controller(InController)
	, Residence(InResidence)
	, Id(InObservableInfo.Id)
	, Name(InObservableInfo.Name)
	, Type(InObservableInfo.Type)
	, Resolution(InObservableInfo.Resolution)
	, ParentName(InObservableInfo.ParentName.IsEmpty() ? TOptional<FString>() : InObservableInfo.ParentName)
	, TilePos(InObservableInfo.TilePos == FDCMData_ObservableInfo::InvalidTilePos ? TOptional<FIntPoint>() : InObservableInfo.TilePos)
{
	// Instantiate media source
	MediaSource = NewObject<UNDIMediaSource>();
	ConfigureMediaSource(Name);

	// Instantiate media player
	MediaPlayer = NewObject<UMediaPlayer>();
	MediaPlayer->PlayOnOpen = true;
	MediaPlayer->SetLooping(false);

	// Instantiate media texture
	MediaTexture = NewObject<UMediaTexture>();
	MediaTexture->AutoClear = true;
	MediaTexture->ClearColor = FLinearColor::Black;
	MediaTexture->NewStyleOutput = true;
	MediaTexture->SetMediaPlayer(MediaPlayer);
	MediaTexture->UpdateResource();
}

FClusterObservable::~FClusterObservable()
{
	if (IsValid(MediaPlayer))
	{
		MediaPlayer->OnMediaEvent().RemoveAll(this);
	}

	if (TSharedPtr<IClusterMonitorController> PinnedController = Controller.Pin())
	{
		PinnedController->GetMessenger()->OnMessage<FDCMMessage_StartSessionResponse>().RemoveAll(this);
		PinnedController->GetMessenger()->OnMessage<FDCMMessage_ObservableControlResponse>().RemoveAll(this);
	}
}

const FSlateBrush* FClusterObservable::GetDefaultIcon(EDCObservableType InObservableType)
{
	switch (InObservableType)
	{
	case EDCObservableType::Backbuffer:
		return FDCMonitorEditorStyle::Get().GetBrush("ClusterMonitor.TreeItem.Backbuffer");

	case EDCObservableType::UI:
		return FDCMonitorEditorStyle::Get().GetBrush("ClusterMonitor.TreeItem.UI");;

	case EDCObservableType::Viewport:
		return FDCMonitorEditorStyle::Get().GetBrush("ClusterMonitor.TreeItem.Viewport");

	case EDCObservableType::ICVFXCamera:
		return FDCMonitorEditorStyle::Get().GetBrush("ClusterMonitor.TreeItem.ICVFXCamera");

	case EDCObservableType::ICVFXCameraTile:
		return FDCMonitorEditorStyle::Get().GetBrush("ClusterMonitor.TreeItem.ICVFXCameraTile");

	case EDCObservableType::None:
	default:
		return nullptr;
	}
}

bool FClusterObservable::HasAnyUpdates(const FDCMData_ObservableInfo& InObservableInfo) const
{
	checkSlow(Id == InObservableInfo.Id);
	checkSlow(Name.Equals(InObservableInfo.Name, ESearchCase::IgnoreCase));

	// Compare to the original source
	const bool bSameResolution = (Resolution == InObservableInfo.Resolution);

	// Make conclusion
	const bool bUpdatesRequired = (!bSameResolution);

	return bUpdatesRequired;
}

void FClusterObservable::Update(const FDCMData_ObservableInfo& InObservableInfo)
{
	checkSlow(InObservableInfo.IsValid());
	checkSlow(Id == InObservableInfo.Id);
	checkSlow(Name.Equals(InObservableInfo.Name, ESearchCase::IgnoreCase));

	// Update local state
	Resolution = InObservableInfo.Resolution;

	// And notify the listeners
	OnObservableUpdated().Broadcast();
}

bool FClusterObservable::IsSessionRunning() const
{
	return SessionState != ESessionState::None;
}

void FClusterObservable::StartSession()
{
	// Nothing to do if already running
	const bool bHasStartedSession = IsSessionRunning();
	if (bHasStartedSession)
	{
		return;
	}

	// Starting... state
	SetSessionState(ESessionState::Transition);

	// Get the controller
	TSharedPtr<IClusterMonitorController> PinnedController = Controller.Pin();
	if (!PinnedController.IsValid())
	{
		SetSessionState(ESessionState::Error);
		return;
	}

	// Subscribe to the messenger events
	PinnedController->GetMessenger()->OnMessage<FDCMMessage_StartSessionResponse>().AddSP(this, &FClusterObservable::OnStartSessionResponse);
	PinnedController->GetMessenger()->OnMessage<FDCMMessage_ObservableControlResponse>().AddSP(this, &FClusterObservable::OnObservableControlResponse);

	// Subsribe to media player events
	MediaPlayer->OnMediaEvent().AddRaw(this, &FClusterObservable::OnMediaEvent);

	// Get our observable address (provider side)
	const FMessageAddress ObservableAddress = PinnedController->GetMessenger()->GetAddress(Residence->GetClusterId(), Residence->GetNodeId());
	if (!ObservableAddress.IsValid())
	{
		SetSessionState(ESessionState::Error);
		return;
	}

	// Send request to start session
	PinnedController->GetMessenger()->Send({ ObservableAddress },
		FDCMMessage_StartSessionRequest
		{
			.ObservableId = GetId(),
		});
}

void FClusterObservable::StopSession()
{
	// Nothing to do if no session running
	const bool bHasStartedSession = IsSessionRunning();
	if (!bHasStartedSession)
	{
		return;
	}

	// Reset state
	SetSessionState(ESessionState::None);

	// Get the controller
	TSharedPtr<IClusterMonitorController> PinnedController = Controller.Pin();
	if (!PinnedController.IsValid())
	{
		return;
	}

	// Unsubscribe from the messenger events
	PinnedController->GetMessenger()->OnMessage<FDCMMessage_StartSessionResponse>().RemoveAll(this);
	PinnedController->GetMessenger()->OnMessage<FDCMMessage_ObservableControlResponse>().RemoveAll(this);

	// Stop media player
	if (MediaPlayer)
	{
		MediaPlayer->OnMediaEvent().RemoveAll(this);
		MediaPlayer->Close();
	}

	// Get our observable address (provider side)
	const FMessageAddress ObservableAddress = PinnedController->GetMessenger()->GetAddress(Residence->GetClusterId(), Residence->GetNodeId());
	if (!ObservableAddress.IsValid())
	{
		return;
	}

	// Send request to stop session
	PinnedController->GetMessenger()->Send({ ObservableAddress },
		FDCMMessage_StopSessionRequest
		{
			.ObservableId = GetId(),
		});
}

void FClusterObservable::Play()
{
	if (!IsSessionRunning())
	{
		return;
	}

	if (!IsValid(MediaPlayer) || !IsValid(MediaSource))
	{
		return;
	}

	if (MediaPlayer->IsPlaying())
	{
		return;
	}

	if (MediaPlayer->IsPaused())
	{
		MediaPlayer->Play();
	}

	if (MediaPlayer->IsClosed())
	{
		UE_LOGF(LogClusterMonitorEditor, Display, "Observable '%ls' starting playback", *GetName());
		MediaPlayer->OpenSource(MediaSource);
	}
}

void FClusterObservable::Pause()
{
	if (!IsSessionRunning())
	{
		return;
	}

	if (!IsValid(MediaPlayer))
	{
		return;
	}

	if (MediaPlayer->IsPaused())
	{
		MediaPlayer->Play();
	}
	else
	{
		MediaPlayer->Pause();
	}
}

void FClusterObservable::Stop()
{
	if (!IsSessionRunning())
	{
		return;
	}

	if (!IsValid(MediaPlayer))
	{
		return;
	}

	if (MediaPlayer->IsClosed())
	{
		return;
	}

	MediaPlayer->Close();
}

void FClusterObservable::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(MediaPlayer);
	Collector.AddReferencedObject(MediaSource);
	Collector.AddReferencedObject(MediaTexture);
}

void FClusterObservable::SetSessionState(ESessionState InNewState)
{
	const bool bNewState = (SessionState != InNewState);
	SessionState = InNewState;

	// If the state has changed, notify the listeners
	if (bNewState)
	{
		UE_LOGF(LogClusterMonitorEditor, Verbose, "Observable '%ls' changed state to %d", *GetName(), static_cast<int32>(InNewState));
		OnSessionStateChanged().Broadcast(SessionState);
	}
}

void FClusterObservable::ConfigureMediaSource(const FString& InObservableName)
{
	FString MediaSourceName = *FString::Printf(TEXT("%s (%s)"), *Residence->GetHostname(), *InObservableName);

	MediaSourceName.ReplaceInline(TEXT(":"), TEXT(" "));

	MediaSource->bAutoDetectInput = false;
	MediaSource->MediaConfiguration.MediaConnection.Protocol = FName("ndi");
	MediaSource->MediaConfiguration.MediaConnection.PortIdentifier = 0;
	MediaSource->MediaConfiguration.MediaMode.DeviceModeIdentifier = 0;
	MediaSource->MediaConfiguration.MediaConnection.Device.DeviceIdentifier = 0;
	MediaSource->MediaConfiguration.MediaConnection.Device.DeviceName = *MediaSourceName;
}

void FClusterObservable::OnMediaEvent(EMediaEvent MediaEvent)
{
	switch (MediaEvent)
	{
	case EMediaEvent::MediaConnecting:
		UE_LOGF(LogClusterMonitorEditor, Verbose, "Media event for '%ls': Connecting", *GetName());
		SetSessionState(ESessionState::Transition);
		break;

	case EMediaEvent::MediaOpened:
		UE_LOGF(LogClusterMonitorEditor, Verbose, "Media event for '%ls': Opened", *GetName());
		SetSessionState(ESessionState::Inactive);
		break;

	case EMediaEvent::MediaOpenFailed:
		UE_LOGF(LogClusterMonitorEditor, Verbose, "Media event for '%ls': OpenFailed", *GetName());
		SetSessionState(ESessionState::Error);
		break;

	case EMediaEvent::PlaybackResumed:
		UE_LOGF(LogClusterMonitorEditor, Verbose, "Media event for '%ls': PlaybackResumed", *GetName());
		SetSessionState(ESessionState::Active);
		break;

	case EMediaEvent::PlaybackSuspended:
		UE_LOGF(LogClusterMonitorEditor, Verbose, "Media event for '%ls': Suspended", *GetName());
		SetSessionState(ESessionState::Inactive);
		break;

	case EMediaEvent::MediaClosed:
		UE_LOGF(LogClusterMonitorEditor, Verbose, "Media event for '%ls': Closed", *GetName());
		SetSessionState(ESessionState::Inactive);
		break;

	default:
		UE_LOGF(LogClusterMonitorEditor, Verbose, "Media event for '%ls': %d", *GetName(), static_cast<int32>(MediaEvent));
		break;
	}
}

void FClusterObservable::OnStartSessionResponse(const FDCEndpoint& InEndpoint, const FDCMMessage_StartSessionResponse& InMessage)
{
	// This may be a response to another observable running on the same residence.
	// Ignore it as we only interested in a single one matching our Id.
	if (InMessage.ObservableId != GetId())
	{
		return;
	}

	// Handle errors
	if (InMessage.Result != EDCRequestResult::Ok)
	{
		UE_LOGF(LogClusterMonitorEditor, Warning, "Observable '%ls' failed to start session", *GetName());
		SetSessionState(ESessionState::Error);
		return;
	}

	// Get the controller
	TSharedPtr<IClusterMonitorController> PinnedController = Controller.Pin();
	if (!PinnedController.IsValid())
	{
		UE_LOGF(LogClusterMonitorEditor, Warning, "Observable '%ls' can't access the controller anymore", *GetName());
		SetSessionState(ESessionState::Error);
		return;
	}

	// Send Play command
	PinnedController->GetMessenger()->Send({ InEndpoint.Address },
		FDCMMessage_ObservableControlRequest
		{
			.ObservableId = GetId(),
			.Command      = EDCControlCommand::Play,
		});

	UE_LOGF(LogClusterMonitorEditor, Display, "Observable '%ls' starting playback: %ls",
		*GetName(),
		*MediaSource->MediaConfiguration.MediaConnection.Device.DeviceName.ToString());

	MediaPlayer->OpenSource(MediaSource);
}

void FClusterObservable::OnObservableControlResponse(const FDCEndpoint& InEndpoint, const FDCMMessage_ObservableControlResponse& InMessage)
{
	// This may be a response to another observable running on the same residence.
	// Ignore it as we only interested in a single one matching our Id.
	if (InMessage.ObservableId != GetId())
	{
		return;
	}

	// Handle errors
	if (InMessage.Result != EDCRequestResult::Ok)
	{
		UE_LOGF(LogClusterMonitorEditor, Warning, "Observable '%ls' failed to process control command '%ls'",
			*GetName(), *StaticEnum<EDCControlCommand>()->GetNameStringByValue(static_cast<int64>(InMessage.Command)));

		return;
	}

	UE_LOGF(LogClusterMonitorEditor, Verbose, "Observable '%ls' processed '%ls' successfully",
		*GetName(), *StaticEnum<EDCControlCommand>()->GetNameStringByValue(static_cast<int64>(InMessage.Command)));
}
