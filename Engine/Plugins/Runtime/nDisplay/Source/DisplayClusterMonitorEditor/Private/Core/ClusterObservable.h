// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/IClusterObservable.h"
#include "UObject/GCObject.h"
#include "DisplayClusterMonitorTypes.h"

class IClusterMonitorController;
class IClusterResidence;
class UMediaTexture;
class UNDIMediaSource;
class UMediaPlayer;
enum class EMediaEvent;
struct FDCEndpoint;
struct FDCMMessage_ObservableDiscoveryResponse;


/**
 * Monitor side observable implementation
 */
class FClusterObservable
	: public IClusterObservable
	, public FGCObject
	, public TSharedFromThis<FClusterObservable>
{
public:

	FClusterObservable(const TSharedRef<IClusterResidence>& InResidence, const FDCMData_ObservableInfo& InObservableInfo, TWeakPtr<IClusterMonitorController> InController);
	virtual ~FClusterObservable() override;

public:

	/** Returns default icon for the specified observable type */
	static const FSlateBrush* GetDefaultIcon(EDCObservableType InObservableType);

public:

	//~ Begin IClusterObservable

	virtual TSharedRef<IClusterResidence> GetResidence() const override
	{
		return Residence;
	}

	virtual FGuid GetId() const override
	{
		return Id;
	}

	virtual FString GetName() const override
	{
		return Name;
	}

	virtual EDCObservableType GetType() const override
	{
		return Type;
	}

	virtual const FSlateBrush* GetDisplayIcon() const override
	{
		return GetDefaultIcon(Type);
	}

	virtual FIntPoint GetResolution() const override
	{
		return Resolution;
	}

public:

	virtual bool IsTile() const override
	{
		return ParentName.IsSet() && TilePos.IsSet();
	}

	virtual TOptional<FString> GetParentName() const override
	{
		return ParentName;
	}

	virtual TOptional<FIntPoint> GetTilePos() const override
	{
		return TilePos;
	}

public:

	virtual ESessionState GetSessionState() const override
	{
		return SessionState;
	}

	virtual bool HasAnyUpdates(const FDCMData_ObservableInfo& InObservableInfo) const override;
	virtual void Update(const FDCMData_ObservableInfo& InObservableInfo) override;

public:

	virtual bool IsSessionRunning() const override;
	virtual void StartSession() override;
	virtual void StopSession() override;

public:

	virtual void Play() override;
	virtual void Pause() override;
	virtual void Stop() override;

public:

	virtual UNDIMediaSource* GetMediaSource() const override
	{
		return MediaSource;
	}

	virtual UMediaTexture* GetMediaTexture() const override
	{
		return MediaTexture;
	}

	virtual UMediaPlayer* GetMediaPlayer() const override
	{
		return MediaPlayer;
	}

public:

	virtual FSessionStateChangedEvent& OnSessionStateChanged() override
	{
		return SessionStateChangedEvent;
	}

	virtual FObservableUpdatedEvent& OnObservableUpdated() override
	{
		return ObservableUpdatedEvent;
	}

	//~ End IClusterObservable

public:

	//~ Begin FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FClusterObservable");
	}
	//~ End FGCObject interface

private:

	/** Set new session state */
	void SetSessionState(ESessionState InNewState);

	/** Configures the media source based on the specified name */
	void ConfigureMediaSource(const FString& InObservableName);

private:

	/** Handles state events from the media player */
	void OnMediaEvent(EMediaEvent MediaEvent);

	/** Handles StartSession responses */
	void OnStartSessionResponse(const FDCEndpoint& InEndpoint, const FDCMMessage_StartSessionResponse& InMessage);

	/** Handles control (play, pause, stop) responses */
	void OnObservableControlResponse(const FDCEndpoint& InEndpoint, const FDCMMessage_ObservableControlResponse& InMessage);

private:

	/** Cluster controller */
	TWeakPtr<IClusterMonitorController> Controller;

	/** The entity of a residence that owns this observable */
	TSharedRef<IClusterResidence> Residence;

	/** Observable GUID */
	FGuid Id;

	/** Observable name */
	FString Name;

	/** Observable type */
	EDCObservableType Type = EDCObservableType::None;

	/** Observable texture size*/
	FIntPoint Resolution = FIntPoint::ZeroValue;

	/** Parent name if any (tiles only) */
	TOptional<FString> ParentName;
	/** Tile position (tiles only) */
	TOptional<FIntPoint> TilePos;

	/** Current session state */
	ESessionState SessionState = ESessionState::None;

private:

	/** Session media source (where to get media from) */
	TObjectPtr<UNDIMediaSource> MediaSource;

	/** Session media texture (weher to render media to) */
	TObjectPtr<UMediaTexture>   MediaTexture;

	/** Session media player (who will be doing all that */
	TObjectPtr<UMediaPlayer>    MediaPlayer;

private:

	/** Fired every time when this observation session changes its state */
	FSessionStateChangedEvent SessionStateChangedEvent;

	/** Fired every time when this observate gets updated (e.g. on the provider side) */
	FObservableUpdatedEvent   ObservableUpdatedEvent;
};
