// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/ClusterResidence.h"

#include "DisplayClusterMonitorTypes.h"


FClusterResidence::FClusterResidence(const FDCMData_ResidenceDescriptor& InResidence)
	: ClusterId(InResidence.ClusterId)
	, ClusterName(TEXT("Cluster"))
	, NodeId(InResidence.NodeId)
	, NodeName(InResidence.NodeName)
	, Hostname(InResidence.Hostname)
	, bIsPrimary(InResidence.bIsPrimary)
	, bIsOffscreen(InResidence.bIsOffscreen)
{
}
