// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/IClusterResidence.h"
#include "Misc/Guid.h"

struct FDCMData_ResidenceDescriptor;


/**
 * Cluster residence implementation
 */
class FClusterResidence
	: public IClusterResidence
{
public:

	FClusterResidence(const FDCMData_ResidenceDescriptor& InResidence);
	virtual ~FClusterResidence() override = default;

public:

	//~ Begin IClusterResidence

	virtual FGuid GetClusterId() const override
	{
		return ClusterId;
	}

	virtual FString GetClusterName() const override
	{
		return ClusterName;
	}

	virtual FGuid GetNodeId() const override
	{
		return NodeId;
	}

	virtual FString GetNodeName() const override
	{
		return NodeName;
	}

	virtual FString GetHostname() const override
	{
		return Hostname;
	}

	virtual bool IsNodeOffscreen() const override
	{
		return bIsOffscreen;
	}

	virtual int32 GetActiveObservablesNum() const override
	{
		return ActiveObservablesNum;
	}

	virtual EConnectionState GetConnectionState() const override
	{
		return ConnectionState;
	}

	virtual void SetConnectionState(EConnectionState InNewState) override
	{
		ConnectionState = InNewState;
	}

	//~ End IClusterResidence

private:

	/** Cluster GUID */
	FGuid ClusterId;
	/** Cluster name */
	FString ClusterName;

	/** Cluster node GUID */
	FGuid NodeId;
	/** Cluster node name */
	FString NodeName;

	/** Machine name of this residence */
	FString Hostname;

	/** Whether it's primary node */
	bool bIsPrimary = false;
	/** Whether it's running offscreen */
	bool bIsOffscreen = false;

	/** The number of active observables */
	int32 ActiveObservablesNum = 0;

	/** Current connection state */
	EConnectionState ConnectionState = EConnectionState::Offline;
};
