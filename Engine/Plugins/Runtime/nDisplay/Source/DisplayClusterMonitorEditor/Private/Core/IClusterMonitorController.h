// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Delegates/DelegateCombinations.h"

class IClusterObservable;
class IClusterResidence;
struct FGuid;

namespace UE::nDisplay::Monitor
{
	class FDCMessenger;
}


/**
 * Cluster monitor controller interface
 */
class IClusterMonitorController
{
public:

	virtual ~IClusterMonitorController() = default;

public:

	/** Returns MessageBus messenger */
	virtual TSharedRef<UE::nDisplay::Monitor::FDCMessenger> GetMessenger() = 0;

	/** Appear online for other cluster monitor messengers. Start discovery and messaging. */
	virtual bool StartCommunication() = 0;

	/** Go offline. Stop discovery and messaging. */
	virtual void StopCommunication() = 0;

	/**
	 * Returns observable entity with the specified GUID. If requested observable
	 * has not been discovered yet, or went offline, return nullptr.
	 */
	virtual TSharedPtr<IClusterObservable> GetObservable(const FGuid& Guid) const = 0;

	/** Returns residence entity of an observable with the specified GUID */
	virtual TSharedPtr<IClusterResidence> GetResidence(const FGuid& ObservableGuid) const = 0;

	/** Returns the number of observable entities that have been discovered */
	virtual int32 GetObservablesNum() const = 0;

	/** Returns the number of active observation sessions */
	virtual int32 GetActiveSessionsNum() const = 0;

	/** Returns the number of unresponsive residences (cluster nodes) */
	virtual int32 GetUnresponsiveNodesNum() const = 0;

	/** Re-discovers the cluster monitoring network from scratch */
	virtual void Rescan() = 0;

	/** Forgets about all unresponsive nodes. Those won't be available anymore. */
	virtual void ClearUnresponsiveEndpoints() = 0;

public:

	/** Requests observation session start for an observable entity with the specified GUID */
	virtual void RequestSessionStart(const FGuid& Observable) = 0;

	/** Requests observation session stop for an observable entity with the specified GUID */
	virtual void RequestSessionStop(const FGuid& Observable) = 0;

	/** Requests to stop all active observation sessions */
	virtual void RequestAllSessionsStop() = 0;

public:

	/** Fired when new observable is discovered */
	DECLARE_EVENT_OneParam(IClusterMonitorController, FObservableJoinedEvent, const TSharedRef<IClusterObservable>& /* Observable */);
	virtual FObservableJoinedEvent& OnObservableJoined() = 0;

	/** Fired when a known observable has been updated */
	DECLARE_EVENT_OneParam(IClusterMonitorController, FObservableUpdatedEvent, const TSharedRef<IClusterObservable>& /* Observable */);
	virtual FObservableUpdatedEvent& OnObservableUpdated() = 0;

	/** Fired when a known observable goes offline (intentional shutdown) */
	DECLARE_EVENT_TwoParams(IClusterMonitorController, FObservableLeftEvent, const TSharedRef<IClusterObservable>& /* Observable */, const FString& /* Reason */);
	virtual FObservableLeftEvent& OnObservableLeft() = 0;

	/** Fired when a known observable is considered unresponsive */
	DECLARE_EVENT_OneParam(IClusterMonitorController, FObservableTimeoutEvent, const TSharedRef<IClusterObservable>& /* Observable */);
	virtual FObservableTimeoutEvent& OnObservableTimeout() = 0;

public:

	/** Fired when new observation session is requested to start */
	DECLARE_EVENT_OneParam(IClusterMonitorController, FSessionStartRequestedEvent, const TSharedRef<IClusterObservable>& /* Observable */);
	virtual FSessionStartRequestedEvent& OnSessionStartRequested() = 0;

	/** Fired when an observation session is requested to stop */
	DECLARE_EVENT_OneParam(IClusterMonitorController, FSessionStopRequestedEvent, const TSharedRef<IClusterObservable>& /* Observable */);
	virtual FSessionStopRequestedEvent& OnSessionStopRequested() = 0;

	/** Fired when an observation session starts */
	DECLARE_EVENT_OneParam(IClusterMonitorController, FSessionStartedEvent, const TSharedRef<IClusterObservable>& /* Observable */);
	virtual FSessionStartedEvent& OnSessionStarted() = 0;

	/** Fired when an observation session stops */
	DECLARE_EVENT_OneParam(IClusterMonitorController, FSessionStoppedEvent, const TSharedRef<IClusterObservable>& /* Observable */);
	virtual FSessionStoppedEvent& OnSessionStopped() = 0;
};
