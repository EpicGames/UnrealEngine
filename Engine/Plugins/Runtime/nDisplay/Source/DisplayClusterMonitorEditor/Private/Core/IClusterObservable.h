// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

enum class EDCObservableType : uint8;
class IClusterResidence;
class UMediaPlayer;
class UMediaTexture;
class UNDIMediaSource;
struct FDCMData_ObservableInfo;
struct FGuid;
struct FSlateBrush;


/**
 * Monitor side observable interface
 * 
 * Declares API of an observable entities on the cluster monitor side (data consumer part).
 */
class IClusterObservable
{
public:

	virtual ~IClusterObservable() = default;

public:

	/**
	 * Observation session state
	 */
	enum class ESessionState : uint8
	{
		None,
		Transition,
		Inactive,
		Active,
		Error,
	};

public:

	/** Returns residence of the observable entity */
	virtual TSharedRef<IClusterResidence> GetResidence() const = 0;

	/** Returns GUID of the observable entity */
	virtual FGuid GetId() const = 0;

	/** Returns display name of the observable entity */
	virtual FString GetName() const = 0;

	/** Returns type of the observable entity */
	virtual EDCObservableType GetType() const = 0;

	/** Returns current observable icon */
	virtual const FSlateBrush* GetDisplayIcon() const = 0;

	/** Returns resolution of the observable media stream */
	virtual FIntPoint GetResolution() const = 0;

public:

	/** Returns true if this observable entity is a tile */
	virtual bool IsTile() const = 0;

	/** Returns name of the parent owner if this entity is a tile */
	virtual TOptional<FString> GetParentName() const = 0;

	/** Returns tile position if it's a tile */
	virtual TOptional<FIntPoint> GetTilePos() const = 0;

public:

	/** Returns current observation session state */
	virtual ESessionState GetSessionState() const = 0;

	/** Checks if there were any changes to the original source item */
	virtual bool HasAnyUpdates(const FDCMData_ObservableInfo& InObservableInfo) const = 0;

	/** Update observable data from the source */
	virtual void Update(const FDCMData_ObservableInfo& InObservableInfo) = 0;

public:

	/** Returns true if this observable has started local session */
	virtual bool IsSessionRunning() const = 0;

	/** Start observation session for this observable entity */
	virtual void StartSession() = 0;

	/** Stop observation session */
	virtual void StopSession() = 0;

public:

	/** Session control: Play command */
	virtual void Play() = 0;

	/** Session control: Pause command */
	virtual void Pause() = 0;

	/** Session control: Stop command */
	virtual void Stop() = 0;

public:

	/** Returns media source associated with this observable entity */
	virtual UNDIMediaSource* GetMediaSource() const = 0;

	/** Returns media texture associated with this observable entity */
	virtual UMediaTexture*   GetMediaTexture() const = 0;

	/** Returns media player associated with this observable entity */
	virtual UMediaPlayer*    GetMediaPlayer() const = 0;

public:

	/** Called when observation session changes its state */
	DECLARE_EVENT_OneParam(IClusterObservable, FSessionStateChangedEvent, ESessionState /* NewState */);
	virtual FSessionStateChangedEvent& OnSessionStateChanged() = 0;

	/** Called when this observable entity gets updated from the original source */
	DECLARE_EVENT(IClusterObservable, FObservableUpdatedEvent);
	virtual FObservableUpdatedEvent& OnObservableUpdated() = 0;
};
