// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

struct FGuid;
struct FMessageAddress;


/**
 * Residence interface
 * 
 * Declares the API of residences (cluster nodes)
 */
class IClusterResidence
{
public:

	virtual ~IClusterResidence() = default;

public:

	/**
	 * Residence connection states
	 */
	enum class EConnectionState : uint8
	{
		Online,  // Connected
		Timeout, // Unresponsive
		Offline  // Disconnected
	};

public:

	/** Returns cluster GUID */
	virtual FGuid GetClusterId() const = 0;

	/** Returns cluster name */
	virtual FString GetClusterName() const = 0;

	/** Returns cluster node GUID */
	virtual FGuid GetNodeId() const = 0;

	/** Returns cluster node name */
	virtual FString GetNodeName() const = 0;

	/** Returns the hostname of a machine where this cluster node is running on */
	virtual FString GetHostname() const = 0;

	/** Returns true if this cluster node is runing offscreen */
	virtual bool IsNodeOffscreen() const = 0;

	/** Returns the number of observables on this cluster node */
	virtual int32 GetActiveObservablesNum() const = 0;

	/** Returns current connection state of this cluster node */
	virtual EConnectionState GetConnectionState() const = 0;

	/** Set new connection state */
	virtual void SetConnectionState(EConnectionState InNewState) = 0;
};
