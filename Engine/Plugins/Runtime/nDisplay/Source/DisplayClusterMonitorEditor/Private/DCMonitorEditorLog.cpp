// Copyright Epic Games, Inc. All Rights Reserved.

#include "DCMonitorEditorLog.h"


DEFINE_LOG_CATEGORY(LogClusterMonitorEditor);
