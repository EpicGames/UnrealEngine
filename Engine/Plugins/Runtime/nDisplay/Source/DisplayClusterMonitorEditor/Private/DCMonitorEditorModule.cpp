// Copyright Epic Games, Inc. All Rights Reserved.

#include "DCMonitorEditorModule.h"

#include "Logging/LogMacros.h"
#include "Modules/ModuleManager.h"
#include "Widgets/SClusterMonitorPanel.h"

#include "ISettingsModule.h"
#include "DisplayClusterMonitorSettings.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"


#define LOCTEXT_NAMESPACE "ClusterMonitorEditor"


const FLazyName FDCMonitorEditorModule::NAME_ClusterMonitor_Container(TEXT("Project"));
const FLazyName FDCMonitorEditorModule::NAME_ClusterMonitor_Category(TEXT("Plugins"));
const FLazyName FDCMonitorEditorModule::NAME_ClusterMonitor_Section(TEXT("ClusterMonitorSettings"));


void FDCMonitorEditorModule::StartupModule()
{
	RegisterSettings();
	RegisterTabs();
}

void FDCMonitorEditorModule::ShutdownModule()
{
	UnregisterTabs();
	UnregisterSettings();
}

void FDCMonitorEditorModule::RegisterSettings()
{
	if (ISettingsModule* const SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>(TEXT("Settings")))
	{
		SettingsModule->RegisterSettings(
			NAME_ClusterMonitor_Container,
			NAME_ClusterMonitor_Category,
			NAME_ClusterMonitor_Section,
			LOCTEXT("CME_SettingsName", "Cluster Monitor"),
			LOCTEXT("CME_SettingsDescription", "Configure cluster monitor settings"),
			GetMutableDefault<UDisplayClusterMonitorSettings>());
	}
}

void FDCMonitorEditorModule::UnregisterSettings()
{
	if (ISettingsModule* const SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>(TEXT("Settings")))
	{
		SettingsModule->UnregisterSettings(
			NAME_ClusterMonitor_Container,
			NAME_ClusterMonitor_Category,
			NAME_ClusterMonitor_Section
		);
	}
}

void FDCMonitorEditorModule::RegisterTabs()
{
	const IWorkspaceMenuStructure& WorkspaceMenuStructure = WorkspaceMenu::GetMenuStructure();
	TSharedRef<FWorkspaceItem> Category = WorkspaceMenuStructure.GetLevelEditorVirtualProductionCategory();
	SClusterMonitorPanel::RegisterTabSpawner(Category);
}

void FDCMonitorEditorModule::UnregisterTabs()
{
	SClusterMonitorPanel::UnregisterTabSpawner();
}


#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FDCMonitorEditorModule, DisplayClusterMonitorEditor);
