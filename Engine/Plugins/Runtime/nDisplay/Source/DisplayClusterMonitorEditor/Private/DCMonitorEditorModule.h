// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "UObject/NameTypes.h"


/**
 * DisplayClusterMonitorEditor module implementation
 */
class FDCMonitorEditorModule : public IModuleInterface
{
	/** Cluster monitor settings container */
	static const FLazyName NAME_ClusterMonitor_Container;
	/** Cluster monitor settings category */
	static const FLazyName NAME_ClusterMonitor_Category;
	/** Cluster monitor settings section */
	static const FLazyName NAME_ClusterMonitor_Section;

public:

	//~ Begin IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface interface

private:

	/** Registers internal settings */
	void RegisterSettings();

	/** Unregisters internal settings */
	void UnregisterSettings();

	/** Registers internal UI tabs */
	void RegisterTabs();

	/** Unregisters internal UI tabs */
	void UnregisterTabs();
};
