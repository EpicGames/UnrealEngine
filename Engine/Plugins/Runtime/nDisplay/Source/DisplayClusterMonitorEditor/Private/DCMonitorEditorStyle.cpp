// Copyright Epic Games, Inc. All Rights Reserved.

#include "DCMonitorEditorStyle.h"

#include "Brushes/SlateColorBrush.h"
#include "Brushes/SlateImageBrush.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"


FDCMonitorEditorStyle::FDCMonitorEditorStyle()
	: FSlateStyleSet("ClusterMonitorEditorStyle")
{
	Initialize();
	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FDCMonitorEditorStyle::~FDCMonitorEditorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}

FDCMonitorEditorStyle& FDCMonitorEditorStyle::Get()
{
	static FDCMonitorEditorStyle Instance;
	return Instance;
}

void FDCMonitorEditorStyle::Initialize()
{
	// Make sure nDisplay is loaded
	const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(TEXT("nDisplay"));
	if (!Plugin.IsValid())
	{
		return;
	}

	// Plugin content
	const FString ContentDir = Plugin->GetContentDir();
	SetContentRoot(ContentDir);

	// Core content
	const FString CoreContentDir = FPaths::Combine(FPaths::EngineContentDir(), TEXT("Slate"));
	SetCoreContentRoot(CoreContentDir);

	// Icons
	{
		const FVector2D Icon16x16(16.0f, 16.0f);
		const FVector2D Icon20x20(20.0f, 20.0f);

		// Tab icon
		Set("ClusterMonitor.TabIcon", new IMAGE_BRUSH_SVG("Icons/ClusterMonitor/CM_TabIcon", Icon20x20));

		// Tree items
		Set("ClusterMonitor.TreeItem.Backbuffer",      new IMAGE_BRUSH_SVG("Icons/ClusterMonitor/CM_Item_Backbuffer",      Icon16x16));
		Set("ClusterMonitor.TreeItem.Cluster",         new IMAGE_BRUSH_SVG("Icons/ClusterMonitor/CM_Item_Cluster",         Icon16x16));
		Set("ClusterMonitor.TreeItem.Node",            new IMAGE_BRUSH_SVG("Icons/ClusterMonitor/CM_Item_Node",            Icon16x16));
		Set("ClusterMonitor.TreeItem.NodeOffscreen",   new IMAGE_BRUSH_SVG("Icons/ClusterMonitor/CM_Item_NodeOffscreen",   Icon16x16));
		Set("ClusterMonitor.TreeItem.ICVFXCamera",     new IMAGE_BRUSH_SVG("Icons/ClusterMonitor/CM_Item_ICVFXCamera",     Icon16x16));
		Set("ClusterMonitor.TreeItem.ICVFXCameraTile", new IMAGE_BRUSH_SVG("Icons/ClusterMonitor/CM_Item_ICVFXCameraTile", Icon16x16));
		Set("ClusterMonitor.TreeItem.UI",              new IMAGE_BRUSH_SVG("Icons/ClusterMonitor/CM_Item_UI",              Icon16x16));
		Set("ClusterMonitor.TreeItem.Viewport",        new IMAGE_BRUSH_SVG("Icons/ClusterMonitor/CM_Item_Viewport",        Icon16x16));

		// Tree column headers
		Set("ClusterMonitor.TreeColumn.Connection", new IMAGE_BRUSH_SVG("Icons/ClusterMonitor/CM_Info_Connection", Icon16x16));
		Set("ClusterMonitor.TreeColumn.Streaming",  new IMAGE_BRUSH_SVG("Icons/ClusterMonitor/CM_Info_Streaming",  Icon16x16));

		// Connection status
		Set("ClusterMonitor.ConnStatus.Online",   new IMAGE_BRUSH_SVG("Icons/ClusterMonitor/CM_Connection_Online",       Icon16x16));
		Set("ClusterMonitor.ConnStatus.Timeout",  new IMAGE_BRUSH_SVG("Icons/ClusterMonitor/CM_Connection_Timeout",      Icon16x16));
		Set("ClusterMonitor.ConnStatus.Offline",  new IMAGE_BRUSH_SVG("Icons/ClusterMonitor/CM_Connection_Disconnected", Icon16x16));

		// Session status
		Set("ClusterMonitor.SessionState.Inactive",   new IMAGE_BRUSH_SVG("Icons/ClusterMonitor/CM_Session_Inactive",   Icon16x16));
		Set("ClusterMonitor.SessionState.Active",     new IMAGE_BRUSH_SVG("Icons/ClusterMonitor/CM_Session_Active",     Icon16x16));
		Set("ClusterMonitor.SessionState.Transition", new IMAGE_BRUSH_SVG("Icons/ClusterMonitor/CM_Session_Transition", Icon16x16));
		Set("ClusterMonitor.SessionState.Error",      new IMAGE_BRUSH_SVG("Icons/ClusterMonitor/CM_Session_Error",      Icon16x16));

		// Zoom control
		Set("ClusterMonitor.Viewport.ZoomTo100", new IMAGE_BRUSH_SVG("Icons/ClusterMonitor/CM_Zoom_100", Icon16x16));
		Set("ClusterMonitor.Viewport.ZoomToFit", new IMAGE_BRUSH_SVG("Icons/ClusterMonitor/CM_Zoom_Fit", Icon16x16));
	}
}
