// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"


/**
 * Implements the visual style of the cluster monitor UI.
 */
class FDCMonitorEditorStyle : public FSlateStyleSet
{
private:

	/** Private singleton constructor */
	FDCMonitorEditorStyle();

	/** Private singleton destructor */
	virtual ~FDCMonitorEditorStyle() override;

public:

	/** Access the singleton instance for this style set */
	static FDCMonitorEditorStyle& Get();

private:

	/** Initializes internal data */
	void Initialize();
};
