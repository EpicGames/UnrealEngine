// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SClusterMonitorPanel.h"

#include "Core/ClusterMonitorController.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/TabManager.h"
#include "HAL/IConsoleManager.h"
#include "Misc/MessageDialog.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SClusterSessionsView.h"
#include "Widgets/SClusterTreeView.h"

#include "DCMonitorEditorStyle.h"
#include "DisplayClusterMonitorMessenger.h"
#include "EditorFontGlyphs.h"
#include "ISettingsModule.h"
#include "LevelEditor.h"
#include "OutputLogModule.h"

#define LOCTEXT_NAMESPACE "SClusterMonitorPanel"


TWeakPtr<SClusterMonitorPanel> SClusterMonitorPanel::PanelInstance;
FDelegateHandle SClusterMonitorPanel::LevelEditorTabManagerChangedHandle;

const FLazyName SClusterMonitorPanel::NAME_ClusterMonitorTabId(TEXT("ClusterMonitorTab"));
const FLazyName SClusterMonitorPanel::NAME_LevelEditorModuleName(TEXT("LevelEditor"));


void SClusterMonitorPanel::RegisterTabSpawner(TSharedRef<FWorkspaceItem> InWorkspaceItem)
{
	// Actual registratration procedure
	auto RegisterTabSpawner = [InWorkspaceItem]()
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>(NAME_LevelEditorModuleName);
		TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager();

		if (!LevelEditorTabManager.IsValid())
		{
			return;
		}

		LevelEditorTabManager->RegisterTabSpawner(NAME_ClusterMonitorTabId, FOnSpawnTab::CreateStatic(&SClusterMonitorPanel::SpawnTab))
			.SetDisplayName(LOCTEXT("TabTitle", "nDisplay Monitor"))
			.SetTooltipText(LOCTEXT("TooltipText", "Observe nDisplay cluster nodes"))
			.SetGroup(InWorkspaceItem)
			.SetIcon(FSlateIcon(FDCMonitorEditorStyle::Get().GetStyleSetName(), "ClusterMonitor.TabIcon"));
	};

	// Register panel spawner
	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>(NAME_LevelEditorModuleName);
	if (LevelEditorModule.GetLevelEditorTabManager())
	{
		RegisterTabSpawner();
	}
	else
	{
		LevelEditorTabManagerChangedHandle = LevelEditorModule.OnTabManagerChanged().AddLambda(RegisterTabSpawner);
	}
}

void SClusterMonitorPanel::UnregisterTabSpawner()
{
	if (FSlateApplication::IsInitialized() && FModuleManager::Get().IsModuleLoaded(NAME_LevelEditorModuleName))
	{
		// Unsubscribe from tab manager changes
		FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>(NAME_LevelEditorModuleName);
		TSharedPtr<FTabManager> LevelEditorTabManager;
		if (LevelEditorModule)
		{
			LevelEditorTabManager = LevelEditorModule->GetLevelEditorTabManager();
			LevelEditorModule->OnTabManagerChanged().Remove(LevelEditorTabManagerChangedHandle);
		}

		// Unregister the tab spawner
		if (LevelEditorTabManager.IsValid())
		{
			LevelEditorTabManager->UnregisterTabSpawner(NAME_ClusterMonitorTabId);
		}
	}
}

TSharedPtr<SClusterMonitorPanel> SClusterMonitorPanel::GetPanelInstance()
{
	return SClusterMonitorPanel::PanelInstance.Pin();
}

TSharedRef<SDockTab> SClusterMonitorPanel::SpawnTab(const FSpawnTabArgs& Args)
{
	// Instantiate this panel as a nomad tab
	return SNew(SDockTab)
		.TabRole(ETabRole::NomadTab)
		[
			SNew(SClusterMonitorPanel)
		];
}

void SClusterMonitorPanel::Construct(const FArguments& InArgs)
{
	// Refresh the reference to this panel intance
	PanelInstance = StaticCastSharedRef<SClusterMonitorPanel>(AsShared());

	// Instantiate the cluster monitor controller
	Controller = MakeShared<FClusterMonitorController>();

	// Build UI
	ChildSlot
	[
		SNew(SOverlay)

		// Layer: UI
		+SOverlay::Slot()
		[
			SNew(SVerticalBox)

			// Toolbar
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2.f, 2.f, 2.f, 2.f)
			[
				CreateWidget_Toolbar()
			]

			// Workspace
			+SVerticalBox::Slot()
			.Padding(2.f, 2.f, 2.f, 2.f)
			.FillHeight(.8)
			[
				CreateWidget_Workspace()
			]

			// Status bar
			+SVerticalBox::Slot()
			.Padding(2.f, 2.f, 2.f, 2.f)
			.AutoHeight()
			[
				CreateWidget_Status()
			]
		]
	];

	// Start cluster monitoring
	Controller->StartCommunication();
}

SClusterMonitorPanel::~SClusterMonitorPanel()
{
	// Stop monitoring
	Controller->StopCommunication();
	// Invalidate the reference to this tab
	PanelInstance.Reset();
}

TSharedRef<SWidget> SClusterMonitorPanel::CreateWidget_Toolbar()
{
	// The toolbar widget
	TSharedRef<SWidget> Widget = SNew(SBorder)
	.VAlign(VAlign_Center)
	.Padding(FMargin(2.f, 2.f, 2.f, 2.f))
	[
		SNew(SHorizontalBox)

		// Button: Show cluster monitor settings
		+SHorizontalBox::Slot()
		.Padding(4.f)
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.ToolTipText(LOCTEXT("Button_ShowSettings_ToolTip", "Show the cluster monitor project settings"))
			.ContentPadding(FMargin(4.f, 4.f))
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.OnClicked(this, &SClusterMonitorPanel::OnClicked_ShowSettings)
			[
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.14"))
				.Text(FEditorFontGlyphs::Cogs)
				.ColorAndOpacity(FLinearColor::White)
			]
		]

		// Separator
		+SHorizontalBox::Slot()
		.Padding(4.f)
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SSeparator)
			.Orientation(Orient_Vertical)
		]

		// Button: Rescan
		+SHorizontalBox::Slot()
		.Padding(4.f)
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.ToolTipText(LOCTEXT("Button_Rescan_ToolTip", "Rescan available clusters"))
			.ContentPadding(FMargin(4.f, 4.f))
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.OnClicked(this, &SClusterMonitorPanel::OnClicked_Rescan)
			[
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.14"))
				.Text(FEditorFontGlyphs::Refresh)
				.ColorAndOpacity(FLinearColor::White)
			]
		]

		// Button: Stop all sessions
		+SHorizontalBox::Slot()
		.Padding(4.f)
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.ToolTipText(LOCTEXT("Button_StopAll_ToolTip", "Terminate all active sessions"))
			.ContentPadding(FMargin(4.f, 4.f))
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.IsEnabled_Lambda([this]()
				{
					return Controller ? Controller->GetActiveSessionsNum() > 0 : false;
				})
			.OnClicked(this, &SClusterMonitorPanel::OnClicked_StopAllSessions)
			[
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.14"))
				.Text(FEditorFontGlyphs::Ban)
				.ColorAndOpacity(FLinearColor::White)
			]
		]

		// Button: clear unresponsive entities
		+SHorizontalBox::Slot()
		.Padding(4.f)
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.Text(FText::FromName(TEXT("Clear")))
			.ToolTipText(LOCTEXT("Button_ClearUnresponsive_ToolTip", "Clear unresponsive entities"))
			.ContentPadding(FMargin(4.f, 4.f))
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.IsEnabled_Lambda([this]()
				{
					return Controller ? Controller->GetUnresponsiveNodesNum() > 0 : false;
				})
			.OnClicked(this, &SClusterMonitorPanel::OnClicked_ClearUnresponsiveEntities)
			[
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.14"))
				.Text(FEditorFontGlyphs::Trash)
				.ColorAndOpacity(FLinearColor::White)
			]
		]
	];

	return Widget;
}

TSharedRef<SWidget> SClusterMonitorPanel::CreateWidget_Workspace()
{
	// The main workspace are widget
	TSharedRef<SWidget> Widget = SNew(SBorder)
	[
		SNew(SSplitter)
		.Orientation(Orient_Horizontal)

		// Cluster tree
		+SSplitter::Slot()
		.Value(.35f)
		[
			SAssignNew(ClusterTreeView, SClusterTreeView, Controller)
		]
	
		// Active sessions layout
		+SSplitter::Slot()
		[
			SAssignNew(ClusterSessionsView, SClusterSessionsView, Controller)
		]
	];

	return Widget;
}

TSharedRef<SWidget> SClusterMonitorPanel::CreateWidget_Status()
{
	using namespace UE::OutputLog;

	// Prepare creation parameters
	const FConsoleInputBoxCreationParams CreationParams
	{
		.OnInterceptConsoleCommand = FInterceptConsoleCommandDelegate::CreateSP(this, &SClusterMonitorPanel::OnInterceptConsoleCommand),
		.OnHistoryOverride = FHistoryOverrideDelegate::CreateSP(this, &SClusterMonitorPanel::OnOverrideHistory),
	};

	// Create the console input box widget
	FConsoleInputBoxCreationResult Result = FOutputLogModule::Get().MakeConsoleInputBox(CreationParams);

	// Status bar widget
	TSharedRef<SWidget> Widget = SNew(SHorizontalBox)

		// Console command widget
		+SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.Padding(4.f, 4.f, 4.f, 4.f)
		[
			Result.ConsoleInputBox.IsValid() ? Result.ConsoleInputBox.ToSharedRef() : SNullWidget::NullWidget
		];

	return Widget;
}

void SClusterMonitorPanel::Rescan()
{
	// Forward to the controller
	if (Controller.IsValid())
	{
		Controller->Rescan();
	}
}

void SClusterMonitorPanel::ClearUnresponsiveEntities()
{
	// Forward to the controller
	if (Controller.IsValid())
	{
		Controller->ClearUnresponsiveEndpoints();
	}
}

FReply SClusterMonitorPanel::OnClicked_ShowSettings()
{
	// Open the corresponding settings page
	ISettingsModule& SetttingsModule = FModuleManager::LoadModuleChecked<ISettingsModule>("Settings");
	SetttingsModule.ShowViewer("Project", "Plugins", "ClusterMonitorSettings");
	return FReply::Handled();
}

FReply SClusterMonitorPanel::OnClicked_Rescan()
{
	Rescan();
	return FReply::Handled();
}

FReply SClusterMonitorPanel::OnClicked_StopAllSessions()
{
	// Confirmation required
	const EAppReturnType::Type Response = FMessageDialog::Open(
		EAppMsgType::YesNo,
		LOCTEXT("ConfirmStopAllStreamsMessage", "This will stop all currently active streams. Are you sure?"));

	if (Response == EAppReturnType::Type::Yes)
	{
		if (Controller.IsValid())
		{
			Controller->RequestAllSessionsStop();
		}
	}

	return FReply::Handled();
}

FReply SClusterMonitorPanel::OnClicked_ClearUnresponsiveEntities()
{
	ClearUnresponsiveEntities();
	return FReply::Handled();
}

bool SClusterMonitorPanel::OnOverrideHistory(const FName& InExecutorName, TArray<FString>& OutHistory)
{
	// Always override the command histories of any executor
	OutHistory = CommandHistory.FindOrAdd(InExecutorName);
	return true;
}

bool SClusterMonitorPanel::OnInterceptConsoleCommand(const FName& InExecutorName, const FString& InExecCommand)
{
	// We should never receive empty commands here. If that ever happens,
	// ignore them and let the original pipeline handle it.
	if (InExecCommand.IsEmpty())
	{
		return false;
	}

	// Let this command be executed on the cluster node(s)
	PropagateConsoleCommand(InExecutorName.ToString(), InExecCommand);

	// Manage our own history
	{
		// Add to history. If this command already there, make it appear first
		TArray<FString>& ExecutorHistory = CommandHistory.FindOrAdd(InExecutorName);
		ExecutorHistory.Remove(InExecCommand);
		ExecutorHistory.Add(InExecCommand);

		// Follow history limitation
		constexpr int32 MaxHistoryLen = 30;
		const int32 HistoryLen = ExecutorHistory.Num();
		if (HistoryLen > MaxHistoryLen)
		{
			ExecutorHistory.RemoveAt(0, HistoryLen - MaxHistoryLen);
		}
	}

	// Return true to block this command execution. Never allow to run commands locally.
	return true;
}

void SClusterMonitorPanel::PropagateConsoleCommand(const FString& ExecutorName, const FString& Command)
{
	if (Controller.IsValid())
	{
		// Send to all providers
		Controller->GetMessenger()->SendToRoles({ EDCMessengerRole::ObservablesProvider },
			FDCMMessage_ExecuteConsoleCommand
			{
				.ExecutorName = ExecutorName,
				.Command      = Command,
			});
	}
}

#undef LOCTEXT_NAMESPACE
