// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "UObject/NameTypes.h"

class FSpawnTabArgs;
class FWorkspaceItem;
class IClusterMonitorController;
class SClusterSessionsView;
class SClusterTreeView;
class SDockTab;
class SMultiLineEditableTextBox;


/**
 * The main cluster monitor panel
 */
class SClusterMonitorPanel : public SCompoundWidget
{
	using Super = SCompoundWidget;

public:

	/** Registers this panel widget as a nomad tab */
	static void RegisterTabSpawner(TSharedRef<FWorkspaceItem> InWorkspaceItem);

	/** Unregisters this panel from the tab management sytem */
	static void UnregisterTabSpawner();

	/** Returns this panel instance (may be invalid if not currently spawned) */
	static TSharedPtr<SClusterMonitorPanel> GetPanelInstance();

private:

	/** References the cluster monitor panel widget when its open */
	static TWeakPtr<SClusterMonitorPanel> PanelInstance;

	/**
	 * Delegate responsible for processing the tab manager changes
	 * to keep this panel always properly registered
	 */
	static FDelegateHandle LevelEditorTabManagerChangedHandle;

	/** Tab ID to use for registering this panel */
	static const FLazyName NAME_ClusterMonitorTabId;

	/** LevelEditor module name (owns the tab manager) */
	static const FLazyName NAME_LevelEditorModuleName;

private:

	/** Factory method that instantiates this panel widget */
	static TSharedRef<SDockTab> SpawnTab(const FSpawnTabArgs& Args);

public:

	SLATE_BEGIN_ARGS(SClusterMonitorPanel)
	{ }
	SLATE_END_ARGS()

	/** Widget construction */
	void Construct(const FArguments& InArgs);

	/** Widget destruction */
	virtual ~SClusterMonitorPanel();

private:

	/** Entry point to build the toolbar widget */
	TSharedRef<SWidget> CreateWidget_Toolbar();

	/** Entry point to build the main workspace widget */
	TSharedRef<SWidget> CreateWidget_Workspace();

	/** Entry point to build the status bar widget */
	TSharedRef<SWidget> CreateWidget_Status();

public:

	/** Rescan clusters from scratch */
	void Rescan();

	/** Clears all unresponsive entities */
	void ClearUnresponsiveEntities();

private:

	/** Handles 'show settings' button clicks */
	FReply OnClicked_ShowSettings();

	/** Handles 'rescan' button clicks */
	FReply OnClicked_Rescan();

	/** Handles 'stop all sessions' button clicks */
	FReply OnClicked_StopAllSessions();

	/** Handles 'clear unresponsive entities' button clicks */
	FReply OnClicked_ClearUnresponsiveEntities();

	/** */
	bool OnOverrideHistory(const FName& InExecutorName, TArray<FString>& OutHistory);

	/** */
	bool OnInterceptConsoleCommand(const FName& InExecutorName, const FString& InExecCommand);

	/**  */
	void PropagateConsoleCommand(const FString& ExecutorName, const FString& Command);

private:

	/** A controller that provides cluster data, and manages internal processes */
	TSharedPtr<IClusterMonitorController> Controller;

	/** A tree based layout showing currently existing observables */
	TSharedPtr<SClusterTreeView> ClusterTreeView;

	/** A grid based layout showing active streams */
	TSharedPtr<SClusterSessionsView> ClusterSessionsView;

	/** Console commands widget */
	TSharedPtr<SMultiLineEditableTextBox> ConsoleEditBox;

	TMap<FName, TArray<FString>> CommandHistory;
};
