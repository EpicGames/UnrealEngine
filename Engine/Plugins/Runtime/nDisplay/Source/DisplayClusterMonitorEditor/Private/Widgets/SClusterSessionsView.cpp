// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SClusterSessionsView.h"

#include "Core/IClusterMonitorController.h"
#include "Core/IClusterObservable.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SSessionViewport.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Text/STextBlock.h"

#include "DisplayClusterMonitorSettings.h"

#define LOCTEXT_NAMESPACE "SClusterSessionsView"


void SClusterSessionsView::Construct(const FArguments& InArgs, const TSharedPtr<IClusterMonitorController>& InController)
{
	Controller = InController;

	// Subscribe to controller events
	if (TSharedPtr<IClusterMonitorController> ControllerPtr = Controller.Pin())
	{
		ControllerPtr->OnSessionStartRequested().AddSP(this, &SClusterSessionsView::OnSessionStartRequested);
		ControllerPtr->OnSessionStopRequested().AddSP(this, &SClusterSessionsView::OnSessionStopRequested);
		ControllerPtr->OnObservableLeft().AddSP(this, &SClusterSessionsView::OnObservableLeft);
	}

	// UI content
	ChildSlot
	[
		// Empty container. Its content will be procedurally generated.
		SAssignNew(LayoutContainer, SOverlay)
	];

	// Generate UI widgets
	UpdateDisplays();
}

SClusterSessionsView::~SClusterSessionsView()
{
	if (TSharedPtr<IClusterMonitorController> ControllerPtr = Controller.Pin())
	{
		ControllerPtr->OnSessionStartRequested().RemoveAll(this);
		ControllerPtr->OnSessionStopRequested().RemoveAll(this);
		ControllerPtr->OnObservableLeft().RemoveAll(this);
		ControllerPtr->OnObservableTimeout().RemoveAll(this);
	}
}

bool SClusterSessionsView::CanMaximizeViewport(const FGuid& InObservableId) const
{
	const bool bValidId = InObservableId.IsValid();
	const bool bNoCurrentlyMaximized    = !MaximizedViewportId.IsValid();
	const bool bRequestedViewportExists = HasViewport(InObservableId);

	const bool bCanMaximize = bValidId && bNoCurrentlyMaximized && bRequestedViewportExists;

	return bCanMaximize;
}

void SClusterSessionsView::MaximizeViewport(const FGuid& InObservableId)
{
	// Check if can be maximized
	if (!CanMaximizeViewport(InObservableId))
	{
		return;
	}

	// Remember currently maximized viewport
	MaximizedViewportId = InObservableId;

	// Redraw layout
	UpdateDisplays();

	// Set keyboard focus on the panel widget after it has been maximized
	// so that keyboard shortcuts still work for that panel without the user having to click on it again
	if (const FViewportItem* const ViewportItem = GetViewportItem(MaximizedViewportId))
	{
		FSlateApplication::Get().ClearKeyboardFocus();
		FSlateApplication::Get().SetKeyboardFocus(ViewportItem->Widget);
	}
}

bool SClusterSessionsView::IsViewportMaximized(const FGuid& InObservableId) const
{
	const bool bIsMaximized = (MaximizedViewportId == InObservableId);
	return bIsMaximized;
}

bool SClusterSessionsView::CanResetMaximized() const
{
	return MaximizedViewportId.IsValid();
}

void SClusterSessionsView::ResetMaximized()
{
	// Remember last maximized viewport
	const FGuid LastMaximizedViewportId = MaximizedViewportId;

	// Reset currently maximized ID
	MaximizedViewportId.Invalidate();

	// Redraw the layout
	UpdateDisplays();

	// Reset keyboard focus back the panel widget after it has been repositioned in the panel layout
	// so that keyboard shortcuts still work for that panel without the user having to click on it again
	if (const FViewportItem* const ViewportItem = GetViewportItem(LastMaximizedViewportId))
	{
		FSlateApplication::Get().ClearKeyboardFocus();
		FSlateApplication::Get().SetKeyboardFocus(ViewportItem->Widget);
	}
}

void SClusterSessionsView::SetViewportImmersive(const FGuid& InObservableId)
{
	TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
	if (!ParentWindow.IsValid())
	{
		return;
	}

	// Get viewport item that is going to switch to immersive mode
	const FViewportItem* const ViewportItem = GetViewportItem(InObservableId);
	if (!ViewportItem)
	{
		return;
	}

	// Make sure the widget is valid
	if (!ViewportItem->Widget.IsValid())
	{
		return;
	}

	// Remember immersive viewport
	ImmersiveViewportId = InObservableId;

	// Switch to immmersive
	ParentWindow->SetFullWindowOverlayContent(ViewportItem->Widget);

	// Reset keyboard focus back the panel widget after it has been repositioned in the panel layout
	// so that keyboard shortcuts still work for that panel without the user having to click on it again
	FSlateApplication::Get().ClearKeyboardFocus();
	FSlateApplication::Get().SetKeyboardFocus(ViewportItem->Widget);
}

bool SClusterSessionsView::IsViewportImmersive(const FGuid& InObservableId) const
{
	return ImmersiveViewportId == InObservableId;
}

void SClusterSessionsView::ResetImmersive()
{
	if (!ImmersiveViewportId.IsValid())
	{
		return;
	}

	TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
	if (!ParentWindow.IsValid())
	{
		return;
	}

	// Get viewport item that is going to switch to immersive mode
	const FViewportItem* const ViewportItem = GetViewportItem(ImmersiveViewportId);
	if (!ViewportItem)
	{
		return;
	}

	// Make sure the widget is valid
	if (!ViewportItem->Widget.IsValid())
	{
		return;
	}

	// Reset current immersive GUID
	ImmersiveViewportId.Invalidate();

	// Disable immersive mode
	ParentWindow->SetFullWindowOverlayContent(nullptr);

	// Reset keyboard focus back the panel widget after it has been repositioned in the panel layout
	// so that keyboard shortcuts still work for that panel without the user having to click on it again
	FSlateApplication::Get().ClearKeyboardFocus();
	FSlateApplication::Get().SetKeyboardFocus(ViewportItem->Widget);
}

void SClusterSessionsView::UpdateDisplays()
{
	// Clear whole hierarchy
	LayoutContainer->ClearChildren();

	// And regenerate
	LayoutContainer->AddSlot()
	[
		ConstructViewportLayout()
	];
}

TSharedRef<SWidget> SClusterSessionsView::ConstructViewportLayout()
{
	// If no viewports available, generate a corresponding information widget
	if (ViewportItems.Num() < 1)
	{
		return CreateWidget_NoViewportsAvailable();
	}

	// When a viewport is maximized, show only this one
	if (MaximizedViewportId.IsValid())
	{
		if (HasViewport(MaximizedViewportId))
		{
			return CreateWidget_Viewport(MaximizedViewportId);
		}
		else
		{
			MaximizedViewportId.Invalidate();
		}
	}

	// Generate viewports for every active observable
	return CreateWidget_ViewportGrid();
}

TSharedRef<SWidget> SClusterSessionsView::CreateWidget_NoViewportsAvailable()
{
	return SNew(SBorder)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.BorderImage(FAppStyle::GetBrush("NoBorder"))
		[
			SNew(STextBlock)
			.Text(LOCTEXT("MessageNoActiveSessions", "No active sessions"))
		];
}

TSharedRef<SWidget> SClusterSessionsView::CreateWidget_ViewportGrid()
{
	// Use vertical and horizontal boxes to generate grid like layout
	TSharedPtr<SVerticalBox>   VerticalBox;
	TSharedPtr<SHorizontalBox> HorizontalBox;

	// Vertically scrollable layout
	TSharedRef<SScrollBox> ScrollBox = SNew(SScrollBox)
		.Orientation(Orient_Vertical)

		// A single vertical box with arbitrary amount of rows
		+SScrollBox::Slot()
		.FillContentSize(0.5f)
		[
			SAssignNew(VerticalBox, SVerticalBox)
		];

	// Get cluster monitor settings
	const UDisplayClusterMonitorSettings* const DCMSettings = GetDefault<UDisplayClusterMonitorSettings>();

	// Amout of viewports per row is customizable
	const int32 SlotsInRow = (DCMSettings->ViewportsInRow > 0 ? DCMSettings->ViewportsInRow : 2);

	// Should we lock the viewport height?
	const bool bUseFixedViewportHeight = DCMSettings->bUseFixedViewportHeight;
	const float FixedViewportHeight = FMath::Clamp(DCMSettings->FixedViewportHeight, 200.f, 2000.f);

	// Generate widgets for all active sessions
	for (int32 ViewportIdx = 0; ViewportIdx < ViewportItems.Num(); ++ViewportIdx)
	{
		// Viewport index in current row
		const int32 ItemInRowIdx = (ViewportIdx % SlotsInRow);

		// For every "SlotsInRow"-th viewport, add a new row
		if (ItemInRowIdx == 0)
		{
			VerticalBox->AddSlot()
			.MinHeight(FixedViewportHeight)
			.MaxHeight(bUseFixedViewportHeight ? FixedViewportHeight : TAttribute<float>())
			[
				SAssignNew(HorizontalBox, SHorizontalBox)
			];
		}

		// And put a new viewport
		HorizontalBox->AddSlot()
		[
			SNew(SBox)
			.Padding(FMargin(5))
			[
				SNew(SBorder)
				.BorderBackgroundColor(FLinearColor::Black)
				.BorderImage(FAppStyle::GetBrush("WhiteBrush"))
				[
					CreateWidget_Viewport(ViewportItems[ViewportIdx])
				]
			]
		];
	}

	return ScrollBox;
}

TSharedRef<SWidget> SClusterSessionsView::CreateWidget_Viewport(const FGuid& InObservableId)
{
	FViewportItem* ViewportItem = GetViewportItem(InObservableId);
	return ViewportItem ? CreateWidget_Viewport(*ViewportItem) : SNullWidget::NullWidget;
}

TSharedRef<SWidget> SClusterSessionsView::CreateWidget_Viewport(FViewportItem& InViewportItem)
{
	if (InViewportItem.Widget.IsValid())
	{
		return InViewportItem.Widget.ToSharedRef();
	}

	return SAssignNew(InViewportItem.Widget, SSessionViewport, SharedThis(this), Controller.Pin(), InViewportItem.Observable);
}

void SClusterSessionsView::OnSessionStartRequested(const TSharedRef<IClusterObservable>& InObservable)
{
	AddViewport(InObservable);
}

void SClusterSessionsView::OnSessionStopRequested(const TSharedRef<IClusterObservable>& InObservable)
{
	RemoveViewport(InObservable);
}

void SClusterSessionsView::OnObservableLeft(const TSharedRef<IClusterObservable>& InObservable, const FString& InReason)
{
	RemoveViewport(InObservable);
}

void SClusterSessionsView::AddViewport(const TSharedRef<IClusterObservable>& InObservable)
{
	// Make sure it has not been added already
	const FGuid NewId = InObservable->GetId();
	if (HasViewport(NewId))
	{
		return;
	}

	// Create new content item
	FViewportItem NewItem
	{
		.Id = NewId,
		.Observable = InObservable,
	};

	// Where to put the new viewport
	const bool bAddToFront = GetDefault<UDisplayClusterMonitorSettings>()->bAddNewObservablesToFront;
	if (bAddToFront)
	{
		ViewportItems.Insert(MoveTemp(NewItem), 0);
	}
	else
	{
		ViewportItems.Add(MoveTemp(NewItem));
	}

	// Once new viewport is added, redraw the layout
	UpdateDisplays();
}
void SClusterSessionsView::RemoveViewport(const TSharedRef<IClusterObservable>& InObservable)
{
	// Reset immersive mode for the viewport being removed
	if (IsViewportImmersive(InObservable->GetId()))
	{
		ResetImmersive();
	}

	// Reset maximized mode for the viewport being removed
	if (IsViewportMaximized(InObservable->GetId()))
	{
		ResetMaximized();
	}

	// Remove requested session
	const int32 RemovedNum = ViewportItems.RemoveAll([&InObservable](const FViewportItem& Item)
		{
			return Item.Id == InObservable->GetId();
		});

	// If removed, redraw the UI layout
	if (RemovedNum > 0)
	{
		UpdateDisplays();
	}
}

#undef LOCTEXT_NAMESPACE
