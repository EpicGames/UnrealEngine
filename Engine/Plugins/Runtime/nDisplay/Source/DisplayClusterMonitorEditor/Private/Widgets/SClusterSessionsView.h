// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class SOverlay;
class IClusterMonitorController;
class IClusterObservable;


/**
 * Cluster Sessions View
 * 
 * Represents all the observation sessions as separate widgets.
 * Responsible for:
 * - Creating and deleting of session viewports
 * - Viewport view modes (maximized, immersive)
 */
class SClusterSessionsView : public SCompoundWidget
{
	struct FViewportItem;

public:
	SLATE_BEGIN_ARGS(SClusterSessionsView)
	{ }
	SLATE_END_ARGS()

	/** Widget construction */
	void Construct(const FArguments& InArgs, const TSharedPtr<IClusterMonitorController>& InController);

	/** Widget destruction */
	virtual ~SClusterSessionsView() override;

public:

	/** Whether a viewport can be maximized now */
	bool CanMaximizeViewport(const FGuid& InObservableId) const;

	/** Maximize viewport of a specified session */
	void MaximizeViewport(const FGuid& InObservableId);

	/** Whether a specified viewport is currently maximized */
	bool IsViewportMaximized(const FGuid& InObservableId) const;

	/** Whether there is a maximized viewport, and we can get back to normal grid view */
	bool CanResetMaximized() const;

	/** Get back to normal grid view */
	void ResetMaximized();

public:

	/** Switches specified viewport into immersive mode */
	void SetViewportImmersive(const FGuid& InObservableId);

	/** Whether a specified viewport is in immersive mode */
	bool IsViewportImmersive(const FGuid& InObservableId) const;

	/** Reset immersive mode */
	void ResetImmersive();

private:

	/** Generates UI layout for active sessions */
	void UpdateDisplays();

	/** UI layout generation entry point */
	TSharedRef<SWidget> ConstructViewportLayout();

	/** Generates UI when no active sessions (viewports) available */
	TSharedRef<SWidget> CreateWidget_NoViewportsAvailable();

	/** Generates UI layout */
	TSharedRef<SWidget> CreateWidget_ViewportGrid();

	/** Generates viewport widget */
	TSharedRef<SWidget> CreateWidget_Viewport(const FGuid& InObservableId);

	/** Generates viewport widget */
	TSharedRef<SWidget> CreateWidget_Viewport(FViewportItem& InViewportItem);

private:

	/** Checks if a specified session viewport exists */
	bool HasViewport(const FGuid& Id) const
	{
		return ViewportItems.ContainsByPredicate([&Id](const FViewportItem& Item)
			{
				return Item.Id == Id;
			});
	}

	/** Returns content item by observalbe GUID. Ohterwise nullptr. */
	FViewportItem* GetViewportItem(const FGuid& Id)
	{
		return ViewportItems.FindByPredicate([&Id](const FViewportItem& Item)
			{
				return Item.Id == Id;
			});
	}

private:

	/** Handles session start request notifications. Creates a corresponding viewport. */
	void OnSessionStartRequested(const TSharedRef<IClusterObservable>& InObservable);

	/** Handles session stop request notifications. Deletes a corresponding viewport. */
	void OnSessionStopRequested(const TSharedRef<IClusterObservable>& InObservable);

	/** Handles observable left notifications. Deletes a corresponding viewport. */
	void OnObservableLeft(const TSharedRef<IClusterObservable>& InObservable, const FString& InReason);

private:

	/** Creates viewport for a specified observable session */
	void AddViewport(const TSharedRef<IClusterObservable>& InObservable);

	/** Removes viewport of a specified observable session */
	void RemoveViewport(const TSharedRef<IClusterObservable>& InObservable);

private:

	/** Cluster monitor controller */
	TWeakPtr<IClusterMonitorController> Controller;

	/** The root widget of the viewport layout */
	TSharedPtr<SOverlay> LayoutContainer;

	/**
	 * Internal content item. Keeps data associated with an observation session
	 */
	struct FViewportItem
	{
		/** Observable GUID */
		FGuid Id;

		/** Observable entity */
		TSharedPtr<IClusterObservable> Observable;

		/** Session viewport widget */
		TSharedPtr<SWidget> Widget;
	};

	/** Currently active observation sessions */
	TArray<FViewportItem> ViewportItems;

	/** Viewport that is currently maximized */
	FGuid MaximizedViewportId;

	/** Viewport that is currently in immersive mode */
	FGuid ImmersiveViewportId;
};
