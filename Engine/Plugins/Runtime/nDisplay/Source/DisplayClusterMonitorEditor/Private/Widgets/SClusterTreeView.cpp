// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SClusterTreeView.h"

#include "Core/IClusterMonitorController.h"
#include "Core/IClusterObservable.h"
#include "Core/IClusterResidence.h"
#include "Filters/GenericFilter.h"
#include "Filters/SBasicFilterBar.h"
#include "Filters/SFilterBar.h"
#include "Filters/SFilterSearchBox.h"
#include "Math/NumericLimits.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Views/SExpanderArrow.h"
#include "Widgets/Views/STableRow.h"

#include "DCMonitorEditorStyle.h"
#include "DisplayClusterMonitorTypes.h"

#define LOCTEXT_NAMESPACE "SClusterTreeView"

const FLazyName SClusterTreeView::Column_ItemConnection(TEXT("Column_ItemConnection"));
const FLazyName SClusterTreeView::Column_ItemName(TEXT("Column_ItemName"));
const FLazyName SClusterTreeView::Column_ItemInfo(TEXT("Column_ItemInfo"));
const FLazyName SClusterTreeView::Column_ItemStream(TEXT("Column_ItemStream"));


/**
 * Slate widget for the individual rows of the cluster tree view
 */
class SClusterTreeItemRow : public SMultiColumnTableRow<SClusterTreeView::FTreeItemPtr>
{
	using Super = SMultiColumnTableRow<SClusterTreeView::FTreeItemPtr>;

public:
	SLATE_BEGIN_ARGS(SClusterTreeItemRow)
	{ }
	SLATE_END_ARGS()

public:

	/** Widget construction */
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, SClusterTreeView::FTreeItemPtr InTreeItem)
	{
		TreeItem = InTreeItem;

		FSuperRowType::FArguments Args = FSuperRowType::FArguments()
			.Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("SceneOutliner.TableViewRow"));
		
		Super::Construct(Args, InOwnerTableView);
	}

	/** Returns current connection state icon of a cluster node */
	const FSlateBrush* GetConnectionStatusIcon() const
	{
		// Get tree item associated with this row
		SClusterTreeView::FTreeItemPtr PinnedTreeItem = TreeItem.Pin();
		if (!PinnedTreeItem.IsValid())
		{
			return nullptr;
		}

		// Connection status is valid for cluster nodes only. Ignore any other items.
		if (!PinnedTreeItem->IsClusterNode())
		{
			return nullptr;
		}

		// Residence must be valid for node items
		if (!PinnedTreeItem->Residence.IsValid())
		{
			return nullptr;
		}

		// Get current connection status
		const IClusterResidence::EConnectionState ConnectionState = PinnedTreeItem->Residence->GetConnectionState();

		// Pick an icon according to the current connection status
		const FSlateBrush* ConnectionStatusIcon = nullptr;
		switch (ConnectionState)
		{
		case IClusterResidence::EConnectionState::Online:
			ConnectionStatusIcon = FDCMonitorEditorStyle::Get().GetBrush("ClusterMonitor.ConnStatus.Online");
			break;

		case IClusterResidence::EConnectionState::Timeout:
			ConnectionStatusIcon = FDCMonitorEditorStyle::Get().GetBrush("ClusterMonitor.ConnStatus.Timeout");
			break;

		case IClusterResidence::EConnectionState::Offline:
			ConnectionStatusIcon = FDCMonitorEditorStyle::Get().GetBrush("ClusterMonitor.ConnStatus.Offline");
			break;

		default:
			ConnectionStatusIcon = nullptr;
			break;
		}

		return ConnectionStatusIcon;
	}

	/** Generates item information text */
	FText GetItemInfoText() const
	{
		// Get tree item associated with this row
		SClusterTreeView::FTreeItemPtr PinnedTreeItem = TreeItem.Pin();
		if (!PinnedTreeItem.IsValid())
		{
			return FText::GetEmpty();
		}

		// Only leaves (observables) may run sessions
		if (!PinnedTreeItem->IsLeaf())
		{
			return FText::GetEmpty();
		}

		// Generate resolution text
		const FIntPoint Resolution = PinnedTreeItem->Observable->GetResolution();
		const FText ResolutionText = FText::FromString(
			FString::Printf(TEXT("%d x %d"), Resolution.X, Resolution.Y));

		return  ResolutionText;
	}

	/** Returns current session state icon */
	const FSlateBrush* GetSessionStateIcon() const
	{
		// Get tree item associated with this row
		SClusterTreeView::FTreeItemPtr PinnedTreeItem = TreeItem.Pin();
		if (!PinnedTreeItem.IsValid())
		{
			return nullptr;
		}

		// Only leaves (observables) may run sessions
		if (!PinnedTreeItem->IsLeaf())
		{
			return nullptr;
		}

		// Get current session state
		const IClusterObservable::ESessionState SessionState = PinnedTreeItem->Observable->GetSessionState();

		// Pick an icon according to the current session state
		const FSlateBrush* SessionStateIcon = nullptr;
		switch (SessionState)
		{
		case IClusterObservable::ESessionState::None:
			SessionStateIcon = nullptr;
			break;

		case IClusterObservable::ESessionState::Transition:
			SessionStateIcon = FDCMonitorEditorStyle::Get().GetBrush("ClusterMonitor.SessionState.Transition");
			break;

		case IClusterObservable::ESessionState::Inactive:
			SessionStateIcon = FDCMonitorEditorStyle::Get().GetBrush("ClusterMonitor.SessionState.Inactive");
			break;

		case IClusterObservable::ESessionState::Active:
			SessionStateIcon = FDCMonitorEditorStyle::Get().GetBrush("ClusterMonitor.SessionState.Active");
			break;

		case IClusterObservable::ESessionState::Error:
			SessionStateIcon = FDCMonitorEditorStyle::Get().GetBrush("ClusterMonitor.SessionState.Error");
			break;

		default:
			SessionStateIcon = nullptr;
			break;
		}

		return SessionStateIcon;
	}

	/** Auxiliary function that creates a pair image:label */
	TSharedRef<SWidget> CreateWidget_LabelWithIcon(const FString& InLabel, const FSlateBrush* InIcon)
	{
		// Initialize the label widget
		TSharedPtr<SWidget> LabelWidget = SNullWidget::NullWidget;
		if (!InLabel.IsEmpty())
		{
			LabelWidget = SNew(STextBlock)
				.Text(FText::FromString(InLabel));
		}

		// Initialize the icon widget
		TSharedPtr<SWidget> IconWidget = SNullWidget::NullWidget;
		if (InIcon)
		{
			IconWidget = SNew(SBox)
				.WidthOverride(16.0f)
				.HeightOverride(16.0f)
				[
					SNew(SImage).Image(InIcon)
				];
		}

		// Generate and return the label:icon pair
		const FMargin LeftMargin = InIcon ? FMargin(6.0f, 1.0f, 6.0f, 1.0f) : FMargin();
		return SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Padding(LeftMargin)
			.AutoWidth()
			[
				IconWidget.ToSharedRef()
			]

			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				LabelWidget.ToSharedRef()
			];
	}

	/** Creates widget for the "connection" column */
	TSharedRef<SWidget> CreateColumnWidget_Connection()
	{
		SClusterTreeView::FTreeItemPtr PinnedTreeItem = TreeItem.Pin();
		if (!PinnedTreeItem.IsValid())
		{
			return SNullWidget::NullWidget;
		}

		// Connection state icon only
		return SNew(SBox)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.Image(this, &SClusterTreeItemRow::GetConnectionStatusIcon)
			];
	}

	/** Creates widget for the "name" column */
	TSharedRef<SWidget> CreateColumnWidget_Name()
	{
		SClusterTreeView::FTreeItemPtr PinnedTreeItem = TreeItem.Pin();
		if (!PinnedTreeItem.IsValid())
		{
			return SNullWidget::NullWidget;
		}

		// The root widget
		return SNew(SBox)
			.MinDesiredHeight(20.0f)
			[
				SNew(SHorizontalBox)

				// Expander for children
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(6.f, 0.f, 0.f, 0.f)
				[
					SNew(SExpanderArrow, SharedThis(this))
					.IndentAmount(12)
				]

				// Label:name
				+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				[
					CreateWidget_LabelWithIcon(PinnedTreeItem->Name, SClusterTreeView::FTreeItem::GetTreeItemIcon(PinnedTreeItem->Type))
				]
			];
	}

	/** Creates widget for the "info" column */
	TSharedRef<SWidget> CreateColumnWidget_Info()
	{
		SClusterTreeView::FTreeItemPtr PinnedTreeItem = TreeItem.Pin();
		if (!PinnedTreeItem.IsValid())
		{
			return SNullWidget::NullWidget;
		}

		// Only leaves allowed
		const bool bIsLeaf = PinnedTreeItem->IsLeaf();
		if (!bIsLeaf)
		{
			return SNullWidget::NullWidget;
		}

		// Item resolution
		const FIntPoint& Resolution = PinnedTreeItem->Resolution;
		const bool bValidResolution = (Resolution.X > 0 && Resolution.Y > 0);

		// Item resolution widget
		TSharedPtr<SWidget> ResolutionWidget = SNullWidget::NullWidget;
		if (bValidResolution)
		{
			ResolutionWidget = SNew(STextBlock)
				.Font(FCoreStyle::GetDefaultFontStyle("Mono", 10))
				.Text(this, &SClusterTreeItemRow::GetItemInfoText);
		}

		// Generate and return the info widget
		return SNew(SHorizontalBox)

			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBox)
				.HAlign(HAlign_Left)
				[
					ResolutionWidget.ToSharedRef()
				]
			];
	}

	/** Creates widget for the "streaming status" column */
	TSharedRef<SWidget> CreateColumnWidget_Stream()
	{
		SClusterTreeView::FTreeItemPtr PinnedTreeItem = TreeItem.Pin();
		if (!PinnedTreeItem.IsValid())
		{
			return SNullWidget::NullWidget;
		}

		// Current status image only
		return SNew(SBox)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.Image(this, &SClusterTreeItemRow::GetSessionStateIcon)
			];
	}

	/** The entry point for column widgets generation */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn( const FName& ColumnName ) override
	{
		if (ColumnName == SClusterTreeView::Column_ItemConnection)
		{
			return CreateColumnWidget_Connection();
		}
		else if (ColumnName == SClusterTreeView::Column_ItemName)
		{
			return CreateColumnWidget_Name();
		}
		else if (ColumnName == SClusterTreeView::Column_ItemInfo)
		{
			return CreateColumnWidget_Info();
		}
		else if (ColumnName == SClusterTreeView::Column_ItemStream)
		{
			return CreateColumnWidget_Stream();
		}
		
		return SNullWidget::NullWidget;
	}

private:

	/** Reference to a tree item associated with this row */
	TWeakPtr<SClusterTreeView::FTreeItem> TreeItem;
};


/**
 * Widget for the filter bar, needs to be a subclass that overrides MakeAddFilterMenu to give the filter bar its own unique menu name,
 * otherwise the editor will get confused with any other basic filter bar used elsewhere.
 */
template<typename TFilterType>
class SClusterTreeFilterBar : public SBasicFilterBar<TFilterType>
{
	using Super = SBasicFilterBar<TFilterType>;
	
public:

	using FOnFilterChanged = typename SBasicFilterBar<TFilterType>::FOnFilterChanged;
	using FCreateTextFilter = typename SBasicFilterBar<TFilterType>::FCreateTextFilter;

public:

	SLATE_BEGIN_ARGS(SClusterTreeFilterBar)
	{ }
		SLATE_EVENT(SClusterTreeFilterBar<TFilterType>::FOnFilterChanged, OnFilterChanged)
		SLATE_ARGUMENT(TArray<TSharedRef<FFilterBase<TFilterType>>>, CustomFilters)
	SLATE_END_ARGS()

	/** Widget construction */
	void Construct(const FArguments& InArgs)
	{
		typename SBasicFilterBar<TFilterType>::FArguments Args;
		Args._OnFilterChanged = InArgs._OnFilterChanged;
		Args._CustomFilters = InArgs._CustomFilters;
		Args._UseSectionsForCategories = true;

		SBasicFilterBar<TFilterType>::Construct(Args.FilterPillStyle(EFilterPillStyle::Basic));
	}

private:

	/** Registers and generates the filter bar widget */
	virtual TSharedRef<SWidget> MakeAddFilterMenu() override
	{
		// Register if not already registered
		const FName FilterMenuName = "ClusterMonitorTreeFilterBar.FilterMenu";
		if (!UToolMenus::Get()->IsMenuRegistered(FilterMenuName))
		{
			UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(FilterMenuName);
			Menu->bShouldCloseWindowAfterMenuSelection = true;
			Menu->bCloseSelfOnly = true;

			Menu->AddDynamicSection(NAME_None, FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
			{
				if (UFilterBarContext* Context = InMenu->FindContext<UFilterBarContext>())
				{
					Context->PopulateFilterMenu.ExecuteIfBound(InMenu);
					Context->OnExtendAddFilterMenu.ExecuteIfBound(InMenu);
				}
			}));
		}

		// Create context
		UFilterBarContext* FilterBarContext = NewObject<UFilterBarContext>();
		FilterBarContext->PopulateFilterMenu = FOnPopulateAddFilterMenu::CreateLambda([this](UToolMenu* Menu)
		{
			Super::PopulateCommonFilterSections(Menu);
			Super::PopulateCustomFilters(Menu);
		});
		FToolMenuContext ToolMenuContext(FilterBarContext);
		
		// Create the widget
		return UToolMenus::Get()->GenerateWidget(FilterMenuName, ToolMenuContext);
	}
};


/**
 * Toolbar widget
 */
class SClusterTreeToolbar : public SCompoundWidget
{
private:

	using FFilterType = SClusterTreeView::FTreeItemPtr;
	using FClusterTextFilter = TTextFilter<FFilterType>;

public:

	DECLARE_DELEGATE(FOnFilterChanged)

	SLATE_BEGIN_ARGS(SClusterTreeToolbar)
	{ }
		SLATE_EVENT(FOnFilterChanged, OnFilterChanged)
	SLATE_END_ARGS()

	/** Widget construction */
	void Construct(const FArguments& InArgs)
	{
		OnFilterChanged = InArgs._OnFilterChanged;

		// Initialize filters
		InitializeFilters();

		// Create the filter bar
		FilterBar = SNew(SClusterTreeFilterBar<FFilterType>)
			.CustomFilters(CustomFilters)
			.OnFilterChanged(this, &SClusterTreeToolbar::FilterChanged);

		// Toolbar widget
		ChildSlot
		[
			SNew(SVerticalBox)

			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(0.0f, 0.0f, 2.0f, 0.0f)
				.AutoWidth()
				[
					SClusterTreeFilterBar<FFilterType>::MakeAddFilterButton(FilterBar.ToSharedRef())
				]

				+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				[
					SNew(SFilterSearchBox)
					.HintText(LOCTEXT("FilterSearch", "Search..."))
					.ToolTipText(LOCTEXT("FilterSearch_Hint", "Type here to search for specific media sources or outputs"))
					.OnTextChanged_Lambda([this](const FText& InText)
					{
						TextFilter->SetRawFilterText(InText);
					})
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(4.f, 0.f, 0.f, 0.f)
				[
					SNew(SComboButton)
					.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButtonWithIcon")
					.HasDownArrow(false)
					.ButtonContent()
					[
						SNew(SImage)
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Image(FAppStyle::Get().GetBrush("Icons.Settings"))
					]
				]
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			[
				FilterBar.ToSharedRef()
			]
		];
	}

	/** Tests a tree item for the current set of filters */
	bool ItemPassesFilters(const FFilterType& InItem) const
	{
		// Performs an OR check on any of the filters that are enabled in the filter bar,
		// mirroring the scene outliner filter bar behavior
		auto PassesAnyFilterBarFilter = [this](const FFilterType& InItem)
		{
			const TSharedPtr<TFilterCollection<FFilterType>> FilterCollection = FilterBar->GetAllActiveFilters();
			if (!FilterCollection.IsValid() || FilterCollection->Num() == 0)
			{
				return true;
			}

			for (const TSharedPtr<IFilter<FFilterType>>& Filter : *FilterCollection.Get())
			{
				if (Filter->PassesFilter(InItem))
				{
					return true;
				}
			}

			return false;
		};

		// Text filter + custom active filters
		return TextFilter->PassesFilter(InItem) && PassesAnyFilterBarFilter(InItem);
	}
	
private:

	/** Creates all the filter bar filters and initializes the text filter */
	void InitializeFilters()
	{
		// Iterable collection of filter types
		static const TMap<SClusterTreeView::EItemType, TTuple<const TCHAR*, FText>> ItemTypeNames
		{
			{ SClusterTreeView::EItemType::Obs_Backbuffer,      { TEXT("Backbuffer"),        LOCTEXT("ObservableItemTypeBackbuffer",      "Backbuffer")}},
			{ SClusterTreeView::EItemType::Obs_UI,              { TEXT("UI"),                LOCTEXT("ObservableItemTypeUI",              "UI") }},
			{ SClusterTreeView::EItemType::Obs_Viewport,        { TEXT("Viewport"),          LOCTEXT("ObservableItemTypeViewport",        "Viewport") }},
			{ SClusterTreeView::EItemType::Obs_ICVFXCamera,     { TEXT("ICVFX Camera"),      LOCTEXT("ObservableItemTypeICVFXCamera",     "ICVFX Camera") }},
			{ SClusterTreeView::EItemType::Obs_ICVFXCameraTile, { TEXT("ICVFX Camera Tile"), LOCTEXT("ObservableItemTypeICVFXCameraTile", "ICVFX Camera Tile") }},
		};

		// Type filters
		{
			TSharedPtr<FFilterCategory> TypeFiltersCategory = MakeShared<FFilterCategory>(LOCTEXT("TypeFiltersCategory", "Type"), FText::GetEmpty());

			// Create all filters from the collection
			for (const TPair<SClusterTreeView::EItemType, TTuple<const TCHAR*, FText>>& ItemType : ItemTypeNames)
			{
				TSharedPtr<FGenericFilter<FFilterType>> ItemTypeFilter = MakeShared<FGenericFilter<FFilterType>>(
					TypeFiltersCategory,
					FString::Format(TEXT("{0}TypeFilter"), { ItemType.Value.Key }),
					ItemType.Value.Value,
					FGenericFilter<FFilterType>::FOnItemFiltered::CreateLambda([Type = ItemType.Key](FFilterType InItem)
					{
						if (!InItem.IsValid())
						{
							return false;
						}

						const bool bTypeMatches = (Type == InItem->Type);

						return bTypeMatches;
					}));

				CustomFilters.Add(ItemTypeFilter.ToSharedRef());
			}
		}

		// Activity filters
		{
			TSharedPtr<FFilterCategory> StateFiltersCategory = MakeShared<FFilterCategory>(LOCTEXT("StateFiltersCategory", "State"), FText::GetEmpty());

			TSharedPtr<FGenericFilter<FFilterType>> ActiveSessionsFilter = MakeShared<FGenericFilter<FFilterType>>(
				StateFiltersCategory,
				TEXT("ActiveSessionsFilter"),
				LOCTEXT("ActiveSessionsFilter_Name", "Active Entities"),
				FGenericFilter<FFilterType>::FOnItemFiltered::CreateLambda([](FFilterType InItem)
					{
						if (!InItem.IsValid())
						{
							return false;
						}

						const bool bIsLeaf = InItem->IsLeaf();
						if (!bIsLeaf)
						{
							return false;
						}

						const IClusterObservable::ESessionState SessionState = InItem->Observable->GetSessionState();
						const bool bSessionRunnig = (SessionState != IClusterObservable::ESessionState::None);

						return bSessionRunnig;
					}));

			ActiveSessionsFilter->SetToolTipText(LOCTEXT("ActiveSessionsFilter_Tooltip", "Only show active observation sessions"));
			CustomFilters.Add(ActiveSessionsFilter.ToSharedRef());

			TSharedPtr<FGenericFilter<FFilterType>> InactiveSessionsFilter = MakeShared<FGenericFilter<FFilterType>>(
				StateFiltersCategory,
				TEXT("InactiveSessionsFilter"),
				LOCTEXT("InactiveSessionsFilter_Name", "Inactive Entites"),
				FGenericFilter<FFilterType>::FOnItemFiltered::CreateLambda([](FFilterType InItem)
					{
						if (!InItem.IsValid())
						{
							return false;
						}

						const bool bIsLeaf = InItem->IsLeaf();
						if (!bIsLeaf)
						{
							return false;
						}

						const IClusterObservable::ESessionState SessionState = InItem->Observable->GetSessionState();
						const bool bNoSessionRunnig = (SessionState == IClusterObservable::ESessionState::None);

						return bNoSessionRunnig;
					}));

			InactiveSessionsFilter->SetToolTipText(LOCTEXT("InactiveSessionsFilter_Tooltip", "Only show inactive entities"));
			CustomFilters.Add(InactiveSessionsFilter.ToSharedRef());
		}

		// Text filter
		{
			TextFilter = MakeShared<FClusterTextFilter>(FClusterTextFilter::FItemToStringArray::CreateLambda([&](const FFilterType& InItem, TArray<FString>& OutStrings)
				{
					if (InItem.IsValid())
					{
						OutStrings.Add(InItem->Name);

						if (const TTuple<const TCHAR*, FText>* const FoundItem = ItemTypeNames.Find(InItem->Type))
						{
							OutStrings.Add(FoundItem->Key);
						}
					}
				}));

			TextFilter->OnChanged().AddSP(this, &SClusterTreeToolbar::FilterChanged);
		}
	}

	/** Fires the filter change event */
	void FilterChanged()
	{
		OnFilterChanged.ExecuteIfBound();
	}

private:

	/** Custom filters */
	TArray<TSharedRef<FFilterBase<FFilterType>>> CustomFilters;

	/** Text filter */
	TSharedPtr<FClusterTextFilter> TextFilter;
	
	/** Filter bar */
	TSharedPtr<SClusterTreeFilterBar<FFilterType>> FilterBar;
	
	/** Filter changed delegate */
	FOnFilterChanged OnFilterChanged;
};


SClusterTreeView::EItemType SClusterTreeView::FTreeItem::GetTreeItemType(EDCObservableType InObservableType)
{
	// Here we convert leaves only. Cluster and node types are specified explicitly
	// while building the cluster trees.
	switch (InObservableType)
	{
	case EDCObservableType::None:
		return SClusterTreeView::EItemType::Unknown;

	case EDCObservableType::Backbuffer:
		return SClusterTreeView::EItemType::Obs_Backbuffer;

	case EDCObservableType::UI:
		return SClusterTreeView::EItemType::Obs_UI;

	case EDCObservableType::Viewport:
		return SClusterTreeView::EItemType::Obs_Viewport;

	case EDCObservableType::ICVFXCamera:
		return SClusterTreeView::EItemType::Obs_ICVFXCamera;

	case EDCObservableType::ICVFXCameraTile:
		return SClusterTreeView::EItemType::Obs_ICVFXCameraTile;

	default:
		checkNoEntry();
	}

	return SClusterTreeView::EItemType::Unknown;
}

const FSlateBrush* SClusterTreeView::FTreeItem::GetTreeItemIcon(SClusterTreeView::EItemType Type)
{
	// Pick an icon according to the item type
	const FSlateBrush* ItemIcon = nullptr;
	switch (Type)
	{
		case SClusterTreeView::EItemType::Cluster:
			ItemIcon = FDCMonitorEditorStyle::Get().GetBrush("ClusterMonitor.TreeItem.Cluster");
			break;

		case SClusterTreeView::EItemType::Node:
			ItemIcon = FDCMonitorEditorStyle::Get().GetBrush("ClusterMonitor.TreeItem.Node");
			break;

		case SClusterTreeView::EItemType::NodeOffscreen:
			ItemIcon = FDCMonitorEditorStyle::Get().GetBrush("ClusterMonitor.TreeItem.NodeOffscreen");
			break;

		case SClusterTreeView::EItemType::Obs_Backbuffer:
			ItemIcon = FDCMonitorEditorStyle::Get().GetBrush("ClusterMonitor.TreeItem.Backbuffer");
			break;

		case SClusterTreeView::EItemType::Obs_UI:
			ItemIcon = FDCMonitorEditorStyle::Get().GetBrush("ClusterMonitor.TreeItem.UI");
			break;

		case SClusterTreeView::EItemType::Obs_Viewport:
			ItemIcon = FDCMonitorEditorStyle::Get().GetBrush("ClusterMonitor.TreeItem.Viewport");
			break;

		case SClusterTreeView::EItemType::ICVFXCameraTiled:
		case SClusterTreeView::EItemType::Obs_ICVFXCamera:
			ItemIcon = FDCMonitorEditorStyle::Get().GetBrush("ClusterMonitor.TreeItem.ICVFXCamera");
			break;

		case SClusterTreeView::EItemType::Obs_ICVFXCameraTile:
			ItemIcon = FDCMonitorEditorStyle::Get().GetBrush("ClusterMonitor.TreeItem.ICVFXCameraTile");
			break;

		case SClusterTreeView::EItemType::Unknown:
			ItemIcon = nullptr;
			break;

		default:
		ItemIcon = nullptr;
		break;
	}

	return ItemIcon;
}

bool SClusterTreeView::FTreeItem::IsLeaf() const
{
	const bool bNotNode    = !IsClusterNode();
	const bool bNotCluster = (Type != SClusterTreeView::EItemType::Cluster);
	const bool bNotGroup   = (Type != SClusterTreeView::EItemType::ICVFXCameraTiled);
	const bool bNotUnknown = (Type != SClusterTreeView::EItemType::Unknown);

	return bNotNode && bNotCluster && bNotGroup && bNotUnknown;
}

bool SClusterTreeView::FTreeItem::IsClusterNode() const
{
	return Type == SClusterTreeView::EItemType::Node || Type == SClusterTreeView::EItemType::NodeOffscreen;
}

void SClusterTreeView::FTreeItem::SortChildren()
{
	Children.Sort([&](const TSharedPtr<FTreeItem>& LHS, const TSharedPtr<FTreeItem>& RHS)
		{
			using FOrderPriorityType = uint8;

			// Sorting order based on the item types
			static const TMap<EItemType, FOrderPriorityType> SortPriorities{
				{EItemType::Cluster,              0 },
				{EItemType::Node,                20 },
				{EItemType::NodeOffscreen,       21 },
				{EItemType::Obs_Backbuffer,      40 },
				{EItemType::Obs_UI,              41 },
				{EItemType::Obs_Viewport,        42 },
				{EItemType::Obs_ICVFXCamera,     43 },
				{EItemType::ICVFXCameraTiled,    44 },
				{EItemType::Obs_ICVFXCameraTile, 64 },
				{EItemType::Unknown, TNumericLimits<FOrderPriorityType>::Max() }
			};

			checkSlow(LHS.IsValid() && RHS.IsValid());
			checkSlow(SortPriorities.Contains(LHS->Type) && SortPriorities.Contains(RHS->Type));

			const FOrderPriorityType LHSPrio = LHS.IsValid() ? SortPriorities[LHS->Type] : 0;
			const FOrderPriorityType RHSPrio = RHS.IsValid() ? SortPriorities[RHS->Type] : 0;

			return (LHSPrio == RHSPrio ?
				// Sort entities of the same type by name in lexicographical order
				LHS->Name.Compare(RHS->Name, ESearchCase::IgnoreCase) < 0 :
				// Different types are sorted by priority
				LHSPrio < RHSPrio);
		});
}

void SClusterTreeView::FTreeItem::Update()
{
	Resolution = Observable->GetResolution();
}

TSharedPtr<SClusterTreeView::FTreeItem> SClusterTreeView::FTreeItem::GetTopMostParent() const
{
	// Go to the top of hierarchy
	TSharedPtr<FTreeItem> Current = Parent.Pin();
	while (Current.IsValid() && Current->Parent.IsValid())
	{
		Current = Current->Parent.Pin();
	}

	return Current;
}


void SClusterTreeView::Construct(const FArguments& InArgs, const TSharedPtr<IClusterMonitorController>& InController)
{
	Controller = InController;

	// Listen to external changes
	if (TSharedPtr<IClusterMonitorController> PinnedController = Controller.Pin())
	{
		PinnedController->OnObservableJoined().AddSP(this,  &SClusterTreeView::OnObservableJoined);
		PinnedController->OnObservableUpdated().AddSP(this, &SClusterTreeView::OnObservableUpdated);
		PinnedController->OnObservableLeft().AddSP(this,    &SClusterTreeView::OnObservableLeft);
		PinnedController->OnObservableTimeout().AddSP(this, &SClusterTreeView::OnObservableTimeout);
		PinnedController->OnSessionStarted().AddSP(this,    &SClusterTreeView::OnSessionChanged);
		PinnedController->OnSessionStopped().AddSP(this,    &SClusterTreeView::OnSessionChanged);
	}

	// Whole tree view widget
	ChildSlot
	[
		SNew(SVerticalBox)

		// Toolbar
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(8.0f, 8.0f, 8.0f, 4.0f)
		[
			SAssignNew(Toolbar, SClusterTreeToolbar)
			.OnFilterChanged(this, &SClusterTreeView::OnFilterChanged)
		]
		
		// Cluster tree
		+SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SAssignNew(TreeView, STreeView<FTreeItemPtr>)
			.TreeItemsSource(&TreeItems)
			.SelectionMode(ESelectionMode::Single)
			.OnGetChildren(this, &SClusterTreeView::GetTreeItemChildren)
			.OnGenerateRow(this, &SClusterTreeView::GenerateTreeItemRow)
			.OnMouseButtonDoubleClick(this, &SClusterTreeView::OnItemDoubleClicked)
			.HeaderRow
			(
				SNew(SHeaderRow)

				+SHeaderRow::Column(Column_ItemConnection)
				.FillWidth(0.1f)
				.ToolTipText(LOCTEXT("TreeItemColumnConnection_Tooltip", "Connection status"))
				.HeaderContent()
				[
					SNew(SBox)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.Image(FDCMonitorEditorStyle::Get().GetBrush("ClusterMonitor.TreeColumn.Connection"))
					]
				]

				+SHeaderRow::Column(Column_ItemName)
				.FillWidth(0.5f)
				.DefaultLabel(LOCTEXT("TreeItemColumnName_Label", "Name"))

				+SHeaderRow::Column(Column_ItemInfo)
				.FillWidth(0.3f)
				.DefaultLabel(LOCTEXT("TreeItemColumnInfo_Label", "Info"))

				+SHeaderRow::Column(Column_ItemStream)
				.FillWidth(0.1f)
				.ToolTipText(LOCTEXT("TreeItemColumnStream_Tooltip", "Streaming status"))
				.HeaderContent()
				[
					SNew(SBox)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.Image(FDCMonitorEditorStyle::Get().GetBrush("ClusterMonitor.TreeColumn.Streaming"))
					]
				]
			)
		]
	];

	// Expand all by default
	ExpandAll();
}

SClusterTreeView::~SClusterTreeView()
{
	if (TSharedPtr<IClusterMonitorController> PinnedController = Controller.Pin())
	{
		PinnedController->OnObservableJoined().RemoveAll(this);
		PinnedController->OnObservableUpdated().RemoveAll(this);
		PinnedController->OnObservableLeft().RemoveAll(this);
		PinnedController->OnObservableTimeout().RemoveAll(this);
		PinnedController->OnSessionStarted().RemoveAll(this);
		PinnedController->OnSessionStopped().RemoveAll(this);
	}
}

void SClusterTreeView::ExpandAll(bool bIncludingChildren)
{
	SetExpansion(true, bIncludingChildren);
}

void SClusterTreeView::CollapseAll(bool bIncludingChildren)
{
	SetExpansion(false, bIncludingChildren);
}

void SClusterTreeView::SetExpansion(FTreeItemPtr InTreeItem, bool bExpanded, bool bIncludingChildren)
{
	// Process the top most and its children recursively
	if (bIncludingChildren)
	{
		TFunction<void(const FTreeItemPtr&)> SetExpansionRecursive;

		SetExpansionRecursive = [&](const FTreeItemPtr& Item)
			{
				if (!Item.IsValid())
				{
					return;
				}

				TreeView->SetItemExpansion(Item, bExpanded);

				for (const FTreeItemPtr& Child : Item->Children)
				{
					SetExpansionRecursive(Child);
				}
			};

		SetExpansionRecursive(InTreeItem);
	}
	// Process top level items only
	else
	{
		TreeView->SetItemExpansion(InTreeItem, bExpanded);
	}

	// Redraw the UI
	TreeView->RequestTreeRefresh();
}

void SClusterTreeView::SetExpansion(bool bExpanded, bool bIncludingChildren)
{
	// Expand or collapse all the top level elements
	for (const FTreeItemPtr& TreeItem : TreeItems)
	{
		SetExpansion(TreeItem, bExpanded, bIncludingChildren);
	}

	// Redraw the UI
	TreeView->RequestTreeRefresh();
}

TSharedRef<ITableRow> SClusterTreeView::GenerateTreeItemRow(FTreeItemPtr InTreeItem, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	return SNew(SClusterTreeItemRow, InOwnerTableView, InTreeItem);
}

SClusterTreeView::FTreeItemPtr SClusterTreeView::FindOrCreateClusterItem(const TSharedRef<IClusterObservable>& InObservable)
{
	// See if already exists
	const TSharedRef<IClusterResidence> Residence = InObservable->GetResidence();
	FTreeItemPtr* FoundItem = TreeItems.FindByPredicate([&Residence](const FTreeItemPtr& Item)
		{
			return Item->Id == Residence->GetClusterId();
		});

	// If exists, return it
	if (FoundItem)
	{
		return *FoundItem;
	}

	// Otherwise, create new tree item
	FTreeItemPtr NewItem = FTreeItem::Create(Residence->GetClusterId());
	NewItem->Name = Residence->GetClusterName();
	NewItem->Type = SClusterTreeView::EItemType::Cluster;

	// And store it internally
	TreeItems.Add(NewItem);

	return NewItem;
}

SClusterTreeView::FTreeItemPtr SClusterTreeView::FindOrCreateNodeItem(const TSharedRef<IClusterObservable>& InObservable)
{
	// Get cluster item
	FTreeItemPtr ClusterItem = FindOrCreateClusterItem(InObservable);
	if (!ClusterItem.IsValid())
	{
		return nullptr;
	}

	// Now see if this node already exists
	const TSharedRef<IClusterResidence> Residence = InObservable->GetResidence();
	FTreeItemPtr* FoundItem = ClusterItem->Children.FindByPredicate([&Residence](const FTreeItemPtr& Item)
		{
			return Item->Id == Residence->GetNodeId();
		});

	// If exists, return it
	if (FoundItem)
	{
		return *FoundItem;
	}

	// Otherwise, create new tree item
	FTreeItemPtr NewNodeItem = FTreeItem::Create(Residence->GetNodeId());
	NewNodeItem->Parent    = ClusterItem;
	NewNodeItem->Name      = Residence->GetNodeName();
	NewNodeItem->Type      = Residence->IsNodeOffscreen() ? SClusterTreeView::EItemType::NodeOffscreen : SClusterTreeView::EItemType::Node;
	NewNodeItem->Residence = Residence;

	// And store it internally
	ClusterItem->Children.Add(NewNodeItem);

	// Also re-sort children so the new item takes the right place
	ClusterItem->SortChildren();

	return NewNodeItem;
}

SClusterTreeView::FTreeItemPtr SClusterTreeView::FindObservableItem(const TSharedRef<IClusterObservable>& InObservable)
{
	return FTreeItem::Find(InObservable->GetId());
}

SClusterTreeView::FTreeItemPtr SClusterTreeView::CreateObservableItem(const TSharedRef<IClusterObservable>& InObservable)
{
	// Get cluster node item
	FTreeItemPtr ParentItem = FindOrCreateNodeItem(InObservable);
	if (!ParentItem)
	{
		return nullptr;
	}

	// For tiles, we need to create a group item
	const bool bIsTile = InObservable->IsTile();
	if (bIsTile)
	{
		//@note
		// So far, only camera tiles supported so EItemType::ICVFXCameraTiled is hard-coded. If we ever have
		// another tile owners (like viewport tiles), we would need to generalize the group creation.
		const TOptional<FString> ParentName = InObservable->GetParentName();
		ParentItem = FindOrCreateGroupItem(ParentItem, EItemType::ICVFXCameraTiled, ParentName.Get(TEXT("CameraTiled")));
	}

	// Make sure it has not been added earlier
	{
		const bool bAlreadyExists = ParentItem->Children.ContainsByPredicate([Id = InObservable->GetId()](const FTreeItemPtr& Item)
			{
				return Item->Id == Id;
			});

		if (bAlreadyExists)
		{
			checkfSlow(false, TEXT("Observable name=%ls, id=%ls already exists"), *InObservable->GetName(), *InObservable->GetId().ToString());
			return nullptr;
		};
	}

	// Create new item
	FTreeItemPtr NewObsItem = FTreeItem::Create(InObservable->GetId());
	NewObsItem->Parent     = ParentItem;
	NewObsItem->Name       = InObservable->GetName();
	NewObsItem->Type       = FTreeItem::GetTreeItemType(InObservable->GetType());
	NewObsItem->Resolution = InObservable->GetResolution();
	NewObsItem->Observable = InObservable;
	NewObsItem->Residence  = InObservable->GetResidence();

	// And store it internally
	ParentItem->Children.Add(NewObsItem);

	// Also re-sort children so the new item takes the right place
	ParentItem->SortChildren();

	return NewObsItem;
}

SClusterTreeView::FTreeItemPtr SClusterTreeView::FindOrCreateGroupItem(FTreeItemPtr NodeItem, EItemType InType, const FString& InName)
{
	// Search if already exists
	FTreeItemPtr* FoundItem = NodeItem->Children.FindByPredicate([&InType, &InName](const FTreeItemPtr& Item)
		{
			return Item->Type == InType && Item->Name.Equals(InName, ESearchCase::IgnoreCase);
		});

	// If so, return existing item
	if (FoundItem)
	{
		return *FoundItem;
	}

	// Otherwise, create a new one
	FTreeItemPtr NewGroupItem = FTreeItem::Create(FGuid::NewGuid());
	NewGroupItem->Parent = NodeItem;
	NewGroupItem->Name   = InName;
	NewGroupItem->Type   = InType;

	// And store it internally
	NodeItem->Children.Add(NewGroupItem);

	// Also re-sort children so the new item takes the right place
	NodeItem->SortChildren();

	return NewGroupItem;
}

void SClusterTreeView::GetTreeItemChildren(FTreeItemPtr InTreeItem, TArray<FTreeItemPtr>& OutChildren) const
{
	if (InTreeItem.IsValid())
	{
		for (const FTreeItemPtr& Child : InTreeItem->Children)
		{
			if (Child.IsValid() && !Child->bFilteredOut)
			{
				OutChildren.Add(Child);
			}
		}
	}
}

void SClusterTreeView::TryStartObservationSession(FTreeItemPtr InTreeItem)
{
	if (!InTreeItem.IsValid())
	{
		return;
	}

	if (!InTreeItem->Observable.IsValid())
	{
		return;
	}

	// Get controller
	TSharedPtr<IClusterMonitorController> PinnedController = Controller.Pin();
	if (!PinnedController.IsValid())
	{
		return;
	}

	// Get session state and GUID
	const IClusterObservable::ESessionState SessionState = InTreeItem->Observable->GetSessionState();
	const FGuid ObservableId = InTreeItem->Observable->GetId();
	const bool bHasRunningSession = InTreeItem->Observable->IsSessionRunning();

	// If has started a session
	if (bHasRunningSession)
	{
		// Try restart if not live
		if (SessionState == IClusterObservable::ESessionState::Error ||
			SessionState == IClusterObservable::ESessionState::Inactive)
		{
			PinnedController->RequestSessionStop(ObservableId);
			PinnedController->RequestSessionStart(ObservableId);
		}
		// Otherwise nothing to do, just leave
		else
		{
			return;
		}
	}
	// If no session running
	else
	{
		// Start new session
		PinnedController->RequestSessionStart(ObservableId);
	}

	// Redraw the UI
	TreeView->RequestTreeRefresh();
}

void SClusterTreeView::RemoveEmptyItems()
{
	// Forward declaration of the lambda below so it can call itself recursively
	TFunction<bool(FTreeItemPtr&)> ClearEmptyGroups;

	// Recursively iterate whole tree, and remove all empty branches
	ClearEmptyGroups = [&](FTreeItemPtr& Item) -> bool
		{
			// Should be removed
			if (!Item.IsValid())
			{
				return false;
			}

			// Don't delete this branch
			if (Item->IsLeaf())
			{
				return true;
			}

			// Recursively iterate all children
			bool bHasAnyLeaves = false;
			for (auto It = Item->Children.CreateIterator(); It; ++It)
			{
				// Remove if no leaves underneath
				const bool bItemHasAnyLeaves = ClearEmptyGroups(*It);
				if (!bItemHasAnyLeaves)
				{
					It.RemoveCurrent();
				}

				bHasAnyLeaves |= bItemHasAnyLeaves;
			}

			// Forward leaf availability to the upper level
			return bHasAnyLeaves;
		};

	// Iterate every tree branch
	for (auto It = TreeItems.CreateIterator(); It; ++It)
	{
		// Process this item branch
		const bool bHasAnyLeaves = ClearEmptyGroups(*It);
		if (!bHasAnyLeaves)
		{
			It.RemoveCurrent();
		}
	}
}

void SClusterTreeView::OnObservableJoined(const TSharedRef<IClusterObservable>& InObservable)
{
	// Create new tree item
	const FTreeItemPtr NodeItem = CreateObservableItem(InObservable);
	if (!NodeItem.IsValid())
	{
		return;
	}

	// Get top most parent
	const FTreeItemPtr TopMostItem = NodeItem->GetTopMostParent();
	if (!TopMostItem.IsValid())
	{
		return;
	}

	// Expand new item's tree
	static constexpr bool bExpanded = true;
	static constexpr bool bIncludingChildren = true;
	SetExpansion(TopMostItem, bExpanded, bIncludingChildren);
}

void SClusterTreeView::OnObservableUpdated(const TSharedRef<IClusterObservable>& InObservable)
{
	// Get the tree item associated with this obervable
	const FTreeItemPtr NodeItem = FindObservableItem(InObservable);
	if (!NodeItem.IsValid())
	{
		return;
	}

	// Let the node item update itself
	NodeItem->Update();

	// Redraw the UI
	TreeView->RequestTreeRefresh();
}

void SClusterTreeView::OnObservableLeft(const TSharedRef<IClusterObservable>& InObservable, const FString& InReason)
{
	// Find corresponding tree item
	const FTreeItemPtr Child = FindObservableItem(InObservable);
	if (!Child.IsValid())
	{
		return;
	}

	// Remove from the parent's list
	if (FTreeItemPtr PinnedParent = Child->Parent.Pin())
	{
		PinnedParent->Children.Remove(Child);
	}

	// GUID of the observable entity being removed
	const FGuid ObservableId = InObservable->GetId();

	// First, remove from the items array
	TreeItems.RemoveAll([&ObservableId](const FTreeItemPtr& Item)
		{
			return Item.IsValid() && Item->Id == ObservableId;
		});

	// Remove empty group items
	RemoveEmptyItems();

	// Redraw the UI
	TreeView->RequestTreeRefresh();
}

void SClusterTreeView::OnObservableTimeout(const TSharedRef<IClusterObservable>& InObservable)
{
	TreeView->RequestTreeRefresh();
}

void SClusterTreeView::OnSessionChanged(const TSharedRef<IClusterObservable>& InObservable)
{
	TreeView->RequestTreeRefresh();
}

void SClusterTreeView::OnItemDoubleClicked(FTreeItemPtr InTreeItem)
{
	if (!InTreeItem.IsValid())
	{
		return;
	}

	const bool bIsLeaf = InTreeItem->IsLeaf();
	if (bIsLeaf)
	{
		// Start observation session if not started yet
		TryStartObservationSession(InTreeItem);
	}
	else
	{
		// Toggle expansion
		const bool bIsExpanded = TreeView->IsItemExpanded(InTreeItem);
		TreeView->SetItemExpansion(InTreeItem, !bIsExpanded);
	}
}

void SClusterTreeView::OnFilterChanged()
{
	// Forward declaration of the lambda below so it can call itself recursively
	TFunction<uint32(const FTreeItemPtr&)> FilterTreeItems;

	// Recursively filters tree items based on the currently active filters.
	// If an item passes any filter, all of its parents remain visible as well.
	FilterTreeItems = [&](const FTreeItemPtr& Item) -> uint32
		{
			// Ignore any invalid input
			if (!Item.IsValid())
			{
				return 0;
			}

			// Recursively iterate all children, and remember the passed count
			uint32 ChildrenPassedAnyFilter = 0;
			for (const FTreeItemPtr& Child : Item->Children)
			{
				ChildrenPassedAnyFilter += FilterTreeItems(Child);
			}

			// If any child has passed, there is no sense to test this item.
			// It's automatically marked as passed.
			if (ChildrenPassedAnyFilter > 0)
			{
				Item->bFilteredOut = false;
				++ChildrenPassedAnyFilter; // Plus this
			}
			// No children have passed. Then test this item.
			else
			{
				const bool bPassedFiltering = Toolbar->ItemPassesFilters(Item);
				Item->bFilteredOut = !bPassedFiltering;
				ChildrenPassedAnyFilter += (bPassedFiltering) ? 1 : 0;
			}

			// Return passed count at this depth
			return ChildrenPassedAnyFilter;
		};

	// Let every tree iten to be tested by a set of active filters
	for (const FTreeItemPtr& TreeItem : TreeItems)
	{
		// Process this item branch
		FilterTreeItems(TreeItem);
	}

	// Redraw the UI with all items expanded
	ExpandAll();
}

#undef LOCTEXT_NAMESPACE
