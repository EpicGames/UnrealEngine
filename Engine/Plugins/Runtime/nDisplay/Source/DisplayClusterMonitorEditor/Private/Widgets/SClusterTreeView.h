// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "UObject/NameTypes.h"
#include "Widgets/Views/STreeView.h"

#include "DisplayClusterMonitorTypes.h"

class IClusterMonitorController;
class IClusterObservable;
class IClusterResidence;
class SClusterTreeToolbar;


/**
 * A tree view that shows all available clusters and their observables.
 * Also provides interface to manipulate the observation sessions.
 */
class SClusterTreeView : public SCompoundWidget
{
	friend class SClusterTreeItemRow;
	friend class SClusterTreeToolbar;

private:

	/**
	 * Tree item types
	 */
	enum class EItemType : uint8
	{
		Unknown,             // For validation
		Cluster,             // Cluster root item
		Node,                // Onscreen cluster node
		NodeOffscreen,       // Offscreen cluster node
		ICVFXCameraTiled,    // Non-observable group of tiles
		Obs_Backbuffer,      // Backbuffer (observable)
		Obs_UI,              // UI layer (observable)
		Obs_Viewport,        // Viewport (observable)
		Obs_ICVFXCamera,     // ICVFX camera (observable)
		Obs_ICVFXCameraTile, // ICVFX camera tile (observable)
	};

	/**
	 * Struct to store information for each element displayed in the tree view
	 */
	struct FTreeItem : public TSharedFromThis<FTreeItem>
	{
	public:

		/** Factory method to create new tree items */
		static TSharedRef<FTreeItem> Create(const FGuid& InGuid)
		{
			checkSlow(!TreeItemGuids.Contains(InGuid));

			// Instantiate and register
			TSharedRef<FTreeItem> NewItem = MakeShareable(new FTreeItem(InGuid));
			TreeItemGuids.Emplace(InGuid, NewItem);
			return NewItem;
		}

		/** Quick search for a tree item by GUID */
		static TSharedPtr<FTreeItem> Find(const FGuid& InGuid)
		{
			TWeakPtr<FTreeItem>* FoundItem = TreeItemGuids.Find(InGuid);
			return FoundItem ? FoundItem->Pin() : nullptr;
		}

	private:

		/** No direct instantiation */
		FTreeItem(const FGuid& InGuid)
			: Id(InGuid)
		{ }

	public:

		~FTreeItem()
		{
			// Unregister
			TreeItemGuids.Remove(Id);
		}

	public:

		/** Item residence (optional) */
		TSharedPtr<IClusterResidence>  Residence;

		/** Item observable (optional) */
		TSharedPtr<IClusterObservable> Observable;

		/** Item type */
		EItemType Type = EItemType::Unknown;

		/** Item GUID */
		FGuid Id;

		/** Item name */
		FString Name;

		/** Item resolution (optional) */
		FIntPoint Resolution = FIntPoint::ZeroValue;

		/** Whether this item is currently filtered out */
		bool bFilteredOut = false;

		/** Parent item in the tree */
		TWeakPtr<FTreeItem> Parent;

		/** Child items */
		TArray<TSharedPtr<FTreeItem>> Children;
		
	public:

		/** Converts observable types into tree item types */
		static EItemType GetTreeItemType(EDCObservableType InObservableType);

		/** Returns icon for a specific tree item type */
		static const FSlateBrush* GetTreeItemIcon(EItemType Type);

	public:

		/** Returns true if this tree item is a leaf */
		bool IsLeaf() const;

		/** Returns true if this item is a cluster node */
		bool IsClusterNode() const;

		/** Sort children items */
		void SortChildren();

		/** Updates local tree item data from the original observable item */
		void Update();

		/** Returns the top most parent tree item of this one */
		TSharedPtr<FTreeItem> GetTopMostParent() const;

	private:

		/** Tree items map for quick access */
		inline static TMap<FGuid, TWeakPtr<FTreeItem>> TreeItemGuids;
	};

	using FTreeItemPtr = TSharedPtr<FTreeItem>;

public:

	SLATE_BEGIN_ARGS(SClusterTreeView)
	{ }
	SLATE_END_ARGS()

	/** Widget construction */
	void Construct(const FArguments& InArgs, const TSharedPtr<IClusterMonitorController>& InController);

	/** Widget destruction */
	virtual ~SClusterTreeView() override;

public:

	/** Expands all tree items */
	void ExpandAll(bool bIncludingChildren = true);

	/** Collapses all tree items */
	void CollapseAll(bool bIncludingChildren = true);

	/** Expand or collapse specific tree item */
	void SetExpansion(FTreeItemPtr InTreeItem, bool bExpanded, bool bIncludingChildren);

	/** Expands or collapses all tree items */
	void SetExpansion(bool bExpanded, bool bIncludingChildren = true);

private:

	/** Generates new row */
	TSharedRef<ITableRow> GenerateTreeItemRow(FTreeItemPtr InTreeItem, const TSharedRef<STableViewBase>& InOwnerTableView);

	/** Finds or creates a 1st level cluster item */
	FTreeItemPtr FindOrCreateClusterItem(const TSharedRef<IClusterObservable>& InObservable);

	/** Finds or creates a 2nd level cluster node item */
	FTreeItemPtr FindOrCreateNodeItem(const TSharedRef<IClusterObservable>& InObservable);

	/** Returns a tree item for a specified observable */
	FTreeItemPtr FindObservableItem(const TSharedRef<IClusterObservable>& InObservable);

	/** Creates a tree item for a specified observable */
	FTreeItemPtr CreateObservableItem(const TSharedRef<IClusterObservable>& InObservable);

	/** Finds or creates a tile group tree item */
	FTreeItemPtr FindOrCreateGroupItem(FTreeItemPtr NodeItem, EItemType InType, const FString& InName);

private:

	/** Reports all children that aren't filtered out currently */
	void GetTreeItemChildren(FTreeItemPtr InTreeItem, TArray<FTreeItemPtr>& OutChildren) const;

	/** Starts obsevation of a specified observable if not started yet */
	void TryStartObservationSession(FTreeItemPtr InTreeItem);

	/** Removes empty tree items that aren't the leaves */
	void RemoveEmptyItems();

private:

	/** Called when a new observable is discovered, and becomes available for observation. */
	void OnObservableJoined(const TSharedRef<IClusterObservable>& InObservable);

	/** Called when the state of an observable has been updated */
	void OnObservableUpdated(const TSharedRef<IClusterObservable>& InObservable);

	/** Called when an observable shuts down */
	void OnObservableLeft(const TSharedRef<IClusterObservable>& InObservable, const FString& InReason);

	/** Called when an observable stops responding and is considered timed out */
	void OnObservableTimeout(const TSharedRef<IClusterObservable>& InObservable);

	/** Called when the observation session state for an observable changes */
	void OnSessionChanged(const TSharedRef<IClusterObservable>& InObservable);

	/** Handles LMB double clicks on tree items */
	void OnItemDoubleClicked(FTreeItemPtr InTreeItem);

	/**
	 * Raised when the filter has changed in the toolbar,
	 * updates the tree view's source list to show only filtered items
	 */
	void OnFilterChanged();

private:

	/** Tree view toolbar that provides some additional tree manipulation tools (filters) */
	TSharedPtr<SClusterTreeToolbar> Toolbar;

	/** Cluster items displayed in the tree view */
	TArray<FTreeItemPtr> TreeItems;

	/** Tree view based cluster outliner widget */
	TSharedPtr<STreeView<FTreeItemPtr>> TreeView;

	/** A reference to the cluster monitor controller */
	TWeakPtr<IClusterMonitorController> Controller;

private:

	/** Column name: connection state */
	static const FLazyName Column_ItemConnection;
	/** Column name: item name */
	static const FLazyName Column_ItemName;
	/** Column name: item extra information */
	static const FLazyName Column_ItemInfo;
	/** Column name: item session state */
	static const FLazyName Column_ItemStream;
};
