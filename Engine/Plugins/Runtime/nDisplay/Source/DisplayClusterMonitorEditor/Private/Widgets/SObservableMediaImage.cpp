// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SObservableMediaImage.h"

#include "Core/IClusterObservable.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SScrollBar.h"
#include "Widgets/Layout/SScrollBox.h"

#include "MediaTexture.h"


void SObservableMediaImage::Construct(const FArguments& InArgs, TSharedPtr<IClusterObservable> InObservable)
{
	Observable = InObservable;

	// Store zoom config (use sensible defaults if caller left them at 0)
	ZoomStep = FMath::Clamp(InArgs._ZoomStep, 0.1f, 1.f);
	ZoomMin  = FMath::Clamp(InArgs._ZoomMin,  0.1f, 1.f);
	ZoomMax  = FMath::Clamp(InArgs._ZoomMax,  1.f,  8.f);

	// Media texture to observe
	UMediaTexture* MediaTexture = nullptr;

	if (InObservable.IsValid())
	{
		// Listen to the updates notifications
		InObservable->OnObservableUpdated().AddSP(this, &SObservableMediaImage::OnObservableUpdated);

		// Initial image size (it may be changed in runtime)
		OriginalImageSize = InObservable->GetResolution();

		// Get media texture for the observable
		MediaTexture = InObservable.IsValid() ? InObservable->GetMediaTexture() : nullptr;
		checkSlow(MediaTexture);
	}

	// Instantiate vertical scrollbar
	SAssignNew(VScrollBar, SScrollBar)
		.Orientation(Orient_Vertical)
		.Thickness(InArgs._ScrollBarThickness);

	// Instantiate horizontal scrollbar
	SAssignNew(HScrollBar, SScrollBar)
		.Orientation(Orient_Horizontal)
		.Thickness(InArgs._ScrollBarThickness);

	// Instantiate media image widget in a safe way
	TSharedPtr<SWidget> MediaImageWidget = SNullWidget::NullWidget;
	if (MediaTexture)
	{
		MediaImageWidget = SNew(SMediaImage, MediaTexture)
			.BrushImageSize_Lambda([this]()
				{
					return OriginalImageSize;
				});
	}

	ChildSlot
	[
		SNew(SGridPanel)
		.FillColumn(0, 1.f)
		.FillRow(0, 1.f)

		// Slot [0,0] — scrollable media image area
		+SGridPanel::Slot(0, 0)
		[
			// Vertical scrollbox
			SAssignNew(ScrollBoxV, SScrollBox)
			.Orientation(Orient_Vertical)
			.ExternalScrollbar(VScrollBar)
			.ConsumeMouseWheel(EConsumeMouseWheel::Never)
	
			+SScrollBox::Slot()
			[
				// Horisontal scrollbox
				SAssignNew(ScrollBoxH, SScrollBox)
				.Orientation(Orient_Horizontal)
				.ExternalScrollbar(HScrollBar)
				.ConsumeMouseWheel(EConsumeMouseWheel::Never)
				+SScrollBox::Slot()
				[
					// Image size box
					SAssignNew(ImageSizeBox, SBox)
					.WidthOverride(OriginalImageSize.X)
					.HeightOverride(OriginalImageSize.Y)
					[
						MediaImageWidget.ToSharedRef()
					]
				]
			]
		]

		// Slot [1,0] — vertical scrollbar (right edge)
		+SGridPanel::Slot(1, 0)
		[
			VScrollBar.ToSharedRef()
		]

		// Slot [0,1] — horizontal scrollbar (bottom edge)
		+SGridPanel::Slot(0, 1)
		[
			HScrollBar.ToSharedRef()
		]
	];
}

SObservableMediaImage::~SObservableMediaImage()
{
	if (TSharedPtr<IClusterObservable> PinnedObservable = Observable.Pin())
	{
		PinnedObservable->OnObservableUpdated().RemoveAll(this);
	}
}

FReply SObservableMediaImage::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// When LMB is pressed, activate pan mode
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		bIsPanning = true;
		PanStartMousePos      = MouseEvent.GetScreenSpacePosition();
		PanStartScrollOffsetH = ScrollBoxH->GetScrollOffset();
		PanStartScrollOffsetV = ScrollBoxV->GetScrollOffset();

		// Capture the mouse so we keep receiving OnMouseMove events
		// even when the cursor leaves the widget boundary
		return FReply::Handled().CaptureMouse(SharedThis(this));
	}

	return FReply::Unhandled();
}

FReply SObservableMediaImage::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// When LMB is released, deactivate pan mode, and release mouse input
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton && bIsPanning)
	{
		bIsPanning = false;
		return FReply::Handled().ReleaseMouseCapture();
	}

	return FReply::Unhandled();
}

FReply SObservableMediaImage::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// If pan mode is active, move the image according to input
	if (bIsPanning)
	{
		const FVector2D CurrentMousePos = MouseEvent.GetScreenSpacePosition();
		const FVector2D Delta           = CurrentMousePos - PanStartMousePos;

		// Scrolling opposite to drag direction
		const float NewOffsetH = PanStartScrollOffsetH - Delta.X;
		const float NewOffsetV = PanStartScrollOffsetV - Delta.Y;

		ScrollBoxH->SetScrollOffset(NewOffsetH);
		ScrollBoxV->SetScrollOffset(NewOffsetV);

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply SObservableMediaImage::OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// Compute requested scale
	const float WheelDelta = MouseEvent.GetWheelDelta();
	const float NewScale   = FMath::Clamp(CurrentZoom + WheelDelta * ZoomStep, ZoomMin, ZoomMax);

	// If changed, update related widgets
	if (!FMath::IsNearlyEqual(NewScale, CurrentZoom))
	{
		ApplyZoom(NewScale, MyGeometry, MouseEvent);
	}

	return FReply::Handled();
}

float SObservableMediaImage::GetCurrentZoom() const
{
	return CurrentZoom;
}

void SObservableMediaImage::ZoomToFit()
{
	// Check for zeroes before further division
	if (FMath::IsNearlyZero(OriginalImageSize.X) || FMath::IsNearlyZero(OriginalImageSize.Y))
	{
		return;
	}

	// Layout size
	const FVector2D LocalSize = GetTickSpaceGeometry().GetLocalSize();
	if (FMath::IsNearlyZero(LocalSize.X) || FMath::IsNearlyZero(LocalSize.Y))
	{
		return;
	}

	// Horizontal and vertical relations
	const float RelH = LocalSize.X / OriginalImageSize.X;
	const float RelV = LocalSize.Y / OriginalImageSize.Y;

	// Minimal relation is exactly what we need to fit the image.
	CurrentZoom = FMath::Min(RelH, RelV) * 0.99f;

	// Update image box size according to new image size
	const FVector2D NewBoxSize = OriginalImageSize * CurrentZoom;
	ImageSizeBox->SetWidthOverride(NewBoxSize.X);
	ImageSizeBox->SetHeightOverride(NewBoxSize.Y);

	// Set new scroll offsets
	ScrollBoxH->SetScrollOffset(0.f);
	ScrollBoxV->SetScrollOffset(0.f);

	// Redraw widgets
	Invalidate(EInvalidateWidgetReason::Layout);
}

void SObservableMediaImage::ZoomTo100()
{
	// Reset to 100%
	CurrentZoom = 1.f;

	// Update image box size according to new image size
	const FVector2D NewBoxSize = OriginalImageSize * CurrentZoom;
	if (FMath::IsNearlyZero(NewBoxSize.X) || FMath::IsNearlyZero(NewBoxSize.Y))
	{
		return;
	}

	ImageSizeBox->SetWidthOverride(NewBoxSize.X);
	ImageSizeBox->SetHeightOverride(NewBoxSize.Y);

	// Set new scroll offsets
	ScrollBoxH->SetScrollOffset(0.f);
	ScrollBoxV->SetScrollOffset(0.f);

	// Redraw widgets
	Invalidate(EInvalidateWidgetReason::Layout);
}

void SObservableMediaImage::ApplyZoom(float NewScale, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// Get cursor position in the widget space
	const FVector2D CursorLocalPos = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
	const FVector2D WidgetSize = MyGeometry.GetLocalSize();

	// Current image offsets
	const float OldOffsetH = ScrollBoxH->GetScrollOffset();
	const float OldOffsetV = ScrollBoxV->GetScrollOffset();

	// Coordinates in image space
	const float ImagePointX = OldOffsetH + CursorLocalPos.X;
	const float ImagePointY = OldOffsetV + CursorLocalPos.Y;

	// Scale ratio relative to current
	const float ScaleRatio = NewScale / CurrentZoom;
	CurrentZoom = NewScale;

	// Zoom point coordinates with new scale
	const float NewImagePointX = ImagePointX * ScaleRatio;
	const float NewImagePointY = ImagePointY * ScaleRatio;

	// Update image box size according to new image size
	const FVector2D NewBoxSize = OriginalImageSize * CurrentZoom;
	ImageSizeBox->SetWidthOverride(NewBoxSize.X);
	ImageSizeBox->SetHeightOverride(NewBoxSize.Y);

	// Recalculate new scroll offsets
	const float NewOffsetH = NewImagePointX - CursorLocalPos.X;
	const float NewOffsetV = NewImagePointY - CursorLocalPos.Y;

	// Set new scroll offsets
	ScrollBoxH->SetScrollOffset(NewOffsetH);
	ScrollBoxV->SetScrollOffset(NewOffsetV);
}

void SObservableMediaImage::HandleResize(const FVector2D& InNewImageSize)
{
	const float OldOffsetStartH = ScrollBoxH->GetScrollOffset();
	const float OldOffsetStartV = ScrollBoxV->GetScrollOffset();
	const float OldOffsetEndH = ScrollBoxH->GetScrollOffset();
	const float OldOffsetEndV = ScrollBoxV->GetScrollOffset();

	// Update original image size
	OriginalImageSize = InNewImageSize;

	// Update image box size according to new image size & current zoom
	const FVector2D NewBoxSize = OriginalImageSize * CurrentZoom;
	ImageSizeBox->SetWidthOverride(NewBoxSize.X);
	ImageSizeBox->SetHeightOverride(NewBoxSize.Y);

	// Redraw the widget
	Invalidate(EInvalidateWidgetReason::Layout);
}

void SObservableMediaImage::OnObservableUpdated()
{
	if (const TSharedPtr<IClusterObservable> PinnedObservable = Observable.Pin())
	{
		// Update when the difference is 10+ px any size
		const FVector2D NewSize = PinnedObservable->GetResolution();
		if (!OriginalImageSize.Equals(NewSize, 10))
		{
			HandleResize(NewSize);
		}
	}
}

FCursorReply SObservableMediaImage::OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const
{
	// Show the "currenly panning" cursor
	if (bIsPanning)
	{
		return FCursorReply::Cursor(EMouseCursor::GrabHandClosed);
	}

	// Show the "can pan" cursor
	return FCursorReply::Cursor(EMouseCursor::GrabHand);
}
