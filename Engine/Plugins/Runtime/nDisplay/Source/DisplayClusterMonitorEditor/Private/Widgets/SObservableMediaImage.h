// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SMediaImage.h"

class IClusterObservable;
class SBox;
class SScrollBar;
class SScrollBox;


/**
 * Observable Media Image Widget
 * 
 * Extends SMediaImage with the following features:
 * - Automatically manages horizontal and vertical scrollbars
 * - Supports image panning
 * - Supports image zooming
 */
class SObservableMediaImage : public SMediaImage
{
	using Super = SMediaImage;

public:

	SLATE_BEGIN_ARGS(SObservableMediaImage)
		: _ScrollBarThickness(FVector2D(8.f, 8.f))
	{ }
		/** Thickness of the scrollbar handles */
		SLATE_ARGUMENT(FVector2D, ScrollBarThickness)

		/** Zoom step per wheel tick (0.1 = 10% per notch) */
		SLATE_ARGUMENT(float, ZoomStep)

		/** Minimum zoom scale */
		SLATE_ARGUMENT(float, ZoomMin)

		/** Maximum zoom scale */
		SLATE_ARGUMENT(float, ZoomMax)

	SLATE_END_ARGS()

	/** Widget construction */
	void Construct(const FArguments& InArgs, TSharedPtr<IClusterObservable> InObservable);

	virtual ~SObservableMediaImage() override;

public:

	//~ Begin SWidget Interface
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override;
	//~ End SWidget Interface

public:

	/** Returns current scale multiplier */
	float GetCurrentZoom() const;

	/** Zooms the media image to fit within the available space */
	void ZoomToFit();

	/** Makes the original image look unscaled */
	void ZoomTo100();

private:

	/** Recalculates internal widget states in order to follow the new scale */
	void ApplyZoom(float NewScale, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	/** Recalculates internal widget states on order to follow the new image size */
	void HandleResize(const FVector2D& InNewImageSize);

	/** Called when the observable entity bound to this widget is changed */
	void OnObservableUpdated();

private:

	/** Actual observable being shown by this widget */
	TWeakPtr<IClusterObservable> Observable;

private: // Scrolling

	/** Horizontal scroll box. Manages the visible area of the media image. */
	TSharedPtr<SScrollBox> ScrollBoxH;

	/** Horizontal scroll bar. Used to scroll the image horizontally. */
	TSharedPtr<SScrollBar> HScrollBar;

	/** Vertical scroll box. Manages the visible area of the media image. */
	TSharedPtr<SScrollBox> ScrollBoxV;

	/** Vertical scroll bar. Used to scroll the image vertically. */
	TSharedPtr<SScrollBar> VScrollBar;

private: // Panning

	/** Whether is panning currently */
	bool bIsPanning = false;

	/** Cursor position at which panning has started */
	FVector2D PanStartMousePos = FVector2D::ZeroVector;

	/** Current scroll step provided by a correpsonding scroll box (horizontal). Used to calculate final panning offset. */
	float PanStartScrollOffsetH = 0.f;

	/** Current scroll step provided by a correpsonding scroll box (vertical). Used to calculate final panning offset. */
	float PanStartScrollOffsetV = 0.f;

private: // Zooming

	/** Current image scale */
	float CurrentZoom = 1.f;

	/** Scale step per iteration (mouse wheel notch) */
	float ZoomStep = 0.1f;

	/** Minimum possible zoom */
	float ZoomMin = 0.1f;

	/** Maximum possible zoom */
	float ZoomMax = 8.f;

private: // Image

	/** The box widget used to control the actual content (media image) size */
	TSharedPtr<SBox> ImageSizeBox;

	/** Original non-scaled image size */
	FVector2D OriginalImageSize = FVector2D(512.f, 512.f);
};
