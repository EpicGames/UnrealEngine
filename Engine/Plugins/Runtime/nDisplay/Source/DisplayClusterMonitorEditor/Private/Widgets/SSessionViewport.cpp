// Copyright Epic Games, Inc. All Rights Reserved.


#include "Widgets/SSessionViewport.h"

#include "Core/IClusterMonitorController.h"
#include "Core/IClusterObservable.h"
#include "Framework/Commands/UICommandList.h"
#include "Misc/Guid.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Images/SThrobber.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SClusterSessionsView.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SObservableMediaImage.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"

#include "DCMonitorEditorStyle.h"
#include "EditorFontGlyphs.h"
#include "LevelViewportActions.h"
#include "MediaTexture.h"
#include "ToolMenuEntry.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "SSessionViewport"


void SSessionViewport::Construct(
	const FArguments& InArgs,
	TSharedPtr<SClusterSessionsView> InOwningView,
	TSharedPtr<IClusterMonitorController> InController,
	TSharedPtr<IClusterObservable> InObservable)
{
	OwningView = InOwningView;
	Controller = InController;
	Observable = InObservable;

	CommandList = MakeShared<FUICommandList>();
	BindCommands();

	ChildSlot
	[
		SNew(SVerticalBox)

		// Toolbar
		+SVerticalBox::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
			[
				CreateWidget_Toolbar()
			]
		]

		// Workspace
		+SVerticalBox::Slot()
		.FillHeight(1.f)
		[
			SNew(SOverlay)

			// Media content
			+SOverlay::Slot()
			[
				CreateWidget_Media()
			]

			// Throbber
			+SOverlay::Slot()
			[
				SNew(SBorder)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Visibility(this, &SSessionViewport::GetLayerVisibility_Throbber)
				[
					SNew(SThrobber)
				]
			]

			// Error message
			+SOverlay::Slot()
			[
				SNew(SBorder)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Visibility(this, &SSessionViewport::GetLayerVisibility_Error)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("MessagePlaybackError", "Can't play media stream"))
				]
			]
		]

		// Status bar
		+SVerticalBox::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
			[
				CreateWidget_Statusbar()
			]
		]
	];

	// User friendly output (zoom-to-fit) by default. At this moment, MediaImage has not
	// cached its geometry yet. We need to let it Tick() first, and recalculate everything.
	// The following will be waiting until MediaImage is ready. Once it is, 
	RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateLambda(
		[WeakThis = AsWeak()](double /*InCurrentTime*/, float /*InDeltaTime*/)
		{
			// Stop polling if the main widget has gone
			TSharedPtr<SSessionViewport> PinnedThis = StaticCastSharedPtr<SSessionViewport>(WeakThis.Pin());
			if (!PinnedThis.IsValid())
			{
				return EActiveTimerReturnType::Stop;
			}

			// Stop polling if the image widget is invalid. This is never expected.
			if (!PinnedThis->MediaImage.IsValid())
			{
				return EActiveTimerReturnType::Stop;
			}

			// Ok, everything is fine, but it's not initialized yet. Keep polling. 
			if (PinnedThis->MediaImage->GetTickSpaceGeometry().GetLocalSize().IsNearlyZero())
			{
				return EActiveTimerReturnType::Continue;
			}

			// MediaImage has cached geometry. Set initial zoom, and stop polling.
			PinnedThis->MediaImage->ZoomToFit();
			return EActiveTimerReturnType::Stop;
		}));
}

FReply SSessionViewport::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

bool SSessionViewport::SupportsKeyboardFocus() const
{
	return true;
}

TSharedRef<SWidget> SSessionViewport::CreateWidget_Toolbar()
{
	const FName ViewportToolbarName = "ObservableSessionViewport.Toolbar";

	// Register session toolbar if not registered yet
	if (!UToolMenus::Get()->IsMenuRegistered(ViewportToolbarName))
	{
		UToolMenu* ToolbarMenu = UToolMenus::Get()->RegisterMenu(ViewportToolbarName, NAME_None, EMultiBoxType::SlimHorizontalToolBar);
		ToolbarMenu->StyleName = "ViewportToolbar";

		// Observable title section
		FToolMenuSection& TitleSection = ToolbarMenu->AddSection("Title");
		TitleSection.Alignment = EToolMenuSectionAlign::First;
		TitleSection.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& Section)
			{
				TSharedPtr<SSessionViewport> ContextWidget;
				if (const UClusterObservableViewportToolbarContext* const Context = Section.FindContext<UClusterObservableViewportToolbarContext>())
				{
					ContextWidget = Context->GetViewportWidget();
				}

				if (!ContextWidget.IsValid())
				{
					return;
				}

				Section.AddEntry(ContextWidget->CreateMenu_Title());
			}));

		// Session control section
		FToolMenuSection& SessionCtrlSection = ToolbarMenu->AddSection("SessionCtrl");
		SessionCtrlSection.Alignment = EToolMenuSectionAlign::Middle;
		SessionCtrlSection.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& Section)
			{
				TSharedPtr<SSessionViewport> ContextWidget;
				if (const UClusterObservableViewportToolbarContext* const Context = Section.FindContext<UClusterObservableViewportToolbarContext>())
				{
					ContextWidget = Context->GetViewportWidget();
				}

				if (!ContextWidget.IsValid())
				{
					return;
				}

				Section.AddEntry(ContextWidget->CreateMenu_SessionControl());
			}));

		// Viewport control section
		FToolMenuSection& ViewportCtrlSection = ToolbarMenu->AddSection("ViewportCtrl");
		ViewportCtrlSection.Alignment = EToolMenuSectionAlign::Last;
		ViewportCtrlSection.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& Section)
			{
				TSharedPtr<SSessionViewport> ContextWidget;
				if (const UClusterObservableViewportToolbarContext* const Context = Section.FindContext<UClusterObservableViewportToolbarContext>())
				{
					ContextWidget = Context->GetViewportWidget();
				}

				if (!ContextWidget.IsValid())
				{
					return;
				}

				Section.AddEntry(ContextWidget->CreateMenu_ZoomControl());
				Section.AddEntry(ContextWidget->CreateMenu_ViewportControl());
				Section.AddEntry(ContextWidget->CreateMenu_ViewportClose());
			}));
	}

	// Create context for this viewport instance
	FToolMenuContext Context;
	{
		Context.AppendCommandList(CommandList);

		UClusterObservableViewportToolbarContext* ContextObject = NewObject<UClusterObservableViewportToolbarContext>();
		ContextObject->SetViewportWidget(SharedThis(this));
		Context.AddObject(ContextObject);
	}

	// Finally, spawn the toolbar
	return UToolMenus::Get()->GenerateWidget(ViewportToolbarName, Context);
}

TSharedRef<SWidget> SSessionViewport::CreateWidget_Media()
{
	TSharedPtr<IClusterObservable> PinnedObservable = Observable.Pin();
	if (!PinnedObservable.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	return SNew(SBox)
		[
			SNew(SBorder)
			.BorderBackgroundColor(FLinearColor::Black)
			.BorderImage(FAppStyle::GetBrush("WhiteBrush"))
			[
				SAssignNew(MediaImage, SObservableMediaImage, PinnedObservable)
				 .ScrollBarThickness(FVector2D(6.f, 6.f))
				 .ZoomStep(0.2f) // 20% (absolute) per wheel notch
				 .ZoomMin(0.2f)  // 20% of original size
				 .ZoomMax(8.f)   // 800% of original size
			]
		];
}

TSharedRef<SWidget> SSessionViewport::CreateWidget_Statusbar()
{
	TSharedRef<SWidget> Widget = SNew(SHorizontalBox)

		// Media resolution
		+SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(4.f, 4.f, 4.f, 4.f)
		[
			SNew(STextBlock)
			.Text_Lambda([this]()
				{
					if (TSharedPtr<IClusterObservable> PinndedObservable = Observable.Pin())
					{
						const FIntPoint Res = PinndedObservable->GetResolution();
						return FText::Format(LOCTEXT("StatusResolution", "Res: [{0}x{1}]"), Res.X, Res.Y);
					}
					return LOCTEXT("StatusUnknownResolution", "Res: - ? -");
				})
		]

		// Separator
		+SHorizontalBox::Slot()
		.Padding(4.f)
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SSeparator)
			.Orientation(Orient_Vertical)
		]

		// Current zoom
		+SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(4.f, 4.f, 4.f, 4.f)
		[
			SNew(STextBlock)
			.Text_Lambda([this]()
				{
					if (MediaImage.IsValid())
					{
						const float CurrentZoom = MediaImage->GetCurrentZoom();
						return FText::Format(LOCTEXT("CurrentZoom", "Zoom: {0}%"), static_cast<int32>(CurrentZoom * 100));
					}
					return LOCTEXT("StatusUnknownZoom", "Zoom: - ? -");
				})
		];

	return Widget;
}

TSharedRef<SWidget> SSessionViewport::CreateWidget_ZoomControl()
{
	return SNew(SHorizontalBox)

	// Zoom to fit
	+SHorizontalBox::Slot()
	.AutoWidth()
	.Padding(2.f, 0.f)
	[
		SNew(SButton)
		.ToolTipText(LOCTEXT("Button_ZoomToFit_ToolTip", "Zooms the media image to fit within the available space"))
		.ContentPadding(FMargin(4.f, 4.f))
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.OnClicked_Lambda([this]()
			{
				if (MediaImage.IsValid())
				{
					MediaImage->ZoomToFit();
				}
				return FReply::Handled();
			})
		[
			SNew(SImage).Image(FDCMonitorEditorStyle::Get().GetBrush("ClusterMonitor.Viewport.ZoomToFit"))
		]
	]

	// Zoom to 100 (original size)
	+SHorizontalBox::Slot()
	.AutoWidth()
	.Padding(2.f, 0.f)
	[
		SNew(SButton)
		.ToolTipText(LOCTEXT("Button_ZoomTo100_ToolTip", "Zooms the media image to its original unscaled size"))
		.ContentPadding(FMargin(4.f, 4.f))
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.OnClicked_Lambda([this]()
			{
				if (MediaImage.IsValid())
				{
					MediaImage->ZoomTo100();
				}
				return FReply::Handled();
			})
		[
			SNew(SImage).Image(FDCMonitorEditorStyle::Get().GetBrush("ClusterMonitor.Viewport.ZoomTo100"))
		]
	];
}

FToolMenuEntry SSessionViewport::CreateMenu_Title()
{
	TSharedRef<SWidget> ObservableNameWidget = SNullWidget::NullWidget;

	TSharedPtr<IClusterObservable> PinnedObservable = Observable.Pin();
	if (PinnedObservable.IsValid())
	{
		const FString ObservableName = PinnedObservable->GetName();

		ObservableNameWidget = SNew(SHorizontalBox)

			// Observable type icon
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(2))
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				SNew(SImage).Image(PinnedObservable->GetDisplayIcon())
			]

			// Observable name
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(2))
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(FText::FromString(ObservableName))
				.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
				.ColorAndOpacity(FStyleColors::AccentOrange)
			];
	}

	FToolMenuEntry Entry = FToolMenuEntry::InitWidget("ObservableSessionHeader", ObservableNameWidget, FText::GetEmpty(), true);

	return Entry;
}

FToolMenuEntry SSessionViewport::CreateMenu_SessionControl()
{
	TSharedRef<SWidget> MenuEntryWidget =

		SNew(SHorizontalBox)

		// Play
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2.f, 0.f)
		[
			SNew(SButton)
			.Text(FText::FromString("Play"))
			.OnClicked_Lambda([this]()
				{
					OnClicked_Play();
					return FReply::Handled();
				})
			.Content()
			[
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.14"))
				.Text(FEditorFontGlyphs::Play)
				.ColorAndOpacity(FLinearColor::White)
			]
		]

		// Pause
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2.f, 0.f)
		[
			SNew(SButton)
			.Text(FText::FromString("Pause"))
			.IsEnabled(false) //@note NDI media player doesn't support pause yet, so deactivate this button
			.OnClicked_Lambda([this]()
				{
					OnClicked_Pause();
					return FReply::Handled();
				})
			.Content()
			[
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.14"))
				.Text(FEditorFontGlyphs::Pause)
				.ColorAndOpacity(FLinearColor::White)
			]
		]

		// Stop
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2.f, 0.f)
		[
			SNew(SButton)
			.Text(FText::FromString("Stop"))
			.OnClicked_Lambda([this]()
				{
					OnClicked_Stop();
					return FReply::Handled();
				})
			.Content()
			[
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.14"))
				.Text(FEditorFontGlyphs::Stop)
				.ColorAndOpacity(FLinearColor::White)
			]
		];

	return FToolMenuEntry::InitWidget("ObservableSessionControl", MenuEntryWidget, FText::GetEmpty(), true);
}

FToolMenuEntry SSessionViewport::CreateMenu_ZoomControl()
{
	TSharedRef<SWidget> Widget = CreateWidget_ZoomControl();

	FToolMenuEntry Entry = FToolMenuEntry::InitWidget("ZoomControl", Widget, FText::GetEmpty(), true);

	return Entry;
}

FToolMenuEntry SSessionViewport::CreateMenu_ViewportControl()
{
	FToolMenuEntry Entry = FToolMenuEntry::InitSubMenu(
		"SessionViewportArrangementMenu",
		LOCTEXT("SessionViewportDropdown_Label", "..."),
		LOCTEXT("SessionViewportArrangement_Tooltip", "Viewport arrangements"),
		FNewToolMenuDelegate::CreateLambda([this](UToolMenu* Submenu)
			{
				FToolMenuSection& MaximizeSection = Submenu->FindOrAddSection("MaximizeSection");

				MaximizeSection.AddSeparator("MaximizeSeparator");
				MaximizeSection.AddEntry(FToolMenuEntry::InitMenuEntry(FLevelViewportCommands::Get().ToggleImmersive));
				MaximizeSection.AddEntry(CreateSubMenu_ViewportMaximize());
			}));

	Entry.StyleNameOverride = FName("ViewportToolbarViewportSizingSubmenu");
	Entry.InsertPosition.Position = EToolMenuInsertType::Last;
	Entry.ToolBarData.LabelOverride = FText();
	Entry.ToolBarData.ResizeParams.AllowClipping = false;

	return Entry;
}

FToolMenuEntry SSessionViewport::CreateSubMenu_ViewportMaximize()
{
	FToolMenuEntry MaximizeRestoreEntry = FToolMenuEntry::InitMenuEntry(
		FLevelViewportCommands::Get().ToggleMaximize,
		TAttribute<FText>::CreateLambda([this]()
			{
				return IsMaximized() ?
					LOCTEXT("MaximizeRestoreLabel_Restore", "Restore All Viewports") :
					LOCTEXT("MaximizeRestoreLabel_Maximize", "Maximize Viewport");
			}),
		TAttribute<FText>::CreateLambda([this]()
			{
				return IsMaximized() ?
					LOCTEXT("MaximizeRestoreTooltip_Restore", "Restores the layout to show all viewports") :
					LOCTEXT("MaximizeRestoreTooltip_Maximize", "Maximizes this viewport");
			}),
		TAttribute<FSlateIcon>::CreateLambda([this]()
			{
				return IsMaximized() ?
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorViewportToolBar.Maximize.Checked") :
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorViewportToolBar.Maximize.Normal");
			})
	);

	MaximizeRestoreEntry.SetShowInToolbarTopLevel(true);
	MaximizeRestoreEntry.ToolBarData.ResizeParams.AllowClipping = false;
	MaximizeRestoreEntry.StyleNameOverride = FName("ViewportToolbarViewportSizingSubmenu");

	return MaximizeRestoreEntry;
}

FToolMenuEntry SSessionViewport::CreateMenu_ViewportClose()
{
	TSharedRef<SWidget> Widget = SNew(SButton)
		.Text(FText::FromString("Close"))
		.OnClicked_Lambda([this]()
			{
				// When pressed, ask the controller to stop this session
				TSharedPtr<IClusterObservable> PinnedObservable = Observable.Pin();
				TSharedPtr<IClusterMonitorController> PinnedController = Controller.Pin();
				if (PinnedController.IsValid() && PinnedObservable.IsValid())
				{
					const FGuid ObservableId = PinnedObservable->GetId();
					PinnedController->RequestSessionStop(ObservableId);
				}

				return FReply::Handled();
			})
		.Content()
		[
			SNew(STextBlock)
			.Font(FAppStyle::Get().GetFontStyle("FontAwesome.14"))
			.Text(FEditorFontGlyphs::Times)
			.ColorAndOpacity(FLinearColor::White)
		];

	FToolMenuEntry Entry = FToolMenuEntry::InitWidget("StopSessionViewport", Widget, FText::GetEmpty(), true);
	Entry.InsertPosition.Position = EToolMenuInsertType::Last;

	return Entry;
}

void SSessionViewport::BindCommands()
{
	// Toggle "immersive" mode
	CommandList->MapAction(
		FLevelViewportCommands::Get().ToggleImmersive,
		FUIAction(
			FExecuteAction::CreateLambda([this]()
				{
					TSharedPtr<SClusterSessionsView> PinnedOwningView = OwningView.Pin();
					TSharedPtr<IClusterObservable> PinndedObservable = Observable.Pin();
					if (!PinnedOwningView.IsValid() || !PinndedObservable.IsValid())
					{
						return;
					}

					// If going to immersive
					if (!IsImmersive())
					{
						const FGuid OvservableId = PinndedObservable->GetId();

						// Remember 'maximized' state before immersive
						bWasMaximizedBeforeImmersive = IsMaximized();

						// Maximize viewport, if not currently maximizied
						if (!bWasMaximizedBeforeImmersive)
						{
							PinnedOwningView->MaximizeViewport(OvservableId);
						}

						// Set immersive mode
						PinnedOwningView->SetViewportImmersive(OvservableId);
					}
					// If leaving immersive
					else
					{
						// Reset immersive
						PinnedOwningView->ResetImmersive();

						// Also reset maximized if it was auto-set
						if (!bWasMaximizedBeforeImmersive)
						{
							PinnedOwningView->ResetMaximized();
						}
					}
				}),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([this] { return IsImmersive(); })
		));

	// Toggle "maximize" mode
	CommandList->MapAction(FLevelViewportCommands::Get().ToggleMaximize,
		FUIAction(
			FExecuteAction::CreateLambda([this]()
				{
					TSharedPtr<SClusterSessionsView> PinnedOwningView = OwningView.Pin();
					TSharedPtr<IClusterObservable> PinndedObservable = Observable.Pin();
					if (!PinnedOwningView.IsValid() || !PinndedObservable.IsValid())
					{
						return;
					}

					if (!IsMaximized())
					{
						const FGuid OvservableId = PinndedObservable->GetId();
						PinnedOwningView->MaximizeViewport(OvservableId);
					}
					else
					{
						PinnedOwningView->ResetMaximized();
					}
				}),
			FCanExecuteAction::CreateLambda([this]()
				{
					TSharedPtr<SClusterSessionsView> PinnedOwningView = OwningView.Pin();
					TSharedPtr<IClusterObservable> PinndedObservable = Observable.Pin();
					if (!PinnedOwningView.IsValid() || !PinndedObservable.IsValid())
					{
						return false;
					}

					const FGuid OvservableId = PinndedObservable->GetId();

					// Disable the maximized control when immersive
					if (PinnedOwningView->IsViewportImmersive(OvservableId))
					{
						return false;
					}

					return IsMaximized() ? PinnedOwningView->CanResetMaximized() : PinnedOwningView->CanMaximizeViewport(OvservableId);
				})
		));
}

bool SSessionViewport::IsMaximized() const
{
	TSharedPtr<SClusterSessionsView> PinnedOwningView = OwningView.Pin();
	TSharedPtr<IClusterObservable> PinnedObservable = Observable.Pin();
	if (!PinnedOwningView.IsValid() || !PinnedObservable.IsValid())
	{
		return false;
	}

	// Forward this request to the owner
	const FGuid ObservableId = PinnedObservable->GetId();
	return PinnedOwningView->IsViewportMaximized(ObservableId);
}

bool SSessionViewport::IsImmersive() const
{
	TSharedPtr<SClusterSessionsView> PinnedOwningView = OwningView.Pin();
	TSharedPtr<IClusterObservable>   PinnedObservable = Observable.Pin();
	if (!PinnedOwningView.IsValid() || !PinnedObservable.IsValid())
	{
		return false;
	}

	// Forward this request to the owner
	const FGuid ObservableId = PinnedObservable->GetId();
	return PinnedOwningView->IsViewportImmersive(ObservableId);
}

void SSessionViewport::OnClicked_Play()
{
	if (TSharedPtr<IClusterObservable> PinnedObservable = Observable.Pin())
	{
		PinnedObservable->Play();
	}
}

void SSessionViewport::OnClicked_Pause()
{
	if (TSharedPtr<IClusterObservable> PinnedObservable = Observable.Pin())
	{
		PinnedObservable->Pause();
	}
}

void SSessionViewport::OnClicked_Stop()
{
	if (TSharedPtr<IClusterObservable> PinnedObservable = Observable.Pin())
	{
		PinnedObservable->Stop();
	}
}

EVisibility SSessionViewport::GetLayerVisibility_Throbber() const
{
	TSharedPtr<IClusterObservable> PinnedObservable = Observable.Pin();
	if (!PinnedObservable.IsValid())
	{
		return EVisibility::Hidden;
	}

	const IClusterObservable::ESessionState SessionState = PinnedObservable->GetSessionState();
	if (SessionState == IClusterObservable::ESessionState::Transition)
	{
		return EVisibility::Visible;
	}

	return EVisibility::Hidden;
}

EVisibility SSessionViewport::GetLayerVisibility_Error() const
{
	TSharedPtr<IClusterObservable> PinnedObservable = Observable.Pin();
	if (!PinnedObservable.IsValid())
	{
		return EVisibility::Visible;
	}

	const IClusterObservable::ESessionState SessionState = PinnedObservable->GetSessionState();
	if (SessionState == IClusterObservable::ESessionState::Error)
	{
		return EVisibility::Visible;
	}

	return EVisibility::Hidden;
}

#undef LOCTEXT_NAMESPACE
