// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "SSessionViewport.generated.h"

class FUICommandList;
class IClusterMonitorController;
class IClusterObservable;
class SClusterSessionsView;
class SObservableMediaImage;
class SSessionViewport;
class SWidget;
struct FToolMenuEntry;


/**
 * Session viewport toolbar context
 * 
 * Provides contextual information to the viewport toolbar
 */
UCLASS()
class UClusterObservableViewportToolbarContext : public UObject
{
	GENERATED_BODY()

public:

	/** Returns owning viewport widget */
	TSharedPtr<SSessionViewport> GetViewportWidget() const
	{
		return ViewportWidget.Pin();
	}

	/** Sets owning viewport widget */
	void SetViewportWidget(const TSharedPtr<SSessionViewport>& InWidget)
	{
		ViewportWidget = InWidget;
	}

private:

	/** Session viewport widget that owns the toolbar */
	TWeakPtr<SSessionViewport> ViewportWidget;
};


/**
 * Session Viewport Widget
 * 
 * This widget represents a single media observation session. It also:
 * - Provides some additional session control tools
 * - Provides some session information
 * - Provides Maximized and Immersive panel modes
 */
class SSessionViewport : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SSessionViewport)
	{ }
	SLATE_END_ARGS()

	/** Widget construction */
	void Construct(const FArguments& InArgs, TSharedPtr<SClusterSessionsView> InOwningView, TSharedPtr<IClusterMonitorController> InController, TSharedPtr<IClusterObservable> InObservable);

public:

	//~ Begin SWidget
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	virtual bool SupportsKeyboardFocus() const override;
	//~ End SWidget

private:

	/** Creates viewport toolbar widget */
	TSharedRef<SWidget> CreateWidget_Toolbar();

	/** Creates media output widget */
	TSharedRef<SWidget> CreateWidget_Media();

	/** Creates information/status bar */
	TSharedRef<SWidget> CreateWidget_Statusbar();

	/** Creates zoom control widget */
	TSharedRef<SWidget> CreateWidget_ZoomControl();

private:

	/** Binds UI commands that this viewport handles */
	void BindCommands();

	/** Toolbar: Viewport title */
	FToolMenuEntry CreateMenu_Title();

	/** Toolbar: Session control tools (play, stop, etc.) */
	FToolMenuEntry CreateMenu_SessionControl();

	/** Toolbar: Zoom control tools (Fit, 100%, etc.) */
	FToolMenuEntry CreateMenu_ZoomControl();

	/** Toolbar: Viewport control tools (maximize, immerse) */
	FToolMenuEntry CreateMenu_ViewportControl();

	/** Toolbar: Maximize viewport button */
	FToolMenuEntry CreateSubMenu_ViewportMaximize();

	/** Toolbar: Close viewport button */
	FToolMenuEntry CreateMenu_ViewportClose();

private:

	/** Whether this viewport is maximized currently */
	bool IsMaximized() const;

	/** Whether this viewport is in immersive mode currently */
	bool IsImmersive() const;

private:

	/** Handles 'Play' button pressed */
	void OnClicked_Play();

	/** Handles 'Pause' button pressed */
	void OnClicked_Pause();

	/** Handles 'Stop' button pressed */
	void OnClicked_Stop();

private:

	/** Returns current visibility of the throbber layer */
	EVisibility GetLayerVisibility_Throbber() const;

	/** Returns current visibility of the error message layer */
	EVisibility GetLayerVisibility_Error() const;

private:

	/** The owning session view panel */
	TWeakPtr<SClusterSessionsView> OwningView;

	/** Current cluster monitor controller */
	TWeakPtr<IClusterMonitorController> Controller;

	/** Actual observable entity bound to this session viewport */
	TWeakPtr<IClusterObservable> Observable;

	/** UI commands bound to this viewport */
	TSharedPtr<FUICommandList> CommandList;

	/** Media image widget */
	TSharedPtr<SObservableMediaImage> MediaImage;

	/**
	 * Stores the window's maximized state before entering immersive mode.
	 * This allows restoring the correct state after exiting immersive mode.
	 */
	bool bWasMaximizedBeforeImmersive = false;
};
