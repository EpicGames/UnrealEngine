// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DisplayClusterMoviePipeline : ModuleRules
{
	public DisplayClusterMoviePipeline(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"Json",
				"Imath",
				"UEOpenExr", // Needed for multilayer EXRs
				"UEOpenExrRTTI", // Needed for EXR metadata
				"DisplayCluster"
			}
		);

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"MovieRenderPipelineCore",
				"MovieRenderPipelineRenderPasses",
				"CinematicCamera",
				"MovieScene",
				"LevelSequence",
				"ImageWriteQueue",
				"RenderCore",
				"RHI",
				"ActorLayerUtilities",
				"OpenColorIO",
				"DisplayCluster",
				"DisplayClusterConfiguration",
			}
		);
	}
}
