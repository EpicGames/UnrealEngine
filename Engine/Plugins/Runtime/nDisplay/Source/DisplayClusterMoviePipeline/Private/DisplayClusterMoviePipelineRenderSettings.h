// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DisplayClusterMoviePipelineEnums.h"
#include "Render/Viewport/RenderFrame/DisplayClusterRenderFrameEnums.h"

namespace UE::DisplayClusterMoviePipeline
{
	/** Settings captured from the render pass node during Initialize(). */
	struct FRenderSettings
	{
		/** Render frame mode (mono or stereo) captured from the render pass node during Initialize(). */
		EDisplayClusterRenderFrameMode RenderMode = EDisplayClusterRenderFrameMode::Unknown;

		/** Whether overscan comes from MRP (Default) or from each nDisplay viewport's own overscan setting. */
		EDisplayClusterMoviePipelineOverscanMode OverscanMode = EDisplayClusterMoviePipelineOverscanMode::Default;

		/** Uniform scale applied to the render resolution of all viewports [0..1]. */
		float RenderResolutionScale = 1.f;

		/** When non-zero, overrides the viewport output resolution. */
		FIntPoint OutputResolutionOverride = FIntPoint::ZeroValue;

		/** True when the output node is a multi-layer EXR. */
		bool bIsMultiLayerEXROutput = false;

		/**
		 * Controls whether external overscan is applied and what fraction to pass to SetExternalOverscan().
		 * - Default mode: bActive is true. If UMovieGraphCameraSettingNode overrides OverscanPercentage,
		 *   that value is normalized to [0..1] and stored in Fraction; otherwise Fraction is unset and the
		 *   viewport manager falls back to FMinimalViewInfo.GetOverscan() from the MRP camera.
		 * - Viewport mode: nDisplay's own per-viewport overscan settings apply; bActive is false.
		 */
		struct FExternalOverscan
		{
			/** When false, external overscan is disabled and nDisplay's per-viewport overscan applies instead. */
			bool bActive = false;

			/** Overscan fraction [0..1] to forward; unset means fall back to FMinimalViewInfo.GetOverscan(). */
			TOptional<float> Fraction;
		};
		
		/** External overscan state captured from the render pass node during Initialize(). */
		FExternalOverscan ExternalOverscan;

		/** Controls how warp-blend is applied and composited for this pass. */
		EDisplayClusterMoviePipelineWarpBlendMode WarpBlendMode = EDisplayClusterMoviePipelineWarpBlendMode::WarpBlend;

		/** Ensures the warp-blend unavailability warning is logged only once per initialization. */
		mutable bool bEnabledWarpBlendErrorMsgShowOnce = false;
	};
} // namespace UE::DisplayClusterMoviePipeline
