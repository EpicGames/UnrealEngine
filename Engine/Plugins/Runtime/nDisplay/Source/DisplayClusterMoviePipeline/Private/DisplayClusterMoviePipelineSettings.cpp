// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterMoviePipelineSettings.h"
#include "DisplayClusterConfigurationTypes.h"
#include "MovieRenderPipelineCoreModule.h"
#include "Modules/ModuleManager.h"

///////////////////////////////////////////////////////////////////////////////////////////
// UDisplayClusterMoviePipelineSettings
///////////////////////////////////////////////////////////////////////////////////////////
#if WITH_EDITOR
void UDisplayClusterMoviePipelineSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	static const FName PropName_Configuration = GET_MEMBER_NAME_CHECKED(UDisplayClusterMoviePipelineSettings, Configuration);
	static const FName PropMemberName_RootActor = GET_MEMBER_NAME_CHECKED(FDisplayClusterMoviePipelineConfiguration, DCRootActor);

	if (PropertyChangedEvent.Property
		&& PropertyChangedEvent.Property->GetFName() == PropMemberName_RootActor
		&& PropertyChangedEvent.MemberProperty
		&& PropertyChangedEvent.MemberProperty->GetFName() == PropName_Configuration)
	{
		if (Configuration.DCRootActor.IsValid())
		{
			Configuration.DCRootActorClass = Configuration.DCRootActor->GetClass();      // stores a soft ref to the class
		}
		else
		{
			Configuration.DCRootActorClass = nullptr;
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif /** WITH_EDITOR */

ADisplayClusterRootActor* UDisplayClusterMoviePipelineSettings::GetRootActor(const UWorld* InWorld) const
{
	if (!InWorld)
	{
		return nullptr;
	}

	const TSoftObjectPtr<ADisplayClusterRootActor>& RootActorRef = Configuration.DCRootActor;
	const TSoftClassPtr<ADisplayClusterRootActor>& RootActorClassRef = Configuration.DCRootActorClass;

	UClass* TargetClass = nullptr;

	if (RootActorRef.IsValid())
	{
		// If DCRootActor points to a live actor in this world, just return it.
		if (RootActorRef->GetWorld() == InWorld)
		{
			return RootActorRef.Get();
		}

		// Try DCRootActor if still valid (even if it's in another world)
		TargetClass = RootActorRef->GetClass();
	}
	else if (!RootActorClassRef.IsNull())
	{
		// Optionally load it synchronously if it's not in memory
		TargetClass = RootActorClassRef.LoadSynchronous();
	}

	// If we have a class, return any actor of that class from this world
	if (TargetClass)
	{
		for (const TWeakObjectPtr<ADisplayClusterRootActor> RootActorIt : TActorRange<ADisplayClusterRootActor>(InWorld))
		{
			if (RootActorIt.IsValid() && RootActorIt->IsA(TargetClass))
			{
				return RootActorIt.Get();
			}
		}

		// There is no actor in the world with this class, returning nullptr
		return nullptr;
	}

	// DCRootActor==None, just return the first DCRA of any class.
	for (const TWeakObjectPtr<ADisplayClusterRootActor> RootActorIt : TActorRange<ADisplayClusterRootActor>(InWorld))
	{
		if (RootActorIt.IsValid())
		{
			return RootActorIt.Get();
		}
	}

	// There is no instance of DCRA (of any class) in the world, so nullptr is returned.
	return nullptr;
}

bool UDisplayClusterMoviePipelineSettings::GetViewports(const UWorld* InWorld, TArray<FString>& OutViewports, TArray<FIntPoint>& OutViewportResolutions) const
{
	OutViewports.Reset();
	OutViewportResolutions.Reset();

	// Treat an empty list of allowed viewports as "allow all"
	const bool bRenderAllViewports = Configuration.bRenderAllViewports || Configuration.AllowedViewportNamesList.IsEmpty();

	if (ADisplayClusterRootActor* RootActorPtr = GetRootActor(InWorld))
	{
		if (const UDisplayClusterConfigurationData* InConfigurationData = RootActorPtr->GetConfigData())
		{
			if (const UDisplayClusterConfigurationCluster* InClusterCfg = InConfigurationData->Cluster)
			{
				for (const TPair<FString, TObjectPtr<UDisplayClusterConfigurationClusterNode>>& NodeIt : InClusterCfg->Nodes)
				{
					if (const UDisplayClusterConfigurationClusterNode* InConfigurationClusterNode = NodeIt.Value)
					{
						const FString& InClusterNodeId = NodeIt.Key;
						for (const TPair<FString, TObjectPtr<UDisplayClusterConfigurationViewport>>& InConfigurationViewportIt : InConfigurationClusterNode->Viewports)
						{
							if (const UDisplayClusterConfigurationViewport* InConfigurationViewport = InConfigurationViewportIt.Value)
							{
								if (InConfigurationViewport->bAllowRendering)
								{
									const FString& InViewportId = InConfigurationViewportIt.Key;
									if (bRenderAllViewports || Configuration.AllowedViewportNamesList.Find(InViewportId) != INDEX_NONE)
									{
										OutViewports.Add(InViewportId);

										if (Configuration.bUseViewportResolutions)
										{
											OutViewportResolutions.Add(InConfigurationViewportIt.Value->Region.ToRect().Size());
										}
									}
								}
							}
						}
					}
				}

				return OutViewports.Num() > 0;
			}
		}
	}

	return false;
}

IMPLEMENT_MODULE(FDefaultModuleImpl, DisplayClusterMoviePipeline);
