// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterMoviePipelineViewportCameraInfo.h"

#include "DisplayClusterConfigurationTypes.h"
#include "Render/Viewport/IDisplayClusterViewport.h"

#include "MoviePipelineUtils.h"

namespace UE::DisplayClusterMoviePipeline
{
	/**
	 * Collects cluster node info for the given viewport.
	 * @param InViewport  Viewport whose cluster node ID and configuration data are queried.
	 */
	static FClusterNodeInfo GetClusterNodeInfo(const TSharedRef<IDisplayClusterViewport, ESPMode::ThreadSafe>& InViewport)
	{
		FClusterNodeInfo ClusterNodeInfo;
		ClusterNodeInfo.Id = InViewport->GetClusterNodeId();

		if (const UDisplayClusterConfigurationData* CfgData = InViewport->GetConfiguration().GetConfigurationData())
		{
			if(const UDisplayClusterConfigurationClusterNode* GetClusterNodeCfg = CfgData->GetNode(InViewport->GetClusterNodeId()))
			{
				ClusterNodeInfo.Host = GetClusterNodeCfg->Host;
			}
		}
		
		return ClusterNodeInfo;
	}

	/**
	 * Returns the stereo eye name for the given context.
	 * Returns "left" (context 0) or "right" (context 1+) for stereo, or "" for mono.
	 * 
	 * @param ContextNum        Stereo context index (0 = left eye, 1 = right eye).
	 * @param bStereoRendering  True if the viewport is rendering in stereo.
	 */
	static constexpr auto GetStereoEyeNameImpl(const int32 ContextNum, const bool bStereoRendering)
	{
		if (bStereoRendering)
		{
			// Stereo
			return !ContextNum ? TEXT("left") : TEXT("right");
		}

		return TEXT("");
	}

	/**
	 * Builds a unique camera name for the given viewport context.
	 * 
	 * @param InRenderSettings Render settings (multi-layer EXR output requires a distinct extended name format.)
	 * @param InClusterInfo    Cluster-level info; its topology selects the name layout ({node}_{viewport} for PerNode, {viewport} otherwise).
	 * @param InViewport       Viewport supplying the node ID and base viewport ID segments.
	 * @param InContextNum     Stereo context index (0 = mono/left eye, 1 = right eye).
	 */
	static FString MakeUniqueCameraName(
		const FRenderSettings& InRenderSettings,
		const FClusterInfo& InClusterInfo,
		const TSharedRef<IDisplayClusterViewport, ESPMode::ThreadSafe>& InViewport,
		const int32 InContextNum)
	{
		// TODO: Use InRenderSettings to generate the extended camera name format required for multi-layer EXR output nodes (not yet implemented).

		FString OutCameraName;
		switch (InClusterInfo.ClusterTopology)
		{
		case EClusterTopology::PerNode: // {node}_{viewport}
			OutCameraName = FString::Printf(TEXT("%s_%s"), *InViewport->GetClusterNodeId(), *InViewport->GetBaseId());
			break;

		case EClusterTopology::PerViewport:  // viewport name can be used as a camera name
		case EClusterTopology::None:
		default:
			OutCameraName = InViewport->GetBaseId();
			break;
		}

		const bool bStereoRendering = InViewport->GetContexts().Num() > 1;
		if (bStereoRendering)
		{
			// ..._{eye}
			OutCameraName += TEXT("_");
			OutCameraName += GetStereoEyeNameImpl(InContextNum, bStereoRendering);
		}

		return OutCameraName;
	}

} // namespace UE::DisplayClusterMoviePipeline

//////////////////////////////////////////////////////////////////////////
// UE::DisplayClusterMoviePipeline::FViewportCameraInfo

UE::DisplayClusterMoviePipeline::FViewportCameraInfo::FViewportCameraInfo(
	const FRenderSettings& InRenderSettings,
	const UE::DisplayClusterMoviePipeline::FClusterInfo& InClusterInfo,
	const TSharedRef<IDisplayClusterViewport,
	ESPMode::ThreadSafe>& InViewport,
	const int32 InContextNum)
	: ClusterInfo(InClusterInfo)
	, ClusterNodeInfo(GetClusterNodeInfo(InViewport))
	, CameraName(UE::DisplayClusterMoviePipeline::MakeUniqueCameraName(
		InRenderSettings, InClusterInfo, InViewport, InContextNum))
	, ViewportId(InViewport->GetId())
	, ContextNum(InContextNum)
	, bStereoRendering(InViewport->GetContexts().Num() > 1)
{ }

const FString UE::DisplayClusterMoviePipeline::FViewportCameraInfo::GetStereoEyeName() const
{
	return UE::DisplayClusterMoviePipeline::GetStereoEyeNameImpl(ContextNum, bStereoRendering);
}