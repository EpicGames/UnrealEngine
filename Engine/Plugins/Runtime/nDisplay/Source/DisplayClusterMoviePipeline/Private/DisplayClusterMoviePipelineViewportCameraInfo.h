// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DisplayClusterMoviePipelineRenderSettings.h"

class IDisplayClusterViewport;

namespace UE::DisplayClusterMoviePipeline
{
	/**
	 * Classifies the nDisplay cluster topology by the arrangement of viewports across nodes.
	 */
	enum class EClusterTopology : uint8
	{
		// One node, one viewport.
		None = 0,

		// One node, multiple viewports.
		PerViewport,

		// Multiple nodes, multiple viewports.
		PerNode
	};

	/** General information about the nDisplay cluster. Captured during Initialize(). */
	struct FClusterInfo
	{
		/** Root actor name. */
		FString RootActorName;

		/** Root actor class name. */
		FString RootActorClassName;

		/** nDisplay cluster topology. */
		EClusterTopology ClusterTopology = EClusterTopology::None;
	};

	/** Information about the cluster node that owns a viewport. Captured during Initialize(). */
	struct FClusterNodeInfo
	{
		/** Cluster node ID. */
		FString Id;

		/** Cluster node host address. */
		FString Host;
	};

	/**
	 * Maps a MRG camera index to a specific nDisplay viewport and stereo context.
	 * One entry is created per viewport context (mono = 1, stereo = 2 per viewport).
	 */
	struct FViewportCameraInfo
	{
		FViewportCameraInfo() = default;

		/** Explicit copy constructor required because all members are const. */
		FViewportCameraInfo(const FViewportCameraInfo& In)
			: ClusterInfo(In.ClusterInfo)
			, ClusterNodeInfo(In.ClusterNodeInfo)
			, CameraName(In.CameraName)
			, ViewportId(In.ViewportId)
			, ContextNum(In.ContextNum)
			, bStereoRendering(In.bStereoRendering) { }

		/**
		 * Constructs a camera entry from an nDisplay viewport.
		 *
		 * @param InRenderSettings  Render settings captured from the render pass node during Initialize().
		 * @param InClusterInfo     Cluster-level info (ID, root actor name).
		 * @param InViewport        nDisplay viewport to derive camera name, viewport ID, and cluster node from.
		 * @param InContextNum      Stereo context index (0 = mono/left eye, 1 = right eye).
		 */
		FViewportCameraInfo(
			const FRenderSettings& InRenderSettings,
			const FClusterInfo& InClusterInfo,
			const TSharedRef<IDisplayClusterViewport, ESPMode::ThreadSafe>& InViewport,
			const int32 InContextNum);

		/** Return stereo eye name. */
		const FString GetStereoEyeName() const;

		/** returns true, if this camera can be used for rendering. */
		const bool IsActive() const
		{
			return !CameraName.IsEmpty() && !ViewportId.IsEmpty();
		}

	public:
		/** Cluster info. */
		const FClusterInfo ClusterInfo;

		/** Cluster node info. */
		const FClusterNodeInfo ClusterNodeInfo;

	public:
		/**
		* Unique camera name used by MRG to identify this viewport.
		* This camera name should be unique within the
		* (viewport base ID + optional stereo suffix). */
		const FString CameraName;

		/** Unique runtime ID of the nDisplay viewport. */
		const FString ViewportId;

		/** Stereo context index: 0 = mono or left eye, 1 = right eye. */
		const int32 ContextNum = 0;

		/** True if this viewport use stereo rendering. */
		const bool bStereoRendering = false;
	};
} // namespace UE::DisplayClusterMoviePipeline
