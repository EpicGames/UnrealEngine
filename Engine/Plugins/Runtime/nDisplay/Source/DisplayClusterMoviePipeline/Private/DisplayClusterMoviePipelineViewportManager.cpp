// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterMoviePipelineViewportManager.h"

#include "DisplayClusterRootActor.h"

#include "Render/Projection/IDisplayClusterProjectionPolicy.h"
#include "Render/Viewport/IDisplayClusterViewportManager.h"
#include "Render/Viewport/IDisplayClusterViewportManagerPreview.h"
#include "Render/Viewport/IDisplayClusterViewportProxy.h"

#include "MoviePipelineUtils.h"
#include "MovieRenderPipelineCoreModule.h"

#include "EngineUtils.h"
#include "MovieScene.h"
#include "MovieSceneSequence.h"
#include "MovieSceneSequenceID.h"
#include "MovieSceneSequencePlayer.h"

//////////////////////////////////////////////////////////////////////////
// FDisplayClusterMoviePipelineViewportManager

FDisplayClusterMoviePipelineViewportManager::FDisplayClusterMoviePipelineViewportManager(const FString& InClusterNodeId, ADisplayClusterRootActor* InRootActor)
	: ClusterNodeId(InClusterNodeId)
	, ViewportManagerRef(IDisplayClusterViewportManager::CreateViewportManager())
{
	check(InRootActor);

	// Exclusively lock the DCRA for MoviePipeline rendering: disabling preview and PIE rendering.
	ViewportManagerRef->GetConfiguration().SetExclusiveLockOnRootActors(true);

	// Set the owner's DCRA to the newly created viewport manager.
	ViewportManagerRef->GetConfiguration().SetRootActor(EDisplayClusterRootActorType::Any, InRootActor);	
}

FDisplayClusterMoviePipelineViewportManager::~FDisplayClusterMoviePipelineViewportManager()
{
	// Release the exclusive lock so preview and PIE rendering can resume.
	ViewportManagerRef->GetConfiguration().SetExclusiveLockOnRootActors(false);

	// This function must be called before destructor.
	ViewportManagerRef->Release();
}

ADisplayClusterRootActor* FDisplayClusterMoviePipelineViewportManager::ResolveRootActor(UMovieSceneSequencePlayer* InSequencePlayer, const TSoftObjectPtr<ADisplayClusterRootActor> InRootActor, const TSoftClassPtr<ADisplayClusterRootActor> InRootActorClass)
{
	// Resolve the target class and name from the configuration override.
	UClass* TargetClass = nullptr;
	FName   TargetActorName;
	if (!InRootActor.IsNull())
	{
		if (InRootActor.IsValid())
		{
			// RootActorRef points to a live actor — use its class and name for an exact match.
			TargetClass = InRootActor->GetClass();
			TargetActorName = InRootActor->GetFName();
		}
		else if (!InRootActorClass.IsNull())
		{
			// RootActorRef is stale (actor deleted or not loaded); fall back to class-only match.
			TargetClass = InRootActorClass.LoadSynchronous();
		}
	}

	// Lambda shared by both search passes (sequence and world).
	// Iterates BoundActors, tracks the best candidates, and returns the first exact match.
	// Out params receive the best class-only and any-class candidates for use as fallbacks.
	auto FindBestMatch = [&](ADisplayClusterRootActor* RootActor,
		ADisplayClusterRootActor*& InOutAnyClass,
		ADisplayClusterRootActor*& InOutTargetClass) -> ADisplayClusterRootActor*
		{
			if (!TargetClass)
			{
				// No override active — any DCRA is acceptable.
				return RootActor;
			}

			InOutAnyClass = InOutAnyClass ? InOutAnyClass : RootActor;

			if (RootActor->IsA(TargetClass))
			{
				InOutTargetClass = InOutTargetClass ? InOutTargetClass : RootActor;

				// Exact match: class matches and either name is unspecified or also matches.
				if (TargetActorName.IsNone() || TargetActorName == RootActor->GetFName())
				{
					return RootActor;
				}

				// No actor name, looking only for class
				if (TargetActorName.IsNone() && InOutTargetClass)
				{
					return InOutTargetClass;
				}
			}

			return nullptr;
		};

	// Pass 1: search sequence bindings.
	ADisplayClusterRootActor* Sequence_AnyRootActor = nullptr;
	ADisplayClusterRootActor* Sequence_TargetClassRootActor = nullptr;
	if (InSequencePlayer)
	{
		// GetSequence()/GetMovieScene() may return null if the sequence asset is not loaded yet.
		const UMovieSceneSequence* Sequence = InSequencePlayer->GetSequence();
		if (const UMovieScene* MovieScene = Sequence ? Sequence->GetMovieScene() : nullptr)
		{
			// Track the best sequence-scoped match separately so it doesn't compete with the world-search candidate.
			for (const FMovieSceneBinding& Binding : MovieScene->GetBindings())
			{
				// Resolve runtime objects bound to this GUID at the root sequence level.
				for (const TWeakObjectPtr<UObject>& BoundObjectIt :
					InSequencePlayer->FindBoundObjects(Binding.GetObjectGuid(), MovieSceneSequenceID::Root))
				{
					if (ADisplayClusterRootActor* RootActor = Cast<ADisplayClusterRootActor>(BoundObjectIt.Get()))
					{
						// Returns the best candidate found so far, or nullptr to keep searching.
						if (ADisplayClusterRootActor* Result = FindBestMatch(RootActor, Sequence_AnyRootActor, Sequence_TargetClassRootActor))
						{
							return Result;
						}
					}
				}
			}
			// No DCRA found in the sequence — fall through to world search.
		}
	}

	// Pass 2: search the world actor list.
	ADisplayClusterRootActor* AnyRootActor = nullptr;
	ADisplayClusterRootActor* TargetClassRootActor = nullptr;
	if (const UWorld* CurrentWorld = InSequencePlayer ? InSequencePlayer->GetWorld() : nullptr)
	{
		for (const TWeakObjectPtr<ADisplayClusterRootActor> RootActorIt : TActorRange<ADisplayClusterRootActor>(CurrentWorld))
		{
			if (ADisplayClusterRootActor* RootActor = RootActorIt.Get())
			{
				if (ADisplayClusterRootActor* Result = FindBestMatch(RootActor, AnyRootActor, TargetClassRootActor))
				{
					return Result;
				}
			}
		}
	}

	// No exact name+class match found in either pass; settle for class-only match, sequence before world.
	if (Sequence_TargetClassRootActor)
	{
		return Sequence_TargetClassRootActor;
	}
	else if (TargetClassRootActor)
	{
		return TargetClassRootActor;
	}

	// Last resort: any DCRA from sequence > any DCRA from world
	if (Sequence_AnyRootActor)
	{
		return Sequence_AnyRootActor;
	}

	return AnyRootActor;
}

void FDisplayClusterMoviePipelineViewportManager::ApplyWarpBlend_RenderThread(FRHICommandListImmediate& RHICmdList, IDisplayClusterViewportProxy* InViewportProxy, const uint32 ContextNum, const FTextureRHIRef& InOutRTT) const
{
	if (!InViewportProxy || !InOutRTT.IsValid()
		|| !InViewportProxy->GetContexts_RenderThread().IsValidIndex(ContextNum)
		|| !InViewportProxy->GetProjectionPolicy_RenderThread()
		|| !InViewportProxy->GetProjectionPolicy_RenderThread()->IsWarpBlendSupported_RenderThread(InViewportProxy))
	{
		return;
	}

	/**
	 * nDisplay WarpBlend PP does :
	 * RTT > copy > TempIn > WarpShader > TempOut > copy back > RTT + Transition for MRP
	 *
	 * RTT     = InOutRTT
	 * TempIn  = EDisplayClusterViewportResourceType::InputShaderResource
	 * TempOut = EDisplayClusterViewportResourceType::AdditionalTargetableResource
	 *
	 * Note: After the final copy-back, the RTT must be in the state OnRenderTargetReady_RenderThread expects
	 */

	// @TODO: Replace this with IDisplayClusterViewportProxy::SetResourcesOverride_RenderThread() —
	// inject TempIn/TempOut as per-frame overrides instead of allocating them through the nDisplay ViewportManager.
	
	// Get temp resources for the WarpBlend PP
	TArray<FRHITexture*>  TempIn, TempOut;
	TArray<FIntRect> TempInRects, TempOutRects;
	if (!InViewportProxy->GetResourcesWithRects_RenderThread(EDisplayClusterViewportResourceType::InputShaderResource, TempIn, TempInRects)
		|| !InViewportProxy->GetResourcesWithRects_RenderThread(EDisplayClusterViewportResourceType::AdditionalTargetableResource, TempOut, TempOutRects)
		|| !TempIn.IsValidIndex(ContextNum)  || !TempInRects.IsValidIndex(ContextNum)
		|| !TempOut.IsValidIndex(ContextNum) || !TempOutRects.IsValidIndex(ContextNum))
	{
		return;
	}
	
	const FIntRect& ViewportContentRect = InViewportProxy->GetContexts_RenderThread()[ContextNum].GetViewportContentRect();
	FRHICopyTextureInfo CopyInfo;
	CopyInfo.Size = FIntVector(ViewportContentRect.Size().X, ViewportContentRect.Size().Y, 1);

	// 1. RTT > copy > TempIn
	{
		CopyInfo.SourcePosition.X = ViewportContentRect.Min.X;
		CopyInfo.SourcePosition.Y = ViewportContentRect.Min.Y;
		CopyInfo.DestPosition.X = TempInRects[ContextNum].Min.X;
		CopyInfo.DestPosition.Y = TempInRects[ContextNum].Min.Y;

		TransitionAndCopyTexture(RHICmdList, InOutRTT, TempIn[ContextNum], CopyInfo);
	}

	// 2. TempIn > WarpShader > TempOut
	{
		// Apply Warp: WarpInputTextures->WarpOutputTextures
		InViewportProxy->GetProjectionPolicy_RenderThread()->ApplyWarpBlend_RenderThread(RHICmdList, InViewportProxy);

		// @TODO: MPCDI/ICVFX projection policies write alpha=1.0 into TempOut, but MRP later inverts the alpha channel,
		// producing alpha=0 in the final output image. The alpha needs to be corrected after ApplyWarpBlend_RenderThread.
	}

	// 3. TempOut > copy back > RTT
	{
		CopyInfo.SourcePosition.X = TempOutRects[ContextNum].Min.X;
		CopyInfo.SourcePosition.Y = TempOutRects[ContextNum].Min.Y;
		CopyInfo.DestPosition.X = ViewportContentRect.Min.X;
		CopyInfo.DestPosition.Y = ViewportContentRect.Min.Y;

		TransitionAndCopyTexture(RHICmdList, TempOut[ContextNum], InOutRTT, CopyInfo);
	}

	// 4. RTT > Add transition for MRP
	{
		// After the final copy-back, the RTT must be in the state OnRenderTargetReady_RenderThread expects
		RHICmdList.Transition(FRHITransitionInfo(InOutRTT, ERHIAccess::CopyDest, ERHIAccess::SRVGraphics));
	}
}

bool FDisplayClusterMoviePipelineViewportManager::BeginNewFrame(
	const UE::DisplayClusterMoviePipeline::FRenderSettings& InRenderSettings,
	UWorld* InWorld,
	const uint32* InFrameNumberOverride)
{
	RenderFrame = FDisplayClusterRenderFrame();

	ADisplayClusterRootActor* RootActor = ViewportManagerRef->GetConfiguration().GetRootActor(EDisplayClusterRootActorType::Scene);
	if (!RootActor)
	{
		return false;
	}

	UWorld* CurrentWorld = InWorld ? InWorld : RootActor->GetWorld();
	if (!CurrentWorld)
	{
		return false;
	}	

	// ADisplayClusterRootActor::Tick() only handles follow-camera in cluster mode, so manually apply it here.
	// When bFollowLocalPlayerCamera is set, move the DCRA to match the current player camera's transform.
	if (RootActor->CurrentConfigData && RootActor->CurrentConfigData->bFollowLocalPlayerCamera)
	{
		if (APlayerController* const CurPlayerController = CurrentWorld->GetFirstPlayerController())
		{
			APlayerCameraManager* const CurPlayerCameraManager = CurPlayerController->PlayerCameraManager;
			if (CurPlayerCameraManager)
			{
				RootActor->SetActorLocationAndRotation(CurPlayerCameraManager->GetCameraLocation(), CurPlayerCameraManager->GetCameraRotation());
			}
		}
	}

	// Get preview settings from RootActor properties
	FDisplayClusterViewport_PreviewSettings NewPreviewSettings = RootActor->GetPreviewSettings(true);
	NewPreviewSettings.bPreviewEnable = false;

	// Dont use preview setting on primary RootActor
	ViewportManagerRef->GetConfiguration().SetPreviewSettings(NewPreviewSettings);

	// Apply the resolution scale before building the frame so all viewports are sized correctly.
	ViewportManagerRef->GetConfiguration().SetRenderResolutionScale(InRenderSettings.RenderResolutionScale);

	// Apply the external overscan mode and value determined above.
	ViewportManagerRef->GetConfiguration().SetExternalOverscan(
		InRenderSettings.ExternalOverscan.bActive,
		InRenderSettings.ExternalOverscan.Fraction.IsSet() ? &InRenderSettings.ExternalOverscan.Fraction.GetValue() : nullptr);

	// Update local node viewports (update\create\delete) and build new render frame
	if (ViewportManagerRef->GetConfiguration().UpdateConfigurationForClusterNode(InRenderSettings.RenderMode, CurrentWorld, ClusterNodeId, InFrameNumberOverride))
	{
		// Override render settings for MRP
		for (TSharedPtr<IDisplayClusterViewport, ESPMode::ThreadSafe>& ViewportIt : ViewportManagerRef->GetCurrentRenderFrameViewports())
		{
			if (!ViewportIt.IsValid())
			{
				continue;
			}

			FDisplayClusterViewport_RenderSettings InOutSettings = ViewportIt->GetRenderSettings();

			// Preserve Alpha channel in WarpBlend; BeginUpdateSettings() resets this flag each frame.
			InOutSettings.bWarpBlendRenderAlphaChannel = true;

			if (InRenderSettings.WarpBlendMode == EDisplayClusterMoviePipelineWarpBlendMode::None)
			{
				// Skip rendering when warp-blend is disabled; BeginUpdateSettings() resets this flag each frame.
				InOutSettings.bSkipRendering = true;
			}

			// Save changes
			ViewportIt->SetRenderSettings(InOutSettings);
		}

		if (ViewportManagerRef->BeginNewFrame(nullptr, RenderFrame))
		{
			ViewportManagerRef->InitializeNewFrame();
			ViewportManagerRef->FinalizeNewFrame();

			return true;
		}
	}

	return false;
}
