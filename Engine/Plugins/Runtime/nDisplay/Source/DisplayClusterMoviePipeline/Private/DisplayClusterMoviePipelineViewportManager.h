// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DisplayClusterMoviePipelineRenderSettings.h"
#include "DisplayClusterRootActor.h"
#include "Render/Viewport/IDisplayClusterViewportManager.h"

/**
 * Manages the viewport manager and render frame for a single cluster node during Movie Pipeline rendering.
 * Each cluster node gets its own instance so nodes render independently of each other
 * and of the DCRA's own viewport manager.
 */
struct FDisplayClusterMoviePipelineViewportManager
{
public:
	/**
	 * Creates a viewport manager for the given cluster node and binds it to InRootActor.
	 * 
	 * @param InClusterNodeId The cluster node this instance is responsible for rendering.
	 * @param InRootActor     The DCRA that owns the cluster configuration.
	 */
	FDisplayClusterMoviePipelineViewportManager(const FString& InClusterNodeId, ADisplayClusterRootActor* InRootActor);

	/** Releases the viewport manager. */
	~FDisplayClusterMoviePipelineViewportManager();

	/**
	 * Updates viewport configuration and builds the render frame for this cluster node.
	 * 
	 * @param InWorld               World to use for rendering. If null, uses the world from the root actor.
	 * @param InFrameNumberOverride Optional frame number override passed to the viewport configuration update.
	 * @return True if the render frame was built successfully; false if the root actor or world is unavailable.
	 */
	bool BeginNewFrame(
		const UE::DisplayClusterMoviePipeline::FRenderSettings& InRenderSettings,
		UWorld* InWorld = nullptr,
		const uint32* InFrameNumberOverride = nullptr);

	/**
	 * Resolves the ADisplayClusterRootActor to render, respecting the provided override references.
	 *
	 * Search order:
	 *   1. Sequence bindings — preferred because the sequence explicitly owns the actor.
	 *   2. World actor list — fallback when the actor is in the world but not bound in the sequence.
	 *
	 * Within each source, the match priority is:
	 *   a. Exact name + class match
	 *   b. Class match only
	 *   c. Any DCRA
	 *
	 * @param InSequencePlayer      Sequence player used to access the sequence bindings and the world.
	 * @param InRootActor           Soft reference to a specific actor to prefer; use a null soft pointer to skip.
	 * @param InRootActorClass      Soft reference to a subclass to prefer when no exact actor match is found; use a null soft pointer to skip.
	 * @return                      The best-matching DCRA, or nullptr if none could be found.
	 */
	static ADisplayClusterRootActor* ResolveRootActor(
		class UMovieSceneSequencePlayer* InSequencePlayer,
		const TSoftObjectPtr<ADisplayClusterRootActor> InRootActor,
		const TSoftClassPtr<ADisplayClusterRootActor> InRootActorClass);

public:
	/**
	 * Apply viewport WarpBlend to the InOutRTT
	 *
	 * nDisplay WarpBlend PP does :
	 * RTT > copy > TempIn > WarpShader > TempOut > copy back > RTT + Transition for MRP
	 *
	 * RTT     = InOutRTT
	 * TempIn  = EDisplayClusterViewportResourceType::InputShaderResource
	 * TempOut = EDisplayClusterViewportResourceType::AdditionalTargetableResource
	 *
	 * Note: After the final copy-back, the RTT must be in the state OnRenderTargetReady_RenderThread expects
	 */
	void ApplyWarpBlend_RenderThread(
		FRHICommandListImmediate& RHICmdList,
		class IDisplayClusterViewportProxy* InViewportProxy,
		const uint32 ContextNum,
		const FTextureRHIRef& InOutRTT) const;

public:
	/** The cluster node ID this context is responsible for. */
	const FString ClusterNodeId;

	/** Viewport manager for this cluster node. Created in the constructor, released in the destructor. */
	const TSharedRef<IDisplayClusterViewportManager, ESPMode::ThreadSafe> ViewportManagerRef;

private:
	/** Current render frame data. */
	FDisplayClusterRenderFrame RenderFrame;
};
