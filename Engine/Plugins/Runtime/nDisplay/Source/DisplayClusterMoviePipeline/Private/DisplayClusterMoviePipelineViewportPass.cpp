// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterMoviePipelineViewportPass.h"
#include "DisplayClusterMoviePipelineSettings.h"

#include "MoviePipeline.h"
#include "MoviePipelineHighResSetting.h"

#include "DisplayClusterConfigurationTypes.h"

#include "Components/DisplayClusterCameraComponent.h"
#include "Engine/GameViewportClient.h"
#include "Render/Projection/IDisplayClusterProjectionPolicy.h"
#include "Render/Viewport/IDisplayClusterViewportManager.h"
#include "Render/Viewport/IDisplayClusterViewportManagerProxy.h"
#include "Render/Viewport/IDisplayClusterViewport.h"
#include "Render/Viewport/IDisplayClusterViewportProxy.h"
#include "RHIUtilities.h"

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
namespace UE::DisplayCluster::MoviePipelineViewportPass
{
	static void DisplayClusterWarpBlendImpl_RenderThread(FRHICommandListImmediate& RHICmdList, const FTextureRHIRef& RenderTargetRHI, IDisplayClusterViewportManagerProxy* InViewportManagerProxy, const FString& InViewportId, const FIntPoint& OffsetMin, const FIntPoint& OffsetMax)
	{
		check(InViewportManagerProxy);
		check(RenderTargetRHI.IsValid());

		if (InViewportManagerProxy)
		{
			if (IDisplayClusterViewportProxy* InViewportProxy = InViewportManagerProxy->FindViewport_RenderThread(InViewportId))
			{
				const TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe>& InPolicy = InViewportProxy->GetProjectionPolicy_RenderThread();
				if (InPolicy.IsValid() && InPolicy->IsWarpBlendSupported_RenderThread(InViewportProxy))
				{
					TArray<FRHITexture*> WarpInputTextures, WarpOutputTextures;
					TArray<FIntRect> WarpInputRects, WarpOutputRects;

					if (InViewportProxy->GetResourcesWithRects_RenderThread(EDisplayClusterViewportResourceType::InputShaderResource, WarpInputTextures, WarpInputRects))
					{
						if (InViewportProxy->GetResourcesWithRects_RenderThread(EDisplayClusterViewportResourceType::AdditionalTargetableResource, WarpOutputTextures, WarpOutputRects))
						{
							const FIntRect RTTRect(FIntPoint(0, 0), RenderTargetRHI->GetSizeXY());
							const FIntRect WarpInputRect = WarpInputRects[0];
							const FIntRect WarpOutpuRect = WarpOutputRects[0];

							FRHICopyTextureInfo CopyInfo;
							CopyInfo.Size = FIntVector(
								RTTRect.Width() - OffsetMin.X - OffsetMax.X,
								RTTRect.Height() - OffsetMin.Y - OffsetMax.Y,
								1);
							{
								// Copy RTTRect->WarpInputRect
								CopyInfo.SourcePosition.X = RTTRect.Min.X + OffsetMin.X;
								CopyInfo.SourcePosition.Y = RTTRect.Min.Y + OffsetMin.Y;
								CopyInfo.DestPosition.X = WarpInputRect.Min.X;
								CopyInfo.DestPosition.Y = WarpInputRect.Min.Y;

								TransitionAndCopyTexture(RHICmdList, RenderTargetRHI, WarpInputTextures[0], CopyInfo);
							}

							// Apply Warp: WarpInputTextures->WarpOutputTextures
							InPolicy->ApplyWarpBlend_RenderThread(RHICmdList, InViewportProxy);

							{
								// Copy WarpOutputRect -> RTTRect
								CopyInfo.SourcePosition.X = WarpOutpuRect.Min.X;
								CopyInfo.SourcePosition.Y = WarpOutpuRect.Min.Y;
								CopyInfo.DestPosition.X = RTTRect.Min.X + OffsetMin.X;
								CopyInfo.DestPosition.Y = RTTRect.Min.Y + OffsetMin.Y;

								TransitionAndCopyTexture(RHICmdList, WarpOutputTextures[0], RenderTargetRHI, CopyInfo);
							}

							// Immediatelly flush RHI before PostRendererSubmission
							RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThreadFlushResources);
						}
					}
				}
			}
		}
	}
};
using namespace UE::DisplayCluster::MoviePipelineViewportPass;

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// UDisplayClusterMoviePipelineViewportPassBase
/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
UDisplayClusterMoviePipelineViewportPassBase::UDisplayClusterMoviePipelineViewportPassBase(const FString& InRenderPassName)
	: UMoviePipelineDeferredPassBase()
	, RenderPassName(InRenderPassName)
{
	PassIdentifier = FMoviePipelinePassIdentifier(RenderPassName);

	bAllowCameraAspectRatio = false;
}

ADisplayClusterRootActor* UDisplayClusterMoviePipelineViewportPassBase::GetRootActor() const
{
	return RootActorWeakPtr.Get();
}

void UDisplayClusterMoviePipelineViewportPassBase::SetupImpl(const MoviePipeline::FMoviePipelineRenderPassInitSettings& InPassInitSettings)
{
	if (InitializeDisplayCluster())
	{
		check(GetRootActor());
	}

	Super::SetupImpl(InPassInitSettings);
}

void UDisplayClusterMoviePipelineViewportPassBase::TeardownImpl()
{
	ReleaseDisplayCluster();

	Super::TeardownImpl();
}

FIntPoint UDisplayClusterMoviePipelineViewportPassBase::GetOverscannedResolutionForCamera(const int32 InCameraIndex) const
{
	if (DisplayClusterViewportSizes.IsValidIndex(InCameraIndex))
	{
		return DisplayClusterViewportSizes[InCameraIndex];
	}

	// Use one size from OutputSettings
	UMoviePipelinePrimaryConfig* PrimaryConfig = GetPipeline()->GetPipelinePrimaryConfig();
	UMoviePipelineExecutorShot* CurrentShot = GetPipeline()->GetActiveShotList()[GetPipeline()->GetCurrentShotIndex()];
	const FIntPoint OutputResolution = UMoviePipelineBlueprintLibrary::GetOverscannedResolution(PrimaryConfig, CurrentShot);

	return OutputResolution;
}

void UDisplayClusterMoviePipelineViewportPassBase::UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const
{
	InTelemetry->bUsesNDisplay = true;
}

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
UE::MoviePipeline::FImagePassCameraViewData UDisplayClusterMoviePipelineViewportPassBase::GetCameraInfo(FMoviePipelineRenderPassMetrics& InOutSampleState, IViewCalcPayload* OptPayload) const
{
	UE::MoviePipeline::FImagePassCameraViewData OutCameraData;

	// This has to come from the main camera for consistency's sake, and it's not a per-camera setting in the editor.
	OutCameraData.ViewActor = GetWorld()->GetFirstPlayerController()->GetViewTarget();

	const int32 InViewportIndex = InOutSampleState.OutputState.CameraIndex;
	FString ViewportId;
	if (GetViewportId(InViewportIndex, ViewportId))
	{
		if (DCViews.Contains(InViewportIndex) && DCPrevViews.Contains(InViewportIndex))
		{
			// We override the current/previous transform
			OutCameraData.bUseCustomProjectionMatrix = true;
			OutCameraData.CustomProjectionMatrix = DCViews[InViewportIndex].ProjectionMatrix;

			OutCameraData.ViewInfo.Location = DCViews[InViewportIndex].ViewLocation;
			OutCameraData.ViewInfo.Rotation = DCViews[InViewportIndex].ViewRotation;

			OutCameraData.ViewInfo.PreviousViewTransform = FTransform(DCPrevViews[InViewportIndex].ViewRotation, DCPrevViews[InViewportIndex].ViewLocation);
		}
	}

	return OutCameraData;
}

void UDisplayClusterMoviePipelineViewportPassBase::BlendPostProcessSettings(FSceneView* InView, FMoviePipelineRenderPassMetrics& InOutSampleState, IViewCalcPayload* OptPayload)
{
	const int32 InViewportIndex = InOutSampleState.OutputState.CameraIndex;
	FString ViewportId;
	if (GetViewportId(InViewportIndex, ViewportId))
	{
		if (DCViews.Contains(InViewportIndex) && DCPrevViews.Contains(InViewportIndex))
		{
			// Use DCRA as a camera
			FVector ViewLocation = DCViews[InViewportIndex].ViewLocation;

			for (IInterface_PostProcessVolume& PPVolume : GetWorld()->GetPostProcessVolumeIterator())
			{
				const FPostProcessVolumeProperties VolumeProperties = PPVolume.GetProperties();

				// Skip any volumes which are disabled
				if (!VolumeProperties.bIsEnabled)
				{
					continue;
				}

				float LocalWeight = FMath::Clamp(VolumeProperties.BlendWeight, 0.0f, 1.0f);

				if (!VolumeProperties.bIsUnbound)
				{
					float DistanceToPoint = 0.0f;
					PPVolume.EncompassesPoint(ViewLocation, 0.0f, &DistanceToPoint);

					if (DistanceToPoint >= 0 && DistanceToPoint < VolumeProperties.BlendRadius)
					{
						LocalWeight *= FMath::Clamp(1.0f - DistanceToPoint / VolumeProperties.BlendRadius, 0.0f, 1.0f);
					}
					else
					{
						LocalWeight = 0.0f;
					}
				}

				InView->OverridePostProcessSettings(*VolumeProperties.Settings, LocalWeight);
			}

			//@todo: It is necessary to discuss which post-processes from DCRA can or cannot be used for MRQ.
		}
	}
}

int32 UDisplayClusterMoviePipelineViewportPassBase::GetNumCamerasToRender() const
{
	// nDisplay always uses viewports as a set of cameras.
	return DisplayClusterViewports.Num();
}

int32 UDisplayClusterMoviePipelineViewportPassBase::GetCameraIndexForRenderPass(const int32 InCameraIndex) const
{
	// nDisplay always uses viewports as a set of cameras.
	return InCameraIndex;
}

FString UDisplayClusterMoviePipelineViewportPassBase::GetCameraName(const int32 InCameraIndex) const
{
	return DisplayClusterViewports.IsValidIndex(InCameraIndex) ? DisplayClusterViewports[InCameraIndex] : TEXT("UndefinedCameraName");
}

FString UDisplayClusterMoviePipelineViewportPassBase::GetCameraNameOverride(const int32 InCameraIndex) const
{
	return DisplayClusterViewports.IsValidIndex(InCameraIndex) ? DisplayClusterViewports[InCameraIndex] : TEXT("");
}

void UDisplayClusterMoviePipelineViewportPassBase::ModifyProjectionMatrixForTiling(const FMoviePipelineRenderPassMetrics& InSampleState, const bool bInOrthographic, FMatrix& InOutProjectionMatrix, float& OutDoFSensorScale) const
{
	const FMatrix InProjectionMatrix(InOutProjectionMatrix);
	Super::ModifyProjectionMatrixForTiling(InSampleState, bInOrthographic, InOutProjectionMatrix, OutDoFSensorScale);

	// Support asymmetric projection matrix (Rebuild assym projection matrix for tile + OverlappedPad)
	if(!bInOrthographic)
	{
		const FIntPoint TileCounts = InSampleState.TileCounts;
		const FIntPoint  TileSize = InSampleState.TileSize;

		const double   TileLeft = InSampleState.OverlappedOffset.X;
		const double  TileRight = TileLeft + TileSize.X + InSampleState.OverlappedPad.X * 2;

		const double    TileTop = InSampleState.OverlappedOffset.Y;
		const double TileBottom = TileTop + TileSize.Y + InSampleState.OverlappedPad.Y * 2;

		const double  Width = TileSize.X * TileCounts.X;
		const double Height = TileSize.Y * TileCounts.Y;

		const float ZNear = GNearClippingPlane;

		// Extract FOV values from matrix
		const double FOVWidth = (2.f * ZNear) / InProjectionMatrix.M[0][0];
		const double FOVHeight = (2.f * ZNear) / InProjectionMatrix.M[1][1];
		const double FOVOffsetX = InProjectionMatrix.M[2][0] * FOVWidth;
		const double FOVOffsetY = InProjectionMatrix.M[2][1] * FOVHeight;

		const double   FOVLeft = -(FOVOffsetX + FOVWidth) / 2.;
		const double  FOVRight = -FOVOffsetX - FOVLeft;
		const double FOVBottom = -(FOVOffsetY + FOVHeight) / 2.;
		const double    FOVTop = -(FOVOffsetY + FOVBottom);

		const FVector2D UnscaledFOV(FOVRight - FOVLeft, FOVBottom - FOVTop);
		const FVector2D FOVCenter(FOVLeft + UnscaledFOV.X * 0.5, FOVTop + UnscaledFOV.Y * 0.5);

		// Support camera overscan percentage
		const FVector2D OverscanFOV(UnscaledFOV * (1.0f + InSampleState.OverscanPercentage));
		const FVector2D FOVMin(FOVCenter.X - OverscanFOV.X * .5, FOVCenter.Y - OverscanFOV.Y * .5);

		// FOV values for Tile
		const double TileFOVLeft   = FOVMin.X + OverscanFOV.X * TileLeft / Width;
		const double TileFOVRight  = FOVMin.X + OverscanFOV.X* TileRight / Width;
		const double TileFOVTop    = FOVMin.Y + OverscanFOV.Y * TileTop / Height;
		const double TileFOVBottom = FOVMin.Y + OverscanFOV.Y * TileBottom / Height;

		InOutProjectionMatrix = IDisplayClusterViewport::MakeProjectionMatrix(TileFOVLeft, TileFOVRight, TileFOVTop, TileFOVBottom, ZNear, ZNear);
	}
}

FSceneView* UDisplayClusterMoviePipelineViewportPassBase::GetSceneViewForSampleState(FSceneViewFamily* ViewFamily, FMoviePipelineRenderPassMetrics& InOutSampleState, IViewCalcPayload* OptPayload)
{
	IDisplayClusterViewport* DCViewport = nullptr;

	const int32 InViewportIndex = InOutSampleState.OutputState.CameraIndex;
	FString ViewportId;

	ADisplayClusterRootActor* DCRootActor = GetRootActor();
	if (DCRootActor && GetViewportId(InViewportIndex, ViewportId))
	{
		UMoviePipelineExecutorShot* CurrentShot = GetPipeline()->GetActiveShotList()[GetPipeline()->GetCurrentShotIndex()];
		UMoviePipelineHighResSetting* HighResSettings = GetPipeline()->FindOrAddSettingForShot<UMoviePipelineHighResSetting>(CurrentShot);

		bFrameWarpBlend = bEnabledWarpBlend;
		if (HighResSettings->TileCount > 1 && bFrameWarpBlend)
		{
			// WarpBlend isn't supported when using tiled rendering
			bFrameWarpBlend = false;
			UE_LOGF(LogMovieRenderPipeline, Warning, "DCRA WarpBlend isn't supported when using tiling! (Viewport '%ls')", *ViewportId);
		}

		FIntPoint OffsetMin = FIntPoint::ZeroValue;
		FIntPoint OffsetMax = FIntPoint::ZeroValue;
		GetViewportCutOffset(InOutSampleState, OffsetMin, OffsetMax);

		const FIntPoint FullDestSize(InOutSampleState.TileSize.X, InOutSampleState.TileSize.Y);
		const FIntPoint DestSize = FullDestSize - OffsetMin - OffsetMax;

		FDisplayClusterViewInfo NewView;
		DCViewport = GetAndCalculateDisplayClusterViewport(InOutSampleState, *ViewportId, DestSize, 0, NewView);
		if(DCViewport)
		{
			// Update view cache
			if (!DCViews.Contains(InViewportIndex))
			{
				DCViews.Emplace(InViewportIndex, NewView);
			}

			DCPrevViews.Emplace(InViewportIndex, DCViews[InViewportIndex]);
			DCViews.Emplace(InViewportIndex, NewView);
		}
		else
		{
			UE_LOGF(LogMovieRenderPipeline, Error, "DCRA render failed for viewport '%ls')", *ViewportId);
		}
	}
	else
	{
		UE_LOGF(LogMovieRenderPipeline, Error, "DCRA render failed: undefined viewport index, skipped");
	}

	FSceneView* SceneView = Super::GetSceneViewForSampleState(ViewFamily, InOutSampleState, OptPayload);

	if (DCViewport && SceneView)
	{
		const uint32 ContextNum = 0;
		DCViewport->SetupSceneView(ContextNum, GetWorld(), *ViewFamily, *SceneView);
	}

	return SceneView;
}

void UDisplayClusterMoviePipelineViewportPassBase::PostRendererSubmission(const FMoviePipelineRenderPassMetrics& InSampleState, const FMoviePipelinePassIdentifier InPassIdentifier, const int32 InSortingOrder, FCanvas& InCanvas)
{
	if (bFrameWarpBlend)
	{
		const int32 InViewportIndex = InSampleState.OutputState.CameraIndex;
		FString ViewportId;

		ADisplayClusterRootActor* DCRootActor = GetRootActor();
		if (DCRootActor && GetViewportId(InViewportIndex, ViewportId))
		{
			if (FRenderTarget* RenderTarget = InCanvas.GetRenderTarget())
			{
				if (IDisplayClusterViewportManager* ViewportManager = DCRootActor->GetViewportManager())
				{
					FIntPoint OffsetMin = FIntPoint::ZeroValue;
					FIntPoint OffsetMax = FIntPoint::ZeroValue;
					GetViewportCutOffset(InSampleState, OffsetMin, OffsetMax);

					ENQUEUE_RENDER_COMMAND(DisplayClusterMoviePipelineWarp)([
						RenderTarget,
						ViewportManagerProxy = ViewportManager->GetProxy(),
						InViewportId = ViewportId,
						OffsetMin, OffsetMax](FRHICommandListImmediate& RHICmdList) mutable
					{
						FTextureRHIRef RenderTargetRHIRef = RenderTarget->GetRenderTargetTexture();
						if (RenderTargetRHIRef.IsValid())
						{
							DisplayClusterWarpBlendImpl_RenderThread(RHICmdList, RenderTargetRHIRef, ViewportManagerProxy, InViewportId, OffsetMin, OffsetMax);
						}
					});
				}
			}
		}
	}

	// Use Deffered render pass
	Super::PostRendererSubmission(InSampleState, InPassIdentifier, InSortingOrder, InCanvas);
}

void UDisplayClusterMoviePipelineViewportPassBase::GetViewportCutOffset(const FMoviePipelineRenderPassMetrics& InSampleState, FIntPoint& OutOffsetMin, FIntPoint& OutOffsetMax) const
{
	// Difference between the clipped constrained rect and the tile rect
	OutOffsetMin = FIntPoint::ZeroValue;
	OutOffsetMax = FIntPoint::ZeroValue;

	APlayerCameraManager* PlayerCameraManager = GetWorld()->GetFirstPlayerController()->PlayerCameraManager;
	if (PlayerCameraManager && PlayerCameraManager->GetCameraCacheView().bConstrainAspectRatio)
	{
		const FMinimalViewInfo CameraCache = PlayerCameraManager->GetCameraCacheView();

		// Taking overscan into account.
		FIntPoint FullOutputSize = UMoviePipelineBlueprintLibrary::GetOverscannedResolution(GetPipeline()->GetPipelinePrimaryConfig(), GetPipeline()->GetActiveShotList()[GetPipeline()->GetCurrentShotIndex()]);

		float OutputSizeAspectRatio = FullOutputSize.X / (float)FullOutputSize.Y;
		const FIntPoint ConstrainedFullSize = CameraCache.AspectRatio > OutputSizeAspectRatio ?
			FIntPoint(FullOutputSize.X, FMath::RoundToInt((double)FullOutputSize.X / (double)CameraCache.AspectRatio)) :
			FIntPoint(FMath::RoundToInt(CameraCache.AspectRatio * FullOutputSize.Y), FullOutputSize.Y);

		const FIntPoint TileViewMin = InSampleState.OverlappedOffset;
		const FIntPoint TileViewMax = TileViewMin + InSampleState.BackbufferSize;

		// Camera ratio constrained rect, clipped by the tile rect
		FIntPoint ConstrainedViewMin = (FullOutputSize - ConstrainedFullSize) / 2;
		FIntPoint ConstrainedViewMax = ConstrainedViewMin + ConstrainedFullSize;
		ConstrainedViewMin = FIntPoint(FMath::Clamp(ConstrainedViewMin.X, TileViewMin.X, TileViewMax.X),
			FMath::Clamp(ConstrainedViewMin.Y, TileViewMin.Y, TileViewMax.Y));
		ConstrainedViewMax = FIntPoint(FMath::Clamp(ConstrainedViewMax.X, TileViewMin.X, TileViewMax.X),
			FMath::Clamp(ConstrainedViewMax.Y, TileViewMin.Y, TileViewMax.Y));

		// Difference between the clipped constrained rect and the tile rect
		OutOffsetMin = ConstrainedViewMin - TileViewMin;
		OutOffsetMax = TileViewMax - ConstrainedViewMax;

		// Support camera overscan
		const float BorderRatio = FMath::Clamp((InSampleState.OverscanPercentage / (1.0f + InSampleState.OverscanPercentage)) / 2.0f, 0.0f, 0.5f);
		const FIntPoint InnerSize = FullOutputSize - OutOffsetMin - OutOffsetMax;
		const int32 OverscanBorderWidth = FMath::RoundToInt(InnerSize.X* BorderRatio);
		const int32 OverscanBorderHeight = FMath::RoundToInt(InnerSize.Y * BorderRatio);

		OutOffsetMin.X += OverscanBorderWidth;
		OutOffsetMin.Y += OverscanBorderHeight;
		OutOffsetMax.X += OverscanBorderWidth;
		OutOffsetMax.Y += OverscanBorderHeight;
	}
}

void UDisplayClusterMoviePipelineViewportPassBase::ReleaseDisplayCluster()
{
	ADisplayClusterRootActor* DCRootActor = GetRootActor();
	if (DCRootActor)
	{
		if (IDisplayClusterViewportManager* ViewportManager = DCRootActor->GetViewportManager())
		{
			// Re-enable preview/PIE rendering for this DCRA.
			ViewportManager->GetConfiguration().SetExclusiveLockOnRootActors(false);
		}

		// Delete all used rendering resources
		// Note: FDisplayClusterViewportManager::Release() remove exclusive locks on release.
		DCRootActor->RemoveViewportManager();
	}

	// Reset all runtime values
	RootActorWeakPtr.Reset();

	bFrameWarpBlend = false;

	DisplayClusterViewports.Empty();
	DCViews.Empty();
	DCPrevViews.Empty();
}

bool UDisplayClusterMoviePipelineViewportPassBase::InitializeDisplayCluster()
{
	check(IsInGameThread());

	ReleaseDisplayCluster();

	UWorld* CurrentWorld = GetWorld();
	if (CurrentWorld == nullptr)
	{
		UE_LOGF(LogMovieRenderPipeline, Error, "Failed to find current world for nDisplay render");

		return false;
	}

	if (UMoviePipelineExecutorShot* CurrentShot = GetPipeline()->GetActiveShotList()[GetPipeline()->GetCurrentShotIndex()])
	{
		if (UDisplayClusterMoviePipelineSettings* DCMoviePipelineSettings = GetPipeline()->FindOrAddSettingForShot<UDisplayClusterMoviePipelineSettings>(CurrentShot))
		{
			ADisplayClusterRootActor* DCRootActor = DCMoviePipelineSettings->GetRootActor(CurrentWorld);
			if (DCRootActor == nullptr)
			{
				UE_LOGF(LogMovieRenderPipeline, Error, "Failed to find nDisplay RootActor");

				return false;
			}

			if (!DCMoviePipelineSettings->GetViewports(CurrentWorld, DisplayClusterViewports, DisplayClusterViewportSizes))
			{
				UE_LOGF(LogMovieRenderPipeline, Error, "Failed to find nDisplay viewports");

				return false;
			}

			if (IDisplayClusterViewportManager* ViewportManager = DCRootActor->GetOrCreateViewportManager())
			{
				// Disable preview/PIE rendering for this DCRA.
				ViewportManager->GetConfiguration().SetExclusiveLockOnRootActors(true);

				// Update local node viewports (update\create\delete) and build new render frame
				if (ViewportManager->GetConfiguration().UpdateConfigurationForViewportsList(EDisplayClusterRenderFrameMode::MRQ_Mono, GetWorld(), DisplayClusterViewports))
				{
					// Finally set this DCRA for rendering
					RootActorWeakPtr = DCRootActor;

					return true;
				}
			}
		}
	}

	UE_LOGF(LogMovieRenderPipeline, Error, "Failed to initialize nDisplay render");
	ReleaseDisplayCluster();

	return false;
}

bool UDisplayClusterMoviePipelineViewportPassBase::GetViewportId(int32 InViewportIndex, FString& OutViewportId) const
{
	ADisplayClusterRootActor* DCRootActor = GetRootActor();
	if (DCRootActor && InViewportIndex >= 0 && InViewportIndex < DisplayClusterViewports.Num())
	{
		OutViewportId = DisplayClusterViewports[InViewportIndex];

		return true;
	}

	return false;
}

IDisplayClusterViewport* UDisplayClusterMoviePipelineViewportPassBase::GetAndCalculateDisplayClusterViewport(const FMoviePipelineRenderPassMetrics& InSampleState, const FString& InViewportId, const FIntPoint& InDestSize, const uint32 InContextNum, FDisplayClusterViewInfo& OutView)
{
	ADisplayClusterRootActor* DCRootActor = GetRootActor();
	check(DCRootActor);
	
	UWorld* CurrentWorld = GetWorld();
	check(CurrentWorld);

	UGameViewportClient* GameViewportClient = CurrentWorld->GetGameViewport();
	check(GameViewportClient);

	IDisplayClusterViewportManager* ViewportManager = DCRootActor->GetViewportManager();
	check(ViewportManager);

	if (IDisplayClusterViewport* DCViewport = ViewportManager->FindViewport(InViewportId))
	{
		// Get camera ID assigned to the viewport
		const FString CameraId = DCViewport->GetRenderSettings().CameraId;

		// Force custom viewport size and settings:
		{
			FDisplayClusterViewport_RenderSettings RenderSettings;
			RenderSettings.Rect = FIntRect(FIntPoint(0, 0), InDestSize);
			RenderSettings.CaptureMode = EDisplayClusterViewportCaptureMode::MoviePipeline;

			// Store camera name for next frames (because UpdateConfiguration called once)
			RenderSettings.CameraId = CameraId;

			if (!bFrameWarpBlend)
			{
				// Create internal resources only when warpblend will be used
				RenderSettings.bSkipRendering = true;
			}

			// Disable DCRA frustum modifier features
			RenderSettings.bDisableCustomFrustumFeature = true;
			RenderSettings.bDisableFrustumOverscanFeature = true;

			// Preserve AlphaOutput for MRQ (based on bAccumulatorIncludesAlpha).
			RenderSettings.bWarpBlendRenderAlphaChannel = bAccumulatorIncludesAlpha;

			DCViewport->SetRenderSettings(RenderSettings);
		}

		FDisplayClusterRenderFrame RenderFrame;
		if (ViewportManager->BeginNewFrame(GameViewportClient->Viewport, RenderFrame))
		{
			// Initialize frame for render (update proxies data)
			ViewportManager->InitializeNewFrame();

			// Obtaining the internal viewpoint for a given viewport with stereo eye offset distance.
			FMinimalViewInfo ViewInfo;
			if (!DCViewport->SetupViewPoint(InContextNum, ViewInfo))
			{
				return nullptr;
			}

			OutView.ViewRotation = ViewInfo.Rotation;
			OutView.ViewLocation = ViewInfo.Location;
			
			bool bResult = false;
			// Obtaining the offset of the stereo eye and the values of the projection clipping plane for the given viewport was moved inside CalculateView().
			// Perform view calculations on a policy side
			if (DCViewport->CalculateView(InContextNum, OutView.ViewLocation, OutView.ViewRotation, ViewportManager->GetConfiguration().GetWorldToMeters()))
			{
				OutView.ProjectionMatrix = FMatrix::Identity;

				bResult = ViewportManager->GetConfiguration().IsSceneOpened() && DCViewport->GetProjectionMatrix(InContextNum, OutView.ProjectionMatrix);
			}

			if (bFrameWarpBlend)
			{
				// Initialize proxy only when warpblend will be used
				ViewportManager->FinalizeNewFrame();
			}

			if (bResult)
			{
				return DCViewport;
			}
		}
	}

	return nullptr;
}

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// UDisplayClusterMoviePipelineViewportPass_PathTracer
/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
#if WITH_EDITOR
FText UDisplayClusterMoviePipelineViewportPass_PathTracer::GetFooterText(UMoviePipelineExecutorJob* InJob) const {
	return NSLOCTEXT(
		"MovieRenderPipeline",
		"DCViewportBasePassSetting_FooterText_PathTracer",
		"Sampling for the nDisplay Path Tracer is controlled by the Anti-Aliasing settings and the Reference Motion Blur setting.\n"
		"All other Path Tracer settings are taken from the Post Process settings.");
}
#endif

void UDisplayClusterMoviePipelineViewportPass_PathTracer::ValidateStateImpl()
{
	Super::ValidateStateImpl();
	PathTracerValidationImpl();
}

void UDisplayClusterMoviePipelineViewportPass_PathTracer::SetupImpl(const MoviePipeline::FMoviePipelineRenderPassInitSettings& InPassInitSettings)
{
	if (!CheckIfPathTracerIsSupported())
	{
		UE_LOGF(LogMovieRenderPipeline, Error, "Cannot render a nDisplay Path Tracer pass, Path Tracer is not enabled by this project.");
		GetPipeline()->Shutdown(true);
		return;
	}

	Super::SetupImpl(InPassInitSettings);
}
