// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/DisplayClusterMovieGraphRenderCameraSource.h"

#include "Graph/Nodes/DisplayClusterMovieGraphDeferredPassNode.h"
#include "Graph/Nodes/DisplayClusterMovieGraphPathTracerPassNode.h"
#include "Render/Projection/IDisplayClusterProjectionPolicy.h"
#include "Render/Viewport/IDisplayClusterViewport.h"
#include "Render/Viewport/IDisplayClusterViewportProxy.h"

#include "Graph/MovieGraphDefaultRenderer.h"
#include "Graph/MovieGraphNode.h"
#include "Graph/MovieGraphPipeline.h"
#include "Graph/MovieGraphSequenceDataSource.h"
#include "Graph/Nodes/MovieGraphCameraNode.h"
#include "MoviePipelineUtils.h"

//////////////////////////////////////////////////////////////////////////
// FDisplayClusterMovieGraphRenderCameraSource

void FDisplayClusterMovieGraphRenderCameraSource::OnBeginFrame(const UMovieGraphPipeline* InMoviePipeline, const FMovieGraphTimeStepData* InTimeData)
{
	// Determine whether to use external overscan and what value to pass to SetExternalOverscan().
	// - Default: external overscan is enabled. If UMovieGraphCameraSettingNode overrides OverscanPercentage,
	//   that explicit value is forwarded; otherwise ExternalOverscanPtr remains null and the viewport manager
	//   falls back to FMinimalViewInfo.GetOverscan() from the MRP camera.
	// - Viewport: nDisplay's own per-viewport overscan settings apply; external overscan is disabled.
	{
		RenderSettings.ExternalOverscan = UE::DisplayClusterMoviePipeline::FRenderSettings::FExternalOverscan();
		switch (RenderSettings.OverscanMode)
		{
		case EDisplayClusterMoviePipelineOverscanMode::Default:
		default:
		{
			if (InTimeData && InTimeData->EvaluatedConfig)
			{
				constexpr bool bIncludeCDOs = false;
				if (const UMovieGraphCameraSettingNode* CameraSetting = InTimeData->EvaluatedConfig
					->GetSettingForBranch<UMovieGraphCameraSettingNode>(UMovieGraphNode::GlobalsPinName, bIncludeCDOs))
				{
					if (CameraSetting->bOverride_OverscanPercentage)
					{
						// OverscanPercentage is [0..100]; normalize to [0..1] to match FMinimalViewInfo.Overscan.
						RenderSettings.ExternalOverscan.Fraction = FMath::Clamp(CameraSetting->OverscanPercentage / 100.f, 0.f, 1.f);
					}
				}
			}
			RenderSettings.ExternalOverscan.bActive = true;
			break;
		}
		case EDisplayClusterMoviePipelineOverscanMode::Viewport:
			// nDisplay's own viewport overscan settings are used; external overscan is disabled.
			break;
		}
	}

	const uint32 FrameNumberOverride = InTimeData ? FMath::Max(0, InTimeData->RenderedFrameNumber) : 0;
	UWorld* CurrentWorld = InMoviePipeline ? InMoviePipeline->GetWorld() : nullptr;

	// Build the render frame for each cluster node.
	for (TPair<FString, TSharedRef<FDisplayClusterMoviePipelineViewportManager, ESPMode::ThreadSafe>>& ClusterNodeIt : ClusterNodes)
	{		
		ClusterNodeIt.Value->BeginNewFrame(RenderSettings, CurrentWorld, InTimeData ? &FrameNumberOverride : nullptr);
	}
}

void FDisplayClusterMovieGraphRenderCameraSource::AddReferencedObjects(FReferenceCollector& Collector)
{
	for (TPair<FString, TSharedRef<FDisplayClusterMoviePipelineViewportManager, ESPMode::ThreadSafe>> ClusterNodeIt : ClusterNodes)
	{
		ClusterNodeIt.Value->ViewportManagerRef->AddReferencedObjects(Collector);
	}
}

bool FDisplayClusterMovieGraphRenderCameraSource::GetCameraName(const int32 InCameraIndex, FString& OutCameraName) const
{
	if (CameraViewports.IsValidIndex(InCameraIndex))
	{
		OutCameraName = CameraViewports[InCameraIndex].CameraName;

		return true;
	}

	return false;
}

bool FDisplayClusterMovieGraphRenderCameraSource::GetCameraViewActor(const int32 InCameraIndex, AActor*& OutCameraActor) const
{
	// ViewActor drives engine owner-visibility culling (bOwnerNoSee/bOnlyOwnerSee) and first-person VSM shadow clipping,
	// but does not affect view/projection matrices. nDisplay handles visibility through its own system, so nullptr here
	// intentionally bypasses the engine owner-visibility path.
	OutCameraActor = nullptr;

	return true;
}

bool FDisplayClusterMovieGraphRenderCameraSource::GetCameraOverscan(const int32 InCameraIndex, float& OutCameraOverscan) const
{
	// Disable overscan for MRP because nDisplay already handles it.
	OutCameraOverscan = 0.f;

	return true;
}

bool FDisplayClusterMovieGraphRenderCameraSource::GetCameraViewInfo(const int32 InCameraIndex, FMinimalViewInfo& OutViewInfo) const
{
	int32 ContextNum = 0;
	IDisplayClusterViewport* Viewport = FindViewport(InCameraIndex, ContextNum);

	return Viewport && Viewport->SetupViewPoint(ContextNum, OutViewInfo);
}

void FDisplayClusterMovieGraphRenderCameraSource::PostRendererSubmission(const int32 InCameraIndex,
	const UE::MovieGraph::FMovieGraphSampleState& InSampleState,
	const UE::MovieGraph::DefaultRenderer::FRenderTargetInitParams& InRenderTargetInitParams,
	FCanvas& InCanvas,
	const UE::MovieGraph::DefaultRenderer::FCameraInfo& InCameraInfo) const
{
	if(RenderSettings.WarpBlendMode == EDisplayClusterMoviePipelineWarpBlendMode::None)
	{
		return;
	}

	// WarpBlend is disabled by the user or incompatible with tiled rendering.
	const bool bIsTiled = InCameraInfo.TilingParams.TileCount.X > 1 || InCameraInfo.TilingParams.TileCount.Y > 1;
	if (bIsTiled)
	{
		if (!RenderSettings.bEnabledWarpBlendErrorMsgShowOnce)
		{
			RenderSettings.bEnabledWarpBlendErrorMsgShowOnce = true;
			UE_LOGF(LogMovieRenderPipeline, Warning, "DCRA '%ls': WarpBlend is not supported when using tiled rendering and has been disabled.", *ClusterInfo.RootActorName);
		}

		return;
	}

	FRenderTarget* RenderTarget = InCanvas.GetRenderTarget();
	if (!RenderTarget)
	{
		return;
	}

	int32 ContextNum = 0;
	IDisplayClusterViewport* Viewport = FindViewport(InCameraIndex, ContextNum);
	if (!Viewport)
	{
		return;
	}

	if (!Viewport->GetProjectionPolicy().IsValid()
		|| !Viewport->GetProjectionPolicy()->IsWarpBlendSupported(Viewport))
	{
		// Skip viewports that not supports projection policy
		return;
	}

	const TSharedRef<FDisplayClusterMoviePipelineViewportManager, ESPMode::ThreadSafe> MoviePipelineViewportManager = ClusterNodes[Viewport->GetClusterNodeId()];
	ENQUEUE_RENDER_COMMAND(DisplayClusterMovieGraphWarpBlend)(
		[MoviePipelineViewportManager, ViewportProxy = Viewport->GetViewportProxy(), ContextNum, RenderTarget](FRHICommandListImmediate& RHICmdList)
		{
			MoviePipelineViewportManager->ApplyWarpBlend_RenderThread(RHICmdList, ViewportProxy.ToSharedPtr().Get(), ContextNum, RenderTarget->GetRenderTargetTexture());
		});
}
