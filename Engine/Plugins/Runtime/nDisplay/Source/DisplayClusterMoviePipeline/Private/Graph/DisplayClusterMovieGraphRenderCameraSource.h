// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/MovieGraphRenderCameraSource.h"

#include "DisplayClusterMoviePipelineViewportManager.h"
#include "DisplayClusterMoviePipelineViewportCameraInfo.h"

#include "DisplayClusterMoviePipelineEnums.h"
#include "DisplayClusterMoviePipelineRenderSettings.h"

#include "Graph/MovieGraphDataTypes.h"
#include "Graph/MovieGraphDefaultRenderer.h"

class UDisplayClusterMovieGraphDeferredRenderPassNode;
struct FMovieGraphResolveArgs;
struct FMovieGraphRenderDataIdentifier;

/**
 * nDisplay implementation of FMovieGraphRenderCameraSource.
 *
 * Replaces the default MRP cameras with the nDisplay viewports defined by a render pass node.
 * During Initialize(), it resolves the root actor, enumerates viewports across all cluster nodes,
 * and creates one FDisplayClusterMoviePipelineViewportManager per node so that multiple nDisplay
 * instances in the same graph can render independently.
 */
struct FDisplayClusterMovieGraphRenderCameraSource
	: public FMovieGraphRenderCameraSource
{
	virtual ~FDisplayClusterMovieGraphRenderCameraSource() = default;

public:
	// FMovieGraphRenderCameraSource Interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual void OnBeginFrame(const UMovieGraphPipeline* InMoviePipeline, const FMovieGraphTimeStepData* InTimeData) override;

	virtual int32 GetNumCameras() const override { return CameraViewports.Num(); }

	virtual bool GetCameraName(const int32 InCameraIndex, FString& OutCameraName) const override;
	virtual bool GetCameraViewActor(const int32 InCameraIndex, AActor*& OutCameraActor) const override;
	virtual bool GetCameraOverscan(const int32 InCameraIndex, float& OutCameraOverscan) const override;

	virtual bool GetCameraViewInfo(const int32 InCameraIndex, FMinimalViewInfo& OutViewInfo) const override;
	virtual void SetupCameraViewProjectionData(const int32 InCameraIndex, FSceneViewProjectionData& InOutProjectionData) const override;

	virtual bool GetCameraOverscannedResolution(const int32 InCameraIndex, FIntPoint& InOutOverscannedResolution, FIntRect& InOutOverscanCropRectangle, float& InOutOverscanFraction) const override;
	// ~FMovieGraphRenderCameraSource Interface

public:
	/**
	 * Resolves the root actor and viewports, creates per-cluster-node viewport managers, and
	 * populates this source with the resulting render pass layer data.
	 * Returns false if the root actor could not be resolved or no viewports are available.
	 *
	 * @param InRenderPassNode      Render pass node providing configuration (output method, viewports, etc.).
	 * @param InMovieGraphPipeline  Active pipeline, used to resolve the sequence player and world.
	 * @param InEvaluatedConfig     Evaluated graph config for the current shot.
	 */
	template<typename TDisplayClusterRenderPassNode>
	bool Initialize(
		const TDisplayClusterRenderPassNode& InRenderPassNode,
		const UMovieGraphPipeline* InMovieGraphPipeline,
		const UMovieGraphEvaluatedConfig* InEvaluatedConfig);

	/**
	 * Implements FMovieGraphImagePassBase::ModifyProjectionMatrixForTiling() for nDisplay.
	 * 
 	 * @param InCameraIndex          Index into the camera viewport list.
	 * @param InTilingParams        Tiling configuration for the current sample.
	 * @param bInOrthographic       Whether the camera uses an orthographic projection.
	 * @param InOutProjectionMatrix Projection matrix to adjust for the current tile.
	 * @param OutDoFSensorScale     Receives the sensor scale factor used for depth-of-field calculations.
	 */
	bool ModifyProjectionMatrixForTiling(
		const int32 InCameraIndex,
		const UE::MovieGraph::DefaultRenderer::FMovieGraphTilingParams& InTilingParams,
		const bool bInOrthographic,
		FMatrix& InOutProjectionMatrix,
		float& OutDoFSensorScale) const;

	/**
	 * Returns the set of nDisplay filename tokens for the UI. Uses dummy instances because no DCRA or live viewports are present in that context.
	 * 
	 * @param InRenderDataIdentifier  Identifier used to derive the metadata path prefix.
	 * @param OutMergedFormatArgs     Receives the nDisplay filename and metadata token entries.
	 */
	static void GetFormatResolveArgs(const FMovieGraphRenderDataIdentifier& InRenderDataIdentifier, FMovieGraphResolveArgs& OutMergedFormatArgs);

	/**
	 * Applies nDisplay-specific overrides (e.g. layer name) to the sample state for the given camera.
	 * 
	 * @param InCameraIndex          Index into the camera viewport list.
	 * @param InRenderDataIdentifier Identifier for the current render data, used to derive per-viewport overrides.
	 * @param InOutSampleState       Sample state to modify in place.
	 */
	void ApplyCameraOverridesToSampleState(
		const int32 InCameraIndex,
		const FMovieGraphRenderDataIdentifier& InRenderDataIdentifier,
		UE::MovieGraph::FMovieGraphSampleState& InOutSampleState) const;

	/**
	 * Called after the renderer submits a frame; applies warp-blend to the canvas render target for the given camera.
	 * 
	 * @param InCameraIndex             Index into the camera viewport list.
	 * @param InSampleState             Sample state for the current frame.
	 * @param InRenderTargetInitParams  Parameters used to initialize the render target.
	 * @param InCanvas                  Canvas whose render target receives the warp-blend result.
	 * @param InCameraInfo              Camera and tiling parameters for the current sample.
	 */
	void PostRendererSubmission(const int32 InCameraIndex,
		const UE::MovieGraph::FMovieGraphSampleState& InSampleState,
		const UE::MovieGraph::DefaultRenderer::FRenderTargetInitParams& InRenderTargetInitParams,
		FCanvas& InCanvas,
		const UE::MovieGraph::DefaultRenderer::FCameraInfo& InCameraInfo) const;

	/**
	 * Returns the viewport for the given camera index, or nullptr if out of range.
	 * 
	 * @param CameraIndex    Index into the camera viewport list.
	 * @param OutContextNum  Receives the view context index (0 = mono/left, 1 = right).
	 * @return               The viewport, or nullptr if CameraIndex is out of range.
	 */
	inline IDisplayClusterViewport* FindViewport(const int32 CameraIndex, int32& OutContextNum) const
	{
		OutContextNum = 0;
		
		if (CameraViewports.IsValidIndex(CameraIndex))
		{
			OutContextNum = CameraViewports[CameraIndex].ContextNum;
			return FindViewport(CameraViewports[CameraIndex]);
		}

		return nullptr;
	}

	/**
	 * Returns the viewport for the given camera entry, or nullptr if the cluster node or viewport is not found.
	 * 
	 * @param InCamera  Camera entry identifying the cluster node and viewport to look up.
	 * @return          The viewport, or nullptr if the cluster node or viewport ID is not registered.
	 */
	inline IDisplayClusterViewport* FindViewport(const UE::DisplayClusterMoviePipeline::FViewportCameraInfo& InCamera) const
	{
		if (const TSharedRef<FDisplayClusterMoviePipelineViewportManager, ESPMode::ThreadSafe>* DeviceContext = ClusterNodes.Find(InCamera.ClusterNodeInfo.Id))
		{
			return (*DeviceContext)->ViewportManagerRef->FindViewport(InCamera.ViewportId);
		}

		return nullptr;
	}

private:
	/** Cached render settings. */
	UE::DisplayClusterMoviePipeline::FRenderSettings RenderSettings;

	/** Root actor info, populated by Initialize(). */
	UE::DisplayClusterMoviePipeline::FClusterInfo ClusterInfo;

	/** Viewports selected for rendering this shot, populated by Initialize(). */
	TArray<UE::DisplayClusterMoviePipeline::FViewportCameraInfo> CameraViewports;

	/**
	 * One viewport manager per cluster node (keyed by node ID).
	 * Each node gets its own manager instead of sharing the DCRA's manager,
	 * so multiple nDisplay nodes in the same graph can render independently.
	 */
	TMap<FString, TSharedRef<FDisplayClusterMoviePipelineViewportManager, ESPMode::ThreadSafe>> ClusterNodes;
};
