// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/DisplayClusterMovieGraphRenderCameraSource.h"

#include "Graph/Nodes/DisplayClusterMovieGraphDeferredPassNode.h"
#include "Graph/Nodes/DisplayClusterMovieGraphPathTracerPassNode.h"
#include "Render/Viewport/IDisplayClusterViewport.h"

#include "Graph/MovieGraphNode.h"
#include "Graph/MovieGraphPipeline.h"
#include "Graph/MovieGraphSequenceDataSource.h"
#include "MovieGraphImageSequenceOutputNode.h"
#include "MovieRenderPipelineCoreModule.h"

namespace UE::DisplayClusterMovieGraph
{
	/** Returns the IDs of all cluster nodes defined in the root actor's configuration. */
	static TArray<FString> GetClusterNodeIds(ADisplayClusterRootActor& InRootActor)
	{
		TArray<FString> OutClusterNodeNames;
		if (const UDisplayClusterConfigurationData* InConfigurationData = InRootActor.GetConfigData())
		{
			if (const UDisplayClusterConfigurationCluster* InClusterCfg = InConfigurationData->Cluster)
			{
				for (const TPair<FString, TObjectPtr<UDisplayClusterConfigurationClusterNode>>& NodeIt : InClusterCfg->Nodes)
				{
					if (const UDisplayClusterConfigurationClusterNode* InConfigurationClusterNode = NodeIt.Value)
					{
						OutClusterNodeNames.Add(NodeIt.Key);
					}
				}
			}
		}

		return OutClusterNodeNames;
	}

	/** Returns true if the given cluster node should be rendered. Always true when AllowedNodeNamesList override is not active. */
	template<typename TDisplayClusterRenderPassNode>
	static bool CanUseClusterNode(const TDisplayClusterRenderPassNode& InConfiguration, const FString& InClusterNodeId)
	{
		if (!InConfiguration.bOverride_AllowedNodeNamesList || InConfiguration.AllowedNodeNamesList.IsEmpty())
		{
			return true;
		}

		return InConfiguration.AllowedNodeNamesList.Contains(InClusterNodeId);
	}

	/**
	* Returns true if the given viewport should be rendered.
	* @param InViewport  The viewport to check.
	*/
	template<typename TDisplayClusterRenderPassNode>
	static bool CanUseViewport(const TDisplayClusterRenderPassNode& InConfiguration, const class IDisplayClusterViewport* InViewport)
	{
		if (!InViewport)
		{
			return false;
		}

		// ICVFX viewports are not yet supported.
		//@TODO: add ICVFX rendering rules
		if (EnumHasAnyFlags(InViewport->GetViewportFlags(), EDisplayClusterViewportFlags::ICVFX_AnyRole))
		{
			return false;
		}

		// Filter by name when the allowlist override is active.
		if (InConfiguration.bOverride_AllowedViewportNamesList && !InConfiguration.AllowedViewportNamesList.IsEmpty())
		{
			if (!InConfiguration.AllowedViewportNamesList.Contains(InViewport->GetId()))
			{
				return false;
			}
		}

		return true;
	}

} // namespace UE::DisplayClusterMovieGraph

//////////////////////////////////////////////////////////////////////////
// FDisplayClusterMovieGraphRenderCameraSource::Initialize

template<typename TDisplayClusterRenderPassNode>
bool FDisplayClusterMovieGraphRenderCameraSource::Initialize(
	const TDisplayClusterRenderPassNode& InRenderPassNode,
	const UMovieGraphPipeline* MoviePipeline,
	const UMovieGraphEvaluatedConfig* EvaluatedConfig)
{
	using namespace UE::DisplayClusterMovieGraph;

	CameraViewports.Empty();
	ClusterNodes.Empty();

	// RenderMode controls how many eye contexts the viewport manager allocates per viewport:
	// MRG_Stereo = two contexts (left + right), MRG_Mono = one context (center).
	const bool bRenderStereo = InRenderPassNode.StereoMode != EDisplayClusterMoviePipelineStereoMode::None;
	const EDisplayClusterRenderFrameMode RenderMode = bRenderStereo ?
		EDisplayClusterRenderFrameMode::MRG_Stereo : EDisplayClusterRenderFrameMode::MRG_Mono;

	// When rendering a single eye, SingleEyeContextIndex pins the loop below to one context
	// (0 = left/mono, 1 = right). INDEX_NONE means render all contexts (Mono or full Stereo).
	int32 SingleEyeContextIndex = INDEX_NONE;
	switch (InRenderPassNode.StereoMode)
	{
	case EDisplayClusterMoviePipelineStereoMode::LeftEye:  SingleEyeContextIndex = 0; break;
	case EDisplayClusterMoviePipelineStereoMode::RightEye: SingleEyeContextIndex = 1; break;
	default: break;
	}

	// Get sequence player
	const UMovieGraphSequenceDataSource* DataSource = Cast<UMovieGraphSequenceDataSource>(
		MoviePipeline ? MoviePipeline->GetDataSourceInstance() : nullptr);
	UMovieSceneSequencePlayer* SequencePlayer = DataSource ? DataSource->GetSequencePlayer() : nullptr;

	// Resolves the ADisplayClusterRootActor to render.
	ADisplayClusterRootActor* RootActor = InRenderPassNode.bOverride_RootActorRef
		? FDisplayClusterMoviePipelineViewportManager::ResolveRootActor(SequencePlayer, InRenderPassNode.RootActorRef, InRenderPassNode.RootActorClassRef)
		: FDisplayClusterMoviePipelineViewportManager::ResolveRootActor(SequencePlayer, nullptr, nullptr);
	if (!RootActor)
	{
		return false;
	}

	// Populate cluster info
	{	
		ClusterInfo.RootActorName = RootActor->GetActorLabel();
		RootActor->GetClass()->GetName(ClusterInfo.RootActorClassName);
	}

	// Cache settings from the render pass node; used throughout the render session.
	{
		RenderSettings.RenderMode = RenderMode;

		RenderSettings.OverscanMode = InRenderPassNode.OverscanMode;
		RenderSettings.RenderResolutionScale = InRenderPassNode.ResolutionScale;
		RenderSettings.OutputResolutionOverride = InRenderPassNode.OutputResolution;

		RenderSettings.WarpBlendMode = InRenderPassNode.WarpBlendMode;
		RenderSettings.bEnabledWarpBlendErrorMsgShowOnce = false;

		// Support multi-layer EXR output
		if (EvaluatedConfig)
		{
			RenderSettings.bIsMultiLayerEXROutput =
				!EvaluatedConfig->GetSettingsForBranch(
					UMovieGraphImageSequenceOutputNode_MultiLayerEXR::StaticClass(),
					UMovieGraphNode::GlobalsPinName,
					/*bIncludeCDOs*/false,
					/*bExactMatch*/true).IsEmpty();
		}
	}

	/** One renderable view: an nDisplay viewport paired with a stereo context index (one eye of one viewport). */
	struct FViewToRender 
	{
		/** nDisplay viewport. */
		TSharedRef<IDisplayClusterViewport, ESPMode::ThreadSafe> ViewportRef;

		/** Stereo context index (0 = mono or left eye, 1 = right eye). */
		int32 ContextNum = 0;
	};

	// All views to render across every cluster node — one entry per viewport per stereo eye.
	TArray<FViewToRender> PendingViews;

	// Create cluster nodes and collect viewports
	for (const FString& ClusterNodeId : GetClusterNodeIds(*RootActor))
	{
		if (!CanUseClusterNode(InRenderPassNode, ClusterNodeId))
		{
			continue;
		}

		// Create instance of ViewportManager for this node
		const TSharedRef<FDisplayClusterMoviePipelineViewportManager, ESPMode::ThreadSafe> RenderDeviceContext =
			MakeShared<FDisplayClusterMoviePipelineViewportManager, ESPMode::ThreadSafe>(ClusterNodeId, RootActor);
		if (!RenderDeviceContext->BeginNewFrame(RenderSettings))
		{
			continue;
		}

		// Collect viewports from this node
		int32 NumRenderableViewports = 0;
		for (const TSharedPtr<IDisplayClusterViewport, ESPMode::ThreadSafe>& ViewportIt :
			RenderDeviceContext->ViewportManagerRef->GetCurrentRenderFrameViewports())
		{
			if (!ViewportIt.IsValid())
			{
				continue;
			}

			if (ViewportIt->GetClusterNodeId() != ClusterNodeId)
			{
				continue;
			}

			if (!CanUseViewport(InRenderPassNode, ViewportIt.Get()))
			{
				continue;
			}

			// Render all contexts of this viewport
			int32 NumRenderableContexts = 0;
			for (int32 ContextNum = 0; ContextNum < ViewportIt->GetContexts().Num(); ContextNum++)
			{
				if (SingleEyeContextIndex != INDEX_NONE && ContextNum != SingleEyeContextIndex)
				{
					// Skip contexts
					continue;
				}

				const FDisplayClusterViewport_Context& SrcContext = ViewportIt->GetContexts()[ContextNum];

				// Collect only viewports that can be rendered
				if (SrcContext.bValidData && SrcContext.bCalculated)
				{
					PendingViews.Add({ ViewportIt.ToSharedRef(), ContextNum });
					NumRenderableContexts++;
				}
				else
				{
					// Notify about skipped viewports whose context data is not yet valid or calculated
					UE_LOGF(LogMovieRenderPipeline, Warning, "nDisplay viewport '%ls' on node '%ls' cannot be rendered; ignoring this viewport.", *ViewportIt->GetId(), *ViewportIt->GetClusterNodeId());
				}
			}

			// Count this viewport only if it contributed at least one renderable context;
			// viewports whose contexts were all invalid don't add to the topology count.
			if (NumRenderableContexts > 0)
			{
				NumRenderableViewports++;
			}
		}

		// Keep the node's RenderDeviceContext alive only when it has work to do — empty nodes
		// would otherwise inflate ClusterNodes.Num() and skew the topology classification.
		if (NumRenderableViewports > 0)
		{
			ClusterNodes.Emplace(ClusterNodeId, RenderDeviceContext);
		}
	}

	// Classify the cluster topology.
	{
		using namespace UE::DisplayClusterMoviePipeline;

		ClusterInfo.ClusterTopology =
			  PendingViews.Num() <= 1 ? EClusterTopology::None         // single view — no disambiguation needed
			: ClusterNodes.Num() <= 1 ? EClusterTopology::PerViewport  // multiple views on one node
			:                           EClusterTopology::PerNode;     // views span multiple nodes
	}

	// Emit one camera entry per pending view; CameraViewports drives the actual render loop.
	for (const FViewToRender& ViewIt : PendingViews)
	{
		CameraViewports.Add({
			RenderSettings,
			ClusterInfo,
			ViewIt.ViewportRef,
			ViewIt.ContextNum });
	}

	return !CameraViewports.IsEmpty();
}

// Explicit instantiations — Initialize<T> is defined in this TU; callers in other TUs
// need these symbols to be emitted here so the linker can resolve them.
template bool FDisplayClusterMovieGraphRenderCameraSource::Initialize<UDisplayClusterMovieGraphDeferredRenderPassNode>(
	const UDisplayClusterMovieGraphDeferredRenderPassNode&, const UMovieGraphPipeline*, const UMovieGraphEvaluatedConfig*);

template bool FDisplayClusterMovieGraphRenderCameraSource::Initialize<UDisplayClusterMovieGraphPathTracerRenderPassNode>(
	const UDisplayClusterMovieGraphPathTracerRenderPassNode&, const UMovieGraphPipeline*, const UMovieGraphEvaluatedConfig*);


