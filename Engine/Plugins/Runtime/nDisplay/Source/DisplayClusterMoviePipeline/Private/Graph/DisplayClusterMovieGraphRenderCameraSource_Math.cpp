// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/DisplayClusterMovieGraphRenderCameraSource.h"

#include "Math/DisplayClusterProjectionMath.h"
#include "Render/Viewport/IDisplayClusterViewport.h"

#include "Graph/MovieGraphDefaultRenderer.h"

// Validates the round-trip encoding of the nDisplay projection matrix through FMinimalViewInfo
// (FOV, AspectRatio, OffCenterProjectionOffset). Certain MRP node configurations can modify
// ViewInfo and silently break the encoding. Temporary: remove once the encoding is stable.
#define DISPLAYCLUSTERMOVIEGRAPH_VALIDATE_PROJECTION_ENCODING 1

void FDisplayClusterMovieGraphRenderCameraSource::SetupCameraViewProjectionData(const int32 InCameraIndex, FSceneViewProjectionData& InOutProjectionData) const
{
	int32 ContextNum = 0;
	IDisplayClusterViewport* Viewport = FindViewport(InCameraIndex, ContextNum);
	if (!Viewport || !Viewport->GetContexts().IsValidIndex(ContextNum))
	{
		return;
	}

	const FDisplayClusterViewport_Context& SrcContext = Viewport->GetContexts()[ContextNum];

	// Temporary validation: encoding the nDisplay projection matrix into FMinimalViewInfo parameters
	// (FOV, AspectRatio, OffCenterProjectionOffset) and recovering it via CalculateProjectionMatrix()
	// is a new feature. This check verifies the round-trip math is correct and should be removed
	// once the encoding is well-tested.
#if DISPLAYCLUSTERMOVIEGRAPH_VALIDATE_PROJECTION_ENCODING
	if (!FDisplayClusterProjectionMath::MatricesNearlyEqual(
		InOutProjectionData.ProjectionMatrix, SrcContext.GetRenderProjectionMatrix())
		&& CameraViewports.IsValidIndex(InCameraIndex))
	{
		UE_LOGF(LogMovieRenderPipeline, Warning,
			"nDisplay projection matrix encoding mismatch for viewport '%ls'.\n  Computed: %ls\n  Expected: %ls",
			*CameraViewports[InCameraIndex].CameraName,
			*InOutProjectionData.ProjectionMatrix.ToString(),
			*SrcContext.GetRenderProjectionMatrix().ToString()
		);
	}
#endif /** DISPLAYCLUSTERMOVIEGRAPH_VALIDATE_PROJECTION_ENCODING */

	// nDisplay camera aspect ratio should ignore RTT aspect ratio.
	InOutProjectionData.SetViewRectangle(SrcContext.RenderTargetRect);

	// Set the view origin to the eye location for this context (stereo offset already applied).
	InOutProjectionData.ViewOrigin = SrcContext.ViewData.RenderLocation;
}

bool FDisplayClusterMovieGraphRenderCameraSource::GetCameraOverscannedResolution(const int32 InCameraIndex, FIntPoint& InOutResolution, FIntRect& InOutCropRect, float& InOutOverscanFraction) const
{
	int32 ContextNum = 0;
	IDisplayClusterViewport* Viewport = FindViewport(InCameraIndex, ContextNum);
	if (!Viewport || !Viewport->GetContexts().IsValidIndex(ContextNum))
	{
		return false;
	}

	const FDisplayClusterViewport_Context& SrcContext = Viewport->GetContexts()[ContextNum];
	InOutResolution = SrcContext.RenderTargetRect.Size();
	InOutCropRect   = SrcContext.OverscanInnerRenderTargetRect;

	// Disable overscan for MRP because nDisplay already handles it.
	InOutOverscanFraction = 0.f;

	return true;
}


bool FDisplayClusterMovieGraphRenderCameraSource::ModifyProjectionMatrixForTiling(
		const int32 InCameraIndex,
		const UE::MovieGraph::DefaultRenderer::FMovieGraphTilingParams& InTilingParams,
		const bool bInOrthographic,
		FMatrix& InOutProjectionMatrix,
		float& OutDoFSensorScale) const
{
	// Not implemented
	return false;
}
