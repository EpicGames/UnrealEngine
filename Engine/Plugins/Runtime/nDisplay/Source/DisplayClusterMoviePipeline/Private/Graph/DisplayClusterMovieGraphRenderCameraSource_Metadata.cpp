// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/DisplayClusterMovieGraphRenderCameraSource.h"

#include "Components/DisplayClusterCameraComponent.h"
#include "Render/Projection/IDisplayClusterProjectionPolicy.h"

#include "DisplayClusterMoviePipelineViewportCameraInfo.h"

#include "Graph/MovieGraphDefaultRenderer.h"
#include "MoviePipelineUtils.h"

namespace UE::DisplayClusterMovieGraph
{
	/**
	 * Populates nDisplay format tokens and file metadata.
	 *
	 * @param InCameraInfo             Camera info ref
	 * @param InRenderDataIdentifier   Identifies the render pass; used to build the metadata key prefix.
	 * @param InOutFileMetadata        Map receiving per-file metadata key/value pairs.
	 * @param InOutFormatArgs          Map receiving output path format token/value pairs.
	 */
	static void GetFormatResolveArgsFromCameraInfo(
		const UE::DisplayClusterMoviePipeline::FViewportCameraInfo& InCameraInfo,
		const FMovieGraphRenderDataIdentifier& InRenderDataIdentifier,
		TMap<FString, FString>& InOutFileMetadata,
		TMap<FString, FString>& InOutFormatArgs)
	{
		const FString StereoEyeId = InCameraInfo.GetStereoEyeName();

		InOutFormatArgs.Add(TEXT("ndisplay_dcra"),       InCameraInfo.ClusterInfo.RootActorName);
		InOutFormatArgs.Add(TEXT("ndisplay_dcra_class"), InCameraInfo.ClusterInfo.RootActorClassName);

		InOutFormatArgs.Add(TEXT("ndisplay_node"),      InCameraInfo.ClusterNodeInfo.Id);
		InOutFormatArgs.Add(TEXT("ndisplay_node_host"), InCameraInfo.ClusterNodeInfo.Host);

		InOutFormatArgs.Add(TEXT("ndisplay_viewport"),     InCameraInfo.ViewportId);
		InOutFormatArgs.Add(TEXT("ndisplay_viewport_eye"), StereoEyeId);

		if (!InCameraInfo.IsActive())
		{
			// do not produce metadata for dummy camera info.
			return;
		}

		const FString MetadataPrefix = UE::MoviePipeline::GetMetadataPrefixPath(InRenderDataIdentifier);

		// metadata for multilayer should be in json format?
		InOutFileMetadata.Add(FString::Printf(TEXT("%s/ndisplay/dcra"), *MetadataPrefix), InCameraInfo.ClusterInfo.RootActorName);
		InOutFileMetadata.Add(FString::Printf(TEXT("%s/ndisplay/dcra/class"), *MetadataPrefix), InCameraInfo.ClusterInfo.RootActorClassName);

		InOutFileMetadata.Add(FString::Printf(TEXT("%s/ndisplay/node"), *MetadataPrefix), InCameraInfo.ClusterNodeInfo.Id);
		InOutFileMetadata.Add(FString::Printf(TEXT("%s/ndisplay/node/host"), *MetadataPrefix), InCameraInfo.ClusterNodeInfo.Host);

		InOutFileMetadata.Add(FString::Printf(TEXT("%s/ndisplay/viewport"), *MetadataPrefix), InCameraInfo.ViewportId);

		// Optionally add tag for eye
		if (InCameraInfo.bStereoRendering)
		{
			InOutFileMetadata.Add(FString::Printf(TEXT("%s/ndisplay/viewport/eye"), *MetadataPrefix), StereoEyeId);
		}
	}

	/**
	 * Selects the tokens that replace {camera_name} in the File Name Format.
	 *
	 * nDisplay viewports are always treated as multi-camera, so the result is guaranteed to be non-empty:
	 * if topology and stereo state don't contribute any tokens, {camera_name} is appended as a fallback to
	 * force the multi-camera code paths downstream.
	 *
	 * @param InCameraInfo         Camera and cluster context for this view.
	 * @param OutCameraNameTokens  Receives the token sequence. Never empty on return - falls back to {camera_name}.
	 */
	static void GetCameraNameTokenOverridesFromCameraInfo(
		const UE::DisplayClusterMoviePipeline::FViewportCameraInfo& InCameraInfo,
		TArray<FString>& OutCameraNameTokens)
	{
		using namespace UE::DisplayClusterMoviePipeline;

		switch (InCameraInfo.ClusterInfo.ClusterTopology)
		{
		// Multiple nodes, multiple viewports - use {ndisplay_node} and {ndisplay_viewport} to disambiguate across the cluster.
		case EClusterTopology::PerNode:
			OutCameraNameTokens = {
				TEXT("{ndisplay_node}"),
				TEXT("{ndisplay_viewport}")
			};
			break;

		// One node, multiple viewports - use {ndisplay_viewport} to disambiguate within the node.
		case EClusterTopology::PerViewport:
			OutCameraNameTokens = {
				TEXT("{ndisplay_viewport}")
			};
			break;

		// One node, one viewport - no topology-based disambiguation needed. Cleared here, but the
		// fallback at the bottom of this function will still seed {camera_name} so the result is non-empty.
		case EClusterTopology::None:
		default:
			OutCameraNameTokens.Empty();
			break;
		}

		// Stereo renders the same viewport twice (left + right eye); the eye token keeps the two outputs from colliding.
		if (InCameraInfo.bStereoRendering)
		{
			if(OutCameraNameTokens.IsEmpty())
			{
				// None topology skipped tokens, but the eye suffix needs a viewport base to attach to -
				// inject {ndisplay_viewport} so left/right eyes don't collide on the same output name.
				OutCameraNameTokens.Add(TEXT("{ndisplay_viewport}"));
			}
			
			OutCameraNameTokens.Add(TEXT("{ndisplay_viewport_eye}"));
		}

		// Fallback: nDisplay viewports are always treated as multi-camera, so the token list must not be empty.
		// Reached only by EClusterTopology::None with no stereo rendering - earlier branches already populated tokens otherwise.
		if (OutCameraNameTokens.IsEmpty())
		{
			// {camera_name} keeps the user's filename format unchanged while still forcing the downstream
			// multi-camera code paths to engage. This does not cause additional cameras to render.
			OutCameraNameTokens.Add(TEXT("{camera_name}"));
		}
	}

	/**
	 * Builds the layer name override.
	 *
	 * @param InRenderDataIdentifier  Identifies the render pass.
	 * @return                        The composed layer name override string.
	 */
	static FString GetLayerNameOverride(
		const FMovieGraphRenderDataIdentifier& InRenderDataIdentifier)
	{
		TArray<FString> Tokens;
		Tokens.Add(InRenderDataIdentifier.LayerName);
		Tokens.Add(InRenderDataIdentifier.RendererName);
		Tokens.Add(InRenderDataIdentifier.SubResourceName);
		Tokens.Add(InRenderDataIdentifier.CameraName);

		FString OutLayerName = Tokens[0];
		for (int32 Index = 1; Index < Tokens.Num(); ++Index)
		{
			OutLayerName = FString::Printf(TEXT("%s_%s"), *OutLayerName, *Tokens[Index]);
		}

		return OutLayerName;
	}

	/**
	 * Appends per-sample spatial metadata.
	 *
	 * @param InViewport              nDisplay Viewport.
	 * @param InContextNum            Stereo context index (0 = mono/left eye, 1 = right eye).
	 * @param InOutMetadata           Map to receive the additional key/value metadata pairs.
	 * @param InRenderDataIdentifier  Identifies the render pass; used to build the metadata key prefix.
	 */
	static void UpdateSpatialSampleMetadata(
		IDisplayClusterViewport& InViewport,
		const int32 InContextNum,
		TMap<FString, FString>& InOutMetadata,
		const FMovieGraphRenderDataIdentifier& InRenderDataIdentifier)
	{
		if (!InViewport.GetContexts().IsValidIndex(InContextNum))
		{
			return;
		}

		const FString MetadataPrefix = UE::MoviePipeline::GetMetadataPrefixPath(InRenderDataIdentifier);

		// For stereo viewports, record the render-camera origin position of this eye.
		const bool bStereoRendering = InViewport.GetContexts().Num() > 1;
		if (bStereoRendering)
		{
			const FDisplayClusterViewport_Context& SrcContext = InViewport.GetContexts()[InContextNum];

			const FVector OriginLoc = SrcContext.ViewData.RenderLocation;
			InOutMetadata.Add(FString::Printf(TEXT("%s/originPos/x"), *MetadataPrefix), FString::SanitizeFloat(OriginLoc.X));
			InOutMetadata.Add(FString::Printf(TEXT("%s/originPos/y"), *MetadataPrefix), FString::SanitizeFloat(OriginLoc.Y));
			InOutMetadata.Add(FString::Printf(TEXT("%s/originPos/z"), *MetadataPrefix), FString::SanitizeFloat(OriginLoc.Z));
		}

		// Emit CineCamera metadata (focal length, aperture, focus distance, etc.) when the active camera is a CineCameraComponent.
		// Resolution order: DCRA ViewPoint component's target camera, then projection policy camera override (e.g. 'camera' policy type).
		{
			UCameraComponent* CameraComponent = nullptr;

			// Start with the camera resolved from the DCRA ViewPoint component.
			if (UDisplayClusterCameraComponent* ViewPointComponent = InViewport.GetViewPointCameraComponent(EDisplayClusterRootActorType::Scene))
			{
				CameraComponent = ViewPointComponent->GetTargetCameraComponent(InViewport.GetConfiguration());
			}

			// Projection policy may supply its own camera (e.g. 'camera' policy), which takes priority.
			if (InViewport.GetProjectionPolicy().IsValid())
			{
				if (UCameraComponent* CustomCameraComponent = InViewport.GetProjectionPolicy()->GetCameraComponent())
				{
					CameraComponent = CustomCameraComponent;
				}
			}

			// Only CineCameraComponents carry the extra cinematic properties we want to record.
			if (UCineCameraComponent* CineCameraComponent = Cast<UCineCameraComponent>(CameraComponent))
			{
				UE::MoviePipeline::GetMetadataFromCineCamera(CineCameraComponent, MetadataPrefix, InOutMetadata);
			}
		}
	}
} // namespace UE::DisplayClusterMovieGraph

//////////////////////////////////////////////////////////////////////////
// FDisplayClusterMovieGraphRenderCameraSource

void FDisplayClusterMovieGraphRenderCameraSource::ApplyCameraOverridesToSampleState(
	const int32 InCameraIndex,
	const FMovieGraphRenderDataIdentifier& InRenderDataIdentifier,
	UE::MovieGraph::FMovieGraphSampleState& InOutSampleState) const
{
	if (!CameraViewports.IsValidIndex(InCameraIndex))
	{
		return;
	}

	UE::DisplayClusterMovieGraph::GetFormatResolveArgsFromCameraInfo(CameraViewports[InCameraIndex], InRenderDataIdentifier, InOutSampleState.AdditionalFileMetadata, InOutSampleState.AdditionalFormatArgs);
	UE::DisplayClusterMovieGraph::GetCameraNameTokenOverridesFromCameraInfo(CameraViewports[InCameraIndex], InOutSampleState.CameraNameTokenOverrides);

	// Override layer name
	InOutSampleState.LayerNameOverride = UE::DisplayClusterMovieGraph::GetLayerNameOverride(InRenderDataIdentifier);

	// Get viewport metadata.
	int32 ContextNum = 0;
	if (IDisplayClusterViewport* Viewport = FindViewport(InCameraIndex, ContextNum))
	{
		UE::DisplayClusterMovieGraph::UpdateSpatialSampleMetadata(*Viewport, ContextNum, InOutSampleState.AdditionalFileMetadata, InRenderDataIdentifier);
	}
}

void FDisplayClusterMovieGraphRenderCameraSource::GetFormatResolveArgs(const FMovieGraphRenderDataIdentifier& InRenderDataIdentifier, FMovieGraphResolveArgs& OutMergedFormatArgs)
{
	// Static function called by the editor UI to enumerate available filename tokens.
	// Uses dummy instances because no DCRA or live viewports are present in that context.
	static UE::DisplayClusterMoviePipeline::FViewportCameraInfo DummyCameraInfo;
	UE::DisplayClusterMovieGraph::GetFormatResolveArgsFromCameraInfo(DummyCameraInfo, InRenderDataIdentifier, OutMergedFormatArgs.FileMetadata, OutMergedFormatArgs.FilenameArguments);
}
