// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/Nodes/DisplayClusterMovieGraphDeferredPassNode.h"

#include "Graph/DisplayClusterMovieGraphRenderCameraSource.h"
#include "Graph/Renderers/DisplayClusterMovieGraphDeferredPass.h"
#include "Render/Viewport/IDisplayClusterViewport.h"

#include "Graph/MovieGraphPipeline.h"
#include "MoviePipelineTelemetry.h"
#include "MoviePipelineUtils.h"
#include "MovieRenderPipelineCoreModule.h"

#include "EngineUtils.h"
#include "LevelSequence.h"
#include "LevelSequencePlayer.h"
#include "MovieScene.h"
#include "MovieSceneBinding.h"


UDisplayClusterMovieGraphDeferredRenderPassNode::UDisplayClusterMovieGraphDeferredRenderPassNode()
{
	RendererName = TEXT("nDisplayDeferred");
}

UDisplayClusterMovieGraphDeferredRenderPassNode::~UDisplayClusterMovieGraphDeferredRenderPassNode() = default;

void UDisplayClusterMovieGraphDeferredRenderPassNode::SetupImpl(const FMovieGraphRenderPassSetupData& InSetupData)
{
	Super::SetupImpl(InSetupData);
}

void UDisplayClusterMovieGraphDeferredRenderPassNode::TeardownImpl()
{
	Super::TeardownImpl();
}

TSharedPtr<FMovieGraphRenderCameraSource> UDisplayClusterMovieGraphDeferredRenderPassNode::CreateRenderCameraSourceImpl(
	const UMovieGraphPipeline* InMovieGraphPipeline,
	const UMovieGraphEvaluatedConfig* InEvaluatedConfig)
{
	// Camera source is per-instance — the CDO has no shot context to initialize against.
	if (IsTemplate(RF_ClassDefaultObject))
	{
		return nullptr;
	}

	// Lazy-initialize once per shot. bRenderCameraSourceInitialized guards against
	// re-initialization if Initialize() fails (RenderCameraSource stays null in that case,
	// and the renderer falls back to default sequence cameras).
	if (!bRenderCameraSourceInitialized && InMovieGraphPipeline)
	{
		bRenderCameraSourceInitialized = true;

		TSharedPtr<FDisplayClusterMovieGraphRenderCameraSource> NewRenderCameraSource =
			MakeShared<FDisplayClusterMovieGraphRenderCameraSource>();

		if (NewRenderCameraSource->Initialize(*this, InMovieGraphPipeline, InEvaluatedConfig))
		{
			RenderCameraSource = NewRenderCameraSource;
		}
	}

	return RenderCameraSource;
}

TUniquePtr<UE::MovieGraph::Rendering::FMovieGraphImagePassBase> UDisplayClusterMovieGraphDeferredRenderPassNode::CreateInstance() const
{
	return MakeUnique<UE::MovieGraph::Rendering::FDisplayClusterMovieGraphDeferredPass>();
}

#if WITH_EDITOR
FText UDisplayClusterMovieGraphDeferredRenderPassNode::GetNodeTitle(const bool bGetDescriptive) const
{
	return NSLOCTEXT("DisplayClusterMoviePipeline", "DeferredRenderPassNode_Title", "nDisplay Deferred Renderer");
}

void UDisplayClusterMovieGraphDeferredRenderPassNode::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	static const FName PropMemberName_RootActor = GET_MEMBER_NAME_CHECKED(UDisplayClusterMovieGraphDeferredRenderPassNode, RootActorRef);

	if (PropertyChangedEvent.Property
		&& PropertyChangedEvent.Property->GetFName() == PropMemberName_RootActor)
	{
		if (RootActorRef.IsNull())
		{
			// Path was cleared: drop the cached class.
			RootActorClassRef = nullptr;
		}
		else if (RootActorRef.IsValid())
		{
			// Path is set and the asset is loaded: capture the class now.
			RootActorClassRef = RootActorRef->GetClass();
		}
		// else: path is set but asset is not yet loaded; keep the existing RootActorClassRef.
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif /* WITH_EDITOR */


void UDisplayClusterMovieGraphDeferredRenderPassNode::GetFormatResolveArgs(FMovieGraphResolveArgs& OutMergedFormatArgs, const FMovieGraphRenderDataIdentifier& InRenderDataIdentifier) const
{
	Super::GetFormatResolveArgs(OutMergedFormatArgs, InRenderDataIdentifier);

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		// this is the CDO
		FDisplayClusterMovieGraphRenderCameraSource::GetFormatResolveArgs(InRenderDataIdentifier, OutMergedFormatArgs);
	}
}

void UDisplayClusterMovieGraphDeferredRenderPassNode::UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const
{
	Super::UpdateTelemetry(InTelemetry);

	if (InTelemetry)
	{
		InTelemetry->bUsesNDisplay = true;
	}
}