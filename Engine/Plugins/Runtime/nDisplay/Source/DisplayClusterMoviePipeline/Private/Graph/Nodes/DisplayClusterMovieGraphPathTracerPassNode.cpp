// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/Nodes/DisplayClusterMovieGraphPathTracerPassNode.h"

#include "Graph/DisplayClusterMovieGraphRenderCameraSource.h"
#include "Graph/Renderers/DisplayClusterMovieGraphPathTracerPass.h"
#include "Render/Viewport/IDisplayClusterViewport.h"

#include "Graph/MovieGraphPipeline.h"
#include "MoviePipelineTelemetry.h"
#include "MoviePipelineUtils.h"
#include "MovieRenderPipelineCoreModule.h"

#include "EngineUtils.h"
#include "LevelSequence.h"
#include "LevelSequencePlayer.h"
#include "MovieScene.h"
#include "MovieSceneBinding.h"


UDisplayClusterMovieGraphPathTracerRenderPassNode::UDisplayClusterMovieGraphPathTracerRenderPassNode()
{
	RendererName = TEXT("nDisplayPathTracer");
}

UDisplayClusterMovieGraphPathTracerRenderPassNode::~UDisplayClusterMovieGraphPathTracerRenderPassNode() = default;

void UDisplayClusterMovieGraphPathTracerRenderPassNode::SetupImpl(const FMovieGraphRenderPassSetupData& InSetupData)
{
	Super::SetupImpl(InSetupData);
}

void UDisplayClusterMovieGraphPathTracerRenderPassNode::TeardownImpl()
{
	Super::TeardownImpl();
}

TSharedPtr<FMovieGraphRenderCameraSource> UDisplayClusterMovieGraphPathTracerRenderPassNode::CreateRenderCameraSourceImpl(
	const UMovieGraphPipeline* InMovieGraphPipeline,
	const UMovieGraphEvaluatedConfig* InEvaluatedConfig)
{
	// Camera source is per-instance — the CDO has no shot context to initialize against.
	if (IsTemplate(RF_ClassDefaultObject))
	{
		return nullptr;
	}

	// Lazy-initialize once per shot. bRenderCameraSourceInitialized guards against
	// re-initialization if Initialize() fails (RenderCameraSource stays null in that case,
	// and the renderer falls back to default sequence cameras).
	if (!bRenderCameraSourceInitialized && InMovieGraphPipeline)
	{
		bRenderCameraSourceInitialized = true;

		TSharedPtr<FDisplayClusterMovieGraphRenderCameraSource> NewRenderCameraSource =
			MakeShared<FDisplayClusterMovieGraphRenderCameraSource>();

		if (NewRenderCameraSource->Initialize(*this, InMovieGraphPipeline, InEvaluatedConfig))
		{
			RenderCameraSource = NewRenderCameraSource;
		}
	}

	return RenderCameraSource;
}

TUniquePtr<UE::MovieGraph::Rendering::FMovieGraphImagePassBase> UDisplayClusterMovieGraphPathTracerRenderPassNode::CreateInstance() const
{
	return MakeUnique<UE::MovieGraph::Rendering::FDisplayClusterMovieGraphPathTracerPass>();
}

#if WITH_EDITOR
FText UDisplayClusterMovieGraphPathTracerRenderPassNode::GetNodeTitle(const bool bGetDescriptive) const
{
	return NSLOCTEXT("DisplayClusterMoviePipeline", "PathTracerRenderPassNode_Title", "nDisplay PathTracer Renderer");
}

void UDisplayClusterMovieGraphPathTracerRenderPassNode::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	static const FName PropMemberName_RootActor = GET_MEMBER_NAME_CHECKED(UDisplayClusterMovieGraphPathTracerRenderPassNode, RootActorRef);

	if (PropertyChangedEvent.Property
		&& PropertyChangedEvent.Property->GetFName() == PropMemberName_RootActor)
	{
		if (RootActorRef.IsNull())
		{
			RootActorClassRef = nullptr;
		}
		else if (RootActorRef.IsValid())
		{
			RootActorClassRef = RootActorRef->GetClass();      // stores a soft ref to the class
		}
		// else: path is set but asset not loaded; preserve existing RootActorClassRef
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif

void UDisplayClusterMovieGraphPathTracerRenderPassNode::GetFormatResolveArgs(FMovieGraphResolveArgs& OutMergedFormatArgs, const FMovieGraphRenderDataIdentifier& InRenderDataIdentifier) const
{
	Super::GetFormatResolveArgs(OutMergedFormatArgs, InRenderDataIdentifier);

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		// this is the CDO
		FDisplayClusterMovieGraphRenderCameraSource::GetFormatResolveArgs(InRenderDataIdentifier, OutMergedFormatArgs);
	}
}

void UDisplayClusterMovieGraphPathTracerRenderPassNode::UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const
{
	Super::UpdateTelemetry(InTelemetry);

	if (InTelemetry)
	{
		InTelemetry->bUsesNDisplay = true;
	}
}
