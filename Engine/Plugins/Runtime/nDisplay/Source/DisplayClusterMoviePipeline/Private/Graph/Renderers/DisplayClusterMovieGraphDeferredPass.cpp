// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/Renderers/DisplayClusterMovieGraphDeferredPass.h"

#include "Graph/DisplayClusterMovieGraphRenderCameraSource.h"
#include "Graph/Nodes/DisplayClusterMovieGraphDeferredPassNode.h"

#include "Graph/MovieGraphConfig.h"
#include "Graph/MovieGraphDataTypes.h"
#include "Graph/MovieGraphDefaultRenderer.h"
#include "Graph/MovieGraphPipeline.h"
#include "Graph/Nodes/MovieGraphCameraNode.h"

#include "MoviePipelineUtils.h"

namespace UE::MovieGraph::Rendering
{
	void FDisplayClusterMovieGraphDeferredPass::ModifyProjectionMatrixForTiling(const UE::MovieGraph::DefaultRenderer::FMovieGraphTilingParams& InTilingParams, const bool bInOrthographic, FMatrix& InOutProjectionMatrix, float& OutDoFSensorScale) const
	{
		if (const FDisplayClusterMovieGraphRenderCameraSource* CameraSource = static_cast<const FDisplayClusterMovieGraphRenderCameraSource*>(GetRenderCameraSource().Get()))
		{
			if (CameraSource->ModifyProjectionMatrixForTiling(GetRenderCameraIndex(), InTilingParams, bInOrthographic, InOutProjectionMatrix, OutDoFSensorScale))
			{
				return;
			}
		}

		return FMovieGraphDeferredPass::ModifyProjectionMatrixForTiling(InTilingParams, bInOrthographic, InOutProjectionMatrix, OutDoFSensorScale);
	}

	void FDisplayClusterMovieGraphDeferredPass::PostRendererSubmission(
		const UE::MovieGraph::FMovieGraphSampleState& InSampleState,
		const UE::MovieGraph::DefaultRenderer::FRenderTargetInitParams& InRenderTargetInitParams,
		FCanvas& InCanvas,
		const UE::MovieGraph::DefaultRenderer::FCameraInfo& InCameraInfo) const
	{
		if (const FDisplayClusterMovieGraphRenderCameraSource* CameraSource = static_cast<const FDisplayClusterMovieGraphRenderCameraSource*>(GetRenderCameraSource().Get()))
		{
			CameraSource->PostRendererSubmission(GetRenderCameraIndex(), InSampleState, InRenderTargetInitParams, InCanvas, InCameraInfo);
		}

		FMovieGraphImagePassBase::PostRendererSubmission(InSampleState, InRenderTargetInitParams, InCanvas, InCameraInfo);
	}

	void FDisplayClusterMovieGraphDeferredPass::ApplyMovieGraphOverridesToSampleState(FMovieGraphSampleState& SampleState) const
	{
		if (const FDisplayClusterMovieGraphRenderCameraSource* CameraSource = static_cast<const FDisplayClusterMovieGraphRenderCameraSource*>(GetRenderCameraSource().Get()))
		{
			return CameraSource->ApplyCameraOverridesToSampleState(GetRenderCameraIndex(), RenderDataIdentifier, SampleState);
		}
	}

	UMovieGraphImagePassBaseNode* FDisplayClusterMovieGraphDeferredPass::GetParentNode(UMovieGraphEvaluatedConfig* InConfig) const
	{
		const bool bIncludeCDOs = true;
		UDisplayClusterMovieGraphDeferredRenderPassNode* ParentNode =
			InConfig->GetSettingForBranch<UDisplayClusterMovieGraphDeferredRenderPassNode>(GetBranchName(), bIncludeCDOs);

		if (!ensureMsgf(ParentNode, TEXT("FDisplayClusterMovieGraphDeferredPass should not exist without a UDisplayClusterMovieGraphDeferredRenderPassNode in the graph.")))
		{
			return nullptr;
		}

		return ParentNode;
	}
}