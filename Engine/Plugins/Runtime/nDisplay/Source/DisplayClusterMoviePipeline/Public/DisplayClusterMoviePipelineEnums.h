// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "DisplayClusterMoviePipelineEnums.generated.h"

/**
 * Defines how nDisplay is rendered for Movie Pipeline.
 */
UENUM(BlueprintType)
enum class EDisplayClusterMoviePipelineOutputMethod : uint8
{
	/** Renders each viewport to a separate texture. */
	PerViewportOutput = 0 UMETA(DisplayName = "Per-Viewport Output"),

	/** Renders cluster nodes to a texture using the DCRA output mapping layout. */
	PerNodeOutputMapping UMETA(DisplayName = "Per-Node Output Mapping"),

	/** Renders the full cluster to a texture using the DCRA output mapping layout. */
	FullClusterOutputMapping UMETA(DisplayName = "Full Cluster Output Mapping"),

	/** Renders the full cluster to a texture using the front-facing 180-degree equirectangular projection and DefaultViewPoint as the eye origin. */
	FrontProjection UMETA(DisplayName = "180 Equirectangular Output"),

	/** Renders the full cluster to a texture using the full 360-degree equirectangular projection and DefaultViewPoint as the eye origin. */
	FullProjection UMETA(DisplayName = "360 Equirectangular Output"),

	/** Renders the full cluster to a texture using a custom mesh as the UV projection surface and DefaultViewPoint as the eye origin. */
	CustomMeshProjection UMETA(DisplayName = "Custom Mesh Projection Output"),
};

/**
 * Stereo rendering mode for nDisplay Movie Pipeline.
 */
UENUM(BlueprintType)
enum class EDisplayClusterMoviePipelineStereoMode : uint8
{
	/** No stereo — renders a single mono (center) view. */
	None = 0 UMETA(DisplayName = "None"),

	/** Renders both left and right eye views. */
	Stereo UMETA(DisplayName = "Stereo"),

	/** Renders the left eye view only. */
	LeftEye UMETA(DisplayName = "Left Eye"),

	/** Renders the right eye view only. */
	RightEye UMETA(DisplayName = "Right Eye"),
};

/**
 * Controls how nDisplay warp-blend is composited during Movie Pipeline rendering.
 */
UENUM(BlueprintType)
enum class EDisplayClusterMoviePipelineWarpBlendMode : uint8
{
	/** Warp-blend is disabled; the raw render target is passed through unchanged. */
	None = 0 UMETA(DisplayName = "None"),

	/** Apply warp-blend. */
	WarpBlend UMETA(DisplayName = "Warp Blend")
};

/**
 * Selects the overscan source for nDisplay Movie Pipeline.
 */
UENUM(BlueprintType)
enum class EDisplayClusterMoviePipelineOverscanMode : uint8
{
	/** Use the overscan configured in MRP (default behavior). */
	Default = 0 UMETA(DisplayName = "Default"),

	/** Use the overscan configured on each nDisplay viewport. */
	Viewport UMETA(DisplayName = "nDisplay Viewport")
};

/**
 * Controls how nDisplay viewports are grouped into multi-layer EXR files.
 * Only applies when the Movie Graph output node supports multi-layer output.
 */
UENUM(BlueprintType)
enum class EDisplayClusterMoviePipelineEXRLayerGrouping : uint8
{
	/** Each viewport is written to a separate file. No multi-layer grouping. */
	None = 0 UMETA(DisplayName = "None"),

	/**
	 * Each viewport (and its stereo eye contexts) is written as one or two layers in a
	 * shared EXR file. Mono produces one layer per viewport; stereo produces two (left + right).
	 */
	Viewport UMETA(DisplayName = "Viewport"),

	/**
	 * All viewports belonging to the same cluster node are grouped into a single EXR file,
	 * one layer per viewport.
	 */
	Node UMETA(DisplayName = "Node"),

	/**
	 * All viewports across all cluster nodes are grouped into a single EXR file,
	 * one layer per viewport.
	 */
	Cluster UMETA(DisplayName = "Cluster"),
};


