// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MoviePipelineSetting.h"
#include "DisplayClusterRootActor.h"

#include "DisplayClusterMoviePipelineSettings.generated.h"

USTRUCT(Blueprintable)
struct FDisplayClusterMoviePipelineConfiguration
{
	GENERATED_BODY()

	// Reference to Display Cluster Root Actor
	// If not set, the first available in the scene will be set.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "nDisplay", meta = (DisplayName = "DC Root Actor"))
	TSoftObjectPtr<ADisplayClusterRootActor> DCRootActor;

	// Container with DCRA instance class:
	// for using multiple instances of DCRA of the same class in multiple scenes.
	UPROPERTY()
	TSoftClassPtr<ADisplayClusterRootActor> DCRootActorClass;

	// Render with nDisplay viewport resolutions.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "nDisplay")
	bool bUseViewportResolutions = true;

	// Render all viewports
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "nDisplay")
	bool bRenderAllViewports = true;

	// Render only viewports from this list.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "nDisplay", meta = (DisplayName = "Allowed Viewport Names List", EditCondition = "!bRenderAllViewports && DCRootActor != nullptr"))
	TArray<FString> AllowedViewportNamesList;
};

/**
 * nDisplay settings for MoviePipeline
 */
UCLASS(Blueprintable)
class DISPLAYCLUSTERMOVIEPIPELINE_API UDisplayClusterMoviePipelineSettings : public UMoviePipelineSetting
{
	GENERATED_BODY()

public:
	/**
	 * return a pointer to the specified DCRA from the current world
	 */
	ADisplayClusterRootActor* GetRootActor(const UWorld* InWorld) const;

	/**
	* Collect viewport names for rendering
	* 
	* @param InWorld       - World for render
	* @param OutViewports  - [Out] list of viewports to render in MoviewPipeline
	* @param OutViewportResolutions - [Out] list of viewports dimensions
	* 
	 * return true if is whether it populated 1 or more viewports in the given array.
	 */
	bool GetViewports(const UWorld* InWorld, TArray<FString>& OutViewports, TArray<FIntPoint>& OutViewportResolutions) const;

#if WITH_EDITOR
	virtual FText GetDisplayText() const override { return NSLOCTEXT("MovieRenderPipeline", "DisplayClusterSettingDisplayName", "nDisplay"); }

	// Begin UObject
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	// End UObject
#endif

protected:
	virtual bool IsValidOnShots() const override { return true; }
	virtual bool IsValidOnPrimary() const override { return true; }

public:
	// Reference to Display Cluster Root Actor
	// If not set, the first available in the scene will be set.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "nDisplay", meta = (DisplayName = "nDisplay"))
	FDisplayClusterMoviePipelineConfiguration Configuration;
};
