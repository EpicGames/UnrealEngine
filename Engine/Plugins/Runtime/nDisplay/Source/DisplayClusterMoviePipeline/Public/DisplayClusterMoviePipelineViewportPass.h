// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MoviePipelineDeferredPasses.h"

#include "DisplayClusterMoviePipelineViewportPass.generated.h"

class ADisplayClusterRootActor;

/**
 * nDisplay viewport render pass (Lit)
 */
UCLASS(BlueprintType)
class DISPLAYCLUSTERMOVIEPIPELINE_API UDisplayClusterMoviePipelineViewportPassBase
	: public UMoviePipelineDeferredPassBase
{
	GENERATED_BODY()

	struct FDisplayClusterViewInfo
	{
		FVector  ViewLocation;
		FRotator ViewRotation;
		FMatrix  ProjectionMatrix;
	};

public:
	UDisplayClusterMoviePipelineViewportPassBase(const FString& InRenderPassName);
	UDisplayClusterMoviePipelineViewportPassBase() : UDisplayClusterMoviePipelineViewportPassBase(TEXT("nDisplayLit")) { }

public:
	// UMoviePipelineDeferredPassBase
	virtual void SetupImpl(const MoviePipeline::FMoviePipelineRenderPassInitSettings& InPassInitSettings) override;
	virtual void TeardownImpl() override;

	virtual FSceneView* GetSceneViewForSampleState(FSceneViewFamily* ViewFamily, FMoviePipelineRenderPassMetrics& InOutSampleState, IViewCalcPayload* OptPayload) override;
	virtual void ModifyProjectionMatrixForTiling(const FMoviePipelineRenderPassMetrics& InSampleState, const bool bInOrthographic, FMatrix& InOutProjectionMatrix, float& OutDoFSensorScale) const override;
	virtual void PostRendererSubmission(const FMoviePipelineRenderPassMetrics& InSampleState, const FMoviePipelinePassIdentifier InPassIdentifier, const int32 InSortingOrder, FCanvas& InCanvas) override;

	virtual int32 GetNumCamerasToRender() const override;
	virtual int32 GetCameraIndexForRenderPass(const int32 InCameraIndex) const override;
	virtual FString GetCameraName(const int32 InCameraIndex) const override;
	virtual FString GetCameraNameOverride(const int32 InCameraIndex) const override;

	virtual UE::MoviePipeline::FImagePassCameraViewData GetCameraInfo(FMoviePipelineRenderPassMetrics& InOutSampleState, IViewCalcPayload* OptPayload = nullptr) const override;
	virtual void BlendPostProcessSettings(FSceneView* InView, FMoviePipelineRenderPassMetrics& InOutSampleState, IViewCalcPayload* OptPayload = nullptr) override;

	virtual FIntPoint GetOverscannedResolutionForCamera(const int32 InCameraIndex) const override;
	// ~UMoviePipelineDeferredPassBase

#if WITH_EDITOR
	virtual FText GetDisplayText() const override { return NSLOCTEXT("MovieRenderPipeline", "DCViewportBasePassSetting_DisplayName_Lit", "nDisplay Rendering"); }
#endif

	virtual int32 GetOutputFileSortingOrder() const override { return 2; }
	virtual bool bIsEnabledWarpBlend() const { return true; }

	virtual void UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const override;

protected:
	bool GetViewportId(int32 InViewportIndex, FString& OutViewportId) const;

	class IDisplayClusterViewport* GetAndCalculateDisplayClusterViewport(const FMoviePipelineRenderPassMetrics& InSampleState, const FString& InViewportId, const FIntPoint& InDestSize, const uint32 InContextNum, FDisplayClusterViewInfo& OutView);

	bool InitializeDisplayCluster();
	void ReleaseDisplayCluster();

	void GetViewportCutOffset(const FMoviePipelineRenderPassMetrics& InSampleState, FIntPoint& OutOffsetMin, FIntPoint& OutOffsetMax) const;

	/** Return current DCRA. */
	ADisplayClusterRootActor* GetRootActor() const;

public:
	// Allow warp blend for this pass
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "nDisplay")
	bool bEnabledWarpBlend = true;

private:
	// Render pass name
	const FString RenderPassName;

	// Scene DCRA.
	TWeakObjectPtr<ADisplayClusterRootActor> RootActorWeakPtr;

	// Names of viewports to render in the current shot
	TArray<FString> DisplayClusterViewports;
	TArray<FIntPoint> DisplayClusterViewportSizes;

	// Cache view info for viewports
	TMap<int32, FDisplayClusterViewInfo> DCViews;
	TMap<int32, FDisplayClusterViewInfo> DCPrevViews;

	// Runtime flags
	uint8 bFrameWarpBlend : 1 = 0;
};

/**
 * nDisplay viewport render pass (Unlit)
 */
UCLASS(BlueprintType)
class UDisplayClusterMoviePipelineViewportPass_Unlit
	: public UDisplayClusterMoviePipelineViewportPassBase
{
	GENERATED_BODY()

public:
	UDisplayClusterMoviePipelineViewportPass_Unlit() : UDisplayClusterMoviePipelineViewportPassBase(TEXT("nDisplayUnlit"))
	{ }
#if WITH_EDITOR
	virtual FText GetDisplayText() const override { return NSLOCTEXT("MovieRenderPipeline", "DCViewportBasePassSetting_DisplayName_Unlit", "nDisplay Rendering (Unlit)"); }
#endif

	virtual void GetViewShowFlags(FEngineShowFlags& OutShowFlag, EViewModeIndex& OutViewModeIndex) const override
	{
		OutShowFlag = FEngineShowFlags(EShowFlagInitMode::ESFIM_Game);
		OutViewModeIndex = EViewModeIndex::VMI_Unlit;
	}
};

/**
 * nDisplay viewport render pass (DetailLightingOnly)
 */
UCLASS(BlueprintType)
class UDisplayClusterMoviePipelineViewportPass_DetailLighting
	: public UDisplayClusterMoviePipelineViewportPassBase
{
	GENERATED_BODY()

public:
	UDisplayClusterMoviePipelineViewportPass_DetailLighting() : UDisplayClusterMoviePipelineViewportPassBase(TEXT("nDisplayDetailLightingOnly"))
	{ }
#if WITH_EDITOR
	virtual FText GetDisplayText() const override { return NSLOCTEXT("MovieRenderPipeline", "DCViewportBasePassSetting_DisplayName_DetailLighting", "nDisplay Rendering (Detail Lighting)"); }
#endif
	virtual void GetViewShowFlags(FEngineShowFlags& OutShowFlag, EViewModeIndex& OutViewModeIndex) const override
	{
		OutShowFlag = FEngineShowFlags(EShowFlagInitMode::ESFIM_Game);
		OutShowFlag.SetLightingOnlyOverride(true);
		OutViewModeIndex = EViewModeIndex::VMI_Lit_DetailLighting;
	}
	virtual int32 GetOutputFileSortingOrder() const override { return 2; }
};

/**
 * nDisplay viewport render pass (LightingOnly)
 */
UCLASS(BlueprintType)
class UDisplayClusterMoviePipelineViewportPass_LightingOnly
	: public UDisplayClusterMoviePipelineViewportPassBase
{
	GENERATED_BODY()

public:
	UDisplayClusterMoviePipelineViewportPass_LightingOnly() : UDisplayClusterMoviePipelineViewportPassBase(TEXT("nDisplayLightingOnly"))
	{ }
#if WITH_EDITOR
	virtual FText GetDisplayText() const override { return NSLOCTEXT("MovieRenderPipeline", "DCViewportBasePassSetting_DisplayName_LightingOnly", "nDisplay Rendering (Lighting Only)"); }
#endif
	virtual void GetViewShowFlags(FEngineShowFlags& OutShowFlag, EViewModeIndex& OutViewModeIndex) const override
	{
		OutShowFlag = FEngineShowFlags(EShowFlagInitMode::ESFIM_Game);
		OutShowFlag.SetLightingOnlyOverride(true);
		OutViewModeIndex = EViewModeIndex::VMI_LightingOnly;
	}
	virtual int32 GetOutputFileSortingOrder() const override { return 2; }
};

/**
 * nDisplay viewport render pass (ReflectionsOnly)
 */
UCLASS(BlueprintType)
class UDisplayClusterMoviePipelineViewportPass_ReflectionsOnly
	: public UDisplayClusterMoviePipelineViewportPassBase
{
	GENERATED_BODY()

public:
	UDisplayClusterMoviePipelineViewportPass_ReflectionsOnly() : UDisplayClusterMoviePipelineViewportPassBase(TEXT("nDisplayReflectionsOnly"))
	{ }
#if WITH_EDITOR
	virtual FText GetDisplayText() const override { return NSLOCTEXT("MovieRenderPipeline", "DCViewportBasePassSetting_DisplayName_ReflectionsOnly", "nDisplay Rendering (Reflections Only)"); }
#endif
	virtual void GetViewShowFlags(FEngineShowFlags& OutShowFlag, EViewModeIndex& OutViewModeIndex) const override
	{
		OutShowFlag = FEngineShowFlags(EShowFlagInitMode::ESFIM_Game);
		OutShowFlag.SetReflectionOverride(true);
		OutViewModeIndex = EViewModeIndex::VMI_ReflectionOverride;
	}
	virtual int32 GetOutputFileSortingOrder() const override { return 2; }
};

/**
 * nDisplay viewport render pass (PathTracer)
 */
UCLASS(BlueprintType)
class UDisplayClusterMoviePipelineViewportPass_PathTracer : public UDisplayClusterMoviePipelineViewportPassBase
{
	GENERATED_BODY()

public:
	UDisplayClusterMoviePipelineViewportPass_PathTracer() : UDisplayClusterMoviePipelineViewportPassBase(TEXT("nDisplayPathTracer"))
	{ }
#if WITH_EDITOR
	virtual FText GetDisplayText() const override { return NSLOCTEXT("MovieRenderPipeline", "DCViewportBasePassSetting_DisplayName_PathTracer", "nDisplay Path Tracer"); }
	virtual FText GetFooterText(UMoviePipelineExecutorJob* InJob) const override;
#endif
	virtual void GetViewShowFlags(FEngineShowFlags& OutShowFlag, EViewModeIndex& OutViewModeIndex) const override
	{
		OutShowFlag = FEngineShowFlags(EShowFlagInitMode::ESFIM_Game);
		OutShowFlag.SetPathTracing(true);
		OutShowFlag.SetMotionBlur(!bReferenceMotionBlur);
		OutViewModeIndex = EViewModeIndex::VMI_PathTracing;
	}
	virtual int32 GetOutputFileSortingOrder() const override { return 2; }

	virtual bool IsAntiAliasingSupported() const { return false; }
	virtual void ValidateStateImpl() override;
	virtual void SetupImpl(const MoviePipeline::FMoviePipelineRenderPassInitSettings& InPassInitSettings) override;

	/** When enabled, the path tracer will blend all spatial and temporal samples prior to the denoising and will disable post-processed motion blur.
	 *  In this mode it is possible to use higher temporal sample counts to improve the motion blur quality.
	 *  When this option is disabled, the path tracer will accumulate spatial samples, but denoise them prior to accumulation of temporal samples.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Reference Motion Blur")
	bool bReferenceMotionBlur;
};
