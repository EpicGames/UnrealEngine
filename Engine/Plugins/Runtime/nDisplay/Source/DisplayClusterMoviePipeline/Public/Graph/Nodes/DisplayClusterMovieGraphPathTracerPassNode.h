// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "DisplayClusterRootActor.h"
#include "DisplayClusterMoviePipelineEnums.h"
#include "Graph/Nodes/MovieGraphPathTracerPassNode.h"

#include "DisplayClusterMovieGraphPathTracerPassNode.generated.h"

/**
 * Movie Graph PathTracer render pass node for nDisplay.
 */
UCLASS(BlueprintType)
class DISPLAYCLUSTERMOVIEPIPELINE_API UDisplayClusterMovieGraphPathTracerRenderPassNode : public UMovieGraphPathTracerRenderPassNode
{
	GENERATED_BODY()

public:
	UDisplayClusterMovieGraphPathTracerRenderPassNode();
	~UDisplayClusterMovieGraphPathTracerRenderPassNode();

#if WITH_EDITOR
	virtual FText GetNodeTitle(const bool bGetDescriptive = false) const override;

	//~ Begin UObject
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject
#endif

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_OutputMethod : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_ResolutionScale : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_OutputResolution : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_WarpBlendMode : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_RootActorRef : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_AllowedViewportNamesList : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_AllowedNodeNamesList : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_StereoMode : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_OverscanMode : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_EXRLayerGrouping : 1;

public:
	/** Determines how nDisplay renders output: per-viewport, via output mapping layout, or via projection policy. */
	UPROPERTY(BlueprintReadWrite, Category = "nDisplay", meta = (DisplayName = "nDisplay Output Method", EditCondition = "bOverride_OutputMethod"))
	EDisplayClusterMoviePipelineOutputMethod OutputMethod = EDisplayClusterMoviePipelineOutputMethod::PerViewportOutput;

	/** Controls how nDisplay viewports are grouped into multi-layer EXR files. */
	UPROPERTY(BlueprintReadWrite, Category = "nDisplay", meta = (DisplayName = "EXR Layer Grouping", EditCondition = "bOverride_EXRLayerGrouping"))
	EDisplayClusterMoviePipelineEXRLayerGrouping EXRLayerGrouping = EDisplayClusterMoviePipelineEXRLayerGrouping::None;

	/** Uniform scale applied to the viewport resolution before rendering. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "nDisplay", meta = (DisplayName = "Resolution Scale", ClampMin = "0.01", UIMin = "0.1", UIMax = "1.0", EditCondition = "bOverride_ResolutionScale"))
	float ResolutionScale = 1.f;

	/** Overrides the output resolution. */
	UPROPERTY(BlueprintReadWrite, Category = "nDisplay", meta = (DisplayName = "Override Output Resolution", EditCondition = "bOverride_OutputResolution"))
	FIntPoint OutputResolution = FIntPoint::ZeroValue;

	/** Stereo rendering mode for this pass. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "nDisplay", meta = (DisplayName = "Stereo Mode", EditCondition = "bOverride_StereoMode"))
	EDisplayClusterMoviePipelineStereoMode StereoMode = EDisplayClusterMoviePipelineStereoMode::None;

	/** Selects the overscan source: MRP default or nDisplay viewport overscan. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "nDisplay", meta = (DisplayName = "Overscan Mode", EditCondition = "bOverride_OverscanMode"))
	EDisplayClusterMoviePipelineOverscanMode OverscanMode = EDisplayClusterMoviePipelineOverscanMode::Default;

	/** Controls how warp-blend is applied and composited for this pass. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "nDisplay", meta = (DisplayName = "Warp Blend Mode", EditCondition = "bOverride_WarpBlendMode"))
	EDisplayClusterMoviePipelineWarpBlendMode WarpBlendMode = EDisplayClusterMoviePipelineWarpBlendMode::WarpBlend;

	/**
	 * Reference to the DisplayCluster Root Actor to use for this render.
	 * If not set, the first available ADisplayClusterRootActor in the scene is used.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "nDisplay", meta = (DisplayName = "Override nDisplay Actor", EditCondition = "bOverride_RootActorRef"))
	TSoftObjectPtr<ADisplayClusterRootActor> RootActorRef;

	/**  Container with DCRA instance class: for using multiple instances of DCRA of the same class in multiple scenes. */
	UPROPERTY()
	TSoftClassPtr<ADisplayClusterRootActor> RootActorClassRef;

	/** Render only viewports from this list. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "nDisplay", meta = (DisplayName = "Render Selected Viewports Only", EditCondition = "bOverride_AllowedViewportNamesList"))
	TArray<FString> AllowedViewportNamesList;

	/** Render only nodes from this list. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "nDisplay", meta = (DisplayName = "Render Selected Nodes Only", EditCondition = "bOverride_AllowedNodeNamesList"))
	TArray<FString> AllowedNodeNamesList;

protected:
	//~ Begin UMovieGraphRenderPassNode
	virtual void SetupImpl(const FMovieGraphRenderPassSetupData& InSetupData) override;
	virtual void TeardownImpl() override;
	virtual void UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const override;
	virtual void GetFormatResolveArgs(FMovieGraphResolveArgs& OutMergedFormatArgs, const FMovieGraphRenderDataIdentifier& InRenderDataIdentifier) const override;
	virtual TSharedPtr<FMovieGraphRenderCameraSource> CreateRenderCameraSourceImpl(const UMovieGraphPipeline* InMovieGraphPipeline, const UMovieGraphEvaluatedConfig* InEvaluatedConfig) override;
	virtual TSharedPtr<FMovieGraphRenderCameraSource> GetRenderCameraSourceImpl() const override { return RenderCameraSource; }
	//~ End UMovieGraphRenderPassNode

	//~ Begin UMovieGraphImagePassBaseNode
	virtual TUniquePtr<UE::MovieGraph::Rendering::FMovieGraphImagePassBase> CreateInstance() const override;
	//~ End UMovieGraphImagePassBaseNode

private:
	// Non-CDO only; null on the CDO and valid for the lifetime of the active shot pipeline.
	TSharedPtr<FMovieGraphRenderCameraSource> RenderCameraSource;

	// Guards CreateRenderCameraSourceImpl() so it runs exactly once per shot.
	bool bRenderCameraSourceInitialized = false;
};
