// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/Renderers/MovieGraphDeferredPass.h"

namespace UE::MovieGraph::Rendering
{
	/**
	 * nDisplay override of FMovieGraphDeferredPass.
	 */
	struct FDisplayClusterMovieGraphDeferredPass : public FMovieGraphDeferredPass
	{
		//~ Begin FMovieGraphImagePassBase
		virtual UMovieGraphImagePassBaseNode* GetParentNode(UMovieGraphEvaluatedConfig* InConfig) const override;

		virtual void ModifyProjectionMatrixForTiling(
			const UE::MovieGraph::DefaultRenderer::FMovieGraphTilingParams& InTilingParams,
			const bool bInOrthographic, FMatrix& InOutProjectionMatrix, float& OutDoFSensorScale) const override;

		virtual void PostRendererSubmission(
			const UE::MovieGraph::FMovieGraphSampleState& InSampleState,
			const UE::MovieGraph::DefaultRenderer::FRenderTargetInitParams& InRenderTargetInitParams,
			FCanvas& InCanvas,
			const UE::MovieGraph::DefaultRenderer::FCameraInfo& InCameraInfo) const override;

		virtual void ApplyMovieGraphOverridesToSampleState(FMovieGraphSampleState& SampleState) const override;
		//~ End FMovieGraphImagePassBase
};
}