// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class DisplayClusterMoviePipelineEditor : ModuleRules
{
	public DisplayClusterMoviePipelineEditor(ReadOnlyTargetRules ROTargetRules) : base(ROTargetRules)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"DisplayCluster",
				"DisplayClusterMoviePipeline",
				"DisplayClusterConfiguration",

				"ApplicationCore",
				"AppFramework",
				"AssetTools",
				"Core",
				"CoreUObject",
				"DesktopPlatform",
				"BlueprintGraph",
				"GraphEditor",

				"EditorFramework",
				"EditorSubsystem",
				"EditorWidgets",

				"Engine",
				"UnrealEd",

				"ImageWrapper",
				"InputCore",
				"Kismet",
				"KismetCompiler",
				"MainFrame",
				"MediaAssets",
				"MessageLog",
				"PinnedCommandList",
				"Projects",
				"PropertyEditor",
				"Serialization",
				"Settings",
				"Slate",
				"SlateCore",
				"ToolMenus",
				"SubobjectEditor",
				"SubobjectDataInterface",
				"ToolWidgets",
			});
	}
}
