// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterMoviePipelineEditorBaseTypeCustomization.h"
#include "DisplayClusterRootActor.h"

#include "PropertyHandle.h"
#include "DetailWidgetRow.h"
#include "DetailLayoutBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "IPropertyUtilities.h"

#define LOCTEXT_NAMESPACE "FDisplayClusterMoviePipelineEditorBaseTypeCustomization"

//////////////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterMoviePipelineEditorBaseTypeCustomization
//////////////////////////////////////////////////////////////////////////////////////////////

const FName FDisplayClusterMoviePipelineEditorBaseTypeCustomization::NoHeaderMetadataKey = TEXT("NoHeader");
const FName FDisplayClusterMoviePipelineEditorBaseTypeCustomization::HideChildrenMetadataKey = TEXT("HideChildren");
const FName FDisplayClusterMoviePipelineEditorBaseTypeCustomization::SubstitutionsMetadataKey = TEXT("Substitutions");
const FName FDisplayClusterMoviePipelineEditorBaseTypeCustomization::DefaultSubstitutionsMetadataKey = TEXT("DefaultSubstitutions");

void FDisplayClusterMoviePipelineEditorBaseTypeCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	check(InPropertyHandle->IsValidHandle());

	PropertyUtilities = CustomizationUtils.GetPropertyUtilities();
	if (PropertyUtilities == nullptr)
	{
		// Can be null if PropertyEditorModule.CreateSingleProperty is what triggers this customization.
		return;
	}

	Initialize(InPropertyHandle, CustomizationUtils);

	if (ShouldShowHeader(InPropertyHandle))
	{
		SetHeader(InPropertyHandle, InHeaderRow, CustomizationUtils);
	}
}

void FDisplayClusterMoviePipelineEditorBaseTypeCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	if (ShouldShowChildren(InPropertyHandle))
	{
		SetChildren(InPropertyHandle, InChildBuilder, CustomizationUtils);
	}
}

TSharedPtr<IPropertyHandle> FDisplayClusterMoviePipelineEditorBaseTypeCustomization::GetChildHandleChecked(const TSharedRef<IPropertyHandle>& InPropertyHandle, const FName& ChildPropertyName) const
{
	TSharedPtr<IPropertyHandle> ChildHandle = InPropertyHandle->GetChildHandle(ChildPropertyName);
	check(ChildHandle.IsValid());
	check(ChildHandle->IsValidHandle());

	return ChildHandle;
}

void FDisplayClusterMoviePipelineEditorBaseTypeCustomization::Initialize(const TSharedRef<IPropertyHandle>& InPropertyHandle, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	FillSubstitutionMap(InPropertyHandle);
}

void FDisplayClusterMoviePipelineEditorBaseTypeCustomization::SetHeader(const TSharedRef<IPropertyHandle>& InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	InHeaderRow.NameContent()
	[
		InPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		InPropertyHandle->CreatePropertyValueWidget()
	];
}

void FDisplayClusterMoviePipelineEditorBaseTypeCustomization::SetChildren(const TSharedRef<IPropertyHandle>& InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	AddAllChildren(InPropertyHandle, InChildBuilder);
}

void FDisplayClusterMoviePipelineEditorBaseTypeCustomization::AddAllChildren(const TSharedRef<IPropertyHandle>& InPropertyHandle, IDetailChildrenBuilder& InChildBuilder)
{
	uint32 NumChildren = 0;
	InPropertyHandle->GetNumChildren(NumChildren);
	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		TSharedPtr<IPropertyHandle> ChildHandle = InPropertyHandle->GetChildHandle(ChildIndex);
		if (ChildHandle && ChildHandle->IsValidHandle() && !ChildHandle->IsCustomized())
		{
			FText ChildTooltip = ApplySubstitutions(ChildHandle->GetToolTipText());
			ChildHandle->SetToolTipText(ChildTooltip);

			InChildBuilder.AddProperty(ChildHandle.ToSharedRef());
		}
	}
}

void FDisplayClusterMoviePipelineEditorBaseTypeCustomization::FillSubstitutionMap(const TSharedRef<IPropertyHandle>& InPropertyHandle)
{
	if (FStructProperty* StructProperty = CastField<FStructProperty>(InPropertyHandle->GetProperty()))
	{
		if (StructProperty->Struct->HasMetaData(DefaultSubstitutionsMetadataKey))
		{
			const FString DefaultSubstitutions = StructProperty->Struct->GetMetaDataText(DefaultSubstitutionsMetadataKey).ToString();
			ParseSubstitutions(SubstitutionsMap, DefaultSubstitutions);
		}
	}

	if (InPropertyHandle->HasMetaData(SubstitutionsMetadataKey))
	{
		const FString Substitutions = InPropertyHandle->GetProperty()->GetMetaDataText(SubstitutionsMetadataKey).ToString();
		ParseSubstitutions(SubstitutionsMap, Substitutions);
	}
}

void FDisplayClusterMoviePipelineEditorBaseTypeCustomization::ParseSubstitutions(FFormatNamedArguments& OutSubstitutionsMap, const FString& InSubstitutionsStr) const
{
	TArray<FString> Substitutions;
	InSubstitutionsStr.ParseIntoArray(Substitutions, TEXT(","));

	for (const FString& Substitution : Substitutions)
	{
		if (Substitution.Contains(TEXT("=")))
		{
			FString Key;
			FString Value;
			Substitution.Split(TEXT("="), &Key, &Value);
			if (!OutSubstitutionsMap.Contains(Key.TrimStartAndEnd()))
			{
				OutSubstitutionsMap.Add(Key.TrimStartAndEnd(), FText::FromString(Value.TrimStartAndEnd()));
			}
			else
			{
				OutSubstitutionsMap[Key.TrimStartAndEnd()] = FText::FromString(Value.TrimStartAndEnd());
			}
		}
	}
}

FText FDisplayClusterMoviePipelineEditorBaseTypeCustomization::ApplySubstitutions(const FText& InText) const
{
	return FText::Format(InText, SubstitutionsMap);
}

bool FDisplayClusterMoviePipelineEditorBaseTypeCustomization::ShouldShowHeader(const TSharedRef<IPropertyHandle>& InPropertyHandle) const
{
	if (InPropertyHandle->IsValidHandle())
	{
		if (InPropertyHandle->HasMetaData(TEXT("ShowOnlyInnerProperties")) || InPropertyHandle->HasMetaData(NoHeaderMetadataKey))
		{
			return false;
		}

		if (FStructProperty* StructProperty = CastField<FStructProperty>(InPropertyHandle->GetProperty()))
		{
			if (StructProperty->Struct->HasMetaData(NoHeaderMetadataKey))
			{
				return false;
			}
		}

		return true;
	}

	return false;
}

bool FDisplayClusterMoviePipelineEditorBaseTypeCustomization::ShouldShowChildren(const TSharedRef<IPropertyHandle>& InPropertyHandle) const
{
	if (InPropertyHandle->IsValidHandle())
	{
		if (InPropertyHandle->HasMetaData(HideChildrenMetadataKey))
		{
			return false;
		}

		if (FStructProperty* StructProperty = CastField<FStructProperty>(InPropertyHandle->GetProperty()))
		{
			if (StructProperty->Struct->HasMetaData(HideChildrenMetadataKey))
			{
				return false;
			}
		}

		return true;
	}

	return false;
}

const FString* FDisplayClusterMoviePipelineEditorBaseTypeCustomization::FindMetaData(TSharedPtr<IPropertyHandle> PropertyHandle, const FName& Key) const
{
	if (const FString* MetadataValue = PropertyHandle->GetInstanceMetaData(Key))
	{
		return MetadataValue;
	}
	else if (PropertyHandle->HasMetaData(Key))
	{
		return &PropertyHandle->GetMetaData(Key);
	}

	return nullptr;
}
#undef LOCTEXT_NAMESPACE
