// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterMoviePipelineEditorNodeSelection.h"

#include "DisplayClusterRootActor.h"
#include "DisplayClusterConfigurationTypes.h"
#include "Widgets/SDisplayClusterMoviePipelineEditorSearchableComboBox.h"

#include "DisplayClusterMoviePipelineSettings.h"

#include "DetailLayoutBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyHandle.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "FDisplayClusterMoviePipelineEditorNodeSelection"

//////////////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterMoviePipelineEditorNodeSelection
//////////////////////////////////////////////////////////////////////////////////////////////
const FName FDisplayClusterMoviePipelineEditorNodeSelection::NAME_ElementToolTip = TEXT("ElementToolTip");

FDisplayClusterMoviePipelineEditorNodeSelection::FDisplayClusterMoviePipelineEditorNodeSelection(EOperationMode InMode, const TSharedPtr<IPropertyHandle>& InDCRAPropertyHandle, const TSharedPtr<IPropertyHandle>& InSelectedOptionsHandle)
{
	DCRAPropertyHandle = InDCRAPropertyHandle;
	SelectedOptionsHandle = InSelectedOptionsHandle;
	check(DCRAPropertyHandle.IsValid());

	OperationMode = InMode;

	ResetOptions();
}

UDisplayClusterConfigurationData* FDisplayClusterMoviePipelineEditorNodeSelection::GetConfigData() const
{
	if (DCRAPropertyHandle.IsValid())
	{
		UObject* CurrentValue = nullptr;
		if (FPropertyAccess::Success == DCRAPropertyHandle->GetValue(CurrentValue))
		{
			if (ADisplayClusterRootActor* RootActorPtr = Cast<ADisplayClusterRootActor>(CurrentValue))
			{
				return RootActorPtr->GetConfigData();
			}
		}
	}

	return nullptr;
}

void FDisplayClusterMoviePipelineEditorNodeSelection::ResetOptionsList()
{
	ResetOptions();

	if (OptionsComboBox.IsValid())
	{
		OptionsComboBox->ResetOptionsSource(&Options);
		OptionsComboBox->SetIsOpen(false);
	}
}

void FDisplayClusterMoviePipelineEditorNodeSelection::CreateArrayBuilder(const TSharedRef<IPropertyHandle>& InPropertyHandle,
	IDetailChildrenBuilder& InChildBuilder)
{
	TSharedRef<FDetailArrayBuilder> ArrayBuilder = MakeShareable(new FDetailArrayBuilder(InPropertyHandle));
	ArrayBuilder->OnGenerateArrayElementWidget(FOnGenerateArrayElementWidget::CreateRaw(this,
		&FDisplayClusterMoviePipelineEditorNodeSelection::GenerateSelectionWidget));

	InChildBuilder.AddCustomBuilder(ArrayBuilder);
}

FDisplayClusterMoviePipelineEditorNodeSelection::EOperationMode FDisplayClusterMoviePipelineEditorNodeSelection::GetOperationModeFromProperty(FProperty* Property)
{
	EOperationMode ReturnMode = ClusterNodes;
	if(Property)
	{
		if (const FString* DefinedMode = Property->FindMetaData(TEXT("ConfigurationMode")))
		{
			FString ModeLower = (*DefinedMode).ToLower();
			ModeLower.RemoveSpacesInline();
			if (ModeLower == TEXT("viewports"))
			{
				ReturnMode = FDisplayClusterMoviePipelineEditorNodeSelection::EOperationMode::Viewports;
			}
			else if (ModeLower == TEXT("clusternodes"))
			{
				ReturnMode = FDisplayClusterMoviePipelineEditorNodeSelection::EOperationMode::ClusterNodes;
			}
			// Define any other modes here.
		}
	}

	return ReturnMode;
}

void FDisplayClusterMoviePipelineEditorNodeSelection::GenerateSelectionWidget(
	TSharedRef<IPropertyHandle> PropertyHandle, int32 ArrayIndex, IDetailChildrenBuilder& ChildrenBuilder)
{
	FText ElementTooltip = FText::GetEmpty();

	TSharedPtr<IPropertyHandle> ParentArrayHandle = PropertyHandle->GetParentHandle();
	if (ParentArrayHandle.IsValid() && ParentArrayHandle->IsValidHandle())
	{
		if (const FString* MetaData = ParentArrayHandle->GetInstanceMetaData(NAME_ElementToolTip))
		{
			ElementTooltip = FText::FromString(*MetaData);
		}
		else if (ParentArrayHandle->HasMetaData(NAME_ElementToolTip))
		{
			ElementTooltip = FText::FromString(ParentArrayHandle->GetMetaData(NAME_ElementToolTip));
		}
		else
		{
			ElementTooltip = ParentArrayHandle->GetPropertyDisplayName();
		}
	}

	IDetailPropertyRow& PropertyRow = ChildrenBuilder.AddProperty(PropertyHandle);
	PropertyRow.CustomWidget(false)
		.NameContent()
		[
			PropertyHandle->CreatePropertyNameWidget(FText::GetEmpty(), ElementTooltip)
		]
		.IsEnabled(IsEnabledAttr)
		.ValueContent()
		[
			SAssignNew(OptionsComboBox, SDisplayClusterMoviePipelineEditorSearchableComboBox)
				.OptionsSource(&Options)
				.OnGenerateWidget(this, &FDisplayClusterMoviePipelineEditorNodeSelection::MakeOptionComboWidget)
				.OnSelectionChanged(this, &FDisplayClusterMoviePipelineEditorNodeSelection::OnOptionSelected, PropertyHandle)
				.ContentPadding(2)
				.MaxListHeight(200.0f)
				.IsEnabled(IsEnabledAttr)
				.Content()
				[
					SNew(STextBlock)
					.Text(this, &FDisplayClusterMoviePipelineEditorNodeSelection::GetSelectedOptionText, PropertyHandle)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
		];
}

void FDisplayClusterMoviePipelineEditorNodeSelection::ResetOptions()
{
	if (!bResetOptionsListFunctionInPerformed)
	{
		// recursion not allowed through IPropertyHandle callbacks
		bResetOptionsListFunctionInPerformed = true;

		TArray<FString> UsedValues;
		TArray<FString> ValuesFromCurrentConfig;

		if (SelectedOptionsHandle.IsValid())
		{
			const TSharedPtr<IPropertyHandleArray> ArrayOptionsHandle = SelectedOptionsHandle->AsArray();
			if (ArrayOptionsHandle.IsValid())
			{
				// Get values from DCRA
				if (UDisplayClusterConfigurationData* ConfigData = GetConfigData())
				{
					for (const TTuple<FString, TObjectPtr<UDisplayClusterConfigurationClusterNode>>& Node : ConfigData->Cluster->Nodes)
					{
						if (OperationMode == ClusterNodes)
						{
							ValuesFromCurrentConfig.Add(Node.Value->GetName());
							continue;
						}
						for (const TTuple<FString, TObjectPtr<UDisplayClusterConfigurationViewport>>& Viewport : Node.Value->Viewports)
						{
							ValuesFromCurrentConfig.Add(Viewport.Value->GetName());
						}
					}
				}
				else
				{
					uint32 NumElements = 0;
					if (FPropertyAccess::Result::Success == ArrayOptionsHandle->GetNumElements(NumElements))
					{
						if (NumElements > 0)
						{
							ArrayOptionsHandle->EmptyArray();
						}
					}
				}

				// Get used values and remove invalid values:
				uint32 NumElements = 0;
				if (FPropertyAccess::Result::Success == ArrayOptionsHandle->GetNumElements(NumElements))
				{
					for (int32 ElementIndex = 0; ElementIndex < (int32)NumElements; ElementIndex++)
					{
						const TSharedRef<IPropertyHandle> ElementHandle = ArrayOptionsHandle->GetElement(ElementIndex);
						FString StrValue;
						if (FPropertyAccess::Result::Success == ElementHandle->GetValue(StrValue))
						{
							if (!StrValue.IsEmpty())
							{
								if (ValuesFromCurrentConfig.Find(StrValue) == INDEX_NONE)
								{
									// Remove invalid values
									ArrayOptionsHandle->DeleteItem(ElementIndex);
									NumElements--;
								}
								else
								{
									UsedValues.AddUnique(StrValue);
								}
							}
						}
					}
				}
			}
		}

		Options.Reset();

		// Show in options only new values
		for (const FString& ValueIt : ValuesFromCurrentConfig)
		{
			if (UsedValues.Find(ValueIt) == INDEX_NONE)
			{
				Options.Add(MakeShared<FString>(ValueIt));
			}
		}

		bResetOptionsListFunctionInPerformed = false;
	}
}

TSharedRef<SWidget> FDisplayClusterMoviePipelineEditorNodeSelection::MakeOptionComboWidget(
	TSharedPtr<FString> InItem)
{
	return SNew(STextBlock)
		.Text(FText::FromString(*InItem))
		.Font(IDetailLayoutBuilder::GetDetailFont());
}

void FDisplayClusterMoviePipelineEditorNodeSelection::OnOptionSelected(TSharedPtr<FString> InValue,
	ESelectInfo::Type SelectInfo, TSharedRef<IPropertyHandle> InPropertyHandle)
{
	if (InValue.IsValid())
	{
		check(OptionsComboBox.IsValid());

		InPropertyHandle->SetValue(*InValue);
		
		ResetOptions();
		OptionsComboBox->ResetOptionsSource(&Options);
		OptionsComboBox->SetIsOpen(false);
	}
}

FText FDisplayClusterMoviePipelineEditorNodeSelection::GetSelectedOptionText(TSharedRef<IPropertyHandle> InPropertyHandle) const
{
	FString Value;
	InPropertyHandle->GetValue(Value);
	return FText::FromString(Value);
}
#undef LOCTEXT_NAMESPACE
