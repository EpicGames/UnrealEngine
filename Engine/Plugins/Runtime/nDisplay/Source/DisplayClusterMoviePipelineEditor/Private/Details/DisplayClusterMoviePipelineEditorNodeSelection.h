// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DisplayClusterMoviePipelineEditorBaseTypeCustomization.h"
#include "Misc/Attribute.h"

class UDisplayClusterConfigurationData;
class SDisplayClusterMoviePipelineEditorSearchableComboBox;
class SWidget;

/**
 * Helper class used to construct and manage a custom array property widget that displays for each element a dropdown containing
 * a list of the display cluster's cluster nodes or viewports to select from.
 */
class FDisplayClusterMoviePipelineEditorNodeSelection : public TSharedFromThis<FDisplayClusterMoviePipelineEditorNodeSelection>
{
public:
	/** Custom "ElementToolTip" metadata specifier that can be used to specify the tooltip to apply to each element of the displayed array */
	static const FName NAME_ElementToolTip;

	/** Flag to indicate which cluster item is displayed in the dropdown menus of the elements of the array widget  */
	enum EOperationMode
	{
		Viewports,
		ClusterNodes
	};

	FDisplayClusterMoviePipelineEditorNodeSelection(EOperationMode InMode, const TSharedPtr<IPropertyHandle>& InDCRAPropertyHandle, const TSharedPtr<IPropertyHandle>& InSelectedOptionsHandle);

	~FDisplayClusterMoviePipelineEditorNodeSelection()
	{
		OptionsComboBox.Reset();
		Options.Reset();
	}

	/** Gets the cluster configuration data object of the root actor being edited by the current details panel */
	UDisplayClusterConfigurationData* GetConfigData() const;

	/** Creates the child array builder that will be used to construct the cluster item selection array elements for the details panel */
	void CreateArrayBuilder(const TSharedRef<IPropertyHandle>& InPropertyHandle, IDetailChildrenBuilder& InChildBuilder);

	/** Sets the IsEnabled attribute to use on the array property handle */
	FDisplayClusterMoviePipelineEditorNodeSelection& IsEnabled(const TAttribute<bool>& InIsEnabled)
	{
		IsEnabledAttr = InIsEnabled;
		return *this;
	}

	/** Inspects the specified property's metadata to determine if the node selection array should show cluster nodes or viewports in its dropdown lists */
	static EOperationMode GetOperationModeFromProperty(FProperty* Property);

	void ResetOptionsList();

protected:
	/** Generates a widget to display the currently selected cluster item for the specified element in the array widget and adds it to the details panel */
	void GenerateSelectionWidget(TSharedRef<IPropertyHandle> PropertyHandle, int32 ArrayIndex, IDetailChildrenBuilder& ChildrenBuilder);

	/** Resets the list of cluster items to display in the dropdown lists for each element of the array widget */
	void ResetOptions();

	/** Constructs the text widget to display in the dropdown menu for the specified item */
	TSharedRef<SWidget> MakeOptionComboWidget(TSharedPtr<FString> InItem);

	/** Raised when an item within the dropdown menu is selected */
	void OnOptionSelected(TSharedPtr<FString> InValue, ESelectInfo::Type SelectInfo, TSharedRef<IPropertyHandle> InPropertyHandle);

	/** Gets the display text for the currently selected item of the element in the array widget */
	FText GetSelectedOptionText(TSharedRef<IPropertyHandle> InPropertyHandle) const;

private:
	/** A custom attribute to apply to the array widget to determine if it is enabled or not */
	TAttribute<bool> IsEnabledAttr;

	/** The combo box widget to display for each element in the array widget */
	TSharedPtr<SDisplayClusterMoviePipelineEditorSearchableComboBox> OptionsComboBox;

	/** The list of options to display in each combo box's dropdown menu */
	TArray<TSharedPtr<FString>> Options;

	/** A reference to the current display cluster root actor property handle that is being edited by the details panel */
	TSharedPtr<IPropertyHandle> DCRAPropertyHandle;

	/** A reference to the string list property handle that is being edited by the details panel */
	TSharedPtr<IPropertyHandle> SelectedOptionsHandle;

	/** The operation mode which indicates whether to display cluster nodes or viewports in each element's combo box */
	EOperationMode OperationMode = ClusterNodes;

	/** Protect frunction ResetOptionsList() from recursion */
	bool bResetOptionsListFunctionInPerformed= false;
};
