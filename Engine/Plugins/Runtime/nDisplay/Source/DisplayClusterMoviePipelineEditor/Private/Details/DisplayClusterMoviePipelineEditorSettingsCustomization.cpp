// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterMoviePipelineEditorSettingsCustomization.h"

#include "DisplayClusterRootActor.h"
#include "DisplayClusterConfigurationTypes.h"
#include "Widgets/SDisplayClusterMoviePipelineEditorSearchableComboBox.h"

#include "DisplayClusterMoviePipelineSettings.h"

#include "DetailLayoutBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyHandle.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "DisplayClusterMoviePipelineEditorSettingsCustomization"

//////////////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterMoviePipelineEditorSettingsCustomization
//////////////////////////////////////////////////////////////////////////////////////////////
void FDisplayClusterMoviePipelineEditorSettingsCustomization::Initialize(const TSharedRef<IPropertyHandle>& InPropertyHandle, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	FDisplayClusterMoviePipelineEditorBaseTypeCustomization::Initialize(InPropertyHandle, CustomizationUtils);

	TSharedPtr<IPropertyHandle> DCRootActorHandle = GET_CHILD_HANDLE(FDisplayClusterMoviePipelineConfiguration, DCRootActor);
	check(DCRootActorHandle.IsValid());

	DCRootActorHandle->SetOnPropertyValueChanged(
		FSimpleDelegate::CreateSP(
			this, &FDisplayClusterMoviePipelineEditorSettingsCustomization::OnRootActorReferencedPropertyValueChanged, DCRootActorHandle));

	TSharedPtr<IPropertyHandle> AllowedViewportNamesListHandle = GET_CHILD_HANDLE(FDisplayClusterMoviePipelineConfiguration, AllowedViewportNamesList);
	check(AllowedViewportNamesListHandle);
	AllowedViewportNamesListHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FDisplayClusterMoviePipelineEditorSettingsCustomization::OnAllowedViewportNamesListHandleChanged));

	NodeSelection = MakeShared<FDisplayClusterMoviePipelineEditorNodeSelection>(FDisplayClusterMoviePipelineEditorNodeSelection::EOperationMode::Viewports, DCRootActorHandle, AllowedViewportNamesListHandle);
}

void FDisplayClusterMoviePipelineEditorSettingsCustomization::OnAllowedViewportNamesListHandleChanged()
{
	if (NodeSelection.IsValid())
	{
		NodeSelection->ResetOptionsList();
	}
}
void FDisplayClusterMoviePipelineEditorSettingsCustomization::OnRootActorReferencedPropertyValueChanged(TSharedPtr<IPropertyHandle> PropertyHandle)
{
	UObject* CurrentValue = nullptr;
	if (PropertyHandle.IsValid()
	&& FPropertyAccess::Success == PropertyHandle->GetValue(CurrentValue)
	&& !CurrentValue)
	{
		/**
		* Do not reset the viewport list when the DCRA reference is cleared.
		*
		* Behavior:
		* - An empty DCRA reference means: use any available DCRA in the scene.
		* - Allows configuring specific viewports to render even when the DCRA is undefined.
		*
		* Example:
		* - A DCRA class defines 10 viewports.
		* - Rendering is distributed across 10 computers.
		* - The scene may change during rendering.
		* - Multiple DCRA instances (of the same class) can be used interchangeably.
		*
		* Expectation:
		* - The user creates 10 different configurations (one per viewport).
		* - The DCRA reference is reset to detach from the scene, so each configuration
		*   can pick up any available DCRA instance dynamically.
		*/

		return;
	}

	// When a referenced property is changed, we have to trigger layout refresh
	if (NodeSelection.IsValid())
	{
		NodeSelection->ResetOptionsList();
	}
}

void FDisplayClusterMoviePipelineEditorSettingsCustomization::SetHeader(const TSharedRef<IPropertyHandle>& InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	FText ElementTooltip = FText::GetEmpty();

	TSharedPtr<IPropertyHandle> ParentArrayHandle = InPropertyHandle->GetParentHandle();
	if (ParentArrayHandle.IsValid() && ParentArrayHandle->IsValidHandle())
	{
		ElementTooltip = ParentArrayHandle->GetPropertyDisplayName();
	}

	InHeaderRow.NameContent()
		[
			InPropertyHandle->CreatePropertyNameWidget(FText::GetEmpty(), ElementTooltip)
		];
}

void FDisplayClusterMoviePipelineEditorSettingsCustomization::SetChildren(const TSharedRef<IPropertyHandle>& InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	TSharedPtr<IPropertyHandle> DCRootActorHandle = GET_CHILD_HANDLE(FDisplayClusterMoviePipelineConfiguration, DCRootActor);
	TSharedPtr<IPropertyHandle> UseViewportResolutionsHandle = GET_CHILD_HANDLE(FDisplayClusterMoviePipelineConfiguration, bUseViewportResolutions);
	TSharedPtr<IPropertyHandle> IsRenderAllViewportsHandle = GET_CHILD_HANDLE(FDisplayClusterMoviePipelineConfiguration, bRenderAllViewports);
	TSharedPtr<IPropertyHandle> ArrayHandle = GET_CHILD_HANDLE(FDisplayClusterMoviePipelineConfiguration, AllowedViewportNamesList);

	InChildBuilder.AddProperty(DCRootActorHandle.ToSharedRef());
	InChildBuilder.AddProperty(UseViewportResolutionsHandle.ToSharedRef());
	InChildBuilder.AddProperty(IsRenderAllViewportsHandle.ToSharedRef());

	if (NodeSelection.IsValid())
	{
		const TAttribute<bool> IsEnabledEditCondition = TAttribute<bool>::Create([this, IsRenderAllViewportsHandle, DCRootActorHandle]()
			{
				bool bIsRenderAllViewportsHandleValue = false;
				const bool bRenderAllViewports = 
					IsRenderAllViewportsHandle.IsValid()
					&& FPropertyAccess::Success == IsRenderAllViewportsHandle->GetValue(bIsRenderAllViewportsHandleValue)
					&& bIsRenderAllViewportsHandleValue;
				
				UObject* DCRootActorHandleValue = nullptr;
				const bool bHasRootActor =
					DCRootActorHandle.IsValid()
					&& FPropertyAccess::Success == DCRootActorHandle->GetValue(DCRootActorHandleValue)
					&& DCRootActorHandleValue;

				return !bRenderAllViewports && bHasRootActor;
			});

		ArrayHandle->SetInstanceMetaData(FDisplayClusterMoviePipelineEditorNodeSelection::NAME_ElementToolTip, ArrayHandle->GetPropertyDisplayName().ToString());

		NodeSelection->IsEnabled(IsEnabledEditCondition);
		NodeSelection->CreateArrayBuilder(ArrayHandle.ToSharedRef(), InChildBuilder);
	}
}
#undef LOCTEXT_NAMESPACE
