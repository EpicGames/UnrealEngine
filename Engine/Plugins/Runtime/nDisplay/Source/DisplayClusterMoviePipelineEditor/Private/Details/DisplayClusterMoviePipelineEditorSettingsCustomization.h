// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DisplayClusterMoviePipelineEditorNodeSelection.h"

/** Type customization for the FDisplayClusterMoviePipelineConfiguration type, to customize the AllowedViewportNamesList list property */
class FDisplayClusterMoviePipelineEditorSettingsCustomization final
	: public FDisplayClusterMoviePipelineEditorBaseTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FDisplayClusterMoviePipelineEditorSettingsCustomization>();
	}

	virtual ~FDisplayClusterMoviePipelineEditorSettingsCustomization() override
	{
		NodeSelection.Reset();
	}

protected:
	//~ FDisplayClusterConfiguratorBaseTypeCustomization overrides begin
	virtual void Initialize(const TSharedRef<IPropertyHandle>& InPropertyHandle, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void SetHeader(const TSharedRef<IPropertyHandle>& InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void SetChildren(const TSharedRef<IPropertyHandle>& InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	//~ FDisplayClusterConfiguratorBaseTypeCustomization overrides end

	void OnRootActorReferencedPropertyValueChanged(TSharedPtr<IPropertyHandle> PropertyHandle);
	void OnAllowedViewportNamesListHandleChanged();
private:
	/** The helper object used to generate the custom array widget to display in place of a string list for the AllowedViewportNamesList property */
	TSharedPtr<FDisplayClusterMoviePipelineEditorNodeSelection> NodeSelection;
};
