// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

// Plugin-wide log categories
DECLARE_LOG_CATEGORY_EXTERN(LogDisplayClusterMoviePipelineEditor, Log, All);
