// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterMoviePipelineEditorModule.h"
#include "Details/DisplayClusterMoviePipelineEditorSettingsCustomization.h"

#include "DisplayClusterMoviePipelineSettings.h"

#include "AssetToolsModule.h"
#include "IAssetTools.h"
#include "ISettingsModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetTypeCategories.h"
#include "HAL/IConsoleManager.h"
#include "Modules/ModuleManager.h"
#include "ActorFactories/ActorFactoryBlueprint.h"
#include "PropertyEditorModule.h"

#define REGISTER_PROPERTY_LAYOUT(PropertyType, CustomizationType) { \
	const FName LayoutName = PropertyType::StaticStruct()->GetFName(); \
	RegisteredPropertyLayoutNames.Add(LayoutName); \
	PropertyModule.RegisterCustomPropertyTypeLayout(LayoutName, \
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&CustomizationType::MakeInstance)); \
}

#define REGISTER_OBJECT_LAYOUT(ObjectType, CustomizationType) { \
	const FName LayoutName = ObjectType::StaticClass()->GetFName(); \
	RegisteredClassLayoutNames.Add(LayoutName); \
	PropertyModule.RegisterCustomClassLayout(LayoutName, \
		FOnGetDetailCustomizationInstance::CreateStatic(&CustomizationType::MakeInstance)); \
}

#define LOCTEXT_NAMESPACE "DisplayClusterMoviePipelineEditor"

//////////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterMoviePipelineEditorModule
//////////////////////////////////////////////////////////////////////////////////////////
void FDisplayClusterMoviePipelineEditorModule::StartupModule()
{
	RegisterCustomLayouts();
}

void FDisplayClusterMoviePipelineEditorModule::ShutdownModule()
{
	UnregisterCustomLayouts();
}

void FDisplayClusterMoviePipelineEditorModule::RegisterCustomLayouts()
{
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	REGISTER_PROPERTY_LAYOUT(FDisplayClusterMoviePipelineConfiguration, FDisplayClusterMoviePipelineEditorSettingsCustomization);
}

void FDisplayClusterMoviePipelineEditorModule::UnregisterCustomLayouts()
{
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	for (const FName& LayoutName : RegisteredClassLayoutNames)
	{
		PropertyModule.UnregisterCustomClassLayout(LayoutName);
	}

	for (const FName& LayoutName : RegisteredPropertyLayoutNames)
	{
		PropertyModule.UnregisterCustomPropertyTypeLayout(LayoutName);
	}

	RegisteredClassLayoutNames.Empty();
	RegisteredPropertyLayoutNames.Empty();
}

IMPLEMENT_MODULE(FDisplayClusterMoviePipelineEditorModule, DisplayClusterMoviePipelineEditor);

#undef LOCTEXT_NAMESPACE
