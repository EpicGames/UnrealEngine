// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDisplayClusterMoviePipelineEditor.h"

/**
 * Display Cluster Movie Pipeline Editor module
 */
class FDisplayClusterMoviePipelineEditorModule :
	public IDisplayClusterMoviePipelineEditor
{
public:
	//~ IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	void RegisterCustomLayouts();
	void UnregisterCustomLayouts();

private:
	TArray<FName> RegisteredClassLayoutNames;
	TArray<FName> RegisteredPropertyLayoutNames;
};
