// Copyright Epic Games, Inc. All Rights Reservekd.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/Input/SSearchableComboBox.h"

/**
 * A searchable text combo box
 * Mostly copied from SearchableComboBox, but with extension for dynamically update the Option source list
 */
class SDisplayClusterMoviePipelineEditorSearchableComboBox
	: public SSearchableComboBox
{
public:

	SLATE_BEGIN_ARGS(SDisplayClusterMoviePipelineEditorSearchableComboBox)
		: _Content()
		, _ComboBoxStyle(&FAppStyle::Get().GetWidgetStyle<FComboBoxStyle>("ComboBox"))
		, _ButtonStyle(nullptr)
		, _ItemStyle(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("ComboBox.Row"))
		, _ContentPadding(_ComboBoxStyle->ContentPadding)
		, _ForegroundColor(FSlateColor::UseStyle())
		, _OptionsSource()
		, _OnSelectionChanged()
		, _OnGenerateWidget()
		, _InitiallySelectedItem(nullptr)
		, _Method()
		, _MaxListHeight(450.0f)
		, _HasDownArrow(true)
		, _SearchVisibility()
	{}

	/** Slot for this button's content (optional) */
	SLATE_DEFAULT_SLOT(FArguments, Content)

		SLATE_STYLE_ARGUMENT(FComboBoxStyle, ComboBoxStyle)

		/** The visual style of the button part of the combo box (overrides ComboBoxStyle) */
		SLATE_STYLE_ARGUMENT(FButtonStyle, ButtonStyle)

		SLATE_STYLE_ARGUMENT(FTableRowStyle, ItemStyle)

		SLATE_ATTRIBUTE(FMargin, ContentPadding)
		SLATE_ATTRIBUTE(FSlateColor, ForegroundColor)

		SLATE_ARGUMENT(const TArray< TSharedPtr<FString> >*, OptionsSource)
		SLATE_EVENT(FOnSelectionChanged, OnSelectionChanged)
		SLATE_EVENT(FOnGenerateWidget, OnGenerateWidget)

		/** Called when combo box is opened, before list is actually created */
		SLATE_EVENT(FOnComboBoxOpening, OnComboBoxOpening)

		/** The custom scrollbar to use in the ListView */
		SLATE_ARGUMENT(TSharedPtr<SScrollBar>, CustomScrollbar)

		/** The option that should be selected when the combo box is first created */
		SLATE_ARGUMENT(TSharedPtr<FString>, InitiallySelectedItem)

		SLATE_ARGUMENT(TOptional<EPopupMethod>, Method)

		/** The max height of the combo box menu */
		SLATE_ARGUMENT(float, MaxListHeight)

		/**
		 * When false, the down arrow is not generated and it is up to the API consumer
		 * to make their own visual hint that this is a drop down.
		 */
		SLATE_ARGUMENT(bool, HasDownArrow)

		/** Allow setting the visibility of the search box dynamically */
		SLATE_ATTRIBUTE(EVisibility, SearchVisibility)

	SLATE_END_ARGS()

	/**
	 * Construct the widget from a declaration
	 *
	 * @param InArgs   Declaration from which to construct the combo box
	 */
	void Construct(const FArguments& InArgs);

	/**
	 * Reset the source in the Combo Box List. The options source is intelligently refreshed based on if the Source is the same or not.
	 *
	 * @param NewOptionsSource The options data to use. If null the original will be used.
	 */
	void ResetOptionsSource(const TArray<TSharedPtr<FString>>* NewOptionsSource = nullptr);

protected:
	/** Local version of parent class private variable `OptionsSource` */
	const TArray< TSharedPtr<FString> >* DisplayClusterOptionsSource;
};


