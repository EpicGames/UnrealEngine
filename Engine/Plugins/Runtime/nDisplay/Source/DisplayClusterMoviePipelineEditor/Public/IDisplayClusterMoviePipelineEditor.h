// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"


/**
 * Display Cluster editor module interface
 */
class IDisplayClusterMoviePipelineEditor :
	public IModuleInterface
{
public:
	virtual ~IDisplayClusterMoviePipelineEditor() = default;
};
