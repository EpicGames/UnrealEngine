// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;


public class DisplayClusterMultiUser : ModuleRules
{
	public DisplayClusterMultiUser(ReadOnlyTargetRules ROTargetRules) : base(ROTargetRules)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"ConcertSyncClient",
				"DisplayClusterConfiguration",
				"Engine",
				"MediaAssets"
			});
	}
}
