// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterMultiUserManager.h"
#include "DisplayClusterMultiUserLog.h"

#include "DisplayClusterConfigurationTypes_Base.h"

#include "IConcertClientTransactionBridge.h"
#include "IDisplayClusterConfiguration.h"

#include "IConcertSyncClient.h"
#include "IConcertSyncClientModule.h"

#include "WorldPartition/DataLayer/DataLayerInstance.h"

#define NDISPLAY_MULTIUSER_TRANSACTION_FILTER TEXT("DisplayClusterMultiUser")

FDisplayClusterMultiUserManager::FDisplayClusterMultiUserManager()
{
	const TSharedPtr<IConcertSyncClient> ConcertSyncClient = IConcertSyncClientModule::Get().GetClient(TEXT("MultiUser"));
	if (ConcertSyncClient.IsValid())
	{
		IConcertClientTransactionBridge* Bridge = ConcertSyncClient->GetTransactionBridge();
		check(Bridge != nullptr);

		Bridge->RegisterTransactionFilter(NDISPLAY_MULTIUSER_TRANSACTION_FILTER,
			FOnFilterTransactionDelegate::CreateRaw(this, &FDisplayClusterMultiUserManager::ShouldObjectBeTransacted));
		Bridge->OnApplyTransaction().AddRaw(this, &FDisplayClusterMultiUserManager::OnApplyRemoteTransaction);
	}
}

FDisplayClusterMultiUserManager::~FDisplayClusterMultiUserManager()
{
	const TSharedPtr<IConcertSyncClient> ConcertSyncClient = IConcertSyncClientModule::Get().GetClient(TEXT("MultiUser"));
	if (ConcertSyncClient.IsValid())
	{
		IConcertClientTransactionBridge* Bridge = ConcertSyncClient->GetTransactionBridge();
		check(Bridge != nullptr);

		Bridge->UnregisterTransactionFilter(NDISPLAY_MULTIUSER_TRANSACTION_FILTER);
		Bridge->OnApplyTransaction().RemoveAll(this);
	}
}

void FDisplayClusterMultiUserManager::OnApplyRemoteTransaction(ETransactionNotification Notification, const bool bIsSnapshot)
{
	IDisplayClusterConfiguration& Config = IDisplayClusterConfiguration::Get();
	if (Notification == ETransactionNotification::Begin && bIsSnapshot)
	{
		Config.SetIsSnapshotTransacting(true);
	}
	else if (bIsSnapshot)
	{
		Config.SetIsSnapshotTransacting(false);
	}
}

ETransactionFilterResult FDisplayClusterMultiUserManager::ShouldObjectBeTransacted(const FConcertTransactionFilterArgs& FilterArgs)
{
	if (const UObject* ObjectToFilter = FilterArgs.ObjectToFilter)
	{
		const bool bIsValidObjectType = ObjectToFilter->IsA<UDisplayClusterConfigurationData_Base>() || ObjectToFilter->IsA<UDataLayerInstance>();
		const bool bIsPersistent = !ObjectToFilter->IsTemplate() && !ObjectToFilter->HasAnyFlags(RF_Transient) && FilterArgs.Package != GetTransientPackage();
		if ((bIsValidObjectType && bIsPersistent)
			|| ObjectToFilter->GetClass()->HasMetaData(TEXT("DisplayClusterMultiUserInclude")))
		{
			UE_LOGF(LogDisplayClusterMultiUser, Log, "FDisplayClusterMultiUser transaction for object: %ls", *ObjectToFilter->GetName());
			return ETransactionFilterResult::IncludeObject;
		}
	}

	return ETransactionFilterResult::UseDefault;
}
