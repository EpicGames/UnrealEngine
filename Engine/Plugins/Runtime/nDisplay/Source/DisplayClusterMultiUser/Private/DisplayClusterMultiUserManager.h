// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ConcertTransactionEvents.h"
#include "IConcertClientTransactionBridge.h"

struct FConcertTransactionFilterArgs;

/**
 * Manager for handling multi user transactions in nDisplay.
 */
class FDisplayClusterMultiUserManager
{
public:
	FDisplayClusterMultiUserManager();
	virtual ~FDisplayClusterMultiUserManager();

private:
	void OnApplyRemoteTransaction(ETransactionNotification Notification, const bool bIsSnapshot);
	ETransactionFilterResult ShouldObjectBeTransacted(const FConcertTransactionFilterArgs& FilterArgs);
};
