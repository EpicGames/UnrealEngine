// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterMultiUserModule.h"
#include "Modules/ModuleManager.h"

#if WITH_EDITOR
#include "MediaAssetMultiUserManager.h"
#endif


void FDisplayClusterMultiUserModule::StartupModule()
{
	MultiUserManager = MakeUnique<FDisplayClusterMultiUserManager>();
#if WITH_EDITOR
	MediaAssetMUManager = MakeUnique<FMediaAssetMultiUserManager>();
#endif
}

void FDisplayClusterMultiUserModule::ShutdownModule()
{
	MultiUserManager.Reset();
#if WITH_EDITOR
	MediaAssetMUManager.Reset();
#endif
}

IMPLEMENT_MODULE(FDisplayClusterMultiUserModule, DisplayClusterMultiUser);
