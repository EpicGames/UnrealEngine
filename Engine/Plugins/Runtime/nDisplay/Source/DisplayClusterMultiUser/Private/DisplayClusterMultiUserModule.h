// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DisplayClusterMultiUserManager.h"

#include "Modules/ModuleInterface.h"
#include "Templates/UniquePtr.h"

class FMediaAssetMultiUserManager;

class FDisplayClusterMultiUserModule :
	public IModuleInterface
{
public:
	// Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	// ~ End IModuleInterface
private:
	// Manager for multi user connections.
	TUniquePtr<FDisplayClusterMultiUserManager> MultiUserManager;

#if WITH_EDITOR
	TUniquePtr<FMediaAssetMultiUserManager> MediaAssetMUManager;
#endif
};
