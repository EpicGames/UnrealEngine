// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaAssetMultiUserManager.h"

#if WITH_EDITOR

#include "IMediaAssetsModule.h"
#include "IConcertSession.h"
#include "IConcertSyncClientModule.h"
#include "IConcertClientModule.h"
#include "IConcertSyncClient.h"

FMediaAssetMultiUserManager::FMediaAssetMultiUserManager()
{
	if (TSharedPtr<IConcertSyncClient> ConcertSyncClient = IConcertSyncClientModule::Get().GetClient(TEXT("MultiUser")))
	{
		IConcertClientRef ConcertClient = ConcertSyncClient->GetConcertClient();
		ConcertClient->OnSessionStartup().AddRaw(this, &FMediaAssetMultiUserManager::Register);
		ConcertClient->OnSessionShutdown().AddRaw(this, &FMediaAssetMultiUserManager::Unregister);

		if (TSharedPtr<IConcertClientSession> ConcertClientSession = ConcertClient->GetCurrentSession())
		{
			Register(ConcertClientSession.ToSharedRef());
		}
	}

	IMediaAssetsModule& MediaAssetsModule = FModuleManager::Get().LoadModuleChecked<IMediaAssetsModule>("MediaAssets");
	OnMediaPlateStateChangedHandle = MediaAssetsModule.RegisterOnMediaStateChangedEvent(IMediaAssetsModule::FMediaStateChangedDelegate::FDelegate::CreateRaw(this, &FMediaAssetMultiUserManager::OnMediaPlateStateChanged));
}


FMediaAssetMultiUserManager::~FMediaAssetMultiUserManager()
{
	IMediaAssetsModule& MediaAssetsModule = FModuleManager::Get().GetModuleChecked<IMediaAssetsModule>("MediaAssets");
	MediaAssetsModule.UnregisterOnMediaStateChangedEvent(OnMediaPlateStateChangedHandle);
}

void FMediaAssetMultiUserManager::Register(TSharedRef<IConcertClientSession> InSession)
{
	ConcertSession = InSession;
	InSession->RegisterCustomEventHandler<FConcertMediaStateChangedEvent>(this, &FMediaAssetMultiUserManager::OnStateChangedEvent);
}

void FMediaAssetMultiUserManager::Unregister(TSharedRef<IConcertClientSession> InSession)
{
	if (TSharedPtr<IConcertClientSession> ConcertSessionPinned = ConcertSession.Pin())
	{
		ConcertSessionPinned->UnregisterCustomEventHandler<FConcertMediaStateChangedEvent>(this);
	}
	ConcertSession.Reset();
}

void FMediaAssetMultiUserManager::OnStateChangedEvent(const FConcertSessionContext& InConcertSessionContext, const FConcertMediaStateChangedEvent& InEvent)
{
	IMediaAssetsModule& MediaAssetsModule = FModuleManager::Get().LoadModuleChecked<IMediaAssetsModule>("MediaAssets");
	MediaAssetsModule.BroadcastOnMediaStateChangedEvent(InEvent.ActorsPathNames, InEvent.State, true /* Broadcast came from a remote endpoint. */);
}

void FMediaAssetMultiUserManager::OnMediaPlateStateChanged(const TArray<FString>& InActorsPathNames, uint8 InEnumState, bool bRemoteBroadcast)
{
	// We don't need to broadcast to other endpoints as this is already handled.
	if (bRemoteBroadcast)
	{
		return;
	}

	if (TSharedPtr<IConcertClientSession> ConcertSessionPinned = ConcertSession.Pin())
	{
		TArray<FGuid> ClientIds = ConcertSessionPinned->GetSessionClientEndpointIds();
		FConcertMediaStateChangedEvent EventData;
		EventData.ActorsPathNames = InActorsPathNames;
		EventData.State = InEnumState;
		ConcertSessionPinned->SendCustomEvent(EventData, ClientIds, EConcertMessageFlags::ReliableOrdered | EConcertMessageFlags::UniqueId);
	}
}

#endif

