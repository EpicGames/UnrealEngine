// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Misc/CoreDefines.h"
#include "MediaAssetMultiUserManager.generated.h"

struct FConcertSessionContext;
class IConcertSyncClient;
class IConcertClientSession;

#if WITH_EDITOR


/** 
* The event that is transfered over MU.
*/
USTRUCT()
struct FConcertMediaStateChangedEvent
{
public:
	GENERATED_BODY()

	/** Name used to find the object on the other side of MU. */
	UPROPERTY()
	TArray<FString> ActorsPathNames;

	/** State to replicate on the other side. */
	UPROPERTY()
	uint8 State;
};

/**
 * Reacts to state changes in Media Plates and transfers state change information to remote endpoints.
 */
class FMediaAssetMultiUserManager
{
public:

	FMediaAssetMultiUserManager();

	virtual ~FMediaAssetMultiUserManager();

	/** 
	* Subscribes to the relevant Concert Sync events.
	*/
	void Register(TSharedRef<IConcertClientSession> InSession);

	/** 
	* Unsubscribes from concert sync events.
	*/
	void Unregister(TSharedRef<IConcertClientSession> InSession);

	/** 
	* Reacts to changes in Media Plate from remove endpoints and forwards it to local Media Plates.
	*/
	void OnStateChangedEvent(const FConcertSessionContext& InConcertSessionContext, const FConcertMediaStateChangedEvent& InEvent);

	/**
	* Called when Media Plate broadcasts state change events and sends a custom event to remote endpoints.
	* bRemoteBroadcast identifies if event came from the remote endpoint or broadcasted locally.
	*/
	void OnMediaPlateStateChanged(const TArray<FString>& InNameId, uint8 InEnumState, bool bRemoteBroadcast);

private:

	TWeakPtr<IConcertClientSession> ConcertSession;
	FDelegateHandle OnMediaPlateStateChangedHandle;
};

#endif // WITH_EDITOR
