// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class DisplayClusterOperator : ModuleRules
{
	public DisplayClusterOperator(ReadOnlyTargetRules ROTargetRules) : base(ROTargetRules)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"DisplayCluster",

				"Core",
				"CoreUObject",
				"EditorStyle",
				"Engine",
				"InputCore",
				"Kismet",
				"Projects",
				"Slate",
				"SlateCore",
				"SubobjectDataInterface",
				"SubobjectEditor",
				"ToolMenus",
				"ToolWidgets",
				"UnrealEd",
				"WorkspaceMenuStructure",
			});
	}
}
