// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterOperatorModule.h"

#include "DisplayClusterOperatorStyle.h"
#include "DisplayClusterOperatorViewModel.h"
#include "SDisplayClusterOperatorPanel.h"
#include "DisplayClusterRootActor.h"

#include "Editor.h"
#include "EngineUtils.h"
#include "Engine/World.h"
#include "LevelEditorViewport.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Widgets/Docking/SDockTab.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

#define LOCTEXT_NAMESPACE "DisplayClusterOperator"

const FName FDisplayClusterOperatorModule::OperatorPanelTabName = TEXT("DisplayClusterOperatorTab");

void FDisplayClusterOperatorModule::StartupModule()
{
	OperatorViewModel = MakeShared<FDisplayClusterOperatorViewModel>();
	OperatorToolBarExtensibilityManager = MakeShared<FExtensibilityManager>();
	OperatorMenuExtensibilityManager = MakeShared<FExtensibilityManager>();

	FDisplayClusterOperatorStyle::Get();
	
	RegisterTabSpawners();
}

void FDisplayClusterOperatorModule::ShutdownModule()
{
	UnregisterTabSpawners();
}

FDelegateHandle FDisplayClusterOperatorModule::RegisterApp(const FOnGetAppInstance& InGetAppInstanceDelegate)
{
	const FDelegateHandle Handle = InGetAppInstanceDelegate.GetHandle();
	RegisteredApps.Add(Handle, InGetAppInstanceDelegate);
	return Handle;
}

void FDisplayClusterOperatorModule::UnregisterApp(const FDelegateHandle& InHandle)
{
	AppUnregisteredEvent.Broadcast(InHandle);
	RegisteredApps.Remove(InHandle);
}

TSharedRef<IDisplayClusterOperatorViewModel> FDisplayClusterOperatorModule::GetOperatorViewModel()
{
	return OperatorViewModel.ToSharedRef();
}

FName FDisplayClusterOperatorModule::GetPrimaryOperatorExtensionId()
{
	return SDisplayClusterOperatorPanel::PrimaryTabExtensionId;
}

FName FDisplayClusterOperatorModule::GetAuxilliaryOperatorExtensionId()
{
	return SDisplayClusterOperatorPanel::AuxilliaryTabExtensionId;
}

FName FDisplayClusterOperatorModule::GetDetailsTabId()
{
	return SDisplayClusterOperatorPanel::DetailsTabId;
}

void FDisplayClusterOperatorModule::GetRootActorLevelInstances(TArray<ADisplayClusterRootActor*>& OutRootActorInstances)
{
	UWorld* World = GCurrentLevelEditingViewportClient ? GCurrentLevelEditingViewportClient->GetWorld() : nullptr;
	if (World)
	{
		for (TActorIterator<ADisplayClusterRootActor> Iter(World); Iter; ++Iter)
		{
			ADisplayClusterRootActor* RootActor = *Iter;
			if (IsValid(RootActor))
			{
				OutRootActorInstances.Add(RootActor);
			}
		}
	}
}

void FDisplayClusterOperatorModule::ToggleDrawer(const FName DrawerId)
{
	if (ActiveOperatorPanel.IsValid())
	{
		ActiveOperatorPanel.Pin()->ToggleDrawer(DrawerId);
	}
}

void FDisplayClusterOperatorModule::ForceDismissDrawers()
{
	if (ActiveOperatorPanel.IsValid())
	{
		ActiveOperatorPanel.Pin()->ForceDismissDrawers();
	}
}

void FDisplayClusterOperatorModule::RegisterTabSpawners()
{
	// Register the nDisplay operator panel tab spawner
	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(OperatorPanelTabName, FOnSpawnTab::CreateRaw(this, &FDisplayClusterOperatorModule::SpawnOperatorPanelTab))
		.SetDisplayName(LOCTEXT("TabDisplayName", "In-Camera VFX"))
		.SetTooltipText(LOCTEXT("TabTooltip", "Open the In-Camera VFX tab."))
		.SetIcon(FSlateIcon(FDisplayClusterOperatorStyle::Get().GetStyleSetName(), (TEXT("OperatorPanel.Icon"))))
		.SetGroup(WorkspaceMenu::GetMenuStructure().GetLevelEditorVirtualProductionCategory());
}

void FDisplayClusterOperatorModule::UnregisterTabSpawners()
{
	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(OperatorPanelTabName);
}

TSharedRef<SDockTab> FDisplayClusterOperatorModule::SpawnOperatorPanelTab(const FSpawnTabArgs& SpawnTabArgs)
{
	const TSharedRef<SDockTab> MajorTab = SNew(SDockTab)
		.TabRole(ETabRole::MajorTab)
		.IconColor(FDisplayClusterOperatorStyle::Get().GetColor(TEXT("OperatorPanel.AssetColor")))
		.OnTabClosed_Raw(this, &FDisplayClusterOperatorModule::OnOperatorPanelTabClosed);

	MajorTab->SetContent(SAssignNew(ActiveOperatorPanel, SDisplayClusterOperatorPanel, OperatorViewModel->CreateTabManager(MajorTab), SpawnTabArgs.GetOwnerWindow()));

	return MajorTab;
}

void FDisplayClusterOperatorModule::OnOperatorPanelTabClosed(TSharedRef<SDockTab> Tab)
{
	OperatorViewModel->ResetTabManager();
	OperatorViewModel->SetRootActor(nullptr);

	ActiveOperatorPanel.Reset();
}

IMPLEMENT_MODULE(FDisplayClusterOperatorModule, DisplayClusterOperator);

#undef LOCTEXT_NAMESPACE
