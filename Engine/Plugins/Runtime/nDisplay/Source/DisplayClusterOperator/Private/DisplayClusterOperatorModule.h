// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDisplayClusterOperator.h"

class FDisplayClusterOperatorViewModel;
class FSpawnTabArgs;
class SDockTab;
class SDisplayClusterOperatorPanel;

/**
 * Display Cluster editor module
 */
class FDisplayClusterOperatorModule :
	public IDisplayClusterOperator
{
public:
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnAppUnregistered, const FDelegateHandle&)
	
	/** The name of the tab that the operator panel lives in */
	static const FName OperatorPanelTabName;

public:
	//~ IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface interface

	//~ IDisplayClusterOperator interface
	virtual FDelegateHandle RegisterApp(const FOnGetAppInstance& InGetAppInstanceDelegate) override;
	virtual void UnregisterApp(const FDelegateHandle& InHandle) override;
	virtual TSharedRef<IDisplayClusterOperatorViewModel> GetOperatorViewModel() override;
	virtual FOnRegisterLayoutExtensions& OnRegisterLayoutExtensions() override { return RegisterLayoutExtensions; }
	virtual FOnRegisterStatusBarExtensions& OnRegisterStatusBarExtensions() override { return RegisterStatusBarExtensions; }
	virtual FOnAppendOperatorPanelCommands& OnAppendOperatorPanelCommands() override { return AppendOperatorPanelCommands; }

	virtual FName GetPrimaryOperatorExtensionId() override;
	virtual FName GetAuxilliaryOperatorExtensionId() override;
	virtual FName GetDetailsTabId() override;
	virtual TSharedPtr<FExtensibilityManager> GetOperatorToolBarExtensibilityManager() override { return OperatorToolBarExtensibilityManager; }
	virtual TSharedPtr<FExtensibilityManager> GetOperatorMenuExtensibilityManager() override { return OperatorMenuExtensibilityManager; }
	virtual void GetRootActorLevelInstances(TArray<ADisplayClusterRootActor*>& OutRootActorInstances) override;
	virtual void ToggleDrawer(const FName DrawerId) override;
	virtual void ForceDismissDrawers() override;
	//~ End IDisplayClusterOperator interface

	/** Retrieve currently registered app delegates */
	const TMap<FDelegateHandle, FOnGetAppInstance>& GetRegisteredApps() const { return RegisteredApps; }

	/** Broadcast when an app is unregistered */
	FOnAppUnregistered& OnAppUnregistered() { return AppUnregisteredEvent; }
	
private:
	void RegisterTabSpawners();
	void UnregisterTabSpawners();

	TSharedRef<SDockTab> SpawnOperatorPanelTab(const FSpawnTabArgs& SpawnTabArgs);
	void OnOperatorPanelTabClosed(TSharedRef<SDockTab> Tab);

private:
	FOnRegisterLayoutExtensions RegisterLayoutExtensions;
	FOnRegisterStatusBarExtensions RegisterStatusBarExtensions;
	FOnAppendOperatorPanelCommands AppendOperatorPanelCommands;

	TWeakPtr<SDisplayClusterOperatorPanel> ActiveOperatorPanel;
	TSharedPtr<FDisplayClusterOperatorViewModel> OperatorViewModel;
	TSharedPtr<FExtensibilityManager> OperatorToolBarExtensibilityManager;
	TSharedPtr<FExtensibilityManager> OperatorMenuExtensibilityManager;

	TMap<FDelegateHandle, FOnGetAppInstance> RegisteredApps;
	FOnAppUnregistered AppUnregisteredEvent;
};
