// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterOperatorStatusBarExtender.h"

#include "SDisplayClusterOperatorStatusBar.h"

void FDisplayClusterOperatorStatusBarExtender::AddWidgetDrawer(const FWidgetDrawerConfig& InWidgetDrawerConfig, int32 SlotIndex)
{
	WidgetDrawers.Add(TPair<int32, FWidgetDrawerConfig>(SlotIndex, InWidgetDrawerConfig));
}

void FDisplayClusterOperatorStatusBarExtender::RegisterExtensions(TSharedRef<SDisplayClusterOperatorStatusBar> StatusBar)
{
	for (TPair<int32, FWidgetDrawerConfig>& DrawerConfigPair : WidgetDrawers)
	{
		StatusBar->RegisterDrawer(MoveTemp(DrawerConfigPair.Value), DrawerConfigPair.Key);
	}
}