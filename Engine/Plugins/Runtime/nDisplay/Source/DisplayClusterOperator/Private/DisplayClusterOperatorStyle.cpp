// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterOperatorStyle.h"

#include "Interfaces/IPluginManager.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateStyleMacros.h"

FDisplayClusterOperatorStyle::FDisplayClusterOperatorStyle()
	: FSlateStyleSet(TEXT("DisplayClusterOperatorStyle"))
{
	Initialize();

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FDisplayClusterOperatorStyle::~FDisplayClusterOperatorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}

#define EDITOR_IMAGE_BRUSH_SVG( RelativePath, ... ) FSlateVectorImageBrush(FPaths::EngineContentDir() / "Editor/Slate" / RelativePath + TEXT(".svg"), __VA_ARGS__ )
#define EDITOR_BOX_BRUSH( RelativePath, ... ) FSlateBoxBrush(FPaths::EngineContentDir() / "Editor/Slate" / RelativePath + TEXT(".png"), __VA_ARGS__ )

const FVector2D Icon16x16(16.0f, 16.0f);

void FDisplayClusterOperatorStyle::Initialize()
{
	TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(TEXT("nDisplay"));
	check(Plugin.IsValid());
	if (Plugin.IsValid())
	{
		SetContentRoot(FPaths::Combine(Plugin->GetBaseDir(), TEXT("Content")));
		SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));
	}

	// Icons
	{
		Set("OperatorPanel.Icon", new IMAGE_BRUSH_SVG("Icons/Components/nDisplayCamera", Icon16x16));
		Set("RootActorPanel.Icon", new IMAGE_BRUSH_SVG("Icons/RootActor/nDisplay", Icon16x16));
	}
	
	// Colors
	{
		Set("OperatorPanel.AssetColor", FLinearColor(0, 188, 212));  // From FDisplayClusterConfiguratorAssetTypeActions::GetTypeColor()
	}
}

#undef EDITOR_IMAGE_BRUSH_SVG
#undef EDITOR_BOX_BRUSH
