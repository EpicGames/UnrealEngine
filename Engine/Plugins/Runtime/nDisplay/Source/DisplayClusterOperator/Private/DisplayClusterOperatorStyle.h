// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

class FDisplayClusterOperatorStyle : public FSlateStyleSet
{
public:
	FDisplayClusterOperatorStyle();
	~FDisplayClusterOperatorStyle();

	static FDisplayClusterOperatorStyle& Get()
	{
		static FDisplayClusterOperatorStyle Inst;
		return Inst;
	}

private:
	void Initialize();

};
