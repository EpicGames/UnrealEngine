// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterOperatorViewModel.h"

#include "DisplayClusterRootActor.h"

#include "Framework/Docking/TabManager.h"

#define LOCTEXT_NAMESPACE "DisplayClusterOperatorViewModel"

ADisplayClusterRootActor* FDisplayClusterOperatorViewModel::GetRootActor(bool bEvenIfPendingKill) const
{
	if (bEvenIfPendingKill)
	{
		return RootActor.IsValid(bEvenIfPendingKill) ? RootActor.GetEvenIfUnreachable() : nullptr;
	}

	return RootActor.IsValid() ? RootActor.Get() : nullptr;
}

void FDisplayClusterOperatorViewModel::SetRootActor(ADisplayClusterRootActor* InRootActor)
{
	RootActor = InRootActor;
	RootActorChanged.Broadcast(RootActor.Get());
}

void FDisplayClusterOperatorViewModel::ShowDetailsForObject(UObject* Object)
{
	ShowDetailsForObjects({ Object });
}

void FDisplayClusterOperatorViewModel::ShowDetailsForObjects(const TArray<UObject*>& Objects)
{
	bool bDetailObjectsChanged = false;

	if (DetailObjects.Num() == Objects.Num())
	{
		for (UObject* NewObject : Objects)
		{
			if (NewObject && !DetailObjects.Contains(NewObject))
			{
				bDetailObjectsChanged = true;
				break;
			}
		}
	}
	else
	{
		bDetailObjectsChanged = true;
	}

	if (bDetailObjectsChanged)
	{
		DetailObjects.Empty();
		DetailObjects.Append(Objects);
		DetailObjectsChanged.Broadcast(Objects);
	}
}

void FDisplayClusterOperatorViewModel::SelectActors(const TArray<AActor*>& Actors, bool bShouldSelect)
{
	ActorsSelectedExternally.Broadcast(Actors, bShouldSelect);
}

TSharedRef<FTabManager> FDisplayClusterOperatorViewModel::CreateTabManager(const TSharedRef<SDockTab>& MajorTabOwner)
{
	TabManager = FGlobalTabmanager::Get()->NewTabManager(MajorTabOwner);
	WorkspaceItem = TabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("OperatorMenuGroupName", "In-Camera VFX"));
	return TabManager.ToSharedRef();
}

void FDisplayClusterOperatorViewModel::ResetTabManager()
{
	TabManager.Reset();
}

#undef LOCTEXT_NAMESPACE