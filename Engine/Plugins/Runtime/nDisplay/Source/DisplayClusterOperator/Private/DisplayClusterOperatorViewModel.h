// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDisplayClusterOperatorViewModel.h"
#include "UObject/WeakObjectPtr.h"

class ADisplayClusterRootActor;
class SDockTab;

/** A view model object that stores any state from the operator panel that should be exposed externally */
class FDisplayClusterOperatorViewModel : public TSharedFromThis<FDisplayClusterOperatorViewModel>, public IDisplayClusterOperatorViewModel
{
public:
	virtual ~FDisplayClusterOperatorViewModel() = default;

	//~ IDisplayClusterOperatorViewModel interface
	virtual bool HasRootActor(bool bEvenIfPendingKill = false) const override { return RootActor.IsValid(bEvenIfPendingKill); }
	virtual ADisplayClusterRootActor* GetRootActor(bool bEvenIfPendingKill = false) const override;
	virtual void SetRootActor(ADisplayClusterRootActor* InRootActor) override;
	virtual FOnActiveRootActorChanged& OnActiveRootActorChanged() override { return RootActorChanged; }
	virtual const TArray<TWeakObjectPtr<UObject>> GetDetailObjects() override { return DetailObjects; }
	virtual void ShowDetailsForObject(UObject* Object) override;
	virtual void ShowDetailsForObjects(const TArray<UObject*>& Objects) override;
	virtual void SelectActors(const TArray<AActor*>& Actors, bool bShouldSelect) override;
	virtual FOnDetailObjectsChanged& OnDetailObjectsChanged() override { return DetailObjectsChanged; }
	virtual FOnOutlinerSelectionChanged& OnOutlinerSelectionChanged() override { return OutlinerSelectionChanged; }
	virtual FOnActorsSelected& OnActorsSelectedExternally() override { return ActorsSelectedExternally; }
	virtual TSharedPtr<FTabManager> GetTabManager() const override { return TabManager; }

	virtual TSharedPtr<FWorkspaceItem> GetWorkspaceMenuGroup() const override { return WorkspaceItem; }
	//~ End IDisplayClusterOperatorViewModel interface

	TSharedRef<FTabManager> CreateTabManager(const TSharedRef<SDockTab>& MajorTabOwner);
	void ResetTabManager();

private:
	TWeakObjectPtr<ADisplayClusterRootActor> RootActor;
	TArray<TWeakObjectPtr<UObject>> DetailObjects;
	TSharedPtr<FTabManager> TabManager;
	TSharedPtr<FWorkspaceItem> WorkspaceItem;

	FOnActiveRootActorChanged RootActorChanged;
	FOnDetailObjectsChanged DetailObjectsChanged;
	FOnActorsSelected ActorsSelectedExternally;
	FOnOutlinerSelectionChanged OutlinerSelectionChanged;
};