// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDisplayClusterOperatorPanel.h"

#include "DisplayClusterOperatorModule.h"
#include "DisplayClusterOperatorStatusBarExtender.h"
#include "DisplayClusterOperatorStyle.h"
#include "IDisplayClusterOperator.h"
#include "IDisplayClusterOperatorViewModel.h"
#include "SDisplayClusterOperatorRootActorPanel.h"
#include "SDisplayClusterOperatorStatusBar.h"
#include "SDisplayClusterOperatorToolbar.h"

#include "Editor.h"
#include "LevelEditor.h"
#include "LevelEditorActions.h"
#include "SKismetInspector.h"
#include "TimerManager.h"
#include "ToolMenus.h"
#include "Framework/Docking/LayoutExtender.h"
#include "Framework/Docking/TabManager.h"
#include "Interfaces/IMainFrameModule.h"
#include "Styling/AppStyle.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "SDisplayClusterOperatorPanel"

const FName SDisplayClusterOperatorPanel::DetailsTabId = TEXT("OperatorDetails");
const FName SDisplayClusterOperatorPanel::RootActorPanelId = TEXT("RootActorPanel");
const FName SDisplayClusterOperatorPanel::PrimaryTabExtensionId = TEXT("PrimaryOperatorTabStack");
const FName SDisplayClusterOperatorPanel::AuxilliaryTabExtensionId = TEXT("AuxilliaryOperatorTabStack");

SDisplayClusterOperatorPanel::~SDisplayClusterOperatorPanel()
{
	FDisplayClusterOperatorModule& OperatorModule = FModuleManager::GetModuleChecked<FDisplayClusterOperatorModule>(IDisplayClusterOperator::ModuleName);
	OperatorModule.OnAppUnregistered().RemoveAll(this);
	OperatorModule.GetOperatorViewModel()->OnDetailObjectsChanged().Remove(DetailObjectsChangedHandle);
}

void SDisplayClusterOperatorPanel::Construct(const FArguments& InArgs, const TSharedRef<FTabManager>& InTabManager, const TSharedPtr<SWindow>& WindowOwner)
{
	CommandList = MakeShareable(new FUICommandList);
	BindCommands();
	
	TabManager = InTabManager;
	const auto& PersistLayout = [](const TSharedRef<FTabManager::FLayout>& LayoutToSave)
	{
		FLayoutSaveRestore::SaveToConfig(GEditorLayoutIni, LayoutToSave);
	};
	TabManager->SetOnPersistLayout(FTabManager::FOnPersistLayout::CreateLambda(PersistLayout));
	TabManager->SetAllowWindowMenuBar(true);

	const TSharedRef<IDisplayClusterOperatorViewModel> ViewModel = IDisplayClusterOperator::Get().GetOperatorViewModel();
	const TSharedRef<FWorkspaceItem> AppMenuGroup = ViewModel->GetWorkspaceMenuGroup().ToSharedRef();

	TabManager->RegisterTabSpawner(DetailsTabId, FOnSpawnTab::CreateSP(this, &SDisplayClusterOperatorPanel::SpawnDetailsTab))
		.SetDisplayName(LOCTEXT("DetailsTabTitle", "Details"))
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"))
		.SetGroup(AppMenuGroup);

	TabManager->RegisterTabSpawner(RootActorPanelId, FOnSpawnTab::CreateSP(this, &SDisplayClusterOperatorPanel::SpawnRootActorPanelTab))
		.SetDisplayName(LOCTEXT("RootActorPanelTabTitle", "nDisplay Settings"))
		.SetIcon(FSlateIcon(FDisplayClusterOperatorStyle::Get().GetStyleSetName(), "RootActorPanel.Icon"))
		.SetGroup(AppMenuGroup);

	// Load any operator apps
	{
		FDisplayClusterOperatorModule& OperatorModule = FModuleManager::GetModuleChecked<FDisplayClusterOperatorModule>(IDisplayClusterOperator::ModuleName);
		OperatorModule.OnAppUnregistered().AddSP(this, &SDisplayClusterOperatorPanel::OnOperatorAppUnregistered);
		for (const TTuple<FDelegateHandle, IDisplayClusterOperator::FOnGetAppInstance>&
			 RegisteredAppKeyVal : OperatorModule.GetRegisteredApps())
		{
			if (RegisteredAppKeyVal.Value.IsBound())
			{
				TSharedRef<IDisplayClusterOperatorApp> AppInstance = RegisteredAppKeyVal.Value.Execute(ViewModel);
				AppInstances.Add(RegisteredAppKeyVal.Key, AppInstance);
			}
		}
	}
	
	TSharedRef<FTabManager::FLayout> Layout = FTabManager::NewLayout("nDisplayOperatorLayout_v2")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(EOrientation::Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()
					->SetOrientation(Orient_Horizontal)
					->Split
					(
						FTabManager::NewSplitter()
						->SetOrientation(Orient_Vertical)
						->SetSizeCoefficient(0.75f)
						->Split
						(
							FTabManager::NewStack()
								->SetHideTabWell(true)
								->SetExtensionId(PrimaryTabExtensionId)
						)
						->Split
						(
							FTabManager::NewStack()
								->SetExtensionId(AuxilliaryTabExtensionId)
						)
					)
					->Split
					(
						FTabManager::NewSplitter()
						->SetOrientation(Orient_Vertical)
						->SetSizeCoefficient(0.25f)
						->Split
						(
							FTabManager::NewStack()
							->AddTab(DetailsTabId, ETabState::OpenedTab)
							->AddTab(RootActorPanelId, ETabState::OpenedTab)
							->SetForegroundTab(DetailsTabId)
							->SetHideTabWell(false)
						)
						
					)
			)
		);

	Layout = FLayoutSaveRestore::LoadFromConfig(GEditorLayoutIni, Layout);
	
	static FName MenuName = "DisplayClusterOperator.MainMenu";
	const FLevelEditorModule& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));

	// Fill toolbar
	{
		UToolMenus* ToolMenus = UToolMenus::Get();
		const FName ParentMenuName("MainFrame.MainMenu");
		
		if (!ToolMenus->IsMenuRegistered(MenuName))
		{
			ToolMenus->RegisterMenu(MenuName, ParentMenuName);

			// File menu
			{
				UToolMenu* SubMenu = ToolMenus->RegisterMenu(*(MenuName.ToString() + TEXT(".File")), *(ParentMenuName.ToString() + TEXT(".File")));
				const FToolMenuInsert InsertPos(NAME_None, EToolMenuInsertType::First);
				FToolMenuSection& Section = SubMenu->FindOrAddSection("FileSave");

				// Add generic save level options. If we ever add other types of apps that don't rely on the level
				// instance being saved we may want to refactor this so the save option is added per app instead.
				const FLevelEditorCommands& Commands = LevelEditor.GetLevelEditorCommands();
				Section.AddMenuEntry(Commands.Save).InsertPosition = InsertPos;
				Section.AddMenuEntry(Commands.SaveAs).InsertPosition = InsertPos;
			}
		}
	};
	
	LayoutExtender = MakeShared<FLayoutExtender>();
	IDisplayClusterOperator::Get().OnRegisterLayoutExtensions().Broadcast(*LayoutExtender);
	Layout->ProcessExtensions(*LayoutExtender);

	DetailObjectsChangedHandle = IDisplayClusterOperator::Get().GetOperatorViewModel()->OnDetailObjectsChanged().AddSP(this, &SDisplayClusterOperatorPanel::DisplayObjectsInDetailsPanel);
	
	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SAssignNew(ToolbarContainer, SBox)
		]
		
		+SVerticalBox::Slot()
		.Padding(4.0f, 2.f, 4.f, 2.f)
		.FillHeight(1.0f)
		[
			TabManager->RestoreFrom(Layout, WindowOwner).ToSharedRef()
		]

		+SVerticalBox::Slot()
		.Padding(0.0f, 2.0f, 0.0f, 0.0f)
		.AutoHeight()
		[
			SAssignNew(StatusBar, SDisplayClusterOperatorStatusBar)
		]
	];

	TArray<TSharedPtr<FExtender>> MenuExtenders;
	const TSharedPtr<FExtender> MenuExtender = IDisplayClusterOperator::Get().GetOperatorMenuExtensibilityManager()->GetAllExtenders();

	CommandList->Append(LevelEditor.GetGlobalLevelEditorActions());
	FToolMenuContext ToolMenuContext(CommandList, MenuExtender);
	
	const IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
	MainFrameModule.MakeMainMenu(TabManager, MenuName, ToolMenuContext);
	
	// Allow any external modules to register extensions to the operator panel's status bar
	FDisplayClusterOperatorStatusBarExtender StatusBarExtender;
	IDisplayClusterOperator::Get().OnRegisterStatusBarExtensions().Broadcast(StatusBarExtender);
	StatusBarExtender.RegisterExtensions(StatusBar.ToSharedRef());

	if (ToolbarContainer.IsValid())
	{
		// Create toolbar after tab has been restored so child windows can register their toolbar extensions
		ToolbarContainer->SetContent(
		SAssignNew(Toolbar, SDisplayClusterOperatorToolbar)
		.CommandList(CommandList));
	}
}

FReply SDisplayClusterOperatorPanel::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (CommandList.IsValid() && CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SDisplayClusterOperatorPanel::ToggleDrawer(const FName DrawerId)
{
	if (StatusBar.IsValid())
	{
		bool bWasDismissed = false;

		if (StatusBar->IsDrawerOpened(DrawerId))
		{
			StatusBar->DismissDrawer(nullptr);
			bWasDismissed = true;
		}

		if (!bWasDismissed)
		{
			GEditor->GetTimerManager()->SetTimerForNextTick(FTimerDelegate::CreateLambda([this, DrawerId]()
			{
				if (StatusBar.IsValid())
				{
					StatusBar->OpenDrawer(DrawerId);
				}
			}));
		}
	}
}

void SDisplayClusterOperatorPanel::ForceDismissDrawers()
{
	if (StatusBar.IsValid())
	{
		StatusBar->DismissDrawer(nullptr);
	}
}

void SDisplayClusterOperatorPanel::BindCommands()
{
	if (CommandList.IsValid())
	{
		IDisplayClusterOperator::Get().OnAppendOperatorPanelCommands().Broadcast(CommandList.ToSharedRef());
	}
}

TSharedRef<SDockTab> SDisplayClusterOperatorPanel::SpawnDetailsTab(const FSpawnTabArgs& Args)
{
	SAssignNew(DetailsView, SKismetInspector)
	.HideNameArea(true);

	return SNew(SDockTab)
		.TabRole(ETabRole::PanelTab)
		[
			DetailsView.ToSharedRef()
		];
}

TSharedRef<SDockTab> SDisplayClusterOperatorPanel::SpawnRootActorPanelTab(const FSpawnTabArgs& Args)
{
	SAssignNew(RootActorPanel, SDisplayClusterOperatorRootActorPanel, TabManager, RootActorPanelId);

	return SNew(SDockTab)
		.TabRole(ETabRole::PanelTab)
		[
			RootActorPanel.ToSharedRef()
		];
}

void SDisplayClusterOperatorPanel::DisplayObjectsInDetailsPanel(const TArray<UObject*>& Objects)
{
	if (DetailsView.IsValid())
	{
		SKismetInspector::FShowDetailsOptions Options;
		Options.bShowComponents = false;
		DetailsView->ShowDetailsForObjects(Objects, MoveTemp(Options));
	}
}

void SDisplayClusterOperatorPanel::OnOperatorAppUnregistered(const FDelegateHandle& InHandle)
{
	AppInstances.Remove(InHandle);
}

#undef LOCTEXT_NAMESPACE
