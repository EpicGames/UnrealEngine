// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"

class SDisplayClusterOperatorRootActorPanel;
class FTabManager;
class FLayoutExtender;
class FSpawnTabArgs;
class FMenuBuilder;
class FUICommandList;
class SBox;
class SDockTab;
class SWindow;
class SDisplayClusterOperatorToolbar;
class SDisplayClusterOperatorStatusBar;
class IDisplayClusterOperatorApp;

/** The nDisplay operator panel that allows users to edit root actor instances with a variety of tools */
class SDisplayClusterOperatorPanel : public SCompoundWidget
{
public:
	/** The tab ID that the details panel lives in */
	static const FName DetailsTabId;

	/** The tab ID that the root actor properties panel lives in */
	static const FName RootActorPanelId;

	/** The ID of the main tab stack that can be extended by external tabs */
	static const FName PrimaryTabExtensionId;

	/** The ID of the auxilliary tab stack that can be extended by external tabs; intended for lower-thirds windows like log outputs */
	static const FName AuxilliaryTabExtensionId;

public:
	SLATE_BEGIN_ARGS(SDisplayClusterOperatorPanel) {}
	SLATE_END_ARGS()

	~SDisplayClusterOperatorPanel();

	void Construct(const FArguments& InArgs, const TSharedRef<FTabManager>& InTabManager, const TSharedPtr<SWindow>& WindowOwner);

	//~ SWidget interface
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	//~ End SWidget interface

	/** Toggles the state of a drawer with the specified ID, closing the drawer if it is open, and opening the drawer if it is closed */
	void ToggleDrawer(const FName DrawerId);

	/** Forces the operator panel to dismiss any open drawers */
	void ForceDismissDrawers();

private:
	/** Binds any commands to the operator panel's command list, as well as allowing any external modules to register their own commands */
	void BindCommands();

	/** Creates a tab with the details panel in it */
	TSharedRef<SDockTab> SpawnDetailsTab(const FSpawnTabArgs& Args);

	/** Creates a tab with the root actor properties panel in it */
	TSharedRef<SDockTab> SpawnRootActorPanelTab(const FSpawnTabArgs& Args);

	/** Raised when something wants to display a list of objects in the operator's details panel */
	void DisplayObjectsInDetailsPanel(const TArray<UObject*>& Objects);

	/** Called when app is unregistered */
	void OnOperatorAppUnregistered(const FDelegateHandle& InHandle);
	
private:
	/** Holds the tab manager that manages the operator's tabs. */
	TSharedPtr<FTabManager> TabManager;

	/** The layout extender used by the operator's layout */
	TSharedPtr<FLayoutExtender> LayoutExtender;

	/** The container for the toolbar */
	TSharedPtr<SBox> ToolbarContainer;
	
	/** A reference to the operator panel's toolbar widget */
	TSharedPtr<SDisplayClusterOperatorToolbar> Toolbar;

	/** A reference to the operator panel's details view */
	TSharedPtr<class SKismetInspector> DetailsView;

	/** A reference to the operator panel's root actor panel */
	TSharedPtr<SDisplayClusterOperatorRootActorPanel> RootActorPanel;

	/** A reference to the operator panel's status bar widget */
	TSharedPtr<SDisplayClusterOperatorStatusBar> StatusBar;

	/** A command list for processing commands for the window */
	TSharedPtr<FUICommandList> CommandList;

	/** The delegate handle for the operator module's OnDetailObjectsChanged event */
	FDelegateHandle DetailObjectsChangedHandle;

	/** Loaded app instances */
	TMap<FDelegateHandle, TSharedRef<IDisplayClusterOperatorApp>> AppInstances;
};