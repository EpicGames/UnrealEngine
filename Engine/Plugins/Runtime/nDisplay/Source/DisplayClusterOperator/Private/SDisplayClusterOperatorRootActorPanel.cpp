// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDisplayClusterOperatorRootActorPanel.h"

#include "DisplayClusterRootActor.h"
#include "Editor/UnrealEdEngine.h"
#include "IDisplayClusterOperator.h"
#include "IDisplayClusterOperatorViewModel.h"
#include "LevelEditor.h"
#include "PropertyEditorModule.h"
#include "SlateOptMacros.h"
#include "SSubobjectEditorModule.h"
#include "SSubobjectInstanceEditor.h"
#include "UnrealEdGlobals.h"
#include "Widgets/Text/SRichTextBlock.h"

#define LOCTEXT_NAMESPACE "SDisplayClusterOperatorRootActorPanel"

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

namespace UE::DisplayClusterOperatorRootActorPanel
{
	/** A widget to display a warning message */
	class SWarningLabel : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SWarningLabel)
			: _WarningText()
			, _OnHyperlinkClicked()
		{}
			
			/** The rich text to show in the warning */
			SLATE_ATTRIBUTE(FText, WarningText)

			/** Called when the hyperlink in the rich text is clicked */
			SLATE_EVENT(FSlateHyperlinkRun::FOnClick, OnHyperlinkClicked)

		SLATE_END_ARGS()

		/** Constructs the widget */
		void Construct(const FArguments& InArgs)
		{
			ChildSlot
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Padding(2)
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.Warning"))
					]
					+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.Padding(2)
						[
							SNew(SRichTextBlock)
							.DecoratorStyleSet(&FAppStyle::Get())
							.Justification(ETextJustify::Left)
							.TextStyle(FAppStyle::Get(), "DetailsView.BPMessageTextStyle")
							.Text(InArgs._WarningText)
							.AutoWrapText(true)
							+ SRichTextBlock::HyperlinkDecorator(TEXT("HyperlinkDecorator"), InArgs._OnHyperlinkClicked)
						]
				]
			];
		}
	};
}

SDisplayClusterOperatorRootActorPanel::~SDisplayClusterOperatorRootActorPanel()
{
	if (GEditor)
	{
		GEditor->UnregisterForUndo(this);
	}

	FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);

	if (FLevelEditorModule* LevelEditor = FModuleManager::GetModulePtr<FLevelEditorModule>("LevelEditor"))
	{
		LevelEditor->OnComponentsEdited().RemoveAll(this);
	}

	if (OperatorViewModel.IsValid())
	{
		OperatorViewModel->OnActiveRootActorChanged().RemoveAll(this);
	}
}

void SDisplayClusterOperatorRootActorPanel::Construct(const FArguments& InArgs, TSharedPtr<FTabManager> InTabManager, const FName& TabIdentifier)
{
	GEditor->RegisterForUndo(this);

	FCoreUObjectDelegates::OnObjectsReplaced.AddRaw(this, &SDisplayClusterOperatorRootActorPanel::OnObjectsReplaced);

	FLevelEditorModule& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	LevelEditor.OnComponentsEdited().AddSP(this, &SDisplayClusterOperatorRootActorPanel::OnComponentsEdited);

	OperatorViewModel = IDisplayClusterOperator::Get().GetOperatorViewModel();
	OperatorViewModel->OnActiveRootActorChanged().AddSP(this, &SDisplayClusterOperatorRootActorPanel::OnRootActorChanged);

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bUpdatesFromSelection = true;
	DetailsViewArgs.bLockable = true;
	DetailsViewArgs.bAllowFavoriteSystem = true;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::ObjectsUseNameArea | FDetailsViewArgs::ComponentsAndActorsUseNameArea;
	DetailsViewArgs.NotifyHook = GUnrealEd;
	DetailsViewArgs.ViewIdentifier = TabIdentifier;
	DetailsViewArgs.bCustomNameAreaLocation = true;
	DetailsViewArgs.bCustomFilterAreaLocation = true;
	DetailsViewArgs.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Hide;
	DetailsViewArgs.HostTabManager = InTabManager;
	DetailsViewArgs.bShowSectionSelector = true;

	FPropertyEditorModule& PropertyEditor = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	DetailsView = PropertyEditor.CreateDetailView(DetailsViewArgs);

	DetailsView->SetIsPropertyVisibleDelegate(FIsPropertyVisible::CreateSP(this, &SDisplayClusterOperatorRootActorPanel::IsPropertyVisible));
	DetailsView->SetIsPropertyReadOnlyDelegate(FIsPropertyReadOnly::CreateSP(this, &SDisplayClusterOperatorRootActorPanel::IsPropertyReadOnly));
	DetailsView->SetIsPropertyEditingEnabledDelegate(FIsPropertyEditingEnabled::CreateSP(this, &SDisplayClusterOperatorRootActorPanel::IsPropertyEditingEnabled));

	FModuleManager::LoadModuleChecked<FSubobjectEditorModule>("SubobjectEditor");

	SubobjectEditor = SNew(SSubobjectInstanceEditor)
		.ObjectContext(this, &SDisplayClusterOperatorRootActorPanel::GetRootActorContextObject)
		.AllowEditing(this, &SDisplayClusterOperatorRootActorPanel::CanEditRootActorComponents)
		.OnSelectionUpdated(this, &SDisplayClusterOperatorRootActorPanel::OnSelectedSubobjectsChanged);

	TSharedRef<SWidget> SubobjectButtonsBox = SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SubobjectEditor->GetToolButtonsBox().ToSharedRef()
		];

	DetailsView->SetNameAreaCustomContent(SubobjectButtonsBox);

	ChildSlot
	[
		SNew(SVerticalBox)

		+SVerticalBox::Slot()
		.Padding(10.f, 4.f, 0.f, 0.f)
		.AutoHeight()
		[
			DetailsView->GetNameAreaWidget().ToSharedRef()
		]

		+SVerticalBox::Slot()
		[
			SNew(SSplitter)
			.MinimumSlotHeight(80.0f)
			.Orientation(Orient_Vertical)
			.Style(FAppStyle::Get(), "SplitterDark")
			.PhysicalSplitterHandleSize(2.0f)

			+SSplitter::Slot()
			.Value(0.2f)
			[
				SNew(SBox)
				.Padding(FMargin(2.0f, 0.0f, 2.0f, 0.0f))
				.Visibility(this, &SDisplayClusterOperatorRootActorPanel::GetSubobjectEditorVisibility)
				[
					SubobjectEditor.ToSharedRef()
				]
			]

			+SSplitter::Slot()
			[
				SNew( SVerticalBox )
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding( FMargin(0,0,0,1) )
				[
					SNew(UE::DisplayClusterOperatorRootActorPanel::SWarningLabel)
					.Visibility(this, &SDisplayClusterOperatorRootActorPanel::GetUCSComponentWarningVisibility)
					.WarningText(LOCTEXT("BlueprintUCSComponentWarning", "Components created by the User Construction Script can only be edited in the <a id=\"HyperlinkDecorator\" style=\"DetailsView.BPMessageHyperlinkStyle\">Blueprint</>"))
					.OnHyperlinkClicked(this, &SDisplayClusterOperatorRootActorPanel::OnComponentBlueprintHyperlinkClicked)
				]
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding( FMargin(0,0,0,1) )
				[
					SNew(UE::DisplayClusterOperatorRootActorPanel::SWarningLabel)
					.Visibility(this, &SDisplayClusterOperatorRootActorPanel::GetInheritedComponentWarningVisibility)
					.WarningText(LOCTEXT("BlueprintUneditableInheritedComponentWarning", "Components flagged as not editable when inherited must be edited in the <a id=\"HyperlinkDecorator\" style=\"DetailsView.BPMessageHyperlinkStyle\">Blueprint</>"))
					.OnHyperlinkClicked(this, &SDisplayClusterOperatorRootActorPanel::OnComponentBlueprintHyperlinkClicked)
				]
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding( FMargin(0,0,0,1) )
				[
					SNew(UE::DisplayClusterOperatorRootActorPanel::SWarningLabel)
					.Visibility(this, &SDisplayClusterOperatorRootActorPanel::GetNativeComponentWarningVisibility)
					.WarningText(LOCTEXT("UneditableNativeComponentWarning", "Native components are editable when declared as a FProperty in <a id=\"HyperlinkDecorator\" style=\"DetailsView.BPMessageHyperlinkStyle\">C++</>"))
					.OnHyperlinkClicked(this, &SDisplayClusterOperatorRootActorPanel::OnComponentBlueprintHyperlinkClicked)
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					DetailsView->GetFilterAreaWidget().ToSharedRef()
				]
				+ SVerticalBox::Slot()
				[
					DetailsView.ToSharedRef()
				]
			]
		]
	];

	if (OperatorViewModel->GetRootActor())
	{
		DetailsView->SetObject(OperatorViewModel->GetRootActor());
	}
	else
	{
		// Force the details view to clear the set objects. This ensures the "Select an object to show details" message properly displays
		DetailsView->SetObjects(TArray<UObject*>());
	}
}

void SDisplayClusterOperatorRootActorPanel::PostUndo(bool bSuccess)
{
	if (SubobjectEditor.IsValid())
	{
		SubobjectEditor->UpdateTree();
	}

	if (DetailsView.IsValid())
	{
		// As long as there is a valid root actor, the details view should have a selected object to display
		// In the case where the deletion of a root actor was undone, the details view will not remember its
		// last selected object, so attempt to sync with the subobject editor
		if (!DetailsView->GetSelectedObjects().Num())
		{
			// Make sure the details view is still synced with the subobject editor selection after the undo. If there are no
			// selected subobjects, set the root actor as the displayed object in the details view
			TArray<FSubobjectEditorTreeNodePtrType> SelectedSubobjects = SubobjectEditor->GetSelectedNodes();
			if (SelectedSubobjects.Num())
			{
				OnSelectedSubobjectsChanged(SubobjectEditor->GetSelectedNodes());
			}
			else if (OperatorViewModel->GetRootActor())
			{
				DetailsView->SetObject(OperatorViewModel->GetRootActor());
			}
		}
	}
}

void SDisplayClusterOperatorRootActorPanel::PostRedo(bool bSuccess)
{
	PostUndo(bSuccess);
}

UObject* SDisplayClusterOperatorRootActorPanel::GetRootActorContextObject() const
{
	if (OperatorViewModel.IsValid())
	{
		return OperatorViewModel->GetRootActor();
	}

	return nullptr;
}

EVisibility SDisplayClusterOperatorRootActorPanel::GetSubobjectEditorVisibility() const
{
	return OperatorViewModel->HasRootActor() ? EVisibility::Visible : EVisibility::Collapsed;
}

bool SDisplayClusterOperatorRootActorPanel::CanEditRootActorComponents() const
{
	return GEditor->PlayWorld == nullptr;
}

bool SDisplayClusterOperatorRootActorPanel::IsPropertyVisible(const FPropertyAndParent& PropertyAndParent) const
{
	if (PropertyAndParent.Property.HasAllPropertyFlags(CPF_DisableEditOnInstance))
	{
		return false;
	}

	return true;
}

bool SDisplayClusterOperatorRootActorPanel::IsPropertyReadOnly(const FPropertyAndParent& PropertyAndParent) const
{
	bool bIsReadOnly = false;
	for (const FSubobjectEditorTreeNodePtrType& Node : SubobjectEditor->GetSelectedNodes())
	{
		const UActorComponent* Component = Node->GetComponentTemplate();
		if (Component && Component->CreationMethod == EComponentCreationMethod::SimpleConstructionScript)
		{
			TSet<const FProperty*> UCSModifiedProperties;
			Component->GetUCSModifiedProperties(UCSModifiedProperties);
			if (UCSModifiedProperties.Contains(&PropertyAndParent.Property) ||
				(PropertyAndParent.ParentProperties.Num() > 0 && UCSModifiedProperties.Contains(PropertyAndParent.ParentProperties[0])))
			{
				bIsReadOnly = true;
				break;
			}
		}
	}
	return bIsReadOnly;
}

bool SDisplayClusterOperatorRootActorPanel::IsPropertyEditingEnabled() const
{
	FLevelEditorModule& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	if (!LevelEditor.AreObjectsEditable(DetailsView->GetSelectedObjects()))
	{
		return false;
	}

	bool bIsEditable = true;
	for (const FSubobjectEditorTreeNodePtrType& Node : SubobjectEditor->GetSelectedNodes())
	{
		if (const FSubobjectData* Data = Node->GetDataSource())
		{
			bIsEditable = Data->CanEdit();
			if (!bIsEditable)
			{
				break;
			}
		}
	}
	return bIsEditable;
}

bool SDisplayClusterOperatorRootActorPanel::IsBlueprintNotEditable(const UActorComponent* Component) const
{
	// Determine if it is locked out from a blueprint or from the native
	UActorComponent* Archetype = CastChecked<UActorComponent>(Component->GetArchetype());
	while (Archetype)
	{
		if (Archetype->GetOuter()->IsA<UBlueprintGeneratedClass>() || Archetype->GetOuter()->GetClass()->HasAllClassFlags(CLASS_CompiledFromBlueprint))
		{
			if (!Archetype->bEditableWhenInherited)
			{
				return true;
			}

			Archetype = CastChecked<UActorComponent>(Archetype->GetArchetype());
		}
		else
		{
			Archetype = nullptr;
		}
	}

	return false;
}

EVisibility SDisplayClusterOperatorRootActorPanel::GetUCSComponentWarningVisibility() const
{
	bool bIsUneditableBlueprintComponent = false;

	// Check to see if any selected components are inherited from blueprint
	for (const FSubobjectEditorTreeNodePtrType& Node : SubobjectEditor->GetSelectedNodes())
	{
		if (const FSubobjectData* Data = Node->GetDataSource())
		{
			if (!Data->IsNativeComponent())
			{
				const UActorComponent* Component = Data->GetComponentTemplate();
				bIsUneditableBlueprintComponent = Component ? Component->CreationMethod == EComponentCreationMethod::UserConstructionScript : false;
				if (bIsUneditableBlueprintComponent)
				{
					break;
				}
			}
		}

	}

	return bIsUneditableBlueprintComponent ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SDisplayClusterOperatorRootActorPanel::GetInheritedComponentWarningVisibility() const
{
	bool bIsUneditableBlueprintComponent = false;

	// Check to see if any selected components are inherited from blueprint
	for (const FSubobjectEditorTreeNodePtrType& Node : SubobjectEditor->GetSelectedNodes())
	{
		if (const FSubobjectData* Data = Node->GetDataSource())
		{
			if (!Data->IsNativeComponent())
			{
				if (const UActorComponent* Component = Data->GetComponentTemplate())
				{
					if (!Component->IsEditableWhenInherited() && Component->CreationMethod == EComponentCreationMethod::SimpleConstructionScript)
					{
						bIsUneditableBlueprintComponent = true;
						break;
					}
				}
			}
			else if (!Data->CanEdit() && IsBlueprintNotEditable(Data->GetComponentTemplate()))
			{
				bIsUneditableBlueprintComponent = true;
				break;
			}
		}
	}

	return bIsUneditableBlueprintComponent ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SDisplayClusterOperatorRootActorPanel::GetNativeComponentWarningVisibility() const
{
	bool bIsUneditableNative = false;
	for (const FSubobjectEditorTreeNodePtrType& Node : SubobjectEditor->GetSelectedNodes())
	{
		if (const FSubobjectData* Data = Node->GetDataSource())
		{
			// Check to see if the component is native and not editable
			if (Data->IsNativeComponent() && !Data->CanEdit() && !IsBlueprintNotEditable(Data->GetComponentTemplate()))
			{
				bIsUneditableNative = true;
				break;
			}
		}

	}

	return bIsUneditableNative ? EVisibility::Visible : EVisibility::Collapsed;
}

void SDisplayClusterOperatorRootActorPanel::OnComponentBlueprintHyperlinkClicked(const FSlateHyperlinkRun::FMetadata& Metadata)
{
	UBlueprint* Blueprint = SubobjectEditor->GetBlueprint();
	if (Blueprint)
	{
		// Open the blueprint
		GEditor->EditObject(Blueprint);
	}
}

void SDisplayClusterOperatorRootActorPanel::OnRootActorChanged(ADisplayClusterRootActor* RootActor)
{
	if (DetailsView.IsValid())
	{
		if (RootActor)
		{
			DetailsView->SetObject(RootActor);
		}
		else
		{
			// Force the details view to clear the set objects. This ensures the "Select an object to show details" message properly displays
			DetailsView->SetObjects(TArray<UObject*>());
		}
	}

	if (SubobjectEditor.IsValid())
	{
		SubobjectEditor->UpdateTree();
	}
}

void SDisplayClusterOperatorRootActorPanel::OnSelectedSubobjectsChanged(const TArray<TSharedPtr<FSubobjectEditorTreeNode>>& SelectedNodes)
{
	if (SelectedNodes.Num() == 0)
	{
		return;
	}

	if (!DetailsView->IsLocked())
	{
		auto IsRootActor = [](const FSubobjectEditorTreeNodePtrType& Node)
		{
			if (Node.IsValid())
			{
				if (FSubobjectData* Data = Node->GetDataSource())
				{
					return Data->IsRootActor();
				}
			}

			return false;
		};

		if (SelectedNodes.ContainsByPredicate(IsRootActor))
		{
			DetailsView->SetObject(OperatorViewModel->GetRootActor());
		}
		else
		{
			TArray<UObject*> Components;
			for (const FSubobjectEditorTreeNodePtrType& Node : SelectedNodes)
			{
				if (Node.IsValid())
				{
					if (FSubobjectData* Data = Node->GetDataSource())
					{
						if (Data->IsComponent())
						{
							if (const UActorComponent* Component = Data->FindComponentInstanceInActor(OperatorViewModel->GetRootActor()))
							{
								Components.Add(const_cast<UActorComponent*>(Component));
							}
						}
					}
				}
			}

			DetailsView->SetObjects(Components);
		}
	}
}

void SDisplayClusterOperatorRootActorPanel::OnComponentsEdited()
{
	SubobjectEditor->UpdateTree();
	DetailsView->ForceRefresh();
}

void SDisplayClusterOperatorRootActorPanel::OnObjectsReplaced(const TMap<UObject*, UObject*>& InReplacementObjects)
{
	SubobjectEditor->UpdateTree();
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

#undef LOCTEXT_NAMESPACE