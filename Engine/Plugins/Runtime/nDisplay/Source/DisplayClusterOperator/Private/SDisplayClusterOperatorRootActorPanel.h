// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorUndoClient.h"
#include "Framework/Text/SlateHyperlinkRun.h"
#include "Widgets/SCompoundWidget.h"

struct FPropertyAndParent;
class ADisplayClusterRootActor;
class FSubobjectEditorTreeNode;
class FTabManager;
class IDisplayClusterOperatorViewModel;
class IDetailsView;
class SSubobjectEditor;
class UActorComponent;

/**
 * A panel that mimics the appearance of SActorDetails to display the root actor's components and properties in the operator panel
 */
class DISPLAYCLUSTEROPERATOR_API SDisplayClusterOperatorRootActorPanel : public SCompoundWidget, public FEditorUndoClient
{
public:
	virtual ~SDisplayClusterOperatorRootActorPanel() override;

	SLATE_BEGIN_ARGS(SDisplayClusterOperatorRootActorPanel) {}
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, TSharedPtr<FTabManager> InTabManager, const FName& TabIdentifier);

	//~ FEditorUndoClient Interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;

private:
	/** Gets the root actor displayed by the subobject editor and the details panel */
	UObject* GetRootActorContextObject() const;

	/** Gets the visibility of the subobject editor */
	EVisibility GetSubobjectEditorVisibility() const;

	/** Gets whether the root actor's components can be edited */
	bool CanEditRootActorComponents() const;

	/** Determines if a property is visible or not */
	bool IsPropertyVisible(const FPropertyAndParent& PropertyAndParent) const;

	/** Determines if a property is read only or not */
	bool IsPropertyReadOnly(const FPropertyAndParent& PropertyAndParent) const;

	/** Determines if property editing is enabled or not */
	bool IsPropertyEditingEnabled() const;

	/** Determines if the component is set as not editable through its blueprint */
	bool IsBlueprintNotEditable(const UActorComponent* Component) const;

	/** Gets the visibility of the warning label for user compile script generated components */
	EVisibility GetUCSComponentWarningVisibility() const;

	/** Gets the visibility of the warning label for inherited blueprint components */
	EVisibility GetInheritedComponentWarningVisibility() const;

	/** Gets the visibility of the warning label for native components */
	EVisibility GetNativeComponentWarningVisibility() const;

	/** Raised when a hyperlink is clicked in the warning labels for components */
	void OnComponentBlueprintHyperlinkClicked(const FSlateHyperlinkRun::FMetadata& Metadata);

	/** Raised when the operator panel's active root actor has been changed */
	void OnRootActorChanged(ADisplayClusterRootActor* RootActor);

	/** Raised when the selection in the subobject editor is changed */
	void OnSelectedSubobjectsChanged(const TArray<TSharedPtr<FSubobjectEditorTreeNode>>& SelectedNodes);

	/** Raised by the level editor when any components in the world have been edited */
	void OnComponentsEdited();

	/** Raised when any objects in the editor are being destroyed and replaced with new instances */
	void OnObjectsReplaced(const TMap<UObject*, UObject*>& InReplacementObjects);

private:
	/** Subobject editor used to display the components of the root actor */
	TSharedPtr<SSubobjectEditor> SubobjectEditor;

	/** Details view widget used to display the actor's properties */
	TSharedPtr<IDetailsView> DetailsView;

	/** A reference to the operator panel's view model */
	TSharedPtr<IDisplayClusterOperatorViewModel> OperatorViewModel;
};
