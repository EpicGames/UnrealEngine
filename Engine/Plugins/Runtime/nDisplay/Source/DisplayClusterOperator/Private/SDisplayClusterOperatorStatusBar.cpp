// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDisplayClusterOperatorStatusBar.h"

#include "Framework/Application/SlateApplication.h"
#include "SWidgetDrawer.h"
#include "Widgets/Layout/SBox.h"

const FName SDisplayClusterOperatorStatusBar::StatusBarId = FName("DisplayClusterOperatorStatusBar");

void SDisplayClusterOperatorStatusBar::Construct(const FArguments& InArgs)
{
    ChildSlot
	[
		SNew(SBox)
		.HeightOverride(FAppStyle::Get().GetFloat("StatusBar.Height"))
		[
			SAssignNew(WidgetDrawer, SWidgetDrawer, StatusBarId)
		]
	];
}

void SDisplayClusterOperatorStatusBar::RegisterDrawer(FWidgetDrawerConfig&& Drawer, int32 SlotIndex)
{
	WidgetDrawer->RegisterDrawer(MoveTemp(Drawer), SlotIndex);
}

bool SDisplayClusterOperatorStatusBar::IsDrawerOpened(const FName DrawerId) const
{
	return WidgetDrawer->IsDrawerOpened(DrawerId);
}

void SDisplayClusterOperatorStatusBar::OpenDrawer(const FName DrawerId)
{
	WidgetDrawer->OpenDrawer(DrawerId);
}

bool SDisplayClusterOperatorStatusBar::DismissDrawer(const TSharedPtr<SWidget>& NewlyFocusedWidget)
{
	return WidgetDrawer->DismissDrawer(NewlyFocusedWidget);
}