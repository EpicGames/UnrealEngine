// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class SWidgetDrawer;
struct FWidgetDrawerConfig;

class SDisplayClusterOperatorStatusBar : public SCompoundWidget
{
public:
	static const FName StatusBarId;

public:
	SLATE_BEGIN_ARGS(SDisplayClusterOperatorStatusBar)
	{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	//~ Begin SWidget Interface
	virtual bool SupportsKeyboardFocus() const override { return false; }
	//~ End SWidget Interface

	/** Registers a new drawer with this status bar. Registering will add a button to open and close the drawer */
	void RegisterDrawer(FWidgetDrawerConfig&& Drawer, int32 SlotIndex = INDEX_NONE);

	/** Gets whether a drawer with the specified ID is currently open */
	bool IsDrawerOpened(const FName DrawerId) const;

	/** Opens a drawer with the specified ID */
	void OpenDrawer(const FName DrawerId);

	/**
	 * Dismisses an open drawer with an animation.  The drawer contents are removed once the animation is complete
	 *
	 * @param NewlyFocusedWidget Optional widget to focus
	 * @return true if any open drawer was dismissed
	 */
	bool DismissDrawer(const TSharedPtr<SWidget>& NewlyFocusedWidget);

private:
	TSharedPtr<SWidgetDrawer> WidgetDrawer;
};