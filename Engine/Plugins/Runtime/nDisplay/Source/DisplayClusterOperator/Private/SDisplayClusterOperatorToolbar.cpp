// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDisplayClusterOperatorToolbar.h"

#include "Engine/Blueprint.h"
#include "IDisplayClusterOperator.h"
#include "IDisplayClusterOperatorViewModel.h"
#include "DisplayClusterRootActor.h"

#include "Styling/AppStyle.h"
#include "LevelEditor.h"
#include "LevelEditorActions.h"
#include "EditorSupportDelegates.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Misc/TransactionObjectEvent.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Input/STextComboBox.h"

#define LOCTEXT_NAMESPACE "SDisplayClusterOperatorToolbar"

SDisplayClusterOperatorToolbar::~SDisplayClusterOperatorToolbar()
{
	if (LevelActorDeletedHandle.IsValid() && GEngine != nullptr)
	{
		GEngine->OnLevelActorDeleted().Remove(LevelActorDeletedHandle);
	}

	if (MapChangedHandle.IsValid())
	{
		FLevelEditorModule& LevelEditor = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
		LevelEditor.OnMapChanged().Remove(MapChangedHandle);
	}

	if (OnObjectTransactedHandle.IsValid())
	{
		FCoreUObjectDelegates::OnObjectTransacted.Remove(OnObjectTransactedHandle);
	}
	
	FEditorSupportDelegates::PrepareToCleanseEditorObject.RemoveAll(this);

	if (ADisplayClusterRootActor* ActiveRootActor = ViewModel->GetRootActor())
	{
		if (UBlueprint* Blueprint = UBlueprint::GetBlueprintFromClass(ActiveRootActor->GetClass()))
		{
			Blueprint->OnCompiled().RemoveAll(this);
		}
	}
}

void SDisplayClusterOperatorToolbar::Construct(const FArguments& InArgs)
{
	ViewModel = IDisplayClusterOperator::Get().GetOperatorViewModel();
	CommandList = InArgs._CommandList;

	FillRootActorList();

	SAssignNew(RootActorComboBox, SComboBox<TSharedPtr<FString>>)
		.OptionsSource(&RootActorList)
		.OnSelectionChanged(this, &SDisplayClusterOperatorToolbar::OnRootActorChanged)
		.OnComboBoxOpening(this, &SDisplayClusterOperatorToolbar::OnRootActorComboBoxOpening)
		.OnGenerateWidget(this, &SDisplayClusterOperatorToolbar::GenerateRootActorComboBoxWidget)
		.Content()
		[
			SNew(STextBlock)
			.Text(this, &SDisplayClusterOperatorToolbar::GetRootActorComboBoxText)
		];

	const TSharedPtr<FExtender> ToolBarExtender = IDisplayClusterOperator::Get().GetOperatorToolBarExtensibilityManager()->GetAllExtenders();
	FSlimHorizontalToolBarBuilder ToolBarBuilder(CommandList, FMultiBoxCustomization::None, ToolBarExtender);
	ToolBarBuilder.SetStyle(&FAppStyle::Get(), "AssetEditorToolbar");

	const TSharedPtr<SHorizontalBox> RootActorSelectionBox = SNew(SHorizontalBox)
	+SHorizontalBox::Slot()
	.AutoWidth()
	.VAlign(VAlign_Center)
	.Padding(5.f, 0.f)
	[
		SNew(STextBlock)
		.Text(LOCTEXT("RootActorPickerLabel", "nDisplay Configuration"))
	]
	+SHorizontalBox::Slot()
	.AutoWidth()
	[
		RootActorComboBox.ToSharedRef()
	];
	
	ToolBarBuilder.BeginSection("General");
	{
		// Add a generic save level option. If we ever add other types of apps that don't rely on the level
		// instance being saved we may want to refactor this so the save option is added per app instead.
		const FLevelEditorModule& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
		ToolBarBuilder.AddToolBarButton(LevelEditor.GetLevelEditorCommands().Save);
		
		ToolBarBuilder.AddSeparator();
		
		ToolBarBuilder.AddWidget(RootActorSelectionBox.ToSharedRef());
	}
	ToolBarBuilder.EndSection();

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		.Padding(0.0f)
		[
			ToolBarBuilder.MakeWidget()
		]
	];
	
	if (RootActorList.Num())
	{
		RootActorComboBox->SetSelectedItem(RootActorList[0]);
	}

	if (GEngine != nullptr)
	{
		LevelActorDeletedHandle = GEngine->OnLevelActorDeleted().AddSP(this, &SDisplayClusterOperatorToolbar::OnLevelActorDeleted);
	}

	FLevelEditorModule& LevelEditor = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	MapChangedHandle = LevelEditor.OnMapChanged().AddRaw(this, &SDisplayClusterOperatorToolbar::HandleMapChanged);

	FEditorSupportDelegates::PrepareToCleanseEditorObject.AddSP(this, &SDisplayClusterOperatorToolbar::OnPrepareToCleanseEditorObject);

	OnObjectTransactedHandle = FCoreUObjectDelegates::OnObjectTransacted.AddSP(this, &SDisplayClusterOperatorToolbar::OnObjectTransacted);
}

TSharedPtr<FString> SDisplayClusterOperatorToolbar::FillRootActorList(const FString& InitiallySelectedRootActor)
{
	RootActorList.Empty();

	TArray<ADisplayClusterRootActor*> RootActors;
	IDisplayClusterOperator::Get().GetRootActorLevelInstances(RootActors);

	TSharedPtr<FString> SelectedItem = nullptr;
	for (ADisplayClusterRootActor* RootActor : RootActors)
	{
		TSharedPtr<FString> ActorName = MakeShared<FString>(RootActor->GetActorNameOrLabel());
		if (InitiallySelectedRootActor == *ActorName)
		{
			SelectedItem = ActorName;
		}

		RootActorList.Add(ActorName);
	}

	return SelectedItem;
}

void SDisplayClusterOperatorToolbar::ClearSelectedRootActor()
{
	if (ADisplayClusterRootActor* ActiveRootActor = ViewModel->GetRootActor(/*bEvenIfPendingKill*/ true))
	{
		if (UBlueprint* Blueprint = UBlueprint::GetBlueprintFromClass(ActiveRootActor->GetClass()))
		{
			Blueprint->OnCompiled().RemoveAll(this);
		}
	}

	ActiveRootActorName.Reset();
	ViewModel->SetRootActor(nullptr);
	RootActorComboBox->SetSelectedItem(nullptr);
}

void SDisplayClusterOperatorToolbar::OnRootActorChanged(TSharedPtr<FString> ItemSelected, ESelectInfo::Type SelectInfo)
{
	ActiveRootActorName = ItemSelected;
	if (!ItemSelected.IsValid())
	{
		return;
	}

	TArray<ADisplayClusterRootActor*> RootActors;
	IDisplayClusterOperator::Get().GetRootActorLevelInstances(RootActors);

	ADisplayClusterRootActor* SelectedRootActor = nullptr;
	for (ADisplayClusterRootActor* RootActor : RootActors)
	{
		if (RootActor->GetActorNameOrLabel() == *ItemSelected)
		{
			SelectedRootActor = RootActor;
			break;
		}
	}

	if (ADisplayClusterRootActor* ActiveRootActor = ViewModel->GetRootActor())
	{
		if (UBlueprint* Blueprint = UBlueprint::GetBlueprintFromClass(ActiveRootActor->GetClass()))
		{
			Blueprint->OnCompiled().RemoveAll(this);
		}
	}
	
	if (SelectedRootActor)
	{
		if (UBlueprint* Blueprint = UBlueprint::GetBlueprintFromClass(SelectedRootActor->GetClass()))
		{
			Blueprint->OnCompiled().RemoveAll(this);
			Blueprint->OnCompiled().AddSP(this, &SDisplayClusterOperatorToolbar::OnBlueprintCompiled);
		}
	}

	ViewModel->SetRootActor(SelectedRootActor);
}

void SDisplayClusterOperatorToolbar::OnBlueprintCompiled(UBlueprint* Blueprint)
{
	if (RootActorComboBox.IsValid() && ActiveRootActorName.IsValid())
	{
		// Compiling the blueprint invalidates the instance so we need to find and set the instance again.
		RootActorComboBox->SetSelectedItem(ActiveRootActorName);
		OnRootActorChanged(ActiveRootActorName, ESelectInfo::Direct);
	}
}

void SDisplayClusterOperatorToolbar::OnRootActorComboBoxOpening()
{
	FString SelectedRootActor = "";
	if (TSharedPtr<FString> SelectedItem = RootActorComboBox->GetSelectedItem())
	{
		SelectedRootActor = *SelectedItem;
	}

	TSharedPtr<FString> NewSelectedItem = FillRootActorList(SelectedRootActor);

	RootActorComboBox->RefreshOptions();
	RootActorComboBox->SetSelectedItem(NewSelectedItem);
}

TSharedRef<SWidget> SDisplayClusterOperatorToolbar::GenerateRootActorComboBoxWidget(TSharedPtr<FString> InItem) const
{
	return SNew(STextBlock).Text(FText::FromString(*InItem.Get()));
}

FText SDisplayClusterOperatorToolbar::GetRootActorComboBoxText() const
{
	TSharedPtr<FString> SelectedRootActor = RootActorComboBox->GetSelectedItem();

	if (!SelectedRootActor.IsValid())
	{
		return LOCTEXT("NoRootActorSelectedLabel", "No nDisplay Actor Selected");
	}
	
	return FText::FromString(*RootActorComboBox->GetSelectedItem());
}

void SDisplayClusterOperatorToolbar::OnLevelActorDeleted(AActor* Actor)
{
	if (Actor == ViewModel->GetRootActor(/*bEvenIfPendingKill*/ true))
	{
		if (Actor && Actor->GetClass()->HasAnyClassFlags(CLASS_NewerVersionExists))
		{
			// When a blueprint class is regenerated instances are deleted and replaced.
			// In this case the OnCompiled() delegate will fire and refresh the actor.
			return;
		}
		
		ClearSelectedRootActor();
	}
}

void SDisplayClusterOperatorToolbar::HandleMapChanged(UWorld* InWorld, EMapChangeType InMapChangeType)
{
	if (InMapChangeType == EMapChangeType::TearDownWorld &&
		(!ViewModel->HasRootActor() || ViewModel->GetRootActor()->GetWorld() == InWorld))
	{
		ClearSelectedRootActor();
	}
}

void SDisplayClusterOperatorToolbar::OnPrepareToCleanseEditorObject(UObject* Object)
{
	if (ULevel* Level = Cast<ULevel>(Object))
	{
		const bool bEvenIfPendingKill = true;
		if (Level->Actors.Contains(ViewModel->GetRootActor(bEvenIfPendingKill)))
		{
			// A level that contains the current root actor is being purged, so clear the root actor to ensure
			// no UI elements continue to reference the root actor
			ClearSelectedRootActor();
		}
	}
}

void SDisplayClusterOperatorToolbar::OnObjectTransacted(UObject* Object, const FTransactionObjectEvent& TransactionObjectEvent)
{
	if (TransactionObjectEvent.GetEventType() == ETransactionObjectEventType::UndoRedo
		&& TransactionObjectEvent.HasPendingKillChange()
		&& Object == ViewModel->GetRootActor(/*bEvenIfPendingKill*/ true))
	{
		// In this case the user has redone a deletion, and we need to update appropriately.
		ClearSelectedRootActor();
	}
}

#undef LOCTEXT_NAMESPACE
