// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "UObject/WeakObjectPtr.h"

enum class EMapChangeType : uint8;
class FUICommandList;
class IDisplayClusterOperatorViewModel;
class ADisplayClusterRootActor;
class FTransactionObjectEvent;

template<class T>
class SComboBox;

/** A toolbar widget used by the nDisplay operator panel */
class SDisplayClusterOperatorToolbar : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDisplayClusterOperatorToolbar)
	{}
		SLATE_ARGUMENT(TSharedPtr<FUICommandList>, CommandList)
	SLATE_END_ARGS()

	~SDisplayClusterOperatorToolbar();

	void Construct(const FArguments& InArgs);

private:
	/** 
	 * Constructs the list of root actors that exist on the current level to use for the root actor picker dropdown.
	 * 
	 * @param InitiallySelectedRootActor - The actor name of the root actor that should be initially selected in the combo box
	 * @eturns The pointer to the item in the root actor list to pass to the combo box as the initially selected item
	 */
	TSharedPtr<FString> FillRootActorList(const FString& InitiallySelectedRootActor = TEXT(""));

	/** Deselect and clear the current root actor */
	void ClearSelectedRootActor();
	
	/** Raised when the user selects a new root actor from the root actor picker dropdown */
	void OnRootActorChanged(TSharedPtr<FString> ItemSelected, ESelectInfo::Type SelectInfo);

	/** When the root actor blueprint has been compiled */
	void OnBlueprintCompiled(UBlueprint* Blueprint);

	/** Raised when the root actor picker dropdown is being opened */
	void OnRootActorComboBoxOpening();

	/** Creates the widget to display for the specified dropdown item */
	TSharedRef<SWidget> GenerateRootActorComboBoxWidget(TSharedPtr<FString> InItem) const;

	/** Gets the text to display in the combo box for the selected root actor */
	FText GetRootActorComboBoxText() const;

	/** Raised when the user deletes an actor from the level */
	void OnLevelActorDeleted(AActor* Actor);

	/** When the map changes */
	void HandleMapChanged(UWorld* InWorld, EMapChangeType InMapChangeType);

	/** Raised when an editor object is going to be cleansed, usually used when a sublevel is about to be removed */
	void OnPrepareToCleanseEditorObject(UObject* Object);
	
	/** Raised when any object is transacted */
	void OnObjectTransacted(UObject* Object, const FTransactionObjectEvent& TransactionObjectEvent);

private:
	/** A reference the the operator panel's view model, which stores the operator panel state */
	TSharedPtr<IDisplayClusterOperatorViewModel> ViewModel;

	/** The command list used by the toolbar */
	TSharedPtr<FUICommandList> CommandList;
	
	/** The list of root actor names on the current level to display in the root actor picker dropdown */
	TArray<TSharedPtr<FString>> RootActorList;

	/** The name or label of the active actor */
	TSharedPtr<FString> ActiveRootActorName;

	/** The combo box widget that allows the uesr to pick the active root actor from */
	TSharedPtr<SComboBox<TSharedPtr<FString>>> RootActorComboBox;

	/** The OnLevelActorDeleted delegate handle */
	FDelegateHandle LevelActorDeletedHandle;

	/** The handle to OnMapChanged */
	FDelegateHandle MapChangedHandle;

	/** Delegate handle for when an object is transacted */
	FDelegateHandle OnObjectTransactedHandle;
};