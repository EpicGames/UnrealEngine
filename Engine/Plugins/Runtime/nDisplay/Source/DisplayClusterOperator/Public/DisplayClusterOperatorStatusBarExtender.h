// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WidgetDrawerConfig.h"

class SDisplayClusterOperatorStatusBar;

/** An extender that can be passed to external modules to allow them to extend the operator panel's status bar */
class FDisplayClusterOperatorStatusBarExtender
{
public:
	/**
	 * Adds a new widget drawer to the status bar
	 * 
	 * @param InWidgetDrawerConfig - The widget drawer config to create the widget drawer from
	 * @param SlotIndex - An optional slot index of the widget drawer to add, which determines where the button for the widget drawer is placed on the status bar
	 */
	void DISPLAYCLUSTEROPERATOR_API AddWidgetDrawer(const FWidgetDrawerConfig& InWidgetDrawerConfig, int32 SlotIndex = INDEX_NONE);

	/** Registers any status bar extensions with the specified status bar */
	void RegisterExtensions(TSharedRef<SDisplayClusterOperatorStatusBar> StatusBar);

private:
	TArray<TPair<int32, FWidgetDrawerConfig>> WidgetDrawers;
};