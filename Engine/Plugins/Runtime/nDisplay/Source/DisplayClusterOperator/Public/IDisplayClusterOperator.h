// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class IDisplayClusterOperatorApp;
class IDisplayClusterOperatorViewModel;
class FLayoutExtender;
class FTabManager;
class FExtensibilityManager;
class FUICommandList;
class FDisplayClusterOperatorStatusBarExtender;
class ADisplayClusterRootActor;

/**
 * Display Cluster Operator module interface
 */
class IDisplayClusterOperator : public IModuleInterface
{
public:
	static constexpr const TCHAR* ModuleName = TEXT("DisplayClusterOperator");

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnRegisterLayoutExtensions, FLayoutExtender&);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnRegisterStatusBarExtensions, FDisplayClusterOperatorStatusBarExtender&);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnAppendOperatorPanelCommands, TSharedRef<FUICommandList>)
	DECLARE_DELEGATE_RetVal_OneParam(TSharedRef<IDisplayClusterOperatorApp>, FOnGetAppInstance, TSharedRef<IDisplayClusterOperatorViewModel>);
	
public:
	virtual ~IDisplayClusterOperator() = default;

	/**
	* Singleton-like access to this module's interface.  This is just for convenience!
	* Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	*
	* @return Returns singleton instance, loading the module on demand if needed
	*/
	static inline IDisplayClusterOperator& Get()
	{
		return FModuleManager::GetModuleChecked<IDisplayClusterOperator>(ModuleName);
	}

	/**
	* Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	*
	* @return True if the module is loaded and ready to use
	*/
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded(ModuleName);
	}

	/** Register an app with the operator panel */
	virtual FDelegateHandle RegisterApp(const FOnGetAppInstance& InGetAppInstanceDelegate) = 0;

	/** Unregisters an app from the operator panel */
	virtual void UnregisterApp(const FDelegateHandle& InHandle) = 0;
	
	/** Gets the operator panel's view model, which stores any state used by the operator panel */
	virtual TSharedRef<IDisplayClusterOperatorViewModel> GetOperatorViewModel() = 0;

	/** Gets the event handler that is raised when the operator panel processes extensions to its layout */
	virtual FOnRegisterLayoutExtensions& OnRegisterLayoutExtensions() = 0;

	/** Gets the event handler that is raised when the operator panel processes extensions to its status bar */
	virtual FOnRegisterStatusBarExtensions& OnRegisterStatusBarExtensions() = 0;

	/** Gets the event handler that is raised when the operator panel binds commands to its command list */
	virtual FOnAppendOperatorPanelCommands& OnAppendOperatorPanelCommands() = 0;

	/** Gets the extension ID for the main window region that can be used to add tabs to the operator panel */
	virtual FName GetPrimaryOperatorExtensionId() = 0;

	/** Gets the extension ID for the auxilliary window region intended for lower-thirds windows (e.g log output) that can be used to add tabs to the operator panel */
	virtual FName GetAuxilliaryOperatorExtensionId() = 0;

	/** Gets the tab ID used for the details panel */
	virtual FName GetDetailsTabId() = 0;
	
	/** Gets the extensibility manager for the operator panel's toolbar */
	virtual TSharedPtr<FExtensibilityManager> GetOperatorToolBarExtensibilityManager() = 0;

	/** Gets the extensibility manager for the operator panel's menu bar */
	virtual TSharedPtr<FExtensibilityManager> GetOperatorMenuExtensibilityManager() = 0;
	
	/** Gets a list of all nDisplay root actor instances that are on the currently loaded level */
	virtual void GetRootActorLevelInstances(TArray<ADisplayClusterRootActor*>& OutRootActorInstances) = 0;

	/** Toggles the state of a drawer with the specified ID, closing the drawer if it is open, and opening the drawer if it is closed */
	virtual void ToggleDrawer(const FName DrawerId) = 0;

	/** Forces the operator panel to dismiss any open drawers */
	virtual void ForceDismissDrawers() = 0;
};