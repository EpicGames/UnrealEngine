// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class AActor;
class ADisplayClusterRootActor;
class FTabManager;
class FWorkspaceItem;


/** Interface for a view model object that stores any state from the operator panel that should be exposed externally */
class DISPLAYCLUSTEROPERATOR_API IDisplayClusterOperatorViewModel
{
public:
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnActiveRootActorChanged, ADisplayClusterRootActor*);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnDetailObjectsChanged, const TArray<UObject*>&);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnOutlinerSelectionChanged, const TArray<AActor*>&);
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnActorsSelected, const TArray<AActor*>&, bool bShouldSelect);

public:
	/** Gets whether the view model has been populated with a valid root actor */
	virtual bool HasRootActor(bool bEvenIfPendingKill = false) const = 0;

	/** Gets the root actor that is actively being edited by the operator panel */
	virtual ADisplayClusterRootActor* GetRootActor(bool bEvenIfPendingKill = false) const = 0;

	/** Sets the root actor that is actively being edited by the operator panel */
	virtual void SetRootActor(ADisplayClusterRootActor* InRootActor) = 0;

	/** Gets the event handler that is raised when the operator panel changes the root actor being operated on */
	virtual FOnActiveRootActorChanged& OnActiveRootActorChanged() = 0;

	/** Gets the list of objects being displayed in the operator's details panel */
	virtual const TArray<TWeakObjectPtr<UObject>> GetDetailObjects() = 0;

	/** Displays the properties of the specified object in the operator's details panel */
	virtual void ShowDetailsForObject(UObject* Object) = 0;

	/** Displays the properties of the specified object in the operator's details panel */
	virtual void ShowDetailsForObjects(const TArray<UObject*>& Objects) = 0;

	/** Selects the specified actors in the operator's outliner panel, or if bShouldSelect is false, deselects them */
	virtual void SelectActors(const TArray<AActor*>& Actors, bool bShouldSelect) = 0;

	/** Gets the event handler that is raised when the objects being displayed in the operator's details panel are changed */
	virtual FOnDetailObjectsChanged& OnDetailObjectsChanged() = 0;

	/** Gets the event handler that is raised when the actors selected in the operator's outliner panel are changed */
	virtual FOnOutlinerSelectionChanged& OnOutlinerSelectionChanged() = 0;

	/** Gets the event handler that is raised when actors are selected from outside the operator itself, i.e. directly from this view model */
	virtual FOnActorsSelected& OnActorsSelectedExternally() = 0;

	/** Gets the tab manager of the active operator panel, if there is an open operator panel */
	virtual TSharedPtr<FTabManager> GetTabManager() const = 0;

	/** Gets the registered workspace menu group */
	virtual TSharedPtr<FWorkspaceItem> GetWorkspaceMenuGroup() const = 0;
};