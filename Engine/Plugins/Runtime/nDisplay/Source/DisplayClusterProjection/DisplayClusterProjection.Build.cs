// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class DisplayClusterProjection : ModuleRules
{
	public DisplayClusterProjection(ReadOnlyTargetRules ROTargetRules) : base(ROTargetRules)
	{
		PublicIncludePathModuleNames.AddRange(
			new string[] {
				"DisplayClusterConfiguration",
				"DisplayClusterShaders",
				"DisplayClusterWarp"
			});

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"CinematicCamera",
				"Core",
				"CoreUObject",
				"DisplayCluster",
				"Engine",
				"Projects",
				"Slate",
				"SlateCore"
			});

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"DisplayCluster",
				"ProceduralMeshComponent",
				"Projects",
				"RenderCore",
				"RHI"
			});

		if (Target.bBuildEditor == true)
		{
			PrivateDependencyModuleNames.Add("UnrealEd");
		}

		if (Target.Platform == UnrealTargetPlatform.Win64)
		{
			PrivateDependencyModuleNames.AddRange(new string[] {
				"D3D11RHI",
				"D3D12RHI",
			});

			AddEngineThirdPartyPrivateStaticDependencies(Target, "DX11", "DX12");
		}

		AddThirdPartyDependencies(ROTargetRules);
	}

	public void AddThirdPartyDependencies(ReadOnlyTargetRules ROTargetRules)
	{
		string ThirdPartyPath = Path.Combine(PluginDirectory, "Source", "ThirdParty");

		if (Target.Platform == UnrealTargetPlatform.Win64)
		{
			// EasyBlend SDK
			PrivateIncludePaths.Add(Path.Combine(ThirdPartyPath, "EasyBlend", "Include"));
			RuntimeDependencies.Add(Path.Combine(ThirdPartyPath, "EasyBlend", "DLL", "mplEasyBlendSDKDX1164.dll"));
			RuntimeDependencies.Add(Path.Combine(ThirdPartyPath, "EasyBlend", "DLL", "mplEasyBlendSDK.dll"));

			// VIOSO SDK
			PrivateIncludePaths.Add(Path.Combine(ThirdPartyPath, "VIOSO", "Include"));
			RuntimeDependencies.Add(Path.Combine(ThirdPartyPath, "VIOSO", "DLL", "VIOSOWarpBlend64.dll"));

			// Domeprojection SDK
			PrivateIncludePaths.Add(Path.Combine(ThirdPartyPath, "Domeprojection", "Include"));
			RuntimeDependencies.Add(Path.Combine(ThirdPartyPath, "Domeprojection", "DLL", "dpLib.dll"));
			RuntimeDependencies.Add(Path.Combine(ThirdPartyPath, "Domeprojection", "DLL", "WibuCm64.dll"));
		}
	}
}
