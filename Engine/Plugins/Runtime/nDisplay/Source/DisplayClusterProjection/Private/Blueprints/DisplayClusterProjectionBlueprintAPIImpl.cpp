// Copyright Epic Games, Inc. All Rights Reserved.

#include "Blueprints/DisplayClusterProjectionBlueprintAPIImpl.h"
#include "Blueprints/DisplayClusterProjectionBlueprintLib.h"

//////////////////////////////////////////////////////////////////////////////////////////////
// Policy: CAMERA
//////////////////////////////////////////////////////////////////////////////////////////////
void UDisplayClusterProjectionBlueprintAPIImpl::CameraPolicySetCamera(const FString& ViewportId, UCameraComponent* NewCamera, float FOVMultiplier)
{
	UDisplayClusterProjectionBlueprintLib::CameraPolicySetCamera(ViewportId, NewCamera, FOVMultiplier);
}
