// Copyright Epic Games, Inc. All Rights Reserved.

#include "Blueprints/DisplayClusterProjectionBlueprintLib.h"
#include "Blueprints/DisplayClusterProjectionBlueprintAPIImpl.h"

#include "IDisplayCluster.h"
#include "IDisplayClusterProjection.h"

#include "Policy/Camera/DisplayClusterProjectionCameraPolicy.h"
#include "Render/IDisplayClusterRenderManager.h"
#include "Render/Viewport/IDisplayClusterViewportManager.h"
#include "Render/Viewport/IDisplayClusterViewport.h"

#include "UObject/Package.h"


// [DEPRECATED]
void UDisplayClusterProjectionBlueprintLib::GetAPI(TScriptInterface<IDisplayClusterProjectionBlueprintAPI>& OutAPI)
{
	static UDisplayClusterProjectionBlueprintAPIImpl* Obj = NewObject<UDisplayClusterProjectionBlueprintAPIImpl>(GetTransientPackage(), NAME_None, RF_MarkAsRootSet);
	OutAPI = Obj;
}

void UDisplayClusterProjectionBlueprintLib::CameraPolicySetCamera(const FString& ViewportId, UCameraComponent* NewCamera, float FOVMultiplier)
{
	IDisplayClusterRenderManager* RenderManager = IDisplayCluster::Get().GetRenderMgr();
	if (RenderManager && RenderManager->GetViewportManager())
	{
		IDisplayClusterViewport* Viewport = RenderManager->GetViewportManager()->FindViewport(ViewportId);
		if (Viewport != nullptr)
		{
			// @todo: Add extra settings latter to BP call if required
			FDisplayClusterProjectionCameraPolicySettings Settings;
			Settings.FOVMultiplier = FOVMultiplier;
			Settings.bCameraOverrideDefaults = true;

			IDisplayClusterProjection::Get().CameraPolicySetCamera(Viewport->GetProjectionPolicy(), NewCamera, Settings);
		}
	}
}
