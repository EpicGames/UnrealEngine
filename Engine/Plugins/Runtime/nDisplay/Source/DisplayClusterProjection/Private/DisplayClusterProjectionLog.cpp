// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterProjectionLog.h"


DEFINE_LOG_CATEGORY(LogDisplayClusterProjection);
DEFINE_LOG_CATEGORY(LogDisplayClusterProjectionCamera);
DEFINE_LOG_CATEGORY(LogDisplayClusterProjectionDomeprojection);
DEFINE_LOG_CATEGORY(LogDisplayClusterProjectionEasyBlend);
DEFINE_LOG_CATEGORY(LogDisplayClusterProjectionManual);
DEFINE_LOG_CATEGORY(LogDisplayClusterProjectionMesh);
DEFINE_LOG_CATEGORY(LogDisplayClusterProjectionMPCDI);
DEFINE_LOG_CATEGORY(LogDisplayClusterProjectionSimple);
DEFINE_LOG_CATEGORY(LogDisplayClusterProjectionVIOSO);
DEFINE_LOG_CATEGORY(LogDisplayClusterProjectionReference);
