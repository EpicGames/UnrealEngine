// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

DECLARE_LOG_CATEGORY_EXTERN(LogDisplayClusterProjection,               Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogDisplayClusterProjectionCamera,         Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogDisplayClusterProjectionDomeprojection, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogDisplayClusterProjectionEasyBlend,      Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogDisplayClusterProjectionManual,         Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogDisplayClusterProjectionMesh,           Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogDisplayClusterProjectionMPCDI,          Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogDisplayClusterProjectionSimple,         Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogDisplayClusterProjectionVIOSO,          Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogDisplayClusterProjectionReference,      Log, All);
