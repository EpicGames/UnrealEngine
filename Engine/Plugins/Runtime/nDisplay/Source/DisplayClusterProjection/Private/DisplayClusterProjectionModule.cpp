// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterProjectionModule.h"

#include "DisplayClusterProjectionLog.h"
#include "DisplayClusterProjectionStrings.h"

#include "Policy/Camera/DisplayClusterProjectionCameraPolicyFactory.h"
#include "Policy/Domeprojection/DisplayClusterProjectionDomeprojectionPolicyFactory.h"
#include "Policy/EasyBlend/DisplayClusterProjectionEasyBlendPolicyFactory.h"
#include "Policy/Link/DisplayClusterProjectionLinkPolicyFactory.h"
#include "Policy/Manual/DisplayClusterProjectionManualPolicyFactory.h"
#include "Policy/MPCDI/DisplayClusterProjectionMPCDIPolicyFactory.h"
#include "Policy/Reference/DisplayClusterProjectionReferencePolicyFactory.h"
#include "Policy/VIOSO/DisplayClusterProjectionVIOSOPolicyFactory.h"

#include "Policy/Mesh/DisplayClusterProjectionMeshPolicy.h"

#include "IDisplayCluster.h"
#include "Render/IDisplayClusterRenderManager.h"

#include "Components/SceneComponent.h"
#include "Components/StaticMeshComponent.h"


#include "IDisplayCluster.h"
#include "Render/IDisplayClusterRenderManager.h"
#include "Render/Viewport/IDisplayClusterViewportManager.h"
#include "Render/Viewport/IDisplayClusterViewport.h"

#include "Policy/Camera/DisplayClusterProjectionCameraPolicy.h"

FDisplayClusterProjectionModule::FDisplayClusterProjectionModule()
{
	TSharedPtr<IDisplayClusterProjectionPolicyFactory> Factory;

	// Camera projection
	Factory = MakeShared<FDisplayClusterProjectionCameraPolicyFactory>();
	ProjectionPolicyFactories.Emplace(DisplayClusterProjectionStrings::projection::Camera, Factory);

	// Domeprojection projection
	Factory = MakeShared<FDisplayClusterProjectionDomeprojectionPolicyFactory>();
	ProjectionPolicyFactories.Emplace(DisplayClusterProjectionStrings::projection::Domeprojection, Factory);

	// EasyBlend projection
	Factory = MakeShared<FDisplayClusterProjectionEasyBlendPolicyFactory>();
	ProjectionPolicyFactories.Emplace(DisplayClusterProjectionStrings::projection::EasyBlend, Factory);

	// Link projection
	Factory = MakeShared<FDisplayClusterProjectionLinkPolicyFactory>();
	ProjectionPolicyFactories.Emplace(DisplayClusterProjectionStrings::projection::Link, Factory);

	// Reference projection
	Factory = MakeShared<FDisplayClusterProjectionReferencePolicyFactory>();
	ProjectionPolicyFactories.Emplace(DisplayClusterProjectionStrings::projection::Reference, Factory);

	// Manual projection
	Factory = MakeShared<FDisplayClusterProjectionManualPolicyFactory>();
	ProjectionPolicyFactories.Emplace(DisplayClusterProjectionStrings::projection::Manual, Factory);

	// MPCDI and Mesh projection
	Factory = MakeShared<FDisplayClusterProjectionMPCDIPolicyFactory>();
	ProjectionPolicyFactories.Emplace(DisplayClusterProjectionStrings::projection::MPCDI, Factory);
	ProjectionPolicyFactories.Emplace(DisplayClusterProjectionStrings::projection::Mesh, Factory);

	// VIOSO projection
	Factory = MakeShared<FDisplayClusterProjectionVIOSOPolicyFactory>();
	ProjectionPolicyFactories.Emplace(DisplayClusterProjectionStrings::projection::VIOSO, Factory);

	UE_LOGF(LogDisplayClusterProjection, Log, "Projection module has been instantiated");
}

FDisplayClusterProjectionModule::~FDisplayClusterProjectionModule()
{
	UE_LOGF(LogDisplayClusterProjection, Log, "Projection module has been destroyed");
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IModuleInterface
//////////////////////////////////////////////////////////////////////////////////////////////
void FDisplayClusterProjectionModule::StartupModule()
{
	UE_LOGF(LogDisplayClusterProjection, Log, "Projection module startup");

	IDisplayClusterRenderManager* RenderMgr = IDisplayCluster::Get().GetRenderMgr();
	if (RenderMgr)
	{
		for (auto it = ProjectionPolicyFactories.CreateIterator(); it; ++it)
		{
			UE_LOGF(LogDisplayClusterProjection, Log, "Registering <%ls> projection policy factory...", *it->Key);

			if (!RenderMgr->RegisterProjectionPolicyFactory(it->Key, it->Value))
			{
				UE_LOGF(LogDisplayClusterProjection, Warning, "Couldn't register <%ls> projection policy factory", *it->Key);
			}
		}
	}

	UE_LOGF(LogDisplayClusterProjection, Log, "Projection module has started");
}

void FDisplayClusterProjectionModule::ShutdownModule()
{
	UE_LOGF(LogDisplayClusterProjection, Log, "Projection module shutdown");

	IDisplayClusterRenderManager* RenderMgr = IDisplayCluster::Get().GetRenderMgr();
	if (RenderMgr)
	{
		for (auto it = ProjectionPolicyFactories.CreateConstIterator(); it; ++it)
		{
			UE_LOGF(LogDisplayClusterProjection, Log, "Un-registering <%ls> projection factory...", *it->Key);

			if (!RenderMgr->UnregisterProjectionPolicyFactory(it->Key))
			{
				UE_LOGF(LogDisplayClusterProjection, Warning, "An error occurred during un-registering the <%ls> projection factory", *it->Key);
			}
		}
	}
}

//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProjection
//////////////////////////////////////////////////////////////////////////////////////////////
void FDisplayClusterProjectionModule::GetSupportedProjectionTypes(TArray<FString>& OutProjectionTypes)
{
	ProjectionPolicyFactories.GenerateKeyArray(OutProjectionTypes);
}

TSharedPtr<IDisplayClusterProjectionPolicyFactory> FDisplayClusterProjectionModule::GetProjectionFactory(const FString& ProjectionType)
{
	if (ProjectionPolicyFactories.Contains(ProjectionType))
	{
		return ProjectionPolicyFactories[ProjectionType];
	}

	UE_LOGF(LogDisplayClusterProjection, Warning, "No <%ls> projection factory available", *ProjectionType);

	return nullptr;
}

bool FDisplayClusterProjectionModule::CameraPolicySetCamera(const TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe>& InPolicy, UCameraComponent* const NewCamera, const FDisplayClusterProjectionCameraPolicySettings& CameraSettings)
{
	if (InPolicy.IsValid())
	{
		if (FDisplayClusterProjectionCameraPolicy* CameraPolicyInstance = static_cast<FDisplayClusterProjectionCameraPolicy*>(InPolicy.Get()))
		{
			CameraPolicyInstance->SetCamera(NewCamera, CameraSettings);
			return true;
		}
	}

	return false;
}

UCameraComponent* FDisplayClusterProjectionModule::CameraPolicyGetCameraComponent(const TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe>& InPolicy)
{
	if (InPolicy.IsValid())
	{
		if (FDisplayClusterProjectionCameraPolicy* CameraPolicyInstance = static_cast<FDisplayClusterProjectionCameraPolicy*>(InPolicy.Get()))
		{
			return CameraPolicyInstance->GetCameraComponent();
		}
	}

	return nullptr;
}

IMPLEMENT_MODULE(FDisplayClusterProjectionModule, DisplayClusterProjection);
