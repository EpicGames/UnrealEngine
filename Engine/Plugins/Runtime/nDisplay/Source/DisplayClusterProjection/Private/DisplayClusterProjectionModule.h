// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDisplayClusterProjection.h"
#include "Render/Projection/IDisplayClusterProjectionPolicyFactory.h"

class IDisplayClusterProjectionPolicyFactory;
class USceneComponent;
class UStaticMeshComponent;


class FDisplayClusterProjectionModule
	: public IDisplayClusterProjection
{
public:
	FDisplayClusterProjectionModule();
	virtual ~FDisplayClusterProjectionModule();

public:
	//////////////////////////////////////////////////////////////////////////////////////////////
	// IModuleInterface
	//////////////////////////////////////////////////////////////////////////////////////////////
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

public:
	//////////////////////////////////////////////////////////////////////////////////////////////
	// IDisplayClusterProjection
	//////////////////////////////////////////////////////////////////////////////////////////////
	virtual void GetSupportedProjectionTypes(TArray<FString>& OutProjectionTypes) override;
	virtual TSharedPtr<IDisplayClusterProjectionPolicyFactory> GetProjectionFactory(const FString& InProjectionType) override;

	virtual bool CameraPolicySetCamera(const TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe>& InPolicy, UCameraComponent* const NewCamera, const FDisplayClusterProjectionCameraPolicySettings& CameraSettings) override;

	virtual UCameraComponent* CameraPolicyGetCameraComponent(const TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe>& InPolicy) override;

private:
	// Available factories
	TMap<FString, TSharedPtr<IDisplayClusterProjectionPolicyFactory>> ProjectionPolicyFactories;
};
