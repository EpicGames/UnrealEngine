// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "DisplayClusterProjectionStrings.h"

namespace DisplayClusterProjectionStrings
{
	namespace ThirdParty
	{
		namespace DLL
		{
			static constexpr const TCHAR* EasyBlendDX11 = TEXT("EasyBlend/DLL/mplEasyBlendSDKDX1164.dll");
			static constexpr const TCHAR* EasyBlendDX12 = TEXT("EasyBlend/DLL/mplEasyBlendSDK.dll");

			static constexpr const TCHAR* VIOSO = TEXT("VIOSO/DLL/ViosoWarpBlend64.dll");

			static constexpr const TCHAR* DomeprojectionDX11 = TEXT("Domeprojection/DLL/dpLib.dll");
		}
	}
};
