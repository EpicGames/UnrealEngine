// Copyright Epic Games, Inc. All Rights Reserved.

#include "Policy/Camera/DisplayClusterProjectionCameraPolicy.h"

#include "DisplayClusterProjectionLog.h"
#include "DisplayClusterProjectionStrings.h"

#include "Game/IDisplayClusterGameManager.h"

#include "Engine/World.h"
#include "Camera/CameraComponent.h"
#include "CineCameraComponent.h"
#include "Math/PerspectiveMatrix.h"

#include "Render/Viewport/IDisplayClusterViewport.h"
#include "Render/Viewport/IDisplayClusterViewportManager.h"

#include "DisplayClusterRootActor.h"


//////////////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterProjectionPolicy
//////////////////////////////////////////////////////////////////////////////////////////////
FDisplayClusterProjectionCameraPolicy::FDisplayClusterProjectionCameraPolicy(const FString& ProjectionPolicyId, const FDisplayClusterConfigurationProjection* InConfigurationProjectionPolicy)
	: FDisplayClusterProjectionPolicyBase(ProjectionPolicyId, InConfigurationProjectionPolicy)
{
}

const FString& FDisplayClusterProjectionCameraPolicy::GetType() const
{
	static const FString Type(DisplayClusterProjectionStrings::projection::Camera);
	return Type;
}

void FDisplayClusterProjectionCameraPolicy::UpdatePostProcessSettings(IDisplayClusterViewport* InViewport)
{
	// Override viewport PP from camera, except ICVFX inner camera
	// because the ICVFX InCamera postprocess is overridden separately in FDisplayClusterViewportConfigurationHelpers_Postprocess::UpdateCameraPostProcessSettings();
	// The ICVFX camera postprocess is more detailed and is defined in the FDisplayClusterConfigurationICVFX_CameraSettings structure.
	if (InViewport && !EnumHasAnyFlags(InViewport->GetRenderSettingsICVFX().RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::InCamera))
	{
		const float DeltaTime = InViewport->GetConfiguration().GetRootActorWorldDeltaSeconds(EDisplayClusterRootActorType::Scene);

		FMinimalViewInfo ViewInfo;
		if (ImplSetupProjectionViewPoint(InViewport, DeltaTime, ViewInfo) && ViewInfo.PostProcessBlendWeight > 0.0f)
		{
			InViewport->GetViewport_CustomPostProcessSettings().AddCustomPostProcess(IDisplayClusterViewport_CustomPostProcessSettings::ERenderPass::Override, ViewInfo.PostProcessSettings, ViewInfo.PostProcessBlendWeight, true);
		}
	}
}

bool FDisplayClusterProjectionCameraPolicy::HandleStartScene(IDisplayClusterViewport* InViewport)
{
	check(IsInGameThread());

	return true;
}

void FDisplayClusterProjectionCameraPolicy::HandleEndScene(IDisplayClusterViewport* InViewport)
{
	check(IsInGameThread());

	CameraRef.ResetSceneComponent();
}

void FDisplayClusterProjectionCameraPolicy::SetupProjectionViewPoint(IDisplayClusterViewport* InViewport, const float InDeltaTime, FMinimalViewInfo& InOutViewInfo, float* OutCustomNearClippingPlane)
{
	// Get camera position
	ImplSetupProjectionViewPoint(InViewport, InDeltaTime, InOutViewInfo, OutCustomNearClippingPlane);

	// Saving camera parameters
	CameraFOVDegrees = InOutViewInfo.FOV;
	CameraAspectRatio = InOutViewInfo.AspectRatio;
}

UCameraComponent* FDisplayClusterProjectionCameraPolicy::GetCameraComponent() const
{
	if (USceneComponent* SceneComponent = CameraRef.GetOrFindSceneComponent())
	{
		return Cast<UCameraComponent>(SceneComponent);
	}

	return nullptr;
}

EDisplayClusterProjectionPolicyFlags FDisplayClusterProjectionCameraPolicy::GetProjectionPolicyFlags(const IDisplayClusterViewport* InViewport, const uint32 InContextNum) const
{
	if (InViewport && EnumHasAnyFlags(InViewport->GetRenderSettingsICVFX().RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::InCamera))
	{
		// The ICVFX camera is always in the DCRA area and can follow the camera.
		return EDisplayClusterProjectionPolicyFlags::EnableFollowCameraFeature;
	}

	return EDisplayClusterProjectionPolicyFlags::None;
}

bool FDisplayClusterProjectionCameraPolicy::CalculateView(IDisplayClusterViewport* InViewport, const uint32 InContextNum, FVector& InOutViewLocation, FRotator& InOutViewRotation, const FVector& ViewOffset, const float WorldToMeters, const float NCP, const float FCP)
{
	check(IsInGameThread());

	// Camera location already defined from SetupProjectionViewPoint()

	// Saving clipping planes
	ZNear = NCP;
	ZFar = FCP;

	return true;
}


bool FDisplayClusterProjectionCameraPolicy::GetProjectionMatrix(IDisplayClusterViewport* InViewport, const uint32 InContextNum, FMatrix& OutPrjMatrix)
{
	check(IsInGameThread());

	// The horizontal field of view (in degrees)
	const float ScaledCameraFOVDegrees = CameraFOVDegrees * CameraSettings.FOVMultiplier;

	// Clamp camera fov to valid range [1.f, 178.f]
	const float ClampedCameraFOVDegrees = FMath::Clamp(ScaledCameraFOVDegrees, 1.f, 178.f);
	if (ClampedCameraFOVDegrees != ScaledCameraFOVDegrees && !IsEditorOperationMode(InViewport))
	{
		UE_LOGF(LogDisplayClusterProjectionCamera, Warning, "CameraFOV clamped: '%f' -> '%f'. (FieldOfView='%f', FOVMultiplier='%f'", ScaledCameraFOVDegrees, ClampedCameraFOVDegrees, CameraFOVDegrees, CameraSettings.FOVMultiplier);
	}

	// Support inner camera custom frustum
	const float HalfFOVH = ZNear * FMath::Tan(FMath::DegreesToRadians(0.5 * ClampedCameraFOVDegrees));
	const float HalfFOVV = HalfFOVH / CameraAspectRatio;

	if (InViewport)
	{
		InViewport->CalculateProjectionMatrix(InContextNum, -HalfFOVH, HalfFOVH, HalfFOVV, -HalfFOVV, ZNear, ZFar, false);
		OutPrjMatrix = InViewport->GetContexts()[InContextNum].ProjectionMatrix;
	}
	else
	{
		// FReversedZPerspectiveMatrix() can't be used because it returns the wrong matrix when ZNear==ZFar (nDisplay encoding for infinite far clipping plane).
		OutPrjMatrix = IDisplayClusterViewport::MakeProjectionMatrix(-HalfFOVH, HalfFOVH, HalfFOVV, -HalfFOVV, ZNear, ZFar);
	}

	if (!CameraSettings.OffCenterProjectionOffset.IsZero())
	{
		const float Left = -1.0f + CameraSettings.OffCenterProjectionOffset.X;
		const float Right = Left + 2.0f;
		const float Bottom = -1.0f + CameraSettings.OffCenterProjectionOffset.Y;
		const float Top = Bottom + 2.0f;
		OutPrjMatrix.M[2][0] = (Left + Right) / (Left - Right);
		OutPrjMatrix.M[2][1] = (Bottom + Top) / (Bottom - Top);
	}

	return true;
}

//////////////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterProjectionCameraPolicy
//////////////////////////////////////////////////////////////////////////////////////////////
bool FDisplayClusterProjectionCameraPolicy::ImplSetupProjectionViewPoint(IDisplayClusterViewport* InViewport, const float InDeltaTime, FMinimalViewInfo& InOutViewInfo, float* OutCustomNearClippingPlane) const
{
	if (OutCustomNearClippingPlane)
	{
		*OutCustomNearClippingPlane = -1;
	}

	bool bResult = false;
	if (UCameraComponent* CameraComponent = GetCameraComponent())
	{
		// Use assigned camera component
		// Store CustomNearClippingPlane locally, and then use that value for the projection matrix
		bResult = IDisplayClusterViewport::GetCameraComponentView(CameraComponent, InDeltaTime, CameraSettings.bUseCameraPostprocess, InOutViewInfo, OutCustomNearClippingPlane);
	}
	else if (UWorld* CurrentWorld = InViewport ? InViewport->GetConfiguration().GetCurrentWorld() : nullptr)
	{
		// Get active player camera
		bResult = IDisplayClusterViewport::GetPlayerCameraView(CurrentWorld, CameraSettings.bUseCameraPostprocess, InOutViewInfo, OutCustomNearClippingPlane);
	}

	// Fix camera lens deffects (prototype)
	InOutViewInfo.Location += CameraSettings.FrustumOffset;
	InOutViewInfo.Rotation += CameraSettings.FrustumRotation;

	return bResult;
}

void FDisplayClusterProjectionCameraPolicy::SetCamera(UCameraComponent* NewCamera, const FDisplayClusterProjectionCameraPolicySettings& InCameraSettings)
{
	if(CameraSettings.bCameraOverrideDefaults && InCameraSettings.bCameraOverrideDefaults == false)
	{
		// Ignore default camera updates (UE-137222)
		return;
	}

	CameraSettings = InCameraSettings;

	if (NewCamera)
	{
		UE_LOGF(LogDisplayClusterProjectionCamera, Verbose, "New camera set: %ls", *NewCamera->GetFullName());

		CameraRef.SetSceneComponent(NewCamera);
	}
	else
	{
		if (InCameraSettings.bCameraOverrideDefaults == false)
		{
			if (!IsEditorOperationMode(nullptr))
			{
				UE_LOGF(LogDisplayClusterProjectionCamera, Warning, "Trying to set nullptr camera pointer");
			}
		}

		CameraRef.ResetSceneComponent();

		// After ref reset, allow to use default cameras again
		CameraSettings.bCameraOverrideDefaults = false;
	}
}
