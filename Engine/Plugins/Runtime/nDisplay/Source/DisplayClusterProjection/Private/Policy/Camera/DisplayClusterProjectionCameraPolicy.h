// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Policy/DisplayClusterProjectionPolicyBase.h"
#include "Containers/DisplayClusterProjectionCameraPolicySettings.h"
#include "Misc/DisplayClusterObjectRef.h"

/**
 * Camera projection policy implementation
 */
class FDisplayClusterProjectionCameraPolicy
	: public FDisplayClusterProjectionPolicyBase
{
public:
	FDisplayClusterProjectionCameraPolicy(const FString& ProjectionPolicyId, const FDisplayClusterConfigurationProjection* InConfigurationProjectionPolicy);

public:
	//~ Begin IDisplayClusterProjectionPolicy
	virtual const FString& GetType() const override;

	virtual bool HandleStartScene(IDisplayClusterViewport* InViewport) override;
	virtual void HandleEndScene(IDisplayClusterViewport* InViewport) override;

	virtual bool CalculateView(IDisplayClusterViewport* InViewport, const uint32 InContextNum, FVector& InOutViewLocation, FRotator& InOutViewRotation, const FVector& ViewOffset, const float WorldToMeters, const float NCP, const float FCP) override;
	virtual bool GetProjectionMatrix(IDisplayClusterViewport* InViewport, const uint32 InContextNum, FMatrix& OutPrjMatrix) override;
	virtual void SetupProjectionViewPoint(IDisplayClusterViewport* InViewport, const float InDeltaTime, FMinimalViewInfo& InOutViewInfo, float* OutCustomNearClippingPlane = nullptr) override;

	virtual void UpdatePostProcessSettings(IDisplayClusterViewport* InViewport) override;

	virtual bool ShouldUseSourceTextureWithMips(const IDisplayClusterViewport* InViewport) const override
	{
		return true;
	}

	virtual bool ShouldUseViewPointComponentPostProcesses(IDisplayClusterViewport* InViewport) const override
	{
		// This projection policy uses its own camera with postprocessing.
		// Disable the use of PP settings from the ViewPoint component.
		return false;
	}

	virtual EDisplayClusterProjectionPolicyFlags GetProjectionPolicyFlags(const IDisplayClusterViewport* InViewport, const uint32 InContextNum) const override;
	virtual UCameraComponent* GetCameraComponent() const override;
	//~ End IDisplayClusterProjectionPolicy

public:
	void SetCamera(UCameraComponent* const NewCamera, const FDisplayClusterProjectionCameraPolicySettings& InCameraSettings);

private:
	bool ImplSetupProjectionViewPoint(IDisplayClusterViewport* InViewport, const float InDeltaTime, FMinimalViewInfo& InOutViewInfo, float* OutCustomNearClippingPlane = nullptr) const;

protected:

private:
	// Camera to use for rendering
	FDisplayClusterSceneComponentRef CameraRef;

	// Camera settings
	FDisplayClusterProjectionCameraPolicySettings CameraSettings;

	// Values of the clipping planes (saved from CalculateView())
	float ZNear = 1.f;
	float ZFar = 1.f;

	// camera FOV value in degrees (saved from the SetupProjectionViewPoint() function)
	float CameraFOVDegrees = 90;

	// camera aspect ratio value (saved from the SetupProjectionViewPoint() function)
	float CameraAspectRatio = 1;
};
