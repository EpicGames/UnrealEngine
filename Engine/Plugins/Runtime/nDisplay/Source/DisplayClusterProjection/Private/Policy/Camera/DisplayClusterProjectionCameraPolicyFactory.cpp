// Copyright Epic Games, Inc. All Rights Reserved.

#include "Policy/Camera/DisplayClusterProjectionCameraPolicyFactory.h"
#include "Policy/Camera/DisplayClusterProjectionCameraPolicy.h"

#include "DisplayClusterProjectionLog.h"

#include "DisplayClusterConfigurationTypes.h"


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProjectionPolicyFactory
//////////////////////////////////////////////////////////////////////////////////////////////
TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe> FDisplayClusterProjectionCameraPolicyFactory::Create(const FString& ProjectionPolicyId, const FDisplayClusterConfigurationProjection* InConfigurationProjectionPolicy)
{
	if(!InConfigurationProjectionPolicy)
	{
		if (CanShowMsgOnce(ProjectionPolicyId))
		{
			UE_LOGF(LogDisplayClusterProjectionCamera, Error, "Camera policy '%ls': null configuration", *ProjectionPolicyId);
		}
		return nullptr;
	}

	UE_LOGF(LogDisplayClusterProjectionCamera, Verbose, "Instantiating projection policy <%ls> id='%ls'", *InConfigurationProjectionPolicy->Type, *ProjectionPolicyId);
	return  MakeShared<FDisplayClusterProjectionCameraPolicy, ESPMode::ThreadSafe>(ProjectionPolicyId, InConfigurationProjectionPolicy);
}
