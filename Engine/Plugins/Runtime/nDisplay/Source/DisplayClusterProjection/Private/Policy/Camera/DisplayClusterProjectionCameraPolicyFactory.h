// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Render/Projection/IDisplayClusterProjectionPolicyFactory.h"
#include "Policy/DisplayClusterProjectionPolicyFactoryBase.h"

/**
 * Factory for the 'camera' projection policy.
 * Creates policies that derive view and projection from a UCameraComponent assigned to the projection policy.
 */
class FDisplayClusterProjectionCameraPolicyFactory
	: public IDisplayClusterProjectionPolicyFactory
	, protected FDisplayClusterProjectionPolicyFactoryBase
{
public:
	//~ Begin IDisplayClusterProjectionPolicyFactory
	virtual TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe> Create(const FString& ProjectionPolicyId, const FDisplayClusterConfigurationProjection* InConfigurationProjectionPolicy) override;
	//~ End IDisplayClusterProjectionPolicyFactory
};
