// Copyright Epic Games, Inc. All Rights Reserved.

#include "Policy/DisplayClusterProjectionPolicyBase.h"

#include "DisplayClusterProjectionLog.h"

#include "IDisplayCluster.h"
#include "Game/IDisplayClusterGameManager.h"

#include "DisplayClusterRootActor.h"

#include "Render/Viewport/IDisplayClusterViewportManager.h"
#include "Render/Viewport/IDisplayClusterViewport.h"

namespace UE::DisplayCluster::Projection::PolicyBaseHelpers
{
	static inline USceneComponent* FindOriginComponent(IDisplayClusterViewport* InViewport, const EDisplayClusterRootActorType InRootActorType, const FString& OriginCompId, const FString& PolicyId, bool bAllowExtraLog)
	{
		if (ADisplayClusterRootActor* InRootActor = InViewport->GetConfiguration().GetRootActor(InRootActorType))
		{
			// Try to get a node specified in the config file
			if (!OriginCompId.IsEmpty())
			{
				UE_LOGF(LogDisplayClusterProjection, Verbose, "Looking for an origin component '%ls'...", *OriginCompId);
				if (USceneComponent* PolicyOriginComp = InRootActor->GetComponentByName<USceneComponent>(OriginCompId))
				{
					return PolicyOriginComp;
				}

				if (bAllowExtraLog)
				{
					UE_LOGF(LogDisplayClusterProjection, Error, "No custom origin set or component '%ls' not found for policy '%ls'. VR root will be used.", *OriginCompId, *PolicyId);
				}
			}

			// default use root actor as Origin
			return InRootActor->GetRootComponent();
		}

		return nullptr;
	}
};
using namespace UE::DisplayCluster::Projection;

//////////////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterProjectionPolicyBase
//////////////////////////////////////////////////////////////////////////////////////////////
FDisplayClusterProjectionPolicyBase::FDisplayClusterProjectionPolicyBase(const FString& InProjectionPolicyId, const FDisplayClusterConfigurationProjection* InConfigurationProjectionPolicy)
	: PolicyInstanceId(InProjectionPolicyId)
{
	Parameters.Append(InConfigurationProjectionPolicy->Parameters);
}


bool FDisplayClusterProjectionPolicyBase::IsEditorOperationMode_RenderThread(const IDisplayClusterViewportProxy* InViewportProxy)
{
	check(IsInRenderingThread());

	if (const bool bIsClusterOperationMode = IDisplayCluster::Get().GetOperationMode() == EDisplayClusterOperationMode::Cluster)
	{
		return false;
	}

	return true;
}

bool FDisplayClusterProjectionPolicyBase::IsEditorOperationMode(IDisplayClusterViewport* InViewport)
{
	check(IsInGameThread());

	// Hide spam in logs when configuring VP in editor
	if (const bool bIsClusterOperationMode = IDisplayCluster::Get().GetOperationMode() == EDisplayClusterOperationMode::Cluster)
	{
		return false;
	}

	// Get state from viewport world (UE-114493)
	return InViewport && InViewport->GetConfiguration().IsCurrentWorldHasAnyType(EWorldType::Editor, EWorldType::EditorPreview);
}

bool FDisplayClusterProjectionPolicyBase::IsConfigurationChanged(const FDisplayClusterConfigurationProjection* InConfigurationProjectionPolicy) const
{
	if (InConfigurationProjectionPolicy->Parameters.Num() != Parameters.Num()) {

		return true;
	}

	for (const TPair<FString, FString>& NewParamIt : InConfigurationProjectionPolicy->Parameters) {

		const FString* CurrentValue = Parameters.Find(NewParamIt.Key);
		if (CurrentValue==nullptr || CurrentValue->Compare(NewParamIt.Value, ESearchCase::IgnoreCase) != 0) {
			return true;
		}
	}

	// Parameters not changed
	return false;
}

void FDisplayClusterProjectionPolicyBase::InitializeOriginComponent(IDisplayClusterViewport* InViewport, const FString& OriginCompId)
{
	const bool bAllowExtraLog = !IsEditorOperationMode(InViewport);

	ReleaseOriginComponent();

	if (InViewport)
	{
		// Get origin from scene DCRA (uses for math)
		PolicySceneOriginComponentRef.SetSceneComponent(PolicyBaseHelpers::FindOriginComponent(InViewport, EDisplayClusterRootActorType::Scene, OriginCompId, GetId(), bAllowExtraLog));

		// This origin used to get origin from preview DCRA (uses for preview mesh)
		PolicyPreviewMeshOriginComponentRef.SetSceneComponent(PolicyBaseHelpers::FindOriginComponent(InViewport, EDisplayClusterRootActorType::Preview, OriginCompId, GetId(), bAllowExtraLog));
	}
}

void FDisplayClusterProjectionPolicyBase::ReleaseOriginComponent()
{
	PolicySceneOriginComponentRef.ResetSceneComponent();
	PolicyPreviewMeshOriginComponentRef.ResetSceneComponent();
}
