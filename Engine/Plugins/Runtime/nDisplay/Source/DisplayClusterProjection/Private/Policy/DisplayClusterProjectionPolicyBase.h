// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Render/Projection/IDisplayClusterProjectionPolicy.h"
#include "Misc/DisplayClusterObjectRef.h"

#include "DisplayClusterProjectionStrings.h"

class USceneComponent;


/**
 * Base projection policy
 */
class FDisplayClusterProjectionPolicyBase
	: public IDisplayClusterProjectionPolicy
{
public:
	FDisplayClusterProjectionPolicyBase(const FString& ProjectionPolicyId, const FDisplayClusterConfigurationProjection* InConfigurationProjectionPolicy);

public:
	//~BEGIN IDisplayClusterProjectionPolicy
	virtual const FString& GetId() const override
	{
		return PolicyInstanceId;
	}

	virtual USceneComponent* const GetOriginComponent() const override
	{
		return PolicySceneOriginComponentRef.GetOrFindSceneComponent();
	}

	virtual USceneComponent* const GetPreviewMeshOriginComponent(IDisplayClusterViewport* InViewport) const override
	{
		return PolicyPreviewMeshOriginComponentRef.GetOrFindSceneComponent();
	}

	virtual const TMap<FString, FString>& GetParameters() const override
	{
		return Parameters;
	}

	virtual bool IsConfigurationChanged(const FDisplayClusterConfigurationProjection* InConfigurationProjectionPolicy) const override;
	//~~END IDisplayClusterProjectionPolicy


	static bool IsEditorOperationMode(IDisplayClusterViewport* InViewport);
	static bool IsEditorOperationMode_RenderThread(const IDisplayClusterViewportProxy* InViewportProxy);

protected:
	void InitializeOriginComponent(IDisplayClusterViewport* InViewport, const FString& OriginCopmId);
	void ReleaseOriginComponent();

private:
	// The unique name of this projection policy.
	const FString PolicyInstanceId;

	// Origin component name.
	FString PolicyOriginCompId;

	// Projection policy parameters. Used for initialization.
	TMap<FString, FString> Parameters;

	// Origin component in DCRA from the scene
	FDisplayClusterSceneComponentRef PolicySceneOriginComponentRef;

	// Origin component in the parent DCRA that is used to render the preview.
	FDisplayClusterSceneComponentRef PolicyPreviewMeshOriginComponentRef;
};
