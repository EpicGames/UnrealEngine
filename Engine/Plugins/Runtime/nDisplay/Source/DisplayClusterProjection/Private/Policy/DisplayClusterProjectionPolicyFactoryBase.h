// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * Base class for all projection policy factories.
 * Provides CanShowMsgOnce() to suppress repeated log messages — call it before
 * any warning or error that may fire every frame (e.g. unsupported RHI, missing library).
 */
class FDisplayClusterProjectionPolicyFactoryBase
{
protected:
	/** Returns true the first time InMsgTag is seen; false on every subsequent call with the same tag.
	 * Use this to gate log messages that should only fire once per factory instance.
	 *
	 * @param InMsgTag - unique string identifying the message site
	 * @return true if the caller should emit the message, false if it has already been shown
	 */
	bool CanShowMsgOnce(const FString& InMsgTag)
	{
		bool bAlreadyShown = false;
		ShownMsgTags.Add(InMsgTag, &bAlreadyShown);
		return !bAlreadyShown;
	}

	/** Returns true the first time InMsgTag is seen. */
	bool CanShowMsgOnce(const TCHAR* InMsgTag)
	{
		return CanShowMsgOnce(FString(InMsgTag));
	}

private:
	/** Tags of messages already shown; prevents the same log line from firing every frame. */
	TSet<FString> ShownMsgTags;
};
