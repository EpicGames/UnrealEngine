// Copyright Epic Games, Inc. All Rights Reserved.

#include "Policy/Domeprojection/DisplayClusterProjectionDomeprojectionPolicyBase.h"

#include "DisplayClusterProjectionLog.h"
#include "DisplayClusterProjectionStrings.h"

#include "Misc/DisplayClusterHelpers.h"

#include "IDisplayCluster.h"
#include "Config/IDisplayClusterConfigManager.h"
#include "Game/IDisplayClusterGameManager.h"

#include "Render/Viewport/IDisplayClusterViewport.h"
#include "Render/Viewport/IDisplayClusterViewportProxy.h"

int32 GDisplayClusterProjectionDomeprojectionPolicyEnableNearClippingPlane = 0;
static FAutoConsoleVariableRef CVarDisplayClusterProjectionDomeprojectionPolicyEnableNearClippingPlane(
	TEXT("nDisplay.render.projection.EnableNearClippingPlane.Dome"),
	GDisplayClusterProjectionDomeprojectionPolicyEnableNearClippingPlane,
	TEXT("Enable NearClippingPlane for DomeProjection (0 - disable).\n"),
	ECVF_RenderThreadSafe
);

//////////////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterProjectionDomeprojectionPolicyBase
//////////////////////////////////////////////////////////////////////////////////////////////
FDisplayClusterProjectionDomeprojectionPolicyBase::FDisplayClusterProjectionDomeprojectionPolicyBase(const FString& ProjectionPolicyId, const FDisplayClusterConfigurationProjection* InConfigurationProjectionPolicy)
	: FDisplayClusterProjectionPolicyBase(ProjectionPolicyId, InConfigurationProjectionPolicy)
{
}

const FString& FDisplayClusterProjectionDomeprojectionPolicyBase::GetType() const
{
	static const FString& Type(DisplayClusterProjectionStrings::projection::Domeprojection);
	return Type;
}

bool FDisplayClusterProjectionDomeprojectionPolicyBase::HandleStartScene(IDisplayClusterViewport* InViewport)
{
	check(IsInGameThread());

	// The game side of the nDisplay has been initialized by the nDisplay Game Manager already
	// so we can extend it by our projection related functionality/components/etc.	

	// Read domeprojection config data from nDisplay config file
	FString File;
	if (!ReadConfigData(InViewport, File, OriginCompId, DomeprojectionChannel, GeometryScale))
	{
		if (!IsEditorOperationMode(InViewport))
		{
			UE_LOGF(LogDisplayClusterProjectionDomeprojection, Error, "Couldn't read Domeprojection configuration from the config file");
		}

		return false;
	}

	if (File.IsEmpty())
	{
		if (!IsEditorOperationMode(InViewport))
		{
			UE_LOGF(LogDisplayClusterProjectionDomeprojection, Error, "Domeprojection configuration file is empty");
		}

		return false;
	}

	// Find origin component if it exists
	InitializeOriginComponent(InViewport, OriginCompId);

	const FString FullFilePath = DisplayClusterHelpers::filesystem::GetFullPathForConfigResource(File);
	if (!FPaths::FileExists(FullFilePath))
	{
		UE_LOGF(LogDisplayClusterProjectionDomeprojection, Error, "Couldn't read Domeprojection configuration file: %ls", *FullFilePath);
		return false;
	}

	// Create and store nDisplay-to-Domeprojection viewport adapter
	const int32 MaxViewsAmount = 2; // always init contexts for stereo

	ViewAdapter = CreateViewAdapter(FDisplayClusterProjectionDomeprojectionViewAdapterBase::FInitParams{ MaxViewsAmount });
	if (!(ViewAdapter && ViewAdapter->Initialize(InViewport, FullFilePath)))
	{
		if (!IsEditorOperationMode(InViewport))
		{
			UE_LOGF(LogDisplayClusterProjectionDomeprojection, Error, "An error occurred during Domeprojection viewport adapter initialization");
		}
		return false;
	}

	UE_LOGF(LogDisplayClusterProjectionDomeprojection, Verbose, "A Domeprojection viewport adapter has been initialized");

	return true;
}

void FDisplayClusterProjectionDomeprojectionPolicyBase::HandleEndScene(IDisplayClusterViewport* InViewport)
{
	check(IsInGameThread());

	ReleaseOriginComponent();
}

void FDisplayClusterProjectionDomeprojectionPolicyBase::ApplyClippingPlanesOverrides(const IDisplayClusterViewport* InViewport, float& InOutNCP, float& InOutFCP) const
{
	// Disable NearClippingPlane for DomeProjection.
	if (!GDisplayClusterProjectionDomeprojectionPolicyEnableNearClippingPlane)
	{
		InOutNCP = 1;
		
		// Infinite FCP
		InOutFCP = InOutNCP;
	}
}

bool FDisplayClusterProjectionDomeprojectionPolicyBase::CalculateView(IDisplayClusterViewport* InViewport, const uint32 InContextNum, FVector& InOutViewLocation, FRotator& InOutViewRotation, const FVector& ViewOffset, const float WorldToMeters, const float InNCP, const float InFCP)
{
	check(IsInGameThread());

	if (!InViewport || !InViewport->GetContexts().IsValidIndex(InContextNum) || !ViewAdapter.IsValid())
	{
		return false;
	}

	// Forward data to the RHI dependend Domeprojection implementation
	InOutViewRotation = FRotator::ZeroRotator;
	if (!ViewAdapter->CalculateView(InViewport, InContextNum, DomeprojectionChannel, InOutViewLocation, InOutViewRotation, ViewOffset, WorldToMeters, InNCP, InFCP))
	{
		if (!IsEditorOperationMode(InViewport))
		{
			UE_LOGF(LogDisplayClusterProjectionDomeprojection, Warning, "Couldn't compute view info for <%ls> viewport", *InViewport->GetId());
		}

		return false;
	}

	return true;
}

bool FDisplayClusterProjectionDomeprojectionPolicyBase::GetProjectionMatrix(IDisplayClusterViewport* InViewport, const uint32 InContextNum, FMatrix& OutPrjMatrix)
{
	check(IsInGameThread());

	if (!ViewAdapter.IsValid())
	{
		return false;
	}

	// Pass request to the adapter
	return ViewAdapter->GetProjectionMatrix(InViewport, InContextNum, DomeprojectionChannel, OutPrjMatrix);
}

bool FDisplayClusterProjectionDomeprojectionPolicyBase::IsWarpBlendSupported(const IDisplayClusterViewport* InViewport) const
{
	return true;
}

bool FDisplayClusterProjectionDomeprojectionPolicyBase::IsWarpBlendSupported_RenderThread(const IDisplayClusterViewportProxy* InViewportProxy) const
{
	return true;
}

void FDisplayClusterProjectionDomeprojectionPolicyBase::ApplyWarpBlend_RenderThread(FRHICommandListImmediate& RHICmdList, const IDisplayClusterViewportProxy* InViewportProxy)
{
	check(IsInRenderingThread());

	if (ViewAdapter.IsValid())
	{
		ViewAdapter->ApplyWarpBlend_RenderThread(RHICmdList, InViewportProxy, DomeprojectionChannel);
	}
}

//////////////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterProjectionDomeprojectionPolicy
//////////////////////////////////////////////////////////////////////////////////////////////
bool FDisplayClusterProjectionDomeprojectionPolicyBase::ReadConfigData(IDisplayClusterViewport* InViewport, FString& OutFile, FString& OutOrigin, uint32& OutChannel, float& OutGeometryScale)
{
	check(InViewport);

	const FString InViewportId = InViewport->GetId();

	// Domeprojection file (mandatory)
	if (DisplayClusterHelpers::map::ExtractValue(GetParameters(), DisplayClusterProjectionStrings::cfg::domeprojection::File, OutFile))
	{
		UE_LOGF(LogDisplayClusterProjectionDomeprojection, Verbose, "Viewport <%ls>: Projection parameter '%ls' - '%ls'", *InViewportId, DisplayClusterProjectionStrings::cfg::domeprojection::File, *OutFile);
	}
	else
	{
		if (!IsEditorOperationMode(InViewport))
		{
			UE_LOGF(LogDisplayClusterProjectionDomeprojection, Error, "Viewport <%ls>: Projection parameter '%ls' not found", *InViewportId, DisplayClusterProjectionStrings::cfg::domeprojection::File);
		}

		return false;
	}

	// Channel (mandatory)
	if(DisplayClusterHelpers::map::ExtractValueFromString(GetParameters(), DisplayClusterProjectionStrings::cfg::domeprojection::Channel, OutChannel))
	{
		UE_LOGF(LogDisplayClusterProjectionDomeprojection, Verbose, "Viewport <%ls>: Projection parameter '%ls' - '%d'", *InViewportId, DisplayClusterProjectionStrings::cfg::domeprojection::Channel, OutChannel);
	}
	else
	{
		if (!IsEditorOperationMode(InViewport))
		{
			UE_LOGF(LogDisplayClusterProjectionDomeprojection, Error, "Viewport <%ls>: Parameter <%ls> not found in the config file", *InViewportId, DisplayClusterProjectionStrings::cfg::domeprojection::Channel);
		}

		return false;
	}

	// Origin node (optional)
	if (DisplayClusterHelpers::map::ExtractValue(GetParameters(), DisplayClusterProjectionStrings::cfg::domeprojection::Origin, OutOrigin))
	{
		UE_LOGF(LogDisplayClusterProjectionDomeprojection, Verbose, "Viewport <%ls>: Projection parameter '%ls' - '%ls'", *InViewportId, DisplayClusterProjectionStrings::cfg::domeprojection::Origin, *OutOrigin);
	}
	else
	{
		if (!IsEditorOperationMode(InViewport))
		{
			UE_LOGF(LogDisplayClusterProjectionDomeprojection, Log, "Viewport <%ls>: No <%ls> parameter found for projection %ls", *InViewportId, DisplayClusterProjectionStrings::cfg::domeprojection::Origin, *OutOrigin);
		}
	}

	// UnitsToMeter (optional)
	if(DisplayClusterHelpers::map::ExtractValueFromString(GetParameters(), DisplayClusterProjectionStrings::cfg::domeprojection::UnitsToMeter, OutGeometryScale))
	{
		UE_LOGF(LogDisplayClusterProjectionDomeprojection, Verbose, "Viewport <%ls>: Projection parameter '%ls' - '%f'", *InViewportId, DisplayClusterProjectionStrings::cfg::domeprojection::UnitsToMeter, OutGeometryScale);
	}
	else
	{
		//@TODO: add LOG when this parameter is added to the domeprojection configuration user interface
	}

	return true;
}
