// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Policy/DisplayClusterProjectionPolicyBase.h"
#include "Policy/Domeprojection/DisplayClusterProjectionDomeprojectionViewAdapterBase.h"


/**
 * Domeprojection projection policy base class
 */
class FDisplayClusterProjectionDomeprojectionPolicyBase
	: public FDisplayClusterProjectionPolicyBase
{
public:
	FDisplayClusterProjectionDomeprojectionPolicyBase(const FString& ProjectionPolicyId, const FDisplayClusterConfigurationProjection* InConfigurationProjectionPolicy);

public:
	//~Begin IDisplayClusterProjectionPolicy
	virtual const FString& GetType() const override;

	virtual bool HandleStartScene(IDisplayClusterViewport* InViewport) override;
	virtual void HandleEndScene(IDisplayClusterViewport* InViewport) override;

	virtual bool CalculateView(IDisplayClusterViewport* InViewport, const uint32 InContextNum, FVector& InOutViewLocation, FRotator& InOutViewRotation, const FVector& ViewOffset, const float WorldToMeters, const float NCP, const float FCP) override;
	virtual bool GetProjectionMatrix(IDisplayClusterViewport* InViewport, const uint32 InContextNum, FMatrix& OutPrjMatrix) override;

	virtual bool IsWarpBlendSupported(const IDisplayClusterViewport* InViewport) const override;
	virtual bool IsWarpBlendSupported_RenderThread(const IDisplayClusterViewportProxy* InViewportProxy) const override;

	virtual void ApplyWarpBlend_RenderThread(FRHICommandListImmediate& RHICmdList, const IDisplayClusterViewportProxy* InViewportProxy) override;
	
	// Request additional targetable resources for domeprojection  external warpblend
	virtual bool ShouldUseAdditionalTargetableResource(const IDisplayClusterViewport* InViewport) const override
	{
		return true;
	}

	virtual float GetGeometryToMeters(const IDisplayClusterViewport* InViewport, const float InWorldToMeters) const override
	{
		return GeometryScale;
	}

	virtual void ApplyClippingPlanesOverrides(const IDisplayClusterViewport* InViewport, float& InOutNCP, float& InOutFCP) const override;

	virtual EDisplayClusterProjectionPolicyFlags GetProjectionPolicyFlags(const IDisplayClusterViewport* InViewport, const uint32 InContextNum) const override
	{
		return EDisplayClusterProjectionPolicyFlags::EnableFollowCameraFeature
			| EDisplayClusterProjectionPolicyFlags::UseEyeTracking
			| EDisplayClusterProjectionPolicyFlags::UseLocalSpace;
	}

	//~End IDisplayClusterProjectionPolicy

protected:
	// Delegate view adapter instantiation to the RHI specific children
	virtual TUniquePtr<FDisplayClusterProjectionDomeprojectionViewAdapterBase> CreateViewAdapter(const FDisplayClusterProjectionDomeprojectionViewAdapterBase::FInitParams& InitParams) = 0;

private:
	// Parse Domeprojection related data from the nDisplay config file
	bool ReadConfigData(IDisplayClusterViewport* InViewport, FString& OutFile, FString& OutOrigin, uint32& OutChannel, float& OutGeometryScale);

private:
	FString OriginCompId;
	uint32 DomeprojectionChannel = 0;

	// Dome geometry units. Default it is in mm.
	//@TODO: add this parameter to the domeprojection configuration UI
	float GeometryScale = 1000.f;

	// RHI depended view adapter (different RHI require different DLL/API etc.)
	TUniquePtr<FDisplayClusterProjectionDomeprojectionViewAdapterBase> ViewAdapter;
};
