// Copyright Epic Games, Inc. All Rights Reserved.

#include "Policy/Domeprojection/DisplayClusterProjectionDomeprojectionPolicyFactory.h"

#include "DisplayClusterProjectionLog.h"
#include "Misc/DisplayClusterStrings.h"

#include "DisplayClusterConfigurationTypes.h"
#include "RHI.h"

#if PLATFORM_WINDOWS
#include "Policy/Domeprojection/Windows/DX11/DisplayClusterProjectionDomeprojectionPolicyDX11.h"
#endif


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProjectionPolicyFactory
//////////////////////////////////////////////////////////////////////////////////////////////
TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe> FDisplayClusterProjectionDomeprojectionPolicyFactory::Create(const FString& ProjectionPolicyId, const FDisplayClusterConfigurationProjection* InConfigurationProjectionPolicy)
{
	if(!InConfigurationProjectionPolicy)
	{
		if (CanShowMsgOnce(ProjectionPolicyId))
		{
			UE_LOGF(LogDisplayClusterProjectionDomeprojection, Error, "Domeprojection policy '%ls': null configuration", *ProjectionPolicyId);
		}
		return nullptr;
	}

	const ERHIInterfaceType RHIType = RHIGetInterfaceType();

#if PLATFORM_WINDOWS
	if (RHIType == ERHIInterfaceType::D3D11)
	{
		UE_LOGF(LogDisplayClusterProjectionDomeprojection, Verbose, "Instantiating projection policy <%ls> id='%ls'", *InConfigurationProjectionPolicy->Type, *ProjectionPolicyId);
		return MakeShared<FDisplayClusterProjectionDomeprojectionPolicyDX11, ESPMode::ThreadSafe>(ProjectionPolicyId, InConfigurationProjectionPolicy);
	}
#endif
	
	if (CanShowMsgOnce(GDynamicRHI->GetName()))
	{
		UE_LOGF(LogDisplayClusterProjectionDomeprojection, Warning, "There is no implementation of '%ls' projection policy for RHI %ls", *InConfigurationProjectionPolicy->Type, GDynamicRHI->GetName());
	}

	return nullptr;
}
