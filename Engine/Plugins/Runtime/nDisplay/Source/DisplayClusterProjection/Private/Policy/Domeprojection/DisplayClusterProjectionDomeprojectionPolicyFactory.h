// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Render/Projection/IDisplayClusterProjectionPolicyFactory.h"
#include "Policy/DisplayClusterProjectionPolicyFactoryBase.h"

/**
 * Factory for the 'domeprojection' projection policy (Windows/DX11 only).
 * Creates policies backed by the Domeprojection warping and blending library,
 * used for dome and planetarium projection setups.
 */
class FDisplayClusterProjectionDomeprojectionPolicyFactory
	: public IDisplayClusterProjectionPolicyFactory
	, protected FDisplayClusterProjectionPolicyFactoryBase
{
public:
	//~ Begin IDisplayClusterProjectionPolicyFactory
	virtual TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe> Create(const FString& ProjectionPolicyId, const FDisplayClusterConfigurationProjection* InConfigurationProjectionPolicy) override;
	//~ End IDisplayClusterProjectionPolicyFactory
};
