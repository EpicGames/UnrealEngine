// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHI.h"

class IDisplayClusterViewport;
class IDisplayClusterViewportProxy;


/**
 * Base Domeprojection view adapter
 */
class FDisplayClusterProjectionDomeprojectionViewAdapterBase
{
public:
	struct FInitParams
	{
		uint32 NumViews;
	};

public:
	FDisplayClusterProjectionDomeprojectionViewAdapterBase(const FInitParams& InitializationParams)
		: InitParams(InitializationParams)
	{ }

	virtual ~FDisplayClusterProjectionDomeprojectionViewAdapterBase() = default;

public:
	virtual bool Initialize(IDisplayClusterViewport* InViewport, const FString& File) = 0;
	
	virtual void Release()
	{ }

public:
	uint32 GetNumViews() const
	{
		return InitParams.NumViews;
	}

public:
	virtual bool CalculateView(IDisplayClusterViewport* InViewport, const uint32 InContextNum, const uint32 Channel, FVector& InOutViewLocation, FRotator& InOutViewRotation, const FVector& ViewOffset, const float WorldToMeters, const float NCP, const float FCP) = 0;
	virtual bool GetProjectionMatrix(IDisplayClusterViewport* InViewport, const uint32 InContextNum, const uint32 Channel, FMatrix& OutPrjMatrix) = 0;
	virtual bool ApplyWarpBlend_RenderThread(FRHICommandListImmediate& RHICmdList, const IDisplayClusterViewportProxy* InViewportProxy, const uint32 Channel) = 0;

private:
	const FInitParams InitParams;
};
