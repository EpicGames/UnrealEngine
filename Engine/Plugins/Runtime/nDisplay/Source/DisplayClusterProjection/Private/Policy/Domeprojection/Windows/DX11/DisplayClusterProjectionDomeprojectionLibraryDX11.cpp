// Copyright Epic Games, Inc. All Rights Reserved.

#include "Policy/Domeprojection/Windows/DX11/DisplayClusterProjectionDomeprojectionLibraryDX11.h"
#include "DisplayClusterProjectionLog.h"

#include "IDisplayClusterProjection.h"

#include "HAL/PlatformProcess.h"
#include "Misc/DisplayClusterHelpers.h"
#include "PDisplayClusterProjectionStrings.h"


void* DisplayClusterProjectionDomeprojectionLibraryDX11::DllHandle = nullptr;
FCriticalSection DisplayClusterProjectionDomeprojectionLibraryDX11::CritSec;
bool DisplayClusterProjectionDomeprojectionLibraryDX11::bInitializeOnce = false;

// Dynamic DLL API:
DisplayClusterProjectionDomeprojectionLibraryDX11::dpCreateContextD3D11 DisplayClusterProjectionDomeprojectionLibraryDX11::dpCreateContextFunc = nullptr;
DisplayClusterProjectionDomeprojectionLibraryDX11::dpDestroyContextD3D11 DisplayClusterProjectionDomeprojectionLibraryDX11::dpDestroyContextFunc = nullptr;
DisplayClusterProjectionDomeprojectionLibraryDX11::dpLoadConfigurationFromFileD3D11 DisplayClusterProjectionDomeprojectionLibraryDX11::dpLoadConfigurationFromFileFunc = nullptr;
DisplayClusterProjectionDomeprojectionLibraryDX11::dpSetClippingPlanesD3D11 DisplayClusterProjectionDomeprojectionLibraryDX11::dpSetClippingPlanesFunc = nullptr;
DisplayClusterProjectionDomeprojectionLibraryDX11::dpSetActiveChannelD3D11 DisplayClusterProjectionDomeprojectionLibraryDX11::dpSetActiveChannelFunc = nullptr;
DisplayClusterProjectionDomeprojectionLibraryDX11::dpSetCorrectionPassD3D11 DisplayClusterProjectionDomeprojectionLibraryDX11::dpSetCorrectionPassFunc = nullptr;
DisplayClusterProjectionDomeprojectionLibraryDX11::dpSetCorrectionPassD3D11_1 DisplayClusterProjectionDomeprojectionLibraryDX11::dpSetCorrectionPass1Func = nullptr;
DisplayClusterProjectionDomeprojectionLibraryDX11::dpSetFlipWarpmeshVerticesYD3D11 DisplayClusterProjectionDomeprojectionLibraryDX11::dpSetFlipWarpmeshVerticesYFunc = nullptr;
DisplayClusterProjectionDomeprojectionLibraryDX11::dpSetFlipWarpmeshTexcoordsVD3D11 DisplayClusterProjectionDomeprojectionLibraryDX11::dpSetFlipWarpmeshTexcoordsVFunc = nullptr;
DisplayClusterProjectionDomeprojectionLibraryDX11::dpPreDrawD3D11_1 DisplayClusterProjectionDomeprojectionLibraryDX11::dpPreDrawFunc = nullptr;
DisplayClusterProjectionDomeprojectionLibraryDX11::dpPostDrawD3D11 DisplayClusterProjectionDomeprojectionLibraryDX11::dpPostDrawFunc = nullptr;
DisplayClusterProjectionDomeprojectionLibraryDX11::dpGetOrientation DisplayClusterProjectionDomeprojectionLibraryDX11::dpGetOrientationFunc = nullptr;


void DisplayClusterProjectionDomeprojectionLibraryDX11::Release()
{
	if (DllHandle)
	{
		FScopeLock lock(&CritSec);

		if (DllHandle)
		{
			dpCreateContextFunc = nullptr;
			dpDestroyContextFunc = nullptr;
			dpLoadConfigurationFromFileFunc = nullptr;
			dpSetClippingPlanesFunc = nullptr;
			dpSetActiveChannelFunc = nullptr;
			dpSetCorrectionPassFunc = nullptr;
			dpSetCorrectionPass1Func = nullptr;
			dpSetFlipWarpmeshVerticesYFunc = nullptr;
			dpSetFlipWarpmeshTexcoordsVFunc = nullptr;
			dpPreDrawFunc = nullptr;
			dpPostDrawFunc = nullptr;
			dpGetOrientationFunc = nullptr;

			FPlatformProcess::FreeDllHandle(DllHandle);
			DllHandle = nullptr;
			bInitializeOnce = false;
		}
	}
}

bool DisplayClusterProjectionDomeprojectionLibraryDX11::Initialize()
{
	if (!DllHandle)
	{
		FScopeLock lock(&CritSec);

		if (!DllHandle && !bInitializeOnce)
		{
			bInitializeOnce = true;

			// Try to load DLL
			const FString DllPath = DisplayClusterHelpers::filesystem::GetFullPathForThirdPartyDLL(DisplayClusterProjectionStrings::ThirdParty::DLL::DomeprojectionDX11);
			const FString DllDirectory = FPaths::GetPath(DllPath);

			FPlatformProcess::PushDllDirectory(*DllDirectory);
			DllHandle = FPlatformProcess::GetDllHandle(*DllPath);
			FPlatformProcess::PopDllDirectory(*DllDirectory);

			if (DllHandle)
			{
				const FString StrOk(TEXT("ok"));
				const FString StrFail(TEXT("NOT FOUND!"));

				// dpCreateContextD3D11
				dpCreateContextFunc = (dpCreateContextD3D11)FPlatformProcess::GetDllExport((HMODULE)DllHandle, TEXT("dpCreateContextD3D11"));
				check(dpCreateContextFunc);
				UE_LOGF(LogDisplayClusterProjectionDomeprojection, Verbose, "Domeprojection API: %ls - %ls.", TEXT("dpCreateContextD3D11"), dpCreateContextFunc ? *StrOk : *StrFail);

				// dpDestroyContextD3D11
				dpDestroyContextFunc = (dpDestroyContextD3D11)FPlatformProcess::GetDllExport((HMODULE)DllHandle, TEXT("dpDestroyContextD3D11"));
				check(dpDestroyContextFunc);
				UE_LOGF(LogDisplayClusterProjectionDomeprojection, Verbose, "Domeprojection API: %ls - %ls.", TEXT("dpDestroyContextD3D11"), dpDestroyContextFunc ? *StrOk : *StrFail);

				// dpLoadConfigurationFromFileD3D11
				dpLoadConfigurationFromFileFunc = (dpLoadConfigurationFromFileD3D11)FPlatformProcess::GetDllExport((HMODULE)DllHandle, TEXT("dpLoadConfigurationFromFileD3D11"));
				check(dpLoadConfigurationFromFileFunc);
				UE_LOGF(LogDisplayClusterProjectionDomeprojection, Verbose, "Domeprojection API: %ls - %ls.", TEXT("dpLoadConfigurationFromFileD3D11"), dpLoadConfigurationFromFileFunc ? *StrOk : *StrFail);

				// dpSetClippingPlanesD3D11
				dpSetClippingPlanesFunc = (dpSetClippingPlanesD3D11)FPlatformProcess::GetDllExport((HMODULE)DllHandle, TEXT("dpSetClippingPlanesD3D11"));
				check(dpSetClippingPlanesFunc);
				UE_LOGF(LogDisplayClusterProjectionDomeprojection, Verbose, "Domeprojection API: %ls - %ls.", TEXT("dpSetClippingPlanesD3D11"), dpSetClippingPlanesFunc ? *StrOk : *StrFail);

				// dpSetActiveChannelD3D11
				dpSetActiveChannelFunc = (dpSetActiveChannelD3D11)FPlatformProcess::GetDllExport((HMODULE)DllHandle, TEXT("dpSetActiveChannelD3D11"));
				check(dpSetActiveChannelFunc);
				UE_LOGF(LogDisplayClusterProjectionDomeprojection, Verbose, "Domeprojection API: %ls - %ls.", TEXT("dpSetActiveChannelD3D11"), dpSetActiveChannelFunc ? *StrOk : *StrFail);

				// dpSetCorrectionPassD3D11
				dpSetCorrectionPassFunc = (dpSetCorrectionPassD3D11)FPlatformProcess::GetDllExport((HMODULE)DllHandle, TEXT("dpSetCorrectionPassD3D11"));
				check(dpSetCorrectionPassFunc);
				UE_LOGF(LogDisplayClusterProjectionDomeprojection, Verbose, "Domeprojection API: %ls - %ls.", TEXT("dpSetCorrectionPassD3D11"), dpSetCorrectionPassFunc ? *StrOk : *StrFail);

				// dpSetCorrectionPassD3D11_1
				dpSetCorrectionPass1Func = (dpSetCorrectionPassD3D11_1)FPlatformProcess::GetDllExport((HMODULE)DllHandle, TEXT("dpSetCorrectionPassD3D11_1"));
				check(dpSetCorrectionPass1Func);
				UE_LOGF(LogDisplayClusterProjectionDomeprojection, Verbose, "Domeprojection API: %ls - %ls.", TEXT("dpSetCorrectionPassD3D11_1"), dpSetCorrectionPass1Func ? *StrOk : *StrFail);

				// dpSetFlipWarpmeshVerticesYD3D11
				dpSetFlipWarpmeshVerticesYFunc = (dpSetFlipWarpmeshVerticesYD3D11)FPlatformProcess::GetDllExport((HMODULE)DllHandle, TEXT("dpSetFlipWarpmeshVerticesYD3D11"));
				check(dpSetFlipWarpmeshVerticesYFunc);
				UE_LOGF(LogDisplayClusterProjectionDomeprojection, Verbose, "Domeprojection API: %ls - %ls.", TEXT("dpSetFlipWarpmeshVerticesYD3D11"), dpSetFlipWarpmeshVerticesYFunc ? *StrOk : *StrFail);

				// dpSetFlipWarpmeshTexcoordsVD3D11
				dpSetFlipWarpmeshTexcoordsVFunc = (dpSetFlipWarpmeshTexcoordsVD3D11)FPlatformProcess::GetDllExport((HMODULE)DllHandle, TEXT("dpSetFlipWarpmeshTexcoordsVD3D11"));
				check(dpSetFlipWarpmeshTexcoordsVFunc);
				UE_LOGF(LogDisplayClusterProjectionDomeprojection, Verbose, "Domeprojection API: %ls - %ls.", TEXT("dpSetFlipWarpmeshTexcoordsVD3D11"), dpSetFlipWarpmeshTexcoordsVFunc ? *StrOk : *StrFail);

				// dpPreDrawD3D11_1
				dpPreDrawFunc = (dpPreDrawD3D11_1)FPlatformProcess::GetDllExport((HMODULE)DllHandle, TEXT("dpPreDrawD3D11_1"));
				check(dpPreDrawFunc);
				UE_LOGF(LogDisplayClusterProjectionDomeprojection, Verbose, "Domeprojection API: %ls - %ls.", TEXT("dpPreDrawD3D11_1"), dpPreDrawFunc ? *StrOk : *StrFail);

				// dpPostDrawD3D11
				dpPostDrawFunc = (dpPostDrawD3D11)FPlatformProcess::GetDllExport((HMODULE)DllHandle, TEXT("dpPostDrawD3D11"));
				check(dpPostDrawFunc);
				UE_LOGF(LogDisplayClusterProjectionDomeprojection, Verbose, "Domeprojection API: %ls - %ls.", TEXT("dpPostDrawD3D11"), dpPostDrawFunc ? *StrOk : *StrFail);

				// dpGetOrientation
				dpGetOrientationFunc = (dpGetOrientation)FPlatformProcess::GetDllExport((HMODULE)DllHandle, TEXT("dpGetOrientation"));
				check(dpGetOrientationFunc);
				UE_LOGF(LogDisplayClusterProjectionDomeprojection, Verbose, "Domeprojection API: %ls - %ls.", TEXT("dpGetOrientation"), dpGetOrientationFunc ? *StrOk : *StrFail);
			}
			else
			{
				UE_LOGF(LogDisplayClusterProjectionDomeprojection, Error, "Couldn't initialize Domeprojection API. No <%ls> library found.", *DllPath);
				return false;
			}
		}
	}

	return DllHandle != nullptr;
}
