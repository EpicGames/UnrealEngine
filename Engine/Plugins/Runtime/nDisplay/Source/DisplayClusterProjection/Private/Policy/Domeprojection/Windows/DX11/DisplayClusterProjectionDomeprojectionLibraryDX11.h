// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RHI.h"
#include "ID3D11DynamicRHI.h"

THIRD_PARTY_INCLUDES_START
#include "dpTypes.h"
THIRD_PARTY_INCLUDES_END


struct DisplayClusterProjectionDomeprojectionLibraryDX11
{
	static bool Initialize();
	static void Release();

	typedef dpResult(__cdecl *dpCreateContextD3D11)(dpContext** ppContext, ID3D11Device* pDevice, unsigned int pluginId);
	static dpCreateContextD3D11 dpCreateContextFunc;

	typedef dpResult(__cdecl *dpDestroyContextD3D11)(dpContext* pContext);
	static dpDestroyContextD3D11 dpDestroyContextFunc;

	typedef dpResult(__cdecl *dpLoadConfigurationFromFileD3D11)(dpContext* pContext, const char* filename);
	static dpLoadConfigurationFromFileD3D11 dpLoadConfigurationFromFileFunc;

	typedef dpResult(__cdecl *dpSetClippingPlanesD3D11)(dpContext* pContext, float cNear, float cFar);
	static dpSetClippingPlanesD3D11 dpSetClippingPlanesFunc;

	typedef dpResult(__cdecl *dpSetActiveChannelD3D11)(dpContext* pContext, unsigned int channelId, ID3D11Device* pDevice, unsigned int width, unsigned int height);
	static dpSetActiveChannelD3D11 dpSetActiveChannelFunc;

	typedef dpResult(__cdecl *dpSetCorrectionPassD3D11)(dpContext* pContext, dpCorrectionPassType pass, bool enabled);
	static dpSetCorrectionPassD3D11 dpSetCorrectionPassFunc;

	typedef dpResult(__cdecl *dpSetCorrectionPassD3D11_1)(dpContext* pContext, dpCorrectionPassType pass, float value);
	static dpSetCorrectionPassD3D11_1 dpSetCorrectionPass1Func;

	typedef dpResult(__cdecl *dpSetFlipWarpmeshVerticesYD3D11)(dpContext* pContext, bool flip);
	static dpSetFlipWarpmeshVerticesYD3D11 dpSetFlipWarpmeshVerticesYFunc;

	typedef dpResult(__cdecl *dpSetFlipWarpmeshTexcoordsVD3D11)(dpContext* pContext, bool flip);
	static dpSetFlipWarpmeshTexcoordsVD3D11 dpSetFlipWarpmeshTexcoordsVFunc;

	typedef dpResult(__cdecl *dpPreDrawD3D11_1)(dpContext* pContext, const dpVec3f eyepoint, dpCamera* pCamera);
	static dpPreDrawD3D11_1 dpPreDrawFunc;

	typedef dpResult(__cdecl *dpPostDrawD3D11)(dpContext* pContext, ID3D11ShaderResourceView* pTexture, ID3D11DeviceContext* pDeviceContext);
	static dpPostDrawD3D11 dpPostDrawFunc;

	typedef dpResult(__cdecl *dpGetOrientation)(const dpVec3f dir, const dpVec3f up, dpVec3f* pOrientation);
	static dpGetOrientation dpGetOrientationFunc;

private:
	static void* DllHandle;
	static FCriticalSection CritSec;
	static bool bInitializeOnce;
};
