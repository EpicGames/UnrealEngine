// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Policy/Domeprojection/DisplayClusterProjectionDomeprojectionPolicyBase.h"
#include "Policy/Domeprojection/Windows/DX11/DisplayClusterProjectionDomeprojectionViewAdapterDX11.h"

#include "DisplayClusterProjectionLog.h"


/**
 * Domeprojection projection policy (DX11 implementation)
 */
class FDisplayClusterProjectionDomeprojectionPolicyDX11
	: public FDisplayClusterProjectionDomeprojectionPolicyBase
{
public:
	FDisplayClusterProjectionDomeprojectionPolicyDX11(const FString& ProjectionPolicyId, const FDisplayClusterConfigurationProjection* InConfigurationProjectionPolicy)
		: FDisplayClusterProjectionDomeprojectionPolicyBase(ProjectionPolicyId, InConfigurationProjectionPolicy)
	{ }

protected:
	virtual TUniquePtr<FDisplayClusterProjectionDomeprojectionViewAdapterBase> CreateViewAdapter(const FDisplayClusterProjectionDomeprojectionViewAdapterBase::FInitParams& InitParams) override
	{
		UE_LOGF(LogDisplayClusterProjectionDomeprojection, Verbose, "Instantiating Domeprojection DX11 viewport adapter...");
		return MakeUnique<FDisplayClusterProjectionDomeprojectionViewAdapterDX11>(InitParams);
	}
};
