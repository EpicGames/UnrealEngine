// Copyright Epic Games, Inc. All Rights Reserved.

#include "Policy/Domeprojection/Windows/DX11/DisplayClusterProjectionDomeprojectionViewAdapterDX11.h"
#include "Policy/Domeprojection/Windows/DX11/DisplayClusterProjectionDomeprojectionLibraryDX11.h"
#include "Policy/DisplayClusterProjectionPolicyBase.h"

#include "DisplayClusterProjectionLog.h"
#include "Misc/DisplayClusterHelpers.h"

#include "ID3D11DynamicRHI.h"

#include "Engine/GameViewportClient.h"
#include "Engine/Engine.h"
#include "Engine/RendererSettings.h"
#include "Misc/Paths.h"
#include "UnrealClient.h"

#include "Render/Viewport/IDisplayClusterViewport.h"
#include "Render/Viewport/IDisplayClusterViewportProxy.h"

#define DP_PLUGIN_ID 12

FDisplayClusterProjectionDomeprojectionViewAdapterDX11::FDisplayClusterProjectionDomeprojectionViewAdapterDX11(const FDisplayClusterProjectionDomeprojectionViewAdapterBase::FInitParams& InitParams)
	: FDisplayClusterProjectionDomeprojectionViewAdapterBase(InitParams)
{
	check(InitParams.NumViews > 0);

	Views.AddDefaulted(InitParams.NumViews);
}

FDisplayClusterProjectionDomeprojectionViewAdapterDX11::~FDisplayClusterProjectionDomeprojectionViewAdapterDX11()
{
	for (FViewData& It : Views)
	{
		It.Release(DllAccessCS);
	}

	Views.Empty();
}

bool FDisplayClusterProjectionDomeprojectionViewAdapterDX11::Initialize(IDisplayClusterViewport* InViewport, const FString& InFile)
{
	bool bResult = true;

	for (FViewData& ViewIt : Views)
	{
		if (!ViewIt.Initialize(InViewport, InFile, DllAccessCS))
		{
			bResult = false;
		}
	}

	return bResult;
}

// Location/Rotation inside the function is in Domeprojection space
bool FDisplayClusterProjectionDomeprojectionViewAdapterDX11::CalculateView(IDisplayClusterViewport* InViewport, const uint32 InContextNum, const uint32 Channel, FVector& InOutViewLocation, FRotator& InOutViewRotation, const FVector& ViewOffset, const float WorldToMeters, const float NCP, const float FCP)
{
	check(GDynamicRHI);
	if (!GDynamicRHI)
	{
		return false;
	}

	ID3D11Device* D3D11Device = GetID3D11DynamicRHI()->RHIGetDevice();
	if (!D3D11Device)
	{
		return false;
	}

	if (InContextNum >= (uint32)(Views.Num()))
	{
		Views.AddDefaulted(InContextNum - Views.Num() + 1);
	}

	ZNear = NCP;
	ZFar  = FCP;

	const auto InContext = InViewport->GetContexts()[InContextNum];
	const auto InViewportSize = InContext.ContextSize;

	const float WorldScale = WorldToMeters / 1000.0f; // we use mm

	dpVec3f Eyepoint(InOutViewLocation.Y / WorldScale, InOutViewLocation.Z / WorldScale, -InOutViewLocation.X / WorldScale);
	dpVec3f Orientation;

	{
		FScopeLock lock(&DllAccessCS);

		check(DisplayClusterProjectionDomeprojectionLibraryDX11::dpSetActiveChannelFunc);
		DisplayClusterProjectionDomeprojectionLibraryDX11::dpSetActiveChannelFunc(Views[InContextNum].Context, Channel, D3D11Device, InViewportSize.X, InViewportSize.Y);

		check(DisplayClusterProjectionDomeprojectionLibraryDX11::dpPreDrawFunc);
		DisplayClusterProjectionDomeprojectionLibraryDX11::dpPreDrawFunc(Views[InContextNum].Context, Eyepoint, &Views[InContextNum].Camera);

		check(DisplayClusterProjectionDomeprojectionLibraryDX11::dpGetOrientationFunc);
		DisplayClusterProjectionDomeprojectionLibraryDX11::dpGetOrientationFunc(Views[InContextNum].Camera.dir, Views[InContextNum].Camera.up, &Orientation);
	}

	// Forward location to a caller
	InOutViewLocation = FVector(-Views[InContextNum].Camera.position.z * WorldScale, Views[InContextNum].Camera.position.x * WorldScale, Views[InContextNum].Camera.position.y * WorldScale);

	// Forward view rotation to a caller
	InOutViewRotation = FRotator(Orientation.y, Orientation.x, -Orientation.z);

	return true;
}

bool FDisplayClusterProjectionDomeprojectionViewAdapterDX11::GetProjectionMatrix(IDisplayClusterViewport* InViewport, const uint32 InContextNum, const uint32 Channel, FMatrix& OutPrjMatrix)
{
	check(Views.Num() > (int32)InContextNum);

	// Build Projection matrix:
	const float Left   = Views[InContextNum].Camera.tanLeft;
	const float Right  = Views[InContextNum].Camera.tanRight;
	const float Bottom = Views[InContextNum].Camera.tanBottom;
	const float Top    = Views[InContextNum].Camera.tanTop;

	InViewport->CalculateProjectionMatrix(InContextNum, Left, Right, Top, Bottom, ZNear, ZFar, false);
	OutPrjMatrix = InViewport->GetContexts()[InContextNum].ProjectionMatrix;

	return true;
}

bool FDisplayClusterProjectionDomeprojectionViewAdapterDX11::ImplApplyWarpBlend_RenderThread(FRHICommandListImmediate& RHICmdList, uint32 InContextNum, const uint32 Channel, FRHITexture* InputTextures, FRHITexture* OutputTextures)
{
	if (!IsRHID3D11())
	{
		return false;
	}

	ID3D11DynamicRHI* D3D11RHI = GetID3D11DynamicRHI();

	ID3D11ShaderResourceView* SrcTextureSRVD3D11 = D3D11RHI->RHIGetShaderResourceView(InputTextures);
	ID3D11RenderTargetView* DstTextureRTV = D3D11RHI->RHIGetRenderTargetView(OutputTextures);
	if (!(SrcTextureSRVD3D11 && DstTextureRTV))
	{
		return false;
	}

	const FIntPoint InViewportSize = InputTextures->GetSizeXY();

	D3D11_VIEWPORT RenderViewportData;
	RenderViewportData.MinDepth = 0.0f;
	RenderViewportData.MaxDepth = 1.0f;
	RenderViewportData.Width = static_cast<float>(InViewportSize.X);
	RenderViewportData.Height = static_cast<float>(InViewportSize.Y);
	RenderViewportData.TopLeftX = 0.0f;
	RenderViewportData.TopLeftY = 0.0f;

	FD3D11Device* D3D11Device = GetID3D11DynamicRHI()->RHIGetDevice();
	FD3D11DeviceContext* DeviceContext = GetID3D11DynamicRHI()->RHIGetDeviceContext();
	if (D3D11Device && DeviceContext)
	{
		DeviceContext->RSSetViewports(1, &RenderViewportData);
		DeviceContext->OMSetRenderTargets(1, &DstTextureRTV, nullptr);
		DeviceContext->Flush();

		// perform warp/blend
		{
			FScopeLock lock(&DllAccessCS);

			check(DisplayClusterProjectionDomeprojectionLibraryDX11::dpPostDrawFunc);
			dpResult Result = DisplayClusterProjectionDomeprojectionLibraryDX11::dpPostDrawFunc(Views[InContextNum].Context, SrcTextureSRVD3D11, DeviceContext);

			if (Result != dpNoError)
			{
				UE_LOGF(LogDisplayClusterProjectionDomeprojection, Error, "Domeprojection couldn't perform rendering operation");
				return false;
			}

			return true;
		}
	}

	UE_LOGF(LogDisplayClusterProjectionDomeprojection, Error, "UE couldn't perform domeprojection rendering on current render device");
	return false;
}

bool FDisplayClusterProjectionDomeprojectionViewAdapterDX11::ApplyWarpBlend_RenderThread(FRHICommandListImmediate& RHICmdList, const IDisplayClusterViewportProxy* InViewportProxy, const uint32 Channel)
{
	check(IsInRenderingThread());

	if (!GEngine)
	{
		return false;
	}

	// Get in\out remp resources ref from viewport
	TArray<FRHITexture*> InputTextures, OutputTextures;

	if (!InViewportProxy->GetResources_RenderThread(EDisplayClusterViewportResourceType::InputShaderResource, InputTextures))
	{
		// no source textures
		return false;
	}

	// Get output resources with rects
	// warp result is now inside AdditionalRTT.  Later, from the DC ViewportManagerProxy it will be resolved to FrameRTT
	if (!InViewportProxy->GetResources_RenderThread(EDisplayClusterViewportResourceType::AfterWarpBlendTargetableResource, OutputTextures))
	{
		return false;
	}

	check(InputTextures.Num() == OutputTextures.Num());
	check(InViewportProxy->GetContexts_RenderThread().Num() == InputTextures.Num());

	// External SDK not use our RHI flow, call flush to finish resolve context image to input resource
	RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThreadFlushResources);

	TRACE_CPUPROFILER_EVENT_SCOPE(nDisplay EasyBlend::Render);
	for (int32 ContextNum = 0; ContextNum < InputTextures.Num(); ContextNum++)
	{
		if (!ImplApplyWarpBlend_RenderThread(RHICmdList, ContextNum, Channel, InputTextures[ContextNum], OutputTextures[ContextNum]))
		{
			return false;
		}
	}

	// warp result is now inside AdditionalRTT.  Later, from the DC ViewportManagerProxy it will be resolved to FrameRTT 
	return true;
}

void FDisplayClusterProjectionDomeprojectionViewAdapterDX11::FViewData::Release(FCriticalSection& InDllAccessCS)
{
	if (Context != nullptr)
	{
		FScopeLock lock(&InDllAccessCS);
		DisplayClusterProjectionDomeprojectionLibraryDX11::dpDestroyContextFunc(Context);
		Context = nullptr;
	}
}

bool FDisplayClusterProjectionDomeprojectionViewAdapterDX11::FViewData::Initialize(IDisplayClusterViewport* InViewport, const FString& InFile, FCriticalSection& InDllAccessCS)
{
	// Initialize Domeprojection DLL API
	FScopeLock lock(&InDllAccessCS);

	if (!DisplayClusterProjectionDomeprojectionLibraryDX11::Initialize())
	{
		if (!FDisplayClusterProjectionPolicyBase::IsEditorOperationMode(InViewport))
		{
			UE_LOGF(LogDisplayClusterProjectionDomeprojection, Error, "Couldn't link to the Domeprojection DLL");
		}

		return false;
	}

	if (InFile.IsEmpty())
	{
		if (!FDisplayClusterProjectionPolicyBase::IsEditorOperationMode(InViewport))
		{
			UE_LOGF(LogDisplayClusterProjectionDomeprojection, Error, "File name is empty");
		}

		return false;
	}

	check(GDynamicRHI);

	dpResult Result = DisplayClusterProjectionDomeprojectionLibraryDX11::dpCreateContextFunc(&Context, GetID3D11DynamicRHI()->RHIGetDevice(), DP_PLUGIN_ID);
	if (Result != dpNoError)
	{
		UE_LOGF(LogDisplayClusterProjectionDomeprojection, Error, "Couldn't initialize Domeprojection context");
		return false;
	}

	// Check if configuration file exists
	if (!FPaths::FileExists(InFile))
	{
		UE_LOGF(LogDisplayClusterProjectionDomeprojection, Error, "File '%ls' not found", *InFile);
		return false;
	}

	// Load the configuration
	Result = DisplayClusterProjectionDomeprojectionLibraryDX11::dpLoadConfigurationFromFileFunc(Context, TCHAR_TO_ANSI(*InFile));
	if (Result != dpNoError)
	{
		UE_LOGF(LogDisplayClusterProjectionDomeprojection, Error, "Loading configuration from %ls failed with %d", *InFile, Result);
		return false;
	}

	return true;
}
