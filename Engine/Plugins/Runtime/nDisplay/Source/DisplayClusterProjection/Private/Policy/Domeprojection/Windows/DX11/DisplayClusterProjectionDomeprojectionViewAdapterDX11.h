// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Policy/Domeprojection/DisplayClusterProjectionDomeprojectionViewAdapterBase.h"

THIRD_PARTY_INCLUDES_START
#include "dpTypes.h"
THIRD_PARTY_INCLUDES_END


/**
 * Domeprojection projection policy (DX11 adapter)
 */
class FDisplayClusterProjectionDomeprojectionViewAdapterDX11
	: public FDisplayClusterProjectionDomeprojectionViewAdapterBase
{
public:
	FDisplayClusterProjectionDomeprojectionViewAdapterDX11(const FDisplayClusterProjectionDomeprojectionViewAdapterBase::FInitParams& InitParams);
	virtual ~FDisplayClusterProjectionDomeprojectionViewAdapterDX11();

public:
	virtual bool Initialize(IDisplayClusterViewport* InViewport, const FString& File) override;

public:
	//////////////////////////////////////////////////////////////////////////////////////////////
	// IDisplayClusterProjectionPolicy
	//////////////////////////////////////////////////////////////////////////////////////////////
	virtual bool CalculateView(IDisplayClusterViewport* InViewport, const uint32 InContextNum, const uint32 Channel, FVector& InOutViewLocation, FRotator& InOutViewRotation, const FVector& ViewOffset, const float WorldToMeters, const float NCP, const float FCP) override;
	virtual bool GetProjectionMatrix(IDisplayClusterViewport* InViewport, const uint32 InContextNum, const uint32 Channel, FMatrix& OutPrjMatrix) override;
	virtual bool ApplyWarpBlend_RenderThread(FRHICommandListImmediate& RHICmdList, const IDisplayClusterViewportProxy* InViewportProxy, const uint32 Channel) override;

protected:
	bool ImplApplyWarpBlend_RenderThread(FRHICommandListImmediate& RHICmdList, uint32 ContextNum, const uint32 Channel, FRHITexture* InputTextures, FRHITexture* OutputTextures);

private:
	float ZNear;
	float ZFar;

	class FViewData
	{
	public:
		bool Initialize(IDisplayClusterViewport* InViewport, const FString& InFile, FCriticalSection& DllAccessCS);
		void Release(FCriticalSection& DllAccessCS);

	public:
		dpCamera   Camera;
		// unique context for each eye (hold warp settings, different for each eye)
		dpContext* Context = nullptr;
	};

	TArray<FViewData> Views;

	FCriticalSection DllAccessCS;
};
