// Copyright Epic Games, Inc. All Rights Reserved.

#include "Policy/EasyBlend/DisplayClusterProjectionEasyBlendPolicy.h"
#include "Policy/EasyBlend/DisplayClusterProjectionEasyBlendPolicyGeometryExportData.h"

#include "DisplayClusterProjectionLog.h"
#include "DisplayClusterProjectionStrings.h"

#include "Misc/DisplayClusterHelpers.h"

#include "IDisplayCluster.h"
#include "Game/IDisplayClusterGameManager.h"

#include "Components/DisplayClusterScreenComponent.h"

#include "Render/Viewport/IDisplayClusterViewport.h"
#include "Render/Viewport/IDisplayClusterViewportProxy.h"

#if PLATFORM_WINDOWS
#include "Policy/EasyBlend/Windows/DX11/DisplayClusterProjectionEasyBlendLibraryDX11.h"
#include "Policy/EasyBlend/Windows/DX12/DisplayClusterProjectionEasyBlendLibraryDX12.h"
#include "Policy/EasyBlend/Windows/DX11/DisplayClusterProjectionEasyBlendPolicyViewDataDX11.h"
#include "Policy/EasyBlend/Windows/DX12/DisplayClusterProjectionEasyBlendPolicyViewDataDX12.h"
#endif

#include "Slate/SceneViewport.h"
#include "Engine/GameViewportClient.h"
#include "Framework/Application/SlateApplication.h"
#include "Rendering/SlateRenderer.h"

#include "ProceduralMeshComponent.h"

namespace UE::DisplayCluster::Projection::EasyBlend
{
	/** Find Viewport RHI.*/
	static inline FRHIViewport* FindRHIViewport(IDisplayClusterViewport* InViewport)
	{
		UWorld* World = InViewport ? InViewport->GetConfiguration().GetCurrentWorld() : nullptr;
		UGameViewportClient* ViewportClient = World ? World->GetGameViewport() : nullptr;
		if (FSceneViewport* SceneViewport = ViewportClient ? ViewportClient->GetGameViewport() : nullptr)
		{
			const TSharedPtr<SWindow> SlateWindow = SceneViewport ? SceneViewport->FindWindow() : nullptr;
			if (SlateWindow.IsValid())
			{
				FSlateRenderer* SlateRenderer = FSlateApplication::Get().GetRenderer();
				if (void* ViewportResource = SlateRenderer ? SlateRenderer->GetViewportResource(*SlateWindow) : nullptr)
				{
					return static_cast<FRHIViewport*>(ViewportResource);
				}
			}
		}

		return nullptr;
	}
};

//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProjectionEasyBlendViewData
//////////////////////////////////////////////////////////////////////////////////////////////
TSharedPtr<IDisplayClusterProjectionEasyBlendPolicyViewData, ESPMode::ThreadSafe> IDisplayClusterProjectionEasyBlendPolicyViewData::Create(const FDisplayClusterProjectionEasyBlendPolicyConfiguration& InEasyBlendConfiguration)
{
	if (!FDisplayClusterProjectionEasyBlendPolicy::IsEasyBlendSupported())
	{
		return nullptr;
	}

	const ERHIInterfaceType RHIType = RHIGetInterfaceType();

#if PLATFORM_WINDOWS
	if (RHIType == ERHIInterfaceType::D3D11)
	{
		TSharedPtr<FDisplayClusterProjectionEasyBlendPolicyViewDataDX11, ESPMode::ThreadSafe> ViewData = MakeShared<FDisplayClusterProjectionEasyBlendPolicyViewDataDX11, ESPMode::ThreadSafe>();
		if (ViewData->Initialize(InEasyBlendConfiguration))
		{
			return ViewData;
		}
	}
	else if (RHIType == ERHIInterfaceType::D3D12)
	{
		TSharedPtr<FDisplayClusterProjectionEasyBlendPolicyViewDataDX12, ESPMode::ThreadSafe> ViewData = MakeShared<FDisplayClusterProjectionEasyBlendPolicyViewDataDX12, ESPMode::ThreadSafe>();
		if (ViewData->Initialize(InEasyBlendConfiguration))
		{
			return ViewData;
		}
	}
#endif

	return nullptr;
}

//////////////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterProjectionEasyBlendPolicy
//////////////////////////////////////////////////////////////////////////////////////////////
bool FDisplayClusterProjectionEasyBlendPolicy::IsEasyBlendSupported()
{
	const ERHIInterfaceType RHIType = RHIGetInterfaceType();

#if PLATFORM_WINDOWS
	if (RHIType == ERHIInterfaceType::D3D11)
	{
		// DX11 requires a EasyBlend DLL
		if (FDisplayClusterProjectionEasyBlendLibraryDX11::Get()->IsInitialized())
		{
			return true;
		}
	}
	else if (RHIType == ERHIInterfaceType::D3D12)
	{
		// DX12 requires a EasyBlend DLL
		if (FDisplayClusterProjectionEasyBlendLibraryDX12::Get()->IsInitialized())
		{
			return true;
		}
	}
#endif

	static bool bIsEasyBlendSupportedLogOnce = false;
	if (!bIsEasyBlendSupportedLogOnce)
	{
		bIsEasyBlendSupportedLogOnce = true;
		UE_LOGF(LogDisplayClusterProjectionEasyBlend, Error, "An error occurred during EasyBlend initialization : %ls RHI not supported", GDynamicRHI->GetName());
	}

	return false;
}

FDisplayClusterProjectionEasyBlendPolicy::FDisplayClusterProjectionEasyBlendPolicy(const FString& ProjectionPolicyId, const FDisplayClusterConfigurationProjection* InConfigurationProjectionPolicy)
	: FDisplayClusterProjectionPolicyBase(ProjectionPolicyId, InConfigurationProjectionPolicy)
{
	UE_LOGF(LogDisplayClusterProjectionEasyBlend, Verbose, "Instantiating projection policy <%ls> id='%ls' for RHI %ls", *GetType(), *ProjectionPolicyId, GDynamicRHI->GetName());
}

FDisplayClusterProjectionEasyBlendPolicy::~FDisplayClusterProjectionEasyBlendPolicy()
{
	ImplRelease();
}

bool FDisplayClusterProjectionEasyBlendPolicy::HandleStartScene(IDisplayClusterViewport* InViewport)
{
	check(IsInGameThread());
	check(InViewport);

	PolicyViewData.Reset();

	if (!EasyBlendConfiguration.Initialize(GetParameters(), InViewport))
	{
		return false;
	}

	// Find origin component if it exists
	InitializeOriginComponent(InViewport, EasyBlendConfiguration.OriginCompId);

	PolicyViewData = IDisplayClusterProjectionEasyBlendPolicyViewData::Create(EasyBlendConfiguration);
	if (!PolicyViewData.IsValid())
	{
		// The current configuration is invalid
		EasyBlendConfiguration.RaiseInvalidConfigurationFlag();

		UE_LOGF(LogDisplayClusterProjectionEasyBlend, Error, "An error occurred during EasyBlend viewport '%ls' initialization", *InViewport->GetId());

		return false;
	}

	PolicyViewInfo.AddDefaulted(2);

	return true;
}

void FDisplayClusterProjectionEasyBlendPolicy::HandleEndScene(IDisplayClusterViewport* InViewport)
{
	check(IsInGameThread());

	ImplRelease();
}

void FDisplayClusterProjectionEasyBlendPolicy::UpdateProxyData(IDisplayClusterViewport* InViewport)
{
	using namespace UE::DisplayCluster::Projection::EasyBlend;
	TArray<FDisplayClusterProjectionEasyBlendPolicyViewInfo> NewPolicyViewInfo = PolicyViewInfo;

	// Send view data to rendering thread.
	ENQUEUE_RENDER_COMMAND(DisplayClusterProjectionEasyBlendPolicy_UpdateProxyData)(
		[InProjectionPolicy = SharedThis(this), InPolicyViewData = PolicyViewData, InPolicyViewInfo = std::move(NewPolicyViewInfo), InRHIViewportProxy = FindRHIViewport(InViewport)](FRHICommandListImmediate& RHICmdList)
		{
			// Update rendering thread resoures
			InProjectionPolicy->RHIViewportProxy = InRHIViewportProxy;
			InProjectionPolicy->PolicyViewDataProxy = InPolicyViewData;
			InProjectionPolicy->PolicyViewInfoProxy = InPolicyViewInfo;
		});
}

void FDisplayClusterProjectionEasyBlendPolicy::ImplRelease()
{
	ReleaseOriginComponent();

	PolicyViewData.Reset();
	PolicyViewInfo.Reset();

	ReleasePreviewMeshComponent();
}

const FString& FDisplayClusterProjectionEasyBlendPolicy::GetType() const
{
	static const FString Type(DisplayClusterProjectionStrings::projection::EasyBlend);
	return Type;
}

bool FDisplayClusterProjectionEasyBlendPolicy::IsWarpBlendSupported(const IDisplayClusterViewport* InViewport) const
{
	return PolicyViewData.IsValid();
}

bool FDisplayClusterProjectionEasyBlendPolicy::IsWarpBlendSupported_RenderThread(const IDisplayClusterViewportProxy* InViewportProxy) const
{
	return PolicyViewDataProxy.IsValid();
}

void FDisplayClusterProjectionEasyBlendPolicy::ApplyWarpBlend_RenderThread(FRHICommandListImmediate& RHICmdList, const IDisplayClusterViewportProxy* InViewportProxy)
{
	check(IsInRenderingThread());
	check(InViewportProxy);

	if (!PolicyViewDataProxy.IsValid())
	{
		return;
	}

	// Get in\out remp resources ref from viewport
	TArray<FRHITexture*> InputTextures, OutputTextures;
	if (InViewportProxy->GetResources_RenderThread(EDisplayClusterViewportResourceType::InputShaderResource, InputTextures)
		&& InViewportProxy->GetResources_RenderThread(EDisplayClusterViewportResourceType::AdditionalTargetableResource, OutputTextures))
	{
		check(InputTextures.Num() == OutputTextures.Num());
		check(InViewportProxy->GetContexts_RenderThread().Num() == InputTextures.Num());

		TRACE_CPUPROFILER_EVENT_SCOPE(FDisplayClusterProjectionEasyBlendPolicy::ApplyWarpBlend_RenderThread);

		// Warp all viewport contexts
		for (int32 ContextNum = 0; ContextNum < PolicyViewInfoProxy.Num(); ContextNum++)
		{
			if (InputTextures.IsValidIndex(ContextNum) && OutputTextures.IsValidIndex(ContextNum))
			{
				PolicyViewDataProxy->ApplyWarpBlend_RenderThread(RHICmdList, PolicyViewInfoProxy[ContextNum], InputTextures[ContextNum], OutputTextures[ContextNum], RHIViewportProxy);
			}
		}
	}
}

bool FDisplayClusterProjectionEasyBlendPolicy::CalculateView(IDisplayClusterViewport* InViewport, const uint32 InContextNum, FVector& InOutViewLocation, FRotator& InOutViewRotation, const FVector& ViewOffset, const float WorldToMeters, const float NCP, const float FCP)
{
	check(IsInGameThread());

	if (!PolicyViewData.IsValid() || !PolicyViewInfo.IsValidIndex(InContextNum))
	{
		return false;
	}
	
	FDisplayClusterProjectionEasyBlendPolicyViewInfo& InOutViewInfo = PolicyViewInfo[InContextNum];

	// Calculate view location in origin space
	const FVector InViewLocation = InOutViewLocation;
	const FRotator InViewRotation = InOutViewRotation;

	// Convert to EasyBlend coordinate system
	InOutViewInfo.ViewLocation = FVector(
		InViewLocation.Y,
		-InViewLocation.Z,
		InViewLocation.X
	);
	InOutViewInfo.ViewRotation = FRotator(
		-InViewRotation.Pitch,
		InViewRotation.Yaw,
		InViewRotation.Roll
	);

	if (!PolicyViewData->CalculateWarpBlend(InOutViewInfo))
	{
		if (!IsEditorOperationMode(InViewport))
		{
			UE_LOGF(LogDisplayClusterProjectionEasyBlend, Warning, "Couldn't compute view info for <%ls> viewport", *InViewport->GetId());
		}

		return false;
	}

	// Convert back from the EasyBlend
	const FRotator OutViewRotation = FRotator(
		-(float)InOutViewInfo.ViewRotation.Pitch,
		(float)InOutViewInfo.ViewRotation.Yaw,
		(float)InOutViewInfo.ViewRotation.Roll
	);

	// Convert rotation back from origin to world space
	InOutViewRotation = OutViewRotation;

	return true;
}

bool FDisplayClusterProjectionEasyBlendPolicy::GetProjectionMatrix(IDisplayClusterViewport* InViewport, const uint32 InContextNum, FMatrix& OutPrjMatrix)
{
	check(IsInGameThread());

	if (!PolicyViewData.IsValid() || !PolicyViewInfo.IsValidIndex(InContextNum))
	{
		return false;
	}

	if (!InViewport || !InViewport->GetContexts().IsValidIndex(InContextNum))
	{
		return false;
	}

	const FDisplayClusterViewport_Context& InContext = InViewport->GetContexts()[InContextNum];

	InViewport->CalculateProjectionMatrix(
		InContextNum,
		PolicyViewInfo[InContextNum].FrustumAngles.X,
		PolicyViewInfo[InContextNum].FrustumAngles.Y,
		PolicyViewInfo[InContextNum].FrustumAngles.Z,
		PolicyViewInfo[InContextNum].FrustumAngles.W,
		InContext.ProjectionData.ZNear, InContext.ProjectionData.ZFar, true
	);

	OutPrjMatrix = InViewport->GetContexts()[InContextNum].ProjectionMatrix;

	return true;
}

void FDisplayClusterProjectionEasyBlendPolicy::ReleasePreviewMeshComponent()
{
	USceneComponent* PreviewMeshComp = PreviewMeshComponentRef.GetOrFindSceneComponent();
	if (PreviewMeshComp != nullptr)
	{
		PreviewMeshComp->UnregisterComponent();
		PreviewMeshComp->DestroyComponent();
	}

	PreviewMeshComponentRef.ResetSceneComponent();
}

bool FDisplayClusterProjectionEasyBlendPolicy::HasPreviewMesh(IDisplayClusterViewport* InViewport)
{
	if (EasyBlendConfiguration.bIsPreviewMeshEnabled && PolicyViewData.IsValid()  && PolicyViewData->HasPreviewMesh())
	{
		return true;
	}

	ReleasePreviewMeshComponent();

	return false;
}

UMeshComponent* FDisplayClusterProjectionEasyBlendPolicy::GetOrCreatePreviewMeshComponent(IDisplayClusterViewport* InViewport, bool& bOutIsRootActorComponent)
{
	check(IsInGameThread());

	if (!HasPreviewMesh(InViewport))
	{
		return nullptr;
	}

	// used created mesh component
	bOutIsRootActorComponent = false;

	USceneComponent* OriginComp = GetPreviewMeshOriginComponent(InViewport);

	// Return Exist mesh component
	USceneComponent* PreviewMeshComp = PreviewMeshComponentRef.GetOrFindSceneComponent();
	if (PreviewMeshComp != nullptr)
	{
		UProceduralMeshComponent* PreviewMesh = Cast<UProceduralMeshComponent>(PreviewMeshComp);
		if (PreviewMesh != nullptr)
		{
			// update attachment to parent
			PreviewMesh->AttachToComponent(OriginComp, FAttachmentTransformRules(EAttachmentRule::KeepRelative, false));
			return PreviewMesh;
		}
	}

	// Get geometry data
	TSharedPtr<FDisplayClusterProjectionEasyBlendGeometryExportData, ESPMode::ThreadSafe> GeometryExportData = FDisplayClusterProjectionEasyBlendGeometryExportData::Create(PolicyViewData.ToSharedRef(), EasyBlendConfiguration);
	if (GeometryExportData.IsValid())
	{
		// Create new WarpMesh component
		const FString CompName = FString::Printf(TEXT("EasyBlend_%s_impl"), *GetId());

		// Creta new object
		UProceduralMeshComponent* MeshComp = NewObject<UProceduralMeshComponent>(OriginComp, FName(*CompName), EObjectFlags::RF_DuplicateTransient | RF_Transient | RF_TextExportTransient);
		if (MeshComp)
		{
			MeshComp->RegisterComponent();
			MeshComp->AttachToComponent(OriginComp, FAttachmentTransformRules(EAttachmentRule::KeepRelative, false));
			MeshComp->CreateMeshSection(0, GeometryExportData->Vertices, GeometryExportData->Triangles, GeometryExportData->Normal, GeometryExportData->UV, TArray<FColor>(), TArray<FProcMeshTangent>(), false);

#if WITH_EDITOR
			MeshComp->SetIsVisualizationComponent(true);
#endif

			// Because of "nDisplay.render.show.visualizationcomponents" we need extra flag to exclude this geometry from render
			MeshComp->SetHiddenInGame(true);

			// Store reference to mesh component
			PreviewMeshComponentRef.SetSceneComponent(MeshComp);

			return MeshComp;
		}
	}

	return nullptr;
}
