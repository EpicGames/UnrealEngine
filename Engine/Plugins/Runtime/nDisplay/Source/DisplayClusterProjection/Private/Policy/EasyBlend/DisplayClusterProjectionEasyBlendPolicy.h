// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Policy/DisplayClusterProjectionPolicyBase.h"
#include "Policy/EasyBlend/DisplayClusterProjectionEasyBlendPolicyConfiguration.h"
#include "Policy/EasyBlend/IDisplayClusterProjectionEasyBlendPolicyViewData.h"
#include "Templates/SharedPointer.h"

class FRHIViewport;

/**
 * EasyBlend projection policy
 */
class FDisplayClusterProjectionEasyBlendPolicy
	: public FDisplayClusterProjectionPolicyBase
	, public TSharedFromThis<FDisplayClusterProjectionEasyBlendPolicy, ESPMode::ThreadSafe>
{
public:
	FDisplayClusterProjectionEasyBlendPolicy(const FString& ProjectionPolicyId, const FDisplayClusterConfigurationProjection* InConfigurationProjectionPolicy);
	virtual ~FDisplayClusterProjectionEasyBlendPolicy();

public:
	//~BEGIN IDisplayClusterProjectionPolicy
	virtual const FString& GetType() const override;

	virtual bool HandleStartScene(IDisplayClusterViewport* InViewport) override;
	virtual void HandleEndScene(IDisplayClusterViewport* InViewport) override;

	virtual bool CalculateView(IDisplayClusterViewport* InViewport, const uint32 InContextNum, FVector& InOutViewLocation, FRotator& InOutViewRotation, const FVector& ViewOffset, const float WorldToMeters, const float NCP, const float FCP) override;
	virtual bool GetProjectionMatrix(IDisplayClusterViewport* InViewport, const uint32 InContextNum, FMatrix& OutPrjMatrix) override;

	virtual bool IsWarpBlendSupported(const IDisplayClusterViewport* InViewport) const override;
	virtual bool IsWarpBlendSupported_RenderThread(const IDisplayClusterViewportProxy* InViewportProxy) const override;

	virtual void ApplyWarpBlend_RenderThread(FRHICommandListImmediate& RHICmdList, const IDisplayClusterViewportProxy* InViewportProxy) override;

	virtual float GetGeometryToMeters(const IDisplayClusterViewport* InViewport, const float InWorldToMeters) const override
	{
		// Return how many geometry units in 1 meter
		return EasyBlendConfiguration.GeometryScale;
	}

	virtual EDisplayClusterProjectionPolicyFlags GetProjectionPolicyFlags(const IDisplayClusterViewport* InViewport, const uint32 InContextNum) const override
	{
		return EDisplayClusterProjectionPolicyFlags::EnableFollowCameraFeature
			| EDisplayClusterProjectionPolicyFlags::UseEyeTracking
			| EDisplayClusterProjectionPolicyFlags::UseLocalSpace;
	}

	virtual FVector GetOriginLocationOffsetInGeometryUnits(const IDisplayClusterViewport* InViewport) const override
	{
		return EasyBlendConfiguration.OriginOffset;
	}

	// Request additional targetable resources for easyblend external warpblend
	virtual bool ShouldUseAdditionalTargetableResource(const IDisplayClusterViewport* InViewport) const override
	{
		return true;
	}

	virtual void UpdateProxyData(IDisplayClusterViewport* InViewport) override;

	virtual bool HasPreviewMesh(IDisplayClusterViewport* InViewport) override;
	virtual class UMeshComponent* GetOrCreatePreviewMeshComponent(IDisplayClusterViewport* InViewport, bool& bOutIsRootActorComponent) override;

	//~~END IDisplayClusterProjectionPolicy

	/** Returns true if the current OS and RHI has an EasyBlend implementation. */
	static bool IsEasyBlendSupported();

protected:
	void ReleasePreviewMeshComponent();
	void ImplRelease();

private:
	// Configuration of the EasyBlend
	FDisplayClusterProjectionEasyBlendPolicyConfiguration EasyBlendConfiguration;

	// Views data for game and rendering thread
	TSharedPtr<IDisplayClusterProjectionEasyBlendPolicyViewData, ESPMode::ThreadSafe> PolicyViewData;
	TArray<FDisplayClusterProjectionEasyBlendPolicyViewInfo> PolicyViewInfo;

	TSharedPtr<IDisplayClusterProjectionEasyBlendPolicyViewData, ESPMode::ThreadSafe> PolicyViewDataProxy;
	TArray<FDisplayClusterProjectionEasyBlendPolicyViewInfo> PolicyViewInfoProxy;

	FDisplayClusterSceneComponentRef PreviewMeshComponentRef;

	// Viewport RHI
	FRHIViewport* RHIViewportProxy = nullptr;
};
