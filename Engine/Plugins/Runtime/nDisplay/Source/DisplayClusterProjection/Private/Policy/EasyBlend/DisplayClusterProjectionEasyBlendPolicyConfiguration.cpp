// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterProjectionEasyBlendPolicyConfiguration.h"

#include "DisplayClusterProjectionStrings.h"
#include "DisplayClusterProjectionLog.h"
#include "Misc/DisplayClusterHelpers.h"
#include "Policy/DisplayClusterProjectionPolicyBase.h"
#include "Render/Viewport/IDisplayClusterViewport.h"

#include "Misc/DisplayClusterHelpers.h"

//////////////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterProjectionEasyBlendPolicyConfiguration
//////////////////////////////////////////////////////////////////////////////////////////////
bool FDisplayClusterProjectionEasyBlendPolicyConfiguration::Initialize(const TMap<FString, FString>& InParameters, IDisplayClusterViewport* InViewport)
{
	if (bInitializeOnce)
	{
		return !bInvalidConfiguration;
	}

	bInitializeOnce = true;

	check(InViewport);

	bInvalidConfiguration = true;

	const FString InViewportId = InViewport->GetId();
	const bool bEnableExtraLog = !FDisplayClusterProjectionPolicyBase::IsEditorOperationMode(InViewport);

	// Origin node (optional)
	if (DisplayClusterHelpers::map::template ExtractValueFromString(InParameters, DisplayClusterProjectionStrings::cfg::easyblend::Origin, OriginCompId))
	{
		UE_LOGF(LogDisplayClusterProjectionEasyBlend, Verbose, "Viewport <%ls>: Projection parameter '%ls' - '%ls'", *InViewportId, DisplayClusterProjectionStrings::cfg::easyblend::Origin, *OriginCompId);
	}
	else
	{
		if (bEnableExtraLog)
		{
			UE_LOGF(LogDisplayClusterProjectionEasyBlend, Error, "Viewport <%ls>: Projection parameter '%ls' not found", *InViewportId, DisplayClusterProjectionStrings::cfg::easyblend::Origin);
		}

		return false;
	}

	if (DisplayClusterHelpers::map::template ExtractValueFromString(InParameters, DisplayClusterProjectionStrings::cfg::mpcdi::EnablePreview, bIsPreviewMeshEnabled))
	{
		UE_LOGF(LogDisplayClusterProjectionEasyBlend, Verbose, "Viewport <%ls>: Projection parameter '%ls' - '%ls'", *InViewportId, DisplayClusterProjectionStrings::cfg::mpcdi::EnablePreview, *DisplayClusterHelpers::str::BoolToStr(bIsPreviewMeshEnabled));
	}


	// EasyBlend file (mandatory)
	FString InCalibrationFile;
	if (DisplayClusterHelpers::map::template ExtractValueFromString(InParameters, DisplayClusterProjectionStrings::cfg::easyblend::File, InCalibrationFile))
	{
		UE_LOGF(LogDisplayClusterProjectionEasyBlend, Verbose, "Viewport <%ls>: Projection parameter '%ls' - '%ls'", *InViewportId, DisplayClusterProjectionStrings::cfg::easyblend::File, *InCalibrationFile);

		// Calibration file must be not empty
		if (InCalibrationFile.IsEmpty())
		{
			if (bEnableExtraLog)
			{
				UE_LOGF(LogDisplayClusterProjectionEasyBlend, Error, "EasyBlend calibration file is undefined (Empty)");
			}

			return false;
		}

		// Calibration file must exists
		CalibrationFile = DisplayClusterHelpers::filesystem::GetFullPathForConfigResource(InCalibrationFile);
		if (!FPaths::FileExists(CalibrationFile))
		{
			UE_LOGF(LogDisplayClusterProjectionEasyBlend, Error, "EasyBlend calibration file not found at path '%ls'.", *InCalibrationFile);

			return false;
		}
	}
	else
	{
		if (bEnableExtraLog)
		{
			UE_LOGF(LogDisplayClusterProjectionEasyBlend, Error, "Viewport <%ls>: Projection parameter '%ls' not found", *InViewportId, DisplayClusterProjectionStrings::cfg::easyblend::File);
		}
		return false;
	}

	// Geometry scale (optional)
	if (DisplayClusterHelpers::map::template ExtractValueFromString(InParameters, DisplayClusterProjectionStrings::cfg::easyblend::Scale, GeometryScale))
	{
		UE_LOGF(LogDisplayClusterProjectionEasyBlend, Verbose, "Viewport <%ls>: Projection parameter '%ls' - %f", *InViewportId, DisplayClusterProjectionStrings::cfg::easyblend::Scale, GeometryScale);

		// Geometry scale can't be less or equal to zero
		if (GeometryScale <= 0.f)
		{
			GeometryScale = 1.f;
		}
	}
	else
	{
		if (bEnableExtraLog)
		{
			UE_LOGF(LogDisplayClusterProjectionEasyBlend, Log, "Viewport <%ls>: Projection parameter '%ls' not found", *InViewportId, DisplayClusterProjectionStrings::cfg::easyblend::Scale);
		}
	}

	// Origin offset
	if (DisplayClusterHelpers::map::template ExtractValueFromString(InParameters, DisplayClusterProjectionStrings::cfg::easyblend::OriginOffset, OriginOffset))
	{
		UE_LOGF(LogDisplayClusterProjectionEasyBlend, Verbose, "Viewport <%ls>: Projection parameter '%ls' - %ls", *InViewportId, DisplayClusterProjectionStrings::cfg::easyblend::OriginOffset, *OriginOffset.ToString());
	}
	else
	{
		if (bEnableExtraLog)
		{
			UE_LOGF(LogDisplayClusterProjectionEasyBlend, Log, "Viewport <%ls>: Projection parameter '%ls' not found", *InViewportId, DisplayClusterProjectionStrings::cfg::easyblend::OriginOffset);
		}
	}

	bInvalidConfiguration = false;

	return true;
}


FString FDisplayClusterProjectionEasyBlendPolicyConfiguration::ToString(const bool bOnlyGeometryParameters) const
{
	check(!CalibrationFile.IsEmpty());

	return FString::Printf(TEXT("%s='%s', %s=%f"),
		DisplayClusterProjectionStrings::cfg::easyblend::File, *CalibrationFile,
		DisplayClusterProjectionStrings::cfg::easyblend::Scale, GeometryScale
	);
}
