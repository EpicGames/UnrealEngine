// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class IDisplayClusterViewport;

/**
* EasyBlend projection policy configuration data
*/
struct FDisplayClusterProjectionEasyBlendPolicyConfiguration
{
public:
	/** Initialize exsyblend configuration from parameters. */
	bool Initialize(const TMap<FString, FString>& InParameters, IDisplayClusterViewport* InViewport);

	void RaiseInvalidConfigurationFlag()
	{
		bInvalidConfiguration = true;
	}

	/**
	* convert all parameters to string
	*/
	FString ToString(const bool bOnlyGeometryParameters = false) const;

public:
	// Path to EasyBlend calibration file
	FString CalibrationFile;

	// Origin component name
	FString OriginCompId;

	// EasyBlend unit scale
	float GeometryScale = 1.f;

	// Origin's location in geometry units.
	FVector OriginOffset = FVector::ZeroVector;

	// Allow to use preview mesh
	bool bIsPreviewMeshEnabled = false;

private:
	// Discard this configuration
	bool bInvalidConfiguration = false;

	// Since the projection policy has const-parameters, we must initialize the configuration only once
	bool bInitializeOnce = false;
};
