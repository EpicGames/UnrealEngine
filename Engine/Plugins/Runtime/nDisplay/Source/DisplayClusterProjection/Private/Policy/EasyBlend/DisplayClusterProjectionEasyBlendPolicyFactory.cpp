// Copyright Epic Games, Inc. All Rights Reserved.

#include "Policy/EasyBlend/DisplayClusterProjectionEasyBlendPolicyFactory.h"
#include "Policy/EasyBlend/DisplayClusterProjectionEasyBlendPolicy.h"

#include "DisplayClusterProjectionLog.h"
#include "DisplayClusterProjectionStrings.h"

#include "DisplayClusterConfigurationTypes.h"
#include "RHI.h"

//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProjectionPolicyFactory
//////////////////////////////////////////////////////////////////////////////////////////////
TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe> FDisplayClusterProjectionEasyBlendPolicyFactory::Create(const FString& ProjectionPolicyId, const FDisplayClusterConfigurationProjection* InConfigurationProjectionPolicy)
{
	if(!InConfigurationProjectionPolicy)
	{
		if (CanShowMsgOnce(ProjectionPolicyId))
		{
			UE_LOGF(LogDisplayClusterProjectionEasyBlend, Error, "EasyBlend policy '%ls': null configuration", *ProjectionPolicyId);
		}
		return nullptr;
	}

	if (FDisplayClusterProjectionEasyBlendPolicy::IsEasyBlendSupported())
	{
		return MakeShared<FDisplayClusterProjectionEasyBlendPolicy, ESPMode::ThreadSafe>(ProjectionPolicyId, InConfigurationProjectionPolicy);
	}

	if (CanShowMsgOnce(GDynamicRHI->GetName()))
	{
		UE_LOGF(LogDisplayClusterProjectionEasyBlend, Warning, "There is no implementation of '%ls' projection policy for RHI %ls", *InConfigurationProjectionPolicy->Type, GDynamicRHI->GetName());
	}
	
	return nullptr;
}
