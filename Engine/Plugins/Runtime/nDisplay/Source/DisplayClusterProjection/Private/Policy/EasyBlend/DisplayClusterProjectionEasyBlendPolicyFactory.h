// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Render/Projection/IDisplayClusterProjectionPolicyFactory.h"
#include "Policy/DisplayClusterProjectionPolicyFactoryBase.h"

/**
 * Factory for the 'easyblend' projection policy.
 * Creates policies backed by the Scalable Display EasyBlend warping and blending library.
 * The factory owns shared library instances for DX11 and DX12; the appropriate one
 * is selected at policy creation time based on the active RHI.
 */
class FDisplayClusterProjectionEasyBlendPolicyFactory
	: public IDisplayClusterProjectionPolicyFactory
	, protected FDisplayClusterProjectionPolicyFactoryBase
{
public:
	//~ Begin IDisplayClusterProjectionPolicyFactory
	virtual TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe> Create(const FString& ProjectionPolicyId, const FDisplayClusterConfigurationProjection* InConfigurationProjectionPolicy) override;
	//~ End IDisplayClusterProjectionPolicyFactory

private:
	/** EasyBlend runtime library instances, one per supported RHI. Shared across all policies created by this factory. */
	TSharedPtr<class FDisplayClusterProjectionEasyBlendLibraryDX11, ESPMode::ThreadSafe> EasyBlendLibraryDX11;
	TSharedPtr<class FDisplayClusterProjectionEasyBlendLibraryDX12, ESPMode::ThreadSafe> EasyBlendLibraryDX12;
};
