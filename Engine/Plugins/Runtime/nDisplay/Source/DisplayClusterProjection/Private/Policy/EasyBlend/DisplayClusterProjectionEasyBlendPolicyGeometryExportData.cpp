// Copyright Epic Games, Inc. All Rights Reserved.

#include "Policy/EasyBlend/DisplayClusterProjectionEasyBlendPolicyGeometryExportData.h"
#include "Misc/DisplayClusterDataCache.h"

#include "HAL/IConsoleManager.h"

int32 GDisplayClusterRender_EasyBlendPreviewMeshCacheEnable = 1;
static FAutoConsoleVariableRef CDisplayClusterRender_EasyBlendPreviewMeshCacheEnable(
	TEXT("nDisplay.cache.EasyBlend.PreviewMesh.enable"),
	GDisplayClusterRender_EasyBlendPreviewMeshCacheEnable,
	TEXT("Enables the use of the EasyBlend preview mesh cache.\n"),
	ECVF_Default
);

int32 GDisplayClusterRender_EasyBlendPreviewMeshCacheTimeOutInFrames = 5 * 60 * 60; // Timeout is 5 minutes (for 60 frames per second)
static FAutoConsoleVariableRef CVarDisplayClusterRender_EasyBlendPreviewMeshCacheTimeOutInFrames(
	TEXT("nDisplay.cache.EasyBlend.PreviewMesh.TimeOut"),
	GDisplayClusterRender_EasyBlendPreviewMeshCacheTimeOutInFrames,
	TEXT("The timeout value in frames  for cached EasyBlend preview mesh.\n")
	TEXT("-1 - disable timeout.\n"),
	ECVF_Default
);

//////////////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterProjectionEasyBlendGeometryExportData
//////////////////////////////////////////////////////////////////////////////////////////////
void FDisplayClusterProjectionEasyBlendGeometryExportData::GenerateGeometryNormals()
{
	Normal.Empty();

	const int32 TotalTriangles = Triangles.Num() / 3;
	for (int32 TriIndex = 0; TriIndex < TotalTriangles; TriIndex++)
	{
		const int32 InTriangleIndex = TriIndex * 3;

		const int32 f0 = Triangles[InTriangleIndex];
		const int32 f1 = Triangles[InTriangleIndex + 1];
		const int32 f2 = Triangles[InTriangleIndex + 2];

		// revert tri to CW (face)
		Triangles[InTriangleIndex] = f2;
		Triangles[InTriangleIndex + 1] = f1;
		Triangles[InTriangleIndex + 2] = f0;

		//Update normal
		const FVector FaceDir1 = Vertices[f1] - Vertices[f0];
		const FVector FaceDir2 = Vertices[f2] - Vertices[f0];

		const FVector TriNormal = FVector::CrossProduct(FaceDir1, FaceDir2).GetSafeNormal();

		Normal.Add(TriNormal);
		Normal.Add(TriNormal);
		Normal.Add(TriNormal);
	}
}

int32 FDisplayClusterProjectionEasyBlendGeometryExportData::GetDataCacheTimeOutInFrames()
{
	return FMath::Max(0, GDisplayClusterRender_EasyBlendPreviewMeshCacheTimeOutInFrames);
}

bool FDisplayClusterProjectionEasyBlendGeometryExportData::IsDataCacheEnabled()
{
	return GDisplayClusterRender_EasyBlendPreviewMeshCacheEnable != 0;
}

class FDisplayClusterWarpBlendEasyBlendPreviewMeshCache
	: public TDisplayClusterDataCache<FDisplayClusterProjectionEasyBlendGeometryExportData>
{
public:
	static TSharedPtr<FDisplayClusterProjectionEasyBlendGeometryExportData, ESPMode::ThreadSafe> GetOrCreateEasyBlendPreviewMesh(const TSharedRef<IDisplayClusterProjectionEasyBlendPolicyViewData, ESPMode::ThreadSafe>& InViewData, const FDisplayClusterProjectionEasyBlendPolicyConfiguration& InConfigData)
	{
		static FDisplayClusterWarpBlendEasyBlendPreviewMeshCache EasyBlendPreviewMeshCacheSingleton;

		// Only for parameters related to geometry
		const FString UniqueName = HashString(InConfigData.ToString(true));

		TSharedPtr<FDisplayClusterProjectionEasyBlendGeometryExportData, ESPMode::ThreadSafe> GeometryExportDataRef = EasyBlendPreviewMeshCacheSingleton.Find(UniqueName);
		if (!GeometryExportDataRef.IsValid())
		{
			GeometryExportDataRef = MakeShared<FDisplayClusterProjectionEasyBlendGeometryExportData, ESPMode::ThreadSafe>(UniqueName);

			if (InViewData->GetPreviewMeshGeometry(InConfigData , *GeometryExportDataRef.Get()))
			{
				EasyBlendPreviewMeshCacheSingleton.Add(GeometryExportDataRef);
			}
			else
			{
				// Unable to initialize, invalid data must be deleted
				GeometryExportDataRef.Reset();
			}

		}

		return GeometryExportDataRef;
	}
};

TSharedPtr<FDisplayClusterProjectionEasyBlendGeometryExportData, ESPMode::ThreadSafe> FDisplayClusterProjectionEasyBlendGeometryExportData::Create(const TSharedRef<IDisplayClusterProjectionEasyBlendPolicyViewData, ESPMode::ThreadSafe>& InViewData, const FDisplayClusterProjectionEasyBlendPolicyConfiguration& InConfigData)
{
	return FDisplayClusterWarpBlendEasyBlendPreviewMeshCache::GetOrCreateEasyBlendPreviewMesh(InViewData, InConfigData);
}
