// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "Policy/EasyBlend/DisplayClusterProjectionEasyBlendPolicyConfiguration.h"
#include "Policy/EasyBlend/IDisplayClusterProjectionEasyBlendPolicyViewData.h"

/**
* Export warp geometry
*/
class FDisplayClusterProjectionEasyBlendGeometryExportData
{
public:
	FDisplayClusterProjectionEasyBlendGeometryExportData(const FString& InUniqueName)
		: UniqueName(InUniqueName)
	{ }

	/** Generate normals. */
	void GenerateGeometryNormals();

	//~ Begin TDisplayClusterDataCache
	/** Return DataCache timeout in frames. */
	static int32 GetDataCacheTimeOutInFrames();

	/** Return true if DataCache is enabled. */
	static bool IsDataCacheEnabled();

	/** Returns the unique name of this MPCDI file resource for DataCache. */
	inline const FString& GetDataCacheName() const
	{
		return UniqueName;
	}

	/** Method for releasing a cached data item, called before its destructor. */
	inline void ReleaseDataCacheItem()
	{ }
	// ~~ End TDisplayClusterDataCache

	static TSharedPtr<FDisplayClusterProjectionEasyBlendGeometryExportData, ESPMode::ThreadSafe> Create(const TSharedRef<IDisplayClusterProjectionEasyBlendPolicyViewData, ESPMode::ThreadSafe>& InViewData, const FDisplayClusterProjectionEasyBlendPolicyConfiguration& InConfigData);

public:
	const FString UniqueName;

	TArray<FVector>   Vertices;
	TArray<FVector>   Normal;
	TArray<FVector2D> UV;
	TArray<int32>     Triangles;

	float GeometryScale = 1;
};
