// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "Policy/EasyBlend/DisplayClusterProjectionEasyBlendPolicyConfiguration.h"
#include "RHI.h"

class FDisplayClusterProjectionEasyBlendGeometryExportData;
class IDisplayClusterViewport;
class FRHIViewport;

/**
* View information
*/
struct FDisplayClusterProjectionEasyBlendPolicyViewInfo
{
	FVector  ViewLocation;
	FRotator ViewRotation;
	FVector4 FrustumAngles; // L R T B
};

/**
 *  EasyBlend view data interface
 */
class IDisplayClusterProjectionEasyBlendPolicyViewData
{
public:
	virtual ~IDisplayClusterProjectionEasyBlendPolicyViewData() = default;

	/** Creates and initializes the EasyBlend interface for the current RHI and OS.
	* Returns nullptr if it is not implemented or cannot be initialized.
	*/
	static TSharedPtr<IDisplayClusterProjectionEasyBlendPolicyViewData, ESPMode::ThreadSafe> Create(const FDisplayClusterProjectionEasyBlendPolicyConfiguration& InEasyBlendConfiguration);

public:
	/** Initialize EasyBlend view data. */
	virtual bool Initialize(const FDisplayClusterProjectionEasyBlendPolicyConfiguration& InEasyBlendConfiguration) { return false; }

	virtual bool CalculateWarpBlend(FDisplayClusterProjectionEasyBlendPolicyViewInfo& InOutViewInfo) { return false; }
	virtual bool ApplyWarpBlend_RenderThread(FRHICommandListImmediate& RHICmdList, const FDisplayClusterProjectionEasyBlendPolicyViewInfo& InViewInfo, FRHITexture* InputTexture, FRHITexture* OutputTexture, FRHIViewport* InRHIViewport) { return false; }

	/**
	* Export warp mesh geometry from EasyBlend file
	*/
	virtual bool HasPreviewMesh() { return false; }
	virtual bool GetPreviewMeshGeometry(const FDisplayClusterProjectionEasyBlendPolicyConfiguration& InEasyBlendConfiguration, FDisplayClusterProjectionEasyBlendGeometryExportData& OutMeshData) { return false; }

protected:
	// Lock access to EasyBlendMeshData
	FCriticalSection EasyBlendMeshDataAccessCS;

	// Whether EasyBlendMeshData requires a call to the EasyBlend1Uninitialize() function.
	bool bIsEasyBlendMeshDataInitialized = false;

	// The rendering resources must be initialized once or the rendering thread must be initialized
	bool bIsRenderResourcesInitialized = false;
};
