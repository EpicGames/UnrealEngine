// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterProjectionEasyBlendLibraryDX11.h"

#include "DisplayClusterProjectionLog.h"
#include "PDisplayClusterProjectionStrings.h"

#include "HAL/PlatformProcess.h"
#include "Misc/DisplayClusterHelpers.h"
#include "PDisplayClusterProjectionStrings.h"

TSharedRef<FDisplayClusterProjectionEasyBlendLibraryDX11, ESPMode::ThreadSafe>& FDisplayClusterProjectionEasyBlendLibraryDX11::Get()
{
	static TSharedRef<FDisplayClusterProjectionEasyBlendLibraryDX11, ESPMode::ThreadSafe> EasyBlendLibraryDX11 = MakeShared<FDisplayClusterProjectionEasyBlendLibraryDX11, ESPMode::ThreadSafe>();

	return EasyBlendLibraryDX11;
}

#define ImportWarpBlendFunc(DLLFunc)\
	EasyBlend1##DLLFunc = (EasyBlend1##DLLFunc##Proc)FPlatformProcess::GetDllExport((HMODULE)EasyBlend_DLL_Handler, TEXT("EasyBlend"#DLLFunc));\
	if(EasyBlend1##DLLFunc==nullptr)\
		{\
			UE_LOGF(LogDisplayClusterProjectionEasyBlend, Error, "Couldn't initialize EasyBlend DX11 API func '%ls'.", TEXT("EasyBlend"#DLLFunc));\
			return false;\
		}

#define ReleaseWarpBlendFunc(DLLFunc) EasyBlend1##DLLFunc = nullptr;

bool FDisplayClusterProjectionEasyBlendLibraryDX11::InitializeDLL()
{
	if (EasyBlend_DLL_Handler)
	{
		ImportWarpBlendFunc(Initialize);
		ImportWarpBlendFunc(Uninitialize);

		ImportWarpBlendFunc(InitDeviceObjects);

		ImportWarpBlendFunc(SetEyepoint);
		ImportWarpBlendFunc(SDK_GetHeadingPitchRoll);

		ImportWarpBlendFunc(DXRender);
		ImportWarpBlendFunc(SetInputTexture2D);
		ImportWarpBlendFunc(SetOutputTexture2D);
	}

	// All EasyBlend  DLL functions are initialized successfully
	bInitialized = true;

	return true;
}

void FDisplayClusterProjectionEasyBlendLibraryDX11::ReleaseDLL()
{
	bInitialized = false;

	ReleaseWarpBlendFunc(Initialize);
	ReleaseWarpBlendFunc(Uninitialize);

	ReleaseWarpBlendFunc(SetEyepoint);
	ReleaseWarpBlendFunc(SDK_GetHeadingPitchRoll);

	ReleaseWarpBlendFunc(DXRender);
	ReleaseWarpBlendFunc(SetInputTexture2D);
	ReleaseWarpBlendFunc(SetOutputTexture2D);

	if (EasyBlend_DLL_Handler)
	{
		FPlatformProcess::FreeDllHandle(EasyBlend_DLL_Handler);
	}
}

FDisplayClusterProjectionEasyBlendLibraryDX11::FDisplayClusterProjectionEasyBlendLibraryDX11()
{
	// Try to load DLL
	const FString DllPath = DisplayClusterHelpers::filesystem::GetFullPathForThirdPartyDLL(DisplayClusterProjectionStrings::ThirdParty::DLL::EasyBlendDX11);
	const FString DllDirectory = FPaths::GetPath(DllPath);

	FPlatformProcess::PushDllDirectory(*DllDirectory);
	EasyBlend_DLL_Handler = FPlatformProcess::GetDllHandle(*DllPath);
	FPlatformProcess::PopDllDirectory(*DllDirectory);

	if (EasyBlend_DLL_Handler)
	{
		if (InitializeDLL())
		{
			UE_LOGF(LogDisplayClusterProjectionEasyBlend, Log, "EasyBlend DX11 API was initialized from file '%ls'.", *DllPath);

			return;
		}

		UE_LOGF(LogDisplayClusterProjectionEasyBlend, Error, "EasyBlend DX11 API could not be initialized. Maybe wrong DLL-file '%ls'.", *DllPath);
	}
	else
	{
		UE_LOGF(LogDisplayClusterProjectionEasyBlend, Error, "Could not find EasyBlend DX11 API '%ls' file.", *DllPath);
	}

	ReleaseDLL();
}

FDisplayClusterProjectionEasyBlendLibraryDX11::~FDisplayClusterProjectionEasyBlendLibraryDX11()
{
	UE_LOGF(LogDisplayClusterProjectionEasyBlend, Log, "EasyBlend DX11 API released.");
	ReleaseDLL();
}

#undef ImportWarpBlendFunc
#undef ReleaseWarpBlendFunc
