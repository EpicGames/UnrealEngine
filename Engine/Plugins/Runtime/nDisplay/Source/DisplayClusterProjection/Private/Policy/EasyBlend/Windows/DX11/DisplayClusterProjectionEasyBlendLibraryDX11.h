// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Policy/EasyBlend/Windows/DX11/DisplayClusterProjectionEasyBlendLibraryDX11Types.h"

#define DeclareEasyBlendFunc(DLLFunc) EasyBlend1##DLLFunc##Proc EasyBlend1##DLLFunc = nullptr;

/**
* Implement access to EasyBlend DX11 DLL functions
* The basic idea is to load the DLL dynamically without using LIB files
*
* The DLL function is described in the file ./ThirdParty/EasyBlend/Include/EasyBlendSDKDXApi.h
*/
class FDisplayClusterProjectionEasyBlendLibraryDX11
{
public:
	FDisplayClusterProjectionEasyBlendLibraryDX11();
	~FDisplayClusterProjectionEasyBlendLibraryDX11();

	/** Return EaxyBlend DX11 library api. */
	static TSharedRef<FDisplayClusterProjectionEasyBlendLibraryDX11, ESPMode::ThreadSafe>& Get();

	/** If the DLL is loaded correctly, this function returns true. */
	inline bool IsInitialized() const
	{
		return bInitialized;
	}

public:
	DeclareEasyBlendFunc(Initialize);
	DeclareEasyBlendFunc(Uninitialize);

	DeclareEasyBlendFunc(InitDeviceObjects);

	DeclareEasyBlendFunc(SetEyepoint);
	DeclareEasyBlendFunc(SDK_GetHeadingPitchRoll);

	DeclareEasyBlendFunc(DXRender);
	DeclareEasyBlendFunc(SetInputTexture2D);
	DeclareEasyBlendFunc(SetOutputTexture2D);

private:
	/** Imprort all functionf from EasyBlend DLL. Return true on success. */
	bool InitializeDLL();
	void ReleaseDLL();

private:

	// Initializetion state
	bool bInitialized = false;

	// Saved handle to EasyBlend DLL
	void* EasyBlend_DLL_Handler = nullptr;
};

#undef DeclareEasyBlendFunc
