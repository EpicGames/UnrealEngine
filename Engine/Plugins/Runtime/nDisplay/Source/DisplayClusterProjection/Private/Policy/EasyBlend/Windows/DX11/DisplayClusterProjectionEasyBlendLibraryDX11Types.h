// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ID3D11DynamicRHI.h"

#include "Windows/AllowWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include "Legacy/EasyBlendSDK.h"
THIRD_PARTY_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"

struct EasyBlend1SDKDX_Mesh;

/**
* Initializes a Mesh structure from an EasyBlend calibration file (*.ol file ). The given Mesh pointer should be allocated ( not NULL )
* and will be overwritten.
* @param szFileName - the calibration file name
* @param msm      - a pointer to the mesh
*/
typedef EasyBlend1SDKDXError(__stdcall* EasyBlend1InitializeProc)(const char* szFileName, EasyBlend1SDKDX_Mesh* msm);

/**
* Given a correctly initialized SDK Mesh, releases the resources used by the Mesh structure.  The Mesh pointer memory must still be
* externally deallocated after this call.
* 
* @param msm - a pointer to the mesh
*/
typedef EasyBlend1SDKDXError(__stdcall* EasyBlend1UninitializeProc)(EasyBlend1SDKDX_Mesh* msm);

/**
* The Standard DirectX Commands to deal with losing/regaining a DirectX device.
* 
* @param msm
* @param pDevice
* @param pDeviceContext
* @param pSwapChain
*/
typedef EasyBlend1SDKDXError(__stdcall* EasyBlend1InitDeviceObjectsProc)(EasyBlend1SDKDX_Mesh* msm, ID3D1XDevice* pDevice, ID3D1XDeviceContext* pDeviceContext /* can be null in DX10 */, IDXGISwapChain* pSwapChain);

/**
* Render warpblend
* 
* @param msm
* @param id3d
* @param pDeviceContext
* @param iswapChain
* @param doPresent
*/
typedef EasyBlend1SDKDXError(__stdcall* EasyBlend1DXRenderProc)(EasyBlend1SDKDX_Mesh* msm, ID3D1XDevice* id3d, ID3D1XDeviceContext* pDeviceContext /* can be null in DX10 */, IDXGISwapChain* iswapChain, bool doPresent);

/**
* This comand is used only for Dynamic Eyepoint. The new location of 
* the viewer is sent to the command, and a new frustum is calculated. 
* To use this function, the Inititalize command must have been called
* with a .pol file. After this call, remember to get new frustum and
* use it to update the frustum. This command is currently CPU
* bound.
* 
* @param msm
* @param eyeX
* @param eyeY
* @param eyeZ
*/
typedef EasyBlend1SDKDXError(__stdcall* EasyBlend1SetEyepointProc)(EasyBlend1SDKDX_Mesh* msm, const double& eyeX, const double& eyeY, const double& eyeZ);

/**
* This command returns the view angles for the instance of the SDK.
* As Heading, then Pitch then Roll. The Angles are pre-computed in the SDK file, so that it only works with newer version of EasyBlend
* that write this information to disk. (Newer than Nov 2011 or newer)
* 
* @param rdDegreesHeading
* @param rdDegreesPitch
* @param rdDegreesRoll
* @param msm
*/
typedef EasyBlend1SDKDXError(__stdcall* EasyBlend1SDK_GetHeadingPitchRollProc)(double& rdDegreesHeading, double& rdDegreesPitch, double& rdDegreesRoll, EasyBlend1SDKDX_Mesh* msm);

/**
* These function are for setting input/output textures, which allows chaining filters
* 
* @param msm
* @param texture
*/
typedef EasyBlend1SDKDXError(__stdcall* EasyBlend1SetInputTexture2DProc)(EasyBlend1SDKDX_Mesh* msm, ID3D1XTexture2D* texture);

/**
* These function are for setting input/output textures, which allows chaining filters
*
* @param msm
* @param texture
*/
typedef EasyBlend1SDKDXError(__stdcall* EasyBlend1SetOutputTexture2DProc)(EasyBlend1SDKDX_Mesh* msm, ID3D1XTexture2D* texture);
