// Copyright Epic Games, Inc. All Rights Reserved.

#include "Policy/EasyBlend/Windows/DX11/DisplayClusterProjectionEasyBlendPolicyViewDataDX11.h"
#include "Render/Viewport/IDisplayClusterViewport.h"

#include "DisplayClusterProjectionLog.h"
#include "Misc/DisplayClusterHelpers.h"

#include "ID3D11DynamicRHI.h"

#include "Engine/GameViewportClient.h"
#include "Engine/Engine.h"
#include "Engine/RendererSettings.h"
#include "Misc/Paths.h"
#include "UnrealClient.h"

//------------------------------------------------------------------------------
// FDisplayClusterProjectionEasyBlendPolicyViewDataDX11
//------------------------------------------------------------------------------
FDisplayClusterProjectionEasyBlendPolicyViewDataDX11::~FDisplayClusterProjectionEasyBlendPolicyViewDataDX11()
{
	ImplRelease();
}

void FDisplayClusterProjectionEasyBlendPolicyViewDataDX11::ImplRelease()
{
	if (EasyBlendMeshData.IsValid())
	{
		// Block multi-threaded access to EasyBlendMeshData
		FScopeLock DataScopeLock(&EasyBlendMeshDataAccessCS);
		TSharedRef<FDisplayClusterProjectionEasyBlendLibraryDX11, ESPMode::ThreadSafe>& EasyBlendAPI = FDisplayClusterProjectionEasyBlendLibraryDX11::Get();

		if (bIsEasyBlendMeshDataInitialized)
		{
			// Release the mesh data only if it was previously initialized
			EasyBlendAPI->EasyBlend1Uninitialize(EasyBlendMeshData.Get());

			bIsEasyBlendMeshDataInitialized = false;
		}

		// Release warp data
		EasyBlendMeshData.Reset();
	}

	bIsRenderResourcesInitialized = false;
}

bool FDisplayClusterProjectionEasyBlendPolicyViewDataDX11::Initialize(const FDisplayClusterProjectionEasyBlendPolicyConfiguration& InEasyBlendConfiguration)
{
	if (EasyBlendMeshData.IsValid())
	{
		return false;
	}

	// Block multi-threaded access to EasyBlendMeshData
	FScopeLock DataScopeLock(&EasyBlendMeshDataAccessCS);

	// Create a new warp data
	EasyBlendMeshData = MakeUnique<EasyBlend1SDKDX_Mesh>();
	TSharedRef<FDisplayClusterProjectionEasyBlendLibraryDX11, ESPMode::ThreadSafe>& EasyBlendAPI = FDisplayClusterProjectionEasyBlendLibraryDX11::Get();

	TRACE_CPUPROFILER_EVENT_SCOPE(FDisplayClusterProjectionEasyBlendPolicyViewDataDX11::Initialize);

	// Initialize EasyBlend data for each view
	const auto FileName = StringCast<ANSICHAR>(*InEasyBlendConfiguration.CalibrationFile);
	const EasyBlend1SDKDXError Result = EasyBlendAPI->EasyBlend1Initialize(FileName.Get(), EasyBlendMeshData.Get());
	if (!EasyBlend1SDKDX_SUCCEEDED(Result))
	{
		UE_LOGF(LogDisplayClusterProjectionEasyBlend, Error, "Couldn't initialize EasyBlend internals from file '%ls'", *InEasyBlendConfiguration.CalibrationFile);
		ImplRelease();

		return false;
	}

	// EasyBlendMeshData has been initialized and now requires a call to EasyBlend1Uninitialize().
	bIsEasyBlendMeshDataInitialized = true;

	// Only perspective projection is supported so far
	if (EasyBlendMeshData->Projection != EasyBlend1SDKDX_PROJECTION_Perspective)
	{
		UE_LOGF(LogDisplayClusterProjectionEasyBlend, Error, "EasyBlend mesh data from file '%ls' has projection value %d. Only perspective projection is allowed at this version.", *InEasyBlendConfiguration.CalibrationFile, EasyBlendMeshData->Projection);
		ImplRelease();

		return false;
	}

	return true;
}

bool FDisplayClusterProjectionEasyBlendPolicyViewDataDX11::CalculateWarpBlend(FDisplayClusterProjectionEasyBlendPolicyViewInfo& InOutViewInfo)
{
	if (!EasyBlendMeshData.IsValid())
	{
		return false;
	}

	// Block multi-threaded access to EasyBlendMeshData
	FScopeLock DataScopeLock(&EasyBlendMeshDataAccessCS);
	TSharedRef<FDisplayClusterProjectionEasyBlendLibraryDX11, ESPMode::ThreadSafe>& EasyBlendAPI = FDisplayClusterProjectionEasyBlendLibraryDX11::Get();

	// Update view location
	EasyBlendAPI->EasyBlend1SetEyepoint(EasyBlendMeshData.Get(), InOutViewInfo.ViewLocation.X, InOutViewInfo.ViewLocation.Y, InOutViewInfo.ViewLocation.Z);

	// Get actual view rotation
	EasyBlendAPI->EasyBlend1SDK_GetHeadingPitchRoll(InOutViewInfo.ViewRotation.Yaw, InOutViewInfo.ViewRotation.Pitch, InOutViewInfo.ViewRotation.Roll, EasyBlendMeshData.Get());

	// Save frustum angles
	InOutViewInfo.FrustumAngles = FVector4(
		EasyBlendMeshData->Frustum.LeftAngle,
		EasyBlendMeshData->Frustum.RightAngle,
		EasyBlendMeshData->Frustum.TopAngle,
		EasyBlendMeshData->Frustum.BottomAngle
	);

	return true;
}

bool FDisplayClusterProjectionEasyBlendPolicyViewDataDX11::ApplyWarpBlend_RenderThread(FRHICommandListImmediate& RHICmdList, const FDisplayClusterProjectionEasyBlendPolicyViewInfo& InViewInfo, FRHITexture* InputTexture, FRHITexture* OutputTexture, FRHIViewport* InRHIViewport)
{
	check(IsInRenderingThread());

	if (!EasyBlendMeshData.IsValid() || !InputTexture || !OutputTexture)
	{
		return false;
	}

	// Insert an outer query that encloses the whole batch
	RHICmdList.EnqueueLambda(
		[ViewData = SharedThis(this), InputTexture = InputTexture, OutputTexture = OutputTexture, InRHIViewport = InRHIViewport, InViewLocation = InViewInfo.ViewLocation](FRHICommandList& ExecutingCmdList)
		{
			ID3D11DynamicRHI* D3D11DynamicRHI = GetID3D11DynamicRHI();
			if (!D3D11DynamicRHI)
			{
				return;
			}

			IDXGISwapChain* SwapChain = InRHIViewport ? D3D11DynamicRHI->RHIGetSwapChain(InRHIViewport) : nullptr;
			ID3D11Device* D3D11Device = D3D11DynamicRHI->RHIGetDevice();
			ID3D11DeviceContext* D3D11DeviceContext = D3D11DynamicRHI->RHIGetDeviceContext();
			if (!SwapChain || !D3D11Device || !D3D11DeviceContext)
			{
				return;
			}

			// Block multi-threaded access to EasyBlendMeshData
			FScopeLock DataScopeLock(&ViewData->EasyBlendMeshDataAccessCS);
			TSharedRef<FDisplayClusterProjectionEasyBlendLibraryDX11, ESPMode::ThreadSafe>& EasyBlendAPI = FDisplayClusterProjectionEasyBlendLibraryDX11::Get();

			// Update view location for proxy
			EasyBlendAPI->EasyBlend1SetEyepoint(ViewData->EasyBlendMeshData.Get(), InViewLocation.X, InViewLocation.Y, InViewLocation.Z);

			// Initialize EasyBlend internals
			if (!ViewData->bIsRenderResourcesInitialized)
			{
				EasyBlend1SDKDXError sdkErr = EasyBlendAPI->EasyBlend1InitDeviceObjects(ViewData->EasyBlendMeshData.Get(), D3D11Device, D3D11DeviceContext, SwapChain);
				if (EasyBlend1SDKDX_FAILED(sdkErr))
				{
					UE_LOGF(LogDisplayClusterProjectionEasyBlend, Error, "Couldn't initialize EasyBlend DX11 Device/DeviceContext/SwapChain");
					ViewData->bIsRenderResourcesInitialized = false;

					return;
				}

				ViewData->bIsRenderResourcesInitialized = true;
			}

			TRACE_CPUPROFILER_EVENT_SCOPE(DisplayClusterProjectionEasyBlendPolicyViewDataDX11::ApplyWarpBlend_RenderThread);

			// Prepare the textures
			ID3D11RenderTargetView* DstTextureRTV = D3D11DynamicRHI->RHIGetRenderTargetView(OutputTexture);

			ID3D11Texture2D* DstTextureD3D11 = (ID3D11Texture2D*)D3D11DynamicRHI->RHIGetResource(OutputTexture);
			ID3D11Texture2D* SrcTextureD3D11 = (ID3D11Texture2D*)D3D11DynamicRHI->RHIGetResource(InputTexture);

			// Setup In/Out EasyBlend textures
			const EasyBlend1SDKDXError EasyBlend1SDKDXError1 = EasyBlendAPI->EasyBlend1SetInputTexture2D(ViewData->EasyBlendMeshData.Get(), SrcTextureD3D11);
			const EasyBlend1SDKDXError EasyBlend1SDKDXError2 = EasyBlendAPI->EasyBlend1SetOutputTexture2D(ViewData->EasyBlendMeshData.Get(), DstTextureD3D11);
			if (!(EasyBlend1SDKDX_SUCCEEDED(EasyBlend1SDKDXError1) && EasyBlend1SDKDX_SUCCEEDED(EasyBlend1SDKDXError2)))
			{
				UE_LOGF(LogDisplayClusterProjectionEasyBlend, Error, "Coulnd't configure in/out textures for EasyBlend DX11");

				return;
			}

			D3D11_VIEWPORT RenderViewportData;
			RenderViewportData.MinDepth = 0.0f;
			RenderViewportData.MaxDepth = 1.0f;
			RenderViewportData.Width = static_cast<float>(OutputTexture->GetSizeX());
			RenderViewportData.Height = static_cast<float>(OutputTexture->GetSizeY());
			RenderViewportData.TopLeftX = 0.0f;
			RenderViewportData.TopLeftY = 0.0f;

			D3D11DeviceContext->RSSetViewports(1, &RenderViewportData);
			D3D11DeviceContext->OMSetRenderTargets(1, &DstTextureRTV, nullptr);

			// Perform warp&blend by the EasyBlend
			EasyBlend1SDKDXError EasyBlend1SDKDXError = EasyBlendAPI->EasyBlend1DXRender(
				ViewData->EasyBlendMeshData.Get(),
				D3D11Device,
				D3D11DeviceContext,
				SwapChain,
				false);

			D3D11DeviceContext->Flush();

			if (!EasyBlend1SDKDX_SUCCEEDED(EasyBlend1SDKDXError))
			{
				UE_LOGF(LogDisplayClusterProjectionEasyBlend, Error, "EasyBlend DX11 couldn't perform rendering operation");

				return;
			}
		});


	return true;
}
