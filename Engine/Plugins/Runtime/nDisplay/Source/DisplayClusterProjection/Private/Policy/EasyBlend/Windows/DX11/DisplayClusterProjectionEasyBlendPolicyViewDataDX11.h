// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Policy/EasyBlend/IDisplayClusterProjectionEasyBlendPolicyViewData.h"
#include "Policy/EasyBlend/Windows/DX11/DisplayClusterProjectionEasyBlendLibraryDX11.h"
#include "Templates/SharedPointer.h"

/**
* The DX11 implementation for EasyBlend
*/
class FDisplayClusterProjectionEasyBlendPolicyViewDataDX11
	: public IDisplayClusterProjectionEasyBlendPolicyViewData
	, public TSharedFromThis<FDisplayClusterProjectionEasyBlendPolicyViewDataDX11, ESPMode::ThreadSafe>
{
public:
	~FDisplayClusterProjectionEasyBlendPolicyViewDataDX11();

public:
	//BEGIN ~IDisplayClusterProjectionEasyBlendPolicyViewData
	virtual bool Initialize(const FDisplayClusterProjectionEasyBlendPolicyConfiguration& InEasyBlendConfiguration) override;

	virtual bool CalculateWarpBlend(FDisplayClusterProjectionEasyBlendPolicyViewInfo& InOutViewInfo) override;
	virtual bool ApplyWarpBlend_RenderThread(FRHICommandListImmediate& RHICmdList, const FDisplayClusterProjectionEasyBlendPolicyViewInfo& InViewInfo, FRHITexture* InputTexture, FRHITexture* OutputTexture, FRHIViewport* InRHIViewport) override;
	//END ~~IDisplayClusterProjectionEasyBlendPolicyViewData

protected:
	void ImplRelease();

private:
	// The unique EasyBlend data that useed for warpblend
	TUniquePtr<EasyBlend1SDKDX_Mesh> EasyBlendMeshData;
};
