// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterProjectionEasyBlendLibraryDX12.h"

#include "DisplayClusterProjectionLog.h"
#include "PDisplayClusterProjectionStrings.h"

#include "HAL/PlatformProcess.h"
#include "Misc/DisplayClusterHelpers.h"
#include "PDisplayClusterProjectionStrings.h"

TSharedRef<FDisplayClusterProjectionEasyBlendLibraryDX12, ESPMode::ThreadSafe>& FDisplayClusterProjectionEasyBlendLibraryDX12::Get()
{
	static TSharedRef<FDisplayClusterProjectionEasyBlendLibraryDX12, ESPMode::ThreadSafe> EasyBlendLibraryDX12 = MakeShared<FDisplayClusterProjectionEasyBlendLibraryDX12, ESPMode::ThreadSafe>();

	return EasyBlendLibraryDX12;
}

#define ImportWarpBlendFunc(DLLFunc)\
	DLLFunc = (DLLFunc##Proc)FPlatformProcess::GetDllExport((HMODULE)EasyBlend_DLL_Handler, TEXT(#DLLFunc));\
	if(DLLFunc==nullptr)\
		{\
			UE_LOGF(LogDisplayClusterProjectionEasyBlend, Error, "Couldn't initialize EasyBlend DX12 API func '%ls'.", TEXT(#DLLFunc));\
			return false;\
		}

#define ReleaseWarpBlendFunc(DLLFunc) DLLFunc = nullptr;

bool FDisplayClusterProjectionEasyBlendLibraryDX12::InitializeDLL()
{
	if (EasyBlend_DLL_Handler)
	{
		ImportWarpBlendFunc(EasyBlendSDK_Initialize);
		ImportWarpBlendFunc(EasyBlendSDK_Uninitialize);

		ImportWarpBlendFunc(EasyBlendSDK_SetEyepoint);
		ImportWarpBlendFunc(EasyBlendSDK_GetHeadingPitchRoll);

		ImportWarpBlendFunc(EasyBlendSDK_InitializeDX12_Device);
		ImportWarpBlendFunc(EasyBlendSDK_TransformInputToOutputDX12_CommandList);
	}

	// All EasyBlend  DLL functions are initialized successfully
	bInitialized = true;

	return true;
}

void FDisplayClusterProjectionEasyBlendLibraryDX12::ReleaseDLL()
{
	bInitialized = false;

	ReleaseWarpBlendFunc(EasyBlendSDK_Initialize);
	ReleaseWarpBlendFunc(EasyBlendSDK_Uninitialize);

	ReleaseWarpBlendFunc(EasyBlendSDK_SetEyepoint);
	ReleaseWarpBlendFunc(EasyBlendSDK_GetHeadingPitchRoll);

	ReleaseWarpBlendFunc(EasyBlendSDK_InitializeDX12_Device);
	ReleaseWarpBlendFunc(EasyBlendSDK_TransformInputToOutputDX12_CommandList);

	if (EasyBlend_DLL_Handler)
	{
		FPlatformProcess::FreeDllHandle(EasyBlend_DLL_Handler);
	}
}

FDisplayClusterProjectionEasyBlendLibraryDX12::FDisplayClusterProjectionEasyBlendLibraryDX12()
{
	const FString DllPath = DisplayClusterHelpers::filesystem::GetFullPathForThirdPartyDLL(DisplayClusterProjectionStrings::ThirdParty::DLL::EasyBlendDX12);
	const FString DllDirectory = FPaths::GetPath(DllPath);

	FPlatformProcess::PushDllDirectory(*DllDirectory);
	EasyBlend_DLL_Handler = FPlatformProcess::GetDllHandle(*DllPath);
	FPlatformProcess::PopDllDirectory(*DllDirectory);

	if (EasyBlend_DLL_Handler)
	{
		if (InitializeDLL())
		{
			UE_LOGF(LogDisplayClusterProjectionEasyBlend, Log, "EasyBlend DX12 API was initialized from file '%ls'.", *DllPath);

			return;
		}

		UE_LOGF(LogDisplayClusterProjectionEasyBlend, Error, "EasyBlend DX12 API could not be initialized. Maybe wrong DLL-file '%ls'.", *DllPath);
	}
	else
	{
		UE_LOGF(LogDisplayClusterProjectionEasyBlend, Error, "Could not find EasyBlend DX12 API '%ls' file.", *DllPath);
	}

	ReleaseDLL();
}

FDisplayClusterProjectionEasyBlendLibraryDX12::~FDisplayClusterProjectionEasyBlendLibraryDX12()
{
	UE_LOGF(LogDisplayClusterProjectionEasyBlend, Log, "EasyBlend DX12 API released.");
	ReleaseDLL();
}

#undef ImportWarpBlendFunc
#undef ReleaseWarpBlendFunc
