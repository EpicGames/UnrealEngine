// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Policy/EasyBlend/Windows/DX12/DisplayClusterProjectionEasyBlendLibraryDX12Types.h"

#define DeclareEasyBlendFunc(DLLFunc) DLLFunc##Proc DLLFunc = nullptr;

/**
* Implement access to EasyBlend DX12 DLL functions
* The basic idea is to load the DLL dynamically without using LIB files
*
* The DLL function is described in the file ./ThirdParty/EasyBlendDX12/Include/EasyBlendSDKDXApi.h
*/
class FDisplayClusterProjectionEasyBlendLibraryDX12
{
public:
	FDisplayClusterProjectionEasyBlendLibraryDX12();
	~FDisplayClusterProjectionEasyBlendLibraryDX12();

	/** Return EaxyBlend DX12 library api. */
	static TSharedRef<FDisplayClusterProjectionEasyBlendLibraryDX12, ESPMode::ThreadSafe>& Get();

	/** If the DLL is loaded correctly, this function returns true. */
	inline bool IsInitialized() const
	{
		return bInitialized;
	}

public:
	DeclareEasyBlendFunc(EasyBlendSDK_Initialize);
	DeclareEasyBlendFunc(EasyBlendSDK_Uninitialize);

	DeclareEasyBlendFunc(EasyBlendSDK_SetEyepoint);
	DeclareEasyBlendFunc(EasyBlendSDK_GetHeadingPitchRoll);

	DeclareEasyBlendFunc(EasyBlendSDK_InitializeDX12_Device);
	DeclareEasyBlendFunc(EasyBlendSDK_TransformInputToOutputDX12_CommandList);

private:
	/** Imprort all functionf from EasyBlend DLL. Return true on success. */
	bool InitializeDLL();
	void ReleaseDLL();

private:
	// Initializetion state
	bool bInitialized = false;

	// Saved handle to EasyBlend DLL
	void* EasyBlend_DLL_Handler = nullptr;
};

#undef DeclareEasyBlendFunc
