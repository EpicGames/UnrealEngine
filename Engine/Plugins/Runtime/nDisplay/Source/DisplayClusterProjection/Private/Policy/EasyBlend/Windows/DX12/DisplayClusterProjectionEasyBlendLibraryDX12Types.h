// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Windows/AllowWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include "EasyBlendSDK.h"
THIRD_PARTY_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"

struct EasyBlendSDKDX_Mesh;


/**
* Initializes a Mesh structure from an EasyBlend calibration file (typically a .ol file or .pol file for dynamic eyepoint).
* For OpenGL, this call MUST be made when a valid OpenGL context has been created. Otherwise the EasyBlend SDK will conclude that you only
* have OpenGL 1.0 support. The given Mesh pointer should be allocated ( not NULL ) and will be overwritten.
* 
* @param Filename
* @param msm
* @param SDKMode
* @param cb
* @param cbData
*/
typedef EasyBlendSDKError(__stdcall* EasyBlendSDK_InitializeProc)(const char* Filename, EasyBlendSDK_Mesh* msm, int SDKMode, EasyBlendSDKCallback cb,void* cbData);



/**
* Given a correctly initialized SDK Mesh, releases the resources used by the Mesh structure.  The Mesh pointer memory must still be
* externally deallocated after this call. 
* 
* @param msm
* @param cb
* @param cbData
*/
typedef EasyBlendSDKError(__stdcall* EasyBlendSDK_UninitializeProc)(EasyBlendSDK_Mesh* msm, EasyBlendSDKCallback cb, void* cbData);

/**
* Initialize the DX12 renderer
* 
* @param msm           - EasyBlend SDK mesh initialized using EasyBlendSDK_SDKMODE_DX12
* @param pDevice       - D3D12 device object pointer
*/
typedef EasyBlendSDKError(__stdcall* EasyBlendSDK_InitializeDX12_DeviceProc)(EasyBlendSDK_Mesh* msm, ID3D12Device* pDevice);

/**
* Fill a given command list with commands that will warp the image on the back buffer and 
* write the warped image back into the same buffer.  The swap chain provided to
* EasyBlendSDK_InitializeDX12 will be used to obtain the current backbuffer resource.
* This resource will be assumed to be in the state D3D12_RESOURCE_STATE_PRESENT and will be
* returned to that state after executing the command list.  Caller is responsible for
* executing the command list on the appropriate command queue and then calling present.
*
* @param msm       - EasyBlend SDK mesh initialized using EasyBlendSDK_SDKMODE_DX12
* @param pCmdQueue - D3D12 command list that is ready to accept commands
*/
typedef EasyBlendSDKError(__stdcall* EasyBlendSDK_TransformInputToOutputDX12_CommandListProc)(EasyBlendSDK_Mesh* msm, ID3D12Resource* InputTexture, ID3D12Resource* OutputTexture, D3D12_CPU_DESCRIPTOR_HANDLE OutputRTV, ID3D12GraphicsCommandList* pCommandList);

/**
* This command returns the view angles for the instance of the SDK.
* As Heading, then Pitch then Roll. The Angles are pre-computed in the SDK file, so that it only works with newer version of EasyBlend
* that write this information to disk. (Newer than Nov 2011 or newer)
*
* @param rdDegreesHeading - (out) yaw
* @param rdDegreesPitch   - (out) pitch
* @param rdDegreesRoll    - (out) roll
* @param msm              - EasyBlend SDK mesh
*/
typedef EasyBlendSDKError(__stdcall* EasyBlendSDK_GetHeadingPitchRollProc)(double& rdDegreesHeading, double& rdDegreesPitch, double& rdDegreesRoll, EasyBlendSDK_Mesh* msm);

/**
* This comand is used only for Dynamic Eyepoint. The new location of
* the viewer is sent to the command, and a new frustum is calculated.
* To use this function, the Inititalize command must have been called
* with a .pol file. After this call, remember to get new frustum and
* use it to update the frustum. This command is currently CPU
* bound.
*
* @param msm  - EasyBlend SDK mesh
* @param eyeX - eye X position
* @param eyeY - eye Y position
* @param eyeZ - eye Z position
*/
typedef EasyBlendSDKError(__stdcall* EasyBlendSDK_SetEyepointProc)(EasyBlendSDK_Mesh* msm, const double& eyeX, const double& eyeY, const double& eyeZ);
