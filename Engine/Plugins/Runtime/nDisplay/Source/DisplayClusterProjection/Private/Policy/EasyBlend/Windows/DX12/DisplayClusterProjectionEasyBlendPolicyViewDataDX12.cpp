// Copyright Epic Games, Inc. All Rights Reserved.

#include "Policy/EasyBlend/Windows/DX12/DisplayClusterProjectionEasyBlendPolicyViewDataDX12.h"
#include "Policy/EasyBlend/DisplayClusterProjectionEasyBlendPolicyGeometryExportData.h"

#include "DisplayClusterProjectionLog.h"
#include "Misc/DisplayClusterHelpers.h"

#include "ID3D12DynamicRHI.h"

#include "Engine/GameViewportClient.h"
#include "Engine/Engine.h"
#include "Engine/RendererSettings.h"
#include "Misc/Paths.h"
#include "UnrealClient.h"

//------------------------------------------------------------------------------
// FDisplayClusterProjectionEasyBlendPolicyViewDataDX12
//------------------------------------------------------------------------------
FDisplayClusterProjectionEasyBlendPolicyViewDataDX12::~FDisplayClusterProjectionEasyBlendPolicyViewDataDX12()
{
	ImplRelease();
}

void FDisplayClusterProjectionEasyBlendPolicyViewDataDX12::ImplRelease()
{
	if (EasyBlendMeshData.IsValid())
	{
		// Block multi-threaded access to EasyBlendMeshData
		FScopeLock DataScopeLock(&EasyBlendMeshDataAccessCS);
		TSharedRef<FDisplayClusterProjectionEasyBlendLibraryDX12, ESPMode::ThreadSafe>& EasyBlendAPI = FDisplayClusterProjectionEasyBlendLibraryDX12::Get();

		if (bIsEasyBlendMeshDataInitialized)
		{
			// Release the mesh data only if it was previously initialized
			EasyBlendAPI->EasyBlendSDK_Uninitialize(EasyBlendMeshData.Get(), nullptr, nullptr);

			bIsEasyBlendMeshDataInitialized = false;
		}
		// Release warp data
		EasyBlendMeshData.Reset();
	}


	bIsRenderResourcesInitialized = false;
}

bool FDisplayClusterProjectionEasyBlendPolicyViewDataDX12::Initialize(const FDisplayClusterProjectionEasyBlendPolicyConfiguration& InEasyBlendConfiguration)
{
	if (EasyBlendMeshData.IsValid())
	{
		return false;
	}

	// Block multi-threaded access to EasyBlendMeshData
	FScopeLock DataScopeLock(&EasyBlendMeshDataAccessCS);
	TSharedRef<FDisplayClusterProjectionEasyBlendLibraryDX12, ESPMode::ThreadSafe>& EasyBlendAPI = FDisplayClusterProjectionEasyBlendLibraryDX12::Get();

	// Create a new warp data
	EasyBlendMeshData = MakeUnique<EasyBlendSDK_Mesh>();


	TRACE_CPUPROFILER_EVENT_SCOPE(FDisplayClusterProjectionEasyBlendPolicyViewDataDX12::Initialize);

	// Initialize EasyBlend data for each view
	const auto FileName = StringCast<ANSICHAR>(*InEasyBlendConfiguration.CalibrationFile);
	const EasyBlendSDKError Result = EasyBlendAPI->EasyBlendSDK_Initialize(FileName.Get(), EasyBlendMeshData.Get(), EasyBlendSDK_SDKMODE_DX12, nullptr, nullptr);
	if (EasyBlendSDK_FAILED(Result))
	{
		UE_LOGF(LogDisplayClusterProjectionEasyBlend, Error, "Couldn't initialize EasyBlend internals from file '%ls'", *InEasyBlendConfiguration.CalibrationFile);
		ImplRelease();

		return false;
	}

	// EasyBlendMeshData has been initialized and now requires a call to EasyBlend1Uninitialize().
	bIsEasyBlendMeshDataInitialized = true;

	// Only perspective projection is supported so far
	if (EasyBlendMeshData->Projection != EasyBlendSDK_PROJECTION_Perspective)
	{
		UE_LOGF(LogDisplayClusterProjectionEasyBlend, Error, "EasyBlend mesh data from file '%ls' has projection value %d. Only perspective projection is allowed at this version.", *InEasyBlendConfiguration.CalibrationFile, EasyBlendMeshData->Projection);
		ImplRelease();

		return false;
	}

	return true;
}

bool FDisplayClusterProjectionEasyBlendPolicyViewDataDX12::CalculateWarpBlend(FDisplayClusterProjectionEasyBlendPolicyViewInfo& InOutViewInfo)
{
	if (!EasyBlendMeshData.IsValid())
	{
		return false;
	}

	// Block multi-threaded access to EasyBlendMeshData
	FScopeLock DataScopeLock(&EasyBlendMeshDataAccessCS);
	TSharedRef<FDisplayClusterProjectionEasyBlendLibraryDX12, ESPMode::ThreadSafe>& EasyBlendAPI = FDisplayClusterProjectionEasyBlendLibraryDX12::Get();


	// Update view location
	EasyBlendAPI->EasyBlendSDK_SetEyepoint(EasyBlendMeshData.Get(), InOutViewInfo.ViewLocation.X, InOutViewInfo.ViewLocation.Y, InOutViewInfo.ViewLocation.Z);

	// Get actual view rotation
	EasyBlendAPI->EasyBlendSDK_GetHeadingPitchRoll(InOutViewInfo.ViewRotation.Yaw, InOutViewInfo.ViewRotation.Pitch, InOutViewInfo.ViewRotation.Roll, EasyBlendMeshData.Get());

	// Save frustum angles
	InOutViewInfo.FrustumAngles = FVector4(
		EasyBlendMeshData->Frustum.LeftAngle,
		EasyBlendMeshData->Frustum.RightAngle,
		EasyBlendMeshData->Frustum.TopAngle,
		EasyBlendMeshData->Frustum.BottomAngle
	);

	return true;
}

bool FDisplayClusterProjectionEasyBlendPolicyViewDataDX12::ApplyWarpBlend_RenderThread(FRHICommandListImmediate& RHICmdList, const FDisplayClusterProjectionEasyBlendPolicyViewInfo& InViewInfo, FRHITexture* InputTexture, FRHITexture* OutputTexture, FRHIViewport* InRHIViewport)
{
	check(IsInRenderingThread());

	if (!EasyBlendMeshData.IsValid() || !InputTexture || !OutputTexture)
	{
		return false;
	}
	
	// Setup In/Out EasyBlend textures
	RHICmdList.Transition(FRHITransitionInfo(OutputTexture, ERHIAccess::Unknown, ERHIAccess::RTV));
	// Insert an outer query that encloses the whole batch
	RHICmdList.EnqueueLambda(
	[ViewData = SharedThis(this), InputTexture = InputTexture, OutputTexture = OutputTexture, InViewLocation = InViewInfo.ViewLocation](FRHICommandList& ExecutingCmdList)
	{
		// Block multi-threaded access to EasyBlendMeshData
		FScopeLock DataScopeLock(&ViewData->EasyBlendMeshDataAccessCS);
		TSharedRef<FDisplayClusterProjectionEasyBlendLibraryDX12, ESPMode::ThreadSafe>& EasyBlendAPI = FDisplayClusterProjectionEasyBlendLibraryDX12::Get();

		ID3D12DynamicRHI* D3D12DynamicRHI = GetID3D12DynamicRHI();
		if (!D3D12DynamicRHI)
		{
			return;
		}

		// Ensure EasyBlend uses the same GPU device as the source texture to support multi-GPU configurations
		const uint32 DevIndex = D3D12DynamicRHI->RHIGetResourceDeviceIndex(InputTexture);
		ID3D12Device* D3D12Device = D3D12DynamicRHI->RHIGetDevice(DevIndex);
		if (!D3D12Device)
		{
			return;
		}

		// Update view location for proxy
		const EasyBlendSDKError Result1 = EasyBlendAPI->EasyBlendSDK_SetEyepoint(ViewData->EasyBlendMeshData.Get(), InViewLocation.X, InViewLocation.Y, InViewLocation.Z);

		// Initialize EasyBlend internals
		if (!ViewData->bIsRenderResourcesInitialized)
		{
			const EasyBlendSDKError Result = EasyBlendAPI->EasyBlendSDK_InitializeDX12_Device(ViewData->EasyBlendMeshData.Get(), D3D12Device);
			if (!EasyBlendSDK_SUCCEEDED(Result))
			{
				ViewData->bIsRenderResourcesInitialized = false;

				return;
			}

			ViewData->bIsRenderResourcesInitialized = true;
		}

		ID3D12Resource* SrcTexture = D3D12DynamicRHI->RHIGetResource(InputTexture);
		ID3D12Resource* DestTexture = D3D12DynamicRHI->RHIGetResource(OutputTexture);
		const D3D12_CPU_DESCRIPTOR_HANDLE RTVHandle = D3D12DynamicRHI->RHIGetRenderTargetView(OutputTexture);
		ID3D12GraphicsCommandList* D3D12GraphicsCommandList = D3D12DynamicRHI->RHIGetGraphicsCommandList(ExecutingCmdList, DevIndex);

		if (SrcTexture && DestTexture && D3D12GraphicsCommandList)
		{
			// Render EasyBlend on proxy
			const EasyBlendSDKError Result2 = EasyBlendAPI->EasyBlendSDK_TransformInputToOutputDX12_CommandList(ViewData->EasyBlendMeshData.Get(), SrcTexture, DestTexture, RTVHandle, D3D12GraphicsCommandList);
		}
	});
	RHICmdList.Transition(FRHITransitionInfo(OutputTexture, ERHIAccess::RTV, ERHIAccess::Present));

	return true;
}

namespace UE::DisplayClusterProjection::EasyBlend
{
	EasyBlendSDKError static CustomInitializeImpl(EasyBlendSDK_Mesh* msm, void* cbData)
	{
		if (cbData)
		{
			FDisplayClusterProjectionEasyBlendGeometryExportData& OutMeshData = *((FDisplayClusterProjectionEasyBlendGeometryExportData*)cbData);

			if (EasyBlendSDK_ClientMesh* ClientMesh = msm ? msm->ClientMesh : nullptr)
			{
				const float WorldToMeters = 100.0f;

				//3d warp mesh
				if (ClientMesh->Vertices3)
				{
					const float EasyBlendScale = 100.f / OutMeshData.GeometryScale;

					OutMeshData.Vertices.AddZeroed(ClientMesh->NumVertices);
					OutMeshData.UV.AddZeroed(ClientMesh->NumVertices);
					for (int32 VertexIndex = 0; VertexIndex < (int32)ClientMesh->NumVertices; VertexIndex++)
					{
						const EasyBlendSDK_ClientVertex3& SrcPts = ClientMesh->Vertices3[VertexIndex];

						// Convert from EasyBlend coordinate system
						const FVector   InPts(SrcPts.Z, SrcPts.X, -SrcPts.Y);
						const FVector2D InUV(SrcPts.x / msm->Xres, SrcPts.y / msm->Yres);

						OutMeshData.Vertices[VertexIndex] = InPts * EasyBlendScale;
						OutMeshData.UV[VertexIndex] = InUV;
					}
				}

				//2d warp mesh
				if (ClientMesh->Vertices2)
				{
					// Pixels to units
					const float EasyBlendScale = OutMeshData.GeometryScale;

					OutMeshData.Vertices.AddZeroed(ClientMesh->NumVertices);
					OutMeshData.UV.AddZeroed(ClientMesh->NumVertices);
					for (int32 VertexIndex = 0; VertexIndex < (int32)ClientMesh->NumVertices; VertexIndex++)
					{
						const EasyBlendSDK_ClientVertex2& SrcPts = ClientMesh->Vertices2[VertexIndex];

						// Convert from EasyBlend coordinate system
						const FVector   InPts(0, (msm->Xres * 0.5f) - SrcPts.x, SrcPts.y);
						const FVector2D InUV(SrcPts.u, SrcPts.v);

						OutMeshData.Vertices[VertexIndex] = InPts * EasyBlendScale;
						OutMeshData.UV[VertexIndex] = InUV;
					}
				}

				if (!OutMeshData.Vertices.IsEmpty())
				{
					OutMeshData.Triangles.Reserve(ClientMesh->NumFaces * 3);
					for (int32 TriangleIndex = 0; TriangleIndex < (int32)ClientMesh->NumFaces; TriangleIndex++)
					{
						const EasyBlendSDK_ClientFace& SrcFace = ClientMesh->Faces[TriangleIndex];
						OutMeshData.Triangles.Add(SrcFace.v3);
						OutMeshData.Triangles.Add(SrcFace.v2);
						OutMeshData.Triangles.Add(SrcFace.v1);
					}

					// Generate normals:
					OutMeshData.GenerateGeometryNormals();
				}
			}
		}

		return EasyBlendSDK_ERR_S_OK;
	}
};

bool FDisplayClusterProjectionEasyBlendPolicyViewDataDX12::GetPreviewMeshGeometry(const FDisplayClusterProjectionEasyBlendPolicyConfiguration& InEasyBlendConfiguration, FDisplayClusterProjectionEasyBlendGeometryExportData& OutMeshData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDisplayClusterProjectionEasyBlendPolicyViewDataDX12::ExportGeometry);

	// Export warp geometry
	EasyBlendSDK_Mesh InMeshData;

	// Initialize EasyBlend data for each view
	const auto FileName = StringCast<ANSICHAR>(*InEasyBlendConfiguration.CalibrationFile);

	// Set easyblend scale
	OutMeshData.GeometryScale = InEasyBlendConfiguration.GeometryScale;

	TSharedRef<FDisplayClusterProjectionEasyBlendLibraryDX12, ESPMode::ThreadSafe>& EasyBlendAPI = FDisplayClusterProjectionEasyBlendLibraryDX12::Get();
	const EasyBlendSDKError Result = EasyBlendAPI->EasyBlendSDK_Initialize(FileName.Get(), &InMeshData, EasyBlendSDK_SDKMODE_ClientData, UE::DisplayClusterProjection::EasyBlend::CustomInitializeImpl, &OutMeshData);
	if (!EasyBlendSDK_FAILED(Result))
	{
		// Release the mesh data
		EasyBlendAPI->EasyBlendSDK_Uninitialize(&InMeshData, nullptr, nullptr);

		return !OutMeshData.Vertices.IsEmpty();
	}

	return false;
}
