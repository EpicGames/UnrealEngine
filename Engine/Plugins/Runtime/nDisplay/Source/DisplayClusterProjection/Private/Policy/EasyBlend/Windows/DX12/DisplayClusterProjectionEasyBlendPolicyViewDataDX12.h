// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Policy/EasyBlend/IDisplayClusterProjectionEasyBlendPolicyViewData.h"
#include "Policy/EasyBlend/Windows/DX12/DisplayClusterProjectionEasyBlendLibraryDX12.h"
#include "Templates/SharedPointer.h"

/**
* The DX12 implementation for EasyBlend
*/
class FDisplayClusterProjectionEasyBlendPolicyViewDataDX12
	: public IDisplayClusterProjectionEasyBlendPolicyViewData
	, public TSharedFromThis<FDisplayClusterProjectionEasyBlendPolicyViewDataDX12, ESPMode::ThreadSafe>
{
public:
	~FDisplayClusterProjectionEasyBlendPolicyViewDataDX12();

public:
	//BEGIN ~IDisplayClusterProjectionEasyBlendPolicyViewData
	virtual bool Initialize(const FDisplayClusterProjectionEasyBlendPolicyConfiguration& InEasyBlendConfiguration) override;
	virtual bool CalculateWarpBlend(FDisplayClusterProjectionEasyBlendPolicyViewInfo& InOutViewInfo) override;
	virtual bool ApplyWarpBlend_RenderThread(FRHICommandListImmediate& RHICmdList, const FDisplayClusterProjectionEasyBlendPolicyViewInfo& InViewInfo, FRHITexture* InputTexture, FRHITexture* OutputTexture, FRHIViewport* InRHIViewport) override;

	virtual bool HasPreviewMesh() override { return true; }
	virtual bool GetPreviewMeshGeometry(const FDisplayClusterProjectionEasyBlendPolicyConfiguration& InEasyBlendConfiguration, FDisplayClusterProjectionEasyBlendGeometryExportData& OutMeshData) override;
	//END ~~IDisplayClusterProjectionEasyBlendPolicyViewData

protected:
	void ImplRelease();

private:
	// The unique EasyBlend data that useed for warpblend
	TUniquePtr<EasyBlendSDK_Mesh> EasyBlendMeshData;
};
