// Copyright Epic Games, Inc. All Rights Reserved.

#include "Policy/Link/DisplayClusterProjectionLinkPolicy.h"

#include "Misc/DisplayClusterHelpers.h"

#include "DisplayClusterProjectionLog.h"
#include "DisplayClusterProjectionStrings.h"

#include "Render/Viewport/IDisplayClusterViewportManager.h"
#include "Render/Viewport/IDisplayClusterViewport.h"
#include "Render/Viewport/IDisplayClusterViewportProxy.h"


FDisplayClusterProjectionLinkPolicy::FDisplayClusterProjectionLinkPolicy(const FString& ProjectionPolicyId, const FDisplayClusterConfigurationProjection* InConfigurationProjectionPolicy)
	: FDisplayClusterProjectionPolicyBase(ProjectionPolicyId, InConfigurationProjectionPolicy)
{
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProjectionPolicy
//////////////////////////////////////////////////////////////////////////////////////////////
const FString& FDisplayClusterProjectionLinkPolicy::GetType() const
{
	static const FString Type(DisplayClusterProjectionStrings::projection::Link);
	return Type;
}

const FString& FDisplayClusterProjectionLinkPolicy::GetParentViewport(IDisplayClusterViewport* InViewport) const
{
	check(InViewport);

	const FDisplayClusterViewport_RenderSettings& RenderSettings = InViewport->GetRenderSettings();
	if (RenderSettings.TileSettings.GetType() == EDisplayClusterViewportTileType::Tile)
	{
		return RenderSettings.TileSettings.GetSourceViewportId();
	}

	return RenderSettings.GetParentViewportId();
}

void FDisplayClusterProjectionLinkPolicy::SetupProjectionViewPoint(IDisplayClusterViewport* InViewport, const float InDeltaTime, FMinimalViewInfo& InOutViewInfo, float* OutCustomNearClippingPlane)
{

	// Getting the right data from a parent
	if (IDisplayClusterViewportManager* ViewportManager = InViewport->GetConfiguration().GetViewportManager())
	{
		if (IDisplayClusterViewport* ParentViewport = ViewportManager->FindViewport(GetParentViewport(InViewport)))
		{
			if (ParentViewport->GetProjectionPolicy().IsValid())
			{
				ParentViewport->GetProjectionPolicy()->SetupProjectionViewPoint(ParentViewport, InDeltaTime, InOutViewInfo, OutCustomNearClippingPlane);
			}
		}
	}
}

bool FDisplayClusterProjectionLinkPolicy::CalculateView(IDisplayClusterViewport* InViewport, const uint32 InContextNum, FVector& InOutViewLocation, FRotator& InOutViewRotation, const FVector& ViewOffset, const float WorldToMeters, const float NCP, const float FCP)
{
	check(IsInGameThread());
	check(InViewport);

	if (IDisplayClusterViewportManager* ViewportManager = InViewport->GetConfiguration().GetViewportManager())
	{
		if (IDisplayClusterViewport* ParentViewport = ViewportManager->FindViewport(GetParentViewport(InViewport)))
		{
			const TArray<FDisplayClusterViewport_Context>& ParentContexts = ParentViewport->GetContexts();
			if (InContextNum < (uint32)ParentContexts.Num())
			{
				bool bResult = true;
				if (ParentContexts[InContextNum].bDisableRender && ParentViewport->GetProjectionPolicy().IsValid())
				{
					// When the parent viewport is not rendered, we must force a refresh.
					bResult = ParentViewport->CalculateView(InContextNum, InOutViewLocation, InOutViewRotation, WorldToMeters);
				}

				const FDisplayClusterViewport_Context& ParentContext = ParentContexts[InContextNum];
				InOutViewLocation = ParentContext.ViewLocation;
				InOutViewRotation = ParentContext.ViewRotation;

				return bResult;
			}
		}
	}

	return false;
}

bool FDisplayClusterProjectionLinkPolicy::GetProjectionMatrix(IDisplayClusterViewport* InViewport, const uint32 InContextNum, FMatrix& OutPrjMatrix)
{
	check(IsInGameThread());
	check(InViewport);

	if (IDisplayClusterViewportManager* ViewportManager = InViewport->GetConfiguration().GetViewportManager())
	{
		if (IDisplayClusterViewport* ParentViewport = ViewportManager->FindViewport(GetParentViewport(InViewport)))
		{
			const TArray<FDisplayClusterViewport_Context>& ParentContexts = ParentViewport->GetContexts();
			if (InContextNum < (uint32)ParentContexts.Num())
			{
				bool bResult = true;
				if (ParentContexts[InContextNum].bDisableRender && ParentViewport->GetProjectionPolicy().IsValid())
				{
					// When the parent viewport is not rendered, we must force a refresh.
					bResult = ParentViewport->GetProjectionMatrix(InContextNum, OutPrjMatrix);
				}

				const FDisplayClusterViewport_Context& ParentContext = ParentContexts[InContextNum];

				// Projection angles [Left, Right, Top, Bottom]
				FVector4 ProjectionAngles = ParentContext.ProjectionData.ProjectionAngles;

				const FDisplayClusterViewport_RenderSettings& InRenderSettings = InViewport->GetRenderSettings();
				if (InRenderSettings.TileSettings.GetType() == EDisplayClusterViewportTileType::Tile
				&& InViewport->GetContexts().IsValidIndex(InContextNum))
				{
					const FDisplayClusterViewport_Context& InContext = InViewport->GetContexts()[InContextNum];

					const FIntRect SrcRect      = ParentContext.RenderTargetRect;
					const FIntRect TileDestRect = InContext.TileDestRect;

					// Get frustum split multipliers
					// Frustum split multipliers [Left, Right, Top, Bottom]
					const FVector4 TileFrustumRegion(
						double(TileDestRect.Min.X - SrcRect.Min.X) / double(SrcRect.Width()),
						double(TileDestRect.Max.X - SrcRect.Min.X) / double(SrcRect.Width()),
						1-double(TileDestRect.Min.Y - SrcRect.Min.Y) / double(SrcRect.Height()),
						1-double(TileDestRect.Max.Y - SrcRect.Min.Y) / double(SrcRect.Height())
					);

					if (ParentContext.ProjectionData.bUseOverscan)
					{
						// We are tiling a viewport that uses overscan, so we need to use extended corners
						ProjectionAngles = ParentContext.ProjectionData.OverscanProjectionAngles;
					}
					
					const double FrustumWidth  = ProjectionAngles.Y - ProjectionAngles.X;
					const double FrustumHeight = ProjectionAngles.Z - ProjectionAngles.W;
					const double FrustumLeft   = ProjectionAngles.X;
					const double FrustumBottom = ProjectionAngles.W;

					// Split frustum into tiles.
					ProjectionAngles.X = FrustumLeft   + TileFrustumRegion.X * FrustumWidth;
					ProjectionAngles.Y = FrustumLeft   + TileFrustumRegion.Y * FrustumWidth;
					ProjectionAngles.Z = FrustumBottom + TileFrustumRegion.Z * FrustumHeight;
					ProjectionAngles.W = FrustumBottom + TileFrustumRegion.W * FrustumHeight;
				}

				// Calculate projection matrix:
				InViewport->CalculateProjectionMatrix(
					InContextNum,
					ProjectionAngles.X,ProjectionAngles.Y, ProjectionAngles.Z, ProjectionAngles.W,
					ParentContext.ProjectionData.ZNear,
					ParentContext.ProjectionData.ZFar,
					false
				);

				OutPrjMatrix = InViewport->GetContexts()[InContextNum].ProjectionMatrix;

				return bResult;
			}
		}
	}

	return false;
}
