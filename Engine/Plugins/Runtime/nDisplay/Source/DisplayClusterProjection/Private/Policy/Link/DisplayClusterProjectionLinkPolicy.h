// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Policy/DisplayClusterProjectionPolicyBase.h"


/**
 * Link projection policy (for internal usage)
 */
class FDisplayClusterProjectionLinkPolicy
	: public FDisplayClusterProjectionPolicyBase
{
public:
	FDisplayClusterProjectionLinkPolicy(const FString& ProjectionPolicyId, const FDisplayClusterConfigurationProjection* InConfigurationProjectionPolicy);

public:
	//////////////////////////////////////////////////////////////////////////////////////////////
	// IDisplayClusterProjectionPolicy
	//////////////////////////////////////////////////////////////////////////////////////////////
	virtual const FString& GetType() const override;

	// Return values from linked parent viewport
	// Important note: before doing this, the parent viewports must be updated.
	virtual bool CalculateView(IDisplayClusterViewport* InViewport, const uint32 InContextNum, FVector& InOutViewLocation, FRotator& InOutViewRotation, const FVector& ViewOffset, const float WorldToMeters, const float NCP, const float FCP) override;
	virtual bool GetProjectionMatrix(IDisplayClusterViewport* InViewport, const uint32 InContextNum, FMatrix& OutPrjMatrix) override;
	virtual void SetupProjectionViewPoint(IDisplayClusterViewport* InViewport, const float InDeltaTime, FMinimalViewInfo& InOutViewInfo, float* OutCustomNearClippingPlane = nullptr) override;

	virtual bool ShouldUseSourceTextureWithMips(const IDisplayClusterViewport* InViewport) const override
	{
		return true;
	}

private:
	/** Get parent viewport name. */
	const FString& GetParentViewport(IDisplayClusterViewport* InViewport) const;
};
