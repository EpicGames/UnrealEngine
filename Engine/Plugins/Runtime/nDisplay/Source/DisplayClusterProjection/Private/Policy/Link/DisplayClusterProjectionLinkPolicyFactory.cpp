// Copyright Epic Games, Inc. All Rights Reserved.

#include "Policy/Link/DisplayClusterProjectionLinkPolicyFactory.h"
#include "Policy/Link/DisplayClusterProjectionLinkPolicy.h"

#include "DisplayClusterProjectionLog.h"

#include "DisplayClusterConfigurationTypes.h"


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProjectionPolicyFactory
//////////////////////////////////////////////////////////////////////////////////////////////
TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe> FDisplayClusterProjectionLinkPolicyFactory::Create(const FString& ProjectionPolicyId, const FDisplayClusterConfigurationProjection* InConfigurationProjectionPolicy)
{
	if(!InConfigurationProjectionPolicy)
	{
		if (CanShowMsgOnce(ProjectionPolicyId))
		{
			UE_LOGF(LogDisplayClusterProjectionSimple, Error, "Link policy '%ls': null configuration", *ProjectionPolicyId);
		}
		return nullptr;
	}

	UE_LOGF(LogDisplayClusterProjectionSimple, Verbose, "Instantiating projection policy <%ls> id='%ls'", *InConfigurationProjectionPolicy->Type, *ProjectionPolicyId);
	return  MakeShared<FDisplayClusterProjectionLinkPolicy, ESPMode::ThreadSafe>(ProjectionPolicyId, InConfigurationProjectionPolicy);
}
