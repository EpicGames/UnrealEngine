// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Render/Projection/IDisplayClusterProjectionPolicyFactory.h"
#include "Policy/DisplayClusterProjectionPolicyFactoryBase.h"

/**
 * Factory for the 'link' projection policy (internal use only).
 * A linked viewport inherits its view and projection matrix from a designated parent viewport
 * instead of computing them independently.
 */
class FDisplayClusterProjectionLinkPolicyFactory
	: public IDisplayClusterProjectionPolicyFactory
	, protected FDisplayClusterProjectionPolicyFactoryBase
{
public:
	//~ Begin IDisplayClusterProjectionPolicyFactory
	virtual TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe> Create(const FString& ProjectionPolicyId, const FDisplayClusterConfigurationProjection* InConfigurationProjectionPolicy) override;
	//~ End IDisplayClusterProjectionPolicyFactory
};
