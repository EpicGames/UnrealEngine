// Copyright Epic Games, Inc. All Rights Reserved.

#include "Policy/MPCDI/DisplayClusterProjectionMPCDIPolicy.h"
#include "Policy/MPCDI/DisplayClusterProjectionMPCDIPolicy_Config.h"

#include "DisplayClusterProjectionLog.h"
#include "DisplayClusterProjectionStrings.h"

#include "IDisplayCluster.h"
#include "IDisplayClusterCallbacks.h"
#include "IDisplayClusterShaders.h"
#include "IDisplayClusterWarp.h"

#include "Config/IDisplayClusterConfigManager.h"
#include "Game/IDisplayClusterGameManager.h"

#include "Misc/DisplayClusterHelpers.h"

#include "Engine/RendererSettings.h"
#include "Misc/Paths.h"

#include "Render/Viewport/IDisplayClusterViewportManager.h"
#include "Render/Viewport/IDisplayClusterViewportManagerProxy.h"
#include "Render/Viewport/IDisplayClusterViewport.h"
#include "Render/Viewport/IDisplayClusterViewportProxy.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_RenderSettingsICVFX.h"
#include "Render/Warp/IDisplayClusterWarpPolicy.h"

#include "ShaderParameters/DisplayClusterShaderParameters_WarpBlend.h"
#include "Blueprints/DisplayClusterWarpGeometry.h"

#include "DisplayClusterRootActor.h"
#include "Components/DisplayClusterScreenComponent.h"

namespace UE::DisplayClusterProjection::MPCDIPolicy
{
	static IDisplayClusterWarp& GetWarpAPI()
	{
		static IDisplayClusterWarp& DisplayClusterWarpAPISingleton = IDisplayClusterWarp::Get();

		return DisplayClusterWarpAPISingleton;
	}

	static inline IDisplayClusterShaders& GetShadersAPI()
	{
		IDisplayClusterShaders& ShadersAPISingleton = IDisplayClusterShaders::Get();

		return ShadersAPISingleton;
	}
};
using namespace UE::DisplayClusterProjection::MPCDIPolicy;

//---------------------------------------------------------------------------------------------
// FDisplayClusterProjectionMPCDIPolicy
//---------------------------------------------------------------------------------------------
FDisplayClusterProjectionMPCDIPolicy::FDisplayClusterProjectionMPCDIPolicy(const FString& ProjectionPolicyId, const FDisplayClusterConfigurationProjection* InConfigurationProjectionPolicy)
	: FDisplayClusterProjectionPolicyBase(ProjectionPolicyId, InConfigurationProjectionPolicy)
{
}

FDisplayClusterProjectionMPCDIPolicy::~FDisplayClusterProjectionMPCDIPolicy()
{
	ImplRelease();
}

void FDisplayClusterProjectionMPCDIPolicy::SetWarpPolicy(IDisplayClusterWarpPolicy* InWarpPolicy)
{
	check(IsInGameThread());

	WarpPolicyInterface = InWarpPolicy ? InWarpPolicy->ToSharedPtr() : nullptr;
}

IDisplayClusterWarpPolicy* FDisplayClusterProjectionMPCDIPolicy::GetWarpPolicy() const
{
	check(IsInGameThread());

	return WarpPolicyInterface.Get();
}

IDisplayClusterWarpPolicy* FDisplayClusterProjectionMPCDIPolicy::GetWarpPolicy_RenderThread() const
{
	check(IsInRenderingThread());

	return WarpPolicyInterface_Proxy.Get();
}

bool FDisplayClusterProjectionMPCDIPolicy::ShouldSupportICVFX(const IDisplayClusterViewport* InViewport) const
{
	check(IsInGameThread());
	check(InViewport);

	if (!WarpBlendInterface.IsValid() || !WarpBlendInterface->ShouldSupportICVFX())
	{
		// WarpBlend does not support the ICVFX pipeline (mpcdi 2D,3D,SL profiles)
		return false;
	}

	if (WarpPolicyInterface.IsValid() && !WarpPolicyInterface->ShouldSupportICVFX(InViewport))
	{
		// Warp policy does not support the ICVFX pipeline
		return false;
	}

	return true;
}

void FDisplayClusterProjectionMPCDIPolicy::UpdateProxyData(IDisplayClusterViewport* InViewport)
{
	check(InViewport);

	ENQUEUE_RENDER_COMMAND(DisplayClusterProjectionMPCDIPolicy_UpdateProxyData)(
		[
			This = SharedThis(this),
			WarpBlendInterfacePtr = WarpBlendInterface,
			WarPolicyInterfacePtr = WarpPolicyInterface,
			Contexts = WarpBlendContexts
		](FRHICommandListImmediate& RHICmdList)
	{
		This->WarpBlendInterface_Proxy = WarpBlendInterfacePtr;
		This->WarpPolicyInterface_Proxy = WarPolicyInterfacePtr;
		This->WarpBlendContexts_Proxy = Contexts;
	});
}

//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProjectionPolicy
//////////////////////////////////////////////////////////////////////////////////////////////
const FString& FDisplayClusterProjectionMPCDIPolicy::GetType() const
{
	static const FString Type(DisplayClusterProjectionStrings::projection::MPCDI);
	return Type;
}

bool FDisplayClusterProjectionMPCDIPolicy::HandleStartScene(IDisplayClusterViewport* InViewport)
{
	if (bInvalidConfiguration)
	{
		return false;
	}

	check(IsInGameThread());

	// The game side of the nDisplay has been initialized by the nDisplay Game Manager already
	// so we can extend it by our projection related functionality/components/etc.

	WarpBlendContexts.Empty();

	if (WarpBlendInterface.IsValid() == false && !CreateWarpBlendFromConfig(InViewport))
	{
		// Ignore broken MPCDI config for other attempts
		bInvalidConfiguration = true;

		if (!IsEditorOperationMode(InViewport))
		{
			UE_LOGF(LogDisplayClusterProjectionMPCDI, Warning, "Couldn't load MPCDI config for viewport '%ls'", *InViewport->GetId());
		}

		return false;
	}

	// Finally, initialize internal views data container
	WarpBlendContexts.AddDefaulted(2);

	if (WarpBlendInterface.IsValid())
	{
		WarpBlendInterface->HandleStartScene(InViewport);
	}

	return true;
}

void FDisplayClusterProjectionMPCDIPolicy::HandleEndScene(IDisplayClusterViewport* InViewport)
{
	check(IsInGameThread());

	if (WarpBlendInterface.IsValid())
	{
		WarpBlendInterface->HandleEndScene(InViewport);
	}

	ImplRelease();
}

void FDisplayClusterProjectionMPCDIPolicy::ImplRelease()
{
	ReleaseOriginComponent();

	WarpBlendInterface.Reset();
	WarpBlendContexts.Empty();

	PreviewMeshComponentRef.ResetSceneComponent();
	PreviewEditableMeshComponentRef.ResetSceneComponent();
}

bool FDisplayClusterProjectionMPCDIPolicy::GetWarpBlendInterface(TSharedPtr<IDisplayClusterWarpBlend, ESPMode::ThreadSafe>& OutWarpBlendInterface) const
{
	check(IsInGameThread());

	if (WarpBlendInterface.IsValid())
	{
		OutWarpBlendInterface = WarpBlendInterface;

		return true;
	}

	return false;
}

bool FDisplayClusterProjectionMPCDIPolicy::GetWarpBlendInterface_RenderThread(TSharedPtr<IDisplayClusterWarpBlend, ESPMode::ThreadSafe>& OutWarpBlendInterfaceProxy) const
{
	check(IsInRenderingThread());

	if (WarpBlendInterface.IsValid())
	{
		OutWarpBlendInterfaceProxy = WarpBlendInterface_Proxy;

		return true;
	}

	return false;
}

bool FDisplayClusterProjectionMPCDIPolicy::CalculateView(IDisplayClusterViewport* InViewport, const uint32 InContextNum, FVector& InOutViewLocation, FRotator& InOutViewRotation, const FVector& InViewOffset, const float WorldToMeters, const float NCP, const float FCP)
{
	check(IsInGameThread());

	if (!InViewport || !InViewport->GetContexts().IsValidIndex(InContextNum))
	{
		return false;
	}

	if (WarpBlendInterface.IsValid() == false || WarpBlendContexts.Num() == 0)
	{
		if (!IsEditorOperationMode(InViewport))
		{
			UE_LOGF(LogDisplayClusterProjectionMPCDI, Warning, "Invalid warp data for viewport '%ls'", *InViewport->GetId());
		}

		return false;
	}

	// World scale multiplier
	const float WorldScale = WorldToMeters / 100.f;

	// Initialize frustum
	TSharedPtr<FDisplayClusterWarpEye, ESPMode::ThreadSafe> WarpEye = MakeShared<FDisplayClusterWarpEye, ESPMode::ThreadSafe>(InViewport->ToSharedPtr(), InContextNum);

	const FDisplayClusterViewport_Context& InContext = InViewport->GetContexts()[InContextNum];

	WarpEye->World2LocalTransform = InContext.OriginToWorld;

	// Get our base camera location and view offset in local space (MPCDI space)
	WarpEye->ViewPoint.Location  = InOutViewLocation - InViewOffset;
	WarpEye->ViewPoint.EyeOffset = InViewOffset;
	WarpEye->ViewPoint.Rotation  = InOutViewRotation;

	WarpEye->WorldScale = WorldScale;

	// Use current warp policy
	WarpEye->WarpPolicy = WarpPolicyInterface;

	// discard current context
	WarpBlendContexts[InContextNum].bIsValid = false;

	// Compute frustum
	if (!WarpBlendInterface->CalcFrustumContext(WarpEye))
	{
		return false;
	}

	// Readback warp context
	WarpBlendContexts[InContextNum] = WarpBlendInterface->GetWarpData(InContextNum).WarpContext;

	// Save Origin viewpoint transform to world space.
	WarpBlendContexts[InContextNum].Origin2WorldTransform = WarpEye->World2LocalTransform;

	// Transform viewpoint back to world space
	InOutViewRotation = WarpBlendContexts[InContextNum].Rotation;
	InOutViewLocation = WarpBlendContexts[InContextNum].Location;

	WarpBlendContexts[InContextNum].bIsValid = true;

	return true;
}

bool FDisplayClusterProjectionMPCDIPolicy::GetProjectionMatrix(IDisplayClusterViewport* InViewport, const uint32 InContextNum, FMatrix& OutPrjMatrix)
{
	check(IsInGameThread());

	if (InContextNum < (uint32)WarpBlendContexts.Num())
	{
		OutPrjMatrix = WarpBlendContexts[InContextNum].ProjectionMatrix;

		return true;
	}
	
	return false;
}

bool FDisplayClusterProjectionMPCDIPolicy::IsFrustumRotatedToFitContextSize(IDisplayClusterViewport* InViewport, const uint32 InContextNum)
{
	check(IsInGameThread());

	if (WarpBlendContexts.IsValidIndex(InContextNum))
	{
		return WarpBlendContexts[InContextNum].bFrustumRotated;
	}

	return false;
}

bool FDisplayClusterProjectionMPCDIPolicy::IsFrustumRotatedToFitContextSize_RenderThread(const IDisplayClusterViewportProxy* InViewportProxy, const uint32 InContextNum)
{
	check(IsInRenderingThread());

	if (WarpBlendContexts_Proxy.IsValidIndex(InContextNum))
	{
		return WarpBlendContexts_Proxy[InContextNum].bFrustumRotated;
	}

	return false;
}

bool FDisplayClusterProjectionMPCDIPolicy::IsWarpBlendSupported(const IDisplayClusterViewport* InViewport) const
{
	return true;
}

bool FDisplayClusterProjectionMPCDIPolicy::IsWarpBlendSupported_RenderThread(const IDisplayClusterViewportProxy* InViewportProxy) const
{
	return true;
}

void FDisplayClusterProjectionMPCDIPolicy::ApplyWarpBlend_RenderThread(FRHICommandListImmediate& RHICmdList, const IDisplayClusterViewportProxy* InViewportProxy)
{
	check(IsInRenderingThread());

	if (InViewportProxy==nullptr || WarpBlendInterface_Proxy.IsValid() == false || WarpBlendContexts_Proxy.Num() == 0)
	{
		return;
	}

	const IDisplayClusterViewportManagerProxy* ViewportManagerProxyPtr = InViewportProxy->GetConfigurationProxy().GetViewportManagerProxy_RenderThread();
	if (!ViewportManagerProxyPtr)
	{
		return;
	}

	TArray<FRHITexture*> InputTextures, OutputTextures;
	TArray<FIntRect> InputRects, OutputRects;

	// Use for input first MipsShader texture if enabled in viewport render settings
	if (!InViewportProxy->GetResourcesWithRects_RenderThread(EDisplayClusterViewportResourceType::MipsShaderResource, InputTextures, InputRects))
	{
		// otherwise inputshader texture
		if (!InViewportProxy->GetResourcesWithRects_RenderThread(EDisplayClusterViewportResourceType::InputShaderResource, InputTextures, InputRects))
		{
			// no source textures
			return;
		}
	}

	// Get output resources with rects
	// warp result is now inside AdditionalRTT.  Later, from the DC ViewportManagerProxy it will be resolved to FrameRTT 
	if (!InViewportProxy->GetResourcesWithRects_RenderThread(EDisplayClusterViewportResourceType::AfterWarpBlendTargetableResource, OutputTextures, OutputRects))
	{
		return;
	}

	const FDisplayClusterViewport_RenderSettingsICVFX& SettingsICVFX = InViewportProxy->GetRenderSettingsICVFX_RenderThread();
	// ICVFX use custom rendering logic.
	const bool bRenderICVFX = EnumHasAllFlags(SettingsICVFX.RuntimeFlags, EDisplayClusterViewportRuntimeICVFXFlags::Target)
							&& InViewportProxy->GetConfigurationProxy().IsICVFXEnabled_RenderThread();
	if (bRenderICVFX)
	{
		// This viewport used as the target of ICVFX.
		FDisplayClusterShaderParameters_ICVFX ShaderICVFX(SettingsICVFX.ICVFX);

		// Iterate over all warped viewport contexts (support for stereo views).
		for (int32 ContextNum = 0; ContextNum < InputTextures.Num(); ContextNum++)
		{
			// Update referenced resources and math in the ShaderICVFX.
			ShaderICVFX.IterateViewportResourcesByPredicate([ContextNum, ViewportManagerProxyPtr, &ShaderICVFX, WarpContext = WarpBlendContexts_Proxy[ContextNum]](FDisplayClusterShaderParametersICVFX_ViewportResource& ViewportResourceIt)
			{
				// reset prev resource reference
				ViewportResourceIt.Texture = nullptr;
				if(const IDisplayClusterViewportProxy* SrcViewportProxy = ViewportManagerProxyPtr->FindViewport_RenderThread(ViewportResourceIt.ViewportId))
				{
					if (!SrcViewportProxy->GetRenderSettings_RenderThread().bSkipRendering)
					{
						TArray<FRHITexture*> RefTextures;
						// Use for input first MipsShader texture if enabled in viewport render settings
						if (SrcViewportProxy->GetResources_RenderThread(EDisplayClusterViewportResourceType::MipsShaderResource, RefTextures) ||
							SrcViewportProxy->GetResources_RenderThread(EDisplayClusterViewportResourceType::InputShaderResource, RefTextures))
						{
							if (ContextNum < RefTextures.Num())
							{
								ViewportResourceIt.Texture = RefTextures[ContextNum];
							}
						}
					}

					// Override camera context from referenced viewport:
					if(FDisplayClusterShaderParameters_ICVFX::FCameraSettings* CameraSettings = ShaderICVFX.FindCameraByName(ViewportResourceIt.ViewportId))
					{
						// Support stereo icvfx
						const TArray<FDisplayClusterViewport_Context>& SrcViewportContexts = SrcViewportProxy->GetContexts_RenderThread();
						if (SrcViewportContexts.IsValidIndex(ContextNum))
						{
							// matched camera reference, update context for current eye
							const FDisplayClusterViewport_Context& InContext = SrcViewportContexts[ContextNum];

							// The context stores data in local space. For the shader they must be converted to world space
							FDisplayClusterShaderParametersICVFX_CameraViewProjection LocalSpaceViewProjection;
							LocalSpaceViewProjection.ViewLocation = InContext.ViewLocation;
							LocalSpaceViewProjection.ViewRotation = InContext.ViewRotation;
							LocalSpaceViewProjection.PrjMatrix    = InContext.ProjectionMatrix;

							CameraSettings->SetViewProjection(LocalSpaceViewProjection, WarpContext.Origin2WorldTransform);
						}
					}
				}
			});

			// cleanup camera list for render
			ShaderICVFX.CleanupCamerasForRender();

			// Convert Lightcards gamma from linear to the Outer viewport gamma
			FDisplayClusterColorEncoding InputEncoding = InViewportProxy->GetResourceColorEncoding_RenderThread(EDisplayClusterViewportResourceType::InputShaderResource);
			if (InputEncoding == EDisplayClusterColorEncoding::Linear)
			{
				ShaderICVFX.LightCardGamma = 1.0f;
			}
			else
			{
				ShaderICVFX.LightCardGamma = 2.2f;
			}

			// Initialize shader input data
			FDisplayClusterShaderParameters_WarpBlend WarpBlendParameters;

			WarpBlendParameters.Context = WarpBlendContexts_Proxy[ContextNum];
			WarpBlendParameters.WarpInterface = WarpBlendInterface_Proxy;

			WarpBlendParameters.Src.Set(InputTextures[ContextNum], InputRects[ContextNum]);
			WarpBlendParameters.Dest.Set(OutputTextures[ContextNum], OutputRects[ContextNum]);

			WarpBlendParameters.bRenderAlphaChannel = InViewportProxy->GetRenderSettings_RenderThread().bWarpBlendRenderAlphaChannel;

			// Before starting the whole ICVFX shaders pipeline, we need to provide ability to modify any ICVFX shader parameters.
			// One of use cases is the latency queue that needs to substitute shader parameters.
			IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPreProcessIcvfx_RenderThread().Broadcast(RHICmdList, InViewportProxy, WarpBlendParameters, ShaderICVFX);

			// Start ICVFX pipeline
			if (!GetShadersAPI().RenderWarpBlend_ICVFX(RHICmdList, WarpBlendParameters, ShaderICVFX))
			{
				if (!IsEditorOperationMode_RenderThread(InViewportProxy))
				{
					UE_LOGF(LogDisplayClusterProjectionMPCDI, Warning, "Couldn't apply icvfx warp&blend");
				}

				// Break iteration of target viewport contexts
				break;
			}
		}

		// ICVFX warpblend ready
		return;
	}
	
	// Mesh warp:
	for (int32 ContextNum = 0; ContextNum < InputTextures.Num(); ContextNum++)
	{
		// Initialize shader input data
		FDisplayClusterShaderParameters_WarpBlend WarpBlendParameters;

		WarpBlendParameters.Context = WarpBlendContexts_Proxy[ContextNum];
		WarpBlendParameters.WarpInterface = WarpBlendInterface_Proxy;

		WarpBlendParameters.Src.Set(InputTextures[ContextNum], InputRects[ContextNum]);
		WarpBlendParameters.Dest.Set(OutputTextures[ContextNum], OutputRects[ContextNum]);
		
		WarpBlendParameters.bRenderAlphaChannel = InViewportProxy->GetRenderSettings_RenderThread().bWarpBlendRenderAlphaChannel;

		if (!GetShadersAPI().RenderWarpBlend_MPCDI(RHICmdList, WarpBlendParameters))
		{
			if (!IsEditorOperationMode_RenderThread(InViewportProxy))
			{
				UE_LOGF(LogDisplayClusterProjectionMPCDI, Warning, "Couldn't apply mpcdi warp&blend");
			}

			return;
		}
	}
}

bool FDisplayClusterProjectionMPCDIPolicy::HasPreviewMesh(IDisplayClusterViewport* InViewport)
{
	if (bIsPreviewMeshEnabled)
	{
		return true;
	}

	PreviewMeshComponentRef.ResetSceneComponent();

	return false;
}

UMeshComponent* FDisplayClusterProjectionMPCDIPolicy::GetOrCreatePreviewMeshComponent(IDisplayClusterViewport* InViewport, bool& bOutIsRootActorComponent)
{
	if (!HasPreviewMesh(InViewport))
	{
		return nullptr;
	}

	// If we have already created a preview mesh component before, return that component
	if (UMeshComponent* ExistsPreviewMeshComp = Cast<UMeshComponent>(PreviewMeshComponentRef.GetOrFindSceneComponent()))
	{
		bOutIsRootActorComponent = bIsRootActorHasPreviewMeshComponent;
		return ExistsPreviewMeshComp;
	}

	// Get a new one
	if (UMeshComponent* PreviewMeshComp = WarpBlendInterface.IsValid() ? WarpBlendInterface->GetOrCreatePreviewMeshComponent(InViewport, bOutIsRootActorComponent) : nullptr)
	{
		// Store reference to mesh component
		PreviewMeshComponentRef.SetSceneComponent(PreviewMeshComp);
		bIsRootActorHasPreviewMeshComponent = bOutIsRootActorComponent;

		return PreviewMeshComp;
	}

	return nullptr;
}

USceneComponent* const FDisplayClusterProjectionMPCDIPolicy::GetPreviewEditableMeshOriginComponent(IDisplayClusterViewport* InViewport) const
{
	// Note: currently for the Editable mesh component we expect it to be used only in the scene,
	// so we use the root component from the scene all the time.
	// But if other use cases are found, we need to refine this logic.

	return GetOriginComponent();
}

bool FDisplayClusterProjectionMPCDIPolicy::HasPreviewEditableMesh(IDisplayClusterViewport* InViewport)
{
	if (bIsPreviewMeshEnabled && InViewport)
	{
		// The editable preview Editable mesh is a feature for the warp policy, so we must request permission to use it.
		if (WarpPolicyInterface.IsValid() && WarpPolicyInterface->HasPreviewEditableMesh(InViewport))
		{
			return true;
		}
	}

	PreviewEditableMeshComponentRef.ResetSceneComponent();

	return false;
}

UMeshComponent* FDisplayClusterProjectionMPCDIPolicy::GetOrCreatePreviewEditableMeshComponent(IDisplayClusterViewport* InViewport)
{
	if (!HasPreviewEditableMesh(InViewport))
	{
		return nullptr;
	}

	// If we have already created a preview mesh component before, return that component
	if (UMeshComponent* ExistsPreviewEditableMeshComp = Cast<UMeshComponent>(PreviewEditableMeshComponentRef.GetOrFindSceneComponent()))
	{
		return ExistsPreviewEditableMeshComp;
	}

	// Get a new one
	if (UMeshComponent* PreviewEditableMeshComp = WarpBlendInterface.IsValid() ? WarpBlendInterface->GetOrCreatePreviewEditableMeshComponent(InViewport) : nullptr)
	{
		// Store reference to mesh component
		PreviewEditableMeshComponentRef.SetSceneComponent(PreviewEditableMeshComp);

		return PreviewEditableMeshComp;
	}

	return nullptr;
}

bool FDisplayClusterProjectionMPCDIPolicy::CreateWarpBlendFromConfig(IDisplayClusterViewport* InViewport)
{
	check(IsInGameThread());

	bool bResult = false;

	FDisplayClusterProjectionMPCDIPolicy_ConfigParser CfgData(InViewport, GetParameters());
	if (CfgData.IsValid())
	{
		// Support custom origin node
		InitializeOriginComponent(InViewport, CfgData.OriginType);

		bIsPreviewMeshEnabled = CfgData.bEnablePreview;

		// Load from MPCDI file:
		if (CfgData.PFMFile.IsEmpty())
		{
			// Check if MPCDI file exists
			if (CfgData.MPCDIFileName.IsEmpty())
			{
				if (!IsEditorOperationMode(InViewport))
				{
					UE_LOGF(LogDisplayClusterProjectionMPCDI, Warning, "File not found: Empty");
				}

				return false;
			}

			if (!FPaths::FileExists(CfgData.MPCDIFileName))
			{
				UE_LOGF(LogDisplayClusterProjectionMPCDI, Warning, "File not found: %ls", *CfgData.MPCDIFileName);
				return false;
			}

			// Using the screen component as the "region" for the MPCDI 2d profile.
			const bool bUseScreenComponentForMPCDIProfile2D = CfgData.ScreenComponent && CfgData.MPCDIAttributes.ProfileType == EDisplayClusterWarpProfileType::warp_2D;
			if (!bUseScreenComponentForMPCDIProfile2D)
			{
				FDisplayClusterWarpInitializer_MPCDIFile CreateParameters;
				CreateParameters.MPCDIFileName = CfgData.MPCDIFileName;
				CreateParameters.BufferId = CfgData.BufferId;
				CreateParameters.RegionId = CfgData.RegionId;

				WarpBlendInterface = GetWarpAPI().Create(CreateParameters);
			}
			else
			{
				FDisplayClusterWarpInitializer_MPCDIFile_Profile2DScreen CreateParameters;
				CreateParameters.MPCDIFileName = CfgData.MPCDIFileName;
				CreateParameters.BufferId = CfgData.BufferId;
				CreateParameters.RegionId = CfgData.RegionId;

				// Using the screen component as the "region" for the MPCDI 2d profile.
				CreateParameters.WarpMeshComponent = CfgData.ScreenComponent;
				CreateParameters.PreviewMeshComponent = CfgData.PreviewScreenComponent;

				WarpBlendInterface = GetWarpAPI().Create(CreateParameters);
			}

			if (!WarpBlendInterface.IsValid())
			{
				UE_LOGF(LogDisplayClusterProjectionMPCDI, Warning, "Couldn't load MPCDI file: %ls", *CfgData.MPCDIFileName);
				return false;
			}

			UE_LOGF(LogDisplayClusterProjectionMPCDI, Verbose, "MPCDI policy has been initialized [%ls:%ls in %ls]", *CfgData.BufferId, *CfgData.RegionId, *CfgData.MPCDIFileName);
			return true;
		}

		FDisplayClusterWarpInitializer_PFMFile CreateParameters;

		CreateParameters.MPCDIAttributes = CfgData.MPCDIAttributes;
		CreateParameters.PFMFileName = CfgData.PFMFile;

		CreateParameters.PFMScale = CfgData.PFMFileScale;
		CreateParameters.bIsUnrealGameSpace = CfgData.bIsUnrealGameSpace;

		CreateParameters.AlphaMapFileName = CfgData.AlphaFile;
		CreateParameters.AlphaMapEmbeddedAlpha = CfgData.AlphaGamma;

		CreateParameters.BetaMapFileName = CfgData.BetaFile;

		WarpBlendInterface = GetWarpAPI().Create(CreateParameters);

		if (!WarpBlendInterface.IsValid())
		{
			UE_LOGF(LogDisplayClusterProjectionMPCDI, Error, "Failed to load PFM from file: %ls", *CfgData.PFMFile);
			return false;
		}

		UE_LOGF(LogDisplayClusterProjectionMPCDI, Verbose, "MPCDI policy has been initialized from PFM '%ls", *CfgData.PFMFile);
		return true;
	}

	return false;
}
