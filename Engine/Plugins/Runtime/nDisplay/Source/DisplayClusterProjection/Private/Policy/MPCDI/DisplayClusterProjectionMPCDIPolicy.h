// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Policy/DisplayClusterProjectionPolicyBase.h"
#include "Containers/DisplayClusterWarpContext.h"
#include "IDisplayClusterWarpBlend.h"
#include "Templates/SharedPointer.h"

class UMeshComponent;

/**
 * MPCDI projection policy
 * Supported load from 'MPCDI' and 'PFM' files
 */
class FDisplayClusterProjectionMPCDIPolicy
	: public FDisplayClusterProjectionPolicyBase
	, public TSharedFromThis<FDisplayClusterProjectionMPCDIPolicy, ESPMode::ThreadSafe>
{
public:
	FDisplayClusterProjectionMPCDIPolicy(const FString& ProjectionPolicyId, const FDisplayClusterConfigurationProjection* InConfigurationProjectionPolicy);
	virtual ~FDisplayClusterProjectionMPCDIPolicy();

public:
	// This policy can support ICVFX rendering
	virtual bool ShouldSupportICVFX(const IDisplayClusterViewport* InViewport) const override;

public:
	//~Begin IDisplayClusterProjectionPolicy
	virtual const FString& GetType() const override;

	virtual bool HandleStartScene(IDisplayClusterViewport* InViewport) override;
	virtual void HandleEndScene(IDisplayClusterViewport* InViewport) override;

	virtual bool CalculateView(IDisplayClusterViewport* InViewport, const uint32 InContextNum, FVector& InOutViewLocation, FRotator& InOutViewRotation, const FVector& ViewOffset, const float WorldToMeters, const float NCP, const float FCP) override;
	virtual bool GetProjectionMatrix(IDisplayClusterViewport* InViewport, const uint32 InContextNum, FMatrix& OutPrjMatrix) override;

	virtual bool IsFrustumRotatedToFitContextSize(IDisplayClusterViewport* InViewport, const uint32 InContextNum) override;
	virtual bool IsFrustumRotatedToFitContextSize_RenderThread(const IDisplayClusterViewportProxy* InViewportProxy, const uint32 InContextNum) override;

	virtual bool IsWarpBlendSupported(const IDisplayClusterViewport* InViewport) const override;
	virtual bool IsWarpBlendSupported_RenderThread(const IDisplayClusterViewportProxy* InViewportProxy) const override;

	virtual void ApplyWarpBlend_RenderThread(FRHICommandListImmediate& RHICmdList, const IDisplayClusterViewportProxy* InViewportProxy) override;

	virtual bool GetWarpBlendInterface(TSharedPtr<class IDisplayClusterWarpBlend, ESPMode::ThreadSafe>& OutWarpBlendInterface) const override;
	virtual bool GetWarpBlendInterface_RenderThread(TSharedPtr<class IDisplayClusterWarpBlend, ESPMode::ThreadSafe>& OutWarpBlendInterfaceProxy) const override;

	virtual bool ShouldUseSourceTextureWithMips(const IDisplayClusterViewport* InViewport) const override
	{
		// Support input texture with mips
		return true;
	}

	virtual bool ShouldUseAdditionalTargetableResource(const IDisplayClusterViewport* InViewport) const override
	{
		// Request additional targetable resources for warp&blend output
		return true;
	}

	virtual EDisplayClusterProjectionPolicyFlags GetProjectionPolicyFlags(const IDisplayClusterViewport* InViewport, const uint32 InContextNum) const override
	{
		return EDisplayClusterProjectionPolicyFlags::EnableFollowCameraFeature
			| EDisplayClusterProjectionPolicyFlags::UseEyeTracking
			| EDisplayClusterProjectionPolicyFlags::UseLocalSpace;
	}

	virtual void UpdateProxyData(IDisplayClusterViewport* InViewport) override;

	virtual void SetWarpPolicy(IDisplayClusterWarpPolicy* InWarpPolicy) override;
	virtual IDisplayClusterWarpPolicy* GetWarpPolicy() const override;
	virtual IDisplayClusterWarpPolicy* GetWarpPolicy_RenderThread() const override;

	virtual bool HasPreviewMesh(IDisplayClusterViewport* InViewport) override;
	virtual UMeshComponent* GetOrCreatePreviewMeshComponent(IDisplayClusterViewport* InViewport, bool& bOutIsRootActorComponent) override;

	virtual bool HasPreviewEditableMesh(IDisplayClusterViewport* InViewport) override;
	virtual UMeshComponent* GetOrCreatePreviewEditableMeshComponent(IDisplayClusterViewport* InViewport) override;
	virtual USceneComponent* const GetPreviewEditableMeshOriginComponent(IDisplayClusterViewport* InViewport) const override;

	//~~End IDisplayClusterProjectionPolicy

protected:
	bool CreateWarpBlendFromConfig(IDisplayClusterViewport* InViewport);
	void ImplRelease();

protected:
	// GameThread: WarpBlend and WarpPolicy interfaces
	TSharedPtr<IDisplayClusterWarpBlend, ESPMode::ThreadSafe> WarpBlendInterface;
	TSharedPtr<IDisplayClusterWarpPolicy, ESPMode::ThreadSafe> WarpPolicyInterface;

	// RenderingThread: WarpBlend and WarpPolicy interfaces
	TSharedPtr<IDisplayClusterWarpBlend, ESPMode::ThreadSafe> WarpBlendInterface_Proxy;
	TSharedPtr<IDisplayClusterWarpPolicy, ESPMode::ThreadSafe> WarpPolicyInterface_Proxy;

	// Context for both game and rendering threads
	TArray<FDisplayClusterWarpContext> WarpBlendContexts;
	TArray<FDisplayClusterWarpContext> WarpBlendContexts_Proxy;

	bool bInvalidConfiguration = false;
	bool bIsPreviewMeshEnabled = false;

private:
	// Stored value of the preview mesh
	FDisplayClusterSceneComponentRef PreviewMeshComponentRef;

	// Stored value of the preview meshes belonging flag. True if this component exists in DCRA and cannot be deleted with preview.
	bool bIsRootActorHasPreviewMeshComponent = false;

	// Stored value of the editable preview mesh
	FDisplayClusterSceneComponentRef PreviewEditableMeshComponentRef;
};
