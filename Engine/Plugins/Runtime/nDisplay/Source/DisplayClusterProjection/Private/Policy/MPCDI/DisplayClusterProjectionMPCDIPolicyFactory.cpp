// Copyright Epic Games, Inc. All Rights Reserved.

#include "Policy/MPCDI/DisplayClusterProjectionMPCDIPolicyFactory.h"
#include "Policy/MPCDI/DisplayClusterProjectionMPCDIPolicy.h"
#include "Policy/Mesh/DisplayClusterProjectionMeshPolicy.h"

#include "DisplayClusterProjectionLog.h"
#include "DisplayClusterProjectionStrings.h"

#include "DisplayClusterConfigurationTypes.h"


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProjectionPolicyFactory
//////////////////////////////////////////////////////////////////////////////////////////////
TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe> FDisplayClusterProjectionMPCDIPolicyFactory::Create(const FString& ProjectionPolicyId, const FDisplayClusterConfigurationProjection* InConfigurationProjectionPolicy)
{
	if(!InConfigurationProjectionPolicy)
	{
		if (CanShowMsgOnce(ProjectionPolicyId))
		{
			UE_LOGF(LogDisplayClusterProjectionMPCDI, Error, "MPCDI policy '%ls': null configuration", *ProjectionPolicyId);
		}
		return nullptr;
	}

	UE_LOGF(LogDisplayClusterProjectionMPCDI, Verbose, "Instantiating projection policy <%ls>...id='%ls'", *InConfigurationProjectionPolicy->Type, *ProjectionPolicyId);

	if (InConfigurationProjectionPolicy->Type.Equals(DisplayClusterProjectionStrings::projection::MPCDI, ESearchCase::IgnoreCase))
	{
		return MakeShared<FDisplayClusterProjectionMPCDIPolicy, ESPMode::ThreadSafe>(ProjectionPolicyId, InConfigurationProjectionPolicy);
	}

	if (InConfigurationProjectionPolicy->Type.Equals(DisplayClusterProjectionStrings::projection::Mesh, ESearchCase::IgnoreCase))
	{
		return MakeShared<FDisplayClusterProjectionMeshPolicy, ESPMode::ThreadSafe>(ProjectionPolicyId, InConfigurationProjectionPolicy);
	}

	return MakeShared<FDisplayClusterProjectionMPCDIPolicy, ESPMode::ThreadSafe>(ProjectionPolicyId, InConfigurationProjectionPolicy);
};
