// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Render/Projection/IDisplayClusterProjectionPolicyFactory.h"
#include "Policy/DisplayClusterProjectionPolicyFactoryBase.h"

/**
 * Factory for the 'mpcdi' and 'mesh' projection policies.
 * 'mpcdi' loads warp and blend data from an MPCDI calibration file;
 * 'mesh' drives warp from a UE mesh component instead.
 */
class FDisplayClusterProjectionMPCDIPolicyFactory
	: public IDisplayClusterProjectionPolicyFactory
	, protected FDisplayClusterProjectionPolicyFactoryBase
{
public:
	//~ Begin IDisplayClusterProjectionPolicyFactory
	virtual TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe> Create(const FString& ProjectionPolicyId, const FDisplayClusterConfigurationProjection* InConfigurationProjectionPolicy) override;
	//~ End IDisplayClusterProjectionPolicyFactory
};
