// Copyright Epic Games, Inc. All Rights Reserved.

#include "Policy/MPCDI/DisplayClusterProjectionMPCDIPolicy_Config.h"
#include "Policy/DisplayClusterProjectionPolicyBase.h"

#include "Render/Viewport/IDisplayClusterViewport.h"
#include "Containers/DisplayClusterWarpContainers.h"
#include "Components/DisplayClusterScreenComponent.h"

#include "DisplayClusterProjectionLog.h"
#include "DisplayClusterProjectionStrings.h"

#include "Misc/DisplayClusterHelpers.h"
#include "Misc/DisplayClusterProjectionHelpers.h"
#include "Misc/FileHelper.h"

#include "DisplayClusterRootActor.h"

//----------------------------------------------------------------------
// FDisplayClusterProjectionMPCDIPolicy_ConfigParser
//----------------------------------------------------------------------
FDisplayClusterProjectionMPCDIPolicy_ConfigParser::FDisplayClusterProjectionMPCDIPolicy_ConfigParser(IDisplayClusterViewport* InViewport, const TMap<FString, FString>& InConfigParameters)
	: ConfigParameters(InConfigParameters)
	, Viewport(InViewport ? InViewport->ToSharedPtr() : nullptr)
{
	bValid = Viewport.IsValid() && ReadConfig();
}

bool FDisplayClusterProjectionMPCDIPolicy_ConfigParser::ReadConfig()
{
	FString MPCDITypeKey;
	if (DisplayClusterHelpers::map::template ExtractValue(ConfigParameters, DisplayClusterProjectionStrings::cfg::mpcdi::MPCDITypeKey, MPCDITypeKey))
	{
		UE_LOGF(LogDisplayClusterProjectionMPCDI, Verbose, "Found Argument '%ls'='%ls'", DisplayClusterProjectionStrings::cfg::mpcdi::MPCDITypeKey, *MPCDITypeKey);
	}

	if (MPCDITypeKey.IsEmpty())
	{
		if (!FDisplayClusterProjectionPolicyBase::IsEditorOperationMode(Viewport.Get()))
		{
			UE_LOGF(LogDisplayClusterProjectionMPCDI, Error, "Undefined mpcdi type key '%ls'='%ls'", DisplayClusterProjectionStrings::cfg::mpcdi::MPCDITypeKey, *MPCDITypeKey);
		}

		return false;
	}

	if (!ImplGetBaseConfig())
	{
		return false;
	}

	if (MPCDITypeKey.Compare(DisplayClusterProjectionStrings::cfg::mpcdi::TypeMPCDI) == 0)
	{
		return ImplGetMPCDIConfig();
	}
	else if (MPCDITypeKey.Compare(DisplayClusterProjectionStrings::cfg::mpcdi::TypePFM) == 0)
	{
		return ImplGetPFMConfig();
	}

	UE_LOGF(LogDisplayClusterProjectionMPCDI, Error, "Unknown mpcdi type key '%ls'='%ls'", DisplayClusterProjectionStrings::cfg::mpcdi::MPCDITypeKey, *MPCDITypeKey);

	return false;
}

bool FDisplayClusterProjectionMPCDIPolicy_ConfigParser::ImplGetMPCDIConfig()
{
	// Filename
	FString LocalMPCDIFileName;
	if (DisplayClusterHelpers::map::template ExtractValue(ConfigParameters, DisplayClusterProjectionStrings::cfg::mpcdi::File, LocalMPCDIFileName))
	{
		UE_LOGF(LogDisplayClusterProjectionMPCDI, Verbose, "Found mpcdi file name for %ls:%ls - %ls", *BufferId, *RegionId, *LocalMPCDIFileName);
		MPCDIFileName = DisplayClusterHelpers::filesystem::GetFullPathForConfigResource(LocalMPCDIFileName);
	}

	if (MPCDIFileName.IsEmpty())
	{
		return false;
	}

	// Buffer
	if (!DisplayClusterHelpers::map::template ExtractValue(ConfigParameters, DisplayClusterProjectionStrings::cfg::mpcdi::Buffer, BufferId))
	{
		if (!FDisplayClusterProjectionPolicyBase::IsEditorOperationMode(Viewport.Get()))
		{
			UE_LOGF(LogDisplayClusterProjectionMPCDI, Error, "Argument '%ls' not found in the config file", DisplayClusterProjectionStrings::cfg::mpcdi::Buffer);
		}

		return false;
	}

	if (BufferId.IsEmpty())
	{
		return false;
	}

	// Region
	if (!DisplayClusterHelpers::map::template ExtractValue(ConfigParameters, DisplayClusterProjectionStrings::cfg::mpcdi::Region, RegionId))
	{
		if (!FDisplayClusterProjectionPolicyBase::IsEditorOperationMode(Viewport.Get()))
		{
			UE_LOGF(LogDisplayClusterProjectionMPCDI, Error, "Argument '%ls' not found in the config file", DisplayClusterProjectionStrings::cfg::mpcdi::Region);
		}

		return false;
	}

	if (RegionId.IsEmpty())
	{
		return false;
	}

	return true;
}

bool FDisplayClusterProjectionMPCDIPolicy_ConfigParser::ImplGetPFMConfig()
{
	// PFM file
	FString LocalPFMFile;
	if (DisplayClusterHelpers::map::template ExtractValue(ConfigParameters, DisplayClusterProjectionStrings::cfg::mpcdi::FilePFM, LocalPFMFile))
	{
		UE_LOGF(LogDisplayClusterProjectionMPCDI, Verbose, "Found Argument '%ls'='%ls'", DisplayClusterProjectionStrings::cfg::mpcdi::FilePFM, *LocalPFMFile);
		PFMFile = DisplayClusterHelpers::filesystem::GetFullPathForConfigResource(LocalPFMFile);
	}

	if (PFMFile.IsEmpty())
	{
		return false;
	}

	// Default is UE scale, cm
	PFMFileScale = 1;
	if (DisplayClusterHelpers::map::template ExtractValueFromString(ConfigParameters, DisplayClusterProjectionStrings::cfg::mpcdi::WorldScale, PFMFileScale))
	{
		UE_LOGF(LogDisplayClusterProjectionMPCDI, Verbose, "Found WorldScale value - %.f", PFMFileScale);
	}

	bIsUnrealGameSpace = false;
	if (DisplayClusterHelpers::map::template ExtractValueFromString(ConfigParameters, DisplayClusterProjectionStrings::cfg::mpcdi::UseUnrealAxis, bIsUnrealGameSpace))
	{
		UE_LOGF(LogDisplayClusterProjectionMPCDI, Verbose, "Found bIsUnrealGameSpace value - %ls", bIsUnrealGameSpace ? TEXT("true") : TEXT("false"));
	}

	// AlphaFile file (optional)
	FString LocalAlphaFile;
	if (DisplayClusterHelpers::map::template ExtractValueFromString(ConfigParameters, DisplayClusterProjectionStrings::cfg::mpcdi::FileAlpha, LocalAlphaFile))
	{
		UE_LOGF(LogDisplayClusterProjectionMPCDI, Verbose, "Found external AlphaMap file - %ls", *LocalAlphaFile);
		AlphaFile = DisplayClusterHelpers::filesystem::GetFullPathForConfigResource(LocalAlphaFile);
	}

	AlphaGamma = 1;
	if (DisplayClusterHelpers::map::template ExtractValueFromString(ConfigParameters, DisplayClusterProjectionStrings::cfg::mpcdi::AlphaGamma, AlphaGamma))
	{
		UE_LOGF(LogDisplayClusterProjectionMPCDI, Verbose, "Found AlphaGamma value - %.f", AlphaGamma);
	}

	// BetaFile file (optional)
	FString LocalBetaFile;
	if (DisplayClusterHelpers::map::template ExtractValueFromString(ConfigParameters, DisplayClusterProjectionStrings::cfg::mpcdi::FileBeta, LocalBetaFile))
	{
		UE_LOGF(LogDisplayClusterProjectionMPCDI, Verbose, "Found external BetaMap file - %ls", *LocalBetaFile);
		BetaFile = DisplayClusterHelpers::filesystem::GetFullPathForConfigResource(LocalBetaFile);
	}

	return true;
}

bool FDisplayClusterProjectionMPCDIPolicy_ConfigParser::ImplGetBaseConfig()
{
	// Origin node (optional)
	if (DisplayClusterHelpers::map::template ExtractValue(ConfigParameters, DisplayClusterProjectionStrings::cfg::mpcdi::Origin, OriginType))
	{
		UE_LOGF(LogDisplayClusterProjectionMPCDI, Verbose, "Found origin node - %ls", *OriginType);
	}
	else
	{
		if (!FDisplayClusterProjectionPolicyBase::IsEditorOperationMode(Viewport.Get()))
		{
			UE_LOGF(LogDisplayClusterProjectionMPCDI, Log, "No origin node found. VR root will be used as default.");
		}
	}

	bEnablePreview = false;
	if (DisplayClusterHelpers::map::template ExtractValueFromString(ConfigParameters, DisplayClusterProjectionStrings::cfg::mpcdi::EnablePreview, bEnablePreview))
	{
		UE_LOGF(LogDisplayClusterProjectionMPCDI, Verbose, "Found EnablePreview value - %ls", bEnablePreview ? TEXT("true") : TEXT("false"));
	}

	// Screen component
	FString ScreenComponentName;
	if (DisplayClusterHelpers::map::template ExtractValue(ConfigParameters, DisplayClusterProjectionStrings::cfg::mpcdi::Component, ScreenComponentName))
	{
		if (!ScreenComponentName.IsEmpty())
		{
			if (ADisplayClusterRootActor* SceneRootActor = Viewport->GetConfiguration().GetRootActor(EDisplayClusterRootActorType::Scene))
			{
				if (UDisplayClusterScreenComponent* ScreenComp = SceneRootActor->GetComponentByName<UDisplayClusterScreenComponent>(ScreenComponentName))
				{
					ScreenComponent = ScreenComp;
				}
			}

			if (ADisplayClusterRootActor* PreviewRootActor = Viewport->GetConfiguration().GetRootActor(EDisplayClusterRootActorType::Preview))
			{
				if (UDisplayClusterScreenComponent* PreviewScreenComp = PreviewRootActor->GetComponentByName<UDisplayClusterScreenComponent>(ScreenComponentName))
				{
					PreviewScreenComponent = PreviewScreenComp;
				}
			}
		}
	}

	// MPCDIType (optional)
	FString MPCDITypeStr;
	if (!DisplayClusterHelpers::map::template ExtractValue(ConfigParameters, DisplayClusterProjectionStrings::cfg::mpcdi::MPCDIType, MPCDITypeStr))
	{
		MPCDIAttributes.ProfileType = EDisplayClusterWarpProfileType::warp_A3D;
	}
	else
	{
		MPCDIAttributes.ProfileType = UE::DisplayClusterProjectionHelpers::MPCDI::ProfileTypeFromString(MPCDITypeStr);

		if (MPCDIAttributes.ProfileType == EDisplayClusterWarpProfileType::Invalid)
		{
			UE_LOGF(LogDisplayClusterProjectionMPCDI, Error, "Argument '%ls' has unknown value '%ls'", DisplayClusterProjectionStrings::cfg::mpcdi::MPCDIType, *MPCDITypeStr);
			return false;
		}
	}

	switch (MPCDIAttributes.ProfileType)
	{
	case EDisplayClusterWarpProfileType::warp_2D:
	// Reads additional properties of the mpcdi 2d profile.
	{
		FIntPoint BufferRes;
		if (DisplayClusterHelpers::map::template ExtractValueFromString(ConfigParameters, DisplayClusterProjectionStrings::cfg::mpcdi::Attributes::Buffer::Resolution, BufferRes))
		{
			UE_LOGF(LogDisplayClusterProjectionMPCDI, Verbose, "Found buffer resolution %ls", *DisplayClusterTypesConverter::ToString(BufferRes));
			MPCDIAttributes.Buffer.Resolution = BufferRes;
		}

		FVector2D RegionPos(0,0), RegionSize(1,1);
		if (DisplayClusterHelpers::map::template ExtractValueFromString(ConfigParameters, DisplayClusterProjectionStrings::cfg::mpcdi::Attributes::Region::Pos, RegionPos)
		&& DisplayClusterHelpers::map::template ExtractValueFromString(ConfigParameters, DisplayClusterProjectionStrings::cfg::mpcdi::Attributes::Region::Size, RegionSize))
		{
			UE_LOGF(LogDisplayClusterProjectionMPCDI, Verbose, "Found region pos (%ls) and size (%ls)", *DisplayClusterTypesConverter::ToString(RegionPos), *DisplayClusterTypesConverter::ToString(RegionSize));

			MPCDIAttributes.Region.Pos = RegionPos;
			MPCDIAttributes.Region.Size= RegionSize;
		}
	}
	break;

	default:
		// Todo: Read parameters with names from the DisplayClusterProjectionStrings::cfg::mpcdi::Attributes namespace
		// into a local variable named MPCDIAttributes;
		break;
	}

	return true;
}
