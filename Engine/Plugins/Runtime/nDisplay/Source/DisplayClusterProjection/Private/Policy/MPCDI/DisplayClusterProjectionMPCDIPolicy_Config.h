// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/DisplayClusterWarpEnums.h"
#include "Containers/DisplayClusterWarpContainers.h"

class IDisplayClusterViewport;
class UStaticMeshComponent;
class UDisplayClusterScreenComponent;

/**
 * MPCDI projection policy configuration parser
 */
struct FDisplayClusterProjectionMPCDIPolicy_ConfigParser
{
	// Name of the MPCDI file that is used as the data source
	FString MPCDIFileName; // Single mpcdi file name

	// Name of the buffer inside the MPCDI file that will be used as the data source
	FString BufferId;

	// Name of the region inside the MPCDI file that will be used as the data source
	FString RegionId;

	// The origin point name
	FString OriginType;

	// Todo: fill this data from InConfigParameters
	FDisplayClusterWarpMPCDIAttributes MPCDIAttributes;

	// Screen component with source geometry
	UDisplayClusterScreenComponent* ScreenComponent = nullptr;

	// Screen component with source geometry used as a preview mesh
	UDisplayClusterScreenComponent* PreviewScreenComponent = nullptr;

	// Name of the PFM file that is used as the data source
	FString PFMFile;

	// Scale unit of the PFM file geometry
	float PFMFileScale = 1;

	// Are the axes of the PFM geometry the same as in UE
	bool bIsUnrealGameSpace = false;

	// Filename of the image to be used as an AlphaMap
	FString AlphaFile;
	// Gamma used for the AlphaMap image
	float AlphaGamma = 1;

	// Filename of the image to be used as an BetaMap
	FString BetaFile;

	// The preview geometry must be imported from the PFM\MPCDI file and displayed.
	bool bEnablePreview = false;

public:
	/** Read MPCDI policy configuration from projection policy parameters
	* 
	* @param InViewport         - the projection policy owner viewport
	* @param InConfigParameters - the projection policy parameters
	*/
	FDisplayClusterProjectionMPCDIPolicy_ConfigParser(IDisplayClusterViewport* InViewport, const TMap<FString, FString>& InConfigParameters);

	/** When this configuration is valid, it returns true. */
	inline bool IsValid() const
	{
		return bValid;
	}

private:
	/** Implementation of reading the configuration from the parameters. */
	bool ReadConfig();

	/** Implementation of reading basic parameters. */
	bool ImplGetBaseConfig();

	/** Implementation of parameter reading for the case when we initialize from an MPCDI file. */
	bool ImplGetMPCDIConfig();

	/** Implementation of parameter reading for the case when we initialize from an PFM file. */
	bool ImplGetPFMConfig();

private:
	// The projection policy parameters used to read this configuration
	const TMap<FString, FString>& ConfigParameters;

	// The projection policy owner viewport
	TSharedPtr<IDisplayClusterViewport, ESPMode::ThreadSafe> Viewport;

	// Is this configuration valid.
	bool bValid = false;
};
