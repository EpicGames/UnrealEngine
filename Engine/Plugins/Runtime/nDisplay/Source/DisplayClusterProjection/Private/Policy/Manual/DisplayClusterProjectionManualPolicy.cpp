// Copyright Epic Games, Inc. All Rights Reserved.

#include "Policy/Manual/DisplayClusterProjectionManualPolicy.h"

#include "DisplayClusterProjectionLog.h"
#include "DisplayClusterProjectionStrings.h"

#include "Misc/DisplayClusterHelpers.h"

#include "IDisplayCluster.h"
#include "Game/IDisplayClusterGameManager.h"

#include "Components/DisplayClusterScreenComponent.h"

#include "Render/Viewport/IDisplayClusterViewport.h"


FDisplayClusterProjectionManualPolicy::FDisplayClusterProjectionManualPolicy(const FString& ProjectionPolicyId, const FDisplayClusterConfigurationProjection* InConfigurationProjectionPolicy)
	: FDisplayClusterProjectionPolicyBase(ProjectionPolicyId, InConfigurationProjectionPolicy)
{
	ProjectionMatrix[0] = FMatrix(FPlane(1.0f, 0.0f, 0.0f, 0.0f), FPlane(0.0f, 1.0f, 0.0f, 0.0f), FPlane(0.0f, 0.0f, 0.0f, 1.0f), FPlane(0.0f, 0.0f, 1.0f, 0.0f));
	ProjectionMatrix[1] = ProjectionMatrix[0];
}


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProjectionPolicy
//////////////////////////////////////////////////////////////////////////////////////////////
const FString& FDisplayClusterProjectionManualPolicy::GetType() const
{
	static const FString Type(DisplayClusterProjectionStrings::projection::Manual);
	return Type;
}

bool FDisplayClusterProjectionManualPolicy::HandleStartScene(IDisplayClusterViewport* InViewport)
{
	check(IsInGameThread());
	UE_LOGF(LogDisplayClusterProjectionManual, Verbose, "Initializing internals for the viewport '%ls'", *InViewport->GetId());

	FString DataTypeInString;
	if(DisplayClusterHelpers::map::template ExtractValueFromString(GetParameters(), FString(DisplayClusterProjectionStrings::cfg::manual::Type), DataTypeInString))
	{
		DataType = DataTypeFromString(InViewport, DataTypeInString);
	}
	else
	{
		if (!IsEditorOperationMode(InViewport))
		{
			UE_LOGF(LogDisplayClusterProjectionManual, Log, "Undefined manual type '%ls'", *DataTypeInString);
		}

		return false;
	}	

	// Get view rotation
	if (!DisplayClusterHelpers::map::template ExtractValueFromString(GetParameters(), FString(DisplayClusterProjectionStrings::cfg::manual::Rotation), ViewRotation))
	{
		if (!IsEditorOperationMode(InViewport))
		{
			UE_LOGF(LogDisplayClusterProjectionManual, Log, "No rotation specified for projection policy of viewport '%ls'", *InViewport->GetId());
		}
	}

	FString RenderType;
	DisplayClusterHelpers::map::template ExtractValueFromString(GetParameters(), FString(DisplayClusterProjectionStrings::cfg::manual::Rendering), RenderType);


	if (DataType == EManualDataType::Matrix)
	{
		if (RenderType == DisplayClusterProjectionStrings::cfg::manual::RenderingType::Mono)
		{
			DisplayClusterHelpers::map::template ExtractValueFromString(GetParameters(), FString(DisplayClusterProjectionStrings::cfg::manual::Matrix), ProjectionMatrix[0]);
		}
		else // stereo cases
		{
			DisplayClusterHelpers::map::template ExtractValueFromString(GetParameters(), FString(DisplayClusterProjectionStrings::cfg::manual::MatrixLeft), ProjectionMatrix[0]);
			DisplayClusterHelpers::map::template ExtractValueFromString(GetParameters(), FString(DisplayClusterProjectionStrings::cfg::manual::MatrixRight), ProjectionMatrix[1]);
		}
	}
	else if (DataType == EManualDataType::FrustumAngles)
	{
		FString AnglesLeft;
		FString AnglesRight;

		if (RenderType == DisplayClusterProjectionStrings::cfg::manual::RenderingType::Mono)
		{
			DisplayClusterHelpers::map::template ExtractValue(GetParameters(), FString(DisplayClusterProjectionStrings::cfg::manual::Frustum), AnglesLeft);

			if (!ExtractAngles(AnglesLeft, FrustumAngles[0]))
			{
				if (!IsEditorOperationMode(InViewport))
				{
					UE_LOGF(LogDisplayClusterProjectionManual, Log, "Couldn't extract frustum angles from value '%ls'", *AnglesLeft);
				}

				return false;
			}
		}
		else // stereo cases
		{
			DisplayClusterHelpers::map::template ExtractValue(GetParameters(), FString(DisplayClusterProjectionStrings::cfg::manual::FrustumLeft), AnglesLeft);
			DisplayClusterHelpers::map::template ExtractValue(GetParameters(), FString(DisplayClusterProjectionStrings::cfg::manual::FrustumRight), AnglesRight);

			if (!ExtractAngles(AnglesLeft, FrustumAngles[0]))
			{
				if (!IsEditorOperationMode(InViewport))
				{
					UE_LOGF(LogDisplayClusterProjectionManual, Log, "Couldn't extract frustum angles from value '%ls'", *AnglesLeft);
				}

				return false;
			}

			if (!ExtractAngles(AnglesRight, FrustumAngles[1]))
			{
				if (!IsEditorOperationMode(InViewport))
				{
					UE_LOGF(LogDisplayClusterProjectionManual, Log, "Couldn't extract frustum angles from value '%ls'", *AnglesRight);
				}

				return false;
			}
		}
	}

	return true;
}

void FDisplayClusterProjectionManualPolicy::SetupProjectionViewPoint(IDisplayClusterViewport* InViewport, const float InDeltaTime, FMinimalViewInfo& InOutViewInfo, float* OutCustomNearClippingPlane)
{
	// Add local rotation specified in config
	InOutViewInfo.Rotation += ViewRotation;
}

bool FDisplayClusterProjectionManualPolicy::CalculateView(IDisplayClusterViewport* InViewport, const uint32 InContextNum, FVector& InOutViewLocation, FRotator& InOutViewRotation, const FVector& InViewOffset, const float InWorldToMeters, const float InNCP, const float InFCP)
{
	check(IsInGameThread());
	check(InContextNum < 2);

	// View changes are already defined from SetupProjectionViewPoint()

	// Store culling data
	NCP = InNCP;
	FCP = InFCP;

	return true;
}

bool FDisplayClusterProjectionManualPolicy::GetProjectionMatrix(IDisplayClusterViewport* InViewport, const uint32 InContextNum, FMatrix& OutPrjMatrix)
{
	check(IsInGameThread());
	check(InContextNum < 2);

	bool bResult = false;

	switch (DataType)
	{
	case EManualDataType::Matrix:
		OutPrjMatrix = ProjectionMatrix[InContextNum];
		bResult = true;
		break;

	case EManualDataType::FrustumAngles:
		InViewport->CalculateProjectionMatrix(InContextNum, FrustumAngles[InContextNum].Left, FrustumAngles[InContextNum].Right, FrustumAngles[InContextNum].Top, FrustumAngles[InContextNum].Bottom, NCP, FCP, true);
		OutPrjMatrix = InViewport->GetContexts()[InContextNum].ProjectionMatrix;

		bResult = true;
		break;

	default:
		break;
	}

	return bResult;
}

bool FDisplayClusterProjectionManualPolicy::ExtractAngles(const FString& InAngles, FFrustumAngles& OutAngles)
{
	float Left;
	if (!DisplayClusterHelpers::str::ExtractValue(InAngles, FString(DisplayClusterProjectionStrings::cfg::manual::AngleL), Left))
	{
		return false;
	}

	float Right;
	if (!DisplayClusterHelpers::str::ExtractValue(InAngles, FString(DisplayClusterProjectionStrings::cfg::manual::AngleR), Right))
	{
		return false;
	}

	float Top;
	if (!DisplayClusterHelpers::str::ExtractValue(InAngles, FString(DisplayClusterProjectionStrings::cfg::manual::AngleT), Top))
	{
		return false;
	}

	float Bottom;
	if (!DisplayClusterHelpers::str::ExtractValue(InAngles, FString(DisplayClusterProjectionStrings::cfg::manual::AngleB), Bottom))
	{
		return false;
	}

	OutAngles.Left   = Left;
	OutAngles.Right  = Right;
	OutAngles.Top    = Top;
	OutAngles.Bottom = Bottom;

	return true;
}

FDisplayClusterProjectionManualPolicy::EManualDataType FDisplayClusterProjectionManualPolicy::DataTypeFromString(IDisplayClusterViewport* InViewport, const FString& DataTypeInString) const
{
	if (DataTypeInString == DisplayClusterProjectionStrings::cfg::manual::FrustumType::Matrix)
	{
		return EManualDataType::Matrix;
	}
	else if (DataTypeInString == DisplayClusterProjectionStrings::cfg::manual::FrustumType::Angles)
	{
		return EManualDataType::FrustumAngles;
	}
	else
	{
		if (!IsEditorOperationMode(InViewport))
		{
			UE_LOGF(LogDisplayClusterProjectionManual, Log, "Undefined manual type '%ls'. Matrix by default", *DataTypeInString);
		}
	}

	return EManualDataType::Matrix;
}
