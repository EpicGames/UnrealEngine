// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Policy/DisplayClusterProjectionPolicyBase.h"


/**
 * Manual projection policy
 */
class FDisplayClusterProjectionManualPolicy
	: public FDisplayClusterProjectionPolicyBase
{
public:
	FDisplayClusterProjectionManualPolicy(const FString& ProjectionPolicyId, const FDisplayClusterConfigurationProjection* InConfigurationProjectionPolicy);

public:
	enum class EManualDataType
	{
		Matrix,
		FrustumAngles
	};

	EManualDataType GetDataType() const
	{
		return DataType;
	}

public:
	//////////////////////////////////////////////////////////////////////////////////////////////
	// IDisplayClusterProjectionPolicy
	//////////////////////////////////////////////////////////////////////////////////////////////
	virtual const FString& GetType() const override;
	virtual bool HandleStartScene(IDisplayClusterViewport* InViewport) override;
	virtual bool CalculateView(IDisplayClusterViewport* InViewport, const uint32 InContextNum, FVector& InOutViewLocation, FRotator& InOutViewRotation, const FVector& InViewOffset, const float InWorldToMeters, const float InNCP, const float InFCP) override;
	virtual bool GetProjectionMatrix(IDisplayClusterViewport* InViewport, const uint32 InContextNum, FMatrix& OutPrjMatrix) override;
	virtual void SetupProjectionViewPoint(IDisplayClusterViewport* InViewport, const float InDeltaTime, FMinimalViewInfo& InOutViewInfo, float* OutCustomNearClippingPlane = nullptr) override;

protected:
	struct FFrustumAngles
	{
		float Left   = -30.f;
		float Right  = 30.f;
		float Top    = 30.f;
		float Bottom = -30.f;
	};

	virtual bool ExtractAngles(const FString& InAngles, FFrustumAngles& OutAngles);

private:
	EManualDataType DataTypeFromString(IDisplayClusterViewport* InViewport, const FString& DataTypeInString) const;

private:
	// Current data type (matrix, frustum angle, ...)
	EManualDataType DataType = EManualDataType::Matrix;
	// View rotation
	FRotator ViewRotation = FRotator::ZeroRotator;
	// Projection matrix
	FMatrix  ProjectionMatrix[2] = { FMatrix::Identity };
	// Frustum angles
	FFrustumAngles FrustumAngles[2];
	// Near/far clip planes
	float NCP;
	float FCP;
};
