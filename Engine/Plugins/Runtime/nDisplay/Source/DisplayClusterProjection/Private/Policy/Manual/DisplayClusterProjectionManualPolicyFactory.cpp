// Copyright Epic Games, Inc. All Rights Reserved.

#include "Policy/Manual/DisplayClusterProjectionManualPolicyFactory.h"
#include "Policy/Manual/DisplayClusterProjectionManualPolicy.h"

#include "DisplayClusterProjectionLog.h"

#include "DisplayClusterConfigurationTypes.h"


//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProjectionPolicyFactory
//////////////////////////////////////////////////////////////////////////////////////////////
TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe> FDisplayClusterProjectionManualPolicyFactory::Create(const FString& ProjectionPolicyId, const FDisplayClusterConfigurationProjection* InConfigurationProjectionPolicy)
{
	if(!InConfigurationProjectionPolicy)
	{
		if (CanShowMsgOnce(ProjectionPolicyId))
		{
			UE_LOGF(LogDisplayClusterProjectionManual, Error, "Manual policy '%ls': null configuration", *ProjectionPolicyId);
		}
		return nullptr;
	}

	UE_LOGF(LogDisplayClusterProjectionManual, Verbose, "Instantiating projection policy <%ls> id='%ls'", *InConfigurationProjectionPolicy->Type, *ProjectionPolicyId);
	return MakeShared<FDisplayClusterProjectionManualPolicy, ESPMode::ThreadSafe>(ProjectionPolicyId, InConfigurationProjectionPolicy);
}
