// Copyright Epic Games, Inc. All Rights Reserved.

#include "Policy/Mesh/DisplayClusterProjectionMeshPolicy.h"
#include "DisplayClusterProjectionLog.h"
#include "DisplayClusterProjectionStrings.h"
#include "DisplayClusterRootActor.h"

#include "Misc/DisplayClusterHelpers.h"

#include "Render/Viewport/IDisplayClusterViewportManager.h"
#include "Render/Viewport/IDisplayClusterViewport.h"

#include "IDisplayClusterShaders.h"
#include "IDisplayClusterWarp.h"

#include "Components/StaticMeshComponent.h"
#include "ProceduralMeshComponent.h"

/**
 * WarpBlend configuration
 */
struct FDisplayClusterProjectionMeshPolicyConfiguration
{
	// StaticMesh component with source geometry
	UStaticMeshComponent* StaticMeshComponent = nullptr;
	UStaticMeshComponent* PreviewStaticMeshComponent = nullptr;

	// StaticMesh geometry LOD
	int32 StaticMeshComponentLODIndex = 0;

	// ProceduralMesh component with source geometry
	UProceduralMeshComponent* ProceduralMeshComponent = nullptr;
	UProceduralMeshComponent* PreviewProceduralMeshComponent = nullptr;

	// ProceduralMesh section index
	int32 ProceduralMeshComponentSectionIndex = 0;

	// Customize source geometry UV channels
	int32 BaseUVIndex = INDEX_NONE;
	int32 ChromakeyUVIndex = INDEX_NONE;
};

//------------------------------------------------------------------------------------------
// FDisplayClusterProjectionMeshPolicy
//------------------------------------------------------------------------------------------
FDisplayClusterProjectionMeshPolicy::FDisplayClusterProjectionMeshPolicy(const FString& ProjectionPolicyId, const FDisplayClusterConfigurationProjection* InConfigurationProjectionPolicy)
	: FDisplayClusterProjectionMPCDIPolicy(ProjectionPolicyId, InConfigurationProjectionPolicy)
{
}

bool FDisplayClusterProjectionMeshPolicy::CreateWarpMeshInterface(IDisplayClusterViewport* InViewport)
{
	check(IsInGameThread());

	if (!WarpBlendInterface.IsValid())
	{
		// Read configuration from config
		FDisplayClusterProjectionMeshPolicyConfiguration WarpCfg;
		if (!GetWarpMeshConfiguration(InViewport, WarpCfg))
		{
			return false;
		}

		// Mesh poolicy always shows a preview on an existing mesh
		bIsPreviewMeshEnabled = true;

		// The mesh always uses DCRootActor as its origin because all geometry is in UE space.
		InitializeOriginComponent(InViewport, TEXT(""));

		static IDisplayClusterWarp& DisplayClusterWarpAPI = IDisplayClusterWarp::Get();

		if (WarpCfg.StaticMeshComponent != nullptr)
		{
			FDisplayClusterWarpInitializer_StaticMesh CreateParameters;

			CreateParameters.OriginComponent = GetOriginComponent();

			CreateParameters.PreviewMeshComponent = WarpCfg.PreviewStaticMeshComponent;

			CreateParameters.WarpMeshComponent = WarpCfg.StaticMeshComponent;
			CreateParameters.StaticMeshComponentLODIndex = WarpCfg.StaticMeshComponentLODIndex;

			CreateParameters.BaseUVIndex      = WarpCfg.BaseUVIndex;
			CreateParameters.ChromakeyUVIndex = WarpCfg.ChromakeyUVIndex;

			WarpBlendInterface = DisplayClusterWarpAPI.Create(CreateParameters);
		}
		else
		{
			FDisplayClusterWarpInitializer_ProceduralMesh CreateParameters;

			CreateParameters.OriginComponent = GetOriginComponent();

			CreateParameters.PreviewMeshComponent = WarpCfg.PreviewProceduralMeshComponent;

			CreateParameters.WarpMeshComponent = WarpCfg.ProceduralMeshComponent;
			CreateParameters.ProceduralMeshComponentSectionIndex = WarpCfg.ProceduralMeshComponentSectionIndex;

			CreateParameters.BaseUVIndex      = WarpCfg.BaseUVIndex;
			CreateParameters.ChromakeyUVIndex = WarpCfg.ChromakeyUVIndex;

			WarpBlendInterface = DisplayClusterWarpAPI.Create(CreateParameters);
		}

		if (!WarpBlendInterface.IsValid())
		{
			if (!IsEditorOperationMode(InViewport))
			{
				UE_LOGF(LogDisplayClusterProjectionMesh, Warning, "Couldn't create mesh warpblend interface");
			}

			return false;
		}
	}

	return true;
}

const FString& FDisplayClusterProjectionMeshPolicy::GetType() const
{
	static const FString Type(DisplayClusterProjectionStrings::projection::Mesh);
	return Type;
}

bool FDisplayClusterProjectionMeshPolicy::HandleStartScene(IDisplayClusterViewport* InViewport)
{
	check(IsInGameThread());

	// The game side of the nDisplay has been initialized by the nDisplay Game Manager already
	// so we can extend it by our projection related functionality/components/etc.

	WarpBlendContexts.Empty();

	if (!CreateWarpMeshInterface(InViewport))
	{
		if (!IsEditorOperationMode(InViewport))
		{
			UE_LOGF(LogDisplayClusterProjectionMesh, Error, "Couldn't create warp interface for viewport '%ls'", *InViewport->GetId());
		}

		return false;
	}

	// Finally, initialize internal views data container
	WarpBlendContexts.AddDefaulted(2);

	if (WarpBlendInterface.IsValid())
	{
		WarpBlendInterface->HandleStartScene(InViewport);
	}

	return true;
}

bool FDisplayClusterProjectionMeshPolicy::GetWarpMeshConfiguration(IDisplayClusterViewport* InViewport, FDisplayClusterProjectionMeshPolicyConfiguration& OutWarpCfg)
{
	check(InViewport);

	// Get our VR root
	ADisplayClusterRootActor* PreviewRootActor = InViewport->GetConfiguration().GetRootActor(EDisplayClusterRootActorType::Preview);
	ADisplayClusterRootActor* SceneRootActor = InViewport->GetConfiguration().GetRootActor(EDisplayClusterRootActorType::Scene);
	if (!SceneRootActor)
	{
		UE_LOGF(LogDisplayClusterProjectionMesh, Error, "Couldn't get a VR root object");
		return false;
	}

	FString ComponentId;
	// Get assigned mesh ID
	if (!DisplayClusterHelpers::map::template ExtractValue(GetParameters(), DisplayClusterProjectionStrings::cfg::mesh::Component, ComponentId))
	{

#if WITH_EDITOR
		if (ComponentId.IsEmpty())
		{
			return false;
		}
#endif

		if (!IsEditorOperationMode(InViewport))
		{
			UE_LOGF(LogDisplayClusterProjectionMesh, Error, "No component ID '%ls' specified for projection policy '%ls'", *ComponentId, *GetId());
		}

		return false;
	}

	int32 CfgBaseUVIndex = INDEX_NONE;
	if (DisplayClusterHelpers::map::template ExtractValueFromString(GetParameters(), DisplayClusterProjectionStrings::cfg::mesh::BaseUVIndex, CfgBaseUVIndex))
	{
		if (CfgBaseUVIndex >= 0) //-V547
		{
			UE_LOGF(LogDisplayClusterProjectionMesh, Verbose, "Found BaseUVIndex value - '%d'", CfgBaseUVIndex);
		}
		else if(CfgBaseUVIndex != INDEX_NONE) //-V547
		{
			UE_LOGF(LogDisplayClusterProjectionMesh, Error, "Invalid BaseUVIndex value - '%d'", CfgBaseUVIndex);
			CfgBaseUVIndex = INDEX_NONE;
		}
	}
	OutWarpCfg.BaseUVIndex = CfgBaseUVIndex;

	int32 CfgChromakeyUVIndex = INDEX_NONE;
	if (DisplayClusterHelpers::map::template ExtractValueFromString(GetParameters(), DisplayClusterProjectionStrings::cfg::mesh::ChromakeyUVIndex, CfgChromakeyUVIndex))
	{
		if (CfgChromakeyUVIndex >= 0) //-V547
		{
			UE_LOGF(LogDisplayClusterProjectionMesh, Verbose, "Found ChromakeyUVIndex value - '%d'", CfgChromakeyUVIndex);
		}
		else if(CfgChromakeyUVIndex != INDEX_NONE) //-V547
		{
			UE_LOGF(LogDisplayClusterProjectionMesh, Error, "Invalid ChromakeyUVIndex value - '%d'", CfgChromakeyUVIndex);
			CfgChromakeyUVIndex = INDEX_NONE;
		}
	}
	OutWarpCfg.ChromakeyUVIndex = CfgChromakeyUVIndex;

	// Get the StaticMeshComponent
	OutWarpCfg.StaticMeshComponent        = SceneRootActor->GetComponentByName<UStaticMeshComponent>(ComponentId);
	OutWarpCfg.PreviewStaticMeshComponent = PreviewRootActor ? PreviewRootActor->GetComponentByName<UStaticMeshComponent>(ComponentId) : nullptr;

	if (OutWarpCfg.StaticMeshComponent != nullptr)
	{
		int CfgLODIndex = 0;
		if (DisplayClusterHelpers::map::template ExtractValueFromString(GetParameters(), DisplayClusterProjectionStrings::cfg::mesh::LODIndex, CfgLODIndex))
		{
			if (CfgLODIndex >= 0) //-V547
			{
				UE_LOGF(LogDisplayClusterProjectionMesh, Verbose, "Found StaticMeshComponent LODIndex value - '%d'", CfgLODIndex);

				OutWarpCfg.StaticMeshComponentLODIndex = CfgLODIndex;
			}
			else
			{
				UE_LOGF(LogDisplayClusterProjectionMesh, Error, "Invalid StaticMeshComponent LODIndex value - '%d'", CfgLODIndex);
			}
		}
	}
	else
	{
		// Get the ProceduralMeshComponent
		OutWarpCfg.ProceduralMeshComponent        = SceneRootActor->GetComponentByName<UProceduralMeshComponent>(ComponentId);
		OutWarpCfg.PreviewProceduralMeshComponent = PreviewRootActor ? PreviewRootActor->GetComponentByName<UProceduralMeshComponent>(ComponentId) : nullptr;

		if (OutWarpCfg.ProceduralMeshComponent == nullptr)
		{
			if (!IsEditorOperationMode(InViewport))
			{
				UE_LOGF(LogDisplayClusterProjectionMesh, Warning, "Couldn't find mesh component '%ls' in RootActor", *ComponentId);
			}

			return false;
		}

		int32 CfgSectionIndex = 0;
		if (DisplayClusterHelpers::map::template ExtractValueFromString(GetParameters(), DisplayClusterProjectionStrings::cfg::mesh::SectionIndex, CfgSectionIndex))
		{
			if (CfgSectionIndex >= 0) //-V547
			{
				UE_LOGF(LogDisplayClusterProjectionMesh, Verbose, "Found ProceduralMeshComponent SectionIndex value - '%d'", CfgSectionIndex);

				OutWarpCfg.ProceduralMeshComponentSectionIndex = CfgSectionIndex;
			}
			else
			{
				UE_LOGF(LogDisplayClusterProjectionMesh, Error, "Invalid ProceduralMeshComponent SectionIndex value - '%d'", CfgSectionIndex);
			}
		}
	}

	return true;
}
