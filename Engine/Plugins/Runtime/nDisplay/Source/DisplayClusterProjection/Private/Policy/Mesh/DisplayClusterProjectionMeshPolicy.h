// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Policy/MPCDI/DisplayClusterProjectionMPCDIPolicy.h"

class UStaticMeshComponent;
class UProceduralMeshComponent;


/*
 * Mesh projection policy
 * Supported geometry sources - StaticMeshComponent, ProceduralMeshComponent
 */
class FDisplayClusterProjectionMeshPolicy
	: public FDisplayClusterProjectionMPCDIPolicy
{
public:
	FDisplayClusterProjectionMeshPolicy(const FString& ProjectionPolicyId, const FDisplayClusterConfigurationProjection* InConfigurationProjectionPolicy);

public:
	//////////////////////////////////////////////////////////////////////////////////////////////
	// IDisplayClusterProjectionPolicy
	//////////////////////////////////////////////////////////////////////////////////////////////
	virtual const FString& GetType() const override;
	virtual bool HandleStartScene(IDisplayClusterViewport* InViewport) override;

public:
	/** Parse the config data for a mesh id and try to retrieve it from the root actor. */
	bool CreateWarpMeshInterface(IDisplayClusterViewport* InViewport);

private:
	/** Read mesh policy configuration from the projection policy parameters
	* 
	* @param InViewport - projection policy owner viewport
	* @param OutWarpCfg - (out) warpblend configuration
	* 
	* @return - true when successful.
	*/
	bool GetWarpMeshConfiguration(IDisplayClusterViewport* InViewport, struct FDisplayClusterProjectionMeshPolicyConfiguration& OutWarpCfg);
};
