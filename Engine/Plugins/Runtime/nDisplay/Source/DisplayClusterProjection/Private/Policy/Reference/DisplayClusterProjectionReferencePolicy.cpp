// Copyright Epic Games, Inc. All Rights Reserved.

#include "Policy/Reference/DisplayClusterProjectionReferencePolicy.h"

#include "Misc/DisplayClusterHelpers.h"

#include "DisplayClusterProjectionLog.h"
#include "DisplayClusterProjectionStrings.h"

#include "Render/Viewport/IDisplayClusterViewportManager.h"
#include "Render/Viewport/IDisplayClusterViewportManagerProxy.h"
#include "Render/Viewport/IDisplayClusterViewport.h"
#include "Render/Viewport/IDisplayClusterViewportProxy.h"
#include "RenderingThread.h"
#include "RHI.h"
#include "RHICommandList.h"


FDisplayClusterProjectionReferencePolicy::FDisplayClusterProjectionReferencePolicy(
	const FString& ProjectionPolicyId,
	const FDisplayClusterConfigurationProjection* InConfigurationProjectionPolicy)
	: FDisplayClusterProjectionPolicyBase(ProjectionPolicyId, InConfigurationProjectionPolicy)
{
	if (DisplayClusterHelpers::map::template ExtractValue(GetParameters(),
		DisplayClusterProjectionStrings::cfg::reference::ViewportId,
		ReferencedViewportId))
	{
		UE_LOGF(LogDisplayClusterProjectionReference, Verbose, "Found Argument '%ls'='%ls'",
			DisplayClusterProjectionStrings::cfg::reference::ViewportId, *ReferencedViewportId);
	}
}

const FString& FDisplayClusterProjectionReferencePolicy::GetType() const
{
	static const FString Type(DisplayClusterProjectionStrings::projection::Reference);

	return Type;
}

void FDisplayClusterProjectionReferencePolicy::UpdateProxyData(IDisplayClusterViewport* InViewport)
{
	FString ReferencedViewportIdProxy;
	bool bSupportICVFX = false;
	if (IDisplayClusterViewport* SourceViewport = GetSourceViewport(InViewport))
	{
		bSupportICVFX = SourceViewport->GetProjectionPolicy().IsValid() && SourceViewport->GetProjectionPolicy()->ShouldSupportICVFX(SourceViewport);
		ReferencedViewportIdProxy = ReferencedViewportId;
	}

	// Send view data to rendering thread.
	ENQUEUE_RENDER_COMMAND(DisplayClusterProjectionReferencePolicy_UpdateProxyData)(
		[
			This = SharedThis(this),
			InReferencedViewportId = ReferencedViewportIdProxy,
			bSupportICVFX = bSupportICVFX
		](FRHICommandListImmediate& RHICmdList)
		{
			// Update rendering thread resoures
			This->ReferencedViewportId_RenderThread = InReferencedViewportId;
			This->bSupportICVFXProxy = bSupportICVFX;
		});
}

bool FDisplayClusterProjectionReferencePolicy::ShouldSupportICVFX(const IDisplayClusterViewport* InViewport) const
{
	if (IDisplayClusterViewport* SourceViewport = GetSourceViewport(InViewport))
	{
		// return source viewport settings
		return SourceViewport->GetProjectionPolicy().IsValid()
			&& SourceViewport->GetProjectionPolicy()->ShouldSupportICVFX(SourceViewport);
	}

	return false;
}

bool FDisplayClusterProjectionReferencePolicy::ShouldUseSourceTextureWithMips(const IDisplayClusterViewport* InViewport) const
{
	if (IDisplayClusterViewport* SourceViewport = GetSourceViewport(InViewport))
	{
		// Only ICVFX requires composition. Other projection policies do not. So we can reuse all textures and not do warpblend.
		if (SourceViewport->GetProjectionPolicy().IsValid() && SourceViewport->GetProjectionPolicy()->ShouldSupportICVFX(SourceViewport))
		{
			return SourceViewport->GetProjectionPolicy()->ShouldUseSourceTextureWithMips(SourceViewport);
		}
	}

	return false;
}

bool FDisplayClusterProjectionReferencePolicy::ShouldUseAdditionalTargetableResource(const IDisplayClusterViewport* InViewport) const
{
	if (IDisplayClusterViewport* SourceViewport = GetSourceViewport(InViewport))
	{
		// Only ICVFX requires composition. Other projection policies do not. So we can reuse all textures and not do warpblend.
		if (SourceViewport->GetProjectionPolicy().IsValid() && SourceViewport->GetProjectionPolicy()->ShouldSupportICVFX(SourceViewport))
		{
			return SourceViewport->GetProjectionPolicy()->ShouldUseAdditionalTargetableResource(SourceViewport);
		}
	}

	return false;
}

EDisplayClusterProjectionPolicyFlags FDisplayClusterProjectionReferencePolicy::GetProjectionPolicyFlags(const IDisplayClusterViewport* InViewport, const uint32 InContextNum) const
{
	if (IDisplayClusterViewport* SourceViewport = GetSourceViewport(InViewport))
	{
		// Only ICVFX requires composition.
		if (SourceViewport->GetProjectionPolicy().IsValid() && SourceViewport->GetProjectionPolicy()->ShouldSupportICVFX(SourceViewport))
		{
			return SourceViewport->GetProjectionPolicy()->GetProjectionPolicyFlags(SourceViewport, InContextNum);
		}
	}

	return EDisplayClusterProjectionPolicyFlags::None;
}

bool FDisplayClusterProjectionReferencePolicy::IsWarpBlendSupported(const IDisplayClusterViewport* InViewport) const
{
	if (IDisplayClusterViewport* SourceViewport = GetSourceViewport(InViewport))
	{
		// Only ICVFX requires composition. Other projection policies do not. So we can reuse all textures and not do warpblend.
		if (SourceViewport->GetProjectionPolicy().IsValid() && SourceViewport->GetProjectionPolicy()->ShouldSupportICVFX(SourceViewport))
		{
			return SourceViewport->GetProjectionPolicy()->IsWarpBlendSupported(SourceViewport);
		}
	}

	return false;
}

bool FDisplayClusterProjectionReferencePolicy::IsWarpBlendSupported_RenderThread(const IDisplayClusterViewportProxy* InViewportProxy) const
{
	// Only ICVFX requires composition. Other projection policies do not. So we can reuse all textures and not do warpblend.
	if (!bSupportICVFXProxy)
	{
		return false;
	}

	if (IDisplayClusterViewportProxy* SourceViewportProxy = GetSourceViewport_RenderThread(InViewportProxy))
	{
		return SourceViewportProxy->GetProjectionPolicy_RenderThread().IsValid() && SourceViewportProxy->GetProjectionPolicy_RenderThread()->IsWarpBlendSupported_RenderThread(SourceViewportProxy);
	}

	return false;
}

void FDisplayClusterProjectionReferencePolicy::ApplyWarpBlend_RenderThread(FRHICommandListImmediate& RHICmdList, const IDisplayClusterViewportProxy* InViewportProxy)
{
	// Only ICVFX requires composition. Other projection policies do not. So we can reuse all textures and not do warpblend.
	if (!bSupportICVFXProxy)
	{
		return;
	}

	if (IDisplayClusterViewportProxy* SourceViewportProxy = GetSourceViewport_RenderThread(InViewportProxy))
	{
		if (SourceViewportProxy->GetProjectionPolicy_RenderThread().IsValid())
		{
			// Rendering a this (cloned) viewport using the logic of the source viewport
			SourceViewportProxy->GetProjectionPolicy_RenderThread()->ApplyWarpBlend_RenderThread(RHICmdList, InViewportProxy);
		}
	}
}

void FDisplayClusterProjectionReferencePolicy::PostUpdateBaseConfiguration(IDisplayClusterViewport* InViewport) const
{
	// Get the current rendering setting we want to change
	FDisplayClusterViewport_RenderSettings InOutRenderSettings = InViewport->GetRenderSettings();

	// If a source viewport exists, use it as the source of textures for rendering
	if (IDisplayClusterViewport* SourceViewport = GetSourceViewport(InViewport))
	{
		if (InViewport->UseSameOCIO(SourceViewport))
		{
			// Rule #1: By default, all resources except the output RTT are cloned.
			// The output texture must be rendered using the projection policy from the parent viewport.
			InOutRenderSettings.SetViewportOverride(ReferencedViewportId, EDisplayClusterViewportOverrideMode::InternalViewportResources);
		}
		else
		{
			// Rule#2: If the OCIO on the cloned viewport is different, only the input RTT is cloned. The custom OCIO must then be applied.
			InOutRenderSettings.SetViewportOverride(ReferencedViewportId, EDisplayClusterViewportOverrideMode::InternalRTT);
		}

		// Apply the changes we made to the current viewport.
		InViewport->SetRenderSettings(InOutRenderSettings);
	}
	else
	{
		// Setup is not ready, just disable this viewport
		InOutRenderSettings.bEnable = false;
	}

	// Apply the changes we made to the current viewport.
	InViewport->SetRenderSettings(InOutRenderSettings);
}

IDisplayClusterViewport* FDisplayClusterProjectionReferencePolicy::GetSourceViewport(const IDisplayClusterViewport* InViewport) const
{
	check(IsInGameThread());
	check(InViewport);

	IDisplayClusterViewportManager* ViewportManager = InViewport ? InViewport->GetConfiguration().GetViewportManager() : nullptr;

	// These are settings we change to alter the viewport behaviour.
	if (ReferencedViewportId.IsEmpty() || !ViewportManager)
	{
		return nullptr;
	}

	// Check if the viewport name we referenced to exists
	IDisplayClusterViewport* SourceViewport = ViewportManager->FindViewport(ReferencedViewportId);
	if (!SourceViewport)
	{
		// The viewport referenced does not exist.
		if (!EnumHasAnyFlags(LogMsgState, EDisplayClusterProjectionReferencePolicyLogMsgState::InvalidName))
		{
			EnumAddFlags(LogMsgState, EDisplayClusterProjectionReferencePolicyLogMsgState::InvalidName);

			UE_LOGF(LogDisplayClusterProjectionReference, Warning,
				"The source viewport '%ls' that referenced by the viewport '%ls' not exist.",
				*ReferencedViewportId, *InViewport->GetId());
		}

		return nullptr;
	}

	// The source viewport should be located on the same node of the cluster.
	if (SourceViewport->GetClusterNodeId() != InViewport->GetClusterNodeId())
	{
		if (!EnumHasAnyFlags(LogMsgState, EDisplayClusterProjectionReferencePolicyLogMsgState::LocalNode))
		{
			EnumAddFlags(LogMsgState, EDisplayClusterProjectionReferencePolicyLogMsgState::LocalNode);

			UE_LOGF(LogDisplayClusterProjectionReference, Warning,
				"The source viewport '%ls' that referenced by the viewport '%ls' must be on the same node.",
				*ReferencedViewportId, *InViewport->GetId());
		}

		return nullptr;
	}

	// The viewport we are referencing cannot reference another viewport or use this projection policy.
	if (SourceViewport->GetRenderSettings().IsViewportOverridden()
	|| (SourceViewport->GetProjectionPolicy().IsValid() && SourceViewport->GetProjectionPolicy()->GetType() == GetType()))
	{
		if (!EnumHasAnyFlags(LogMsgState, EDisplayClusterProjectionReferencePolicyLogMsgState::Recursion))
		{
			EnumAddFlags(LogMsgState, EDisplayClusterProjectionReferencePolicyLogMsgState::Recursion);

			UE_LOGF(LogDisplayClusterProjectionReference, Warning,
				"The source viewport '%ls' that referenced by the viewport '%ls' can't be referenced.",
				*ReferencedViewportId, *InViewport->GetId());
		}

		return nullptr;
	}

	// No errors - let's reset the log flags.
	LogMsgState = EDisplayClusterProjectionReferencePolicyLogMsgState::None;

	return SourceViewport;
}

IDisplayClusterViewportProxy* FDisplayClusterProjectionReferencePolicy::GetSourceViewport_RenderThread(const IDisplayClusterViewportProxy* InViewportProxy) const
{
	check(IsInRenderingThread());
	check(InViewportProxy);

	IDisplayClusterViewportManagerProxy* ViewportManagerProxy = InViewportProxy ? InViewportProxy->GetConfigurationProxy().GetViewportManagerProxy_RenderThread() : nullptr;

	// These are settings we change to alter the viewport behaviour.
	if (ReferencedViewportId_RenderThread.IsEmpty() || !ViewportManagerProxy)
	{
		return nullptr;
	}

	// Check if the viewport name we referenced to exists
	IDisplayClusterViewportProxy* SourceViewportProxy = ViewportManagerProxy->FindViewport_RenderThread(ReferencedViewportId_RenderThread);
	if (!SourceViewportProxy)
	{
		// The viewport referenced does not exist.
		if (!EnumHasAnyFlags(LogMsgState_RenderThread, EDisplayClusterProjectionReferencePolicyLogMsgState::InvalidName))
		{
			EnumAddFlags(LogMsgState_RenderThread, EDisplayClusterProjectionReferencePolicyLogMsgState::InvalidName);

			UE_LOGF(LogDisplayClusterProjectionReference, Warning,
				"The source viewport proxy '%ls' that referenced by the viewport '%ls' not exist.",
				*ReferencedViewportId, *InViewportProxy->GetId());
		}

		return nullptr;
	}

	// The source viewport should be located on the same node of the cluster.
	if (SourceViewportProxy->GetClusterNodeId() != InViewportProxy->GetClusterNodeId())
	{
		if (!EnumHasAnyFlags(LogMsgState_RenderThread, EDisplayClusterProjectionReferencePolicyLogMsgState::LocalNode))
		{
			EnumAddFlags(LogMsgState_RenderThread, EDisplayClusterProjectionReferencePolicyLogMsgState::LocalNode);

			UE_LOGF(LogDisplayClusterProjectionReference, Warning,
				"The source viewport proxy '%ls' that referenced by the viewport '%ls' must be on the same node.",
				*ReferencedViewportId, *InViewportProxy->GetId());
		}

		return nullptr;
	}

	// The viewport we are referencing cannot reference another viewport or use this projection policy.
	if (SourceViewportProxy->GetRenderSettings_RenderThread().IsViewportOverridden()||
		(  SourceViewportProxy->GetProjectionPolicy_RenderThread().IsValid()
		&& SourceViewportProxy->GetProjectionPolicy_RenderThread()->GetType() == GetType()))
	{
		if (!EnumHasAnyFlags(LogMsgState_RenderThread, EDisplayClusterProjectionReferencePolicyLogMsgState::Recursion))
		{
			EnumAddFlags(LogMsgState_RenderThread, EDisplayClusterProjectionReferencePolicyLogMsgState::Recursion);

			UE_LOGF(LogDisplayClusterProjectionReference, Warning,
				"The source viewport proxy '%ls' that referenced by the viewport '%ls' can't be referenced.",
				*ReferencedViewportId, *InViewportProxy->GetId());
		}

		return nullptr;
	}

	// No errors - let's reset the log flags.
	LogMsgState_RenderThread = EDisplayClusterProjectionReferencePolicyLogMsgState::None;

	return SourceViewportProxy;
}
