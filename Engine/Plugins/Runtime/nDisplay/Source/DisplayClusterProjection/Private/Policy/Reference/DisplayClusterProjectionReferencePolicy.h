// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Policy/DisplayClusterProjectionPolicyBase.h"
#include "Templates/SharedPointer.h"

/** Enum flags used to mark log messages after they have been displayed. */
enum class EDisplayClusterProjectionReferencePolicyLogMsgState : uint8
{
	None = 0,

	// The referenced viewport should be on the same cluster node.
	LocalNode = 1 << 0,

	// The viewport we are referencing cannot reference another viewport or use this projection policy.
	Recursion= 1 << 1,

	// The viewport referenced does not exist.
	InvalidName = 1 << 2,
};
ENUM_CLASS_FLAGS(EDisplayClusterProjectionReferencePolicyLogMsgState);

/**
 * Link projection policy (for internal usage)
 */
class FDisplayClusterProjectionReferencePolicy
	: public FDisplayClusterProjectionPolicyBase
	, public TSharedFromThis<FDisplayClusterProjectionReferencePolicy, ESPMode::ThreadSafe>
{
public:
	FDisplayClusterProjectionReferencePolicy(
		const FString& ProjectionPolicyId,
		const FDisplayClusterConfigurationProjection* InConfigurationProjectionPolicy);

public:
	//~Begin IDisplayClusterProjectionPolicy
	virtual const FString& GetType() const override;

	virtual bool ShouldSupportICVFX(const IDisplayClusterViewport* InViewport) const override;
	virtual void PostUpdateBaseConfiguration(IDisplayClusterViewport* InViewport) const override;

	virtual bool IsWarpBlendSupported(const IDisplayClusterViewport* InViewport) const override;
	virtual bool IsWarpBlendSupported_RenderThread(const IDisplayClusterViewportProxy* InViewportProxy) const override;

	virtual void ApplyWarpBlend_RenderThread(FRHICommandListImmediate& RHICmdList, const IDisplayClusterViewportProxy* InViewportProxy) override;

	virtual bool ShouldUseSourceTextureWithMips(const IDisplayClusterViewport* InViewport) const override;
	virtual bool ShouldUseAdditionalTargetableResource(const IDisplayClusterViewport* InViewport) const override;

	virtual EDisplayClusterProjectionPolicyFlags GetProjectionPolicyFlags(const IDisplayClusterViewport* InViewport, const uint32 InContextNum) const override;

	virtual void UpdateProxyData(IDisplayClusterViewport* InViewport) override;

	virtual bool CalculateView(
		IDisplayClusterViewport* InViewport,
		const uint32 InContextNum,
		FVector& InOutViewLocation,
		FRotator& InOutViewRotation,
		const FVector& ViewOffset,
		const float WorldToMeters,
		const float NCP,
		const float FCP) override
	{
		// This projection policy does not use rendering
		return false;
	}

	virtual bool GetProjectionMatrix(
		IDisplayClusterViewport* InViewport,
		const uint32 InContextNum,
		FMatrix& OutPrjMatrix) override
	{
		// This projection policy does not use rendering
		return false;
	}
	//~~End IDisplayClusterProjectionPolicy

protected:
	/** Return viewport referenced by this policy. */
	IDisplayClusterViewport* GetSourceViewport(const IDisplayClusterViewport* InViewport) const;

	/** Return viewport proxy referenced by this policy. */
	IDisplayClusterViewportProxy* GetSourceViewport_RenderThread(const IDisplayClusterViewportProxy* InViewportProxy) const;

private:
	// The name of the viewport referenced by this policy.
	FString ReferencedViewportId;
	FString ReferencedViewportId_RenderThread;

	// Allow a message to be displayed in the log only once.
	mutable EDisplayClusterProjectionReferencePolicyLogMsgState LogMsgState = EDisplayClusterProjectionReferencePolicyLogMsgState::None;
	mutable EDisplayClusterProjectionReferencePolicyLogMsgState LogMsgState_RenderThread = EDisplayClusterProjectionReferencePolicyLogMsgState::None;

	bool bSupportICVFXProxy = false;
};
