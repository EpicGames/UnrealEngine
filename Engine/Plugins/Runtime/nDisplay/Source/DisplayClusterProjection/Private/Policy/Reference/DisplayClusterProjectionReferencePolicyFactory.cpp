// Copyright Epic Games, Inc. All Rights Reserved.

#include "Policy/Reference/DisplayClusterProjectionReferencePolicyFactory.h"
#include "Policy/Reference/DisplayClusterProjectionReferencePolicy.h"
#include "DisplayClusterConfigurationTypes.h"
#include "DisplayClusterProjectionLog.h"

TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe> FDisplayClusterProjectionReferencePolicyFactory::Create(
	const FString& ProjectionPolicyId,
	const FDisplayClusterConfigurationProjection* InConfigurationProjectionPolicy)
{
	if (!InConfigurationProjectionPolicy)
	{
		if (CanShowMsgOnce(ProjectionPolicyId))
		{
			UE_LOGF(LogDisplayClusterProjectionReference, Error, "Reference policy '%ls': null configuration", *ProjectionPolicyId);
		}
		return nullptr;
	}

	UE_LOGF(LogDisplayClusterProjectionReference, Verbose,
		"Instantiating projection policy <%ls> id='%ls'", *InConfigurationProjectionPolicy->Type, *ProjectionPolicyId);

	return  MakeShared<FDisplayClusterProjectionReferencePolicy, ESPMode::ThreadSafe>(ProjectionPolicyId, InConfigurationProjectionPolicy);
}
