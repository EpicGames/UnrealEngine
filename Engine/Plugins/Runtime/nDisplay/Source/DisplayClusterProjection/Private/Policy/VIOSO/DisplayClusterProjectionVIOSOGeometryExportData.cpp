// Copyright Epic Games, Inc. All Rights Reserved.

#include "Policy/VIOSO/DisplayClusterProjectionVIOSOGeometryExportData.h"
#include "Policy/VIOSO/Windows/DisplayClusterProjectionVIOSOWarper.h"
#include "Misc/DisplayClusterDataCache.h"

#include "HAL/IConsoleManager.h"

int32 GDisplayClusterRender_VIOSOPreviewMeshCacheEnable = 1;
static FAutoConsoleVariableRef CDisplayClusterRender_VIOSOPreviewMeshCacheEnable(
	TEXT("nDisplay.cache.VIOSO.PreviewMesh.enable"),
	GDisplayClusterRender_VIOSOPreviewMeshCacheEnable,
	TEXT("Enables the use of the VIOSO preview mesh cache.\n"),
	ECVF_Default
);

int32 GDisplayClusterRender_VIOSOPreviewMeshCacheTimeOutInFrames = 5 * 60 * 60; // Timeout is 5 minutes (for 60 frames per second)
static FAutoConsoleVariableRef CVarDisplayClusterRender_VIOSOPreviewMeshCacheTimeOutInFrames(
	TEXT("nDisplay.cache.VIOSO.PreviewMesh.TimeOut"),
	GDisplayClusterRender_VIOSOPreviewMeshCacheTimeOutInFrames,
	TEXT("The timeout value in frames  for cached VIOSO preview mesh.\n")
	TEXT("-1 - disable timeout.\n"),
	ECVF_Default
);

//////////////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterProjectionVIOSOGeometryExportData
//////////////////////////////////////////////////////////////////////////////////////////////
void FDisplayClusterProjectionVIOSOGeometryExportData::GenerateVIOSOGeometryNormals()
{
	Normal.Empty();

	const int32 TotalTriangles = Triangles.Num() / 3;
	for (int32 TriIndex = 0; TriIndex < TotalTriangles; TriIndex++)
	{
		const int32 InTriangleIndex = TriIndex * 3;

		const int32 f0 = Triangles[InTriangleIndex];
		const int32 f1 = Triangles[InTriangleIndex + 1];
		const int32 f2 = Triangles[InTriangleIndex + 2];

		// revert tri to CW (face)
		Triangles[InTriangleIndex] = f2;
		Triangles[InTriangleIndex + 1] = f1;
		Triangles[InTriangleIndex + 2] = f0;

		//Update normal
		const FVector FaceDir1 = Vertices[f1] - Vertices[f0];
		const FVector FaceDir2 = Vertices[f2] - Vertices[f0];

		const FVector TriNormal = FVector::CrossProduct(FaceDir1, FaceDir2).GetSafeNormal();

		Normal.Add(TriNormal);
		Normal.Add(TriNormal);
		Normal.Add(TriNormal);
	}
}

int32 FDisplayClusterProjectionVIOSOGeometryExportData::GetDataCacheTimeOutInFrames()
{
	return FMath::Max(0, GDisplayClusterRender_VIOSOPreviewMeshCacheTimeOutInFrames);
}

bool FDisplayClusterProjectionVIOSOGeometryExportData::IsDataCacheEnabled()
{
	return GDisplayClusterRender_VIOSOPreviewMeshCacheEnable != 0;
}

#if WITH_VIOSO_LIBRARY
/**
 * The cache for VIOSO preview mesh objects. (Singleton)
 */
class FDisplayClusterWarpBlendVIOSOPreviewMeshCache
	: public TDisplayClusterDataCache<FDisplayClusterProjectionVIOSOGeometryExportData>
{
public:
	static TSharedPtr<FDisplayClusterProjectionVIOSOGeometryExportData, ESPMode::ThreadSafe> GetOrCreateVIOSOPreviewMesh(const TSharedRef<FDisplayClusterProjectionVIOSOLibrary, ESPMode::ThreadSafe>& InVIOSOLibrary, const FViosoPolicyConfiguration& InConfigData)
	{
		static FDisplayClusterWarpBlendVIOSOPreviewMeshCache VIOSOPreviewMeshCacheSingleton;

		// Only for parameters related to geometry
		const FString UniqueName = HashString(InConfigData.ToString(true));

		TSharedPtr<FDisplayClusterProjectionVIOSOGeometryExportData, ESPMode::ThreadSafe> GeometryExportDataRef = VIOSOPreviewMeshCacheSingleton.Find(UniqueName);
		if (!GeometryExportDataRef.IsValid())
		{
			GeometryExportDataRef = MakeShared<FDisplayClusterProjectionVIOSOGeometryExportData, ESPMode::ThreadSafe>(UniqueName);

			FDisplayClusterProjectionVIOSOWarper VIOSOWarper(InVIOSOLibrary, InConfigData, TEXT(""));
			if (VIOSOWarper.ExportGeometry(*GeometryExportDataRef.Get()))
			{
				VIOSOPreviewMeshCacheSingleton.Add(GeometryExportDataRef);
			}
			else
			{
				// Unable to initialize, invalid data must be deleted
				GeometryExportDataRef.Reset();
			}

		}

		return GeometryExportDataRef;
	}
};
#endif

TSharedPtr<FDisplayClusterProjectionVIOSOGeometryExportData, ESPMode::ThreadSafe> FDisplayClusterProjectionVIOSOGeometryExportData::Create(const TSharedRef<FDisplayClusterProjectionVIOSOLibrary, ESPMode::ThreadSafe>& InVIOSOLibrary, const FViosoPolicyConfiguration& InConfigData)
{
#if WITH_VIOSO_LIBRARY
	return FDisplayClusterWarpBlendVIOSOPreviewMeshCache::GetOrCreateVIOSOPreviewMesh(InVIOSOLibrary, InConfigData);
#else
	return nullptr;
#endif
}
