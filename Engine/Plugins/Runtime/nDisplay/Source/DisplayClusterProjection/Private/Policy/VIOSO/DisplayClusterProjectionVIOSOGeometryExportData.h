// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FDisplayClusterProjectionVIOSOLibrary;
struct FViosoPolicyConfiguration;

class FDisplayClusterProjectionVIOSOGeometryExportData
{
public:
	FDisplayClusterProjectionVIOSOGeometryExportData(const FString& InUniqueName)
		: UniqueName(InUniqueName)
	{ }

	/** Generate normals. */
	void GenerateVIOSOGeometryNormals();
	

	//~ Begin TDisplayClusterDataCache
	/** Return DataCache timeout in frames. */
	static int32 GetDataCacheTimeOutInFrames();

	/** Return true if DataCache is enabled. */
	static bool IsDataCacheEnabled();

	/** Returns the unique name of this MPCDI file resource for DataCache. */
	inline const FString& GetDataCacheName() const
	{
		return UniqueName;
	}

	/** Method for releasing a cached data item, called before its destructor. */
	inline void ReleaseDataCacheItem()
	{ }
	// ~~ End TDisplayClusterDataCache

	static TSharedPtr<FDisplayClusterProjectionVIOSOGeometryExportData, ESPMode::ThreadSafe> Create(const TSharedRef<FDisplayClusterProjectionVIOSOLibrary, ESPMode::ThreadSafe>& InVIOSOLibrary, const FViosoPolicyConfiguration& InConfigData);

public:
	const FString UniqueName;

	TArray<FVector>   Vertices;
	TArray<FVector>   Normal;
	TArray<FVector2D> UV;
	TArray<int32>     Triangles;
};
