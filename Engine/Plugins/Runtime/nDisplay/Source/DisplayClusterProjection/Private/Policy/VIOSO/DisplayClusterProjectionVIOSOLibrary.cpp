// Copyright Epic Games, Inc. All Rights Reserved.

#include "Policy/VIOSO/DisplayClusterProjectionVIOSOLibrary.h"

#include "DisplayClusterProjectionLog.h"

#include "HAL/PlatformProcess.h"
#include "Misc/DisplayClusterHelpers.h"
#include "PDisplayClusterProjectionStrings.h"

#if !WITH_VIOSO_LIBRARY
FDisplayClusterProjectionVIOSOLibrary::FDisplayClusterProjectionVIOSOLibrary()
{ }

FDisplayClusterProjectionVIOSOLibrary::~FDisplayClusterProjectionVIOSOLibrary()
{ }

#else

#define ImportVIOSOFunc(DLLFunc)\
	DLLFunc = (DLLFunc##Proc)FPlatformProcess::GetDllExport((HMODULE)VIOSO_DLL_Handler, TEXT(#DLLFunc));\
	if(DLLFunc==nullptr)\
		{\
			UE_LOGF(LogDisplayClusterProjectionVIOSO, Error, "Couldn't initialize VIOSO API func '%ls'.", TEXT(#DLLFunc));\
			return false;\
		}

bool FDisplayClusterProjectionVIOSOLibrary::InitializeDLL()
{
	if (VIOSO_DLL_Handler)
	{
		ImportVIOSOFunc(VWB_getVersion);

		ImportVIOSOFunc(VWB_CreateA);
		ImportVIOSOFunc(VWB_Destroy);
		ImportVIOSOFunc(VWB_Init);
		ImportVIOSOFunc(VWB_getViewClip);
		ImportVIOSOFunc(VWB_getPosDirClip);
		ImportVIOSOFunc(VWB_render);

		ImportVIOSOFunc(VWB_getWarpBlendMesh);
		ImportVIOSOFunc(VWB_destroyWarpBlendMesh);
	}

	// All VIOSO DLL functions are initialized successfully
	bInitialized = true;

	return true;
}

void FDisplayClusterProjectionVIOSOLibrary::ReleaseDLL()
{
	bInitialized = false;

	if (VIOSO_DLL_Handler)
	{
		FPlatformProcess::FreeDllHandle(VIOSO_DLL_Handler);
	}
}

FDisplayClusterProjectionVIOSOLibrary::FDisplayClusterProjectionVIOSOLibrary()
{
	const FString DllPath = DisplayClusterHelpers::filesystem::GetFullPathForThirdPartyDLL(DisplayClusterProjectionStrings::ThirdParty::DLL::VIOSO);
	const FString DllDirectory = FPaths::GetPath(DllPath);

	FPlatformProcess::PushDllDirectory(*DllDirectory);
	VIOSO_DLL_Handler = FPlatformProcess::GetDllHandle(*DllPath);
	FPlatformProcess::PopDllDirectory(*DllDirectory);

	if (VIOSO_DLL_Handler)
	{
		if (InitializeDLL())
		{
			int32 VersionMajor, VersionMinor, VersionMaintenance, VersionBuild;
			if (VWB_getVersion && VWB_getVersion(&VersionMajor, &VersionMinor, &VersionMaintenance, &VersionBuild) == VWB_ERROR_NONE)
			{
				UE_LOGF(LogDisplayClusterProjectionVIOSO, Log, "VIOSO API(%d,%d,%d,%d) was initialized from file '%ls'.", VersionMajor, VersionMinor, VersionMaintenance, VersionBuild , *DllPath);

				return;
			}
		}
		
		UE_LOGF(LogDisplayClusterProjectionVIOSO, Error, "VIOSO API could not be initialized. Maybe wrong DLL-file '%ls'.", *DllPath);
	}
	else
	{
		UE_LOGF(LogDisplayClusterProjectionVIOSO, Error, "Could not find VIOSO API '%ls' file.", *DllPath);
	}

	ReleaseDLL();
}

FDisplayClusterProjectionVIOSOLibrary::~FDisplayClusterProjectionVIOSOLibrary()
{
	UE_LOGF(LogDisplayClusterProjectionVIOSO, Log, "VIOSO API released.");
	ReleaseDLL();
}

#endif
