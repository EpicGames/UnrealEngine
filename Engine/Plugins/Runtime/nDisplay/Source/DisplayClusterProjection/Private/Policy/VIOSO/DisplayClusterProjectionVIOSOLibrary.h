// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "DisplayClusterProjectionVIOSOTypes.h"

#define DeclareVIOSOFunc(DLLFunc) DLLFunc##Proc DLLFunc = nullptr;

/**
* Implement access to VIOSO DLL functions
* The basic idea is to load the DLL dynamically without using LIB files
* 
* The DLL function is described in the file //nDisplay/Source/ThirdParty/VIOSO/Include/VIOSOWarpBlend.h
* Supported: IDirect3DDevice9,IDirect3DDevice9Ex,ID3D10Device,ID3D10Device1,ID3D11Device,ID3D12CommandQueue (for ID3D12Device initialization)
*/
class FDisplayClusterProjectionVIOSOLibrary
{
public:
	FDisplayClusterProjectionVIOSOLibrary();
	~FDisplayClusterProjectionVIOSOLibrary();

	/** If the DLL is loaded correctly, this function returns true. */
	inline bool IsInitialized() const
	{
		return bInitialized;
	}

#if WITH_VIOSO_LIBRARY

public:
	DeclareVIOSOFunc(VWB_getVersion);

	DeclareVIOSOFunc(VWB_CreateA);
	DeclareVIOSOFunc(VWB_Destroy);
	DeclareVIOSOFunc(VWB_Init);
	DeclareVIOSOFunc(VWB_getViewClip);
	DeclareVIOSOFunc(VWB_getPosDirClip);
	DeclareVIOSOFunc(VWB_render);

	DeclareVIOSOFunc(VWB_getWarpBlendMesh);
	DeclareVIOSOFunc(VWB_destroyWarpBlendMesh);

	// Protect VIOSO SDK calls: the DLL is not thread-safe.
	FCriticalSection DllAccessCS;

private:
	/** Imprort all functionf from VIOSO DLL. Return true on success. */
	bool InitializeDLL();
	void ReleaseDLL();

#endif

private:
	// Initializetion state
	bool bInitialized = false;

	// Saved handle to VIOSO DLL
	void* VIOSO_DLL_Handler = nullptr;
};

#undef DeclareVIOSOFunc
