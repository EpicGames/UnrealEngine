// Copyright Epic Games, Inc. All Rights Reserved.

#include "Policy/VIOSO/DisplayClusterProjectionVIOSOPolicyFactory.h"

#include "DisplayClusterProjectionStrings.h"
#include "DisplayClusterProjectionLog.h"

#include "DisplayClusterConfigurationTypes.h"

#if PLATFORM_WINDOWS
#include "Policy/VIOSO/Windows/DisplayClusterProjectionVIOSOPolicy.h"
#endif

#include "Policy/VIOSO/DisplayClusterProjectionVIOSOLibrary.h"

//////////////////////////////////////////////////////////////////////////////////////////////
// IDisplayClusterProjectionPolicyFactory
//////////////////////////////////////////////////////////////////////////////////////////////
FDisplayClusterProjectionVIOSOPolicyFactory::FDisplayClusterProjectionVIOSOPolicyFactory()
{
	VIOSOLibrary = MakeShared<FDisplayClusterProjectionVIOSOLibrary, ESPMode::ThreadSafe>();
}

TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe> FDisplayClusterProjectionVIOSOPolicyFactory::Create(const FString& ProjectionPolicyId, const FDisplayClusterConfigurationProjection* InConfigurationProjectionPolicy)
{
	if(!InConfigurationProjectionPolicy)
	{
		if (CanShowMsgOnce(ProjectionPolicyId))
		{
			UE_LOGF(LogDisplayClusterProjectionVIOSO, Error, "VIOSO policy '%ls': null configuration", *ProjectionPolicyId);
		}
		return nullptr;
	}

	if (!VIOSOLibrary.IsValid() || !VIOSOLibrary->IsInitialized())
	{
		if (CanShowMsgOnce(TEXT("VIOSOLibrary")))
		{
			UE_LOGF(LogDisplayClusterProjectionVIOSO, Error, "VIOSO API not initialized: cannot instantiate projection policy <%ls> id='%ls'", *InConfigurationProjectionPolicy->Type, *ProjectionPolicyId);
		}
		return nullptr;
	}

#if PLATFORM_WINDOWS
	UE_LOGF(LogDisplayClusterProjectionVIOSO, Verbose, "Instantiating projection policy <%ls> id='%ls'", *InConfigurationProjectionPolicy->Type, *ProjectionPolicyId);

	return  MakeShared<FDisplayClusterProjectionVIOSOPolicy, ESPMode::ThreadSafe>(ProjectionPolicyId, InConfigurationProjectionPolicy, VIOSOLibrary.ToSharedRef());
#else
	if (CanShowMsgOnce(TEXT("VIOSOWindowsOnly")))
	{
		UE_LOGF(LogDisplayClusterProjectionVIOSO, Error, "VIOSO does not support the current platform: cannot instantiate projection policy <%ls> id='%ls'", *InConfigurationProjectionPolicy->Type, *ProjectionPolicyId);
	}
	return nullptr;
#endif
}
