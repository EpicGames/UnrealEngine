// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Render/Projection/IDisplayClusterProjectionPolicyFactory.h"
#include "Policy/DisplayClusterProjectionPolicyFactoryBase.h"

/**
 * Factory for the 'VIOSO' projection policy (Windows only).
 * Creates policies backed by the VIOSO warping and blending library.
 * The factory owns the shared VIOSO library instance; policy creation fails
 * if the library is unavailable or not yet initialized.
 */
class FDisplayClusterProjectionVIOSOPolicyFactory
	: public IDisplayClusterProjectionPolicyFactory
	, protected FDisplayClusterProjectionPolicyFactoryBase
{
public:
	FDisplayClusterProjectionVIOSOPolicyFactory();
	virtual ~FDisplayClusterProjectionVIOSOPolicyFactory() = default;

public:
	//~ Begin IDisplayClusterProjectionPolicyFactory
	virtual TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe> Create(const FString& ProjectionPolicyId, const FDisplayClusterConfigurationProjection* InConfigurationProjectionPolicy) override;
	//~ End IDisplayClusterProjectionPolicyFactory

private:
	TSharedPtr<class FDisplayClusterProjectionVIOSOLibrary, ESPMode::ThreadSafe> VIOSOLibrary;
};
