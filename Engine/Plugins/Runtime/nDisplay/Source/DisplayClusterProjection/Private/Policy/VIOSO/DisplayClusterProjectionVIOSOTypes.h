// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if PLATFORM_WINDOWS

#define WITH_VIOSO_LIBRARY 1

THIRD_PARTY_INCLUDES_START
#include "Windows/AllowWindowsPlatformTypes.h"
#include "VWBTypes.h"
#include "Windows/HideWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_END

/* Get the version of the API
* 
* @param major       - (out) major version
* @param minor       - (out) minor version
* @param maintenance - (out) maintenance revision
* @param build       - (out) build number
* 
* @return VWB_ERROR_NONE on success, VWB_ERROR_PARAMETER, if parameters are out of range
*/
typedef VWB_ERROR(__stdcall* VWB_getVersionProc)(VWB_int* major, VWB_int* minor, VWB_int* maintenance, VWB_int* build);

/** Creates a new VIOSO Warp & Blend API instance
* 
* @param pDxDevice     - a pointer to a DirectX device for Direct3D 9 to 11; for Direct3D 12, you need to specify a pointer to a ID3D12CommandQueue,
*                        set to NULL for OpenGL,
*                        set to VWB_DUMMYDEVICE, to just hold the data to create a textured mesh.
* @param szConfigFile  - path to a .ini file containing settings, if NULL the default values are used
* @param szChannelName -  a section name to look for in .ini-file.
* @param ppWarper      - (out) this receives the warper
* @param logLevel      - set a log level.
* @param logFile       - set a log file.
* 
* @return VWB_ERROR_NONE on success, otherwise @see VWB_ERROR
* 
* @remarks Default settings are loaded, then, if a .ini file is given, the settings are updated.
*          If there is a [default] section, values are loaded from there, before individual settings are loaded.
*          After creating, you are able to edit all settings. You may specify a logLevel in .ini, it overwrites given loglevel.
*          You have to destroy a created warper using VWB_Destroy.
*          After successful creation, call VWB_Init, to load .vwf file and initialize the warper.
*/
typedef VWB_ERROR(__stdcall * VWB_CreateAProc)(void* pDxDevice, char const* szConfigFile, char const* szChannelName, VWB_Warper** ppWarper, VWB_int logLevel, char const* szLogFile);
	
/** Destroys a warper
* @param pWarper - (in) a warper to be destroyed
*/
typedef void(__stdcall * VWB_DestroyProc)(VWB_Warper* pWarper);

/** Initializes a warper
* 
* @param pWarper - a warper
* 
* @remarks In case of an OpenGL warper, the later used context needs to be active. There is no multi-threading on side of VIOSO API.\
*/
typedef VWB_ERROR(__stdcall* VWB_InitProc)(VWB_Warper* pWarper);

/** Set the new dynamic eye position and direction
* 
* @param pWarper - (in)          a valid warper
* @param pEye    - (in,out, opt) sets the new eye position, if a eye point provider is present, the value is getted from there, pEye is updated
* @param pRot    - (in,out, opt) the new rotation in radian, if a eye point provider is present, the value is getted from there, pRot is updated
* @param pView   - (out)         it gets the updated view matrix to translate and rotate into the viewer's perspective
* @param pClip   - (out)         it gets the updated clip planes: left, top, right, bottom, near, far, where all components are usually positive
* 
* @return VWB_ERROR_NONE on success, VWB_ERROR_GENERIC otherwise
*/
typedef VWB_ERROR(__stdcall* VWB_getViewClipProc)(VWB_Warper* pWarper, VWB_float* pEye, VWB_float* pRot, VWB_float* pView, VWB_float* pClip);

/** Set the new dynamic eye position and direction
*
* @param pWarper - (in)          a valid warper
* @param pEye    - (in,out, opt) sets the new eye position, if a eye point provider is present, the value is getted from there, pEye is updated
* @param pRot    - (in,out, opt) the new rotation in radian, if a eye point provider is present, the value is getted from there, pRot is updated
* @param pPos      (out)         it gets the updated relative position: x,y,z
* @param pDir      (out)         it gets the updated relative direction: euler angles around x,y,z rotation order is y,x,z
* @param pClip   - (out)         it gets the updated clip planes: left, top, right, bottom, near, far, where all components are usually positive
* @param symmetric (in opt)      set to true, to force symmetric frustum, NOTE: symmetric frusta are worse in quality, as the image plane is no longer parallel to the screen plane
* @param aspect    (in, opt)     set to some ratio, to force clip planes to form a rect with this ratio, if set to 0, optimal aspect ratio is used. NOTE: Non-optimal aspect ratia are worse in quality, as part of the input image is rendered but not used.
*
* @return VWB_ERROR_NONE on success, VWB_ERROR_GENERIC otherwise
*/
typedef VWB_ERROR(__stdcall* VWB_getPosDirClipProc)(VWB_Warper* pWarper, VWB_float* pEye, VWB_float* pRot, VWB_float* pPos, VWB_float* pDir, VWB_float* pClip, bool symmetric, VWB_float aspect);

/** Eender a warped and blended source texture into the current back buffer
* 
* @param pWarper   - a valid warper
* @param pSrc      - the source texture, a IDirect3DTexture9*, ID3D10Texture2D*, ID3D11Texture2D*, VWB_D3D12_RENDERINPUT* or a GLint texture index;
*                    if current backbuffer must be read, set to NULL in any DX mode except 12 or to -1 in OpenGL mode
*                    in case of directX 12 you need to provide a @see VWB_D3D12_RENDERINPUT as parameter.
* @param stateMask - @see VWB_STATEMASK enumeration, default is 0 to restore usual stuff
*                    In D3D12 all flags except VWB_STATEMASK_CLEARBACKBUFFER are ignored.
*                    The application is required to set inputs and shader in each term anyway.
* 
* @return VWB_ERROR_NONE on success, VWB_ERROR_GENERIC otherwise
*/
typedef VWB_ERROR(__stdcall* VWB_renderProc)(VWB_Warper* pWarper, VWB_param src, VWB_uint stateMask);

/** Fills a VWB_WarpBlendMesh from currently loaded data. It uses cols * rows vertices or less depending on how .vwf is filled. Warper needs to be initialized as VWB_DUMMYDEVICE.
* 
* @param pWarper - a valid warper
* @param cols    - sets the number of columns
* @param rows    - sets the number of rows
* @param mesh    - (out) the resulting mesh, the mesh will be emptied before filled
* 
* @return VWB_ERROR_NONE on success, VWB_ERROR_PARAMETER, if parameters are out of range, VWB_ERROR_GENERIC otherwise\
*/
typedef VWB_ERROR(__stdcall* VWB_getWarpBlendMeshProc)(VWB_Warper* pWarper, VWB_int cols, VWB_int rows, VWB_WarpBlendMesh& mesh);

/** Destroys a VWB_WarpBlendMesh
* 
* @param pWarper - a valid warper
* @param mesh    - the mesh to be destroyed
* 
* @return VWB_ERROR_NONE on success, VWB_ERROR_PARAMETER, if parameters are out of range, VWB_ERROR_GENERIC otherwise.
*/
typedef VWB_ERROR(__stdcall* VWB_destroyWarpBlendMeshProc)(VWB_Warper* pWarper, VWB_WarpBlendMesh& mesh);

#else

// VIOSO library is not supported by the current platform
#define WITH_VIOSO_LIBRARY 0

#endif
