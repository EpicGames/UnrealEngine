// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViosoPolicyConfiguration.h"

#include "DisplayClusterProjectionStrings.h"
#include "DisplayClusterProjectionLog.h"
#include "Misc/DisplayClusterHelpers.h"
#include "Policy/DisplayClusterProjectionPolicyBase.h"
#include "Render/Viewport/IDisplayClusterViewport.h"

#include "Misc/DisplayClusterHelpers.h"

//////////////////////////////////////////////////////////////////////////////////////////////
// FViosoPolicyConfiguration
//////////////////////////////////////////////////////////////////////////////////////////////
FString FViosoPolicyConfiguration::ToString(const bool bOnlyGeometryParameters) const
{
	if (bOnlyGeometryParameters)
	{
		if (!INIFile.IsEmpty())
		{
			// Initialize from ini file
			return FString::Printf(TEXT("%s='%s', %s='%s'"),
				DisplayClusterProjectionStrings::cfg::VIOSO::INIFile, *INIFile,
				DisplayClusterProjectionStrings::cfg::VIOSO::ChannelName, *ChannelName
			);
		}
		else
		{
			// Initialize from calibration file
			return FString::Printf(TEXT("%s='%s', %s=%d,  %s='%f'"),
				DisplayClusterProjectionStrings::cfg::VIOSO::File, *CalibrationFile,
				DisplayClusterProjectionStrings::cfg::VIOSO::CalibIndex, CalibrationIndex,
				DisplayClusterProjectionStrings::cfg::VIOSO::UnitsInMeter, UnitsInMeter
			);
		}
	}
	else
	{
		if (!INIFile.IsEmpty())
		{
			// Initialize from ini file
			return FString::Printf(TEXT("%s='%s',  %s='%s', %s='%s'"),
				DisplayClusterProjectionStrings::cfg::VIOSO::Origin, *OriginCompId,
				DisplayClusterProjectionStrings::cfg::VIOSO::INIFile, *INIFile,
				DisplayClusterProjectionStrings::cfg::VIOSO::ChannelName, *ChannelName
			);
		}
		else
		{
			// Initialize from calibration file
			return FString::Printf(TEXT("%s='%s',  %s='%s', %s=%d,  %s='%f', %s=%f"),
				DisplayClusterProjectionStrings::cfg::VIOSO::Origin, *OriginCompId,
				DisplayClusterProjectionStrings::cfg::VIOSO::File, *CalibrationFile,
				DisplayClusterProjectionStrings::cfg::VIOSO::CalibIndex, CalibrationIndex,
				DisplayClusterProjectionStrings::cfg::VIOSO::UnitsInMeter, UnitsInMeter,
				DisplayClusterProjectionStrings::cfg::VIOSO::Gamma, Gamma
			);
		}
	}
}

bool FViosoPolicyConfiguration::Initialize(const TMap<FString, FString>& InParameters, IDisplayClusterViewport* InViewport)
{
	check(InViewport);

	const FString InViewportId = InViewport->GetId();

	if (DisplayClusterHelpers::map::template ExtractValueFromString(InParameters, DisplayClusterProjectionStrings::cfg::VIOSO::Origin, OriginCompId))
	{
		UE_LOGF(LogDisplayClusterProjectionVIOSO, Verbose, "Viewport <%ls>: Projection parameter '%ls' - '%ls'", *InViewportId, DisplayClusterProjectionStrings::cfg::VIOSO::Origin, *OriginCompId);
	}
	else
	{
		if (!FDisplayClusterProjectionPolicyBase::IsEditorOperationMode(InViewport))
		{
			UE_LOGF(LogDisplayClusterProjectionVIOSO, Error, "Viewport <%ls>: Projection parameter '%ls' not found", *InViewportId, DisplayClusterProjectionStrings::cfg::VIOSO::Origin);
		}

		return false;
	}

	if (DisplayClusterHelpers::map::template ExtractValueFromString(InParameters, DisplayClusterProjectionStrings::cfg::VIOSO::EnablePreview, bIsPreviewMeshEnabled))
	{
		UE_LOGF(LogDisplayClusterProjectionVIOSO, Verbose, "Viewport <%ls>: Projection parameter '%ls' - '%ls'", *InViewportId, DisplayClusterProjectionStrings::cfg::VIOSO::EnablePreview, *DisplayClusterHelpers::str::BoolToStr(bIsPreviewMeshEnabled));
	}

	FString CfgINIFile;
	if (DisplayClusterHelpers::map::template ExtractValueFromString(InParameters, DisplayClusterProjectionStrings::cfg::VIOSO::INIFile, CfgINIFile) && !CfgINIFile.IsEmpty())
	{
		UE_LOGF(LogDisplayClusterProjectionVIOSO, Verbose, "Viewport <%ls>: Projection parameter '%ls' - '%ls'", *InViewportId, DisplayClusterProjectionStrings::cfg::VIOSO::INIFile, *CfgINIFile);

		// Get full path to calibration file:
		INIFile = DisplayClusterHelpers::filesystem::GetFullPathForConfigResource(CfgINIFile);

		if (INIFile.IsEmpty())
		{
			if (!FDisplayClusterProjectionPolicyBase::IsEditorOperationMode(InViewport))
			{
				UE_LOGF(LogDisplayClusterProjectionVIOSO, Error, "Vioso INI file empty string");
			}

			return false;
		}

		// Test fullpath filename exist
		if (!FPaths::FileExists(INIFile))
		{
			UE_LOGF(LogDisplayClusterProjectionVIOSO, Error, "Vioso INI file '%ls' not found", *INIFile);
			return false;
		}

		if (DisplayClusterHelpers::map::template ExtractValueFromString(InParameters, DisplayClusterProjectionStrings::cfg::VIOSO::ChannelName, ChannelName))
		{
			UE_LOGF(LogDisplayClusterProjectionVIOSO, Verbose, "Viewport <%ls>: Projection parameter '%ls' - '%ls'", *InViewportId, DisplayClusterProjectionStrings::cfg::VIOSO::ChannelName, *ChannelName);
		}
		else
		{
			if (!FDisplayClusterProjectionPolicyBase::IsEditorOperationMode(InViewport))
			{
				UE_LOGF(LogDisplayClusterProjectionVIOSO, Error, "Viewport <%ls>: Projection parameter '%ls' not found", *InViewportId, DisplayClusterProjectionStrings::cfg::VIOSO::ChannelName);
			}

			return false;
		}

		// Get all settings from INIFile for ChannelName data
		return true;
	}


	FString CfgCalibrationFile;
	if (DisplayClusterHelpers::map::template ExtractValueFromString(InParameters, DisplayClusterProjectionStrings::cfg::VIOSO::File, CfgCalibrationFile) && !CfgCalibrationFile.IsEmpty())
	{
		UE_LOGF(LogDisplayClusterProjectionVIOSO, Verbose, "Viewport <%ls>: Projection parameter '%ls' - '%ls'", *InViewportId, DisplayClusterProjectionStrings::cfg::VIOSO::File, *CfgCalibrationFile);

		// Get full path to calibration file:
		CalibrationFile = DisplayClusterHelpers::filesystem::GetFullPathForConfigResource(CfgCalibrationFile);

		if (CalibrationFile.IsEmpty())
		{
			if (!FDisplayClusterProjectionPolicyBase::IsEditorOperationMode(InViewport))
			{
				UE_LOGF(LogDisplayClusterProjectionVIOSO, Error, "Calibration file empty string");
			}

			return false;
		}

		// Test fullpath filename exist
		if (!FPaths::FileExists(CalibrationFile))
		{
			UE_LOGF(LogDisplayClusterProjectionVIOSO, Error, "Calibration file '%ls' not found", *CalibrationFile);
			return false;
		}

		if (DisplayClusterHelpers::map::template ExtractValueFromString(InParameters, DisplayClusterProjectionStrings::cfg::VIOSO::CalibIndex, CalibrationIndex))
		{
			UE_LOGF(LogDisplayClusterProjectionVIOSO, Verbose, "Viewport <%ls>: Projection parameter '%ls' - '%d'", *InViewportId, DisplayClusterProjectionStrings::cfg::VIOSO::CalibIndex, CalibrationIndex);
		}
		else
			if (CalibrationIndex < 0)
			{
				// Use neg CalibrationIndex values for defined adapter
				int32 AdapterIndex = 0;
				if (DisplayClusterHelpers::map::template ExtractValueFromString(InParameters, DisplayClusterProjectionStrings::cfg::VIOSO::CalibAdapter, AdapterIndex))
				{
					UE_LOGF(LogDisplayClusterProjectionVIOSO, Verbose, "Viewport <%ls>: Projection parameter '%ls' - '%d'", *InViewportId, DisplayClusterProjectionStrings::cfg::VIOSO::CalibAdapter, AdapterIndex);
					CalibrationIndex = -1 * AdapterIndex;
				}
				else
				{
					CalibrationIndex = 0;
					if (!FDisplayClusterProjectionPolicyBase::IsEditorOperationMode(InViewport))
					{
						UE_LOGF(LogDisplayClusterProjectionVIOSO, Error, "Viewport <%ls>: Projection parameter '%ls' or '%ls'  not found, Use default value - '%d'", *InViewportId, DisplayClusterProjectionStrings::cfg::VIOSO::CalibIndex, DisplayClusterProjectionStrings::cfg::VIOSO::CalibAdapter, CalibrationIndex);
					}
				}
			}

		if (DisplayClusterHelpers::map::template ExtractValueFromString(InParameters, DisplayClusterProjectionStrings::cfg::VIOSO::UnitsInMeter, UnitsInMeter))
		{
			UE_LOGF(LogDisplayClusterProjectionVIOSO, Verbose, "Viewport <%ls>: Projection parameter '%ls' - '%f'", *InViewportId, DisplayClusterProjectionStrings::cfg::VIOSO::UnitsInMeter, UnitsInMeter);
		}
		else
		{
			UnitsInMeter = 1000.f;
			if (!FDisplayClusterProjectionPolicyBase::IsEditorOperationMode(InViewport))
			{
				UE_LOGF(LogDisplayClusterProjectionVIOSO, Error, "Viewport <%ls>: Projection parameter '%ls' not found, Use default value - '%f'", *InViewportId, DisplayClusterProjectionStrings::cfg::VIOSO::UnitsInMeter, UnitsInMeter);
			}
		}


		if (DisplayClusterHelpers::map::template ExtractValueFromString(InParameters, DisplayClusterProjectionStrings::cfg::VIOSO::Gamma, Gamma))
		{
			UE_LOGF(LogDisplayClusterProjectionVIOSO, Verbose, "Viewport <%ls>: Projection parameter '%ls' - '%f'", *InViewportId, DisplayClusterProjectionStrings::cfg::VIOSO::Gamma, Gamma);
		}
		else
		{
			if (!FDisplayClusterProjectionPolicyBase::IsEditorOperationMode(InViewport))
			{
				UE_LOGF(LogDisplayClusterProjectionVIOSO, Error, "Viewport <%ls>: Projection parameter '%ls' not found. Assigned default value - '%f'", *InViewportId, DisplayClusterProjectionStrings::cfg::VIOSO::Gamma, Gamma);
			}
		}

		return true;
	}

	if (!FDisplayClusterProjectionPolicyBase::IsEditorOperationMode(InViewport))
	{
		UE_LOGF(LogDisplayClusterProjectionVIOSO, Error, "Viewport <%ls>: Projection parameter '%ls' not found", *InViewportId, DisplayClusterProjectionStrings::cfg::VIOSO::File);
	}

	return false;
}
