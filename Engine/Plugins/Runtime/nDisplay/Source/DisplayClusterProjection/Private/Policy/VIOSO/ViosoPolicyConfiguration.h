// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class IDisplayClusterViewport;

/**
* VIOSO projection policy configuration data
*/
struct FViosoPolicyConfiguration
{
	FString OriginCompId;

	FString INIFile;
	FString ChannelName;

	// Calibration file and display segment with geometry
	FString CalibrationFile;
	int32     CalibrationIndex = -1;

	//@todo add more vioso options, if required
	float Gamma = 1.f;

	// How many VIOSO units in meter
	float UnitsInMeter = 1000.f;

	// Allow to use preview mesh
	bool bIsPreviewMeshEnabled = false;

	// Preview mesh dimensions
	int32 PreviewMeshWidth = 100;
	int32 PreviewMeshHeight = 100;

	bool Initialize(const TMap<FString, FString>& InParameters, IDisplayClusterViewport* InViewport);

	/**
	* convert all parameters to string
	*/
	FString ToString(const bool bOnlyGeometryParameters = false) const;
};
