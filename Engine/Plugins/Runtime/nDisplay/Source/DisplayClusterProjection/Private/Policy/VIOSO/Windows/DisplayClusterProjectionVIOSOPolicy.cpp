// Copyright Epic Games, Inc. All Rights Reserved.

#include "Policy/VIOSO/Windows/DisplayClusterProjectionVIOSOPolicy.h"

#include "DisplayClusterProjectionLog.h"
#include "DisplayClusterProjectionStrings.h"

#include "Config/IDisplayClusterConfigManager.h"
#include "Game/IDisplayClusterGameManager.h"
#include "IDisplayCluster.h"
#include "Misc/DisplayClusterHelpers.h"
#include "Render/Viewport/IDisplayClusterViewport.h"
#include "Render/Viewport/IDisplayClusterViewportManager.h"
#include "Render/Viewport/IDisplayClusterViewportProxy.h"

#include "Engine/RendererSettings.h"
#include "Misc/Paths.h"
#include "Misc/ScopeLock.h"
#include "RenderingThread.h"

#include "ProceduralMeshComponent.h"

//////////////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterProjectionVIOSOPolicy
//////////////////////////////////////////////////////////////////////////////////////////////
FDisplayClusterProjectionVIOSOPolicy::FDisplayClusterProjectionVIOSOPolicy(const FString& ProjectionPolicyId, const FDisplayClusterConfigurationProjection* InConfigurationProjectionPolicy, const TSharedRef<FDisplayClusterProjectionVIOSOLibrary, ESPMode::ThreadSafe>& InVIOSOLibrary)
	: FDisplayClusterProjectionPolicyBase(ProjectionPolicyId, InConfigurationProjectionPolicy)
	, VIOSOLibrary(InVIOSOLibrary)
{ }

FDisplayClusterProjectionVIOSOPolicy::~FDisplayClusterProjectionVIOSOPolicy()
{
	ImplRelease();
}


const FString& FDisplayClusterProjectionVIOSOPolicy::GetType() const
{
	static const FString Type(DisplayClusterProjectionStrings::projection::VIOSO);
	return Type;
}

bool FDisplayClusterProjectionVIOSOPolicy::HandleStartScene(IDisplayClusterViewport* InViewport)
{
	if (!InViewport)
	{
		return false;
	}

	check(IsInGameThread());

	// Read VIOSO config data from nDisplay config file
	if (!ViosoConfigData.Initialize(GetParameters(), InViewport))
	{
		if (!IsEditorOperationMode(InViewport))
		{
			UE_LOGF(LogDisplayClusterProjectionVIOSO, Error, "Couldn't read VIOSO configuration from the config file for viewport -'%ls'", *InViewport->GetId());
		}

		return false;
	}

	// Find origin component if it exists
	InitializeOriginComponent(InViewport, ViosoConfigData.OriginCompId);

	const int32 NumViews = InViewport->GetConfiguration().GetViewPerViewportAmount();

	PolicyViewData.Reset();
	for (int32 ViewIndex = 0; ViewIndex < NumViews; ++ViewIndex)
	{
		PolicyViewData.Add(MakeShared<FDisplayClusterProjectionVIOSOPolicyViewData>(VIOSOLibrary, ViosoConfigData, InViewport, ViewIndex));
	}

	UE_LOGF(LogDisplayClusterProjectionVIOSO, Verbose, "VIOSO policy has been initialized: %ls", *ViosoConfigData.ToString());

	return true;
}

void FDisplayClusterProjectionVIOSOPolicy::HandleEndScene(IDisplayClusterViewport* InViewport)
{
	check(IsInGameThread());

	ImplRelease();
}

void FDisplayClusterProjectionVIOSOPolicy::UpdateProxyData(IDisplayClusterViewport* InViewport)
{
	// Snapshot all per-view data before handing off to the render thread — game thread may mutate PolicyViewData after this point.
	TArray<TSharedRef<FDisplayClusterProjectionVIOSOPolicyViewData, ESPMode::ThreadSafe>> ViewDataSnapshot;
	for (const TSharedRef<FDisplayClusterProjectionVIOSOPolicyViewData, ESPMode::ThreadSafe>& ViewDataIt : PolicyViewData)
	{
		ViewDataSnapshot.Add(MakeShared<FDisplayClusterProjectionVIOSOPolicyViewData, ESPMode::ThreadSafe>(ViewDataIt.Get()));
	}

	// Enqueue proxy update so the render thread sees the new snapshots this frame.
	ENQUEUE_RENDER_COMMAND(DisplayClusterProjectionVIOSO_UpdateProxyData)(
		[InProjectionPolicy = SharedThis(this), ViewDataSnapshot](FRHICommandListImmediate& RHICmdList)
		{
			InProjectionPolicy->PolicyViewDataProxy = ViewDataSnapshot;
		});
}

void FDisplayClusterProjectionVIOSOPolicy::ImplRelease()
{
	ReleaseOriginComponent();

	PreviewMeshComponentRef.ResetSceneComponent();
	PolicyViewData.Reset();
}

bool FDisplayClusterProjectionVIOSOPolicy::CalculateView(IDisplayClusterViewport* InViewport, const uint32 InContextNum, FVector& InOutViewLocation, FRotator& InOutViewRotation, const FVector& ViewOffset, const float WorldToMeters, const float InNCP, const float InFCP)
{
	if (!PolicyViewData.IsValidIndex(InContextNum) || !InViewport)
	{
		return false;
	}
	TSharedRef<FDisplayClusterProjectionVIOSOPolicyViewData, ESPMode::ThreadSafe>& CurrentView = PolicyViewData[InContextNum];

	// Get view prj data from VIOSO
	if (!CurrentView->UpdateVIOSO(InViewport, InContextNum, InOutViewLocation, InOutViewRotation, WorldToMeters, InNCP, InFCP))
	{
		if (!FDisplayClusterProjectionPolicyBase::IsEditorOperationMode(InViewport))
		{
			// Vioso api used, but failed inside math. The config base matrix or vioso geometry is invalid
			UE_LOGF(LogDisplayClusterProjectionVIOSO, Error, "Couldn't Calculate View for VIOSO viewport '%ls'", *InViewport->GetId());
		}

		return false;
	}

	// Transform rotation to world space
	InOutViewRotation = CurrentView->ViewRotation;
	InOutViewLocation = CurrentView->ViewLocation;

	return true;
}

bool FDisplayClusterProjectionVIOSOPolicy::GetProjectionMatrix(IDisplayClusterViewport* InViewport, const uint32 InContextNum, FMatrix& OutPrjMatrix)
{
	check(IsInGameThread());

	if (!PolicyViewData.IsValidIndex(InContextNum))
	{
		return false;
	}
	TSharedRef<FDisplayClusterProjectionVIOSOPolicyViewData, ESPMode::ThreadSafe>& CurrentView = PolicyViewData[InContextNum];

	OutPrjMatrix = CurrentView->ProjectionMatrix;
	
	return true;
}

bool FDisplayClusterProjectionVIOSOPolicy::IsWarpBlendSupported(const IDisplayClusterViewport* InViewport) const
{
	return true;
}

bool FDisplayClusterProjectionVIOSOPolicy::IsWarpBlendSupported_RenderThread(const IDisplayClusterViewportProxy* InViewportProxy) const
{
	return true;
}

void FDisplayClusterProjectionVIOSOPolicy::ApplyWarpBlend_RenderThread(FRHICommandListImmediate& RHICmdList, const IDisplayClusterViewportProxy* InViewportProxy)
{
	check(IsInRenderingThread());

	if (!ImplApplyWarpBlend_RenderThread(RHICmdList, InViewportProxy))
	{
		// warp failed, just resolve texture to frame
		InViewportProxy->ResolveResources_RenderThread(RHICmdList, EDisplayClusterViewportResourceType::InputShaderResource, EDisplayClusterViewportResourceType::OutputTargetableResource);
	}
}

bool FDisplayClusterProjectionVIOSOPolicy::ImplApplyWarpBlend_RenderThread(FRHICommandListImmediate& RHICmdList, const IDisplayClusterViewportProxy* InViewportProxy)
{
	check(IsInRenderingThread());

	// Get in\out remp resources ref from viewport
	TArray<FRHITexture*> InputTextures, OutputTextures;

	// Use for input first MipsShader texture if enabled in viewport render settings
	//@todo: test if domeprojection support mips textures as warp input
	//if (!InViewportProxy->GetResources_RenderThread(EDisplayClusterViewportResourceType::MipsShaderResource, InputTextures))
	{
		// otherwise inputshader texture
		if (!InViewportProxy->GetResources_RenderThread(EDisplayClusterViewportResourceType::InputShaderResource, InputTextures))
		{
			// no source textures
			return false;
		}
	}

	// Get output resources with rects
	// warp result is now inside AdditionalRTT.  Later, from the DC ViewportManagerProxy it will be resolved to FrameRTT 
	if (!InViewportProxy->GetResources_RenderThread(EDisplayClusterViewportResourceType::AfterWarpBlendTargetableResource, OutputTextures))
	{
		return false;
	}

	// All parallel arrays (output textures, contexts, proxy view data) must have at least as many entries as input textures.
	const int32 ContextAmmount = InputTextures.Num();
	if(!OutputTextures.IsValidIndex(ContextAmmount-1)
	|| !InViewportProxy->GetContexts_RenderThread().IsValidIndex(ContextAmmount-1)
	|| !PolicyViewDataProxy.IsValidIndex(ContextAmmount-1))
	{
		return false;
	}

	// External SDK not use our RHI flow, call flush to finish resolve context image to input resource
	RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThreadFlushResources);

	TRACE_CPUPROFILER_EVENT_SCOPE(nDisplay VIOSO::Render);
	{
		for (int32 ContextNum = 0; ContextNum < ContextAmmount; ContextNum++)
		{
			if (!PolicyViewDataProxy[ContextNum]->RenderVIOSO_RenderThread(RHICmdList, InputTextures[ContextNum], OutputTextures[ContextNum]))
			{
				return false;
			}
		}
	}

	// warp result is now inside AdditionalRTT.  Later, from the DC ViewportManagerProxy it will be resolved to FrameRTT 
	return true;
}

bool FDisplayClusterProjectionVIOSOPolicy::HasPreviewMesh(IDisplayClusterViewport* InViewport)
{
	if (!ViosoConfigData.bIsPreviewMeshEnabled || PolicyViewData.IsEmpty() || !PolicyViewData[0]->IsWarperInterfaceValid())
	{
		PreviewMeshComponentRef.ResetSceneActor();

		return false;
	}

	return true;
}

UMeshComponent* FDisplayClusterProjectionVIOSOPolicy::GetOrCreatePreviewMeshComponent(IDisplayClusterViewport* InViewport, bool& bOutIsRootActorComponent)
{
	if (!HasPreviewMesh(InViewport))
	{
		return nullptr;
	}

	// Create a new DCRA mesh component
	bOutIsRootActorComponent = false;

	// If we have already created a preview mesh component before, return that component
	if (UMeshComponent* ExistsPreviewMeshComp = Cast<UMeshComponent>(PreviewMeshComponentRef.GetOrFindSceneComponent()))
	{
		return ExistsPreviewMeshComp;
	}

	USceneComponent* OriginComp = GetPreviewMeshOriginComponent(InViewport);
	TSharedPtr<FDisplayClusterProjectionVIOSOGeometryExportData, ESPMode::ThreadSafe> GeometryExportData = FDisplayClusterProjectionVIOSOGeometryExportData::Create(VIOSOLibrary, ViosoConfigData);
	if (OriginComp && GeometryExportData.IsValid())
	{
		// Create new WarpMesh component
		const FString CompName = FString::Printf(TEXT("VIOSO_%s_impl"), *GetId());

		// Creta new object
		UProceduralMeshComponent* MeshComp = NewObject<UProceduralMeshComponent>(OriginComp, FName(*CompName), EObjectFlags::RF_DuplicateTransient | RF_Transient | RF_TextExportTransient);
		if (MeshComp)
		{
			MeshComp->RegisterComponent();
			MeshComp->AttachToComponent(OriginComp, FAttachmentTransformRules(EAttachmentRule::KeepRelative, false));
			MeshComp->CreateMeshSection(0, GeometryExportData->Vertices, GeometryExportData->Triangles, GeometryExportData->Normal, GeometryExportData->UV, TArray<FColor>(), TArray<FProcMeshTangent>(), false);

#if WITH_EDITOR
			MeshComp->SetIsVisualizationComponent(true);
#endif

			// Because of "nDisplay.render.show.visualizationcomponents" we need extra flag to exclude this geometry from render
			MeshComp->SetHiddenInGame(true);

			// Store reference to mesh component
			PreviewMeshComponentRef.SetSceneComponent(MeshComp);

			return MeshComp;
		}
	}

	return nullptr;
}
