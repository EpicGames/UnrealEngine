// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Policy/DisplayClusterProjectionPolicyBase.h"
#include "Policy/VIOSO/Windows/DisplayClusterProjectionVIOSOPolicyViewData.h"
#include "Templates/SharedPointer.h"

/**
 * VIOSO projection policy
 */
class FDisplayClusterProjectionVIOSOPolicy
	: public FDisplayClusterProjectionPolicyBase
	, public TSharedFromThis<FDisplayClusterProjectionVIOSOPolicy, ESPMode::ThreadSafe>
{
public:
	FDisplayClusterProjectionVIOSOPolicy(const FString& ProjectionPolicyId, const FDisplayClusterConfigurationProjection* InConfigurationProjectionPolicy, const TSharedRef<FDisplayClusterProjectionVIOSOLibrary, ESPMode::ThreadSafe>& InVIOSOLibrary);
	virtual ~FDisplayClusterProjectionVIOSOPolicy();

public:
	//~~BEGIN IDisplayClusterProjectionPolicy
	virtual const FString& GetType() const override;

	virtual bool HandleStartScene(IDisplayClusterViewport* InViewport) override;
	virtual void HandleEndScene(IDisplayClusterViewport* InViewport) override;

	virtual bool CalculateView(IDisplayClusterViewport* InViewport, const uint32 InContextNum, FVector& InOutViewLocation, FRotator& InOutViewRotation, const FVector& ViewOffset, const float WorldToMeters, const float NCP, const float FCP) override;
	virtual bool GetProjectionMatrix(IDisplayClusterViewport* InViewport, const uint32 InContextNum, FMatrix& OutPrjMatrix) override;

	virtual bool IsWarpBlendSupported(const IDisplayClusterViewport* InViewport) const override;
	virtual bool IsWarpBlendSupported_RenderThread(const IDisplayClusterViewportProxy* InViewportProxy) const override;

	virtual void ApplyWarpBlend_RenderThread(FRHICommandListImmediate& RHICmdList, const IDisplayClusterViewportProxy* InViewportProxy) override;

	// Request additional targetable resources for domeprojection  external warpblend
	virtual bool ShouldUseAdditionalTargetableResource(const IDisplayClusterViewport* InViewport) const override
	{
		return true;
	}

	virtual bool ShouldUseSourceTextureWithMips(const IDisplayClusterViewport* InViewport) const override
	{
		return true;
	}

	virtual bool HasPreviewMesh(IDisplayClusterViewport* InViewport) override;
	virtual class UMeshComponent* GetOrCreatePreviewMeshComponent(IDisplayClusterViewport* InViewport, bool& bOutIsRootActorComponent) override;

	virtual float GetGeometryToMeters(const IDisplayClusterViewport* InViewport, const float InWorldToMeters) const override
	{
		return ViosoConfigData.UnitsInMeter;
	}

	virtual EDisplayClusterProjectionPolicyFlags GetProjectionPolicyFlags(const IDisplayClusterViewport* InViewport, const uint32 InContextNum) const override
	{
		return EDisplayClusterProjectionPolicyFlags::EnableFollowCameraFeature
			| EDisplayClusterProjectionPolicyFlags::UseEyeTracking
			| EDisplayClusterProjectionPolicyFlags::UseLocalSpace;
	}

	virtual void UpdateProxyData(IDisplayClusterViewport* InViewport) override;

	//~~END IDisplayClusterProjectionPolicy

protected:
	bool ImplApplyWarpBlend_RenderThread(FRHICommandListImmediate& RHICmdList, const IDisplayClusterViewportProxy* InViewportProxy);
	void ImplRelease();

protected:
	// VIOSO DLL api
	const TSharedRef<FDisplayClusterProjectionVIOSOLibrary, ESPMode::ThreadSafe> VIOSOLibrary;

	FViosoPolicyConfiguration ViosoConfigData;

	// View data owned and updated on the game thread
	TArray<TSharedRef<FDisplayClusterProjectionVIOSOPolicyViewData, ESPMode::ThreadSafe>> PolicyViewData;

	// Snapshot of PolicyViewData pushed to the render thread each frame via UpdateProxyData()
	TArray<TSharedRef<FDisplayClusterProjectionVIOSOPolicyViewData, ESPMode::ThreadSafe>> PolicyViewDataProxy;

	FDisplayClusterSceneComponentRef PreviewMeshComponentRef;
};
