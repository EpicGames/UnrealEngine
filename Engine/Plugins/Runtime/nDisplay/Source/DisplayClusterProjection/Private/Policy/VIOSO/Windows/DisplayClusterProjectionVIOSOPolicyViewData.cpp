// Copyright Epic Games, Inc. All Rights Reserved.

#include "Policy/VIOSO/Windows/DisplayClusterProjectionVIOSOPolicyViewData.h"

#include "DisplayClusterProjectionStrings.h"
#include "DisplayClusterProjectionLog.h"

#include "IDisplayCluster.h"
#include "Misc/DisplayClusterHelpers.h"
#include "Config/IDisplayClusterConfigManager.h"
#include "Game/IDisplayClusterGameManager.h"

#include "Misc/Paths.h"
#include "Misc/ScopeLock.h"
#include "Engine/RendererSettings.h"

#include "Render/Viewport/IDisplayClusterViewportManager.h"
#include "Render/Viewport/IDisplayClusterViewport.h"
#include "Render/Viewport/IDisplayClusterViewportProxy.h"

#define VIOSO_USE_GRAPHICS_API_D3D11	1
#define VIOSO_USE_GRAPHICS_API_D3D12	1


//-------------------------------------------------------------------------------------------------
// D3D11
//-------------------------------------------------------------------------------------------------
#if VIOSO_USE_GRAPHICS_API_D3D11
#include "ID3D11DynamicRHI.h"
#endif // VIOSO_USE_GRAPHICS_API_D3D11

//-------------------------------------------------------------------------------------------------
// D3D12
//-------------------------------------------------------------------------------------------------
#if VIOSO_USE_GRAPHICS_API_D3D12
#include "ID3D12DynamicRHI.h"
#endif // VIOSO_USE_GRAPHICS_API_D3D12

#if VIOSO_USE_GRAPHICS_API_D3D11
/**
 * Class for caching D3D11 changed values on scope
 */
class FD3D11ContextHelper
{
public:
	/** Initialization constructor: requires the device context. */
	FD3D11ContextHelper()
	{
		FMemory::Memzero(RenderTargetViews, sizeof(RenderTargetViews));
		FMemory::Memzero(Viewports, sizeof(Viewports));
		DepthStencilView = NULL;


		ID3D11DynamicRHI* D3D11RHI = GDynamicRHI ? GetDynamicRHI<ID3D11DynamicRHI>() : nullptr;
		DeviceContext = D3D11RHI ? D3D11RHI->RHIGetDeviceContext() : nullptr;

		if (DeviceContext)
		{
			ViewportsNum = MaxSimultaneousRenderTargets;
			DeviceContext->OMGetRenderTargets(MaxSimultaneousRenderTargets, &RenderTargetViews[0], &DepthStencilView);
			DeviceContext->RSGetViewports(&ViewportsNum, &Viewports[0]);
		}
	}

	/** Destructor. */
	~FD3D11ContextHelper()
	{
		if (DeviceContext)
		{
			// Flush tail commands
			DeviceContext->Flush();

			// Restore
			DeviceContext->OMSetRenderTargets(MaxSimultaneousRenderTargets, &RenderTargetViews[0], DepthStencilView);
			DeviceContext->RSSetViewports(ViewportsNum, &Viewports[0]);
		}

		// OMGetRenderTargets calls AddRef on each RTV/DSV it returns. We need
		// to make a corresponding call to Release.
		for (int32 TargetIndex = 0; TargetIndex < MaxSimultaneousRenderTargets; ++TargetIndex)
		{
			if (RenderTargetViews[TargetIndex] != nullptr)
			{
				RenderTargetViews[TargetIndex]->Release();
			}
		}

		if (DepthStencilView)
		{
			DepthStencilView->Release();
		}
	}

	inline bool AssignD3D11RenderTarget(FRHITexture* RenderTargetTexture)
	{
		if (DeviceContext)
		{
			// Set RTV
			ID3D11RenderTargetView* DestTextureRTV = GetID3D11DynamicRHI()->RHIGetRenderTargetView(RenderTargetTexture, 0, -1);
			DeviceContext->OMSetRenderTargets(1, &DestTextureRTV, nullptr);

			// Set viewport
			D3D11_VIEWPORT RenderViewportData;
			RenderViewportData.MinDepth = 0.0f;
			RenderViewportData.MaxDepth = 1.0f;
			RenderViewportData.TopLeftX = 0.0f;
			RenderViewportData.TopLeftY = 0.0f;
			RenderViewportData.Width = RenderTargetTexture->GetSizeX();
			RenderViewportData.Height = RenderTargetTexture->GetSizeY();
			DeviceContext->RSSetViewports(1, &RenderViewportData);

			// Clear RTV
			static FVector4f ClearColor(0, 0, 0, 1);
			DeviceContext->ClearRenderTargetView(DestTextureRTV, &ClearColor[0]);

			DeviceContext->Flush();

			return true;
		}

		return false;
	};

private:
	ID3D11DeviceContext*    DeviceContext;
	ID3D11RenderTargetView* RenderTargetViews[MaxSimultaneousRenderTargets];
	ID3D11DepthStencilView* DepthStencilView;
	D3D11_VIEWPORT          Viewports[MaxSimultaneousRenderTargets];
	uint32                  ViewportsNum;
};
#endif // VIOSO_USE_GRAPHICS_API_D3D11

//////////////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterProjectionVIOSOPolicyViewData
//////////////////////////////////////////////////////////////////////////////////////////////
FDisplayClusterProjectionVIOSOPolicyViewData::FDisplayClusterProjectionVIOSOPolicyViewData(const TSharedRef<FDisplayClusterProjectionVIOSOLibrary, ESPMode::ThreadSafe>& InVIOSOLibrary, const FViosoPolicyConfiguration& InConfigData, IDisplayClusterViewport* InViewport, const int32 InContextNum)
#if WITH_VIOSO_LIBRARY
	: WarperInterface(FDisplayClusterProjectionVIOSOWarper::Create(InVIOSOLibrary, InConfigData, FString::Printf(TEXT("%s:%d") , InViewport? *InViewport->GetId() : TEXT("none"), InContextNum)))
#endif
{
	const ERHIInterfaceType RHIType = RHIGetInterfaceType();

	// Create warper for view
	if (RHIType == ERHIInterfaceType::D3D11)
	{
		RenderDevice = ERenderDevice::D3D11;
	}
	else if (RHIType == ERHIInterfaceType::D3D12)
	{
		RenderDevice = ERenderDevice::D3D12;
	}
	else
	{
		UE_LOGF(LogDisplayClusterProjectionVIOSO, Error, "VIOSO warp projection not supported by '%ls' rhi", GDynamicRHI->GetName());
	}
}

bool FDisplayClusterProjectionVIOSOPolicyViewData::IsWarperInterfaceValid()
{
#if WITH_VIOSO_LIBRARY
	return WarperInterface->IsInitialized();
#else
	return false;
#endif
}

bool FDisplayClusterProjectionVIOSOPolicyViewData::UpdateVIOSO(IDisplayClusterViewport* InViewport, const uint32 InContextNum, const FVector& LocalLocation, const FRotator& LocalRotator, const float WorldToMeters, const float NCP, const float FCP)
{
#if WITH_VIOSO_LIBRARY
	// Save used clipping planes
	ClippingPlanes.Set(NCP, FCP);


	if (IsViewDataValid())
	{
		ViewLocation = LocalLocation;
		ViewRotation = LocalRotator;

		// Update ClippingPlanes
		WarperInterface->UpdateClippingPlanes(ClippingPlanes);

		return WarperInterface->CalculateViewProjection(InViewport, InContextNum, ViewLocation, ViewRotation, ProjectionMatrix, WorldToMeters, NCP, FCP);
	}
#endif

	return false;
}

bool FDisplayClusterProjectionVIOSOPolicyViewData::RenderVIOSO_RenderThread(FRHICommandListImmediate& RHICmdList, FRHITexture* ShaderResourceTexture, FRHITexture* RenderTargetTexture)
{
	check(IsInRenderingThread());

#if WITH_VIOSO_LIBRARY

	// make sure that VIOSO is initialized in the rendering thread
	InitializeVIOSO_RenderThread(RHICmdList, RenderTargetTexture);

	// Delayed vioso initialize on render thread
	if (IsViewDataValid())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(nDisplay Vioso::Render);

		switch (RenderDevice)
		{
#ifdef VIOSO_USE_GRAPHICS_API_D3D11
		case ERenderDevice::D3D11:
		{
			FD3D11ContextHelper D3D11ContextHelper;
			ID3D11Texture2D* SourceTexture = static_cast<ID3D11Texture2D*>(ShaderResourceTexture->GetTexture2D()->GetNativeResource());
			if (D3D11ContextHelper.AssignD3D11RenderTarget(RenderTargetTexture) && WarperInterface->Render(SourceTexture, VWB_STATEMASK_ALL))
			{
				return true;
			}
			break;
		}
#endif //VIOSO_USE_GRAPHICS_API_D3D11

#ifdef VIOSO_USE_GRAPHICS_API_D3D12
		case ERenderDevice::D3D12:
		{
			// Insert an outer query that encloses the whole batch
			//experimental: add resource barrier
			RHICmdList.Transition(FRHITransitionInfo(RenderTargetTexture, ERHIAccess::Unknown, ERHIAccess::RTV));
			RHICmdList.EnqueueLambda([ViosoViewData = SharedThis(this), RenderTargetTexture = RenderTargetTexture, ShaderResourceTexture= ShaderResourceTexture](FRHICommandList& ExecutingCmdList)
				{
					ID3D12DynamicRHI* D3D12RHI = GetID3D12DynamicRHI();
					const D3D12_CPU_DESCRIPTOR_HANDLE RTVHandle = D3D12RHI->RHIGetRenderTargetView(RenderTargetTexture);

					VWB_D3D12_RENDERINPUT RenderInput = {};
					RenderInput.textureResource = D3D12RHI->RHIGetResource(ShaderResourceTexture);
					RenderInput.renderTarget = D3D12RHI->RHIGetResource(RenderTargetTexture);
					RenderInput.rtvHandlePtr = RTVHandle.ptr;

					ViosoViewData->WarperInterface->Render(&RenderInput, VWB_STATEMASK_ALL);
				});
			//experimental: add resource barrier
			RHICmdList.Transition(FRHITransitionInfo(RenderTargetTexture, ERHIAccess::RTV, ERHIAccess::Present));

			return true;
		}
#endif // VIOSO_USE_GRAPHICS_API_D3D12

		default:
			break;
		}
	}

#endif

	return false;
}

bool FDisplayClusterProjectionVIOSOPolicyViewData::InitializeVIOSO_RenderThread(FRHICommandListImmediate& RHICmdList, FRHITexture* RenderTargetTexture)
{
#if WITH_VIOSO_LIBRARY

	if ((WarperInterface->IsInitialized() && bInitialized) && RenderTargetTexture == UsedRenderTargetTexture)
	{
		return true;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(nDisplay Vioso::Initialize);

	// RTT changed: VIOSO reinitialization
	UsedRenderTargetTexture = RenderTargetTexture;

	bInitialized = false;
	WarperInterface->Release();

	switch (RenderDevice)
	{

#ifdef VIOSO_USE_GRAPHICS_API_D3D11
	case ERenderDevice::D3D11:
	{
		if (ID3D11DynamicRHI* D3D11DynamicRHI = GetID3D11DynamicRHI())
		{
			if (ID3D11Device* D3D11Device = D3D11DynamicRHI->RHIGetDevice())
			{
				FD3D11ContextHelper D3D11ContextHelper;
				if (D3D11ContextHelper.AssignD3D11RenderTarget(RenderTargetTexture) && WarperInterface->Initialize(D3D11Device, ClippingPlanes))
				{
					bInitialized = true;

					return true;
				}
			}
		}
		break;
	}
#endif // VIOSO_USE_GRAPHICS_API_D3D11

#if VIOSO_USE_GRAPHICS_API_D3D12
	case ERenderDevice::D3D12:
	{
		if (ID3D12DynamicRHI* D3D12DynamicRHI = GetID3D12DynamicRHI())
		{
			if (ID3D12CommandQueue* D3D12CommandQueue = D3D12DynamicRHI->RHIGetCommandQueue())
			{
				if (WarperInterface->Initialize(D3D12CommandQueue, ClippingPlanes))
				{
					bInitialized = true;

					return true;
				}
			}
		}
		break;
	}
#endif // VIOSO_USE_GRAPHICS_API_D3D12

	default:
		UE_LOGF(LogDisplayClusterProjectionVIOSO, Error, "Unsupported render device for VIOSO");
		break;
	}

	UE_LOGF(LogDisplayClusterProjectionVIOSO, Error, "Couldn't initialize VIOSO internals");
#endif 

	return false;
}
