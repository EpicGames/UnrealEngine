// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Policy/VIOSO/Windows/DisplayClusterProjectionVIOSOWarper.h"

#include "RHI.h"
#include "RHICommandList.h"
#include "RHIResources.h"

#include "Templates/SharedPointer.h"

class FDisplayClusterProjectionVIOSOLibrary;
struct FViosoPolicyConfiguration;

/**
 * VIOSO projection policy data
 */
struct FDisplayClusterProjectionVIOSOPolicyViewData
	: public TSharedFromThis<FDisplayClusterProjectionVIOSOPolicyViewData, ESPMode::ThreadSafe>
{
	enum class ERenderDevice : uint8
	{
		Unsupported = 0,
		D3D11,
		D3D12
	};

public:
	FDisplayClusterProjectionVIOSOPolicyViewData(const TSharedRef<FDisplayClusterProjectionVIOSOLibrary, ESPMode::ThreadSafe>& InVIOSOLibrary, const FViosoPolicyConfiguration& InConfigData, IDisplayClusterViewport* InViewport, const int32 InContextNum);
	virtual ~FDisplayClusterProjectionVIOSOPolicyViewData() = default;

	FDisplayClusterProjectionVIOSOPolicyViewData(const FDisplayClusterProjectionVIOSOPolicyViewData& In)
		: ViewLocation(In.ViewLocation)
		, ViewRotation(In.ViewRotation)
		, ProjectionMatrix(In.ProjectionMatrix)
#if WITH_VIOSO_LIBRARY
		, WarperInterface(In.WarperInterface)
#endif
		, RenderDevice(In.RenderDevice)
		, bInitialized(In.bInitialized)
		, UsedRenderTargetTexture(In.UsedRenderTargetTexture)
		, ClippingPlanes(In.ClippingPlanes)
	{ }
	
public:
	bool IsWarperInterfaceValid();

	bool IsViewDataValid()
	{
		return bInitialized && IsWarperInterfaceValid();
	}

	bool UpdateVIOSO(IDisplayClusterViewport* InViewport, const uint32 InContextNum, const FVector& LocalLocation, const FRotator& LocalRotator, const float WorldToMeters, const float NCP, const float FCP);
	bool RenderVIOSO_RenderThread(FRHICommandListImmediate& RHICmdList, FRHITexture* ShaderResourceTexture, FRHITexture* RenderTargetTexture);

protected:
	bool InitializeVIOSO_RenderThread(FRHICommandListImmediate& RHICmdList, FRHITexture* RenderTargetTexture);

public:
	FVector  ViewLocation;
	FRotator ViewRotation;
	FMatrix  ProjectionMatrix;

private:
#if WITH_VIOSO_LIBRARY
	// VIOSO warper interface
	const TSharedRef<FDisplayClusterProjectionVIOSOWarper, ESPMode::ThreadSafe> WarperInterface;
#endif

	ERenderDevice RenderDevice = ERenderDevice::Unsupported;

	bool bInitialized = false;

	// This RTT is used for VIOSO. If we change this resource, we need to re-initialize VIOSO
	FRHITexture* UsedRenderTargetTexture = nullptr;

	FVector2D ClippingPlanes;
};
