// Copyright Epic Games, Inc. All Rights Reserved.

#include "Policy/VIOSO/Windows/DisplayClusterProjectionVIOSOWarper.h"
#include "Policy/VIOSO/DisplayClusterProjectionVIOSOLibrary.h"
#include "Misc/DisplayClusterHelpers.h"

#include "Render/Viewport/IDisplayClusterViewport.h"
#include "Misc/DisplayClusterDataCache.h"

#if WITH_VIOSO_LIBRARY

int32 GDisplayClusterRender_VIOSOWarperCacheEnable = 1;
static FAutoConsoleVariableRef CDisplayClusterRender_VIOSOWarperCacheEnable(
	TEXT("nDisplay.cache.VIOSO.Warper.enable"),
	GDisplayClusterRender_VIOSOWarperCacheEnable,
	TEXT("Enables the use of the VIOSO warper instances cache.\n"),
	ECVF_Default
);

int32 GDisplayClusterRender_VIOSOWarperCacheTimeOutInFrames = 5 * 60 * 60; // Timeout is 5 minutes (for 60 frames per second)
static FAutoConsoleVariableRef CVarDisplayClusterRender_VIOSOWarperCacheTimeOutInFrames(
	TEXT("nDisplay.cache.VIOSO.Warper.TimeOut"),
	GDisplayClusterRender_VIOSOWarperCacheTimeOutInFrames,
	TEXT("The timeout value in frames  for cached VIOSO warper instances.\n")
	TEXT("-1 - disable timeout.\n"),
	ECVF_Default
);

int32 GDisplayClusterProjectionVIOSOPolicyEnableCustomClippingPlane = 0;
static FAutoConsoleVariableRef CVarDisplayClusterProjectionVIOSOPolicyEnableCustomClippingPlane(
	TEXT("nDisplay.render.projection.VIOSO.EnableCustomClippingPlanes"),
	GDisplayClusterProjectionVIOSOPolicyEnableCustomClippingPlane,
	TEXT("Enable custom clipping planes for VIOSO (0 - disable).\n"),
	ECVF_RenderThreadSafe
);

int32 GDisplayClusterProjectionVIOSOPolicyFrustumFitToViewport = 0;
static FAutoConsoleVariableRef CVarDisplayClusterProjectionVIOSOPolicyFrustumFitToViewport(
	TEXT("nDisplay.render.projection.VIOSO.FitFrustumToViewport"),
	GDisplayClusterProjectionVIOSOPolicyFrustumFitToViewport,
	TEXT("Enable frustum fit to viewport size (0 - off).\n"),
	ECVF_RenderThreadSafe
);

int32 GDisplayClusterProjectionVIOSOPolicySymmetricFrustum = 0;
static FAutoConsoleVariableRef CVarDisplayClusterProjectionVIOSOPolicySymmetricFrustum(
	TEXT("nDisplay.render.projection.VIOSO.SymmetricFrustum"),
	GDisplayClusterProjectionVIOSOPolicySymmetricFrustum,
	TEXT("Enable symmetric frustum (0 - off).\n"),
	ECVF_RenderThreadSafe
);

int32 GDisplayClusterProjectionVIOSOPolicyPitchInverse = 1;
static FAutoConsoleVariableRef CVarDisplayClusterProjectionVIOSOPolicyPitchInverse(
	TEXT("nDisplay.render.projection.VIOSO.Pitch.Inverse"),
	GDisplayClusterProjectionVIOSOPolicyPitchInverse,
	TEXT("Inverse Pitch (default 1).\n"),
	ECVF_RenderThreadSafe
);

int32 GDisplayClusterProjectionVIOSOPolicyYawInverse = 0;
static FAutoConsoleVariableRef CVarDisplayClusterProjectionVIOSOPolicyYawInverse(
	TEXT("nDisplay.render.projection.VIOSO.Yaw.Inverse"),
	GDisplayClusterProjectionVIOSOPolicyYawInverse,
	TEXT("Inverse Yaw (default 1).\n"),
	ECVF_RenderThreadSafe
);

int32 GDisplayClusterProjectionVIOSOPolicyRollInverse = 0;
static FAutoConsoleVariableRef CVarDisplayClusterProjectionVIOSOPolicyRollInverse(
	TEXT("nDisplay.render.projection.VIOSO.Roll.Inverse"),
	GDisplayClusterProjectionVIOSOPolicyRollInverse,
	TEXT("Inverse Roll (default 0).\n"),
	ECVF_RenderThreadSafe
);

int32 GDisplayClusterProjectionVIOSOPolicyPitchAxis = 0;
static FAutoConsoleVariableRef CVarDisplayClusterProjectionVIOSOPolicyPitchAxis(
	TEXT("nDisplay.render.projection.VIOSO.Pitch.Axis"),
	GDisplayClusterProjectionVIOSOPolicyPitchAxis,
	TEXT("The source axis for Pitch (Default 0).\n")
	TEXT("0 - X Axis\n")
	TEXT("1 - Y Axis\n")
	TEXT("2 - Z Axis\n"),
	ECVF_RenderThreadSafe
);

int32 GDisplayClusterProjectionVIOSOPolicyYawAxis = 1;
static FAutoConsoleVariableRef CVarDisplayClusterProjectionVIOSOPolicyYawAxis(
	TEXT("nDisplay.render.projection.VIOSO.Yaw.Axis"),
	GDisplayClusterProjectionVIOSOPolicyYawAxis,
	TEXT("The source axis for Yaw (Default 1).\n")
	TEXT("0 - X Axis\n")
	TEXT("1 - Y Axis\n")
	TEXT("2 - Z Axis\n"),
	ECVF_RenderThreadSafe
);

int32 GDisplayClusterProjectionVIOSOPolicyRollAxis = 2;
static FAutoConsoleVariableRef CVarDisplayClusterProjectionVIOSOPolicyRollAxis(
	TEXT("nDisplay.render.projection.VIOSO.Roll.Axis"),
	GDisplayClusterProjectionVIOSOPolicyRollAxis,
	TEXT("The source axis for Roll (Default 2).\n")
	TEXT("0 - X Axis\n")
	TEXT("1 - Y Axis\n")
	TEXT("2 - Z Axis\n"),
	ECVF_RenderThreadSafe
);

namespace UE::DisplayClusterProjection::VIOSOWarper
{
	// Convert from VIOSO to Unreal
	// DirectX left-handed camera-like coordinate system +X = right, +Y= up, +Z front
	// Unreal is Left Handed (Z is up, X in the screen, Y is right)
	static FMatrix FromViosoConventionMatrix(
		FPlane( 0.f,  1.f,  0.f,  0.f),
		FPlane( 0.f,  0.f,  1.f,  0.f),
		FPlane( 1.f,  0.f,  0.f,  0.f),
		FPlane( 0.f,  0.f,  0.f,  1.f));

	static FTransform ViosoToUETransform(FromViosoConventionMatrix);

	static inline FVector3f ToVioso(const FVector& InPos, const float WorldToMeters)
	{
		float ScaleIn = (1.f / WorldToMeters);
		const FVector Pos = ViosoToUETransform.InverseTransformPosition(InPos) * ScaleIn;

		return FVector3f(Pos.X, Pos.Y, Pos.Z);
	}

	static inline FVector FromVioso(const FVector3f& InPos, const float WorldToMeters)
	{
		float ScaleOut = WorldToMeters;
		FVector UELocation = ViosoToUETransform.TransformPosition(FVector(InPos)) * ScaleOut;

		return UELocation;
	}

	static inline constexpr EAxis::Type GetViosoAxis(const int32 InValue)
	{
		switch (InValue)
		{
		case 1:
			return EAxis::Y;
		case 2:
			return EAxis::Z;

		default:
			break;
		}

		return EAxis::X;
	}
};
using namespace UE::DisplayClusterProjection;

//////////////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterProjectionVIOSOWarper
//////////////////////////////////////////////////////////////////////////////////////////////
bool FDisplayClusterProjectionVIOSOWarper::CalculateViewProjection(IDisplayClusterViewport* InViewport, const uint32 InContextNum, FVector& InOutViewLocation, FRotator& InOutViewRotation, FMatrix& OutProjMatrix, const float WorldToMeters, const float NCP, const float FCP)
{
	// Convert to vioso space:
	FVector3f InOutEye = VIOSOWarper::ToVioso(InOutViewLocation, WorldToMeters);
	FVector3f InOutRot = FVector3f::ZeroVector;

	FVector3f OutPos = FVector3f::ZeroVector;
	FVector3f OutDir = FVector3f::ZeroVector;
	FRotator CamDir = FRotator::ZeroRotator;
	
	const bool bSymmetric = GDisplayClusterProjectionVIOSOPolicySymmetricFrustum != 0;
	const FIntPoint ViewportSize = InViewport->GetContexts()[InContextNum].ContextSize;
	const float ViewportAspectRatio = GDisplayClusterProjectionVIOSOPolicyFrustumFitToViewport ? float(ViewportSize.X) / float(ViewportSize.Y) : 0;

	// Protect VIOSO SDK calls: the DLL is not thread-safe.
	FScopeLock lock(&VIOSOLibrary->DllAccessCS);

	if (VWB_ERROR_NONE == VIOSOLibrary->VWB_getPosDirClip(pWarper, &InOutEye.X, &InOutRot.X, &OutPos.X, &OutDir.X, &VWB_ViewClip[0], bSymmetric, ViewportAspectRatio))
	{
		//Direction values in euler degrees - directly from the Warper - Roll inverted
		CamDir.Pitch	=	pWarper->dir[0];
		CamDir.Yaw		=	pWarper->dir[1];
		CamDir.Roll		= -	pWarper->dir[2];

		InOutViewRotation = CamDir;
		OutProjMatrix = GetProjMatrix(InViewport, InContextNum);

		return true;
	}
	return false;
}

FMatrix FDisplayClusterProjectionVIOSOWarper::GetProjMatrix(IDisplayClusterViewport* InViewport, const uint32 InContextNum) const
{
	FViewClip PrjClip = ViewClip;

	PrjClip.Left   = -ViewClip.Left;
	PrjClip.Right  = PrjClip.Left + (ViewClip.Left + ViewClip.Right);
	PrjClip.Bottom = -ViewClip.Bottom;
	PrjClip.Top    = PrjClip.Bottom + (ViewClip.Top + ViewClip.Bottom);

	// The nDisplay never use FCP
	InViewport->CalculateProjectionMatrix(InContextNum, PrjClip.Left, PrjClip.Right, PrjClip.Top, PrjClip.Bottom, PrjClip.NCP, PrjClip.NCP, false);

	return InViewport->GetContexts()[InContextNum].ProjectionMatrix;
}

bool FDisplayClusterProjectionVIOSOWarper::ExportGeometry(FDisplayClusterProjectionVIOSOGeometryExportData& OutMeshData)
{
	bool bResult = false;

	if (Initialize(VWB_DUMMYDEVICE, FVector2D(GNearClippingPlane, GNearClippingPlane)))
	{
		// Protect VIOSO SDK calls: the DLL is not thread-safe.
		FScopeLock lock(&VIOSOLibrary->DllAccessCS);

		VWB_WarpBlendMesh PreviewMesh;
		if (VWB_ERROR_NONE == VIOSOLibrary->VWB_getWarpBlendMesh(pWarper, VIOSOConfigData.PreviewMeshWidth, VIOSOConfigData.PreviewMeshHeight, PreviewMesh))
		{
			const float WorldToMeters = 100.0f;

			OutMeshData.Vertices.AddZeroed(PreviewMesh.nVtx);
			OutMeshData.UV.AddZeroed(PreviewMesh.nVtx);
			for (int32 VertexIndex = 0; VertexIndex < (int32)PreviewMesh.nVtx; VertexIndex++)
			{
				VWB_WarpBlendVertex& SrcVertex = PreviewMesh.vtx[VertexIndex];

				const FVector Pts = VIOSOWarper::FromVioso(FVector3f(SrcVertex.pos[0], SrcVertex.pos[1], SrcVertex.pos[2]), WorldToMeters);

				OutMeshData.Vertices[VertexIndex] = Pts;
				OutMeshData.UV[VertexIndex] = FVector2D(SrcVertex.uv[0], SrcVertex.uv[1]);
			}

			OutMeshData.Triangles.AddZeroed(PreviewMesh.nIdx);
			for (int32 TriangleIndex = 0; TriangleIndex < (int32)PreviewMesh.nIdx; TriangleIndex++)
			{
				OutMeshData.Triangles[TriangleIndex] = PreviewMesh.idx[TriangleIndex];
			}

			// Generate normals:
			OutMeshData.GenerateVIOSOGeometryNormals();

			VIOSOLibrary->VWB_destroyWarpBlendMesh(pWarper, PreviewMesh);

			bResult = true;
		}

		Release();
	}

	return bResult;
}

bool FDisplayClusterProjectionVIOSOWarper::Render(VWB_param RenderParam, VWB_uint StateMask)
{
	if (IsInitialized())
	{
		// Protect VIOSO SDK calls: the DLL is not thread-safe.
		FScopeLock lock(&VIOSOLibrary->DllAccessCS);

		if (VWB_ERROR_NONE == VIOSOLibrary->VWB_render(pWarper, RenderParam, StateMask))
		{
			return true;
		}
	}

	return false;
}

void FDisplayClusterProjectionVIOSOWarper::UpdateClippingPlanes(const FVector2D& InClippingPlanes)
{
	if (pWarper)
	{
		// Use clipping planes from calibration system or from UE
		const FVector2D& NewClippingPlanes = GDisplayClusterProjectionVIOSOPolicyEnableCustomClippingPlane ? DefaultClippingPlanes : InClippingPlanes;

		/// the near plane distance
		/// the far plane distance, note: these values are used to create the projection matrix
		pWarper->nearDist = NewClippingPlanes.X;
		pWarper->farDist = NewClippingPlanes.Y;
	}
}

bool FDisplayClusterProjectionVIOSOWarper::Initialize(void* pDxDevice, const FVector2D& InClippingPlanes)
{
	if (bInitialized)
	{
		// initialize once
		return true;
	}

	if (pDxDevice)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(nDisplay FDisplayClusterProjectionVIOSOWarper::Initialize);

		if (VIOSOConfigData.INIFile.IsEmpty())
		{
			// Protect VIOSO SDK calls: the DLL is not thread-safe.
			FScopeLock lock(&VIOSOLibrary->DllAccessCS);

			// Create without INI file
			if (VWB_ERROR_NONE != VIOSOLibrary->VWB_CreateA(pDxDevice, nullptr, nullptr, &pWarper, 0, nullptr))
			{
				// Failed to create default vioso container
				return false;
			}

			// Assign calib file
			FPlatformString::Strncpy(pWarper->calibFile, TCHAR_TO_ANSI(*VIOSOConfigData.CalibrationFile), UE_ARRAY_COUNT(VWB_Warper::calibFile));

			/// set to true to make the world turn and move with view direction and eye position, this is the case if the viewer gets
			/// moved by a motion platform, defaults to false
			pWarper->bTurnWithView = 0;

			/// set a moving range. This applies only for 3D mappings and dynamic eye point.
			/// This is a factor applied to the projector mapped MIN(width,height)/2
			/// The view plane is widened to cope with a movement to all sides, defaults to 1
			/// Check borderFit in log: 1 means all points stay on unwidened viewplane, 2 means, we had to double it.
			pWarper->autoViewC = 1;

			/// set to true to calculate view parameters while creating warper, defaults to false
			/// All further values are calculated/overwritten, if bAutoView is set.
			/// // set to true to calculate view parameters while creating warper, defaults to false
			pWarper->bAutoView = 1;

			/// set to true to make the world turn and move with view direction and eye position, this is the case if the viewer gets
			/// moved by a motion platform, defaults to false
			//pWarper->bTurnWithView = false;

			/// set to true to enable bicubic sampling from source texture
			pWarper->bBicubic = true;

			/// the calibration index in mapping file, defaults to 0,
			/// you also might set this to negated display number, to search for a certain display:
			pWarper->calibIndex = VIOSOConfigData.CalibrationIndex;

			DefaultClippingPlanes.Set(1, 20000);

			/// set a gamma correction value. This is only useful, if you changed the projector's gamma setting after calibration,
			/// as the gamma is already calculated inside blend map, or to fine-tune, defaults to 1 (no change)
			pWarper->gamma = VIOSOConfigData.Gamma;

			// the transformation matrix to go from VIOSO coordinates to IG coordinates, defaults to indentity
			// note VIOSO maps are always right-handed, to use with a left-handed world like DirectX, invert the z!
			// UE is always left-handed
			const float Scale = VIOSOConfigData.UnitsInMeter;
			const FMatrix BaseMatrix(
				FPlane(Scale, 0.f,    0.f,   0.f),
				FPlane(0.f,   Scale,  0.f,   0.f),
				FPlane(0.f,   0.f,   -Scale, 0.f),
				FPlane(0.f,   0.f,    0.f,   1.f));
			
			const FMatrix TransMatrix = BaseMatrix.GetTransposed();

			for (uint32 MatrixElementIndex = 0; MatrixElementIndex < 16; MatrixElementIndex++)
			{
				pWarper->trans[MatrixElementIndex] = (&TransMatrix.M[0][0])[MatrixElementIndex];
			}
		}
		else
		{
			// Protect VIOSO SDK calls: the DLL is not thread-safe.
			FScopeLock lock(&VIOSOLibrary->DllAccessCS);

			// Create with INI file
			if (VWB_ERROR_NONE != VIOSOLibrary->VWB_CreateA(pDxDevice, TCHAR_TO_ANSI(*VIOSOConfigData.INIFile), TCHAR_TO_ANSI(*VIOSOConfigData.ChannelName), &pWarper, 0, nullptr))
			{
				// Failed initialize vioso from ini file
				return false;
			}

			// Save imported clipping planes from INI file
			DefaultClippingPlanes.Set(pWarper->nearDist, pWarper->farDist);
		}

		// update clipping planes
		UpdateClippingPlanes(InClippingPlanes);

		// Protect VIOSO SDK calls: the DLL is not thread-safe.
		FScopeLock lock(&VIOSOLibrary->DllAccessCS);

		// initialize VIOSO warper
		if (VWB_ERROR_NONE == VIOSOLibrary->VWB_Init(pWarper))
		{
			bInitialized = true;

			return true;
		}
	}

	// Failed initialize vioso
	return false;
}

void FDisplayClusterProjectionVIOSOWarper::Release()
{
	bInitialized = false;

	if (pWarper)
	{
		// Protect VIOSO SDK calls: the DLL is not thread-safe.
		FScopeLock lock(&VIOSOLibrary->DllAccessCS);

		VIOSOLibrary->VWB_Destroy(pWarper);
	}

	pWarper = nullptr;
}

int32 FDisplayClusterProjectionVIOSOWarper::GetDataCacheTimeOutInFrames()
{
	return FMath::Max(0, GDisplayClusterRender_VIOSOWarperCacheTimeOutInFrames);
}

bool FDisplayClusterProjectionVIOSOWarper::IsDataCacheEnabled()
{
	return GDisplayClusterRender_VIOSOWarperCacheEnable != 0;
}

/**
 * The cache for VIOSO warper objects. (Singleton)
 */
class FDisplayClusterWarpBlendVIOSOWarperCache
	: public TDisplayClusterDataCache<FDisplayClusterProjectionVIOSOWarper>
{
public:
	static TSharedRef<FDisplayClusterProjectionVIOSOWarper, ESPMode::ThreadSafe> GetOrCreateVIOSOWarper(const TSharedRef<FDisplayClusterProjectionVIOSOLibrary, ESPMode::ThreadSafe>& InVIOSOLibrary, const FViosoPolicyConfiguration& InConfigData, const FString& InUniqueName)
	{
		static FDisplayClusterWarpBlendVIOSOWarperCache VIOSOWarperCacheSingleton;

		// Only for parameters related to geometry
		const FString UniqueName = FString::Printf(TEXT("%s:%s"), *HashString(InConfigData.ToString()), *InUniqueName);

		TSharedPtr<FDisplayClusterProjectionVIOSOWarper, ESPMode::ThreadSafe> ExistWarperRef = VIOSOWarperCacheSingleton.Find(UniqueName);
		if (ExistWarperRef.IsValid())
		{
			return ExistWarperRef.ToSharedRef();
		}

		TSharedPtr<FDisplayClusterProjectionVIOSOWarper, ESPMode::ThreadSafe> NewWarperRef = MakeShared<FDisplayClusterProjectionVIOSOWarper, ESPMode::ThreadSafe>(InVIOSOLibrary, InConfigData, UniqueName);
		VIOSOWarperCacheSingleton.Add(NewWarperRef);

		return NewWarperRef.ToSharedRef();
	}
};

TSharedRef<FDisplayClusterProjectionVIOSOWarper, ESPMode::ThreadSafe> FDisplayClusterProjectionVIOSOWarper::Create(const TSharedRef<FDisplayClusterProjectionVIOSOLibrary, ESPMode::ThreadSafe>& InVIOSOLibrary, const FViosoPolicyConfiguration& InConfigData, const FString& InUniqueName)
{
	return FDisplayClusterWarpBlendVIOSOWarperCache::GetOrCreateVIOSOWarper(InVIOSOLibrary, InConfigData, InUniqueName);
}

#endif
