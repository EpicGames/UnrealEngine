// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Policy/VIOSO/ViosoPolicyConfiguration.h"
#include "Policy/VIOSO/DisplayClusterProjectionVIOSOTypes.h"

#include "Policy/VIOSO/DisplayClusterProjectionVIOSOGeometryExportData.h"

#if WITH_VIOSO_LIBRARY

class FDisplayClusterProjectionVIOSOLibrary;

/**
 * Warper api for VIOSO
 */
class FDisplayClusterProjectionVIOSOWarper
{
public:
	FDisplayClusterProjectionVIOSOWarper(const TSharedRef<FDisplayClusterProjectionVIOSOLibrary, ESPMode::ThreadSafe>& InVIOSOLibrary, const FViosoPolicyConfiguration& InVIOSOConfigData, const FString& InUniqueName)
		: VIOSOLibrary(InVIOSOLibrary)
		, VIOSOConfigData(InVIOSOConfigData)
		, UniqueName(InUniqueName)
	{ }

	virtual ~FDisplayClusterProjectionVIOSOWarper()
	{
		Release();
	}

	/** Constructor for VIOSO warper instances.
	* (The cache is implemented inside)
	* 
	* @param InVIOSOLibrary
	* @param InConfigData
	* @param InUniqueName is a unique viewname for the viewport (for ex.: "vp_1:0", "vp_1:1", "vp_2:0",...)
	* 
	* @return an instance of the Warper class (the same object if InConfigData are equal).
	*/
	static TSharedRef<FDisplayClusterProjectionVIOSOWarper, ESPMode::ThreadSafe> Create(const TSharedRef<FDisplayClusterProjectionVIOSOLibrary, ESPMode::ThreadSafe>& InVIOSOLibrary, const FViosoPolicyConfiguration& InConfigData, const FString& InUniqueName);

public:
	inline bool IsInitialized() const
	{
		return pWarper != nullptr && bInitialized;
	}

	bool Initialize(void* pDxDevice, const FVector2D& InClippingPLanes);
	void Release();

	void UpdateClippingPlanes(const FVector2D& InClippingPlanes);

	bool CalculateViewProjection(IDisplayClusterViewport* InViewport, const uint32 InContextNum, FVector& InOutViewLocation, FRotator& InOutViewRotation, FMatrix& OutProjMatrix, const float WorldToMeters, const float NCP, const float FCP);

	bool Render(VWB_param RenderParam, VWB_uint StateMask);

	//~ Begin TDisplayClusterDataCache
	/** Return DataCache timeout in frames. */
	static int32 GetDataCacheTimeOutInFrames();

	/** Return true if DataCache is enabled. */
	static bool IsDataCacheEnabled();

	/** Returns the unique name of this MPCDI file resource for DataCache. */
	inline const FString& GetDataCacheName() const
	{
		return UniqueName;
	}

	/** Method for releasing a cached data item, called before its destructor. */
	inline void ReleaseDataCacheItem()
	{ }
	// ~~ End TDisplayClusterDataCache

protected:
	friend class FDisplayClusterWarpBlendVIOSOPreviewMeshCache;

	/**
	* Export warp mesh geometry from VIOSO file
	*/
	bool ExportGeometry(FDisplayClusterProjectionVIOSOGeometryExportData& OutMeshData);


private:
	FMatrix GetProjMatrix(IDisplayClusterViewport* InViewport, const uint32 InContextNum) const;

private:
	// VIOSO API
	const TSharedRef<FDisplayClusterProjectionVIOSOLibrary, ESPMode::ThreadSafe> VIOSOLibrary;

	// vioso confguration data
	const FViosoPolicyConfiguration VIOSOConfigData;

	const FString UniqueName;

	// Internal VIOSO data
	struct VWB_Warper* pWarper = nullptr;

	/** Convert arry to local vars. */
	struct FViewClip
	{
		// The correct order of ViewClips was obtained from the VIOSO sources
		// void VWB_Warper_base::getClip( VWB_VEC3f const& e, VWB_float * pClip )
		VWB_float Left, Top, Right, Bottom, NCP, FCP;
	};

	union
	{
		VWB_float VWB_ViewClip[6];
		FViewClip ViewClip;
	};

	// Warped can be used
	bool bInitialized = false;

	// Custom clipping planes
	FVector2D DefaultClippingPlanes;
};
#endif
