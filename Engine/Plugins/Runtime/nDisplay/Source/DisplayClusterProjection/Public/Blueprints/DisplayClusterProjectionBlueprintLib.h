// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Blueprints/IDisplayClusterProjectionBlueprintAPI.h"
#include "DisplayClusterProjectionBlueprintLib.generated.h"


/**
 * Blueprint API function library
 */
UCLASS()
class UDisplayClusterProjectionBlueprintLib
	: public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/** [DEPRECATED] Returns Display Cluster Projection API interface. */
	UE_DEPRECATED(5.4, "GetAPI function has been deprecated. All functions are now available in UDisplayClusterProjectionBlueprintLib.")
	UFUNCTION(BlueprintPure, meta = (DeprecatedFunction, DeprecationMessage = "GetAPI has been deprecated. All functions are now available in the main blueprint functions list under 'nDisplay' category."))
	static void GetAPI(TScriptInterface<IDisplayClusterProjectionBlueprintAPI>& OutAPI);

public:

	/** Sets camera up for 'camera' projection policy. */
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Set camera"), Category = "NDisplay|Projection")
	static void CameraPolicySetCamera(const FString& ViewportId, UCameraComponent* NewCamera, float FOVMultiplier = 1.f);
};
