// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"

#include "IDisplayClusterProjectionBlueprintAPI.generated.h"

class UCameraComponent;


UINTERFACE(meta = (CannotImplementInterfaceInBlueprint))
class DISPLAYCLUSTERPROJECTION_API UDisplayClusterProjectionBlueprintAPI : public UInterface
{
	GENERATED_BODY()
};


/**
 * Blueprint API interface
 */
class DISPLAYCLUSTERPROJECTION_API IDisplayClusterProjectionBlueprintAPI
{
	GENERATED_BODY()

public:
	//////////////////////////////////////////////////////////////////////////////////////////////
	// Policy: CAMERA
	//////////////////////////////////////////////////////////////////////////////////////////////
	/** Sets active camera component for camera policy */
	UE_DEPRECATED(5.4, "This function has been moved to UDisplayClusterProjectionBlueprintLib.")
	UFUNCTION(BlueprintCallable, meta = (DeprecatedFunction, DeprecationMessage = "This function is now available in the main blueprint functions list under 'nDisplay' section."))
	virtual void CameraPolicySetCamera(const FString& ViewportId, UCameraComponent* NewCamera, float FOVMultiplier = 1.f) = 0;
};
