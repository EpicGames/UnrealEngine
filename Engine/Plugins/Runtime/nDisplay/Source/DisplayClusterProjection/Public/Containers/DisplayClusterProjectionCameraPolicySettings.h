// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

struct FDisplayClusterProjectionCameraPolicySettings
{
	float FOVMultiplier = 1;

	// Rotate incamera frustum on this value to fix broken lens on physic camera
	FRotator  FrustumRotation = FRotator::ZeroRotator;

	// Move incamera frustum on this value to fix broken lens on physic camera
	FVector  FrustumOffset = FVector::ZeroVector;

	// Off-axis / off-center projection offset as proportion of screen dimensions
	FVector2D OffCenterProjectionOffset = FVector2D::ZeroVector;

	// When enabled, ignore all next updates, until null cam ref not assigned (UE-137222)
	bool bCameraOverrideDefaults = false;

	// Allow to use camera postprocess
	bool bUseCameraPostprocess = true;
};
