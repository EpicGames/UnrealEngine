// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
* Text constants used by the projection policies.
*/
namespace DisplayClusterProjectionStrings
{
	namespace cfg
	{
		namespace easyblend
		{
			static constexpr const TCHAR* File   = TEXT("file");
			static constexpr const TCHAR* Origin = TEXT("origin");
			static constexpr const TCHAR* Scale  = TEXT("scale");

			static constexpr const TCHAR* OriginOffset = TEXT("origin_loc");
		}

		namespace VIOSO
		{
			static constexpr const TCHAR* Origin = TEXT("origin");

			static constexpr const TCHAR* INIFile      = TEXT("inifile");
			static constexpr const TCHAR* ChannelName  = TEXT("channel");

			static constexpr const TCHAR* File         = TEXT("file");

			// The VIOSO calibration file contains several instances of geometry (for several screens in one file), so we must use this parameter to be able to select the right one.
			static constexpr const TCHAR* CalibIndex   = TEXT("index");
			static constexpr const TCHAR* CalibAdapter = TEXT("adapter");

			static constexpr const TCHAR* Gamma        = TEXT("gamma");

			// How many VIOSO units in meter
			static constexpr const TCHAR* UnitsInMeter = TEXT("UnitsInMeter");

			static constexpr const TCHAR* EnablePreview = TEXT("EnablePreview");
		}

		namespace manual
		{
			static constexpr const TCHAR* Rendering    = TEXT("ManualRendering");
			static constexpr const TCHAR* Type         = TEXT("ManualFrustum");
			static constexpr const TCHAR* Rotation     = TEXT("rot");

			static constexpr const TCHAR* Matrix       = TEXT("matrix");
			static constexpr const TCHAR* MatrixLeft   = TEXT("matrix_left");
			static constexpr const TCHAR* MatrixRight  = TEXT("matrix_right");

			static constexpr const TCHAR* Frustum      = TEXT("frustum");
			static constexpr const TCHAR* FrustumLeft  = TEXT("frustum_left");
			static constexpr const TCHAR* FrustumRight = TEXT("frustum_right");

			static constexpr const TCHAR* AngleL       = TEXT("l");
			static constexpr const TCHAR* AngleR       = TEXT("r");
			static constexpr const TCHAR* AngleT       = TEXT("t");
			static constexpr const TCHAR* AngleB       = TEXT("b");

			namespace FrustumType
			{
				static constexpr const TCHAR* Matrix = TEXT("Matrix");
				static constexpr const TCHAR* Angles = TEXT("Angles");
			}
			namespace RenderingType
			{
				static constexpr const TCHAR* Mono       = TEXT("Mono");
				static constexpr const TCHAR* Stereo     = TEXT("Stereo");
				static constexpr const TCHAR* MonoStereo = TEXT("Mono & Stereo");
			}
		}

		namespace camera
		{
			static constexpr const TCHAR* Component = TEXT("camera_component");
			static constexpr const TCHAR* Native    = TEXT("native");
		}

		namespace reference
		{
			static constexpr const TCHAR* ViewportId = TEXT("viewport");
		}

		namespace mesh
		{
			static constexpr const TCHAR* Component        = TEXT("mesh_component");

			static constexpr const TCHAR* LODIndex         = TEXT("lod_index");
			static constexpr const TCHAR* SectionIndex     = TEXT("section_index");

			static constexpr const TCHAR* BaseUVIndex      = TEXT("base_uv_index");
			static constexpr const TCHAR* ChromakeyUVIndex = TEXT("chromakey_uv_index");
		}

		namespace mpcdi
		{
			static constexpr const TCHAR* File   = TEXT("file");
			static constexpr const TCHAR* Buffer = TEXT("buffer");
			static constexpr const TCHAR* Region = TEXT("region");
			static constexpr const TCHAR* Origin = TEXT("origin");

			static constexpr const TCHAR* MPCDIType = TEXT("mpcdi");

			static constexpr const TCHAR* FilePFM       = TEXT("pfm");
			static constexpr const TCHAR* WorldScale    = TEXT("scale");
			static constexpr const TCHAR* UseUnrealAxis = TEXT("ue_space");

			static constexpr const TCHAR* FileAlpha  = TEXT("alpha");
			static constexpr const TCHAR* AlphaGamma = TEXT("alpha_gamma");
			static constexpr const TCHAR* FileBeta   = TEXT("beta");

			static constexpr const TCHAR* MPCDITypeKey = TEXT("MPCDIType");
			static constexpr const TCHAR* TypeMPCDI = TEXT("MPCDI");
			static constexpr const TCHAR* TypePFM = TEXT("Explicit PFM");

			static constexpr const TCHAR* EnablePreview = TEXT("EnablePreview");

			static constexpr const TCHAR* Component = TEXT("screen_component");

			namespace Profiles
			{
				static constexpr const TCHAR* mpcdi_2d  = TEXT("2d");
				static constexpr const TCHAR* mpcdi_3d  = TEXT("3d");
				static constexpr const TCHAR* mpcdi_a3d = TEXT("a3d");
				static constexpr const TCHAR* mpcdi_sl  = TEXT("sl");
			}

			namespace Attributes
			{
				namespace Buffer
				{
					static constexpr const TCHAR* Resolution = TEXT("BufferRes");
				}

				namespace Region
				{
					static constexpr const TCHAR* Resolution = TEXT("RegionRes");
					static constexpr const TCHAR* Pos        = TEXT("RegionPos");
					static constexpr const TCHAR* Size       = TEXT("RegionSize");
				}

				namespace Frustum
				{
					static constexpr const TCHAR* Rotation = TEXT("FrustumRot");
					static constexpr const TCHAR* Angles   = TEXT("FrustumAngles");
				}

				namespace CoordinateFrame
				{
					static constexpr const TCHAR* Position = TEXT("CoordFramePos");
					static constexpr const TCHAR* Rotation = TEXT("CoordFrameRot");
				}
			}
		}

		namespace domeprojection
		{
			static constexpr const TCHAR* File    = TEXT("file");
			static constexpr const TCHAR* Channel = TEXT("channel");
			static constexpr const TCHAR* Origin  = TEXT("origin");
			static constexpr const TCHAR* UnitsToMeter = TEXT("scale");
		}
	}

	namespace projection
	{
		static constexpr const TCHAR* Camera         = TEXT("camera");
		static constexpr const TCHAR* Domeprojection = TEXT("domeprojection");
		static constexpr const TCHAR* EasyBlend      = TEXT("easyblend");
		static constexpr const TCHAR* Link           = TEXT("link");
		static constexpr const TCHAR* Manual         = TEXT("manual");
		static constexpr const TCHAR* Mesh           = TEXT("mesh");
		static constexpr const TCHAR* MPCDI          = TEXT("mpcdi");
		static constexpr const TCHAR* Reference      = TEXT("reference");
		static constexpr const TCHAR* VIOSO          = TEXT("vioso");
	}
};
