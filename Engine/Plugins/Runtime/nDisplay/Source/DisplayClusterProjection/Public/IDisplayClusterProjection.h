// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"
#include "Modules/ModuleInterface.h"

#include "Components/SceneComponent.h"
#include "Components/StaticMeshComponent.h"

class IDisplayClusterProjectionPolicyFactory;


class IDisplayClusterProjection : public IModuleInterface
{
public:
	static constexpr const TCHAR* ModuleName = TEXT("DisplayClusterProjection");

public:
	virtual ~IDisplayClusterProjection() = default;

public:
	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IDisplayClusterProjection& Get()
	{
		return FModuleManager::LoadModuleChecked<IDisplayClusterProjection>(IDisplayClusterProjection::ModuleName);
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded(IDisplayClusterProjection::ModuleName);
	}

	/**
	* Returns supported projection types
	*
	* @param OutProjectionTypes - (out) Array of supported projection types
	*/
	virtual void GetSupportedProjectionTypes(TArray<FString>& OutProjectionTypes) = 0;

	/**
	* Returns specified projection factory
	*
	* @param InProjectionType - Projection type
	*
	* @return - Projection policy factory of requested type, null if not available
	*/
	virtual TSharedPtr<IDisplayClusterProjectionPolicyFactory> GetProjectionFactory(const FString& InProjectionType) = 0;

	/**
	* Set camera policy camera
	*
	*/
	virtual bool CameraPolicySetCamera(const TSharedPtr<class IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe>& InPolicy, class UCameraComponent* const NewCamera, const struct FDisplayClusterProjectionCameraPolicySettings& CameraSettings) = 0;

	/**
	* Get camera policy camera
	*/
	virtual class UCameraComponent* CameraPolicyGetCameraComponent(const TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe>& InPolicy) = 0;
};
