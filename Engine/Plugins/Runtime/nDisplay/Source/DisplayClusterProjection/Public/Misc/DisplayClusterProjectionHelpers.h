// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/DisplayClusterHelpers.h"
#include "DisplayClusterProjectionStrings.h"
#include "Containers/DisplayClusterWarpEnums.h"

namespace UE::DisplayClusterProjectionHelpers
{
	namespace MPCDI
	{
		/** Convert the mpcdi profile type from a string to an enum value. */
		static EDisplayClusterWarpProfileType ProfileTypeFromString(const FString& InProfileTypeName)
		{
			if (!InProfileTypeName.Compare(DisplayClusterProjectionStrings::cfg::mpcdi::Profiles::mpcdi_2d, ESearchCase::IgnoreCase))
			{
				return EDisplayClusterWarpProfileType::warp_2D;
			}
			else if (!InProfileTypeName.Compare(DisplayClusterProjectionStrings::cfg::mpcdi::Profiles::mpcdi_3d, ESearchCase::IgnoreCase))
			{
				return EDisplayClusterWarpProfileType::warp_3D;
			}
			else if (!InProfileTypeName.Compare(DisplayClusterProjectionStrings::cfg::mpcdi::Profiles::mpcdi_a3d, ESearchCase::IgnoreCase))
			{
				return EDisplayClusterWarpProfileType::warp_A3D;
			}
			else if (!InProfileTypeName.Compare(DisplayClusterProjectionStrings::cfg::mpcdi::Profiles::mpcdi_sl, ESearchCase::IgnoreCase))
			{
				return EDisplayClusterWarpProfileType::warp_SL;
			}

			return EDisplayClusterWarpProfileType::Invalid;
		}

		/** Convert the mpcdi profile type from an enum value to a string. */
		static FString ProfileTypeToString(const EDisplayClusterWarpProfileType InProfileType)
		{
			switch (InProfileType)
			{
			case EDisplayClusterWarpProfileType::warp_2D:
				return DisplayClusterProjectionStrings::cfg::mpcdi::Profiles::mpcdi_2d;
			case EDisplayClusterWarpProfileType::warp_3D:
				return DisplayClusterProjectionStrings::cfg::mpcdi::Profiles::mpcdi_3d;
			case EDisplayClusterWarpProfileType::warp_A3D:
				return DisplayClusterProjectionStrings::cfg::mpcdi::Profiles::mpcdi_a3d;
			case EDisplayClusterWarpProfileType::warp_SL:
				return DisplayClusterProjectionStrings::cfg::mpcdi::Profiles::mpcdi_sl;

			default:
				break;
			}

			// Returns an empty string if an invalid profile type is used.
			return FString();
		}
	}
};

