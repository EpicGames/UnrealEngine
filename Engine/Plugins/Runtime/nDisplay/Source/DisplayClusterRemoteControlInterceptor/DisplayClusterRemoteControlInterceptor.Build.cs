// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;


public class DisplayClusterRemoteControlInterceptor : ModuleRules
{
	public DisplayClusterRemoteControlInterceptor(ReadOnlyTargetRules ROTargetRules) : base(ROTargetRules)
	{
		ShortName = "RCInterceptor";

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"DisplayCluster",
				"Engine",
				"RemoteControlInterception",
				"Serialization",
			});

		if (Target.bBuildEditor == true)
		{
			PrivateDependencyModuleNames.Add("UnrealEd");
		}
	}
}
