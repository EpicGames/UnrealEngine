// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterRemoteControlInterceptor.h"
#include "DisplayClusterRemoteControlInterceptorLog.h"

#include "IDisplayCluster.h"
#include "Cluster/DisplayClusterClusterEvent.h"

#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"

#include "Features/IModularFeatures.h"
#include "HAL/IConsoleManager.h"
#include "Misc/CoreDelegates.h"
#include "Misc/CommandLine.h"

// Data interception source
static TAutoConsoleVariable<int32> CVarInterceptOnPrimaryOnly(
	TEXT("nDisplay.RemoteControlInterceptor.PrimaryOnly"),
	1,
	TEXT("RemoteControl commands interception location\n")
	TEXT("0 : All nodes\n")
	TEXT("1 : Primary only\n")
	,
	ECVF_ReadOnly
);

// Magic numbers for now. Unfortunately there is no any ID management for binary events yet.
// This will be refactored once we have some global events registry to prevent any ID conflicts.
const int32 EventId_InterceptorQueue		= 0xaabb0704;

FDisplayClusterRemoteControlInterceptor::FDisplayClusterRemoteControlInterceptor()
	: bInterceptOnPrimaryOnly(CVarInterceptOnPrimaryOnly.GetValueOnGameThread() == 1)
	, bForceApply(false)
{
	bForceApply = FParse::Param(FCommandLine::Get(), TEXT("ClusterForceApplyResponse"));
	
	// Set up cluster events handler
	EventsListener.BindRaw(this, &FDisplayClusterRemoteControlInterceptor::OnClusterEventBinaryHandler);
	// Subscribe for cluster events
	IDisplayCluster::Get().GetClusterMgr()->AddClusterEventBinaryListener(EventsListener);
	// Send the queue of replication events at the end of the tick
	FCoreDelegates::OnEndFrame.AddRaw(this, &FDisplayClusterRemoteControlInterceptor::SendReplicationQueue);

	UE_LOGF(LogDisplayClusterRemoteControlInterceptor, Log, "DisplayClusterRemoteControlInterceptor has been registered");
}

FDisplayClusterRemoteControlInterceptor::~FDisplayClusterRemoteControlInterceptor()
{
	// Unsubscribe from cluster events
	IDisplayCluster::Get().GetClusterMgr()->RemoveClusterEventBinaryListener(EventsListener);
	// Unbind cluster events handler
	EventsListener.Unbind();
	// Remove delegates from end of the engine frame
	FCoreDelegates::OnEndFrame.RemoveAll(this);

	UE_LOGF(LogDisplayClusterRemoteControlInterceptor, Log, "DisplayClusterRemoteControlInterceptor has been unregistered");
}

ERCIResponse FDisplayClusterRemoteControlInterceptor::SetObjectProperties(FRCIPropertiesMetadata& InProperties)
{
	// Serialize command data to binary
	TArray<uint8> Buffer;
	FMemoryWriter MemoryWriter(Buffer);
	MemoryWriter << InProperties;

	// Queue interception sending
	QueueInterceptEvent(FRCIPropertiesMetadata::Name, InProperties.GetUniquePath(), MoveTemp(Buffer));

	if (bForceApply)
	{
		return ERCIResponse::Apply;
	}

	return ERCIResponse::Intercept;
}

ERCIResponse FDisplayClusterRemoteControlInterceptor::ResetObjectProperties(FRCIObjectMetadata& InObject)
{
	// Serialize command data to binary
	TArray<uint8> Buffer;
	FMemoryWriter MemoryWriter(Buffer);
	MemoryWriter << InObject;

	// Queue interception sending
	QueueInterceptEvent(FRCIObjectMetadata::Name, InObject.GetUniquePath(), MoveTemp(Buffer));
	
	if (bForceApply)
	{
		return ERCIResponse::Apply;
	}

	return ERCIResponse::Intercept;
}

ERCIResponse FDisplayClusterRemoteControlInterceptor::InvokeCall(FRCIFunctionMetadata& InFunction)
{
	// Serialize command data to binary
	TArray<uint8> Buffer;
	FMemoryWriter MemoryWriter(Buffer);
	MemoryWriter << InFunction;

	// Queue interception sending
	QueueInterceptEvent(FRCIFunctionMetadata::Name, InFunction.GetUniquePath(), MoveTemp(Buffer));

	if (bForceApply)
	{
		return ERCIResponse::Apply;
	}
	
	return ERCIResponse::Intercept;
}

ERCIResponse FDisplayClusterRemoteControlInterceptor::SetPresetController(FRCIControllerMetadata& InController)
{
	// Serialize command data to binary
	TArray<uint8> Buffer;
	FMemoryWriter MemoryWriter(Buffer);
	MemoryWriter << InController;

	// Queue interception sending
	QueueInterceptEvent(FRCIControllerMetadata::Name, InController.GetUniquePath(), MoveTemp(Buffer));

	if (bForceApply)
	{
		return ERCIResponse::Apply;
	}

	return ERCIResponse::Intercept;
}

void FDisplayClusterRemoteControlInterceptor::OnClusterEventBinaryHandler(const FDisplayClusterClusterEventBinary& Event)
{
	// Dispatch data to a proper handler
	if (Event.bIsSystemEvent && Event.EventId == EventId_InterceptorQueue)
	{
		// Deserialize command data
		TMap<FName, TMap<FString, TArray<uint8>>> ReceivedInterceptQueueMap;
		FMemoryReader MemoryReader(Event.EventData);
		MemoryReader << ReceivedInterceptQueueMap;

		for (const TPair<FName, TMap<FString, TArray<uint8>>>& InterceptorTypeMapPair : ReceivedInterceptQueueMap)
		{
			const FName MetadataName = InterceptorTypeMapPair.Key;
			const TMap<FString, TArray<uint8>>& EventPayloadMap = InterceptorTypeMapPair.Value;

			for (const TPair<FString, TArray<uint8>>& EventPayloadPair : EventPayloadMap)
			{
				const TArray<uint8>& EventPayload = EventPayloadPair.Value;
				
				if (MetadataName == FRCIPropertiesMetadata::Name)
				{
					OnReplication_SetObjectProperties(EventPayload);
				}
				else if (MetadataName == FRCIObjectMetadata::Name)
				{
					OnReplication_ResetObjectProperties(EventPayload);
				}
				else if (MetadataName == FRCIFunctionMetadata::Name)
				{
					OnReplication_InvokeCall(EventPayload);
				}
				else if (MetadataName == FRCIControllerMetadata::Name)
				{
					OnReplication_SetPresetController(EventPayload);
				}
			}
		}
	}
}

void FDisplayClusterRemoteControlInterceptor::QueueInterceptEvent(const FName& InterceptEventType, const FName& InUniquePath, TArray<uint8>&& InBuffer)
{
	check(IsInGameThread());
	
	TMap<FName, TArray<uint8>>& InterceptorTypeMap = InterceptQueueMap.FindOrAdd(InterceptEventType);
	InterceptorTypeMap.Add(InUniquePath, MoveTemp(InBuffer));
}

void FDisplayClusterRemoteControlInterceptor::OnReplication_SetObjectProperties(const TArray<uint8>& Buffer)
{
	UE_LOGF(LogDisplayClusterRemoteControlInterceptor, VeryVerbose, "Processing replication event SetObjectProperties: %d bytes", Buffer.Num());

	// Deserialize command data
	FMemoryReader MemoryReader(Buffer);
	FRCIPropertiesMetadata PropsMetadata;
	MemoryReader << PropsMetadata;

	// Initialization
	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	const FName ProcessorFeatureName = IRemoteControlInterceptionFeatureProcessor::GetName();
	const int32 ProcessorsAmount = ModularFeatures.GetModularFeatureImplementationCount(IRemoteControlInterceptionFeatureProcessor::GetName());

	// Send the command to the processor(s)
	for (int32 ProcessorIdx = 0; ProcessorIdx < ProcessorsAmount; ++ProcessorIdx)
	{
		IRemoteControlInterceptionFeatureProcessor* const Processor = static_cast<IRemoteControlInterceptionFeatureProcessor*>(ModularFeatures.GetModularFeatureImplementation(ProcessorFeatureName, ProcessorIdx));
		if (Processor)
		{
			Processor->SetObjectProperties(PropsMetadata);
		}
	}
}

void FDisplayClusterRemoteControlInterceptor::OnReplication_ResetObjectProperties(const TArray<uint8>& Buffer)
{
	UE_LOGF(LogDisplayClusterRemoteControlInterceptor, VeryVerbose, "Processing replication event ResetObjectProperties: %d bytes", Buffer.Num());

	// Deserialize command data
	FMemoryReader MemoryReader(Buffer);
	FRCIObjectMetadata ObjectMetadata;
	MemoryReader << ObjectMetadata;

	// Initialization
	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	const FName ProcessorFeatureName = IRemoteControlInterceptionFeatureProcessor::GetName();
	const int32 ProcessorsAmount = ModularFeatures.GetModularFeatureImplementationCount(IRemoteControlInterceptionFeatureProcessor::GetName());

	// Send the command to the processor(s)
	for (int32 ProcessorIdx = 0; ProcessorIdx < ProcessorsAmount; ++ProcessorIdx)
	{
		IRemoteControlInterceptionFeatureProcessor* const Processor = static_cast<IRemoteControlInterceptionFeatureProcessor*>(ModularFeatures.GetModularFeatureImplementation(ProcessorFeatureName, ProcessorIdx));
		if (Processor)
		{
			Processor->ResetObjectProperties(ObjectMetadata);
		}
	}
}

void FDisplayClusterRemoteControlInterceptor::OnReplication_InvokeCall(const TArray<uint8>& Buffer)
{
	UE_LOGF(LogDisplayClusterRemoteControlInterceptor, VeryVerbose, "Processing replication event InvokeCall: %d bytes", Buffer.Num());

	// Deserialize command data
	FMemoryReader MemoryReader(Buffer);
	FRCIFunctionMetadata FunctionMetadata;
	MemoryReader << FunctionMetadata;

	// Initialization
	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	const FName ProcessorFeatureName = IRemoteControlInterceptionFeatureProcessor::GetName();
	const int32 ProcessorsAmount = ModularFeatures.GetModularFeatureImplementationCount(IRemoteControlInterceptionFeatureProcessor::GetName());

	// Send the command to the processor(s)
	for (int32 ProcessorIdx = 0; ProcessorIdx < ProcessorsAmount; ++ProcessorIdx)
	{
		IRemoteControlInterceptionFeatureProcessor* const Processor = static_cast<IRemoteControlInterceptionFeatureProcessor*>(ModularFeatures.GetModularFeatureImplementation(ProcessorFeatureName, ProcessorIdx));
		if (Processor)
		{
			Processor->InvokeCall(FunctionMetadata);
		}
	}
}

void FDisplayClusterRemoteControlInterceptor::OnReplication_SetPresetController(const TArray<uint8>& Buffer)
{
	UE_LOGF(LogDisplayClusterRemoteControlInterceptor, VeryVerbose, "Processing replication event SetPresetController: %d bytes", Buffer.Num());

	// Deserialize command data
	FMemoryReader MemoryReader(Buffer);
	FRCIControllerMetadata ControllerMetadata;
	MemoryReader << ControllerMetadata;

	// Initialization
	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	const FName ProcessorFeatureName = IRemoteControlInterceptionFeatureProcessor::GetName();
	const int32 ProcessorsAmount = ModularFeatures.GetModularFeatureImplementationCount(IRemoteControlInterceptionFeatureProcessor::GetName());

	// Send the command to the processor(s)
	for (int32 ProcessorIdx = 0; ProcessorIdx < ProcessorsAmount; ++ProcessorIdx)
	{
		IRemoteControlInterceptionFeatureProcessor* const Processor = static_cast<IRemoteControlInterceptionFeatureProcessor*>(ModularFeatures.GetModularFeatureImplementation(ProcessorFeatureName, ProcessorIdx));
		if (Processor)
		{
			Processor->SetPresetController(ControllerMetadata);
		}
	}
}

void FDisplayClusterRemoteControlInterceptor::SendReplicationQueue()
{
	if (!InterceptQueueMap.Num())
	{
		return;
	}
	
	// Serialize the InterceptQueueMap into the buffer
	TArray<uint8> Buffer;
	FMemoryWriter MemoryWriter(Buffer);
	MemoryWriter << InterceptQueueMap;

	// Cluster event instance
	FDisplayClusterClusterEventBinary Event;

	// Fill the event with data
	Event.EventId                = EventId_InterceptorQueue;
	Event.bIsSystemEvent         = true;
	Event.bShouldDiscardOnRepeat = false;
	Event.EventData              = MoveTemp(Buffer);

	// Emit cluster event (or not, it depends on the bInterceptOnPrimaryOnly value and the role of this cluster node)
	IDisplayCluster::Get().GetClusterMgr()->EmitClusterEventBinary(Event, bInterceptOnPrimaryOnly);

	InterceptQueueMap.Empty();
}
