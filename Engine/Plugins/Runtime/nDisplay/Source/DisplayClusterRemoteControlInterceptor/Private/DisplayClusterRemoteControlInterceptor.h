// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IRemoteControlInterceptionFeature.h"
#include "Cluster/IDisplayClusterClusterManager.h"

struct FDisplayClusterClusterEventBinary;


/**
 * Remote control interceptor feature
 */
class FDisplayClusterRemoteControlInterceptor :
	public IRemoteControlInterceptionFeatureInterceptor
{
public:
	FDisplayClusterRemoteControlInterceptor();
	virtual ~FDisplayClusterRemoteControlInterceptor();

public:
	// IRemoteControlInterceptionCommands interface
	virtual ERCIResponse SetObjectProperties(FRCIPropertiesMetadata& InProperties) override;
	virtual ERCIResponse ResetObjectProperties(FRCIObjectMetadata& InObject) override;
	virtual ERCIResponse InvokeCall(FRCIFunctionMetadata& InFunction) override;
	virtual ERCIResponse SetPresetController(FRCIControllerMetadata& InController) override;
	// ~IRemoteControlInterceptionCommands interface

private:
	// Cluster events handler/dispatcher
	void OnClusterEventBinaryHandler(const FDisplayClusterClusterEventBinary& Event);

	// Queue unique intercept event for sending on next engine tick
	void QueueInterceptEvent(const FName& InterceptEventType, const FName& InUniquePath, TArray<uint8>&& InBuffer);

private:
	// Process SetObjectProperties command replication data
	void OnReplication_SetObjectProperties   (const TArray<uint8>& Buffer);
	// Process ResetObjectProperties command replication data
	void OnReplication_ResetObjectProperties (const TArray<uint8>& Buffer);
	// Process InvokeCall command replication data
	void OnReplication_InvokeCall (const TArray<uint8>& Buffer);
	// Process SetPresetController command replication data
	void OnReplication_SetPresetController(const TArray<uint8>& Buffer);
	// Send the queue of replication events at the end of the tick
	void SendReplicationQueue();

private:
	// CVar value PrimaryOnly
	const bool bInterceptOnPrimaryOnly;
	// Cluster events listener
	FOnClusterEventBinaryListener EventsListener;
	// Force Apply the ERCIResponse
	bool bForceApply;

	/**
	 * FName name of the metadata struct
	 * FName unique object path + field path
	 * TArray<uint8> Metadata struct buffer
	 */
	TMap<FName, TMap<FName, TArray<uint8>>> InterceptQueueMap;
};
