// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterRemoteControlInterceptorLog.h"


// Module-wide log categories
DEFINE_LOG_CATEGORY(LogDisplayClusterRemoteControlInterceptor);
