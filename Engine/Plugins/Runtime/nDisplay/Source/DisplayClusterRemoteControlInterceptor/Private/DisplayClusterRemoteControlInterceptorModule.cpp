// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterRemoteControlInterceptorModule.h"
#include "DisplayClusterRemoteControlInterceptorLog.h"
#include "DisplayClusterRemoteControlInterceptor.h"

#include "Modules/ModuleManager.h"
#include "Misc/CoreDelegates.h"
#include "Features/IModularFeatures.h"
#include "IRemoteControlInterceptionFeature.h"
#include "IDisplayCluster.h"

void FDisplayClusterRemoteControlInterceptorModule::StartupModule()
{
	// Instantiate the interceptor feature on module start
	Interceptor = MakeUnique<FDisplayClusterRemoteControlInterceptor>();

	// Register Interceptor only after Engine Loop Complete 
	FCoreDelegates::OnFEngineLoopInitComplete.AddLambda([this]()
	{
		// Add Interceptor only in Cluster mode
		if (IDisplayCluster::Get().GetOperationMode() == EDisplayClusterOperationMode::Cluster)
		{
			// Register the interceptor feature
			IModularFeatures::Get().RegisterModularFeature(IRemoteControlInterceptionFeatureInterceptor::GetName(), Interceptor.Get());	
		}
	});
}

void FDisplayClusterRemoteControlInterceptorModule::ShutdownModule()
{
	// Clean delegates
	FCoreDelegates::OnFEngineLoopInitComplete.RemoveAll(this);

	// Add Interceptor only in Cluster mode
	if (IDisplayCluster::Get().GetOperationMode() == EDisplayClusterOperationMode::Cluster)
	{
		// Unregister the interceptor feature on module shutdown
		IModularFeatures::Get().UnregisterModularFeature(IRemoteControlInterceptionFeatureInterceptor::GetName(), Interceptor.Get());
	}
}

IMPLEMENT_MODULE(FDisplayClusterRemoteControlInterceptorModule, DisplayClusterRemoteControlInterceptor);
