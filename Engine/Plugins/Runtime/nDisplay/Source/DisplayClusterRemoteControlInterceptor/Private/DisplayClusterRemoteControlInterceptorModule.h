// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Templates/UniquePtr.h"

class IRemoteControlInterceptionFeatureInterceptor;


/**
 * Remote control interceptor module
 */
class FDisplayClusterRemoteControlInterceptorModule :
	public IModuleInterface
{
public:
	//////////////////////////////////////////////////////////////////////////////////////////////
	// IModuleInterface
	//////////////////////////////////////////////////////////////////////////////////////////////
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	// RC Interceptor feature instance
	TUniquePtr<IRemoteControlInterceptionFeatureInterceptor> Interceptor;
};
