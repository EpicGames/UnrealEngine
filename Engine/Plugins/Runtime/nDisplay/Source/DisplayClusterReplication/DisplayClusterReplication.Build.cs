// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class DisplayClusterReplication : ModuleRules
{
	public DisplayClusterReplication(ReadOnlyTargetRules ROTargetRules) : base(ROTargetRules)
	{
		PublicDependencyModuleNames.AddRange(
		new string[] {
			"Core",
			"CoreUObject",
			"DisplayCluster",
			"DisplayClusterConfiguration",
			"Engine"
		});

		PrivateDependencyModuleNames.AddRange(
		new string[] {
			"Core",
			"CoreUObject",
			"Engine",
			"NetCore",
			"OnlineSubsystem",
			"OnlineSubsystemUtils",
			"Sockets"
		});
	}
}
