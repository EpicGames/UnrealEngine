// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterNetConnection.h"

#include "Cluster/DisplayClusterNetDriverHelper.h"

#include "DisplayClusterNetDriver.h"
#include "DisplayClusterNetDriverLog.h"

#include "Engine/Engine.h"
#include "Engine/LevelStreaming.h"
#include "Engine/PackageMapClient.h"
#include "Engine/PendingNetGame.h"

#include "IPAddress.h"

#include "Misc/ScopeExit.h"

#include "Net/Core/Misc/PacketAudit.h"
#include "Net/Core/Trace/NetTrace.h"
#include "Net/DataChannel.h"
#include "Net/NetworkProfiler.h"
#include "Net/Core/Connection/RPCDoSDetection.h"
#include "Net/UnrealNetwork.h"

#include "SocketSubsystem.h"
#include "Sockets.h"

#include "UObject/LinkerLoad.h"
#include <Kismet/GameplayStatics.h>

UDisplayClusterNetConnection::UDisplayClusterNetConnection(const FObjectInitializer& ObjectInitializer) :
	Super(ObjectInitializer),
	ClientId(0),
	ClusterId(0),
	ClusterNodesNum(0),
	NodePort(0),
	bNodeIsPrimary(false),
	bIsClusterConnection(false),
	bSynchronousMode(false)
{
	// By default, networking settings are set for internet based gameplay with a limited bandwidth capacity
	// Those settings are not working well in high speed LAN environment
	SetUnlimitedBunchSizeAllowed(true);
}

void UDisplayClusterNetConnection::SetClientLoginState(const EClientLoginState::Type NewState)
{
	Super::SetClientLoginState(NewState);

	const FURL ClientURL(NULL, *RequestURL, TRAVEL_Absolute);

	const TCHAR* NodeOption = TEXT("node=");
	if (!ClientURL.HasOption(NodeOption))
	{
		return;
	}

	NodeName = ClientURL.GetOption(NodeOption, nullptr);

	UDisplayClusterNetDriver* DisplayClusterNetDriver = Cast<UDisplayClusterNetDriver>(Driver.Get());

	if (DisplayClusterNetDriver == nullptr)
	{
		return;
	}

	if (ClientLoginState == EClientLoginState::Type::ReceivedJoin)
	{
		const TCHAR* ClusterIdString = nullptr;
		const TCHAR* PrimaryNodeId = nullptr;
		const TCHAR* PrimaryNodePortString = nullptr;
		const TCHAR* ClusterNodesNumString = nullptr;

		if (!FDisplayClusterNetDriverHelper::GetRequiredArguments(ClientURL, ClusterIdString, PrimaryNodeId, PrimaryNodePortString, ClusterNodesNumString))
		{
			UE_LOGF(LogDisplayClusterNetDriver, Verbose, "Cluster connection Join has failed: URI arguments are invalid");
			return;
		}

		ClientId = TextKeyUtil::HashString(Challenge);
		ClusterId = TextKeyUtil::HashString(ClusterIdString);

		NodeAddress = GetRemoteAddr()->ToString(false);
		NodePort = FCString::Atoi(PrimaryNodePortString);

		ClusterNodesNum = FCString::Atoi(ClusterNodesNumString);

		bIsClusterConnection = true;

		if (PrimaryNodeId == NodeName)
		{
			bNodeIsPrimary = true;

			UE_LOGF(LogDisplayClusterNetDriver, Verbose, "Cluster connection joined: Client [%u]; Primary node [%ls]; RemoteAddr: %ls:%i", ClientId, *NodeName, *NodeAddress, NodePort);
		}
		else
		{
			UE_LOGF(LogDisplayClusterNetDriver, Verbose, "Cluster connection joined: Client [%u]; Node [%ls]; RemoteAddr: %ls", ClientId, *NodeName, *NodeAddress);
		}

		DisplayClusterNetDriver->AddNodeConnection(this);
	}
	else if (ClientLoginState == EClientLoginState::Type::CleanedUp)
	{
		DisplayClusterNetDriver->RemoveNodeConnection(this);
	}
}

void UDisplayClusterNetConnection::ReceivedPacket(FBitReader& Reader, bool bIsReinjectedPacket, bool bDispatchPacket)
{
	UDisplayClusterNetDriver* DisplayClusterNetDriver = Cast<UDisplayClusterNetDriver>(Driver.Get());

	if (DisplayClusterNetDriver == nullptr)
	{
		return;
	}

	if (bSynchronousMode && !DisplayClusterNetDriver->IsServer())
	{
		// This is client and it's ready for replication: receive and accumulate packet
		Super::ReceivedPacket(Reader, bIsReinjectedPacket, false);
		InPackets.Add(InPacketId, Reader);

		UE_LOGF(LogDisplayClusterNetDriver, VeryVerbose, "Packet %i received and accumulated", InPacketId);
	}
	else
	{
		// This is server or client that is not ready to replication
		Super::ReceivedPacket(Reader, bIsReinjectedPacket, bDispatchPacket);

		UE_LOGF(LogDisplayClusterNetDriver, VeryVerbose, "Packet %i received and processed", InPacketId);
	}
}

void UDisplayClusterNetConnection::ProcessPacket(int32 PacketId)
{
	UDisplayClusterNetDriver* DisplayClusterNetDriver = Cast<UDisplayClusterNetDriver>(Driver.Get());

	if (DisplayClusterNetDriver == nullptr)
	{
		UE_LOGF(LogDisplayClusterNetDriver, Error, "ProcessPacket: Can't cast NetDriver to DisplayClusterNetDriver");
		return;
	}

	if (!InPackets.Contains(PacketId))
	{
		UE_LOGF(LogDisplayClusterNetDriver, VeryVerbose, "Packet %i skipped!", PacketId);

		return;
	}

	PacketIDs.Reset();
	InPackets.GetKeys(PacketIDs);

	for (int32 CurrentPacketId : PacketIDs)
	{
		if (CurrentPacketId > PacketId)
		{
			break;
		}

		UE_LOGF(LogDisplayClusterNetDriver, VeryVerbose, "Packet %i processed", CurrentPacketId);

		bool bSkipAck = false;
		bool bHasBunchErrors = false;
		DispatchPacket(InPackets[CurrentPacketId], CurrentPacketId, bSkipAck, bHasBunchErrors);

		InPackets.Remove(CurrentPacketId);
	}
}
