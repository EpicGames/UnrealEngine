// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterNetDriver.h"

#include "Config/IDisplayClusterConfigManager.h"

#include "DisplayClusterNetDriver.h"
#include "DisplayClusterNetDriverLog.h"
#include "DisplayClusterGameEngine.h"

#include "Engine/ActorChannel.h"
#include "Engine/ChildConnection.h"
#include "Engine/Engine.h"
#include "Engine/Level.h"
#include "Engine/LocalPlayer.h"
#include "Engine/NetConnection.h"
#include "Engine/NetworkObjectList.h"
#include "Engine/PackageMapClient.h"
#include "Engine/World.h"

#include "GameFramework/Actor.h"
#include "GameFramework/GameNetworkManager.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/PlayerController.h"

#include "IDisplayCluster.h"
#include "IDisplayClusterCallbacks.h"

#include "Misc/CommandLine.h"

#include "NetworkingDistanceConstants.h"

#include "OnlineSubsystem.h"
#include "OnlineSubsystemNames.h"

#include "SocketSubsystem.h"
#include "Sockets.h"

#include "UObject/Package.h"

#include "Math/NumericLimits.h"

UDisplayClusterNetDriver::UDisplayClusterNetDriver(const FObjectInitializer& ObjectInitializer) :
	Super(ObjectInitializer),
	bClusterHasConnected(false),
	bLastBunchWasAcked(true),
	bConnectionViewersAreReady(false),
	ListenClusterId(0),
	ListenClusterNodesNum(0)
{
	bSkipServerReplicateActors = true;
	bTickingThrottleEnabled = false;

	// Preallocate elements to prevent allocs at runtime
	const int NumReservedElements = 32;
	ReadyOutPackets.Reserve(NumReservedElements);
	PacketsParams.Reserve(NumReservedElements);

	// Register event binary listener
	if (!HasAnyFlags(RF_ArchetypeObject) &&  IDisplayCluster::IsAvailable() && (IDisplayCluster::Get().GetOperationMode() == EDisplayClusterOperationMode::Cluster))
	{
		IDisplayClusterClusterManager* ClusterManager = IDisplayCluster::Get().GetClusterMgr();
	
		if (ClusterManager)
		{
			EventBinaryListener = FOnClusterEventBinaryListener::CreateUObject(this, &UDisplayClusterNetDriver::HandleEvent);
			ClusterManager->AddClusterEventBinaryListener(EventBinaryListener);
		}
	}

	// Delegates to modify ConsiderList processing in UNetDriver::ServerReplicateActors method
#if WITH_SERVER_CODE
	OnPreConsiderListUpdateOverride.BindUObject(this, &UDisplayClusterNetDriver::PreListUpdate);
	OnPostConsiderListUpdateOverride.BindUObject(this, &UDisplayClusterNetDriver::PostListUpdate);
	OnProcessConsiderListOverride.BindUObject(this, &UDisplayClusterNetDriver::ListUpdate);
#endif
}

UDisplayClusterNetDriver::~UDisplayClusterNetDriver()
{
	// Unregister event binary listener
	if (!HasAnyFlags(RF_ArchetypeObject) && IDisplayCluster::IsAvailable() && (IDisplayCluster::Get().GetOperationMode() == EDisplayClusterOperationMode::Cluster))
	{
		IDisplayClusterClusterManager* ClusterManager = IDisplayCluster::Get().GetClusterMgr();

		if (ClusterManager)
		{
			ClusterManager->RemoveClusterEventBinaryListener(EventBinaryListener);
		}
	}
}

bool UDisplayClusterNetDriver::InitListen(FNetworkNotify* InNotify, FURL& ListenURL, bool bReuseAddressAndPort, FString& Error)
{
	ClusterNetworkDriverHelper = MakeUnique<FDisplayClusterNetDriverHelper>();

	if (World->GetNetMode() == NM_ListenServer)
	{
		const TCHAR* ClusterIdString = nullptr;
		const TCHAR* NodeId = nullptr;
		const TCHAR* NodePortString = nullptr;
		const TCHAR* ClusterNodesNumString = nullptr;

		if (!FDisplayClusterNetDriverHelper::GetRequiredArguments(ListenURL, ClusterIdString, NodeId, NodePortString, ClusterNodesNumString))
		{
			UE_LOGF(LogDisplayClusterNetDriver, Warning, "InitListen:: Can't get required URL arguments");

			return Super::InitListen(InNotify, ListenURL, bReuseAddressAndPort, Error);
		}

		const UDisplayClusterConfigurationData* ConfigData = IDisplayCluster::Get().GetConfigMgr()->GetConfig();

		const uint32 ClusterId = TextKeyUtil::HashString(ClusterIdString);
		const uint16 NodePort = FCString::Atoi(NodePortString);
		const FString NodeAddress = ConfigData->GetPrimaryNodeAddress();

		if (ClusterNetworkDriverHelper->RegisterClusterEventsBinaryClient(ClusterId, NodeAddress, NodePort))
		{
			ListenClusterId = ClusterId;
			ListenClusterNodesNum = FCString::Atoi(ClusterNodesNumString);

			UE_LOGF(LogDisplayClusterNetDriver, Verbose, "Registered primary node for cluster [%u] node [%ls:%u]", ClusterId, *NodeAddress, NodePort);
		}
	}

	return Super::InitListen(InNotify, ListenURL, bReuseAddressAndPort, Error);
}

void UDisplayClusterNetDriver::TickDispatch(float DeltaTime)
{
	Super::TickDispatch(DeltaTime);

	// Check if driver is in sync mode
	if (!IsServer())
	{
		return;
	}

	// Register binary client when ALL nodes connected
	if (!bClusterHasConnected)
	{
		UE_LOGF(LogDisplayClusterNetDriver, VeryVerbose, "Primary Node connections = [%u]", PrimaryNodeConnections.Num());

		if (World->GetNetMode() == NM_ListenServer && (ListenClusterNodesNum == NodeConnections.Num() + 1))
		{
			bClusterHasConnected = NotifyClusterAsReadyForSync(ListenClusterId);
		}

		for (UDisplayClusterNetConnection* PrimaryNodeConnectionIt : PrimaryNodeConnections)
		{
			TArray<UDisplayClusterNetConnection*>* ClusterNodeConnectionsFound = ClusterConnections.Find(PrimaryNodeConnectionIt->ClusterId);

			if (ClusterNodeConnectionsFound == nullptr)
			{
				continue;
			}

			TArray<UDisplayClusterNetConnection*>& ClusterNodeConnections = *ClusterNodeConnectionsFound;

			if (PrimaryNodeConnectionIt->ClusterNodesNum != ClusterNodeConnections.Num())
			{
				UE_LOGF(LogDisplayClusterNetDriver, VeryVerbose, "PrimaryNodeConnectionIt->ClusterNodesNum = [%u], ClusterNodeConnections.Num() = [%u]", PrimaryNodeConnectionIt->ClusterNodesNum, ClusterNodeConnections.Num());
				continue;
			}

			if (ClusterNetworkDriverHelper->RegisterClusterEventsBinaryClient(PrimaryNodeConnectionIt->ClusterId, PrimaryNodeConnectionIt->NodeAddress, PrimaryNodeConnectionIt->NodePort))
			{
				UE_LOGF(LogDisplayClusterNetDriver, VeryVerbose, "Registered primary node for cluster [%u] node [%ls]", PrimaryNodeConnectionIt->ClusterId, *PrimaryNodeConnectionIt->NodeName);
			}

			bClusterHasConnected = NotifyClusterAsReadyForSync(PrimaryNodeConnectionIt->ClusterId);
		}
	}
}

void UDisplayClusterNetDriver::AddNodeConnection(UDisplayClusterNetConnection* NetConnection)
{
	NodeConnections.Add(NetConnection);

	if (NetConnection->bNodeIsPrimary)
	{
		PrimaryNodeConnections.Add(NetConnection);
	}

	TArray<UDisplayClusterNetConnection*>& Connections = ClusterConnections.FindOrAdd(NetConnection->ClusterId);
	Connections.Add(NetConnection);
}

void UDisplayClusterNetDriver::RemoveNodeConnection(UDisplayClusterNetConnection* NetConnection)
{
	if (NetConnection->bNodeIsPrimary)
	{
		if (ClusterNetworkDriverHelper->RemoveClusterEventsBinaryClient(NetConnection->ClusterId))
		{
			UE_LOGF(LogDisplayClusterNetDriver, VeryVerbose, "Removed primary node for cluster [%u] node [%ls]", NetConnection->ClusterId, *NetConnection->NodeName);
		}

		PrimaryNodeConnections.Remove(NetConnection);
	}


	TArray<UDisplayClusterNetConnection*>* Connections = ClusterConnections.Find(NetConnection->ClusterId);

	if (Connections != nullptr)
	{
		for (UDisplayClusterNetConnection* NodeConnectionIt : *Connections)
		{
			SyncConnections.Remove(NodeConnectionIt);
			NodeConnections.Remove(NodeConnectionIt);
		}
		Connections->Remove(NetConnection);
	}

	// Reset packet queues
	if (SyncConnections.IsEmpty())
	{
		OutPacketsQueues.Reset();
		bClusterHasConnected = false;
	}
}

void UDisplayClusterNetDriver::HandleEvent(FDisplayClusterClusterEventBinary const& InEvent)
{
	if ((InEvent.EventId != NodeSyncEvent) && (InEvent.EventId != PacketSyncEvent))
	{
		return;
	}

	if (ServerConnection == nullptr)
	{
		return;
	}

	// This is a client
	UDisplayClusterNetConnection* ClusterServerConnection = Cast<UDisplayClusterNetConnection>(ServerConnection.Get());

	if (ClusterServerConnection == nullptr)
	{
		UE_LOGF(LogDisplayClusterNetDriver, Error, "HandleEvent: Can't cast NetConnection to DriverClusterNetConnection");
		return;
	}

	if (InEvent.EventId == NodeSyncEvent)
	{
		UE_LOGF(LogDisplayClusterNetDriver, VeryVerbose, "[StartSynchronousMode] command received");

		// Client received command to start synchrosonus packets processing
		ClusterServerConnection->bSynchronousMode = true;
		ClusterServerConnection->ClientId = TextKeyUtil::HashString(ClusterServerConnection->Challenge);
		
		UDisplayClusterGameEngine* ClusterGameEngine = Cast<UDisplayClusterGameEngine>(GEngine);
		
		if (ClusterGameEngine != nullptr)
		{
			ClusterGameEngine->ResetForcedIdleMode();
		}
		else
		{
			UE_LOGF(LogDisplayClusterNetDriver, Error, "Can't reset internal idle mode, GEngine is not UDisplayClusterGameEngine instance");
		}

		UE_LOGF(LogDisplayClusterNetDriver, VeryVerbose, "Client node self-introducing: %u", ClusterServerConnection->ClientId);
	}

	if (InEvent.EventId == PacketSyncEvent)
	{
		// Deserializing parameters from binary event data
		const uint8* RawData = InEvent.EventData.GetData();

		// Event data contains packets for all connected clients.
		// We do iterate on event data and search for own packet by ClientId.
		// If found - break
		int ReadOffset = 0;
		while (ReadOffset < InEvent.EventData.Num())
		{
			uint32 CommandClientId = 0;
			int32 CommandPacketId = 0;

			FMemory::Memcpy(&CommandClientId, RawData + ReadOffset, sizeof(CommandClientId));

			if (CommandClientId == ClusterServerConnection->ClientId)
			{
				// Client has received a command to process packet with given ID
				FMemory::Memcpy(&CommandPacketId, RawData + ReadOffset + sizeof(CommandClientId), sizeof(CommandPacketId));

				UE_LOGF(LogDisplayClusterNetDriver, Verbose, "[ProcessPacket %i] command received", CommandPacketId);

				ClusterServerConnection->ProcessPacket(CommandPacketId);

				break;
			}

			ReadOffset += sizeof(CommandClientId) + sizeof(CommandPacketId);
		}
	}
}

void UDisplayClusterNetDriver::TickFlush(float DeltaSeconds)
{
	if (IsServer())
	{
		int32 ClusterNumNodes = -1;

		for (UDisplayClusterNetConnection* SyncConnectionIt : SyncConnections)
		{
			ClusterNumNodes = SyncConnectionIt->ClusterNodesNum;
		}

		if (World->GetNetMode() == NM_ListenServer)
		{
			bSkipServerReplicateActors = (SyncConnections.Num() != ClusterNumNodes - 1);
		}
		else
		{
			bSkipServerReplicateActors = (SyncConnections.Num() != ClusterNumNodes);
		}

		UE_LOGF(LogDisplayClusterNetDriver, VeryVerbose, "bSkipServerReplicateActors %u; ClusterNumNodes: %d, SyncConnections.Num(): %d", bSkipServerReplicateActors,
			ClusterNumNodes, SyncConnections.Num());
	}

	Super::TickFlush(DeltaSeconds);

	// Check if we the server or if any synchronous connections exists for processing
	if (!IsServer() || SyncConnections.IsEmpty())
	{
		// This is a client
		return;
	}

	if (World->GetNetMode() == NM_ListenServer)
	{
		IDisplayClusterConfigManager* ConfigMgr = IDisplayCluster::Get().GetConfigMgr();

		if (ConfigMgr != nullptr)
		{
			const UDisplayClusterConfigurationData* ConfigData = ConfigMgr->GetConfig();

			if (ConfigData == nullptr)
			{
				return;
			}

			if (SyncConnections.Num() < static_cast<int32>(ConfigData->GetNumberOfClusterNodes() - 1))
			{
				return;
			}
		}
	}

	// Check if next slice can be formed
	bool bCanFormSlice = true;

	for (UDisplayClusterNetConnection* SyncConnectionIt : SyncConnections)
	{
		const int32 ConnectionUniqueId = SyncConnectionIt->GetUniqueID();
		const int32 LastOutPacketId = SyncConnectionIt->LastOut.PacketId;

		TDeque<int32>& ChannelPacketQueue = OutPacketsQueues.FindOrAdd(ConnectionUniqueId);

		if (!ChannelPacketQueue.IsEmpty())
		{
			if (ChannelPacketQueue.Last() == LastOutPacketId)
			{
				bCanFormSlice = false;
				break;
			}
		}
	}

	bLastBunchWasAcked = true;

	for (UDisplayClusterNetConnection* SyncConnectionIt : SyncConnections)
	{
		const int32 LastOutPacketId = SyncConnectionIt->LastOut.PacketId;

		// Driver waits until last sent bunch of any sync connection will be acked
		bLastBunchWasAcked &= (LastOutPacketId - SyncConnectionIt->OutAckPacketId <= 1);
	}

	int32 NumPacketsInQueue = 0;

	if (bCanFormSlice)
	{
		// Fill queues of last send packets for each connection, update last acked packet for each connection
		for (UDisplayClusterNetConnection* SyncConnectionIt : SyncConnections)
		{
			const int32 ConnectionUniqueId = SyncConnectionIt->GetUniqueID();
			const int32 LastOutPacketId = SyncConnectionIt->LastOut.PacketId;

			// Add last send packet of current Connection to queue
			TDeque<int32>& ChannelPacketQueue = OutPacketsQueues.FindOrAdd(ConnectionUniqueId);
			ChannelPacketQueue.PushLast(LastOutPacketId);

			// Get size of queue. Size will be the same for each Connection
			NumPacketsInQueue = ChannelPacketQueue.Num();

			UE_LOGF(LogDisplayClusterNetDriver, VeryVerbose, "Client %u; Packet %i sent, %i acked", ConnectionUniqueId, LastOutPacketId, SyncConnectionIt->OutAckPacketId);
		}
	}

	// Find slice of packets which are ready for processing on clients
	// Slice corresponds to common denominator of acked packets
	int32 PacketIdx = INDEX_NONE;

	bool bPacketsReadyForSync = false;

	for (PacketIdx = NumPacketsInQueue - 1; PacketIdx >= 0; PacketIdx--)
	{
		if (bPacketsReadyForSync)
		{
			break;
		}

		ReadyOutPackets.Reset();

		for (UDisplayClusterNetConnection* SyncConnectionIt : SyncConnections)
		{
			int32 CurrentConnectionId = SyncConnectionIt->GetUniqueID();
			int32 CurrentPacketId = OutPacketsQueues[CurrentConnectionId][PacketIdx];

			if ((CurrentPacketId > 0) && (CurrentPacketId <= SyncConnectionIt->OutAckPacketId))
			{
				ReadyOutPackets.Add(CurrentConnectionId, CurrentPacketId);
			}
		}

		bPacketsReadyForSync = (ReadyOutPackets.Num() == SyncConnections.Num());
	}

	// Slice of acked packets is found - form PacketParams and send cluster event to primary nodes
	if (bPacketsReadyForSync)
	{
		PacketsParams.Reset();

		for (UDisplayClusterNetConnection* SyncConnection : SyncConnections)
		{
			int32 UniqueID = SyncConnection->GetUniqueID();
			int32 ReadyForProcessingPacket = ReadyOutPackets[UniqueID];

			// Fill parameters for binary event (key - cliend id, value - packet id)
			PacketsParams.Add(SyncConnection->ClientId, ReadyForProcessingPacket);

			// Slice queue by removing all packets which are behind ReadyForProcessingPacket
			TDeque<int32>& ConnectionPacketQueue = OutPacketsQueues[UniqueID];
			if (!ConnectionPacketQueue.IsEmpty())
			{
				for (int32 ConnectionPacketIndex = 0; ConnectionPacketIndex <= PacketIdx; ConnectionPacketIndex++)
				{
					ConnectionPacketQueue.PopFirst();
				}
			}

			UE_LOGF(LogDisplayClusterNetDriver, VeryVerbose, "Client %u; [ProcessPacket %i] command sent", SyncConnection->ClientId, ReadyForProcessingPacket);
		}

		// Generate cluster event
		FDisplayClusterClusterEventBinary NetworkDriverSyncEvent;
		GenerateClusterCommandsEvent(NetworkDriverSyncEvent, PacketSyncEvent, PacketsParams);

		// Send cluster event to all primary nodes
		ClusterNetworkDriverHelper->SendCommandToAllClusters(NetworkDriverSyncEvent);
	}
}

#if WITH_SERVER_CODE
void UDisplayClusterNetDriver::PreListUpdate(ConsiderListUpdateParams const& UpdateParams, int& OutUpdated, const TArray<FNetworkObjectInfo*>& ConsiderList)
{
	bConnectionViewersAreReady = false;

	if (!bLastBunchWasAcked || NodeConnections.IsEmpty())
	{
		return;
	}

	const float DeltaSeconds = UpdateParams.DeltaSeconds;
	const bool bCPUSaturated = UpdateParams.bCPUSaturated;
	UNetConnection* Connection = UpdateParams.Connection;

	ClusterReplicationState = FDisplayClusterReplicationState();
	PivotNodeConnectionViewers.Reset();

	UDisplayClusterNetConnection* PivotNodeConnection = nullptr;

	// Find first valid primary node connection and choose it as pivot
	for (UDisplayClusterNetConnection* PrimaryNodeConnection : PrimaryNodeConnections)
	{
		if (PrimaryNodeConnection->ViewTarget == nullptr)
		{
			continue;
		}

		PivotNodeConnection = PrimaryNodeConnection;
		break;
	}
	
	// Primary node connection doesn't exist on listen server - picking the first node connection instead
	if (World->GetNetMode() == NM_ListenServer)
	{
		if (UDisplayClusterNetConnection** ClusterConnection = NodeConnections.FindArbitraryElement())
		{
			PivotNodeConnection = *ClusterConnection;
		}
	}

	if (PivotNodeConnection == nullptr)
	{
		return;
	}

	AWorldSettings* WorldSettings = World->GetWorldSettings();
	TArray<FNetViewer>& ConnectionViewers = WorldSettings->ReplicationViewers;

	ConnectionViewers.Reset();

	new(ConnectionViewers)FNetViewer(PivotNodeConnection, DeltaSeconds);
	for (int32 ViewerIndex = 0; ViewerIndex < PivotNodeConnection->Children.Num(); ViewerIndex++)
	{
		if (PivotNodeConnection->Children[ViewerIndex]->ViewTarget != NULL)
		{
			new(ConnectionViewers)FNetViewer(PivotNodeConnection->Children[ViewerIndex], DeltaSeconds);
		}
	}

	PivotNodeConnectionViewers = ConnectionViewers;

	ClusterReplicationState.FinalSortedCount = ServerReplicateActors_PrioritizeActors(PivotNodeConnection, PivotNodeConnectionViewers,
		ConsiderList, bCPUSaturated, ClusterReplicationState.PriorityList, ClusterReplicationState.PriorityActors);

	bConnectionViewersAreReady = true;

	for (UDisplayClusterNetConnection* ClusterConnection : SyncConnections)
	{
		if (ClusterConnection->ViewTarget == nullptr)
		{
			bConnectionViewersAreReady = false;
			break;
		}

		for (TObjectPtr<UChannel> Channel : ClusterConnection->Channels)
		{
			if (!IsValid(Channel))
			{
				continue;
			}

			bConnectionViewersAreReady &= !(Channel->bPausedUntilReliableACK && (Channel->NumOutRec > 0));
		}
	}
}

void UDisplayClusterNetDriver::PostListUpdate(ConsiderListUpdateParams const& UpdateParams, int& OutUpdated, const TArray<FNetworkObjectInfo*>& ConsiderList)
{
	// Additional replication for actors number equalization
	if (!bConnectionViewersAreReady)
	{
		return;
	}

	for (UDisplayClusterNetConnection* DisplayClusterConnection : NodeConnections)
	{
		if (!DisplayClusterConnection->bIsClusterConnection || DisplayClusterConnection->ViewTarget == nullptr)
		{
			continue;
		}

		for (int j = 0; j < ClusterReplicationState.FinalSortedCount; j++)
		{
			if (ClusterReplicationState.PriorityList[j].ActorInfo == nullptr)
			{
				continue;
			}

			ClusterReplicationState.PriorityList[j].Channel = DisplayClusterConnection->FindActorChannelRef(ClusterReplicationState.PriorityList[j].ActorInfo->WeakActor);
		}

		const int32* FoundLastProcessedActors = ClusterReplicationState.LastProcessedActors.Find(DisplayClusterConnection->GetUniqueID());

		if (FoundLastProcessedActors == nullptr)
		{
			continue;
		}

		const int32 LastProcessedActors = *FoundLastProcessedActors;

		if (LastProcessedActors < ClusterReplicationState.MaxLastProcessedActor && ClusterReplicationState.MaxLastProcessedActor > 0)
		{
			// We do use generated ConnectionViewers above, but than use stored ConnectionViewers from primary nodes?
			const int32 ActorStartIndex = FMath::Clamp<int32>(LastProcessedActors, 0, ClusterReplicationState.FinalSortedCount - 1);
			const int32 ActorsToProcess = FMath::Min(ClusterReplicationState.MaxLastProcessedActor - ActorStartIndex, ClusterReplicationState.FinalSortedCount - ActorStartIndex);

			constexpr bool bForceUpdate = true;
			OutUpdated = 0;

			TInterval<int32> ActorsIndexRange(ActorStartIndex, ActorsToProcess);
			ServerReplicateActors_ProcessPrioritizedActorsRange(DisplayClusterConnection, PivotNodeConnectionViewers, ClusterReplicationState.PriorityActors, ActorsIndexRange,
				OutUpdated, bForceUpdate);
		}

		const int32 StartMarkedActor = ClusterReplicationState.MaxLastProcessedActor > 0 ? ClusterReplicationState.MaxLastProcessedActor + 1 : 0;

		ServerReplicateActors_MarkRelevantActors(DisplayClusterConnection, PivotNodeConnectionViewers, StartMarkedActor, ClusterReplicationState.FinalSortedCount, ClusterReplicationState.PriorityActors);
	}
}

void UDisplayClusterNetDriver::ListUpdate(ConsiderListUpdateParams const& UpdateParams, int& OutUpdated, const TArray<FNetworkObjectInfo*>& ConsiderList)
{
	UDisplayClusterNetConnection* DisplayClusterConnection = Cast<UDisplayClusterNetConnection>(UpdateParams.Connection);
	check(DisplayClusterConnection);

	const float DeltaSeconds = UpdateParams.DeltaSeconds;
	const bool bCPUSaturated = UpdateParams.bCPUSaturated;
	UNetConnection* Connection = UpdateParams.Connection;

	// If not ClusterConnection - fallback to default prioritization
	if (!DisplayClusterConnection->bIsClusterConnection)
	{
		AWorldSettings* WorldSettings = World->GetWorldSettings();
		TArray<FNetViewer>& ConnectionViewers = WorldSettings->ReplicationViewers;

		FActorPriority* PriorityList = nullptr;
		FActorPriority** PriorityActors = nullptr;	

		// Get a sorted list of actors for this connection
		const int32 FinalSortedCount = ServerReplicateActors_PrioritizeActors(Connection, ConnectionViewers, ConsiderList, bCPUSaturated, PriorityList, PriorityActors);

		// Process the sorted list of actors for this connection
		TInterval<int32> ActorsIndexRange(0, FinalSortedCount);
		const int32 LastProcessedActor = ServerReplicateActors_ProcessPrioritizedActorsRange(Connection, ConnectionViewers, PriorityActors, ActorsIndexRange, OutUpdated);

		ServerReplicateActors_MarkRelevantActors(Connection, ConnectionViewers, LastProcessedActor, FinalSortedCount, PriorityActors);
	}

	if (DisplayClusterConnection->bIsClusterConnection && bConnectionViewersAreReady)
	{
		// Update channels for each actor
		for (int j = 0; j < ClusterReplicationState.FinalSortedCount; j++)
		{
			if (ClusterReplicationState.PriorityList[j].ActorInfo == nullptr)
			{
				continue;
			}

			ClusterReplicationState.PriorityList[j].Channel = Connection->FindActorChannelRef(ClusterReplicationState.PriorityList[j].ActorInfo->WeakActor);
		}

		// Use single priority list per cluster which was calculated for Primary node
		// Process the sorted list of actors for this connection
		TInterval<int32> ActorsIndexRange(0, ClusterReplicationState.FinalSortedCount);
		
		OutUpdated = 0;
		int32 LastProcessedActor = ServerReplicateActors_ProcessPrioritizedActorsRange(Connection, PivotNodeConnectionViewers, ClusterReplicationState.PriorityActors,
			ActorsIndexRange, OutUpdated);

		// Cache last processed actors for equalization on PostListUpdate step
		// MaxLastProcessed will be used for equalization
		ClusterReplicationState.LastProcessedActors.Add(Connection->GetUniqueID(), OutUpdated);
		ClusterReplicationState.MaxLastProcessedActor = FMath::Max<int32>(ClusterReplicationState.MaxLastProcessedActor, OutUpdated);
	}
}
#endif


bool UDisplayClusterNetDriver::NotifyClusterAsReadyForSync(int32 ClusterId)
{
	// Notify primary node that current node connection is ready
	FDisplayClusterClusterEventBinary NetworkDriverSyncEvent;
	GenerateClusterCommandsEvent(NetworkDriverSyncEvent, NodeSyncEvent);

	// send command 
	const bool bEventWasSent = ClusterNetworkDriverHelper->SendCommandToCluster(ClusterId, NetworkDriverSyncEvent);

	// If event was sent to primary cluster node
	if (bEventWasSent)
	{
		// Mark all node connections related to current ClusterId as ready
		for (UDisplayClusterNetConnection* ClusterNodeConnectionIt : NodeConnections)
		{
			ClusterNodeConnectionIt->bSynchronousMode = true;
			SyncConnections.Add(ClusterNodeConnectionIt);
		}
		
		UE_LOGF(LogDisplayClusterNetDriver, VeryVerbose, "Setting synchronous mode in cluster [%u]", ClusterId);

		return true;
	}

	return false;
}

void UDisplayClusterNetDriver::GenerateClusterCommandsEvent(FDisplayClusterClusterEventBinary& NetworkDriverSyncEvent, 
	int32 EventId)
{
	NetworkDriverSyncEvent.EventId = EventId;
	NetworkDriverSyncEvent.bIsSystemEvent = false;
}

void UDisplayClusterNetDriver::GenerateClusterCommandsEvent(FDisplayClusterClusterEventBinary& NetworkDriverSyncEvent, 
	int32 EventId, const TMap<uint32, int32>& Parameters)
{
	GenerateClusterCommandsEvent(NetworkDriverSyncEvent, EventId);

	const int KeySize = sizeof(uint32);
	const int ValueSize = sizeof(int32);
	const int RecordSize = KeySize + ValueSize;

	ClusterEventData.SetNumUninitialized(Parameters.Num() * RecordSize, EAllowShrinking::Yes);

	uint8* RawData = ClusterEventData.GetData();

	int32 WriteOffset = 0;
	for (const TPair<uint32, int32>& Pair : Parameters)
	{
		FMemory::Memcpy(RawData + WriteOffset, (uint8*)(&Pair.Key), KeySize);
		FMemory::Memcpy(RawData + WriteOffset + KeySize, (uint8*)(&Pair.Value), ValueSize);

		WriteOffset += RecordSize;
	}

	NetworkDriverSyncEvent.EventData = ClusterEventData;
}
