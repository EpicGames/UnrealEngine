// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/TaskGraphInterfaces.h"
#include "CoreMinimal.h"
#include "DisplayClusterConfigurationTypes.h"
#include "Engine/NetConnection.h"
#include "IpConnection.h"
#include "Net/NetPacketNotify.h"
#include "SocketTypes.h"
#include "UObject/ObjectMacros.h"

#include "DisplayClusterNetConnection.generated.h"

class UDisplayClusterNetDriver;

UCLASS(transient, config = Engine)
class UDisplayClusterNetConnection : public UIpConnection
{
	GENERATED_UCLASS_BODY()
	
	//~ Begin UIpConnection Interface
	virtual void ReceivedPacket(FBitReader& Reader, bool bIsReinjectedPacket = false, bool bDispatchPacket = true) override;
	virtual void SetClientLoginState(const EClientLoginState::Type NewState) override;
	//~ End UIpConnection Interface

public:
	/** The name of the node, parsed from node URL address */
	FString NodeName;

	/** The ip address of the node, converted to FString from current NetConnection */
	FString NodeAddress;

	/** Cluster client unique identifier, HashString from NetConnection Challenge */
	uint32 ClientId;

	/** Cluster unique identifier, HashString from node config file path */
	uint32 ClusterId;

	/** Cluster nodes number */
	uint32 ClusterNodesNum;

	/** Cluster node port for binary cluster events */
	uint16 NodePort;

	/** Whether current connection belongs to primary cluster node */
	bool bNodeIsPrimary;

	/** Whether current connection belongs to cluster node */
	bool bIsClusterConnection;

	/** Whether current connection works in synchronous mode */
	bool bSynchronousMode;

	/** Process accumulated packets in the packet queue until PacketId
	 *  @param PacketId identified of packet
	 */
	void ProcessPacket(int32 PacketId);

protected:
	// Storage for InPackets map keys, prevents reallocs 
	TArray<int32> PacketIDs;

	// Data associated with each packet id
	TSortedMap<int32, FBitReader> InPackets;
};
