// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/DelegateCombinations.h"

#include "Cluster/DisplayClusterClusterEvent.h"
#include "Cluster/DisplayClusterNetDriverHelper.h"
#include "Cluster/IDisplayClusterClusterManager.h"

#include "Containers/Deque.h"
#include "CoreMinimal.h"

#include "DisplayClusterNetConnection.h"

#include "GameFramework/WorldSettings.h"
#include "IDisplayCluster.h"

#include "IpNetDriver.h"
#include "UObject/ObjectMacros.h"

#include "DisplayClusterNetDriver.generated.h"

/**
 * Helper structure to cache PriorityActor list for additional replication
 * in order to equalize number of replicated actors among all synced connections
 */
struct FDisplayClusterReplicationState
{
	// Number of actors that were prioritized
	int32 FinalSortedCount = 0;
	
	// Max processed actor among all synced connections
	int32 MaxLastProcessedActor = -1;

	// Cached priority lists
	FActorPriority* PriorityList = nullptr;
	FActorPriority** PriorityActors = nullptr;

	// Last processed actor for each sync connection
	TMap<uint32, int32> LastProcessedActors;
};

/**
 * Custom NetDriver for DisplayCluster.
 * Responsible for synchronous application of network packets across display cluster nodes.
 * Results in seamless image when used with Actor replication system.
 */
UCLASS(transient, config = Engine)
class UDisplayClusterNetDriver : public UIpNetDriver
{
	GENERATED_UCLASS_BODY()

	virtual ~UDisplayClusterNetDriver();

	//~ Begin UIpNetDriver Interface
	virtual bool InitListen(FNetworkNotify* InNotify, FURL& ListenURL, bool bReuseAddressAndPort, FString& Error) override;
	virtual void TickDispatch(float DeltaTime) override;
	virtual void TickFlush(float DeltaSeconds) override;
	//~ End UIpNetDriver Interface
public:
	/** Performs Cluster Event handling: sync mode start and packet processing on client side
	 *
	 * @param Binary event to handle
	 */
	void HandleEvent(FDisplayClusterClusterEventBinary const& InEvent);

	/** Adds connection to nDisplay connection TSet 
	 *
	 * @param NetConnection connection to add
	 */
	void AddNodeConnection(UDisplayClusterNetConnection* NetConnection);

	/** Removes connection from nDisplay connection TSet and from synchronized connections 
	 *
	 * @param NetConnection connection to remove
	 */
	void RemoveNodeConnection(UDisplayClusterNetConnection* NetConnection);

protected:
	// Contains queue of non-processed packets for specific connection (key - connectionID, value - packetID)
	TMap<int32, TDeque<int32>> OutPacketsQueues;

	// Holds packets that are ready for replication (key - connectionID, value - packetID)
	TMap<int32, int32> ReadyOutPackets;

	// Holds ready packets for each client. Member of the class to prevent runtime reallocations.
	TMap<uint32, int32> PacketsParams;

	// Binary event data. Member of the class to prevent runtime reallocations
	TArray<uint8> ClusterEventData;

	// nDisplay node connections
	TSet<UDisplayClusterNetConnection*> NodeConnections;

	// nDisplay Primary node connections
	TSet<UDisplayClusterNetConnection*> PrimaryNodeConnections;

	// nDisplay Node connections which corresponds to specific cluster
	TMap<uint32, TArray<UDisplayClusterNetConnection*>> ClusterConnections;

	// nDisplay node connections that participate in synchronous replication
	TSet<UDisplayClusterNetConnection*> SyncConnections;

	// ConnectionViewers for Pivot node (primary node connection)
	TArray<FNetViewer> PivotNodeConnectionViewers;

	// NetDriver helper for Display Cluster Events
	TUniquePtr<FDisplayClusterNetDriverHelper> ClusterNetworkDriverHelper;

	// Stores replication state to produce additional replication in order to equalize number of actors for each sync connection
	FDisplayClusterReplicationState ClusterReplicationState;

	// Binary event listener
	FOnClusterEventBinaryListener EventBinaryListener;

	// True if cluster has connected
	bool bClusterHasConnected;

	// Used to control state of acked packets queue
	bool bLastBunchWasAcked;

	// Checks if connections viewers for Primary node were formed
	bool bConnectionViewersAreReady;

	// ClusterId for listen Server
	int32 ListenClusterId;

	// Num cluster nodes for listen Server
	int32 ListenClusterNodesNum;

	// Cluster event id, used to start synchrosonus packets processing
	inline static const int NodeSyncEvent = GetTypeHash(FStringView(TEXT("nDCRNodeSyncEvent")));

	// Cluster event id, used to identify packet data blob for sync
	inline static const int PacketSyncEvent = GetTypeHash(FStringView(TEXT("nDCRPacketSyncEvent")));

protected:
#if WITH_SERVER_CODE
	void PreListUpdate(ConsiderListUpdateParams const& UpdateParams, int& OutUpdated, const TArray<FNetworkObjectInfo*>& ConsiderList);
	void PostListUpdate(ConsiderListUpdateParams const& UpdateParams, int& OutUpdated, const TArray<FNetworkObjectInfo*>& ConsiderList);
	void ListUpdate(ConsiderListUpdateParams const& UpdateParams, int& OutUpdated, const TArray<FNetworkObjectInfo*>& ConsiderList);
#endif
	
	// Helper functions used by server side to notify cluster via cluster event that it is are ready for sync
	bool NotifyClusterAsReadyForSync(int32 ClusterId);

	// Helper functions for parameters serialization into binary event data
	void GenerateClusterCommandsEvent(FDisplayClusterClusterEventBinary& NetworkDriverSyncEvent, int32 EventId);
	void GenerateClusterCommandsEvent(FDisplayClusterClusterEventBinary& NetworkDriverSyncEvent, int32 EventId, const TMap<uint32, int32>& Parameters);
};
