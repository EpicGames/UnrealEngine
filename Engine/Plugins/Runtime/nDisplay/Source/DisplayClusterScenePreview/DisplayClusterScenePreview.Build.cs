// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DisplayClusterScenePreview : ModuleRules
{
	public DisplayClusterScenePreview(ReadOnlyTargetRules ROTargetRules) : base(ROTargetRules)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"DisplayClusterLightCardEditorShaders",
				"DisplayClusterLightCardExtender",
				"RHI",
				"DisplayCluster"
			});

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"DisplayClusterConfiguration",

				"Core",
				"CoreUObject",
				"Engine",
				"ProceduralMeshComponent",
				"RenderCore"
			});

		if (Target.bBuildEditor == true)
		{
			PrivateDependencyModuleNames.Add("UnrealEd");
		}
	}
}
