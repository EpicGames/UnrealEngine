// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterLightCardEditorHelper.h"

#include "IDisplayClusterScenePreview.h"

#include "DisplayClusterConfigurationTypes.h"

#include "DisplayClusterChromakeyCardActor.h"
#include "DisplayClusterLightCardActor.h"
#include "DisplayClusterRootActor.h"
#include "DisplayClusterRootActorContainers.h"
#include "Components/DisplayClusterCameraComponent.h"

#include "StageActor/DisplayClusterStageActorTemplate.h"

#include "CanvasTypes.h"
#include "KismetProceduralMeshLibrary.h"
#include "PreviewScene.h"
#include "ProceduralMeshComponent.h"
#include "Components/DisplayClusterStageGeometryComponent.h"
#include "Containers/ArrayView.h"
#include "Engine/Blueprint.h"
#include "Engine/StaticMesh.h"
#include "Engine/Texture2D.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"

#if WITH_EDITOR
#include "EditorViewportClient.h"
#endif

//////////////////////////////////////////////////////////////////////////
// FSphericalCoordinates

FDisplayClusterLightCardEditorHelper::FSphericalCoordinates::FSphericalCoordinates(const FVector& CartesianPosition)
{
	Radius = CartesianPosition.Size();

	if (Radius > UE_SMALL_NUMBER)
	{
		Inclination = FMath::Acos(CartesianPosition.Z / Radius);
	}
	else
	{
		Inclination = 0;
	}

	Azimuth = FMath::Atan2(CartesianPosition.Y, CartesianPosition.X);
}

FDisplayClusterLightCardEditorHelper::FSphericalCoordinates::FSphericalCoordinates()
	: Radius(0)
	, Inclination(0)
	, Azimuth(0)
{

}

FVector FDisplayClusterLightCardEditorHelper::FSphericalCoordinates::AsCartesian() const
{
	const double SinAzimuth = FMath::Sin(Azimuth);
	const double CosAzimuth = FMath::Cos(Azimuth);

	const double SinInclination = FMath::Sin(Inclination);
	const double CosInclination = FMath::Cos(Inclination);

	return FVector(
		Radius * CosAzimuth * SinInclination,
		Radius * SinAzimuth * SinInclination,
		Radius * CosInclination
	);
}

FDisplayClusterLightCardEditorHelper::FSphericalCoordinates
FDisplayClusterLightCardEditorHelper::FSphericalCoordinates::operator+(FSphericalCoordinates const& Other) const
{
	FSphericalCoordinates Result;

	Result.Radius = Radius + Other.Radius;
	Result.Inclination = Inclination + Other.Inclination;
	Result.Azimuth = Azimuth + Other.Azimuth;

	return Result;
}

FDisplayClusterLightCardEditorHelper::FSphericalCoordinates
FDisplayClusterLightCardEditorHelper::FSphericalCoordinates::operator-(FSphericalCoordinates const& Other) const
{
	FSphericalCoordinates Result;

	Result.Radius = Radius - Other.Radius;
	Result.Inclination = Inclination - Other.Inclination;
	Result.Azimuth = Azimuth - Other.Azimuth;

	return Result;
}

void FDisplayClusterLightCardEditorHelper::FSphericalCoordinates::Conform()
{
	if (Radius < 0)
	{
		Radius = -Radius;
		Inclination += PI;
	}

	if (Inclination < 0 || Inclination > PI)
	{
		// -2PI to 2PI
		Inclination = FMath::Fmod(Inclination, 2 * PI);

		// 0 to 2PI
		if (Inclination < 0)
		{
			Inclination += 2 * PI;
		}

		// 0 to PI
		if (Inclination > PI)
		{
			Inclination = 2 * PI - Inclination;
			Azimuth += PI;
		}
	}

	if (Azimuth < -PI || Azimuth > PI)
	{
		// -2PI to 2PI
		Azimuth = FMath::Fmod(Azimuth, 2 * PI);

		// -PI to PI
		if (Azimuth > PI)
		{
			Azimuth -= 2 * PI;
		}
		else if (Azimuth < -PI)
		{
			Azimuth += 2 * PI;
		}
	}

	checkSlow(Radius >= 0);
	checkSlow(Inclination >= 0 && Inclination <= PI);
	checkSlow(Azimuth >= -PI && Azimuth <= PI);
}

FDisplayClusterLightCardEditorHelper::FSphericalCoordinates
FDisplayClusterLightCardEditorHelper::FSphericalCoordinates::GetConformed() const
{
	FSphericalCoordinates Result = *this;
	Result.Conform();
	return Result;
}

bool FDisplayClusterLightCardEditorHelper::FSphericalCoordinates::IsPointingAtPole(double Margin) const
{
	FSphericalCoordinates CoordsConformed = GetConformed();

	return FMath::IsNearlyZero(CoordsConformed.Inclination, Margin)
		|| FMath::IsNearlyEqual(CoordsConformed.Inclination, PI, Margin);
}


//////////////////////////////////////////////////////////////////////////
// FDisplayClusterLightCardEditorHelper

FDisplayClusterLightCardEditorHelper::FDisplayClusterLightCardEditorHelper()
	: FDisplayClusterLightCardEditorHelper(IDisplayClusterScenePreview::Get().CreateRenderer())
{
	bCreatedRenderer = true;
}

FDisplayClusterLightCardEditorHelper::FDisplayClusterLightCardEditorHelper(int32 RendererId)
	: RendererId(RendererId), bCreatedRenderer(false)
{
}

FDisplayClusterLightCardEditorHelper::~FDisplayClusterLightCardEditorHelper()
{
	if (bCreatedRenderer)
	{
		IDisplayClusterScenePreview::Get().DestroyRenderer(RendererId);
	}

	FWorldDelegates::OnWorldCleanup.RemoveAll(this);

#if WITH_EDITORONLY_DATA
	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);

	if (ADisplayClusterRootActor* RootActor = CachedRootActor.Get())
	{
		if (UBlueprint* Blueprint = UBlueprint::GetBlueprintFromClass(RootActor->GetClass()))
		{
			Blueprint->OnCompiled().RemoveAll(this);
		}
	}
#endif
}

#if WITH_EDITOR
void FDisplayClusterLightCardEditorHelper::SetEditorViewportClient(TWeakPtr<FEditorViewportClient> InViewportClient)
{
	ViewportClient = InViewportClient;
}
#endif

void FDisplayClusterLightCardEditorHelper::SetProjectionMode(EDisplayClusterMeshProjectionType Value)
{
	ProjectionMode = Value;
}

EDisplayClusterMeshProjectionType FDisplayClusterLightCardEditorHelper::GetProjectionMode() const
{
	return ProjectionMode;
}

void FDisplayClusterLightCardEditorHelper::SetIsOrthographic(bool bValue)
{
	bIsOrthographic = bValue;
}

bool FDisplayClusterLightCardEditorHelper::GetIsOrthographic() const
{
	return bIsOrthographic;
}

void FDisplayClusterLightCardEditorHelper::SetRootActor(ADisplayClusterRootActor& NewRootActor)
{
	if (!ensureMsgf(bCreatedRenderer, TEXT("SetRootActor can't be called on an FDisplayClusterLightCardEditorHelper that was created with an existing preview renderer")))
	{
		return;
	}

	// Use custom settings for root actor.
	FDisplayClusterRootActorPropertyOverrides PropertyOverrides;
	{
		PropertyOverrides.bPreviewICVFXFrustums = false;
		PropertyOverrides.bEnablePreviewTechvis = false;
		PropertyOverrides.bPreviewEnableOverlayMaterial = false;
		PropertyOverrides.bFreezePreviewRender = false;

		PropertyOverrides.bPreviewEnablePostProcess = true;
		PropertyOverrides.bPreviewEnable = true;

		// Can render a DCRA preview without the HoldoutComposite plugin.
		PropertyOverrides.bPreviewEnableHoldoutComposite = false;

		PropertyOverrides.PreviewSetttingsSource = EDisplayClusterConfigurationRootActorPreviewSettingsSource::RootActor;
	}

	IDisplayClusterScenePreview::Get().SetRendererRootActor(RendererId, &NewRootActor, PropertyOverrides);
}

void FDisplayClusterLightCardEditorHelper::SetLevelInstanceRootActor(ADisplayClusterRootActor& NewRootActor)
{
	LevelInstanceRootActor = &NewRootActor;
}

const UTexture2D* FDisplayClusterLightCardEditorHelper::GetNormalMapTexture(bool bShowNorthMap)
{
	// TODO: Exposure to the raw stage geometry normal maps was removed since they are generally undecipherable, and an upcoming task (UE-153862) will add a
	// way to visualize the normal maps in a way that is far more user friendly
	return nullptr;
}

void FDisplayClusterLightCardEditorHelper::MoveActorsToPixel(
	const TArray<FDisplayClusterWeakStageActorPtr>& Actors, const FIntPoint& PixelPos, const FSceneView& SceneView)
{
	UpdateProjectionOriginComponent();

	FVector Origin;
	FVector Direction;
	CalculateOriginAndDirectionFromPixelPosition(PixelPos, SceneView, FVector::ZeroVector, Origin, Direction);

	if (ProjectionMode == EDisplayClusterMeshProjectionType::UV)
	{
		// Find the average position of all selected actors. This group average is what is moved to the specified pixel
		FVector2D AverageUVCoords = FVector2D::ZeroVector;
		int32 NumLightCards = 0;

		for (const FDisplayClusterWeakStageActorPtr& Actor : Actors)
		{
			if (Actor.IsValid() && Actor->IsUVActor())
			{
				AverageUVCoords += Actor->GetUVCoordinates();
				++NumLightCards;
			}
		}

		AverageUVCoords /= NumLightCards;

		// Compute the desired coordinates by projecting the specified screen ray onto the UV projection plane
		const float UVProjectionPlaneSize = ADisplayClusterLightCardActor::UVPlaneDefaultSize;
		const float UVProjectionPlaneDistance = ADisplayClusterLightCardActor::UVPlaneDefaultDistance;

		const FPlane UVProjectionPlane(FVector::ForwardVector * UVProjectionPlaneDistance, -FVector::ForwardVector);
		const FVector PlaneIntersection = FMath::RayPlaneIntersection(FVector::ZeroVector, Direction, UVProjectionPlane);
		const FVector2D DesiredUVCoords = FVector2D(PlaneIntersection.Y / UVProjectionPlaneSize + 0.5f, 0.5f - PlaneIntersection.Z / UVProjectionPlaneSize);

		const FVector2D DeltaUVCoords = DesiredUVCoords - AverageUVCoords;

		for (const FDisplayClusterWeakStageActorPtr& Actor : Actors)
		{
			if (Actor.IsValid() && Actor->IsUVActor())
			{
				Actor->SetUVCoordinates(Actor->GetUVCoordinates() + DeltaUVCoords);
			}
		}
	}
	else
	{
		// Find the average position of all selected light cards. This group average is what is moved to the specified pixel
		FSphericalCoordinates AverageCoords = FSphericalCoordinates();
		int32 NumActors = 0;

		for (const FDisplayClusterWeakStageActorPtr& Actor : Actors)
		{
			if (Actor.IsValid())
			{
				VerifyAndFixActorOrigin(Actor);

				const FSphericalCoordinates ActorCoords = GetActorCoordinates(Actor);

				AverageCoords = AverageCoords + ActorCoords;
				++NumActors;
			}
		}

		if (NumActors == 0)
		{
			NumActors = 1;
		}
		
		AverageCoords.Radius /= NumActors;
		AverageCoords.Azimuth /= NumActors;
		AverageCoords.Inclination /= NumActors;
		AverageCoords.Conform();

		// Compute desired coordinates (radius doesn't matter here since we will use the flush constraint on the light cards after moving them)
		const FVector LocalDirection = CachedRootActor->GetActorRotation().RotateVector(Direction);
		const FSphericalCoordinates DesiredCoords(LocalDirection * 100.0f);
		const FSphericalCoordinates DeltaCoords = DesiredCoords - AverageCoords;

		// Update each light card with the delta coordinates; the flush constraint is applied by MoveLightCardTo, ensuring the light card is always flush to screens
		for (const FDisplayClusterWeakStageActorPtr& LightCard : Actors)
		{
			if (LightCard.IsValid() && !LightCard->IsUVActor())
			{
				const FSphericalCoordinates LightCardCoords = GetActorCoordinates(LightCard);
				const FSphericalCoordinates NewCoords = LightCardCoords + DeltaCoords;

				MoveActorsTo({ LightCard }, NewCoords);
			}
		}
	}
}

void FDisplayClusterLightCardEditorHelper::MoveActorsTo(
	const TArray<FDisplayClusterWeakStageActorPtr>& Actors, const FSphericalCoordinates& SphericalCoords)
{
	ADisplayClusterRootActor* RootActor = UpdateRootActor();
	if (!RootActor)
	{
		return;
	}

	if (!UpdateNormalMaps())
	{
		return;
	}

	for (const FDisplayClusterWeakStageActorPtr& Actor : Actors)
	{
		if (!Actor.IsValid())
		{
			continue;
		}

		InternalMoveActorTo(Actor, SphericalCoords, true);
	}
}

void FDisplayClusterLightCardEditorHelper::DragActors(
	const TArray<FDisplayClusterWeakStageActorPtr>& Actors, const FIntPoint& PixelPos,
	const FSceneView& SceneView, ECoordinateSystem CoordinateSystem, const FVector& DragWidgetOffset, EAxisList::Type DragAxis,
	FDisplayClusterWeakStageActorPtr PrimaryActor)
{
	if (Actors.IsEmpty() || !UpdateNormalMaps() || !UpdateRootActor())
	{
		return;
	}

	InternalDragActors(Actors, PixelPos, SceneView, CoordinateSystem, DragWidgetOffset, DragAxis, PrimaryActor);
}

void FDisplayClusterLightCardEditorHelper::DragUVActors(
	const TArray<FDisplayClusterWeakStageActorPtr>& Actors, const FIntPoint& PixelPos, const FSceneView& SceneView,
	const FVector& DragWidgetOffset, EAxisList::Type DragAxis, FDisplayClusterWeakStageActorPtr PrimaryActor)
{
	if (Actors.IsEmpty() || !UpdateNormalMaps() || !UpdateRootActor())
	{
		return;
	}

	if (!PrimaryActor.IsValid())
	{
		PrimaryActor = Actors.Last();
	}

	if (!PrimaryActor.IsValid() || !PrimaryActor->IsUVActor())
	{
		return;
	}

	const FVector2D DeltaUV = GetUVActorTranslationDelta(PixelPos, SceneView, PrimaryActor, DragAxis, DragWidgetOffset);
	for (const FDisplayClusterWeakStageActorPtr& Actor : Actors)
	{
		if (!Actor.IsValid() || !Actor->IsUVActor())
		{
			continue;
		}

		Actor->SetUVCoordinates(Actor->GetUVCoordinates() + DeltaUV);

#if WITH_EDITOR
		if (!Actor->IsProxy())
		{
			PostEditChangePropertiesForMovedActor(Actor);
		}
#endif
	}
}

void FDisplayClusterLightCardEditorHelper::VerifyAndFixActorOrigin(const FDisplayClusterWeakStageActorPtr& Actor)
{
	// Center actor on the current view point, let it keep its current world placement 
	// (but not its spin/yaw/pitch since that will be happen later using the cache)

	const ADisplayClusterRootActor* RootActor = UpdateRootActor();
	const ADisplayClusterRootActor* OwningRootActor = (Actor->IsProxy() || !LevelInstanceRootActor.IsValid()) ? RootActor : LevelInstanceRootActor.Get();

	const USceneComponent* OriginComponent = nullptr;
	if (OwningRootActor)
	{
		OriginComponent = Actor->IsProxy() && ProjectionOriginComponent.IsValid() ? ProjectionOriginComponent.Get() : OwningRootActor->GetCommonViewPoint();
	}

	if (!OriginComponent)
	{
		return;
	}

	// Set location and rotation to match the root actor
	const FVector& NewActorLocation = OriginComponent ? OriginComponent->GetComponentLocation() : FVector::ZeroVector;
	const FRotator& NewActorRotation = OwningRootActor ? OwningRootActor->GetActorRotation() : FRotator::ZeroRotator;

	const FTransform OldOrigin = Actor->GetOrigin();
	Actor->SetOrigin({ NewActorRotation, NewActorLocation, FVector::One() });

	const FTransform NewOrigin = Actor->GetOrigin();
	bool bActorMoved = !OldOrigin.Equals(NewOrigin, 0.f);
	
	if (Actor.AsActorChecked()->IsA<ADisplayClusterLightCardActor>())
	{
		const FDisplayClusterPositionalParams OldPositionalParams = Actor->GetPositionalParams();
	
		// Update the light card spherical coordinates to match its current world coordinates
		const FVector LightCardEndEffectorLocation = Actor->GetStageActorTransform(true).GetLocation();
		const FVector ActorRelativeLocation = NewActorRotation.UnrotateVector(LightCardEndEffectorLocation);

		const FSphericalCoordinates SphericalCoords(ActorRelativeLocation);
		SetActorCoordinates(Actor, SphericalCoords);

		const FDisplayClusterPositionalParams NewPositionalParams = Actor->GetPositionalParams();
		bActorMoved = bActorMoved || NewPositionalParams != OldPositionalParams;
	}

#if WITH_EDITOR
	if (bActorMoved)
	{
		PostEditChangePropertiesForMovedActor(Actor);
	}
#endif
}

bool FDisplayClusterLightCardEditorHelper::CalculateNormalAndPositionInDirection(
	const FVector& InOrigin,
	const FVector& InDirection,
	FVector& OutWorldPosition,
	FVector& OutRelativeNormal,
	double InDesiredDistanceFromFlush)
{
	if (!UpdateNormalMaps())
	{
		return false;
	}

	if (ProjectionMode == EDisplayClusterMeshProjectionType::UV)
	{
		// In UV projection mode, all relevant geometry is projected onto the UV plane, so compute the world position and normal
		// by performing a ray-plane intersection on the UV projection plane

		const float UVProjectionPlaneDistance = ADisplayClusterLightCardActor::UVPlaneDefaultDistance;

		const FPlane UVProjectionPlane(InOrigin + FVector::ForwardVector * UVProjectionPlaneDistance, -FVector::ForwardVector);
		const FVector PlaneIntersection = FMath::RayPlaneIntersection(InOrigin, InDirection, UVProjectionPlane);

		OutRelativeNormal = UVProjectionPlane.GetNormal();
		OutWorldPosition = PlaneIntersection;
	}
	else
	{
		float Distance;

		CachedRootActor->GetStageGeometryComponent()->GetStageDistanceAndNormal(InDirection, Distance, OutRelativeNormal);
		Distance = CalculateFinalLightCardDistance(Distance, InDesiredDistanceFromFlush);

		// Calculate world position
		OutWorldPosition = InOrigin + Distance * InDirection;
	}

	return true;
}

bool FDisplayClusterLightCardEditorHelper::CalculateOriginAndDirectionFromPixelPosition(const FIntPoint& PixelPos, const FSceneView& SceneView, const FVector& OriginOffset,
	FVector& OutOrigin, FVector& OutDirection)
{
	PixelToWorld(SceneView, PixelPos, OutOrigin, OutDirection);

	if (bIsOrthographic)
	{
		if (!UpdateNormalMaps())
		{
			return false;
		}

		// For orthogonal projections, PixelToWorld does not return the view point or a direction from the view point. Use TraceScreenRay
		// to find a useful direction away from the origin to use

		const FVector NewOrigin = SceneView.ViewLocation;

		OutDirection = TraceScreenRay(OutOrigin + OriginOffset, OutDirection, NewOrigin);
		OutOrigin = NewOrigin;
	}

	return true;
}

void FDisplayClusterLightCardEditorHelper::PixelToWorld(const FSceneView& View, const FIntPoint& PixelPos, FVector& OutOrigin, FVector& OutDirection) const
{
	const FMatrix& InvProjMatrix = View.ViewMatrices.GetClipToView();
	FMatrix InvViewMatrix = View.ViewMatrices.GetViewToWorld();

	// Cancel out the root actor's orientation
	if (USceneComponent* OriginComponent = ProjectionOriginComponent.Get())
	{
		if (const AActor* Actor = OriginComponent->GetOwner())
		{
			if (!bIsOrthographic)
			{
				FTransform Transform;

				Transform.SetRotation(Actor->GetActorRotation().Quaternion());
				Transform.SetTranslation(OriginComponent->GetComponentLocation());

				InvViewMatrix *= Transform.ToInverseMatrixWithScale();
			}
		}
	}

	FVector4 ScreenPos = View.PixelToScreen(PixelPos.X, PixelPos.Y, 0);
	ScreenPos.Z = 1; // Force near clip plane

	FVector4 ViewPos = FVector(InvProjMatrix.TransformFVector4(ScreenPos));
	ViewPos /= ViewPos.W;

	if (bIsOrthographic)
	{
		ViewPos.Z = 0;
	}

	const FVector UnprojectedViewPos = FDisplayClusterMeshProjectionRenderer::UnprojectViewPosition(ViewPos, ProjectionMode);

	if (bIsOrthographic)
	{
		OutOrigin = InvViewMatrix.TransformFVector4(UnprojectedViewPos);
		OutDirection = InvViewMatrix.TransformVector(FVector(0, 0, 1)).GetSafeNormal();
	}
	else
	{
		OutOrigin = View.ViewMatrices.GetViewOrigin();
		OutDirection = InvViewMatrix.TransformVector(UnprojectedViewPos).GetSafeNormal();
	}
}

bool FDisplayClusterLightCardEditorHelper::WorldToPixel(const FSceneView& View, const FVector& WorldPos, FVector2D& OutPixelPos) const
{
	return WorldToPixel(View, WorldPos, OutPixelPos, ProjectionMode);
}

bool FDisplayClusterLightCardEditorHelper::WorldToPixel(const FSceneView& View, const FVector& WorldPos, FVector2D& OutPixelPos, EDisplayClusterMeshProjectionType OverrideProjectionMode) const
{
	const FMatrix& ViewMatrix = View.ViewMatrices.GetWorldToView();
	const FMatrix& ProjMatrix = View.ViewMatrices.GetViewToClip();

	const FVector ViewPos = ViewMatrix.TransformPosition(WorldPos);
	const FVector ProjectedViewPos = FDisplayClusterMeshProjectionRenderer::ProjectViewPosition(ViewPos, OverrideProjectionMode);
	const FVector4 ScreenPos = ProjMatrix.TransformFVector4(FVector4(ProjectedViewPos, 1));

	return View.ScreenToPixel(ScreenPos, OutPixelPos);
}

void FDisplayClusterLightCardEditorHelper::GetSceneViewInitOptions(
	FSceneViewInitOptions& OutViewInitOptions,
	float InFOV,
	const FIntPoint& InViewportSize,
	const FVector& InLocation,
	const FRotator& InRotation,
	const EAspectRatioAxisConstraint InAspectRatioAxisConstraint,
	float InNearClipPlane,
	const FMatrix* InRotationMatrix,
	float InDPIScale)
{
	OutViewInitOptions.ViewLocation = InLocation;
	OutViewInitOptions.ViewRotation = InRotation;
	OutViewInitOptions.ViewOrigin = OutViewInitOptions.ViewLocation;

	FIntPoint ViewportSize = InViewportSize;
	ViewportSize.X = FMath::Max(ViewportSize.X, 1);
	ViewportSize.Y = FMath::Max(ViewportSize.Y, 1);
	FIntPoint ViewportOffset(0, 0);

	OutViewInitOptions.SetViewRectangle(FIntRect(ViewportOffset, ViewportOffset + ViewportSize));

	if (const ADisplayClusterRootActor* RootActor = UpdateRootActor())
	{
		const AWorldSettings* WorldSettings = nullptr;
		if (RootActor->GetWorld() != nullptr)
		{
			WorldSettings = RootActor->GetWorld()->GetWorldSettings();
		}

		if (WorldSettings != nullptr)
		{
			OutViewInitOptions.WorldToMetersScale = WorldSettings->WorldToMeters;
		}
	}

	// Rotate view 90 degrees
	const FMatrix Rotate90(
		FPlane(0, 0, 1, 0),
		FPlane(1, 0, 0, 0),
		FPlane(0, 1, 0, 0),
		FPlane(0, 0, 0, 1)
	);
	OutViewInitOptions.ViewRotationMatrix = (InRotationMatrix ? *InRotationMatrix : FInverseRotationMatrix(OutViewInitOptions.ViewRotation)) * Rotate90;

	// Avoid zero ViewFOV's which cause divide by zero's in projection matrix
	const float RadianFOV = FMath::DegreesToRadians(InFOV);
	const float HalfFOV = FMath::Max(0.001f, RadianFOV * 0.5f);

	// Determine FOV multipliers to match render target's aspect ratio
	float XAxisMultiplier;
	float YAxisMultiplier;

	if (((ViewportSize.X > ViewportSize.Y) && (InAspectRatioAxisConstraint == AspectRatio_MajorAxisFOV)) || (InAspectRatioAxisConstraint == AspectRatio_MaintainXFOV))
	{
		//if the viewport is wider than it is tall
		XAxisMultiplier = 1.0f;
		YAxisMultiplier = ViewportSize.X / (float)ViewportSize.Y;
	}
	else
	{
		//if the viewport is taller than it is wide
		XAxisMultiplier = ViewportSize.Y / (float)ViewportSize.X;
		YAxisMultiplier = 1.0f;
	}

	if (bIsOrthographic)
	{
		const float ZScale = 0.5f / UE_OLD_HALF_WORLD_MAX;
		const float ZOffset = UE_OLD_HALF_WORLD_MAX;

		const float FOVScale = FMath::Tan(HalfFOV) / InDPIScale;

		const float OrthoWidth = 0.5f * FOVScale * ViewportSize.X;
		const float OrthoHeight = 0.5f * FOVScale * ViewportSize.Y;

		OutViewInitOptions.ProjectionMatrix = FReversedZOrthoMatrix(
			OrthoWidth,
			OrthoHeight,
			ZScale,
			ZOffset
		);
	}
	else
	{
		OutViewInitOptions.ProjectionMatrix = FReversedZPerspectiveMatrix(
			HalfFOV,
			HalfFOV,
			XAxisMultiplier,
			YAxisMultiplier,
			InNearClipPlane,
			InNearClipPlane
		);
	}

	if (!OutViewInitOptions.IsValidViewRectangle())
	{
		// Zero sized rects are invalid, so fake to 1x1 to avoid asserts later on
		OutViewInitOptions.SetViewRectangle(FIntRect(0, 0, 1, 1));
	}

	OutViewInitOptions.BackgroundColor = FLinearColor::Black;
	OutViewInitOptions.FOV = InFOV;
}

void FDisplayClusterLightCardEditorHelper::ConfigureRenderProjectionSettings(FDisplayClusterMeshProjectionRenderSettings& OutRenderSettings, const FVector ViewLocation) const
{
	OutRenderSettings.ProjectionType = ProjectionMode;
	if (ProjectionMode == EDisplayClusterMeshProjectionType::UV)
	{
		OutRenderSettings.ProjectionTypeSettings.UVProjectionIndex = 1;
		OutRenderSettings.ProjectionTypeSettings.UVProjectionPlaneSize = ADisplayClusterLightCardActor::UVPlaneDefaultSize;
		OutRenderSettings.ProjectionTypeSettings.UVProjectionPlaneDistance = ADisplayClusterLightCardActor::UVPlaneDefaultDistance;

		// Compute the UV plane offset to allow panning. Need to convert to view space, since the UV projection assumes all coordinates are in view space
		const FMatrix WorldToViewTransform = FMatrix(FVector::ZAxisVector, FVector::XAxisVector, FVector::YAxisVector, FVector::ZeroVector);
		OutRenderSettings.ProjectionTypeSettings.UVProjectionPlaneOffset = -WorldToViewTransform.TransformVector(ViewLocation);
	}
}

FDisplayClusterLightCardEditorHelper::FSphericalCoordinates FDisplayClusterLightCardEditorHelper::GetActorCoordinates(const FDisplayClusterWeakStageActorPtr& Actor)
{
	const FVector ActorLocation = Actor->GetStageActorTransform(true).GetTranslation();
	FSphericalCoordinates ActorSphericalCoords(ActorLocation);

	// If the light card points at any of the poles, the spherical coordinates will have an "undefined" azimuth value. 
	// For continuity when dragging a light card positioned there, 
	// we can manually set the azimuthal value to match the light card's configured longitude

	if (ActorSphericalCoords.IsPointingAtPole())
	{
		ActorSphericalCoords.Azimuth = FMath::DegreesToRadians(Actor->GetLongitude() + 180.f);
	}

	return ActorSphericalCoords;
}

FDisplayClusterMeshProjectionPrimitiveFilter::FPrimitiveFilter FDisplayClusterLightCardEditorHelper::CreateDefaultShouldRenderPrimitiveFilter() const
{
	const bool bIsUVProjection = ProjectionMode == EDisplayClusterMeshProjectionType::UV;

	// Create a lambda function so that it's safe to access even if this helper is destroyed
	return FDisplayClusterMeshProjectionPrimitiveFilter::FPrimitiveFilter::CreateLambda([bIsUVProjection](const UPrimitiveComponent* PrimitiveComponent)
		{
			if (IDisplayClusterStageActor* StageActor = Cast<IDisplayClusterStageActor>(PrimitiveComponent->GetOwner()))
			{
				// Only render the UV actors when in UV projection mode, and only render non-UV actors in any other projection mode
				return bIsUVProjection ? StageActor->IsUVActor() : !StageActor->IsUVActor();
			}

			return true;
		});
}

FDisplayClusterMeshProjectionPrimitiveFilter::FPrimitiveFilter FDisplayClusterLightCardEditorHelper::CreateDefaultShouldApplyProjectionToPrimitiveFilter() const
{
	const bool bIsUVProjection = ProjectionMode == EDisplayClusterMeshProjectionType::UV;

	// Create a lambda function so that it's safe to access even if this helper is destroyed
	return FDisplayClusterMeshProjectionPrimitiveFilter::FPrimitiveFilter::CreateLambda([bIsUVProjection](const UPrimitiveComponent* PrimitiveComponent)
		{
			if (IDisplayClusterStageActor* StageActor = Cast<IDisplayClusterStageActor>(PrimitiveComponent->GetOwner()))
			{
				// When in UV projection mode, don't render the UV actors using the UV projection, render them linearly
				if (bIsUVProjection && StageActor->IsUVActor())
				{
					return false;
				}
			}

			return true;
		});
}

AActor* FDisplayClusterLightCardEditorHelper::SpawnStageActor(const FSpawnActorArgs& InSpawnArgs)
{
	ADisplayClusterRootActor* RootActor = InSpawnArgs.RootActor;
	const TSubclassOf<AActor> ActorClass = InSpawnArgs.ActorClass;
	const UDisplayClusterStageActorTemplate* Template = InSpawnArgs.Template;
	
	check(RootActor)
	check(ActorClass || Template);
	
	FName ActorName = InSpawnArgs.ActorName;
	const EDisplayClusterMeshProjectionType ProjectionMode = InSpawnArgs.ProjectionMode;
	ULevel* Level = InSpawnArgs.Level ? InSpawnArgs.Level : RootActor->GetWorld()->GetCurrentLevel();
	const bool bIsPreview = InSpawnArgs.bIsPreview;

	AActor* NewActor = nullptr;
	
	if (Template)
	{
		const AActor* TemplateActor = Template->GetTemplateActor();
		check(TemplateActor);

		// For now only light card templates are supported
		check(TemplateActor->GetClass()->IsChildOf(ADisplayClusterLightCardActor::StaticClass()));
		
		FName UniqueName = *Template->GetName().Replace(TEXT("Template"), TEXT(""));
		if (StaticFindObjectFast(TemplateActor->GetClass(), Level, UniqueName))
		{
			UniqueName = MakeUniqueObjectName(Level, TemplateActor->GetClass(), UniqueName);
		}
	
		// Duplicate, don't copy properties or spawn from a template. Doing so will copy component data incorrectly,
		// specifically the static mesh override textures. They will be parented to the template, not the level instance
		// and prevent the map from saving.
		ADisplayClusterLightCardActor* NewLightCard = CastChecked<ADisplayClusterLightCardActor>(StaticDuplicateObject(TemplateActor, Level, UniqueName));
		ActorName = UniqueName;

#if WITH_EDITOR
		Level->AddLoadedActor(NewLightCard);
#endif
		
		NewActor = NewLightCard;
	}
	else
	{
		const FVector SpawnLocation = RootActor->GetDefaultCamera()->GetComponentLocation();
		FRotator SpawnRotation = RootActor->GetDefaultCamera()->GetComponentRotation();
		SpawnRotation.Yaw -= 180.f;

		FActorSpawnParameters SpawnParameters;
		SpawnParameters.bNoFail = true;
		SpawnParameters.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AdjustIfPossibleButAlwaysSpawn;
		SpawnParameters.Name = ActorName;
		SpawnParameters.NameMode = FActorSpawnParameters::ESpawnActorNameMode::Requested;
		SpawnParameters.OverrideLevel = Level;

		NewActor = CastChecked<AActor>(
			RootActor->GetWorld()->SpawnActor(ActorClass,
				&SpawnLocation, &SpawnRotation, MoveTemp(SpawnParameters)));
	}

	check(NewActor);

#if WITH_EDITOR
	if (!bIsPreview)
	{
		FActorLabelUtilities::SetActorLabelUnique(NewActor, ActorName.ToString());
	}
#endif
	
	if (ADisplayClusterLightCardActor* NewLightCard = Cast<ADisplayClusterLightCardActor>(NewActor))
	{
		if (ProjectionMode == EDisplayClusterMeshProjectionType::UV)
		{
			// If this already is a UV light card leave everything alone
			if (!NewLightCard->bIsUVLightCard)
			{
				NewLightCard->bIsUVLightCard = true;
				NewLightCard->Scale /= 4;

				// Don't override feathering if this was spawned from a template
				if (!Template)
				{
					NewLightCard->Feathering = 0.05f; // Just enough to avoid jagged look on UV lightcards.
				}
			}
		}

		FDisplayClusterLabelConfiguration LabelConfiguration = InSpawnArgs.AddLightCardArgs.LabelConfiguration;
		LabelConfiguration.RootActor = RootActor;

		NewLightCard->ShowLightCardLabel(LabelConfiguration);
		
		if (ADisplayClusterChromakeyCardActor* ChromakeyCardActor = Cast<ADisplayClusterChromakeyCardActor>(NewActor))
		{
			ChromakeyCardActor->AddToRootActor(RootActor);
		}
		else if (!bIsPreview)
		{
			AddLightCardsToRootActor({ NewLightCard }, RootActor, InSpawnArgs.AddLightCardArgs);
		}
	}

#if WITH_EDITOR
		// Need to call this if spawned from a template since this would normally be called in SpawnActor
		if (Template && GIsEditor)
		{
			GEditor->BroadcastLevelActorAdded(NewActor);
		}
#endif

	return NewActor;
}

void FDisplayClusterLightCardEditorHelper::AddLightCardsToRootActor(
	const TArray<ADisplayClusterLightCardActor*>& LightCards, ADisplayClusterRootActor* RootActor,
	const FAddLightCardArgs& AddLightCardArgs)
{
	if (LightCards.Num() == 0)
	{
		return;
	}

	check(RootActor);
	
	UDisplayClusterConfigurationData* ConfigData = RootActor->GetConfigData();
	ConfigData->Modify();
	FDisplayClusterConfigurationICVFX_VisibilityList& RootActorLightCards = ConfigData->StageSettings.Lightcard.ShowOnlyList;
	
	for (ADisplayClusterLightCardActor* LightCard : LightCards)
	{
		check(LightCard);

		if (!RootActorLightCards.Actors.ContainsByPredicate([&](const TSoftObjectPtr<AActor>& Actor)
			{
				// Don't add if a loaded actor is already present.
				return Actor.Get() == LightCard;
			}))
		{
			const TSoftObjectPtr<AActor> LightCardSoftObject(LightCard);

			// Remove any exact paths to this actor. It's possible invalid actors are present if a light card
			// was force deleted from a level.
			RootActorLightCards.Actors.RemoveAll([&](const TSoftObjectPtr<AActor>& Actor)
				{
					return Actor == LightCardSoftObject;
				});

			LightCard->AddToRootActor(RootActor);
		}

#if WITH_EDITOR
		// Fire this event so that e.g. ICVFX panel can update proxies based on the new root/layer
		LightCard->PostEditChange();
#endif
	}
}

ADisplayClusterRootActor* FDisplayClusterLightCardEditorHelper::UpdateRootActor()
{
	ADisplayClusterRootActor* NewRootActor = IDisplayClusterScenePreview::Get().GetRendererRootActor(RendererId);
	if (NewRootActor == CachedRootActor)
	{
		return NewRootActor;
	}

#if WITH_EDITOR
	// Clean up old subscribed events
	if (ADisplayClusterRootActor* RootActor = CachedRootActor.Get())
	{
		if (UBlueprint* Blueprint = UBlueprint::GetBlueprintFromClass(RootActor->GetClass()))
		{
			Blueprint->OnCompiled().RemoveAll(this);
		}
	}

	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);
#endif

	CachedRootActor = NewRootActor;
	InvalidateNormalMap();

	if (NewRootActor)
	{
		RootActorBoundingRadius = NewRootActor->GetStageGeometryComponent()->GetStageBoundingRadius();

#if WITH_EDITOR
		FCoreUObjectDelegates::OnObjectPropertyChanged.AddRaw(this, &FDisplayClusterLightCardEditorHelper::OnActorPropertyChanged);

		if (UBlueprint* Blueprint = UBlueprint::GetBlueprintFromClass(NewRootActor->GetClass()))
		{
			Blueprint->OnCompiled().AddRaw(this, &FDisplayClusterLightCardEditorHelper::OnRootActorBlueprintCompiled);
		}
#endif
	}
	else
	{
		RootActorBoundingRadius = 0.f;
	}

	return NewRootActor;
}

USceneComponent* FDisplayClusterLightCardEditorHelper::UpdateProjectionOriginComponent()
{
	if (ADisplayClusterRootActor* RootActor = UpdateRootActor())
	{
		ProjectionOriginComponent = RootActor->GetCommonViewPoint();
	}
	else
	{
		ProjectionOriginComponent = nullptr;
	}

	return ProjectionOriginComponent.Get();
}

FVector FDisplayClusterLightCardEditorHelper::GetProjectionOrigin() const
{
	if (ProjectionOriginComponent.IsValid())
	{
		return ProjectionOriginComponent->GetComponentTransform().GetLocation();
	}

	if (CachedRootActor.IsValid())
	{
		return CachedRootActor->GetTransform().GetLocation();
	}

	return FVector::Zero();
}

FRotator FDisplayClusterLightCardEditorHelper::GetActorRotationDelta(const FIntPoint& PixelPos, const FSceneView& View, const FDisplayClusterWeakStageActorPtr& Actor,
	ECoordinateSystem CoordinateSystem, EAxisList::Type DragAxis, const FVector& DragWidgetOffset)
{
	const FSphericalCoordinates DeltaCoords = GetActorTranslationDelta(PixelPos, View, Actor, CoordinateSystem, DragAxis, DragWidgetOffset);
	const FSphericalCoordinates LightCardCoords = GetActorCoordinates(Actor);
	const FSphericalCoordinates NewCoords = LightCardCoords + DeltaCoords;

	const FVector LightCardPos = LightCardCoords.AsCartesian();
	const FVector NewPos = NewCoords.AsCartesian();

	const FVector PosCrossProduct = FVector::CrossProduct(LightCardPos, NewPos);

	if (FMath::IsNearlyZero(PosCrossProduct.Length()))
	{
		return FQuat(FVector::ForwardVector, 0).Rotator();
	}

	const FVector AxisOfRotation = PosCrossProduct.GetSafeNormal();
	const double Angle = FMath::Acos(FVector::DotProduct(LightCardPos.GetSafeNormal(), NewPos.GetSafeNormal()));

	return FQuat(AxisOfRotation, Angle).Rotator();
}

FDisplayClusterLightCardEditorHelper::FSphericalCoordinates FDisplayClusterLightCardEditorHelper::GetActorTranslationDelta(
	const FIntPoint& PixelPos,
	const FSceneView& View,
	const FDisplayClusterWeakStageActorPtr& Actor,
	ECoordinateSystem CoordinateSystem,
	EAxisList::Type DragAxis,
	const FVector& DragWidgetOffset)
{
	FVector Origin;
	FVector Direction;
	CalculateOriginAndDirectionFromPixelPosition(PixelPos, View, -DragWidgetOffset, Origin, Direction);

	if (!bIsOrthographic)
	{
		Direction = (Direction - DragWidgetOffset).GetSafeNormal();
	}

	checkSlow(CachedRootActor.IsValid());

	const FVector LocalDirection = CachedRootActor->GetActorRotation().RotateVector(Direction);
	const FVector LightCardLocation = Actor->GetStageActorTransform().GetTranslation() - Origin;
	const FSphericalCoordinates ActorCoords = GetActorCoordinates(Actor);
	FSphericalCoordinates DeltaCoords;

	// If we are in a cartesian coordinate system and are constraining to an axis, perform the constraint calculations in
	// cartesian coordinates, then convert to spherical coordinates at the end. Otherwise, perform all calculations in spherical coordinates
	if (CoordinateSystem == ECoordinateSystem::Cartesian && DragAxis != EAxisList::Type::XYZ)
	{
		// For consistency, project the cursor direction vector onto the sphere the light card is currently on. Gives a good balance of approximating
		// the true projection plane (one that works with all view projections) and the general stage normal map
		const FVector RequestedLocation = LocalDirection * LightCardLocation.Size();

		// Compute the axis to constrain the translation to based on the axis that was dragged. Axis must be rotated into the light card's local space
		FVector Axis = FVector::ZeroVector;
		if (DragAxis == EAxisList::Type::X)
		{
			Axis = FVector::XAxisVector;
		}
		else if (DragAxis == EAxisList::Type::Y)
		{
			Axis = FVector::YAxisVector;
		}
		else if (DragAxis == EAxisList::Type::Z)
		{
			Axis = FVector::ZAxisVector;
		}

		const FVector LocalAxis = CachedRootActor->GetActorRotation().RotateVector(Axis);

		// Compute the offset between the requested location and the light card's current location, and project that
		// offset onto the constraint axis
		const FVector DeltaLocation = ((RequestedLocation - LightCardLocation) | LocalAxis) * LocalAxis;

		const FVector ConstrainedLocation = LightCardLocation + DeltaLocation;

		const FSphericalCoordinates ConstrainedCoords(ConstrainedLocation);
		DeltaCoords = ConstrainedCoords - ActorCoords;
	}
	else
	{
		FVector FlushPosition;
		FVector FlushNormal;

		CachedRootActor->GetFlushPositionAndNormal(Actor->GetStageActorTransform().GetTranslation(), FlushPosition, FlushNormal);
		const float Distance = (FlushPosition - CachedRootActor->GetCommonViewPoint()->GetComponentLocation()).Size();
		const FSphericalCoordinates RequestedCoords(LocalDirection * Distance);

		DeltaCoords = RequestedCoords - ActorCoords;

		if (CoordinateSystem == ECoordinateSystem::Spherical)
		{
			if (DragAxis == EAxisList::Type::X)
			{
				DeltaCoords.Inclination = 0;
			}
			else if (DragAxis == EAxisList::Type::Y)
			{
				// Convert the inclination to Cartesian coordinates, project it to the x-z plane, and convert back to spherical coordinates. This ensures that the motion in the inclination
				// plane always lines up with the mouse's projected location along that plane
				const double FixedInclination = FMath::Abs(FMath::Atan2(
					FMath::Cos(DeltaCoords.Azimuth) * FMath::Sin(RequestedCoords.Inclination),
					FMath::Cos(RequestedCoords.Inclination))
				);

				// When translating along the inclination axis, the azimuth delta can only be intervals of pi
				const double FixedAzimuth = FMath::RoundToInt(DeltaCoords.Azimuth / PI) * PI;

				DeltaCoords.Azimuth = FixedAzimuth;
				DeltaCoords.Inclination = FixedInclination - ActorCoords.Inclination;
			}
		}
	}

	return DeltaCoords;
}

FVector2D FDisplayClusterLightCardEditorHelper::GetUVActorTranslationDelta(
	const FIntPoint& PixelPos,
	const FSceneView& View,
	const FDisplayClusterWeakStageActorPtr& Actor,
	EAxisList::Type DragAxis,
	const FVector& DragWidgetOffset)
{
	FVector Origin;
	FVector Direction;
	PixelToWorld(View, PixelPos, Origin, Direction);

	const float UVProjectionPlaneSize = ADisplayClusterLightCardActor::UVPlaneDefaultSize;
	const float UVProjectionPlaneDistance = ADisplayClusterLightCardActor::UVPlaneDefaultDistance;

	const FVector ViewOrigin = View.ViewMatrices.GetViewOrigin();
	const FPlane UVProjectionPlane(ViewOrigin + FVector::ForwardVector * UVProjectionPlaneDistance, -FVector::ForwardVector);
	const FVector PlaneIntersection = FMath::RayPlaneIntersection(Origin, Direction, UVProjectionPlane);
	
	const FVector DesiredLocation = PlaneIntersection - DragWidgetOffset;
	const FVector2D DesiredUVLocation = FVector2D(DesiredLocation.Y / UVProjectionPlaneSize + 0.5f, 0.5f - DesiredLocation.Z / UVProjectionPlaneSize);

	const FVector2D UVDelta = DesiredUVLocation - Actor->GetUVCoordinates();

	FVector2D UVAxis = FVector2D::ZeroVector;
	if (DragAxis & EAxisList::Type::X)
	{
		UVAxis += FVector2D(1.0, 0.0);
	}

	if (DragAxis & EAxisList::Type::Y)
	{
		UVAxis += FVector2D(0.0, 1.0);
	}

	return UVDelta * UVAxis;
}

void FDisplayClusterLightCardEditorHelper::InternalMoveActorTo(
	const FDisplayClusterWeakStageActorPtr& Actor, const FSphericalCoordinates& Position, bool bIsFinalChange) const
{
	if (bNormalMapInvalid || !CachedRootActor.IsValid())
	{
		ensure(false);
		return;
	}

	// Remove actor rotation from the position before setting the coordinate. Note that we don't need to do this for the normal map,
	// which already takes the root actor rotation into account as part of its view matrix.
	const FQuat RootRotation = CachedRootActor->GetTransform().GetRotation().Inverse();
	const FVector InverseRotatedPosition = RootRotation.RotateVector(Position.AsCartesian());
	
	SetActorCoordinates(Actor, FSphericalCoordinates(InverseRotatedPosition));
	Actor->UpdateStageActorTransform();

	if (Actor->IsAlwaysFlushToWall())
	{
		ADisplayClusterRootActor* RootActor = (Actor->IsProxy() || !LevelInstanceRootActor.IsValid()) ? CachedRootActor.Get() : LevelInstanceRootActor.Get();
		RootActor->MakeStageActorFlushToWall(Actor.AsActor());
	}

#if WITH_EDITOR
	if (!Actor->IsProxy())
	{
		if (bIsFinalChange)
		{
			Actor->UpdateEditorGizmos();
			PostEditChangePropertiesForMovedActor(Actor);
		}
	}
#endif
}

void FDisplayClusterLightCardEditorHelper::InternalDragActors(const TArray<FDisplayClusterWeakStageActorPtr>& Actors, const FIntPoint& PixelPos, const FSceneView& View,
	ECoordinateSystem CoordinateSystem, const FVector& DragWidgetOffset, EAxisList::Type DragAxis, FDisplayClusterWeakStageActorPtr PrimaryActor)
{
	if (Actors.IsEmpty() || !ensure(!bNormalMapInvalid) || !UpdateRootActor())
	{
		return;
	}

	if (!PrimaryActor.IsValid())
	{
		PrimaryActor = Actors.Last();
	}

	if (PrimaryActor.IsValid() && !PrimaryActor->IsUVActor())
	{
		const bool bUseDeltaRotation = (DragAxis == EAxisList::Type::XYZ) || (DragAxis == EAxisList::Type::Y) || (CoordinateSystem == ECoordinateSystem::Cartesian);

		const FRotator DeltaRotation =
			bUseDeltaRotation ?
			GetActorRotationDelta(PixelPos, View, PrimaryActor, CoordinateSystem, DragAxis, DragWidgetOffset)
			: FRotator::ZeroRotator;

		const FSphericalCoordinates DeltaCoords =
			bUseDeltaRotation ?
			FSphericalCoordinates()
			: GetActorTranslationDelta(PixelPos, View, PrimaryActor, CoordinateSystem, DragAxis, DragWidgetOffset);

		for (const FDisplayClusterWeakStageActorPtr& Actor : Actors)
		{
			if (!Actor.IsValid() || Actor->IsUVActor())
			{
				continue;
			}

			VerifyAndFixActorOrigin(Actor);

			// Note: GetActorCoordinates maintains last known Azimuth when looking at the poles
			const FSphericalCoordinates CurrentCoords = GetActorCoordinates(Actor);

			// We will adjust the spin (to maintain the apparent spin) when using center of gizmo
			// or dragging longitudinally (this seems to provide an intuitive behavior)
			if (bUseDeltaRotation) // Dragging center of gizmo
			{
				// We might need this to put back the LightCard exactly as it was
				const FDisplayClusterPositionalParams OriginalPositionalParams = Actor->GetPositionalParams();

				const FVector CurrentPos = CurrentCoords.AsCartesian();
				const FVector NewPos = DeltaRotation.RotateVector(CurrentPos);

				// Calculations are only valid if translation is not too small
				const FSphericalCoordinates NewCoords(NewPos);

				const FTransform Transform_A = Actor->GetStageActorTransform(true);

				// Don't fire off property change events yet since we're not done modifying the lightcard
				InternalMoveActorTo(Actor, NewCoords, false);
				Actor->UpdateStageActorTransform(); // We must call this for GetStageActorTransform to be valid

				const FTransform Transform_B = Actor->GetStageActorTransform(true);

				// Calculate world delta translation of moving from A to B
				FVector WorldDelta = Transform_B.GetLocation() - Transform_A.GetLocation(); // X towards front of stage. Y towards right of stage. Z towards ceiling.

				// Calculate LC "Y" unit vector at A and B. ("X" is LC normal)
				const FVector Y_A = Transform_A.Rotator().RotateVector(FVector::YAxisVector);
				const FVector Y_B = Transform_B.Rotator().RotateVector(FVector::YAxisVector);

				// Calculate card normal vector
				const FVector CardNormal_A = Transform_A.Rotator().RotateVector(FVector::XAxisVector); // When card is on ceiling, expect around (0,0,-1).
				const FVector CardNormal_B = Transform_B.Rotator().RotateVector(FVector::XAxisVector);

				// Calculate projection of movement onto surface tangent plane at A and B
				const FVector UnitWorldDeltaPlane_A = FVector::VectorPlaneProject(WorldDelta, CardNormal_A).GetSafeNormal();
				const FVector UnitWorldDeltaPlane_B = FVector::VectorPlaneProject(WorldDelta, CardNormal_B).GetSafeNormal();

				if (!FMath::IsNearlyZero(UnitWorldDeltaPlane_A.Length()) && !FMath::IsNearlyZero(UnitWorldDeltaPlane_B.Length()))
				{
					// Calculate relative spin angle at A, which is the angle between Y_A and UnitWorldDeltaPlane_A
					const double SpinDotProduct_A = FVector::DotProduct(Y_A, UnitWorldDeltaPlane_A);
					const FVector SpinCrossProduct_A = FVector::CrossProduct(Y_A, UnitWorldDeltaPlane_A);
					const int32 SpinSign_A = FVector::DotProduct(CardNormal_A, SpinCrossProduct_A) > 0 ? -1 : 1;
					const double RelativeSpinAngle_A = FMath::Acos(SpinDotProduct_A) * SpinSign_A; // radians

					// Now we need to find the spin that keeps the same RelativeSpinAngle in B as it was in A
					const double SpinDotProduct_B = FVector::DotProduct(Y_B, UnitWorldDeltaPlane_B);
					const FVector SpinCrossProduct_B = FVector::CrossProduct(Y_B, UnitWorldDeltaPlane_B);
					const int32 SpinSign_B = FVector::DotProduct(CardNormal_B, SpinCrossProduct_B) > 0 ? -1 : 1;
					const double RelativeSpinAngle_B = FMath::Acos(SpinDotProduct_B) * SpinSign_B; // radians

					const double DeltaSpin = RelativeSpinAngle_B - RelativeSpinAngle_A;

					// Apply delta spin to lightcard
					Actor->SetSpin(Actor->GetSpin() + FMath::RadiansToDegrees(DeltaSpin));
				}
				else
				{
					// Leave it where it was to avoid apparent spins even though motion would have been insignificant.
					Actor->SetPositionalParams(OriginalPositionalParams);
				}

#if WITH_EDITOR
				PostEditChangePropertiesForMovedActor(Actor);
#endif
			}
			else // Dragging latitudinally
			{
				InternalMoveActorTo(Actor, CurrentCoords + DeltaCoords, true);
			}
		}
	}
}

void FDisplayClusterLightCardEditorHelper::SetActorCoordinates(const FDisplayClusterWeakStageActorPtr& Actor, const FSphericalCoordinates& SphericalCoords) const
{
	Actor->SetDistanceFromCenter(SphericalCoords.Radius);
	Actor->SetLatitude(90.f - FMath::RadiansToDegrees(SphericalCoords.Inclination));

	// Keep the same longitude when pointing at the pole. This helps with continuity 
	// and also mitigates sudden changes in apparent spin when moving around the poles
	if (!SphericalCoords.IsPointingAtPole())
	{
		Actor->SetLongitude(FRotator::ClampAxis(FMath::RadiansToDegrees(SphericalCoords.Azimuth) - 180.f));
	}
}

bool FDisplayClusterLightCardEditorHelper::TraceStage(const FVector& RayStart, const FVector& RayEnd, FVector& OutHitLocation)
{
	ADisplayClusterRootActor* RootActor = UpdateRootActor();
	if (RootActor)
	{
		FCollisionQueryParams TraceParams(SCENE_QUERY_STAT(DisplayClusterStageTrace), true);

		FHitResult HitResult;
		if (RootActor->ActorLineTraceSingle(HitResult, RayStart, RayEnd, ECollisionChannel::ECC_PhysicsBody, TraceParams))
		{
			OutHitLocation = HitResult.Location;
			return true;
		}
	}

	OutHitLocation = FVector::ZeroVector;
	return false;
}

FVector FDisplayClusterLightCardEditorHelper::TraceScreenRay(const FVector& OrthogonalOrigin, const FVector& OrthogonalDirection, const FVector& ViewOrigin)
{
	const FVector RayStart = OrthogonalOrigin;
	const FVector RayEnd = OrthogonalOrigin + OrthogonalDirection * WORLD_MAX;

	FVector Direction = FVector::ZeroVector;

	if (ProjectionMode == EDisplayClusterMeshProjectionType::UV)
	{
		// In the UV projection mode, all stage geometry has been projected onto a UV projection plane, so perform a ray trace against that plane to find the 
		// screen ray direction
		const float UVProjectionPlaneDistance = ADisplayClusterLightCardActor::UVPlaneDefaultDistance;

		const FPlane UVProjectionPlane(ViewOrigin + FVector::ForwardVector * UVProjectionPlaneDistance, -FVector::ForwardVector);
		const FVector PlaneIntersection = FMath::RayPlaneIntersection(OrthogonalOrigin, OrthogonalDirection, UVProjectionPlane);

		Direction = (PlaneIntersection - ViewOrigin).GetSafeNormal();
	}
	else
	{
		// First, trace against the stage actor to see if the screen ray hits it; if so, simply return the direction from the view point to this hit point
		FVector HitLocation = FVector::ZeroVector;
		if (TraceStage(RayStart, RayEnd, HitLocation))
		{
			Direction = (HitLocation - ViewOrigin).GetSafeNormal();
		}
		else
		{
			// If we didn't hit any stage geometry, try to trace against the normal map mesh.

			FCollisionQueryParams TraceParams(SCENE_QUERY_STAT(DisplayClusterStageTrace), true);

			FHitResult HitResult;
			if (ensure(!bNormalMapInvalid) && NormalMapMeshComponent->LineTraceComponent(HitResult, RayStart, RayEnd, TraceParams))
			{
				HitLocation = HitResult.Location;
				Direction = (HitLocation - ViewOrigin).GetSafeNormal();
			}
			else
			{
				// If the screen ray does not hit the stage or the normal map mesh, then simply use the closest point on the ray to the view point
				const FVector ClosestPoint = OrthogonalOrigin + ((ViewOrigin - OrthogonalOrigin) | OrthogonalDirection) * OrthogonalDirection;

				Direction = (ClosestPoint - ViewOrigin).GetSafeNormal();
			}
		}

		ADisplayClusterRootActor* RootActor = UpdateRootActor();
		if (RootActor)
		{
			// Rotate direction back into local space
			const FQuat RootRotation = RootActor->GetTransform().GetRotation().Inverse();
			Direction = RootRotation.RotateVector(Direction);
		}
	}

	return Direction;
}

double FDisplayClusterLightCardEditorHelper::CalculateFinalLightCardDistance(double FlushDistance, double DesiredOffsetFromFlush) const
{
	double Distance = FMath::Min(FlushDistance, RootActorBoundingRadius) + DesiredOffsetFromFlush;

	return FMath::Max(Distance, 0);
}

void FDisplayClusterLightCardEditorHelper::InvalidateNormalMap()
{
	bNormalMapInvalid = true;
}

bool FDisplayClusterLightCardEditorHelper::UpdateNormalMaps()
{
	if (!bNormalMapInvalid && NormalMapMeshComponent.IsValid())
	{
		return true;
	}

	ADisplayClusterRootActor* RootActor = UpdateRootActor();
	if (!RootActor)
	{
		return false;
	}

	// Update this so we render from the latest projection point
	UpdateProjectionOriginComponent();

	RootActor->GetStageGeometryComponent()->Invalidate(true);

	bNormalMapInvalid = false;
	UpdateNormalMapMesh();

	return !bNormalMapInvalid;
}

void FDisplayClusterLightCardEditorHelper::UpdateNormalMapMesh()
{
	if (!NormalMeshScene.IsValid() || !IsValid(NormalMeshScene->GetWorld()))
	{
		FWorldDelegates::OnWorldCleanup.RemoveAll(this);
		FWorldDelegates::OnWorldCleanup.AddRaw(this, &FDisplayClusterLightCardEditorHelper::OnWorldCleanup);
		NormalMeshScene = MakeShared<FPreviewScene>(FPreviewScene::ConstructionValues());
	}
	else if (NormalMapMeshComponent.IsValid())
	{
		NormalMeshScene->RemoveComponent(NormalMapMeshComponent.Get());
	}

	UpdateProjectionOriginComponent();

	const FName UniqueName = MakeUniqueObjectName(GetTransientPackage(), UProceduralMeshComponent::StaticClass(), TEXT("NormalMapMesh"));
	NormalMapMeshComponent = NewObject<UProceduralMeshComponent>(GetTransientPackage(), UniqueName);
	NormalMapMeshComponent->SetCollisionEnabled(ECollisionEnabled::QueryOnly);
	NormalMeshScene->AddComponent(NormalMapMeshComponent.Get(), FTransform(GetProjectionOrigin()));

	UStaticMesh* IcoSphereMesh = LoadObject<UStaticMesh>(nullptr, TEXT("/nDisplay/Meshes/SM_IcoSphere.SM_IcoSphere"), nullptr, LOAD_None, nullptr);

	if (ensure(IcoSphereMesh))
	{
		IcoSphereMesh->bAllowCPUAccess = true;

		int32 NumSections = IcoSphereMesh->GetNumSections(0);
		for (int32 SectionIndex = 0; SectionIndex < NumSections; SectionIndex++)
		{
			TArray<FVector> Vertices;
			TArray<int32> Triangles;
			TArray<FVector> Normals;
			TArray<FVector2D> UVs;
			TArray<FProcMeshTangent> Tangents;

			UKismetProceduralMeshLibrary::GetSectionFromStaticMesh(IcoSphereMesh, 0, SectionIndex, Vertices, Triangles, Normals, UVs, Tangents);

			TArray<FVector2D> EmptyUVs;
			TArray<FLinearColor> EmptyColors;
			NormalMapMeshComponent.Get()->CreateMeshSection_LinearColor(SectionIndex, Vertices, Triangles, Normals, UVs, EmptyUVs, EmptyUVs, EmptyUVs, EmptyColors, Tangents, true);

			for (int32 Index = 0; Index < IcoSphereMesh->GetStaticMaterials().Num(); ++Index)
			{
				UMaterialInterface* MaterialInterface = IcoSphereMesh->GetStaticMaterials()[Index].MaterialInterface;
				NormalMapMeshComponent.Get()->SetMaterial(Index, MaterialInterface);
			}
		}

		CachedRootActor->GetStageGeometryComponent()->MorphProceduralMesh(NormalMapMeshComponent.Get());
	}
}

void FDisplayClusterLightCardEditorHelper::OnWorldCleanup(UWorld* World, bool bSessionEnded, bool bCleanupResources)
{
	if (!NormalMeshScene.IsValid() || !CachedRootActor.IsValid() || World != CachedRootActor->GetWorld())
	{
		return;
	}

	if (NormalMapMeshComponent.IsValid())
	{
		NormalMeshScene->RemoveComponent(NormalMapMeshComponent.Get());
	}

	NormalMeshScene.Reset();
}

#if WITH_EDITORONLY_DATA
void FDisplayClusterLightCardEditorHelper::OnActorPropertyChanged(UObject* ObjectBeingModified, FPropertyChangedEvent& PropertyChangedEvent)
{
	if (!CachedRootActor.IsValid())
	{
		FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);
		return;
	}

	if (ObjectBeingModified == CachedRootActor.Get())
	{
		InvalidateNormalMap();
	}
}

void FDisplayClusterLightCardEditorHelper::OnRootActorBlueprintCompiled(UBlueprint* Blueprint)
{
	InvalidateNormalMap();
}
#endif

#if WITH_EDITOR
void FDisplayClusterLightCardEditorHelper::PostEditChangePropertiesForMovedActor(const FDisplayClusterWeakStageActorPtr& Actor) const
{
	if (Actor->IsProxy())
	{
		return;
	}

	Actor.AsActorChecked()->Modify();

	auto ModifyLightCardProperty = [&](const IDisplayClusterStageActor::FPropertyPair& PropertyPair) -> void
	{
		// Broadcast the event directly instead of PostEditChangeProperty on the object itself, which would attempt to create unnecessary snapshots for each call
		FPropertyChangedEvent PropertyChangedEvent(PropertyPair.Value, EPropertyChangeType::Interactive);
		FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(Actor.AsActorChecked(), PropertyChangedEvent);
	};

	IDisplayClusterStageActor::FPositionalPropertyArray PropertyPairs;
	Actor->GetPositionalProperties(PropertyPairs);
	
	TArray<const FProperty*> ChangedProperties;
	ChangedProperties.Reserve(PropertyPairs.Num());

	for (const IDisplayClusterStageActor::FPropertyPair& PropertyPair : PropertyPairs)
	{
		ModifyLightCardProperty(PropertyPair);

		ChangedProperties.Add(PropertyPair.Value);
	}

	SnapshotTransactionBuffer(Actor.AsActorChecked(), MakeArrayView(ChangedProperties.GetData(), ChangedProperties.Num()));

	// Force the actor to update its position in a way that the multi-user server will see
	Actor.AsActorChecked()->PostEditMove(false);
}
#endif
