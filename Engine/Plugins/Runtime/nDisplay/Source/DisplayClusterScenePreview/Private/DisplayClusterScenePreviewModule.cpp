// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterScenePreviewModule.h"
#include "DisplayClusterScenePreviewProxyManager.h"

#include "Blueprints/DisplayClusterBlueprintLib.h"
#include "CanvasTypes.h"
#include "Components/DisplayClusterScreenComponent.h"
#include "DisplayClusterChromakeyCardActor.h"
#include "DisplayClusterLightCardActor.h"
#include "DisplayClusterRootActor.h"
#include "Engine/Blueprint.h"
#include "Engine/Canvas.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Misc/TransactionObjectEvent.h"
#include "TextureResource.h"

#if WITH_EDITOR
#include "LevelEditorViewport.h"
#endif

#define LOCTEXT_NAMESPACE "DisplayClusterScenePreview"

static TAutoConsoleVariable<float> CVarDisplayClusterScenePreviewRenderTickDelay(
	TEXT("nDisplay.ScenePreview.RenderTickDelay"),
	0.1f,
	TEXT("The number of seconds to wait between processing queued renders.")
);

void FDisplayClusterScenePreviewModule::StartupModule()
{ }

void FDisplayClusterScenePreviewModule::ShutdownModule()
{
	Release();
}

void FDisplayClusterScenePreviewModule::Release()
{
	if (RenderTickerHandle.IsValid())
	{
		FTSTicker::RemoveTicker(RenderTickerHandle);
		RenderTickerHandle.Reset();
	}

	TArray<int32> RendererIds;
	RendererConfigs.GenerateKeyArray(RendererIds);
	for (const int32 RendererId : RendererIds)
	{
		DestroyRenderer(RendererId);
	}

	// Release the RootActor and PreviewWorld proxies.
	ProxyManagerPtr.Reset();
}

#if WITH_EDITOR
void FDisplayClusterScenePreviewModule::OnEditorClosed()
{
	// On editor close, Exit() should run to clean up, but this happens very late.
	Release();

	if (EditorClosedEventHandle.IsValid() && GEditor)
	{
		GEditor->OnEditorClose().Remove(EditorClosedEventHandle);
	}
}
#endif // #if WITH_EDITOR

FDisplayClusterScenePreviewProxyManager& FDisplayClusterScenePreviewModule::GetProxyManager()
{
#if WITH_EDITOR
	if (!EditorClosedEventHandle.IsValid() && GEditor)
	{
		EditorClosedEventHandle = GEditor->OnEditorClose().AddRaw(this, &FDisplayClusterScenePreviewModule::OnEditorClosed);
	}
#endif //#if WITH_EDITOR

	if (!ProxyManagerPtr.IsValid())
	{
		ProxyManagerPtr = MakeUnique<FDisplayClusterScenePreviewProxyManager>();
	}

	check(ProxyManagerPtr.IsValid());

	return *ProxyManagerPtr;
}


int32 FDisplayClusterScenePreviewModule::CreateRenderer()
{
	FRendererConfig& Config = RendererConfigs.Add(NextRendererId);

	Config.Renderer = MakeShared<FDisplayClusterMeshProjectionRenderer>();

	return NextRendererId++;
}

bool FDisplayClusterScenePreviewModule::DestroyRenderer(int32 RendererId)
{
	if (FRendererConfig* Config = RendererConfigs.Find(RendererId))
	{
		// Release the proxy resources that were used by this renderer.
		GetProxyManager().SetSceneRootActorForRenderer(RendererId, nullptr, EDisplayClusterScenePreviewFlags::None);

		RegisterRootActorEvents(RendererId, *Config, false /* bShouldRegister */);

		// Remove this config.
		RendererConfigs.Remove(RendererId);

		RegisterOrUnregisterGlobalActorEvents();

		return true;
	}

	return false;
}

bool FDisplayClusterScenePreviewModule::SetRendererRootActorPath(int32 RendererId, const FString& ActorPath, const FDisplayClusterRootActorPropertyOverrides& InPropertyOverrides, const EDisplayClusterScenePreviewFlags PreviewFlags)
{
	if (FRendererConfig* RendererConfig = RendererConfigs.Find(RendererId))
	{
		// Use custom properties on root actor
		RendererConfig->RootActorPropertyOverrides = InPropertyOverrides;
		EnumAddFlags(RendererConfig->Flags, EDisplayClusterRendererConfigFlags::RootActorPropertyOverridesModified);

		// Determine these values before we update the config's RootActor
		if (RendererConfig->PreviewFlags != PreviewFlags)
		{
			RendererConfig->PreviewFlags = PreviewFlags;
			EnumAddFlags(RendererConfig->Flags, EDisplayClusterRendererConfigFlags::PreviewFlagsModified);
		}

		// Update root actor path
		if (RendererConfig->RootActorPath != ActorPath)
		{
			RendererConfig->RootActorPath = ActorPath;
			EnumAddFlags(RendererConfig->Flags, EDisplayClusterRendererConfigFlags::RootActorPathModified);
		}

		InternalUpdateRendererConfig(RendererId, *RendererConfig);

		return true;
	}

	return false;
}

bool FDisplayClusterScenePreviewModule::SetRendererRootActor(int32 RendererId, ADisplayClusterRootActor* Actor, const FDisplayClusterRootActorPropertyOverrides& InPropertyOverrides, const EDisplayClusterScenePreviewFlags PreviewFlags)
{
	if (FRendererConfig* RendererConfig = RendererConfigs.Find(RendererId))
	{
		// Use custom properties on root actor
		RendererConfig->RootActorPropertyOverrides = InPropertyOverrides;
		EnumAddFlags(RendererConfig->Flags, EDisplayClusterRendererConfigFlags::RootActorPropertyOverridesModified);

		// Determine these values before we update the config's RootActor
		if (RendererConfig->PreviewFlags != PreviewFlags)
		{
			RendererConfig->PreviewFlags = PreviewFlags;
			EnumAddFlags(RendererConfig->Flags, EDisplayClusterRendererConfigFlags::PreviewFlagsModified);
		}

		// Clear root actor path
		RendererConfig->RootActorPath.Empty();

		InternalSetRendererRootActor(RendererId, *RendererConfig, Actor);
		InternalUpdateRendererConfig(RendererId, *RendererConfig);

		return true;
	}

	return false;
}

ADisplayClusterRootActor* FDisplayClusterScenePreviewModule::GetRendererRootActor(int32 RendererId)
{
	if (FRendererConfig* Config = RendererConfigs.Find(RendererId))
	{
		return InternalGetRendererRootActor(RendererId, *Config);
	}

	return nullptr;
}

ADisplayClusterRootActor* FDisplayClusterScenePreviewModule::GetRendererRootActorOrProxy(int32 RendererId)
{
	if (FRendererConfig* Config = RendererConfigs.Find(RendererId))
	{
		return InternalGetRendererRootActorOrProxy(RendererId, *Config);
	}

	return nullptr;
}

bool FDisplayClusterScenePreviewModule::Render(int32 RendererId, FDisplayClusterMeshProjectionRenderSettings& RenderSettings, FCanvas& Canvas)
{
	if (FRendererConfig* Config = RendererConfigs.Find(RendererId))
	{
		return InternalRenderImmediate(RendererId, *Config, RenderSettings, Canvas);
	}

	return false;
}

bool FDisplayClusterScenePreviewModule::RenderQueued(int32 RendererId, FDisplayClusterMeshProjectionRenderSettings& RenderSettings, const FIntPoint& Size, FRenderResultDelegate ResultDelegate)
{
	return InternalRenderQueued(RendererId, RenderSettings, nullptr, Size, ResultDelegate);
}

bool FDisplayClusterScenePreviewModule::RenderQueued(int32 RendererId, FDisplayClusterMeshProjectionRenderSettings& RenderSettings, const TWeakPtr<FCanvas> Canvas, FRenderResultDelegate ResultDelegate)
{
	if (!Canvas.IsValid())
	{
		return false;
	}

	FRenderTarget* RenderTarget = Canvas.Pin()->GetRenderTarget();
	if (!RenderTarget)
	{
		return false;
	}

	return InternalRenderQueued(RendererId, RenderSettings, Canvas, RenderTarget->GetSizeXY(), ResultDelegate);
}

bool FDisplayClusterScenePreviewModule::InternalRenderQueued(int32 RendererId, FDisplayClusterMeshProjectionRenderSettings& RenderSettings, TWeakPtr<FCanvas> Canvas,
	const FIntPoint& Size, FRenderResultDelegate ResultDelegate)
{
	if (FRendererConfig* Config = RendererConfigs.Find(RendererId))
	{
		RenderQueue.Enqueue(FPreviewRenderJob(RendererId, RenderSettings, Size, Canvas, ResultDelegate));

		if (!RenderTickerHandle.IsValid())
		{
			RenderTickerHandle = FTSTicker::GetCoreTicker().AddTicker(
				FTickerDelegate::CreateRaw(this, &FDisplayClusterScenePreviewModule::OnTick),
				CVarDisplayClusterScenePreviewRenderTickDelay.GetValueOnGameThread()
			);
		}

		return true;
	}

	return false;
}

bool FDisplayClusterScenePreviewModule::IsRealTimePreviewEnabled() const
{
	return bIsRealTimePreviewEnabled;
}

void FDisplayClusterScenePreviewModule::InternalSetRendererRootActor(int32 RendererId, FRendererConfig& RendererConfig, ADisplayClusterRootActor* Actor)
{
	const bool bRootChanged = !RendererConfig.IsRootActorEquals(Actor);
	if (bRootChanged)
	{
		// Unregister events for current root actor
		RegisterRootActorEvents(RendererId, RendererConfig, false /* bShouldRegister */);

		RendererConfig.SetRootActor(Actor);
	}
}

ADisplayClusterRootActor* FDisplayClusterScenePreviewModule::InternalGetRendererRootActor(int32 RendererId, FRendererConfig& RendererConfig)
{
	ADisplayClusterRootActor* SceneRootActor = RendererConfig.GetRootActor();

	const bool bRequiredRootActor = !IsValid(SceneRootActor)
								|| !RendererConfig.IsDefinedRootActor()
								|| EnumHasAnyFlags(RendererConfig.Flags, EDisplayClusterRendererConfigFlags::RootActorPathModified);
	if (bRequiredRootActor && !RendererConfig.RootActorPath.IsEmpty())
	{
		// If we don't have a RootActor, but we do have a RootActorPath, use it to find the RootActor object.
		ADisplayClusterRootActor* RootActor = FindObject<ADisplayClusterRootActor>(nullptr, *RendererConfig.RootActorPath);
		InternalSetRendererRootActor(RendererId, RendererConfig, RootActor);

		SceneRootActor = RendererConfig.GetRootActor();
	}

	return IsValid(SceneRootActor) ? SceneRootActor : nullptr;
}

ADisplayClusterRootActor* FDisplayClusterScenePreviewModule::InternalGetRendererRootActorOrProxy(int32 RendererId, FRendererConfig& RendererConfig)
{
	// Note: we call this function first because it can assign a new root actor and proxy.
	ADisplayClusterRootActor* RootActor = InternalGetRendererRootActor(RendererId, RendererConfig);

	// When proxy is required, get it from ProxyManager.
	const bool bUseRootActorProxy = EnumHasAnyFlags(RendererConfig.PreviewFlags, EDisplayClusterScenePreviewFlags::UseRootActorProxy);

	return bUseRootActorProxy ? GetProxyManager().GetProxyRootActor(RendererId) : RootActor;
}

void FDisplayClusterScenePreviewModule::InternalUpdateRendererConfig(int32 RendererId, FRendererConfig& RendererConfig)
{
	// Get RootActor from scene
	// Call this function at the very beginning because it can internally change the RendererConfig.Flags.
	ADisplayClusterRootActor* SceneRootActor = InternalGetRendererRootActor(RendererId, RendererConfig);

	// Check that the RootActor proxy still has the same reference to the RootActor in the scene as the renderer.
	if (EnumHasAnyFlags(RendererConfig.PreviewFlags, EDisplayClusterScenePreviewFlags::UseRootActorProxy))
	{
		ADisplayClusterRootActor* PrevSceneRootActor = GetProxyManager().GetSceneRootActor(RendererId);
		if (PrevSceneRootActor != SceneRootActor)
		{
			// The RootActor proxy is no longer valid, have to be re-created
			EnumAddFlags(RendererConfig.Flags, EDisplayClusterRendererConfigFlags::UpdateRootActorRef);
		}
	}

	const bool bAutoUpdateStageActors = EnumHasAnyFlags(RendererConfig.PreviewFlags, EDisplayClusterScenePreviewFlags::AutoUpdateStageActors);
	if (bAutoUpdateStageActors)
	{
		// If the RootActor is missing or any changes have occurred, we must clear the rendering scene
		if (!SceneRootActor || RendererConfig.Flags != EDisplayClusterRendererConfigFlags::None)
		{
			RendererConfig.ClearRendererScene();
		}
	}

	// If the flags have not changed, do nothing
	if (RendererConfig.Flags == EDisplayClusterRendererConfigFlags::None)
	{
		return;
	}

	// Optionally, update the proxy when change the RootActor.
	if (EnumHasAnyFlags(RendererConfig.Flags, EDisplayClusterRendererConfigFlags::UpdateRootActorRef
											| EDisplayClusterRendererConfigFlags::PreviewFlagsModified
											| EDisplayClusterRendererConfigFlags::RootActorBeingModified))
	{
		// Updates the proxy for the new root actor.
		if (EnumHasAnyFlags(RendererConfig.PreviewFlags, EDisplayClusterScenePreviewFlags::UseRootActorProxy))
		{
			// Create new RootActor proxy
			GetProxyManager().SetSceneRootActorForRenderer(RendererId, SceneRootActor, RendererConfig.PreviewFlags);

			EnumAddFlags(RendererConfig.Flags, EDisplayClusterRendererConfigFlags::UpdateRootActorProxyRef);
		}
		else
		{
			// Remove proxy if exists.
			GetProxyManager().SetSceneRootActorForRenderer(RendererId, nullptr, RendererConfig.PreviewFlags);
		}

		// Update global events
		RegisterRootActorEvents(RendererId, RendererConfig, true /* bShouldRegister */);
		RegisterOrUnregisterGlobalActorEvents();
	}

	// RootActor property overrides has been changed
	if (EnumHasAnyFlags(RendererConfig.Flags, EDisplayClusterRendererConfigFlags::RootActorPropertyOverridesModified | EDisplayClusterRendererConfigFlags::UpdateRootActorRef | EDisplayClusterRendererConfigFlags::UpdateRootActorProxyRef))
	{
		if (ADisplayClusterRootActor* RendererRootActorOrProxy = InternalGetRendererRootActorOrProxy(RendererId, RendererConfig))
		{
			RendererRootActorOrProxy->OverrideRootActorProperties(RendererConfig.RootActorPropertyOverrides);
		}
	}

	AutoPopulateScene(RendererId, RendererConfig);

	// Reset the flags at the end.
	RendererConfig.Flags = EDisplayClusterRendererConfigFlags::None;
}

void FDisplayClusterScenePreviewModule::AutoPopulateScene(int32 RendererId, FRendererConfig& RendererConfig)
{
	// The renderer can use a proxy.
	ADisplayClusterRootActor* RootActor = InternalGetRendererRootActor(RendererId, RendererConfig);
	ADisplayClusterRootActor* RootActorProxy = InternalGetRendererRootActorOrProxy(RendererId, RendererConfig);
	if (RootActor && RootActorProxy)
	{
		TArray<FString> ProjectionMeshNames;

		if (UDisplayClusterConfigurationData* Config = RootActor->GetConfigData())
		{
			Config->GetReferencedMeshNames(ProjectionMeshNames);
		}

		RendererConfig.AddActorToRenderer(RootActorProxy, [&ProjectionMeshNames](const UPrimitiveComponent* PrimitiveComponent)
			{
				// Filter out any primitive component that isn't a projection mesh (a static mesh that has a Mesh projection configured for it) or a screen component
				const bool bIsProjectionMesh = PrimitiveComponent->IsA<UStaticMeshComponent>() && ProjectionMeshNames.Contains(PrimitiveComponent->GetName());
				const bool bIsScreen = PrimitiveComponent->IsA<UDisplayClusterScreenComponent>();
				return bIsProjectionMesh || bIsScreen;
			});

		const bool bAutoUpdateStageActors = EnumHasAnyFlags(RendererConfig.PreviewFlags, EDisplayClusterScenePreviewFlags::AutoUpdateStageActors);
		if (bAutoUpdateStageActors)
		{
			// Automatically add the lightcards found on this actor
			TSet<ADisplayClusterLightCardActor*> LightCards;
			UDisplayClusterBlueprintLib::FindLightCardsForRootActor(RootActor, LightCards);

			TSet<ADisplayClusterChromakeyCardActor*> ChromaKeyCards;
			UDisplayClusterBlueprintLib::FindChromakeyCardsForRootActor(RootActor, ChromaKeyCards);
			LightCards.Append(reinterpret_cast<TSet<ADisplayClusterLightCardActor*>&>(ChromaKeyCards));

			TSet<AActor*> Actors;
			Actors.Reserve(LightCards.Num());
			for (ADisplayClusterLightCardActor* LightCard : LightCards)
			{
				Actors.Add(LightCard);
			}

			// Also check for any non-lightcard actors in the world that are valid to control from ICVFX editors
			if (UWorld* World = RootActor->GetWorld())
			{
				for (const TWeakObjectPtr<AActor> WeakActor : TActorRange<AActor>(World))
				{
					if (WeakActor.IsValid() && WeakActor->Implements<UDisplayClusterStageActor>() && !WeakActor->IsA<ADisplayClusterLightCardActor>())
					{
						Actors.Add(WeakActor.Get());
					}
				}
			}

			for (AActor* Actor : Actors)
			{
				RendererConfig.AddActorToRenderer(Actor, true);
			}
		}
	}
}

bool FDisplayClusterScenePreviewModule::InternalRenderImmediate(int32 RendererId, FRendererConfig& RendererConfig, FDisplayClusterMeshProjectionRenderSettings& RenderSettings, FCanvas& Canvas)
{
	if (!RendererConfig.Renderer.IsValid())
	{
		return false;
	}

	// Update this so that whoever gets the callback can immediately check whether the nDisplay preview may be out of date
	UpdateIsRealTimePreviewEnabled();

	InternalUpdateRendererConfig(RendererId, RendererConfig);

	// Get the Root Actor or proxy for rendering previews.
	ADisplayClusterRootActor* ProxyRootActor = InternalGetRendererRootActorOrProxy(RendererId, RendererConfig);
	ADisplayClusterRootActor* SceneRootActor = InternalGetRendererRootActor(RendererId, RendererConfig);
	UWorld* PreviewWorld = ProxyRootActor ? ProxyRootActor->GetWorld() : nullptr;
	UWorld* SceneWorld = SceneRootActor ? SceneRootActor->GetWorld() : nullptr;
	if (!PreviewWorld || !SceneWorld)
	{
		return false;
	}

	// Push any deferred render state updates to ensure that light card positions, preview meshes modified above, etc. are up to date
	PreviewWorld->SendAllEndOfFrameUpdates();

	// FDisplayClusterMeshProjectionRenderer uses references to UPrimitiveComponent.
	// These primitives are taken from RootActorProxy in PreviewWorld and StageActors in SceneWorld.
	// But we can't render UPrimitiveComponent belonging to different scenes at the same time,
	// because they reference the internal data of the scene they belong to.
	// The FDisplayClusterMeshProjectionRenderer::RenderScenes() function implements a new approach that
	// allows us to render a UPrimitiveComponent from multiple worlds.
	if (PreviewWorld != SceneWorld)
	{
		// Rendering is performed for all scenes in the array in the same order.
		// First, the RootActorProxy geometries from PreviewWorld are drawn.
		// Then StageActors from SceneWorld are drawn.
		// Here is the rendering order: ClearRTT-> RootActorProxy -> StageActors.
		RendererConfig.Renderer->RenderScenes(&Canvas, { PreviewWorld->Scene, SceneWorld->Scene }, RenderSettings);
	}
	else
	{
		RendererConfig.Renderer->Render(&Canvas, PreviewWorld->Scene, RenderSettings);
	}

	return true;
}

void FDisplayClusterScenePreviewModule::RegisterOrUnregisterGlobalActorEvents()
{
	// Check whether any of our configs need actor events
	bool bShouldBeRegistered = false;
	for (const TPair<int32, FRendererConfig>& ConfigPair : RendererConfigs)
	{
		const bool bAutoUpdateStageActors = EnumHasAnyFlags(ConfigPair.Value.PreviewFlags, EDisplayClusterScenePreviewFlags::AutoUpdateStageActors);
		if (bAutoUpdateStageActors)
		{
			bShouldBeRegistered = true;
			break;
		}
	}

#if WITH_EDITOR
	if (bShouldBeRegistered && !bIsRegisteredForActorEvents)
	{
		// Register for events
		FCoreUObjectDelegates::OnObjectPropertyChanged.AddRaw(this, &FDisplayClusterScenePreviewModule::OnActorPropertyChanged);
		FCoreUObjectDelegates::OnObjectTransacted.AddRaw(this, &FDisplayClusterScenePreviewModule::OnObjectTransacted);

		if (GEngine != nullptr)
		{
			GEngine->OnLevelActorDeleted().AddRaw(this, &FDisplayClusterScenePreviewModule::OnLevelActorDeleted);
			GEngine->OnLevelActorAdded().AddRaw(this, &FDisplayClusterScenePreviewModule::OnLevelActorAdded);
		}
	}
	else if (!bShouldBeRegistered && bIsRegisteredForActorEvents)
	{
		// Unregister for events
		FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);
		FCoreUObjectDelegates::OnObjectTransacted.RemoveAll(this);

		if (GEngine != nullptr)
		{
			GEngine->OnLevelActorDeleted().RemoveAll(this);
			GEngine->OnLevelActorAdded().RemoveAll(this);
		}
	}
#endif
}

void FDisplayClusterScenePreviewModule::RegisterRootActorEvents(int32 RendererId, FRendererConfig& RendererConfig, bool bShouldRegister)
{
#if WITH_EDITOR
	ADisplayClusterRootActor* Actor = RendererConfig.GetRootActor();
	if (!Actor)
	{
		return;
	}

	const uint8* GenericThis = reinterpret_cast<uint8*>(this);

	// Register/unregister for Blueprint events
	if (UBlueprint* Blueprint = UBlueprint::GetBlueprintFromClass(Actor->GetClass()))
	{
		// Store a reference to the BP class.
		// This reference is used to look up FRendererConfig from UBlueprint ptr in the OnBlueprintCompiled(UBlueprint* Blueprint) function.
		RendererConfig.RootActorBlueprintClass = Blueprint;

		Blueprint->OnCompiled().RemoveAll(this);

		if (bShouldRegister)
		{
			Blueprint->OnCompiled().AddRaw(this, &FDisplayClusterScenePreviewModule::OnBlueprintCompiled);
		}
	}
#endif
}

bool FDisplayClusterScenePreviewModule::UpdateIsRealTimePreviewEnabled()
{
#if WITH_EDITOR
	bIsRealTimePreviewEnabled = false;

	if (GIsEditor && !IsValid(GEditor))
	{
		return false;
	}

	for (const FLevelEditorViewportClient* LevelViewport : GEditor->GetLevelViewportClients())
	{
		if (LevelViewport && LevelViewport->IsRealtime())
		{
			bIsRealTimePreviewEnabled = true;
			break;
		}
	}

	return bIsRealTimePreviewEnabled;
#else
	return false;
#endif
}

bool FDisplayClusterScenePreviewModule::OnTick(float DeltaTime)
{
	GetProxyManager().TickPreviewWorld(DeltaTime);

	// This loop should break when we either run out of jobs or complete a single job
	while (!RenderQueue.IsEmpty())
	{
		FPreviewRenderJob Job;
		if (!RenderQueue.Dequeue(Job))
		{
			break;
		}

		ensure(Job.ResultDelegate.IsBound());

		if (FRendererConfig* Config = RendererConfigs.Find(Job.RendererId))
		{
			if (Job.bWasCanvasProvided)
			{
				// We were provided a canvas for this render job, so use it if possible
				if (!Job.Canvas.IsValid())
				{
					Job.ResultDelegate.Execute(nullptr);
					continue;
				}

				TSharedPtr<FCanvas, ESPMode::ThreadSafe> Canvas = Job.Canvas.Pin();
				FRenderTarget* RenderTarget = Canvas->GetRenderTarget();
				if (!RenderTarget)
				{
					Job.ResultDelegate.Execute(nullptr);
					continue;
				}

				InternalRenderImmediate(Job.RendererId, *Config, Job.Settings, *Canvas);
				Job.ResultDelegate.Execute(RenderTarget);
				break;
			}

			ADisplayClusterRootActor* RootActor = InternalGetRendererRootActorOrProxy(Job.RendererId, *Config);
			if (UWorld* World = RootActor ? RootActor->GetWorld() : nullptr)
			{
				// We need to provide the render target for this job
				UTextureRenderTarget2D* RenderTarget = Config->RenderTarget.Get();

				if (!RenderTarget)
				{
					// Create a new render target (which will be reused for this config in the future)
					RenderTarget = NewObject<UTextureRenderTarget2D>();
					RenderTarget->InitCustomFormat(Job.Size.X, Job.Size.Y, PF_B8G8R8A8, true);

					Config->RenderTarget = TStrongObjectPtr<UTextureRenderTarget2D>(RenderTarget);
				}
				else if (RenderTarget->SizeX != Job.Size.X || RenderTarget->SizeY != Job.Size.Y)
				{
					// Resize to match the new size
					RenderTarget->ResizeTarget(Job.Size.X, Job.Size.Y);

					// Flush commands so target is immediately ready to render at the new size
					FlushRenderingCommands();
				}

				FTextureRenderTargetResource* RenderTargetResource = RenderTarget->GameThread_GetRenderTargetResource();
				FCanvas Canvas(RenderTargetResource, nullptr, FGameTime::GetTimeSinceAppStart(), World->Scene->GetFeatureLevel());

				InternalRenderImmediate(Job.RendererId, *Config, Job.Settings, Canvas);
				Job.ResultDelegate.Execute(RenderTargetResource);
				break;
			}
			
			// No canvas and no world, so try the next render
		}

		// Config no longer exists, so try the next render
		Job.ResultDelegate.Execute(nullptr);
	}

	if (RenderQueue.IsEmpty())
	{
		RenderTickerHandle.Reset();
		return false;
	}

	return true;
}

void FDisplayClusterScenePreviewModule::OnActorPropertyChanged(UObject* ObjectBeingModified, FPropertyChangedEvent& PropertyChangedEvent)
{
	for (TPair<int32, FRendererConfig>& RendererConfigIt : RendererConfigs)
	{
		const bool bAutoUpdateStageActors = EnumHasAnyFlags(RendererConfigIt.Value.PreviewFlags, EDisplayClusterScenePreviewFlags::AutoUpdateStageActors);
		if (bAutoUpdateStageActors)
		{
			ADisplayClusterRootActor* RootActorPtr = RendererConfigIt.Value.GetRootActor();

			if (RootActorPtr == ObjectBeingModified)
			{
				EnumAddFlags(RendererConfigIt.Value.Flags, EDisplayClusterRendererConfigFlags::RootActorBeingModified);
			}
			else if (UActorComponent* Component = Cast<UActorComponent>(ObjectBeingModified))
			{
				if (Component->GetOwner() == RootActorPtr)
				{
					EnumAddFlags(RendererConfigIt.Value.Flags, EDisplayClusterRendererConfigFlags::LevelActorBeingModified);
				}
			}
		}
	}
}

void FDisplayClusterScenePreviewModule::OnLevelActorDeleted(AActor* Actor)
{
	for (TPair<int32, FRendererConfig>& RendererConfigIt : RendererConfigs)
	{
		if (RendererConfigIt.Value.AutoPopulateActors.Contains(Actor))
		{
			EnumAddFlags(RendererConfigIt.Value.Flags, EDisplayClusterRendererConfigFlags::LevelActorDeleted);
		}
	}
}

void FDisplayClusterScenePreviewModule::OnLevelActorAdded(AActor* Actor)
{
	if (!Actor || !Actor->Implements<UDisplayClusterStageActor>())
	{
		return;
	}

	// The actor won't be added to a root actor yet, so we can't check who it belongs to. Easier to just mark all configs as dirty.
	for (TPair<int32, FRendererConfig>& RendererConfigIt : RendererConfigs)
	{
		EnumAddFlags(RendererConfigIt.Value.Flags, EDisplayClusterRendererConfigFlags::LevelActorAdded);
	}
}

void FDisplayClusterScenePreviewModule::OnBlueprintCompiled(UBlueprint* Blueprint)
{
#if WITH_EDITOR
	for (TPair<int32, FRendererConfig>& RendererConfigIt : RendererConfigs)
	{
		if(RendererConfigIt.Value.IsBlueprintMatchesRendererRootActor(Blueprint))
		{
			EnumAddFlags(RendererConfigIt.Value.Flags, EDisplayClusterRendererConfigFlags::BlueprintCompiled);
		}
	}
#endif
}

void FDisplayClusterScenePreviewModule::OnObjectTransacted(UObject* Object, const FTransactionObjectEvent& TransactionObjectEvent)
{
	if (TransactionObjectEvent.GetEventType() == ETransactionObjectEventType::UndoRedo)
	{
		for (TPair<int32, FRendererConfig>& RendererConfigIt : RendererConfigs)
		{
			EnumAddFlags(RendererConfigIt.Value.Flags, EDisplayClusterRendererConfigFlags::ObjectTransacted);
		}
	}
}

bool FDisplayClusterScenePreviewModule::FRendererConfig::SetRootActor(ADisplayClusterRootActor* InRootActorPtr)
{
	if (!IsRootActorEquals(InRootActorPtr))
	{
		// Update flags
		EnumAddFlags(Flags, EDisplayClusterRendererConfigFlags::UpdateRootActorRef);

		// Store ref to root actor. This type of link supports BP and recompile.
		RootActorRef.SetSceneActor(InRootActorPtr);

		return true;
	}

	return false;
}

bool FDisplayClusterScenePreviewModule::FRendererConfig::IsRootActorEquals(ADisplayClusterRootActor* InRootActorPtr) const
{
	const TWeakObjectPtr<AActor> CurrentRootActor = RootActorRef.GetSceneActorWeakPtr().Get();

	return CurrentRootActor == InRootActorPtr;
}

ADisplayClusterRootActor* FDisplayClusterScenePreviewModule::FRendererConfig::GetRootActor()
{
	const bool bDefinedSceneActor = RootActorRef.IsDefinedSceneActor();
	const TWeakObjectPtr<AActor> RootActorWeakPtr = RootActorRef.GetSceneActorWeakPtr();
	if (ADisplayClusterRootActor* RootActorPtr = Cast<ADisplayClusterRootActor>(RootActorRef.GetOrFindSceneActor()))
	{
		const TWeakObjectPtr<AActor> NewRootActorWeakPtr = RootActorRef.GetSceneActorWeakPtr();
		if (NewRootActorWeakPtr != RootActorWeakPtr)
		{
			// The reference to the RootActor has been changed.
			EnumAddFlags(Flags, EDisplayClusterRendererConfigFlags::UpdateRootActorRef);
		}

		if (IsValid(RootActorPtr))
		{
			return RootActorPtr;
		}
	}

	if (bDefinedSceneActor)
	{
		// We lost the reference to the RootActor object.
		EnumAddFlags(Flags, EDisplayClusterRendererConfigFlags::LostRootActorRef);
	}

	return nullptr;
}

bool FDisplayClusterScenePreviewModule::FRendererConfig::ClearRendererScene()
{
	if (Renderer.IsValid())
	{
		Renderer->ClearScene();
	}

	AddedActors.Empty();
	AutoPopulateActors.Empty();

	return true;
}

bool FDisplayClusterScenePreviewModule::FRendererConfig::AddActorToRenderer(AActor* Actor, bool bAutoPopulate)
{
	if (!Renderer.IsValid())
	{
		return false;
	}

	if (!AddedActors.Contains(Actor))
	{
		Renderer->AddActor(Actor);
		AddedActors.Add(Actor);

		if (bAutoPopulate)
		{
			AutoPopulateActors.Add(Actor);
		}
	}

	return true;
}

bool FDisplayClusterScenePreviewModule::FRendererConfig::AddActorToRenderer(AActor* Actor, const TFunctionRef<bool(const UPrimitiveComponent*)>& PrimitiveFilter, bool bAutoPopulate)
{
	if (!Renderer.IsValid())
	{
		return false;
	}

	if (!AddedActors.Contains(Actor))
	{
		Renderer->AddActor(Actor, PrimitiveFilter);
		AddedActors.Add(Actor);

		if (bAutoPopulate)
		{
			AutoPopulateActors.Add(Actor);
		}
	}

	return true;
}

bool FDisplayClusterScenePreviewModule::FRendererConfig::RemoveActorFromRenderer(AActor* Actor)
{
	if (!Renderer.IsValid())
	{
		return false;
	}

	if (!AddedActors.Contains(Actor))
	{
		return false;
	}

	Renderer->RemoveActor(Actor);
	AddedActors.Remove(Actor);
	
	return true;
}

bool FDisplayClusterScenePreviewModule::FRendererConfig::GetActorsInRendererScene(bool bIncludeRoot, TArray<AActor*>& OutActors)
{
	if (bIncludeRoot)
	{
		if (ADisplayClusterRootActor* RootActorPtr = GetRootActor())
		{
			OutActors.Add(RootActorPtr);
		}
	}

	for (const TWeakObjectPtr<AActor>& Actor : AddedActors)
	{
		if (!Actor.IsValid())
		{
			continue;
		}

		OutActors.Add(Actor.Get());
	}

	return true;
}

bool FDisplayClusterScenePreviewModule::FRendererConfig::SetRendererActorSelectedDelegate(FDisplayClusterMeshProjectionRenderer::FSelection ActorSelectedDelegate)
{
	if (Renderer.IsValid())
	{
		Renderer->ActorSelectedDelegate = ActorSelectedDelegate;

		return true;
	}

	return false;
}

bool FDisplayClusterScenePreviewModule::FRendererConfig::SetRendererRenderSimpleElementsDelegate(FDisplayClusterMeshProjectionRenderer::FSimpleElementPass RenderSimpleElementsDelegate)
{
	if (Renderer.IsValid())
	{
		Renderer->RenderSimpleElementsDelegate = RenderSimpleElementsDelegate;

		return true;
	}

	return false;
}

bool FDisplayClusterScenePreviewModule::FRendererConfig::IsBlueprintMatchesRendererRootActor(UBlueprint* Blueprint) const
{
#if WITH_EDITOR
	UBlueprint* RootActorBlueprint = RootActorBlueprintClass.Get();
	if (Blueprint == RootActorBlueprint)
	{
		return true;
	}
#endif

	return false;
}

IMPLEMENT_MODULE(FDisplayClusterScenePreviewModule, DisplayClusterScenePreview);

#undef LOCTEXT_NAMESPACE
