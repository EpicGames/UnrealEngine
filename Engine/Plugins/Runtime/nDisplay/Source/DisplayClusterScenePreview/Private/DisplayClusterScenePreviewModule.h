// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDisplayClusterScenePreview.h"
#include "DisplayClusterRootActorContainers.h"

#include "Containers/Ticker.h"
#include "DisplayClusterMeshProjectionRenderer.h"
#include "Engine/TextureRenderTarget2D.h"
#include "UObject/StrongObjectPtr.h"

#include "Misc/DisplayClusterObjectRef.h"

class ADisplayClusterRootActor;
class ADisplayClusterLightCardActor;
class UTextureRenderTarget2D;
class FDisplayClusterScenePreviewProxyManager;

/** Flags for the renderer config. */
enum class EDisplayClusterRendererConfigFlags : uint16
{
	None = 0,

	/** Updated in OnObjectTransacted(). */
	ObjectTransacted = 1 << 0,

	/** When BP is recompiled. */
	BlueprintCompiled = 1 << 1,

	/** OnLevelActorAdded() */
	LevelActorAdded = 1 << 2,

	/** OnLevelActorDeleted()*/
	LevelActorDeleted = 1 << 3,

	/** OnActorPropertyChanged() for objects owned by the RootActor.*/
	LevelActorBeingModified = 1 << 4,

	/** OnActorPropertyChanged() for the RootActor.*/
	RootActorBeingModified = 1 << 5,

	/** The reference to RootActor has been updated.  */
	UpdateRootActorRef = 1 << 6,

	/** The reference to RootActorProxy has been updated.  */
	UpdateRootActorProxyRef = 1 << 7,

	/** we have lost the reference to the RootActor, it needs to be restored in another way.*/
	LostRootActorRef = 1 << 8,

	/** RootActorPath has been changed. */
	RootActorPathModified = 1 << 9,

	/** Root actor property overrides has been changed. */
	RootActorPropertyOverridesModified = 1 << 14,

	/** RootActor preview flags has been changed. */
	PreviewFlagsModified = 1 << 15,
};
ENUM_CLASS_FLAGS(EDisplayClusterRendererConfigFlags);

/**
 * Module containing tools for rendering nDisplay scene previews.
 */
class FDisplayClusterScenePreviewModule :
	public IDisplayClusterScenePreview
{
public:
	//~ IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	//~ Begin IDisplayClusterScenePreview Interface
	virtual int32 CreateRenderer() override;
	virtual bool DestroyRenderer(int32 RendererId) override;
	virtual bool SetRendererRootActorPath(int32 RendererId, const FString& ActorPath, const FDisplayClusterRootActorPropertyOverrides& InPropertyOverrides, const EDisplayClusterScenePreviewFlags PreviewFlags = EDisplayClusterScenePreviewFlags::None) override;
	virtual bool SetRendererRootActor(int32 RendererId, ADisplayClusterRootActor* Actor, const FDisplayClusterRootActorPropertyOverrides& InPropertyOverrides, const EDisplayClusterScenePreviewFlags PreviewFlags = EDisplayClusterScenePreviewFlags::None) override;
	virtual ADisplayClusterRootActor* GetRendererRootActor(int32 RendererId) override;
	virtual ADisplayClusterRootActor* GetRendererRootActorOrProxy(int32 RendererId) override;

	virtual bool GetActorsInRendererScene(int32 RendererId, bool bIncludeRoot, TArray<AActor*>& OutActors) override
	{
		FRendererConfig* Config = RendererConfigs.Find(RendererId);
		return Config ? Config->GetActorsInRendererScene(bIncludeRoot, OutActors) : false;
	}

	virtual bool AddActorToRenderer(int32 RendererId, AActor* Actor) override
	{
		FRendererConfig* Config = RendererConfigs.Find(RendererId);
		return Config ? Config->AddActorToRenderer(Actor) : false;
	}

	virtual bool AddActorToRenderer(int32 RendererId, AActor* Actor, const TFunctionRef<bool(const UPrimitiveComponent*)>& PrimitiveFilter) override
	{
		FRendererConfig* Config = RendererConfigs.Find(RendererId);
		return Config ? Config->AddActorToRenderer(Actor, PrimitiveFilter) : false;
	}
	
	virtual bool RemoveActorFromRenderer(int32 RendererId, AActor* Actor) override
	{
		FRendererConfig* Config = RendererConfigs.Find(RendererId);
		return Config ? Config->RemoveActorFromRenderer(Actor) : false;
	}

	virtual bool ClearRendererScene(int32 RendererId) override
	{
		FRendererConfig* Config = RendererConfigs.Find(RendererId);
		return Config ? Config->ClearRendererScene() : false;
	}

	virtual bool SetRendererActorSelectedDelegate(int32 RendererId, FDisplayClusterMeshProjectionRenderer::FSelection ActorSelectedDelegate) override
	{
		FRendererConfig* Config = RendererConfigs.Find(RendererId);
		return Config ? Config->SetRendererActorSelectedDelegate(ActorSelectedDelegate) : false;
	}

	virtual bool SetRendererRenderSimpleElementsDelegate(int32 RendererId, FDisplayClusterMeshProjectionRenderer::FSimpleElementPass RenderSimpleElementsDelegate) override
	{
		FRendererConfig* Config = RendererConfigs.Find(RendererId);
		return Config ? Config->SetRendererRenderSimpleElementsDelegate(RenderSimpleElementsDelegate) : false;
	}

	virtual bool Render(int32 RendererId, FDisplayClusterMeshProjectionRenderSettings& RenderSettings, FCanvas& Canvas) override;
	virtual bool RenderQueued(int32 RendererId, FDisplayClusterMeshProjectionRenderSettings& RenderSettings, const FIntPoint& Size, FRenderResultDelegate ResultDelegate) override;
	virtual bool RenderQueued(int32 RendererId, FDisplayClusterMeshProjectionRenderSettings& RenderSettings, const TWeakPtr<FCanvas> Canvas, FRenderResultDelegate ResultDelegate) override;
	virtual bool IsRealTimePreviewEnabled() const override;
	//~ End IDisplayClusterScenePreview Interface

private:
	/** Holds information about an active renderer created by this module. */
	struct FRendererConfig
	{
		/** Get the actors that were used in this render scene.*/
		bool GetActorsInRendererScene(bool bIncludeRoot, TArray<AActor*>& OutActors);

		/** Add actor to renderer. */
		bool AddActorToRenderer(AActor* Actor, bool bAutoPopulate = false);

		/** Add actor to renderer with primitive filter. */
		bool AddActorToRenderer(AActor* Actor, const TFunctionRef<bool(const UPrimitiveComponent*)>& PrimitiveFilter, bool bAutoPopulate = false);

		/** remove actor from renderer. */
		bool RemoveActorFromRenderer(AActor* Actor);

		/** Clear renderer scene. */
		bool ClearRendererScene();

		bool SetRendererActorSelectedDelegate(FDisplayClusterMeshProjectionRenderer::FSelection ActorSelectedDelegate);
		bool SetRendererRenderSimpleElementsDelegate(FDisplayClusterMeshProjectionRenderer::FSimpleElementPass RenderSimpleElementsDelegate);

		/** Returns true if the Blueprint class matches the RootActor class used in the renderer configuration. */
		bool IsBlueprintMatchesRendererRootActor(UBlueprint* Blueprint) const;

		/** Get root of the display cluster that this renderer is previewing. */
		ADisplayClusterRootActor* GetRootActor();

		/** Set new root actor ptr. */
		bool SetRootActor(ADisplayClusterRootActor* InRootActorPtr);

		/** Compare current root actor ptr and input. */
		bool IsRootActorEquals(ADisplayClusterRootActor* InRootActorPtr) const;

		/** return true if RootActor already defined. */
		bool IsDefinedRootActor() const
		{
			return RootActorRef.IsDefinedSceneActor();
		}

		/** Reference to root actor BP. */
		TWeakObjectPtr<UBlueprint> RootActorBlueprintClass;

		/** The renderer itself. */
		TSharedPtr<FDisplayClusterMeshProjectionRenderer> Renderer;

		/** The path of the root actor that this renderer is previewing. If this is not empty and the root actor becomes invalid, we will attempt to find it again using this path. */
		FString RootActorPath;

		/** All actors that have been added to the renderer (except for the root actor). */
		TArray<TWeakObjectPtr<AActor>> AddedActors;

		/** Actors that have been automatically added to the scene. */
		TArray<TWeakObjectPtr<AActor>> AutoPopulateActors;

		/** Special flags that control the behavior of the renderer. */
		EDisplayClusterScenePreviewFlags PreviewFlags = EDisplayClusterScenePreviewFlags::None;

		/** Container with properties to be overridden for the root actor used by this renderer. */
		FDisplayClusterRootActorPropertyOverrides RootActorPropertyOverrides;

		/** Renderer flags. */
		EDisplayClusterRendererConfigFlags Flags = EDisplayClusterRendererConfigFlags::None;

		/** The render target to use for queued renders. */
		TStrongObjectPtr<UTextureRenderTarget2D> RenderTarget = nullptr;

	private:
		/** Saved reference to the root actor. It is used to restore a reference to a new root actor object after BP recompilation. */
		FDisplayClusterActorRef RootActorRef;
	};

	/** Holds information about a preview render that was queued to be completed later. */
	struct FPreviewRenderJob
	{
		FPreviewRenderJob() {}

		FPreviewRenderJob(int32 RendererId, const FDisplayClusterMeshProjectionRenderSettings& Settings, const FIntPoint& Size,
			TWeakPtr<FCanvas> Canvas, FRenderResultDelegate ResultDelegate)
			: RendererId(RendererId), Settings(Settings), Size(Size), Canvas(Canvas), bWasCanvasProvided(Canvas.IsValid()), ResultDelegate(ResultDelegate)
		{
		}

		/** The ID of the renderer to use. */
		int32 RendererId;

		/** The settings to use for the render. */
		FDisplayClusterMeshProjectionRenderSettings Settings;

		/** The size of the image to render. */
		FIntPoint Size;

		/** The canvas to render to, if provided. */
		TWeakPtr<FCanvas> Canvas;

		/** Whether a canvas was provided for this job. */
		bool bWasCanvasProvided;

		/** The delegate to call when the render is completed. */
		FRenderResultDelegate ResultDelegate;
	};

	/** Handle all changes in the renderer config. */
	void InternalUpdateRendererConfig(int32 RendererId, FRendererConfig& RendererConfig);

	/** Get the root actor for a config. If the root actor pointer is invalid but we have a path to the actor, try to reacquire a pointer using the path first. */
	ADisplayClusterRootActor* InternalGetRendererRootActor(int32 RendererId, FRendererConfig& RendererConfig);

	/** Get the root actor or proxy. */
	ADisplayClusterRootActor* InternalGetRendererRootActorOrProxy(int32 RendererId, FRendererConfig& RendererConfig);

	/** Set the root actor for a config, update its scene, and register events accordingly. */
	void InternalSetRendererRootActor(int32 RendererId, FRendererConfig& RendererConfig, ADisplayClusterRootActor* Actor);

	/** Queue a preview to be rendered. */
	bool InternalRenderQueued(int32 RendererId, FDisplayClusterMeshProjectionRenderSettings& RenderSettings, TWeakPtr<FCanvas> Canvas,
		const FIntPoint& Size, FRenderResultDelegate ResultDelegate);

	/** Immediately render with the given renderer config and settings to the given canvas. */
	bool InternalRenderImmediate(int32 RendererId, FRendererConfig& RendererConfig, FDisplayClusterMeshProjectionRenderSettings& RenderSettings, FCanvas& Canvas);

	/** Check if any of the tracked root actors are set to auto-update their lightcards and register/unregister event listeners accordingly. */
	void RegisterOrUnregisterGlobalActorEvents();

	/** Register/unregister to events affecting a cluster root actor. */
	void RegisterRootActorEvents(int32 RendererId, FRendererConfig& RendererConfig, bool bShouldRegister);

	/** Clear and re-populate a renderer's scene with the root actor and lightcards if applicable. */
	void AutoPopulateScene(int32 RendererId, FRendererConfig& RendererConfig);

	/** Check whether nDisplay preview textures are being updated in real time. */
	bool UpdateIsRealTimePreviewEnabled();

	/** Called on tick to process the queued renders. */
	bool OnTick(float DeltaTime);

	/** Called when a property on a root DisplayCluster actor has changed. */
	void OnActorPropertyChanged(UObject* ObjectBeingModified, FPropertyChangedEvent& PropertyChangedEvent);

	/** Called when the user deletes an actor from the level. */
	void OnLevelActorDeleted(AActor* Actor);

	/** Called when the user adds an actor to the level. */
	void OnLevelActorAdded(AActor* Actor);

	/** Called when a blueprint for an actor we care about is compiled. */
	void OnBlueprintCompiled(UBlueprint* Blueprint);

	/** Called when any object is transacted. */
	void OnObjectTransacted(UObject* Object, const FTransactionObjectEvent& TransactionObjectEvent);

	/** release all internals data. */
	void Release();

	/** Return ProxyManager object. */
	FDisplayClusterScenePreviewProxyManager& GetProxyManager();

private:
#if WITH_EDITOR
	// Callbacks
	FDelegateHandle EditorClosedEventHandle;
	void OnEditorClosed();
#endif

private:

	/** Map from renderer ID to configuration data for that renderer. */
	TMap<int32, FRendererConfig> RendererConfigs;

	/** Queue of render jobs pending completion. */
	TQueue<FPreviewRenderJob> RenderQueue;

	/** Handle for the render ticker. */
	FTSTicker::FDelegateHandle RenderTickerHandle;

	/** The ID to use for the next created renderer. */
	int32 NextRendererId = 0;

	/** Whether this is currently registered for actor update events. */
	bool bIsRegisteredForActorEvents = false;

	/** Whether nDisplay preview textures are being updated in real time. */
	bool bIsRealTimePreviewEnabled = false;

	/** Manager for DCRA proxy objects. */
	TUniquePtr<FDisplayClusterScenePreviewProxyManager> ProxyManagerPtr;
};