// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterScenePreviewProxyManager.h"
#include "Components/DisplayClusterStageGeometryComponent.h"
#include "Engine/World.h"

FDisplayClusterScenePreviewProxyManager::~FDisplayClusterScenePreviewProxyManager()
{
	Release();
}

void FDisplayClusterScenePreviewProxyManager::Release()
{
	TickableGameObject.Reset();
	RendererProxies.Empty();

	DestroyPreviewWorld();
}

void FDisplayClusterScenePreviewProxyManager::CreatePreviewWorld()
{
	if (IsValid(PreviewWorld))
	{
		// Preview world already exists.
		return;
	}

#if WITH_EDITOR
	if (IsValid(GEngine))
	{
		PreviewWorld = NewObject<UWorld>(GetTransientPackage(), TEXT("DisplayClusterScenePreview"), RF_NoFlags);
		PreviewWorld->WorldType = EWorldType::EditorPreview;

		FWorldContext& WorldContext = GEngine->CreateNewWorldContext(PreviewWorld->WorldType);
		WorldContext.SetCurrentWorld(PreviewWorld);

		PreviewWorld->InitializeNewWorld(UWorld::InitializationValues()
			.AllowAudioPlayback(false)
			.CreatePhysicsScene(false)
			.RequiresHitProxies(true) // Only Need hit proxies in an editor scene
			.CreateNavigation(false)
			.CreateAISystem(false)
			.ShouldSimulatePhysics(false)
			.SetTransactional(false));
	}
#endif
}

void FDisplayClusterScenePreviewProxyManager::DestroyPreviewWorld()
{
	UWorld* LocalPreviewWorld = IsValid(PreviewWorld) ? PreviewWorld.Get() : nullptr;
	PreviewWorld = nullptr;

	if (IsValid(LocalPreviewWorld) && IsValid(GEngine))
	{
		LocalPreviewWorld->CleanupWorld();
		GEngine->DestroyWorldContext(LocalPreviewWorld);

		// Release PhysicsScene for fixing big fbx importing bug
		LocalPreviewWorld->ReleasePhysicsScene();
	}
}

void FDisplayClusterScenePreviewProxyManager::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(PreviewWorld);
}

FString FDisplayClusterScenePreviewProxyManager::GetReferencerName() const
{
	return TEXT("FDisplayClusterScenePreviewProxyManager");
}

void FDisplayClusterScenePreviewProxyManager::TickPreviewWorld(float DeltaTime)
{
#if WITH_EDITOR
	if (PreviewWorld)
	{
		PreviewWorld->Tick(ELevelTick::LEVELTICK_All, DeltaTime);
	}
#endif
}

void FDisplayClusterScenePreviewProxyManager::OnTick(float DeltaTime)
{
#if WITH_EDITOR
	// Tick all proxy root actors.
	for(const TPair<int32, FRendererProxy>& RendererProxyIt : RendererProxies)
	{
		RendererProxyIt.Value.TickProxyRootActor();
	}
#endif
}

ADisplayClusterRootActor* FDisplayClusterScenePreviewProxyManager::CreateRootActorProxy(int32 RendererId, ADisplayClusterRootActor* SceneRootActor)
{
	if (!SceneRootActor)
	{
		return nullptr;
	}

	check(IsValid(SceneRootActor));

	ADisplayClusterRootActor* RootActorProxy = nullptr;

#if WITH_EDITOR
	// Create new preview world for DCRA proxy
	CreatePreviewWorld();
	if (!PreviewWorld)
	{
		return nullptr;
	}

	// Create a proxy for render
	FObjectDuplicationParameters DupeActorParameters(SceneRootActor, PreviewWorld->GetCurrentLevel());
	DupeActorParameters.FlagMask = RF_AllFlags & ~(RF_ArchetypeObject | RF_Transactional); // Keeps archetypes correct in config data.
	DupeActorParameters.PortFlags = PPF_DuplicateVerbatim;
	static int32 UniqueIndex = 0;
	DupeActorParameters.DestName = FName(FString::Printf(TEXT("Preview-%s-%d-%d"), *SceneRootActor->GetName(), RendererId, UniqueIndex++));

	RootActorProxy = CastChecked<ADisplayClusterRootActor>(StaticDuplicateObjectEx(DupeActorParameters));

	// Use root actor from scene to render
	if (IDisplayClusterViewportManager* ViewportManager = RootActorProxy ? RootActorProxy->GetOrCreateViewportManager() : nullptr)
	{
		// Using DCRA from the scene for rendering
		ViewportManager->GetConfiguration().SetRootActor(EDisplayClusterRootActorType::Scene | EDisplayClusterRootActorType::Configuration, SceneRootActor);
	}

	RootActorProxy->SetFlags(RF_Transient); // This signals to the stage actor it is a proxy

	PreviewWorld->GetCurrentLevel()->AddLoadedActor(RootActorProxy);

	// Draw the geometry map for the proxy stage actor immediately to avoid a race condition where the geometry map could render
	// before the actor location changes propagate to its component proxies, resulting in an inaccurate proxy geometry map
	RootActorProxy->GetStageGeometryComponent()->Invalidate(true);

	// Spawned actor will take the transform values from the template, so manually reset them to zero here
	RootActorProxy->SetActorLocation(FVector::ZeroVector);
	RootActorProxy->SetActorRotation(FRotator::ZeroRotator);

	if (UDisplayClusterConfigurationData* ProxyConfig = RootActorProxy->GetConfigData())
	{
		// Disable lightcards so that it doesn't try to update the ones in the level instance world.
		ProxyConfig->StageSettings.Lightcard.bEnable = false;
	}

	// Set translucency sort priority of root actor proxy primitive components so that actors that are flush with screens are rendered on top of them
	RootActorProxy->ForEachComponent<UPrimitiveComponent>(false, [](UPrimitiveComponent* InPrimitiveComponent)
	{
		InPrimitiveComponent->SetTranslucentSortPriority(-10);
	});
#endif

	return RootActorProxy;
}

void FDisplayClusterScenePreviewProxyManager::DestroyRootActorProxy(ADisplayClusterRootActor* ProxyRootActor)
{
#if WITH_EDITOR
	if (PreviewWorld && ProxyRootActor)
	{
		PreviewWorld->GetCurrentLevel()->RemoveLoadedActors({ ProxyRootActor });
	}
#endif
}

void FDisplayClusterScenePreviewProxyManager::SetSceneRootActorForRenderer(int32 RendererId, ADisplayClusterRootActor* SceneRootActor, EDisplayClusterScenePreviewFlags PreviewFlags)
{
	if (RendererProxies.Contains(RendererId))
	{
		FRendererProxy& RendererProxy = RendererProxies[RendererId];

		if (RendererProxy.GetProxyRootActor())
		{
			// If the proxy object already exists, check whether the root actor is the same or not.
			if (RendererProxy.SceneRootActorWeakPtr == SceneRootActor)
			{
				// Re-use existing proxy, but update flags
				RendererProxy.PreviewFlags = PreviewFlags;

				return;
			}
		}

		// The root agent has changed, destroy the proxy currently in use.
		DestroyRootActorProxy(RendererProxy.GetProxyRootActor());

		RendererProxies.Remove(RendererId);
	}

	if (IsValid(SceneRootActor))
	{
		if (ADisplayClusterRootActor* ProxyRootActor = CreateRootActorProxy(RendererId, SceneRootActor))
		{
			// Create new proxy
			FRendererProxy RendererProxy;
			RendererProxy.SceneRootActorWeakPtr = SceneRootActor;
			RendererProxy.ProxyRootActorWeakPtr = ProxyRootActor;
			RendererProxy.PreviewFlags = PreviewFlags;

			// Update RootActor proxy
			RendererProxy.TickProxyRootActor();

			RendererProxies.Emplace(RendererId, RendererProxy);
		}
	}

	// Configure a tick callback for existing RendererProxies.
	if (RendererProxies.IsEmpty())
	{
		TickableGameObject.Reset();
	}
	else if (!TickableGameObject.IsValid())
	{
		TickableGameObject = MakeUnique<FDisplayClusterTickableGameObject >();
		TickableGameObject->OnTick().AddRaw(this, &FDisplayClusterScenePreviewProxyManager::OnTick);
	}
}

ADisplayClusterRootActor* FDisplayClusterScenePreviewProxyManager::GetProxyRootActor(int32 RendererId) const
{
	if (RendererProxies.Contains(RendererId))
	{
		return RendererProxies[RendererId].GetProxyRootActor();
	}

	return nullptr;
}

ADisplayClusterRootActor* FDisplayClusterScenePreviewProxyManager::GetSceneRootActor(int32 RendererId) const
{
	if (RendererProxies.Contains(RendererId))
	{
		if (ADisplayClusterRootActor* SceneRootActor = RendererProxies[RendererId].GetSceneRootActor())
		{
			return IsValid(SceneRootActor) ? SceneRootActor : nullptr;
		}
	}

	return nullptr;
}

void FDisplayClusterScenePreviewProxyManager::FRendererProxy::TickProxyRootActor() const
{
	if (ADisplayClusterRootActor* ProxyRootActor = GetProxyRootActor())
	{
		if (EnumHasAnyFlags(PreviewFlags, EDisplayClusterScenePreviewFlags::ProxyFollowSceneRootActor))
		{
			if (ADisplayClusterRootActor* SceneRootActor = GetSceneRootActor())
			{
				if (ProxyRootActor != SceneRootActor)
				{
					// Move the RootActorProxy to the same position as in the scene to match the position of the LC in world space.
					if (EnumHasAnyFlags(PreviewFlags, EDisplayClusterScenePreviewFlags::ProxyFollowSceneRootActor))
					{
						const FTransform NewTransform = SceneRootActor->GetActorTransform();
						const FTransform OldTransform = ProxyRootActor->GetActorTransform();
						if (!NewTransform.Equals(OldTransform, UE_KINDA_SMALL_NUMBER))
						{
							ProxyRootActor->SetActorTransform(NewTransform);
						}
					}
				}
			}
		}

		// Force preview renderer call for proxy root actor.
		if (EnumHasAnyFlags(PreviewFlags, EDisplayClusterScenePreviewFlags::ProxyTickPreviewRenderer))
		{
			ProxyRootActor->TickPreviewRenderer();
		}
	}
}

ADisplayClusterRootActor* FDisplayClusterScenePreviewProxyManager::FRendererProxy::GetProxyRootActor() const
{
	if (ADisplayClusterRootActor* RootActor = ProxyRootActorWeakPtr.Get())
	{
		if (IsValid(RootActor))
		{
			return RootActor;
		}
	}

	return nullptr;
}

ADisplayClusterRootActor* FDisplayClusterScenePreviewProxyManager::FRendererProxy::GetSceneRootActor() const
{
	if (ADisplayClusterRootActor* RootActor = SceneRootActorWeakPtr.Get())
	{
		if (IsValid(RootActor))
		{
			return RootActor;
		}
	}

	return nullptr;
}

