// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/GCObject.h"

#include "DisplayClusterScenePreviewEnums.h"
#include "DisplayClusterRootActor.h"
#include "Misc/DisplayClusterTickableGameObject.h"

/**
 * Creates and handle DCRA proxies.
 */
class FDisplayClusterScenePreviewProxyManager : public FGCObject
{
public:
	~FDisplayClusterScenePreviewProxyManager();

	/** Assign SceneRootActor to the renderer with the specified ID to create a root proxy actor for it.
	* @param RendererId - ID of renderer that uses this DCRA
	* @param RootActor  - the new scene root actor that used by this renderer
	* 
	* Note: If RootActor is null, it means that the renderer is no longer using the proxy.
	*/
	void SetSceneRootActorForRenderer(int32 RendererId, ADisplayClusterRootActor* RootActor, EDisplayClusterScenePreviewFlags PreviewFlags);

	/** Get the DCRA proxy for the renderer by id.
	* 
	* @param RendererId - ID of renderer that uses this DCRA
	* 
	* @return - A reference to the RootActor proxy, if it exists, or nullptr.
	* 
	* Note: You must call SetSceneRootActorForRenderer() for the renderer and the root actor being used before using this function.
	*/
	ADisplayClusterRootActor* GetProxyRootActor(int32 RendererId) const;

	/** Get the DCRA in scene used to create a proxy for the renderer by id.
	*
	* @param RendererId - ID of renderer that uses this DCRA
	*
	* @return - A reference to the RootActor in scene, if it exists, or nullptr.
	*
	* Note: You must call SetSceneRootActorForRenderer() for the renderer and the root actor being used before using this function.
	*/
	ADisplayClusterRootActor* GetSceneRootActor(int32 RendererId) const;

	/** Run the Tick() function for the preview world. */
	void TickPreviewWorld(float DeltaTime);

	/** Release all internal data.*/
	void Release();

private:
	/** This function calls Tick() for a preview world with proxy root actors, which triggers rendering of previews for them. */
	void OnTick(float DeltaTime);
	void CreatePreviewWorld();
	void DestroyPreviewWorld();

	/** Create RootActor proxy. */
	ADisplayClusterRootActor* CreateRootActorProxy(int32 RendererId, ADisplayClusterRootActor* SceneRootActor);

	void DestroyRootActorProxy(ADisplayClusterRootActor* ProxyRootActor);

	//~ Begin FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override;
	//~ End FGCObject interface

private:
	/** Container for RootActorProxy and related data. */
	struct FRendererProxy
	{
		/** Get value from the ProxyRootActorWeakPtr variable. */
		ADisplayClusterRootActor* GetProxyRootActor() const;

		/** Get value from the SceneRootActorWeakPtr variable. */
		ADisplayClusterRootActor* GetSceneRootActor() const;

		/** Update ProxyRootActor. */
		void TickProxyRootActor() const;

		// RootActorProxy object created based on the RootActor in the scene.
		TWeakObjectPtr<ADisplayClusterRootActor> ProxyRootActorWeakPtr;

		// RootActor in scene that used to create DCRA proxy
		TWeakObjectPtr<ADisplayClusterRootActor> SceneRootActorWeakPtr;

		// Special flags that control the behavior of the renderer.
		EDisplayClusterScenePreviewFlags PreviewFlags;
	};

	/** Registered proxies for renderers by ID. */
	TMap<int32, FRendererProxy> RendererProxies;

	/** The preview world is only used when using the DCRA proxy. */
	TObjectPtr<class UWorld> PreviewWorld = nullptr;

	// When CachedObjects is not empty, this ticking object will be created.
	// Also, this object will be deleted when CachedObjects becomes empty.
	TUniquePtr<FDisplayClusterTickableGameObject> TickableGameObject;
};
