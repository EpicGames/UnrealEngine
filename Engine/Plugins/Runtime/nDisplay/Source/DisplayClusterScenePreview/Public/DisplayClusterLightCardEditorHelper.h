// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StageActor/DisplayClusterWeakStageActorPtr.h"

#include "Components/DisplayClusterLabelConfiguration.h"

#include "DisplayClusterMeshProjectionRenderer.h"

#include "SceneView.h"
#include "UnrealClient.h"
#include "Containers/Union.h"
#include "UObject/StrongObjectPtr.h"

class ADisplayClusterLightCardActor;
class ADisplayClusterRootActor;
class FPreviewScene;
class FSceneView;
class UProceduralMeshComponent;
class UTexture2D;
struct FSceneViewInitOptions;

#if WITH_EDITOR
class FEditorViewportClient;
#endif

/**
 * Helper class for moving lightcards in an nDisplay cluster in various projection modes.
 * Contains functions to perform coordinate conversion, scene setup, and lightcard movement, and manages the
 * normal maps needed to convert from projected to world coordinates.
*/
struct FDisplayClusterLightCardEditorHelper
{
public:
	enum class ECoordinateSystem : uint8
	{
		Cartesian,
		Spherical
	};

	struct FSphericalCoordinates
	{
	public:

		/** Constructors */
		DISPLAYCLUSTERSCENEPREVIEW_API FSphericalCoordinates(const FVector& CartesianPosition);
		DISPLAYCLUSTERSCENEPREVIEW_API FSphericalCoordinates();

		/** Return equivalent cartesian coordinates */
		DISPLAYCLUSTERSCENEPREVIEW_API FVector AsCartesian() const;

		/** Addition operator */
		DISPLAYCLUSTERSCENEPREVIEW_API FSphericalCoordinates operator+(FSphericalCoordinates const& Other) const;

		/** Subtraction operator */
		DISPLAYCLUSTERSCENEPREVIEW_API FSphericalCoordinates operator-(FSphericalCoordinates const& Other) const;

		/** Conform parameters to their normal ranges */
		DISPLAYCLUSTERSCENEPREVIEW_API void Conform();

		/** Returns a conformed version of this struct without changing the current one */
		DISPLAYCLUSTERSCENEPREVIEW_API FSphericalCoordinates GetConformed() const;

		/** Returns true if the inclination is pointing at north or south poles, within the given margin (in radians) */
		DISPLAYCLUSTERSCENEPREVIEW_API bool IsPointingAtPole(double Margin = 1e-6) const;

		double Radius = 0;      // unitless   0+      (when conforming)
		double Inclination = 0; // radians    0 to PI (when conforming)
		double Azimuth = 0;     // radians  -PI to PI (when conforming)
	};
	
public:
	/** Create a projection helper that automatically creates its own preview renderer for normal map generation. */
	DISPLAYCLUSTERSCENEPREVIEW_API FDisplayClusterLightCardEditorHelper();

	/** Create a projection helper that uses an existing preview renderer for normal map generation. */
	DISPLAYCLUSTERSCENEPREVIEW_API FDisplayClusterLightCardEditorHelper(int32 RendererId);

	/** Clean up the projection renderer and any data used by the helper. */
	DISPLAYCLUSTERSCENEPREVIEW_API ~FDisplayClusterLightCardEditorHelper();

#if WITH_EDITOR
	/** Set the viewport client for which this is handling rendering. If set, its data will be included in the SceneViewInitOptions generated by this helper. */
	DISPLAYCLUSTERSCENEPREVIEW_API void SetEditorViewportClient(TWeakPtr<FEditorViewportClient> InViewportClient);
#endif

	/** Set the projection mode to use. */
	DISPLAYCLUSTERSCENEPREVIEW_API void SetProjectionMode(EDisplayClusterMeshProjectionType Value);

	/** Get the projection mode to use. */
	DISPLAYCLUSTERSCENEPREVIEW_API EDisplayClusterMeshProjectionType GetProjectionMode() const;

	/** Set whether to use an orthographic projection. */
	DISPLAYCLUSTERSCENEPREVIEW_API void SetIsOrthographic(bool bValue);

	/** Get whether this is using an orthographic projection. */
	DISPLAYCLUSTERSCENEPREVIEW_API bool GetIsOrthographic() const;

	/**
	 * Set the root actor of the DisplayCluster being controlled. This will also update the renderer's root actor.
	 * Don't call this if the helper was created with an existing preview renderer, as it will automatically the root actor from that renderer.
	 */
	DISPLAYCLUSTERSCENEPREVIEW_API void SetRootActor(ADisplayClusterRootActor& NewRootActor);

	/**
	 * If the root actor is a proxy, use this to set the corresponding level instance that is being proxied.
	 */
	DISPLAYCLUSTERSCENEPREVIEW_API void SetLevelInstanceRootActor(ADisplayClusterRootActor& NewRootActor);

	/**
	 * Get (or generate) a visualization of one of the helper's normal maps.
	 *
	 * @param bShowNorthMap If true, return the texture for the north normal map. Otherwise, return the texture for the south normal map.
	 */
	UE_DEPRECATED(5.3, "The raw stage geoemetry normal maps are no longer exposed by the stage geometry component")
	DISPLAYCLUSTERSCENEPREVIEW_API const UTexture2D* GetNormalMapTexture(bool bShowNorthMap);

	/**
	 * Moves the given light cards to the specified pixel position within the provided scene view.
	 *
	 * @param Actors The actors that we are moving
	 * @param PixelPos The pixel location to move the actors to
	 * @param SceneView The scene view used to convert from pixel position to 3D position
	*/
	DISPLAYCLUSTERSCENEPREVIEW_API void MoveActorsToPixel(const TArray<FDisplayClusterWeakStageActorPtr>& Actors, const FIntPoint& PixelPos, const FSceneView& SceneView);

	/**
	 * Moves specified actors to desired coordinates. Actual radius will be based on flush constraint and actor's RadialOffset.
	 *
	 * @param Actors The actors that we are moving
	 * @param SphericalCoords The desired location of the actors in spherical coordinates with respect to origin.
	*/
	DISPLAYCLUSTERSCENEPREVIEW_API void MoveActorsTo(const TArray<FDisplayClusterWeakStageActorPtr>& Actors, const FSphericalCoordinates& SphericalCoords);
	
	/**
	 * Moves specified cards to a coordinate in viewport space as if dragged by a translate widget.
	 * 
	 * @param Actors The actors that we are moving
	 * @param PixelPos The screen pixel position of the widget
	 * @param SceneView The scene view used to convert from pixel position to 3D position
	 * @param CoordinateSystem The coordinate system to use when computing drag constraints
	 * @param DragWidgetOffset The offset between the actual cursor position and the position of the widget when the drag action started
	 * @param DragAxis The axis along which the widget is being dragged
	 * @param PrimaryActor The actor used to calculate the translation/rotation delta. If not provided, the last entry in Actors will be used.
	 */
	DISPLAYCLUSTERSCENEPREVIEW_API void DragActors(const TArray<FDisplayClusterWeakStageActorPtr>& Actors, const FIntPoint& PixelPos, const FSceneView& SceneView,
		ECoordinateSystem CoordinateSystem, const FVector& DragWidgetOffset, EAxisList::Type DragAxis, FDisplayClusterWeakStageActorPtr PrimaryActor = nullptr);
	
	/**
	 * Moves specified UV actors to a coordinate in viewport space as if dragged by a translate widget.
	 *
	 * @param Actors The actors that we are moving
	 * @param PixelPos The screen pixel position of the widget
	 * @param SceneView The scene view used to convert from pixel position to 3D position
	 * @param DragWidgetOffset The offset between the actual cursor position and the position of the widget when the drag action started
	 * @param DragAxis The axis along which the widget is being dragged
	 * @param PrimaryActor The actor used to calculate the translation/rotation delta. If not provided, the last entry in Actors will be used.
	 */
	DISPLAYCLUSTERSCENEPREVIEW_API void DragUVActors(const TArray<FDisplayClusterWeakStageActorPtr>& Actors, const FIntPoint& PixelPos, const FSceneView& SceneView,
		const FVector& DragWidgetOffset, EAxisList::Type DragAxis, FDisplayClusterWeakStageActorPtr PrimaryActor = nullptr);

	/** Ensures that the actor root component is at the same location as the projection/origin */
	DISPLAYCLUSTERSCENEPREVIEW_API void VerifyAndFixActorOrigin(const FDisplayClusterWeakStageActorPtr& Actor);

	/**
	 * Calculates the relative normal vector and world position in the specified direction from the given origin.
	 * 
	 * @param InOrigin The origin point of the view.
	 * @param InDirection The direction in which check the normal.
	 * @param OutWorldPosition The world-space coordinates of the calculated position.
	 * @param OutRelativeNormal The normal vector at the calculated position.
	 * @param InDesiredDistanceFromFlush The desired flush distance from the initial calculated position.
	 * @return true if the position was found, or false if the normal maps need to be updated first.
	 */
	DISPLAYCLUSTERSCENEPREVIEW_API bool CalculateNormalAndPositionInDirection(const FVector& InOrigin, const FVector& InDirection, FVector& OutWorldPosition,
		FVector& OutRelativeNormal, double InDesiredDistanceFromFlush = 0.);

	/**
	 * Calculates the desired direction from the origin given a pixel position within the view.
	 * 
	 * @param InPixelPos The desired position within the view in pixels.
	 * @param InSceneView The view that was displayed when the pixel position was chosen.
	 * @param InOriginOffset An offset to apply to the origin before calling TraceScreenRay (if this is in orthographic mode).
	 * @param OutOrigin The origin point of the view.
	 * @param OutDirection The direction relative to the origin.
	 * @return true if the direction was found
	 */
	DISPLAYCLUSTERSCENEPREVIEW_API bool CalculateOriginAndDirectionFromPixelPosition(const FIntPoint& InPixelPos, const FSceneView& InSceneView, const FVector& InOriginOffset,
		FVector& OutOrigin, FVector& OutDirection);

	/** Converts a pixel coordinate into a point and direction vector in world space */
	DISPLAYCLUSTERSCENEPREVIEW_API void PixelToWorld(const FSceneView& View, const FIntPoint& PixelPos, FVector& OutOrigin, FVector& OutDirection) const;

	/** Converts a world coordinate into a point in screen space, and returns true if the world position is on the screen. */
	DISPLAYCLUSTERSCENEPREVIEW_API bool WorldToPixel(const FSceneView& View, const FVector& WorldPos, FVector2D& OutPixelPos) const;

	/** Converts a world coordinate into a point in screen space using a specific projection mode, and returns true if the world position is on the screen. */
	DISPLAYCLUSTERSCENEPREVIEW_API bool WorldToPixel(const FSceneView& View, const FVector& WorldPos, FVector2D& OutPixelPos, EDisplayClusterMeshProjectionType OverrideProjectionMode) const;

	/**
	 * Sets up FSceneViewInitOptions' basic settings in a consistent way for preview renders.
	 * If RotationMatrix is not provided, one will be automatically generated based on the view rotation.
	 */
	DISPLAYCLUSTERSCENEPREVIEW_API void GetSceneViewInitOptions(
		FSceneViewInitOptions& OutViewInitOptions,
		float InFOV,
		const FIntPoint& InViewportSize,
		const FVector& InLocation,
		const FRotator& InRotation,
		const EAspectRatioAxisConstraint InAspectRatioAxisConstraint = EAspectRatioAxisConstraint::AspectRatio_MaintainYFOV,
		float InNearClipPlane = GNearClippingPlane,
		const FMatrix* InRotationMatrix = nullptr,
		float InDPIScale = 1.f);

	/**
	 * Sets up projection-related settings of a preview render in a consistent way. 
	 * This includes the ProjectionType and its related ProjectionTypeSettings.
	 * If provided, the UV projection's panning settings will updated to reflect the ViewLocation passed in.
	 */
	DISPLAYCLUSTERSCENEPREVIEW_API void ConfigureRenderProjectionSettings(FDisplayClusterMeshProjectionRenderSettings& OutRenderSettings, FVector ViewLocation = FVector::ZeroVector) const;

	/** Gets the spherical coordinates of the specified actor. */
	DISPLAYCLUSTERSCENEPREVIEW_API FSphericalCoordinates GetActorCoordinates(const FDisplayClusterWeakStageActorPtr& Actor);

	/**
	 * Creates the default ShouldRenderPrimitive filter that most FDisplayClusterMeshProjectionRenderSettings will use.
	 * Filters actors so that only UV lightcards are shown in UV mode, and only non-UV light cards are shown otherwise.
	 */
	DISPLAYCLUSTERSCENEPREVIEW_API FDisplayClusterMeshProjectionPrimitiveFilter::FPrimitiveFilter CreateDefaultShouldRenderPrimitiveFilter() const;

	/**
	 * Creates the default ShouldApplyProjectionToPrimitive filter that most FDisplayClusterMeshProjectionRenderSettings will use.
	 * Filters actors so that in UV projection mode, UV light cards are rendered linearly.
	 */
	DISPLAYCLUSTERSCENEPREVIEW_API FDisplayClusterMeshProjectionPrimitiveFilter::FPrimitiveFilter CreateDefaultShouldApplyProjectionToPrimitiveFilter() const;

	struct FAddLightCardArgs
	{
		UE_DEPRECATED(5.3, "Use LabelConfiguration to configure label visibility.")
		bool bShowLabels = false;
		UE_DEPRECATED(5.3, "Use LabelConfiguration to configure label scale.")
		float LabelScale = 1.f;

		/** Label configuration to use for the new light card */
		FDisplayClusterLabelConfiguration LabelConfiguration;

		FAddLightCardArgs():
			LabelConfiguration()
		{
		}
	};
	
	struct FSpawnActorArgs
	{
		/** [Required] The root actor managing the stage */
		ADisplayClusterRootActor* RootActor;
		
		/** [Required if Template is null] The class to spawn */
		TSubclassOf<AActor> ActorClass;

		/** [Required if ActorClass is null] The template to use */
		const class UDisplayClusterStageActorTemplate* Template;
		
		/** The name to set for the actor and label */
		FName ActorName;
		
		/** The projection mode the actor should be spawned in under */
		EDisplayClusterMeshProjectionType ProjectionMode;

		/** An override level for the actor, otherwise the RootActor level is used */
		ULevel* Level;

		/** Args if this is a light card being added to the root actor */
		FAddLightCardArgs AddLightCardArgs;

		/** Notify this actor should be used as a preview (such as for drag & drop) */
		bool bIsPreview;

		FSpawnActorArgs():
		RootActor(nullptr),
		Template(nullptr),
		ActorName(NAME_None),
		ProjectionMode(),
		Level(nullptr),
		bIsPreview(false)
		{}
	};

	/** Spawns a new actor for stage use and adds it to the root actor if it is a light card */
	static DISPLAYCLUSTERSCENEPREVIEW_API AActor* SpawnStageActor(const FSpawnActorArgs& InSpawnArgs);
	
	/** Adds the given Light Card to the root actor */
	static DISPLAYCLUSTERSCENEPREVIEW_API void AddLightCardsToRootActor(const TArray<ADisplayClusterLightCardActor*>& LightCards, ADisplayClusterRootActor* RootActor,
		const FAddLightCardArgs& AddLightCardArgs = FAddLightCardArgs());
	
private:
	/**
	 * Update the cached pointer to the root actor from the associated preview renderer and, if the actor has changed, update related data.
	 * Returns the root actor.
	 */
	ADisplayClusterRootActor* UpdateRootActor();

	/**
	 * Update the cached pointer to the component marking the origin point of the projection and return it.
	 * Note that this has a cost, so avoid calling it frequently.
	 */
	USceneComponent* UpdateProjectionOriginComponent();

	/** Get the origin point of the projection from the ProjectionOriginComponent (or root actor as a fallback). */
	FVector GetProjectionOrigin() const;

	/** Determines the appropriate delta rotation needed to move the specified light card to the given position within the view. */
	FRotator GetActorRotationDelta(const FIntPoint& PixelPos, const FSceneView& View, const FDisplayClusterWeakStageActorPtr& Actor,
		ECoordinateSystem CoordinateSystem, EAxisList::Type DragAxis, const FVector& DragWidgetOffset);

	/** Determines the appropriate delta in spherical coordinates needed to move the specified actor to the given position within the view. */
	FSphericalCoordinates GetActorTranslationDelta(const FIntPoint& PixelPos, const FSceneView& View, const FDisplayClusterWeakStageActorPtr& Actor,
		ECoordinateSystem CoordinateSystem, EAxisList::Type DragAxis, const FVector& DragWidgetOffset);

	/** Determines the appropriate delta in UV coordinates needed to move the specified UV actor to the given position within the view. */
	FVector2D GetUVActorTranslationDelta(const FIntPoint& PixelPos, const FSceneView& View, const FDisplayClusterWeakStageActorPtr& Actor,
		EAxisList::Type DragAxis, const FVector& DragWidgetOffset);

	/** Moves specified actor to desired coordinates immediately using the current normal maps. Requires valid normal maps. */
	void InternalMoveActorTo(const FDisplayClusterWeakStageActorPtr& Actor, const FSphericalCoordinates& Position, bool bIsFinalChange) const;

	/** Moves specified actors to a coordinate in viewport space as if dragged by a widget. Requires valid normal maps. */
	void InternalDragActors(const TArray<FDisplayClusterWeakStageActorPtr>& Actors, const FIntPoint& PixelPos, const FSceneView& View,
		ECoordinateSystem CoordinateSystem, const FVector& DragWidgetOffset, EAxisList::Type DragAxis, FDisplayClusterWeakStageActorPtr PrimaryActor);

	/** Sets the actor position to the given spherical coordinates */
	void SetActorCoordinates(const FDisplayClusterWeakStageActorPtr& Actor, const FSphericalCoordinates& SphericalCoords) const;

	/** Performs a ray trace against the stage's geometry, and returns the hit point. */
	bool TraceStage(const FVector& RayStart, const FVector& RayEnd, FVector& OutHitLocation);

	/** Traces the world geometry to find the best direction vector from the origin to a valid point in space using a screen ray. Requires valid normal maps. */
	FVector TraceScreenRay(const FVector& RayOrigin, const FVector& RayDirection, const FVector& ViewOrigin);

	/** Calculates the final distance from the origin of a light card, given its flush distance and a desired offset */
	double CalculateFinalLightCardDistance(double FlushDistance, double DesiredOffsetFromFlush = 0.) const;

	/** Invalidates the viewport's normal map, forcing it to be re-rendered before using it next */
	void InvalidateNormalMap();

	/** Update the normal maps and mesh if necessary. Returns true if there are now valid normal maps to read from. */
	bool UpdateNormalMaps();

	/** Update the normal map mesh. Only call this once both maps are valid. */
	void UpdateNormalMapMesh();

	/** Called when a world beings cleanup. We destroy our preview scene if the world containing the root actor is cleaned up. */
	void OnWorldCleanup(UWorld* World, bool bSessionEnded, bool bCleanupResources);

#if WITH_EDITORONLY_DATA
	/** Called when any actor properties changed, so we can invalidate normals if it is the root actor. */
	void OnActorPropertyChanged(UObject* ObjectBeingModified, FPropertyChangedEvent& PropertyChangedEvent);

	/** Called when the root actor's blueprint is compiled so we can invalidate normals. */
	void OnRootActorBlueprintCompiled(UBlueprint* Blueprint);
#endif

#if WITH_EDITOR
	/** Send property change events for any properties that may have changed due to a actor being moved. */
	void PostEditChangePropertiesForMovedActor(const FDisplayClusterWeakStageActorPtr& Actor) const;
#endif

private:
#if WITH_EDITOR
	/** The viewport client for which this is handling projections. This can be null. */
	TWeakPtr<FEditorViewportClient> ViewportClient;
#endif

	/** The preview scene containing NormalMapMeshComponent. */
	TSharedPtr<FPreviewScene> NormalMeshScene;

	/** A morphed ico-sphere mesh component that approximates the normal and depth map. */
	TWeakObjectPtr<UProceduralMeshComponent> NormalMapMeshComponent;

	/** The cached root actor of the DisplayCluster being controlled. You should generally use UpdateRootActor() instead of accessing this directly. */
	TWeakObjectPtr<ADisplayClusterRootActor> CachedRootActor;

	/** The level instance root actor of the DisplayCluster being controlled, if RootActor is a proxy. */
	TWeakObjectPtr<ADisplayClusterRootActor> LevelInstanceRootActor;

	/** The component of the root actor that is acting as the projection origin. Can be either the root component (stage origin) or a view point component */
	TWeakObjectPtr<USceneComponent> ProjectionOriginComponent;

	/** The projection mode of the view this is helping. */
	EDisplayClusterMeshProjectionType ProjectionMode;

	/** The ID of the preview renderer used for normal map generation, as returned by IDisplayClusterScenePreview. */
	int32 RendererId;

	/** The radius of the bounding sphere that entirely encapsulates the root actor */
	float RootActorBoundingRadius = 0.0f;

	/** Whether this created its own preview renderer (and therefore is responsible for destroying it). */
	bool bCreatedRenderer = false;

	/** If true, this is using an orthographic projection mode. */
	bool bIsOrthographic = false;

	/** Indicates if the cached normal map is invalid and needs to be redrawn */
	bool bNormalMapInvalid = false;
};
