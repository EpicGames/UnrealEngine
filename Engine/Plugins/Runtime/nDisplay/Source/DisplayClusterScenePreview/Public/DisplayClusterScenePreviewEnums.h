// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * Special flags that control the behavior of the renderer.
*/
enum class EDisplayClusterScenePreviewFlags : uint8
{
	None = 0,

	/** Use DCRA proxy for rendering. */
	UseRootActorProxy = 1 << 0,

	/** Automatically update the renderer with stage actors belonging to the RootActor in scene. */
	AutoUpdateStageActors = 1 << 1,

	/* Move the RootActorProxy to the same position as RootActor in the scene to match the position of the StageActors in world space. */
	ProxyFollowSceneRootActor = 1 << 2,

	/** The proxy always calls the TickPreviewRenderer() function. */
	ProxyTickPreviewRenderer= 1 << 3,
};
ENUM_CLASS_FLAGS(EDisplayClusterScenePreviewFlags);
