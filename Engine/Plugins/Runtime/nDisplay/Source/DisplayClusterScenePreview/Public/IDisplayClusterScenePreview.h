// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#include "ShowFlags.h"
#include "DisplayClusterMeshProjectionRenderer.h"
#include "DisplayClusterScenePreviewEnums.h"

class ADisplayClusterRootActor;
struct FDisplayClusterRootActorPropertyOverrides;

DECLARE_DELEGATE_OneParam(FRenderResultDelegate, FRenderTarget*);

/**
 * Interface for module containing tools for generation Display Cluster scene previews.
 */
class IDisplayClusterScenePreview : public IModuleInterface
{
public:
	static constexpr const TCHAR* ModuleName = TEXT("DisplayClusterScenePreview");

public:
	virtual ~IDisplayClusterScenePreview() = default;

	/**
	 * Singleton-like access to this module's interface. This is just for convenience!
	 * Beware of calling this during the shutdown phase, though. Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IDisplayClusterScenePreview& Get()
	{
		return FModuleManager::GetModuleChecked<IDisplayClusterScenePreview>(ModuleName);
	}

	/**
	 * Checks to see if this module is loaded and ready. It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded(ModuleName);
	}

	/**
	 * Create a preview renderer. Returns an index used to refer to this renderer in future API calls.
	 */
	virtual int32 CreateRenderer() = 0;

	/** 
     * Destroy a renderer, cleaning up all of its resources.
	 * 
	 * @param RendererId The ID of the renderer as returned from CreateRenderer.
	 */
	virtual bool DestroyRenderer(int32 RendererId) = 0;

	/**
	 * Set the path of the ADisplayClusterRootActor actor for a renderer, which will be added to the preview scene and used to determine the render world.
	 * Whenever a render is performed with this renderer, it will try to find the actor at that path and use it as the root.
	 * 
	 * @param RendererId The ID of the renderer as returned from CreateRenderer.
	 * @param ActorPath The path of the ADisplayClusterRootActor actor to preview.
	 * @param InPropertyOverrides optional overrides for root actor
	 * @param PreviewFlags Special flags that control the behavior of the renderer.
	 */
	virtual bool SetRendererRootActorPath(int32 RendererId, const FString& ActorPath, const FDisplayClusterRootActorPropertyOverrides& InPropertyOverrides, const EDisplayClusterScenePreviewFlags PreviewFlags = EDisplayClusterScenePreviewFlags::None) = 0;

	/**
	 * Set the root DisplayCluster actor for the renderer with the given ID, which will be added to the preview scene and used to determine the render world.
	 * Note that if the actor is destroyed (including by a recompile, reload, etc.), you will need to call this again to update the actor.
	 * This will also clear the path set by SetRendererRootActorPath.
	 * 
	 * @param RendererId The ID of the renderer as returned from CreateRenderer.
	 * @param Actor The actor to preview.
	 * @param InPropertyOverrides optional overrides for root actor
	 * @param PreviewFlags Special flags that control the behavior of the renderer.
	 */
	virtual bool SetRendererRootActor(int32 RendererId, ADisplayClusterRootActor* Actor, const FDisplayClusterRootActorPropertyOverrides& InPropertyOverrides, const EDisplayClusterScenePreviewFlags PreviewFlags = EDisplayClusterScenePreviewFlags::None) = 0;

	/**
	 * Get the root actor of a renderer. 
	 * 
	 * @param RendererId The ID of the renderer as returned from CreateRenderer.
	 */
	virtual ADisplayClusterRootActor* GetRendererRootActor(int32 RendererId) = 0;

	/**
	 * Get the root actor (or its proxy) of a renderer.
	 *
	 * @param RendererId The ID of the renderer as returned from CreateRenderer.
	 */
	virtual ADisplayClusterRootActor* GetRendererRootActorOrProxy(int32 RendererId) = 0;

	/**
	 * Get a list of all actors that have been added to a renderer's scene.
	 * This includes any actors that the renderer automatically added to itself.
	 *
	 * @param RendererId The ID of the renderer as returned from CreateRenderer.
	 * @param bIncludeRoot If true, include the renderer's root actor.
	 * @param OutActors The array to fill with actors.
	 */
	virtual bool GetActorsInRendererScene(int32 RendererId, bool bIncludeRoot, TArray<AActor*>& OutActors) = 0;

	/**
	 * Add an actor to the preview scene for a renderer.
	 * 
	 * @param RendererId The ID of the renderer as returned from CreateRenderer.
	 * @param Actor The actor to add to the renderer's scene.
	 */
	virtual bool AddActorToRenderer(int32 RendererId, AActor* Actor) = 0;

	/**
	 * Add an actor to the preview scene for a renderer.
	 *
	 * @param RendererId The ID of the renderer as returned from CreateRenderer.
	 * @param Actor The actor to add to the renderer's scene.
	 * @param PrimitiveFilter A custom filter used to determine which of the actor's primitives are added to the renderer's scene.
	 */
	virtual bool AddActorToRenderer(int32 RendererId, AActor* Actor, const TFunctionRef<bool(const UPrimitiveComponent*)>& PrimitiveFilter) = 0;

	/**
	 * Removes an actor from the preview scene for a renderer.
	 *
	 * @param RendererId The ID of the renderer as returned from CreateRenderer.
	 * @param Actor The actor to remove to the renderer's scene.
	 */
	virtual bool RemoveActorFromRenderer(int32 RendererId, AActor* Actor) = 0;

	/**
	 * Clear the scene of a renderer.
	 * 
	 * @param RendererId The ID of the renderer as returned from CreateRenderer.
	 */
	virtual bool ClearRendererScene(int32 RendererId) = 0;

	/**
	 * Set the ActorSelectedDelegate of a renderer.
	 * See FDisplayClusterMeshProjectionRenderer::ActorSelectedDelegate for more info.
	 * 
	 * @param RendererId The ID of the renderer as returned from CreateRenderer.
	 * @param ActorSelectedDelegate The delegate to set on the renderer.
	 */
	virtual bool SetRendererActorSelectedDelegate(int32 RendererId, FDisplayClusterMeshProjectionRenderer::FSelection ActorSelectedDelegate) = 0;

	/**
	 * Set the RenderSimpleElementsDelegate of a renderer.
	 * See FDisplayClusterMeshProjectionRenderer::RenderSimpleElementsDelegate for more info.
	 * 
	 * @param RendererId The ID of the renderer as returned from CreateRenderer.
	 * @param RenderSimpleElementsDelegate The delegate to set on the renderer.
	 */
	virtual bool SetRendererRenderSimpleElementsDelegate(int32 RendererId, FDisplayClusterMeshProjectionRenderer::FSimpleElementPass RenderSimpleElementsDelegate) = 0;

	/**
	 * Immediately render a preview.
	 * 
	 * @param RendererId The ID of the renderer as returned from CreateRenderer.
	 * @param RenderSettings Settings controlling how the scene will be rendered.
	 * @param Canvas The canvas to render to.
	 */
	virtual bool Render(int32 RendererId, FDisplayClusterMeshProjectionRenderSettings& RenderSettings, FCanvas& Canvas) = 0;

	/**
	 * Queue a preview to be rendered in the future, automatically creating a canvas and render target.
	 *
	 * @param RendererId The ID of the renderer as returned from CreateRenderer.
	 * @param RenderSettings Settings controlling how the scene will be rendered.
	 * @param Size The size of the image to produce. Note that whenever this changes for a given renderer, the underlying RenderTarget will be resized, which has a performance cost.
	 * @param ResultDelegate The delegate to call when the render is complete. It will be passed a FRenderTarget containing the rendered preview, or null if the render failed.
	 */
	virtual bool RenderQueued(int32 RendererId, FDisplayClusterMeshProjectionRenderSettings& RenderSettings, const FIntPoint& Size, FRenderResultDelegate ResultDelegate) = 0;

	/**
	 * Queue a preview to be rendered in the future using an existing canvas. The canvas must have a valid render target assigned.
	 *
	 * @param RendererId The ID of the renderer as returned from CreateRenderer.
	 * @param RenderSettings Settings controlling how the scene will be rendered.
	 * @param Canvas The canvas to draw to. If this is invalid when the render is ready to start, the render will be skipped.
	 * @param ResultDelegate The delegate to call when the render is complete. It will be passed a FRenderTarget containing the rendered preview, or null if the render failed.
	 */
	virtual bool RenderQueued(int32 RendererId, FDisplayClusterMeshProjectionRenderSettings& RenderSettings, TWeakPtr<FCanvas> Canvas, FRenderResultDelegate ResultDelegate) = 0;

	/**
	 * Check whether nDisplay preview textures are being updated in real time.
	 */
	virtual bool IsRealTimePreviewEnabled() const = 0;

	/**
	 * Set the path of the ADisplayClusterRootActor actor for a renderer, which will be added to the preview scene and used to determine the render world.
	 * Whenever a render is performed with this renderer, it will try to find the actor at that path and use it as the root.
	 * 
	 * @param RendererId The ID of the renderer as returned from CreateRenderer.
	 * @param ActorPath The path of the ADisplayClusterRootActor actor to preview.
	 * @param bAutoUpdateLightcards If true, the renderer scene will also be automatically populated with the actor's associated lightcards, including future changes to them.
	 */
	UE_DEPRECATED(5.5, "This function has been deprecated. Use EDisplayClusterScenePreviewFlags.")
	virtual bool SetRendererRootActorPath(int32 RendererId, const FString& ActorPath, bool bAutoUpdateLightcards = false)
	{
		return false;
	};

	/**
	 * Set the root DisplayCluster actor for the renderer with the given ID, which will be added to the preview scene and used to determine the render world.
	 * Note that if the actor is destroyed (including by a recompile, reload, etc.), you will need to call this again to update the actor.
	 * This will also clear the path set by SetRendererRootActorPath.
	 * 
	 * @param RendererId The ID of the renderer as returned from CreateRenderer.
	 * @param Actor The actor to preview.
	 * @param bAutoUpdateLightcards If true, the renderer scene will also be automatically populated with the actor's associated lightcards, including future changes to them.
	 */
	UE_DEPRECATED(5.5, "This function has been deprecated.")
	virtual bool SetRendererRootActor(int32 RendererId, ADisplayClusterRootActor* Actor, bool bAutoUpdateLightcards = false)
	{
		return false;
	};

	/**
	 * Set whether the renderer should use post-processed nDisplay preview textures.
	 *
	 * @param RendererId The ID of the renderer as returned from CreateRenderer.
	 * @param bUsePostProcessTexture Whether to use post-processed nDisplay preview texture for future renders.
	 */
	UE_DEPRECATED(5.5, "This function has been deprecated.")
	virtual bool SetRendererUsePostProcessTexture(int32 RendererId, bool bUsePostProcessTexture)
	{
		return false;
	};

};