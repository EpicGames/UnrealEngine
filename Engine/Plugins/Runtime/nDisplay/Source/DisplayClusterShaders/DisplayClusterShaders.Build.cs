// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class DisplayClusterShaders : ModuleRules
{
	public DisplayClusterShaders(ReadOnlyTargetRules ROTargetRules) : base(ROTargetRules)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"Projects",
				"RenderCore",
			});

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"DisplayCluster",
				"DisplayClusterWarp",
				"RHI",
				"Renderer"
			});

		if (Target.bBuildEditor == true)
		{
			PrivateDependencyModuleNames.Add("UnrealEd");
		}
	}
}
