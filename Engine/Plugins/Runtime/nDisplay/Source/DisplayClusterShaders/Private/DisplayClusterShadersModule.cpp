// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterShadersModule.h"

#include "ShaderCore.h"

#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"

#include "Shaders/DisplayClusterShadersGenerateMips.h"
#include "Shaders/DisplayClusterShadersMedia.h"
#include "Shaders/DisplayClusterShadersOverlay.h"
#include "Shaders/DisplayClusterShadersTransformTexture.h"
#include "Shaders/DisplayClusterShadersPreprocess_UVLightCards.h"
#include "Shaders/DisplayClusterShadersPostprocess_Blur.h"
#include "Shaders/DisplayClusterShadersPostprocess_OutputRemap.h"
#include "Shaders/DisplayClusterShadersWarpblend_ICVFX.h"
#include "Shaders/DisplayClusterShadersWarpblend_MPCDI.h"
#include "DisplayClusterShadersTextureUtils.h"

//////////////////////////////////////////////////////////////////////////////////////////////
// IModuleInterface
//////////////////////////////////////////////////////////////////////////////////////////////

#define NDISPLAY_SHADERS_MAP TEXT("/Plugin/nDisplay")

void FDisplayClusterShadersModule::StartupModule()
{
	if (!AllShaderSourceDirectoryMappings().Contains(NDISPLAY_SHADERS_MAP))
	{
		FString PluginShaderDir = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("nDisplay"))->GetBaseDir(), TEXT("Shaders"));
		AddShaderSourceDirectoryMapping(NDISPLAY_SHADERS_MAP, PluginShaderDir);
	}
}

void FDisplayClusterShadersModule::ShutdownModule()
{
}

bool FDisplayClusterShadersModule::RenderWarpBlend_MPCDI(FRHICommandListImmediate& RHICmdList, const FDisplayClusterShaderParameters_WarpBlend& InWarpBlendParameters) const
{
	return FDisplayClusterShadersWarpblend_MPCDI::RenderWarpBlend_MPCDI(RHICmdList, InWarpBlendParameters);
}

bool FDisplayClusterShadersModule::RenderWarpBlend_ICVFX(FRHICommandListImmediate& RHICmdList, const FDisplayClusterShaderParameters_WarpBlend& InWarpBlendParameters, const FDisplayClusterShaderParameters_ICVFX& InICVFXParameters) const
{
	return FDisplayClusterShadersWarpblend_ICVFX::RenderWarpBlend_ICVFX(RHICmdList, InWarpBlendParameters, InICVFXParameters);
}

bool FDisplayClusterShadersModule::RenderPreprocess_UVLightCards(FRHICommandListImmediate& RHICmdList, FSceneInterface* InScene, FRenderTarget* InRenderTarget, const FDisplayClusterShaderParameters_UVLightCards& InParameters) const
{
	return FDisplayClusterShadersPreprocess_UVLightCards::RenderPreprocess_UVLightCards(RHICmdList, InScene, InRenderTarget, InParameters);
}

bool FDisplayClusterShadersModule::RenderPostprocess_OutputRemap(FRHICommandListImmediate& RHICmdList, FRHITexture* InSourceTexture, FRHITexture* InRenderTargetableDestTexture, const IDisplayClusterRender_MeshComponentProxy& MeshProxy) const
{
	return FDisplayClusterShadersPostprocess_OutputRemap::RenderPostprocess_OutputRemap(RHICmdList, InSourceTexture, InRenderTargetableDestTexture, MeshProxy);
}

bool FDisplayClusterShadersModule::RenderPostprocess_Blur(FRHICommandListImmediate& RHICmdList, FRHITexture* InSourceTexture, FRHITexture* InRenderTargetableDestTexture, const FDisplayClusterShaderParameters_PostprocessBlur& InSettings) const
{
	return FDisplayClusterShadersPostprocess_Blur::RenderPostprocess_Blur(RHICmdList, InSourceTexture, InRenderTargetableDestTexture, InSettings);
}

bool FDisplayClusterShadersModule::GenerateMips(FRHICommandListImmediate& RHICmdList, FRHITexture* InOutMipsTexture, const FDisplayClusterShaderParameters_GenerateMips& InSettings) const
{
	return FDisplayClusterShadersGenerateMips::GenerateMips(RHICmdList, InOutMipsTexture, InSettings);
}

void FDisplayClusterShadersModule::AddLinearToPQPass(FRDGBuilder& GraphBuilder, const FDisplayClusterShaderParameters_MediaPQ& Parameters) const
{
	FDisplayClusterShadersMedia::AddLinearToPQPass(GraphBuilder, Parameters);
}

void FDisplayClusterShadersModule::AddPQToLinearPass(FRDGBuilder& GraphBuilder, const FDisplayClusterShaderParameters_MediaPQ& Parameters) const
{
	FDisplayClusterShadersMedia::AddPQToLinearPass(GraphBuilder, Parameters);
}

void FDisplayClusterShadersModule::AddDrawOverlayPass(FRDGBuilder& GraphBuilder, const FDisplayClusterShaderParameters_Overlay& Parameters) const
{
	FDisplayClusterShadersOverlay::AddOverlayBlendingPass(GraphBuilder, Parameters);
}

void FDisplayClusterShadersModule::AddTransformTexturePass(FRDGBuilder& GraphBuilder, const FDisplayClusterShaderParameters_TransformTexture& Parameters) const
{
	FDisplayClusterShadersTransformTexture::AddTransformTexturePass(GraphBuilder, Parameters);
}

TSharedRef<IDisplayClusterShadersTextureUtils> FDisplayClusterShadersModule::CreateTextureUtils_RenderThread(FRHICommandListImmediate& RHICmdList) const
{
	return FDisplayClusterShadersTextureUtils::CreateTextureUtils_RenderThread(RHICmdList);
}

TSharedRef<IDisplayClusterShadersTextureUtils> FDisplayClusterShadersModule::CreateTextureUtils_RenderThread(FRDGBuilder& GraphBuilder) const
{
	return FDisplayClusterShadersTextureUtils::CreateTextureUtils_RenderThread(GraphBuilder);
}

IMPLEMENT_MODULE(FDisplayClusterShadersModule, DisplayClusterShaders);
