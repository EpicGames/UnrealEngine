// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDisplayClusterShaders.h"

class FDisplayClusterShadersModule
	: public IDisplayClusterShaders
{
public:
	//////////////////////////////////////////////////////////////////////////////////////////////
	// IModuleInterface
	//////////////////////////////////////////////////////////////////////////////////////////////
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

public:
	//~Begin IDisplayClusterShaders
	virtual bool RenderWarpBlend_MPCDI(FRHICommandListImmediate& RHICmdList, const FDisplayClusterShaderParameters_WarpBlend& InWarpBlendParameters) const override;
	virtual bool RenderWarpBlend_ICVFX(FRHICommandListImmediate& RHICmdList, const FDisplayClusterShaderParameters_WarpBlend& InWarpBlendParameters, const FDisplayClusterShaderParameters_ICVFX& InICVFXParameters) const override;
	virtual bool RenderPreprocess_UVLightCards(FRHICommandListImmediate& RHICmdList, FSceneInterface* InScene, FRenderTarget* InRenderTarget, float ProjectionPlaneSize, bool bRenderFinalColor) const override { return false; }
	virtual bool RenderPreprocess_UVLightCards(FRHICommandListImmediate& RHICmdList, FSceneInterface* InScene, FRenderTarget* InRenderTarget, const FDisplayClusterShaderParameters_UVLightCards& InParameters) const override;
	virtual bool RenderPostprocess_OutputRemap(FRHICommandListImmediate& RHICmdList, FRHITexture* InSourceTexture, FRHITexture* InRenderTargetableDestTexture, const IDisplayClusterRender_MeshComponentProxy& MeshProxy) const override;
	virtual bool RenderPostprocess_Blur(FRHICommandListImmediate& RHICmdList, FRHITexture* InSourceTexture, FRHITexture* InRenderTargetableDestTexture, const FDisplayClusterShaderParameters_PostprocessBlur& InSettings) const override;
	virtual bool GenerateMips(FRHICommandListImmediate& RHICmdList, FRHITexture* InOutMipsTexture, const FDisplayClusterShaderParameters_GenerateMips& InSettings) const override;
	virtual void AddLinearToPQPass(FRDGBuilder& GraphBuilder, const FDisplayClusterShaderParameters_MediaPQ& Parameters) const override;
	virtual void AddPQToLinearPass(FRDGBuilder& GraphBuilder, const FDisplayClusterShaderParameters_MediaPQ& Parameters) const override;
	virtual void AddDrawOverlayPass(FRDGBuilder& GraphBuilder, const FDisplayClusterShaderParameters_Overlay& Parameters) const override;
	virtual void AddTransformTexturePass(FRDGBuilder& GraphBuilder, const FDisplayClusterShaderParameters_TransformTexture& Parameters) const override;

	virtual TSharedRef<IDisplayClusterShadersTextureUtils> CreateTextureUtils_RenderThread(FRHICommandListImmediate& RHICmdList) const override;
	virtual TSharedRef<IDisplayClusterShadersTextureUtils> CreateTextureUtils_RenderThread(FRDGBuilder& GraphBuilder) const override;
	//~~End IDisplayClusterShaders
};
