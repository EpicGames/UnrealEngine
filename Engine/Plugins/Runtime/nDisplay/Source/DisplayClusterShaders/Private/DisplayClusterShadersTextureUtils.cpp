// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterShadersTextureUtils.h"
#include "Render/Viewport/IDisplayClusterViewportProxy.h"
#include "Shaders/DisplayClusterShadersCopyTexture.h"

#include "RenderResource.h"
#include "RenderGraphUtils.h"
#include "RenderGraphResources.h"

#include "RenderTargetPool.h"

namespace UE::DisplayClusterShaders::Private
{
	/** Return resource name. */
	static constexpr auto GetDisplayClusterViewportResourceTypeName(const EDisplayClusterViewportResourceType InResourceType)
	{
		switch (InResourceType)
		{
		case EDisplayClusterViewportResourceType::InternalRenderTargetEntireRectResource : return TEXT("nDisplay.InternalRenderTargetEntireRectResource");
		case EDisplayClusterViewportResourceType::InternalRenderTargetResource           : return TEXT("nDisplay.InternalRenderTargetResource");

		case EDisplayClusterViewportResourceType::InputShaderResource:              return TEXT("nDisplay.InputShaderResource");
		case EDisplayClusterViewportResourceType::MipsShaderResource:               return TEXT("nDisplay.MipsShaderResource");
		case EDisplayClusterViewportResourceType::AdditionalTargetableResource:     return TEXT("nDisplay.AdditionalTargetableResource");

		case EDisplayClusterViewportResourceType::BeforeWarpBlendTargetableResource:return TEXT("nDisplay.BeforeWarpBlendTargetableResource");
		case EDisplayClusterViewportResourceType::AfterWarpBlendTargetableResource: return TEXT("nDisplay.AfterWarpBlendTargetableResource");

		case EDisplayClusterViewportResourceType::OutputTargetableResource:         return TEXT("nDisplay.OutputTargetableResource");
		case EDisplayClusterViewportResourceType::OutputPreviewTargetableResource:  return TEXT("nDisplay.OutputPreviewTargetableResource");

		case EDisplayClusterViewportResourceType::OutputFrameTargetableResource:      return TEXT("nDisplay.OutputFrameTargetableResource");
		case EDisplayClusterViewportResourceType::AdditionalFrameTargetableResource:  return TEXT("nDisplay.AdditionalFrameTargetableResource");
		default:
			break;
		}

		return TEXT("");
	}

	/** Return pixel format. */
	static inline EPixelFormat GetPixelFormat(const FDisplayClusterShadersTextureViewport& In)
	{
		if (In.TextureRHI)
		{
			return In.TextureRHI->GetDesc().Format;
		}
		else if (In.TextureRDG)
		{
			return In.TextureRDG->Desc.Format;
		}

		return EPixelFormat::PF_Unknown;
	}

	/** Return size of the referenced texture. */
	static inline FIntPoint GetTextureSize(const FDisplayClusterShadersTextureViewport& In)
	{
		if (In.TextureRHI)
		{
			return In.TextureRHI->GetDesc().Extent;
		}
		else if (In.TextureRDG)
		{
			return In.TextureRDG->Desc.Extent;
		}

		return FIntPoint::ZeroValue;
	}

	/**
	 * Create RHI texture viewport resource 
	 * 
	 * @param RHICmdList               - API
 	 * @param OutTextureViewport       - (out) The new texture
	 * @param InTextureViewport        - (in) the source texture to be clone (size and format)
	 * @param InOutPooledRenderTargets - (in, out) save all temporary textures here so they exist for the lifetime of this external variable
	 * @param InDebugName              - (in) the unique debug name for the new resource
	 */
	static inline bool CloneTextureViewportResource(
		FRHICommandListImmediate& RHICmdList,
		FDisplayClusterShadersTextureViewport& OutTextureViewport,
		const FDisplayClusterShadersTextureViewport& InTextureViewport,
		TArray<TRefCountPtr<IPooledRenderTarget>>& InOutPooledRenderTargets,
		const TCHAR* InDebugName)
	{
		const FIntPoint Size = GetTextureSize(InTextureViewport);
		const EPixelFormat Format = GetPixelFormat(InTextureViewport);

		if (Size.GetMin() > 0 && Format != EPixelFormat::PF_Unknown)
		{
			// Create a temporary pool texture
			const FPooledRenderTargetDesc NewResourceDesc = FPooledRenderTargetDesc::Create2DDesc(
				Size, Format, FClearValueBinding::None,
				TexCreate_None, TexCreate_RenderTargetable | TexCreate_ShaderResource, false);

			TRefCountPtr<IPooledRenderTarget> RenderTargetPoolResource;
			GRenderTargetPool.FindFreeElement(RHICmdList, NewResourceDesc, RenderTargetPoolResource, InDebugName);

			if (RenderTargetPoolResource.IsValid())
			{
				if (FRHITexture* RHITexture = RenderTargetPoolResource->GetRHI())
				{
					// Maintains an internal link to this resource.
					// It will be released late, after the TS has completed all operations on that resource.
					InOutPooledRenderTargets.Add(RenderTargetPoolResource);

					OutTextureViewport = FDisplayClusterShadersTextureViewport(RHITexture, FIntRect(FIntPoint::ZeroValue, Size));

					return true;
				}
			}
		}

		return false;
	}

	/**
	 * Initialize texture viewport for RHI
	 *
	 * @param RHICmdList         - API
	 * @param InTextureViewport  - Texture for cloning
	 */
	static inline bool InitializeTextureViewport(
		FRHICommandListImmediate& RHICmdList,
		FDisplayClusterShadersTextureViewport& InOutTextureViewport)
	{
		if (InOutTextureViewport.TextureRDG && HasBeenProduced(InOutTextureViewport.TextureRDG))
		{
			if (FRHITexture* TextureRHI = InOutTextureViewport.TextureRDG->GetRHI())
			{
				InOutTextureViewport.TextureRHI = TextureRHI;
				InOutTextureViewport.TextureRDG = nullptr;

				return true;
			}
		}
		else if (InOutTextureViewport.TextureRHI)
		{
			return true;
		}

		return false;
	}

	/**
	 * Create RDG texture viewport resource
	 *
	 * @param GraphBuilder       - RDG API
	 * @param OutTextureViewport - (out) The new texture
	 * @param InTextureViewport  - (in) the source texture to be clone (size and format)
	 * @param InDebugName        - (in) the unique debug name for the new resource
	 */
	static inline bool CloneTextureViewportResource(
		FRDGBuilder& GraphBuilder,
		FDisplayClusterShadersTextureViewport& OutTextureViewport,
		const FDisplayClusterShadersTextureViewport& InTextureViewport,
		const TCHAR* InDebugName)
	{
		const FIntPoint Size = GetTextureSize(InTextureViewport);
		const EPixelFormat Format = GetPixelFormat(InTextureViewport);

		if (Size.GetMin() > 0 && Format != EPixelFormat::PF_Unknown)
		{
			// Use temporary RTT
			const FRDGTextureDesc TemporaryTextureDesc = FRDGTextureDesc::Create2D(
				Size, Format, FClearValueBinding::None,
				TexCreate_RenderTargetable | TexCreate_ShaderResource);

			if (FRDGTextureRef RDGTexture = GraphBuilder.CreateTexture(TemporaryTextureDesc, InDebugName))
			{
				OutTextureViewport = FDisplayClusterShadersTextureViewport(RDGTexture, FIntRect(FIntPoint::ZeroValue, Size));

				return true;
			}
		}

		return false;
	}

	/**
	 * Initialize texture viewport for RDG
	 *
	 * @param GraphBuilder       - API
	 * @param InTextureViewport  - Texture for cloning
	 * @param InDebugName        - the unique debug name for the new resource
	 */
	static inline bool InitializeTextureViewport(
		FRDGBuilder& GraphBuilder,
		FDisplayClusterShadersTextureViewport& InOutTextureViewport,
		const TCHAR* InDebugName)
	{
		if (InOutTextureViewport.TextureRHI)
		{
			const TCHAR* DebugName = (InOutTextureViewport.DebugName && InOutTextureViewport.DebugName[0])
				? InOutTextureViewport.DebugName : InDebugName;

			InOutTextureViewport.bExternalTextureRDG = true;

			InOutTextureViewport.TextureRDG = RegisterExternalTexture(GraphBuilder, InOutTextureViewport.TextureRHI->GetTexture2D(), DebugName);
			InOutTextureViewport.TextureRHI = nullptr;

			return true;
		}
		else if (InOutTextureViewport.bExternalTextureRDG || HasBeenProduced(InOutTextureViewport.TextureRDG))
		{
			return true;
		}

		return false;
	}

	/**
	* Check if resources with the specified regions can be resolved.
	* If any rect exceeds the texture size, RHI will crash.
	* This function adjusts the rects to the size of the textures.
	*
	* @param InOutSource      - (in, out) Input texture with rect
	* @param InOutDestination - (inm out) Output texture with rect
	* @param Settings         - (in) current settings
	*
	* @return false, if it isn't possible
	*/
	static inline bool UpdateResourcesRectsForResolve(
		FDisplayClusterShadersTextureViewport& InOutSource,
		FDisplayClusterShadersTextureViewport& InOutDestination,
		const FDisplayClusterShadersTextureUtilsSettings& Settings)
	{
		using namespace UE::DisplayClusterShaders::Private;

		const FIntPoint InputTextureSize = GetTextureSize(InOutSource);
		const FIntPoint OutputTextureSize = GetTextureSize(InOutDestination);

		// One of the texture dimensions is zero
		if (InputTextureSize.GetMin() <= 0 || OutputTextureSize.GetMin() <= 0)
		{
			return false;
		}

		if (InOutSource.Rect.IsEmpty())
		{
			InOutSource.Rect = FIntRect(FIntPoint(0, 0), InputTextureSize);
		}
		if (InOutDestination.Rect.IsEmpty())
		{
			InOutDestination.Rect = FIntRect(FIntPoint(0, 0), OutputTextureSize);
		}

		if (Settings.HasAnyFlags(EDisplayClusterShaderTextureUtilsFlags::DisableUpdateResourcesRectsForResolve))
		{
			return true;
		}

		FIntRect InputRect(InOutSource.Rect);
		FIntRect OutputRect(InOutDestination.Rect);

		// if InputRect.Min<0, also adjust the OutputRect.Min
		OutputRect.Min += FIntPoint(
			FMath::Max(0, -InOutSource.Rect.Min.X),
			FMath::Max(0, -InOutSource.Rect.Min.Y));

		// if OutputRect.Min<0, also adjust the InputRect.Min
		InputRect.Min += FIntPoint(
			FMath::Max(0, -InOutDestination.Rect.Min.X),
			FMath::Max(0, -InOutDestination.Rect.Min.Y));

		// If InputRect or OutputRect exceeds the texture size, RHI will crash. Let's adjust it to the texture size.
		{
			InputRect.Min.X = FMath::Clamp(InputRect.Min.X, 0, InputTextureSize.X);
			InputRect.Min.Y = FMath::Clamp(InputRect.Min.Y, 0, InputTextureSize.Y);

			InputRect.Max.X = FMath::Clamp(InputRect.Max.X, 0, InputTextureSize.X);
			InputRect.Max.Y = FMath::Clamp(InputRect.Max.Y, 0, InputTextureSize.Y);

			OutputRect.Min.X = FMath::Clamp(OutputRect.Min.X, 0, OutputTextureSize.X);
			OutputRect.Min.Y = FMath::Clamp(OutputRect.Min.Y, 0, OutputTextureSize.Y);

			OutputRect.Max.X = FMath::Clamp(OutputRect.Max.X, 0, OutputTextureSize.X);
			OutputRect.Max.Y = FMath::Clamp(OutputRect.Max.Y, 0, OutputTextureSize.Y);
		}

		// InputRect.Min and OutputRect.Min always > 0

		// Check the SrcRect and DestRect
		if (InputRect.Size().GetMin() <= 0
			|| OutputRect.Size().GetMin() <= 0)
		{
			// The SrcRect or DestRect is invalid.
			return false;
		}

		// Input and output rects will be cropped to match each other.
		if (EnumHasAnyFlags(Settings.Flags, EDisplayClusterShaderTextureUtilsFlags::DisableResize))
		{
			const FIntPoint MinSize(
				FMath::Min(InputRect.Size().X, OutputRect.Size().X),
				FMath::Min(InputRect.Size().Y, OutputRect.Size().Y));

			// Crop the rects to the smallest size
			InputRect.Max = InputRect.Min + MinSize;
			OutputRect.Max = OutputRect.Min + MinSize;
		}

		// Can be resolved
		InOutSource.Rect = InputRect;
		InOutDestination.Rect = OutputRect;

		return true;
	}
};

/**
* FDisplayClusterShadersTextureUtils
*/
TSharedRef<IDisplayClusterShadersTextureUtils> FDisplayClusterShadersTextureUtils::CreateTextureUtils_RenderThread(FRHICommandListImmediate& RHICmdList)
{
	check(IsInRenderingThread());

	return MakeShared<FDisplayClusterShadersRHITextureUtils>(RHICmdList);
}

TSharedRef<IDisplayClusterShadersTextureUtils> FDisplayClusterShadersTextureUtils::CreateTextureUtils_RenderThread(FRDGBuilder& GraphBuilder)
{
	check(IsInRenderingThread());

	return MakeShared<FDisplayClusterShadersRDGTextureUtils>(GraphBuilder);
}

TSharedRef<IDisplayClusterShadersTextureUtils> FDisplayClusterShadersTextureUtils::ResolveTextureContext(const FDisplayClusterShadersTextureViewportContext& Input, const FDisplayClusterShadersTextureViewportContext& Output)
{
	return ResolveTextureContext(FDisplayClusterShadersTextureUtilsSettings(), Input, Output);
}

TSharedRef<IDisplayClusterShadersTextureUtils> FDisplayClusterShadersTextureUtils::Resolve()
{
	return Resolve(FDisplayClusterShadersTextureUtilsSettings());
}

bool FDisplayClusterShadersTextureUtils::ImplementTextureContextResolve(
	const FDisplayClusterShadersTextureViewportContext& Input,
	const FDisplayClusterShadersTextureViewportContext& Output,
	const FDisplayClusterShadersTextureUtilsSettings& Settings)
{
	// Don't use a shader if possible.
	if (!ShouldUseResampleShader(Input, Output, Settings))
	{
		return TransitionAndCopyTexture(Input, Output, Settings);
	}
	else if (Settings.HasAnyFlags(EDisplayClusterShaderTextureUtilsFlags::DisableResampleShader))
	{
		// A resampling shader needs to be used, but it is disabled by the user.
		return false;
	}

	// Custom implementations must perform the copying from the output texture to the input texture themselves.
	if (Settings.HasAnyFlags(EDisplayClusterShaderTextureUtilsFlags::UseOutputTextureAsInput))
	{
		if (!TransitionAndCopyTexture(Output, Input, Settings))
		{
			return false;
		}
	}

	return ResampleColorEncodingCopyRect(Input, Output, Settings);
}

TSharedRef<IDisplayClusterShadersTextureUtils> FDisplayClusterShadersTextureUtils::Resolve(const FDisplayClusterShadersTextureUtilsSettings& Settings)
{
	return ForEachContextByPredicate([&](
		const FDisplayClusterShadersTextureViewportContext& Input,
		const FDisplayClusterShadersTextureViewportContext& Output)
		{
			ImplementTextureContextResolve(Input, Output, Settings);
		});
}

TSharedRef<IDisplayClusterShadersTextureUtils> FDisplayClusterShadersTextureUtils::ResolveTextureContext(const FDisplayClusterShadersTextureUtilsSettings& Settings, const FDisplayClusterShadersTextureViewportContext& InputContext, const FDisplayClusterShadersTextureViewportContext& OutputContext)
{
	using namespace UE::DisplayClusterShaders::Private;

	FDisplayClusterShadersTextureViewportContext Input(InputContext);
	FDisplayClusterShadersTextureViewportContext Output(OutputContext);

	if (InitializeTextureViewportForRenderPass(Output, TEXT("nDisplay.Output")))
	{
		const bool bInputInitialized = Settings.HasAnyFlags(EDisplayClusterShaderTextureUtilsFlags::UseOutputTextureAsInput)
			? CloneTextureViewportResourceForRenderPass(Input, Output, TEXT("nDisplay.OutputClone"))
			: InitializeTextureViewportForRenderPass(Input, TEXT("nDisplay.Input"));
		if (bInputInitialized && UpdateResourcesRectsForResolve(Input, Output, Settings))
		{
			ImplementTextureContextResolve(Input, Output, Settings);
		}
	}

	return SharedThis(this);
}

FDisplayClusterShadersTextureParameters FDisplayClusterShadersTextureUtils::GetTextureParametersFromViewport(
	const IDisplayClusterViewportProxy* InViewportProxy,
	const EDisplayClusterViewportResourceType InResourceType)
{
	using namespace UE::DisplayClusterShaders::Private;

	FDisplayClusterShadersTextureParameters OutTextureParameters;

	TArray<FRHITexture*> Textures;
	TArray<FIntRect> TextureRects;
	if (InViewportProxy && InViewportProxy->GetResourcesWithRects_RenderThread(InResourceType, Textures, TextureRects)
		&& Textures.Num() == TextureRects.Num())
	{
		// Get resource color encoding
		OutTextureParameters.ColorEncoding = InViewportProxy->GetResourceColorEncoding_RenderThread(InResourceType);

		// Get all contexts
		for (int32 ContextNum = 0; ContextNum < Textures.Num(); ContextNum++)
		{
			if (Textures[ContextNum])
			{
				OutTextureParameters.TextureViewports.Emplace(ContextNum,
					FDisplayClusterShadersTextureViewport(
						Textures[ContextNum],
						TextureRects[ContextNum],
						GetDisplayClusterViewportResourceTypeName(InResourceType)));
			}
		}
	}

	return OutTextureParameters;
}

TSharedRef<IDisplayClusterShadersTextureUtils> FDisplayClusterShadersTextureUtils::SetInputEncoding(const FDisplayClusterColorEncoding& InColorEncoding)
{
	InputTextureParameters.ColorEncoding = InColorEncoding;

	return SharedThis(this);
}
TSharedRef<IDisplayClusterShadersTextureUtils> FDisplayClusterShadersTextureUtils::SetOutputEncoding(const FDisplayClusterColorEncoding& InColorEncoding)
{
	OutputTextureParameters.ColorEncoding = InColorEncoding;

	return SharedThis(this);
}


TSharedRef<IDisplayClusterShadersTextureUtils> FDisplayClusterShadersTextureUtils::SetInput(
	const IDisplayClusterViewportProxy* InViewportProxy,
	const EDisplayClusterViewportResourceType InResourceType,
	const int32 InContextNum)
{
	const FDisplayClusterShadersTextureParameters NewTextureParameters = GetTextureParametersFromViewport(InViewportProxy, InResourceType);

	if (InContextNum == INDEX_NONE)
	{
		// Override entire input parameters.
		SetInputEncoding(NewTextureParameters.ColorEncoding);
		for (const TPair<uint32, FDisplayClusterShadersTextureViewport>& TextureIt : NewTextureParameters.TextureViewports)
		{
			SetInput(TextureIt.Value, TextureIt.Key);
		}
	}
	else if(NewTextureParameters.TextureViewports.Contains(InContextNum))
	{
		SetInputEncoding(NewTextureParameters.ColorEncoding);
		SetInput(NewTextureParameters.TextureViewports[InContextNum], InContextNum);
	}


	return SharedThis(this);
}

TSharedRef<IDisplayClusterShadersTextureUtils> FDisplayClusterShadersTextureUtils::SetOutput(
	const IDisplayClusterViewportProxy* InViewportProxy,
	const EDisplayClusterViewportResourceType InResourceType,
	const int32 InContextNum)
{
	const FDisplayClusterShadersTextureParameters NewTextureParameters = GetTextureParametersFromViewport(InViewportProxy, InResourceType);

	if (InContextNum == INDEX_NONE)
	{
		// Override entire input parameters.
		SetOutputEncoding(NewTextureParameters.ColorEncoding);
		for (const TPair<uint32, FDisplayClusterShadersTextureViewport>& TextureIt : NewTextureParameters.TextureViewports)
		{
			SetOutput(TextureIt.Value, TextureIt.Key);
		}
	}
	else if (NewTextureParameters.TextureViewports.Contains(InContextNum))
	{
		SetOutputEncoding(NewTextureParameters.ColorEncoding);
		SetOutput(NewTextureParameters.TextureViewports[InContextNum], InContextNum);
	}

	return SharedThis(this);
}

TSharedRef<IDisplayClusterShadersTextureUtils> FDisplayClusterShadersTextureUtils::SetInput(const FDisplayClusterShadersTextureViewport& InTextureViewport, const int32 InContextNum)
{
	const uint32 ContextNum = InContextNum > 0 ? InContextNum : 0;
	InputTextureParameters.TextureViewports.Emplace(ContextNum, InTextureViewport);

	return SharedThis(this);
}

TSharedRef<IDisplayClusterShadersTextureUtils> FDisplayClusterShadersTextureUtils::SetOutput(const FDisplayClusterShadersTextureViewport& InTextureViewport, const int32 InContextNum)
{
	const uint32 ContextNum = InContextNum > 0 ? InContextNum : 0;
	OutputTextureParameters.TextureViewports.Emplace(ContextNum, InTextureViewport);

	return SharedThis(this);
}

/** Iterate through contexts matching these settings. */
void FDisplayClusterShadersTextureUtils::ImplForEachContextByPredicate(
	const FDisplayClusterShadersTextureUtilsSettings& Settings,
	TFunctionDisplayClusterShaders_TextureContextIterator&& TextureContextIteratorFunc)
{
	using namespace UE::DisplayClusterShaders::Private;
	// The input is ignored, and temporary textures is used for every output texture
	if (Settings.HasAnyFlags(EDisplayClusterShaderTextureUtilsFlags::UseOutputTextureAsInput))
	{
		for (const TPair<uint32, FDisplayClusterShadersTextureViewport>& OutputTextureIt : OutputTextureParameters.TextureViewports)
		{
			const uint32 OutputContextNum = OutputTextureIt.Key;
			FDisplayClusterShadersTextureViewportContext Output(OutputTextureIt.Value, OutputTextureParameters.ColorEncoding, OutputContextNum);
			if (InitializeTextureViewportForRenderPass(Output, TEXT("nDisplay.Output")))
			{
				FDisplayClusterShadersTextureViewportContext Input;
				if (CloneTextureViewportResourceForRenderPass(Input, Output, TEXT("nDisplay.OutputClone")))
				{
					// Use input color encoding
					Input.ColorEncoding = InputTextureParameters.ColorEncoding;
					Input.ContextNum = OutputContextNum;

					if (InitializeTextureViewportForRenderPass(Input, TEXT("nDisplay.Input")))
					{
						if (UpdateResourcesRectsForResolve(Input, Output, Settings))
						{
							// Copy Output to Input (temp)
							TransitionAndCopyTexture(Output, Input, Settings);

							// Continue the process using the external functor.
							if (Settings.HasAnyFlags(EDisplayClusterShaderTextureUtilsFlags::InvertDirection))
							{
								TextureContextIteratorFunc(Output, Input);
							}
							else
							{
								TextureContextIteratorFunc(Input, Output);
							}
						}
					}
				}
			}
		}
		return;
	}

	// Map input to output:
	for (const TPair<uint32, FDisplayClusterShadersTextureViewport>& OutputTextureIt : OutputTextureParameters.TextureViewports)
	{
		// Monoscopic input can be copied to stereoscopic output
		const uint32 OutputContextNum = OutputTextureIt.Key;
		const uint32 InputContextNum = InputTextureParameters.TextureViewports.Contains(OutputContextNum) ? OutputContextNum : 0;

		if (const FDisplayClusterShadersTextureViewport* InputTexturePtr = InputTextureParameters.TextureViewports.Find(InputContextNum))
		{
			FDisplayClusterShadersTextureViewportContext Output(OutputTextureIt.Value, OutputTextureParameters.ColorEncoding, OutputContextNum);
			FDisplayClusterShadersTextureViewportContext Input(*InputTexturePtr, InputTextureParameters.ColorEncoding, InputContextNum);

			if (UpdateResourcesRectsForResolve(Input, Output, Settings)
				&& InitializeTextureViewportForRenderPass(Input, TEXT("nDisplay.Input"))
				&& InitializeTextureViewportForRenderPass(Output, TEXT("nDisplay.Output")))
			{
				// Continue the process using the external functor.
				if (Settings.HasAnyFlags(EDisplayClusterShaderTextureUtilsFlags::InvertDirection))
				{
					TextureContextIteratorFunc(Output, Input);
				}
				else
				{
					TextureContextIteratorFunc(Input, Output);
				}
			}
		}
	}
}

TSharedRef<IDisplayClusterShadersTextureUtils> FDisplayClusterShadersTextureUtils::ForEachContextByPredicate(TFunctionDisplayClusterShaders_TextureContextIterator&& InFunction)
{
	const FDisplayClusterShadersTextureUtilsSettings DefaultSettings;

	ImplForEachContextByPredicate(DefaultSettings, [&](
		const FDisplayClusterShadersTextureViewportContext& Input,
		const FDisplayClusterShadersTextureViewportContext& Output)
		{
			InFunction(Input, Output);
		});

	return SharedThis(this);
}

TSharedRef<IDisplayClusterShadersTextureUtils> FDisplayClusterShadersTextureUtils::ForEachContextByPredicate(
	const FDisplayClusterShadersTextureUtilsSettings& Settings,
	TFunctionDisplayClusterShaders_TextureContextIterator&& InFunction)
{
	ImplForEachContextByPredicate(Settings, [&](
		const FDisplayClusterShadersTextureViewportContext& Input,
		const FDisplayClusterShadersTextureViewportContext& Output)
		{
			InFunction(Input, Output);
		});

	return SharedThis(this);
}

bool FDisplayClusterShadersTextureUtils::ShouldUseResampleShader(
	const FDisplayClusterShadersTextureViewportContext& InputTextureContext,
	const FDisplayClusterShadersTextureViewportContext& OutputTextureContext,
	const FDisplayClusterShadersTextureUtilsSettings& Settings) const
{
	using namespace UE::DisplayClusterShaders::Private;

	if (Settings.HasAnyFlags(
	    EDisplayClusterShaderTextureUtilsFlags::EnableLinearAlphaFeather
	  | EDisplayClusterShaderTextureUtilsFlags::EnableSmoothAlphaFeather))
	{
		// Alpha blending and alpha feathering require a shader.
		return true;
	}

	if (Settings.ColorMask != EColorWriteMask::CW_RGBA)
	{
		// Color mask requires a shader
		return true;
	}

	if (Settings.OverrideAlpha != EDisplayClusterShaderTextureUtilsOverrideAlpha::None)
	{
		// Use shader to overide alpha channel value
		return true;
	}

	if (InputTextureContext.Rect.Size() != OutputTextureContext.Rect.Size())
	{
		// Resizing should be done using a resampling shader.
		return true;
	}

	if (GetPixelFormat(InputTextureContext) != GetPixelFormat(OutputTextureContext))
	{
		// changing format requires a shader
		return true;
	}

	if (InputTextureContext.ColorEncoding != OutputTextureContext.ColorEncoding)
	{
		// Encode color in pixel shader
		return true;
	}

	return false;
}

/**
* FDisplayClusterShadersRHITextureUtils
*/
bool FDisplayClusterShadersRHITextureUtils::TransitionAndCopyTexture(const FDisplayClusterShadersTextureViewport& Input, const FDisplayClusterShadersTextureViewport& Output, const FDisplayClusterShadersTextureUtilsSettings& Settings)
{
	if (Input.TextureRDG && Output.TextureRDG)
	{
		return FDisplayClusterShadersCopyTexture::AddPassTransitionAndCopyTexture_RenderThread(GetOrCreateRDGBuilder(), Input, Output, Settings);
	}
	else if (Input.TextureRHI && Output.TextureRHI)
	{
		return FDisplayClusterShadersCopyTexture::TransitionAndCopyTexture_RenderThread(RHICmdList, Input, Output, Settings);
	}

	return false;
}

bool FDisplayClusterShadersRHITextureUtils::ResampleColorEncodingCopyRect(const FDisplayClusterShadersTextureViewportContext& Input, const FDisplayClusterShadersTextureViewportContext& Output, const FDisplayClusterShadersTextureUtilsSettings& Settings)
{
	if (Input.TextureRDG && Output.TextureRDG)
	{
		return FDisplayClusterShadersCopyTexture::AddPassColorEncodingCopyRect_RenderThread(GetOrCreateRDGBuilder(), Input, Output, Settings);
	}
	else if (Input.TextureRHI && Output.TextureRHI)
	{
		return FDisplayClusterShadersCopyTexture::ColorEncodingCopyRect_RenderThread(RHICmdList, Input, Output, Settings);
	}

	return false;
}

bool FDisplayClusterShadersRHITextureUtils::CloneTextureViewportResourceForRenderPass(
	FDisplayClusterShadersTextureViewport& OutTextureViewport,
	const FDisplayClusterShadersTextureViewport& InTextureViewport,
	const TCHAR* InDebugName)
{
	using namespace UE::DisplayClusterShaders::Private;

	if (ShouldUseRDG() || InTextureViewport.TextureRDG)
	{
		return CloneTextureViewportResource(GetOrCreateRDGBuilder(), OutTextureViewport, InTextureViewport, InDebugName);
	}

	return CloneTextureViewportResource(RHICmdList, OutTextureViewport, InTextureViewport, PooledRenderTargets, InDebugName);
}

bool FDisplayClusterShadersRHITextureUtils::InitializeTextureViewportForRenderPass(
	FDisplayClusterShadersTextureViewport& InOutTextureViewport,
	const TCHAR* InDebugName)
{
	using namespace UE::DisplayClusterShaders::Private;

	if (ShouldUseRDG() || InOutTextureViewport.TextureRDG)
	{
		return InitializeTextureViewport(GetOrCreateRDGBuilder(), InOutTextureViewport, InDebugName);
	}

	return InitializeTextureViewport(RHICmdList, InOutTextureViewport);
}

/**
* FDisplayClusterShadersRDGTextureUtils
*/
bool FDisplayClusterShadersRDGTextureUtils::CloneTextureViewportResourceForRenderPass(
	FDisplayClusterShadersTextureViewport& OutTextureViewport,
	const FDisplayClusterShadersTextureViewport& InTextureViewport,
	const TCHAR* InDebugName)
{
	using namespace UE::DisplayClusterShaders::Private;

	return CloneTextureViewportResource(GraphBuilder, OutTextureViewport, InTextureViewport, InDebugName);
}

bool FDisplayClusterShadersRDGTextureUtils::InitializeTextureViewportForRenderPass(
	FDisplayClusterShadersTextureViewport& InOutTextureViewport,
	const TCHAR* InDebugName)
{
	using namespace UE::DisplayClusterShaders::Private;

	return InitializeTextureViewport(GraphBuilder, InOutTextureViewport, InDebugName);
}

bool FDisplayClusterShadersRDGTextureUtils::TransitionAndCopyTexture(const FDisplayClusterShadersTextureViewport& Input, const FDisplayClusterShadersTextureViewport& Output, const FDisplayClusterShadersTextureUtilsSettings& Settings)
{
	return FDisplayClusterShadersCopyTexture::AddPassTransitionAndCopyTexture_RenderThread(GraphBuilder, Input, Output, Settings);
}

bool FDisplayClusterShadersRDGTextureUtils::ResampleColorEncodingCopyRect(const FDisplayClusterShadersTextureViewportContext& Input, const FDisplayClusterShadersTextureViewportContext& Output, const FDisplayClusterShadersTextureUtilsSettings& Settings)
{
	return FDisplayClusterShadersCopyTexture::AddPassColorEncodingCopyRect_RenderThread(GraphBuilder, Input, Output, Settings);
}
