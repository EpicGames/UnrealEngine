// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "IDisplayClusterShadersTextureUtils.h"
#include "Templates/SharedPointer.h"

#include "RHICommandList.h"

/**
 *  Texture utils class for nDisplay.
 */
class FDisplayClusterShadersTextureUtils
	: public IDisplayClusterShadersTextureUtils
	, public TSharedFromThis<FDisplayClusterShadersTextureUtils>
{
public:
	virtual ~FDisplayClusterShadersTextureUtils() = default;

	/**
	* Return new instance of the resource utils.
	*
	* @param RHICmdList   - RHI API
	*/
	static TSharedRef<IDisplayClusterShadersTextureUtils> CreateTextureUtils_RenderThread(FRHICommandListImmediate& RHICmdList);

	/**
	* Return new instance of the resource utils.
	* @param GraphBuilder - RDG builder
	*/
	static TSharedRef<IDisplayClusterShadersTextureUtils> CreateTextureUtils_RenderThread(FRDGBuilder& GraphBuilder);


public:
	//~Begin IDisplayClusterShadersTextureUtils
	virtual TSharedRef<IDisplayClusterShadersTextureUtils> ResolveTextureContext(const FDisplayClusterShadersTextureViewportContext& Input, const FDisplayClusterShadersTextureViewportContext& Output) override;
	virtual TSharedRef<IDisplayClusterShadersTextureUtils> ResolveTextureContext(const FDisplayClusterShadersTextureUtilsSettings& InSettings, const FDisplayClusterShadersTextureViewportContext& Input, const FDisplayClusterShadersTextureViewportContext& Output) override;

	virtual TSharedRef<IDisplayClusterShadersTextureUtils> Resolve() override;
	virtual TSharedRef<IDisplayClusterShadersTextureUtils> Resolve(const FDisplayClusterShadersTextureUtilsSettings& Settings) override;

	virtual TSharedRef<IDisplayClusterShadersTextureUtils> ForEachContextByPredicate(TFunctionDisplayClusterShaders_TextureContextIterator&& InFunction) override;
	virtual TSharedRef<IDisplayClusterShadersTextureUtils> ForEachContextByPredicate(const FDisplayClusterShadersTextureUtilsSettings& Settings, TFunctionDisplayClusterShaders_TextureContextIterator&& InFunction) override;

	virtual TSharedRef<IDisplayClusterShadersTextureUtils> SetInput(const FDisplayClusterShadersTextureViewport& InTextureViewport, const int32 InContextNum = INDEX_NONE) override;
	virtual TSharedRef<IDisplayClusterShadersTextureUtils> SetOutput(const FDisplayClusterShadersTextureViewport& InTextureViewport, const int32 InContextNum = INDEX_NONE) override;

	virtual TSharedRef<IDisplayClusterShadersTextureUtils> SetInput(const IDisplayClusterViewportProxy* InViewportProxy, const EDisplayClusterViewportResourceType InResourceType, const int32 InContextNum = INDEX_NONE) override;
	virtual TSharedRef<IDisplayClusterShadersTextureUtils> SetOutput(const IDisplayClusterViewportProxy* InViewportProxy, const EDisplayClusterViewportResourceType InResourceType,const int32 InContextNum = INDEX_NONE) override;


	virtual TSharedRef<IDisplayClusterShadersTextureUtils> SetInputEncoding(const FDisplayClusterColorEncoding& InColorEncoding) override;
	virtual TSharedRef<IDisplayClusterShadersTextureUtils> SetOutputEncoding(const FDisplayClusterColorEncoding& InColorEncoding) override;

	virtual const FDisplayClusterShadersTextureParameters& GetInputTextureParameters() const override
	{
		return InputTextureParameters;
	}
	virtual const FDisplayClusterShadersTextureParameters& GetOutputTextureParameters() const override
	{
		return OutputTextureParameters;
	}

	//~~ End IDisplayClusterShadersTextureUtils

public:
	/** Return true if RDG is required. */
	virtual bool ShouldUseRDG() const
	{
		return false;
	}

	/**
	* Implements Transition and copy
	*
	* @param Input    - Input texture with rect
	* @param Output   - Output texture with rect
	* @param Settings - current settings
	*
	* @return false, if it isn't possible
	*/
	virtual bool TransitionAndCopyTexture(
		const FDisplayClusterShadersTextureViewport& Input,
		const FDisplayClusterShadersTextureViewport& Output,
		const FDisplayClusterShadersTextureUtilsSettings& Settings)
	{
		return false;
	}

	/**
	* Implements resample shader
	*
	* @param Input    - Input texture with rect
	* @param Output   - Output texture with rect
	* @param Settings - current settings
	*
	* @return false, if it isn't possible
	*/
	virtual bool ResampleColorEncodingCopyRect(
		const FDisplayClusterShadersTextureViewportContext& Input,
		const FDisplayClusterShadersTextureViewportContext& Output,
		const FDisplayClusterShadersTextureUtilsSettings& Settings)
	{
		return false;
	}

	/** Create a new input texture from the output and copy the image as well.
	* 
	* @param OutTextureViewport - (out) The new texture
	* @param InTextureViewport  - (in) the source texture to be clone (size and format)
	* @param InDebugName - (opt) texture debug name
	* 
	* @return true on success.
	*/
	virtual bool CloneTextureViewportResourceForRenderPass(
		FDisplayClusterShadersTextureViewport& OutTextureViewport,
		const FDisplayClusterShadersTextureViewport& InTextureViewport,
		const TCHAR* InDebugName)
	{
		return false;
	};

	/** Update texture viewport data for render pass
	* return false if error
	*/
	virtual bool InitializeTextureViewportForRenderPass(
		FDisplayClusterShadersTextureViewport& InOutTextureViewport,
		const TCHAR* InDebugName)
	{
		return true;
	};

	/**
	* Implements texture context resolving
	*
	* @param Input    - Input texture with rect
	* @param Output   - Output texture with rect
	* @param Settings - current settings
	*
	* @return false, if it isn't possible
	*/
	virtual bool ImplementTextureContextResolve(
		const FDisplayClusterShadersTextureViewportContext& Input,
		const FDisplayClusterShadersTextureViewportContext& Output,
		const FDisplayClusterShadersTextureUtilsSettings& Settings);

public:
	/** Get texture parameter from the nDisplay viewport resourece
	*
	* @param InViewportProxy  - viewport proxy object with resources to retrieve
	* @param InResourceType   - Type of the viewport resource
	*
	* @return themself
	*/
	static FDisplayClusterShadersTextureParameters GetTextureParametersFromViewport(
		const IDisplayClusterViewportProxy* InViewportProxy,
		const EDisplayClusterViewportResourceType InResourceType);

	/** Returns true if a resampling shader should be used to copy this texture.
	*
	* @param InputTextureContext  - input texture context
	* @param OutputTextureContext - output texture context
	* @param Settings             - current settings
	*/
	bool ShouldUseResampleShader(
		const FDisplayClusterShadersTextureViewportContext& InputTextureContext,
		const FDisplayClusterShadersTextureViewportContext& OutputTextureContext,
		const FDisplayClusterShadersTextureUtilsSettings& Settings) const;


	/** Iterate through contexts matching these settings. */
	void ImplForEachContextByPredicate(
		const FDisplayClusterShadersTextureUtilsSettings& Settings,
		TFunctionDisplayClusterShaders_TextureContextIterator&& TextureContextIteratorFunc);

protected:
	/** Input textures parameters. */
	FDisplayClusterShadersTextureParameters InputTextureParameters;

	/** Output textures parameters. */
	FDisplayClusterShadersTextureParameters OutputTextureParameters;
};

/**
 *  RDI:Texture utils class for nDisplay.
 */
class FDisplayClusterShadersRHITextureUtils
	: public FDisplayClusterShadersTextureUtils
{
public:
	virtual ~FDisplayClusterShadersRHITextureUtils()
	{
		if (GraphBuilderUniquePtr)
		{
			// If using RDG, call execute at the end
			GraphBuilderUniquePtr->Execute();
			GraphBuilderUniquePtr.Reset();
		}

		// And finally release pooled RTT
		PooledRenderTargets.Reset();
	}

	FDisplayClusterShadersRHITextureUtils(FRHICommandListImmediate& InRHICmdList)
			: FDisplayClusterShadersTextureUtils(), RHICmdList(InRHICmdList)
	{ }

public:
	//~Begin IDisplayClusterShadersTextureUtils
	virtual FRDGBuilder& GetOrCreateRDGBuilder() override
	{
		if (!GraphBuilderUniquePtr.IsValid())
		{
			GraphBuilderUniquePtr = MakeUnique<FRDGBuilder>(RHICmdList);
		}

		return *GraphBuilderUniquePtr;
	}
	//~~End IDisplayClusterShadersTextureUtils

public:
	//~Begin FDisplayClusterShadersTextureUtils
	virtual bool TransitionAndCopyTexture(const FDisplayClusterShadersTextureViewport& Input, const FDisplayClusterShadersTextureViewport& Output, const FDisplayClusterShadersTextureUtilsSettings& Settings) override;
	virtual bool ResampleColorEncodingCopyRect(const FDisplayClusterShadersTextureViewportContext& Input, const FDisplayClusterShadersTextureViewportContext& Output, const FDisplayClusterShadersTextureUtilsSettings& Settings) override;
	virtual bool CloneTextureViewportResourceForRenderPass(FDisplayClusterShadersTextureViewport& OutTextureViewport, const FDisplayClusterShadersTextureViewport& InTextureViewport, const TCHAR* InDebugName) override;
	virtual bool InitializeTextureViewportForRenderPass(FDisplayClusterShadersTextureViewport& InOutTextureViewport, const TCHAR* InDebugName) override;

	virtual bool ShouldUseRDG() const override
	{
		return GraphBuilderUniquePtr.IsValid();
	}

	//~~End FDisplayClusterShadersTextureUtils

private:
	// Stored RHI api ref
	FRHICommandListImmediate& RHICmdList;

	/** Temporary RTTs*/
	TArray<TRefCountPtr<IPooledRenderTarget>> PooledRenderTargets;

	// A GraphBuilder that can be created on request.
	TUniquePtr<FRDGBuilder> GraphBuilderUniquePtr;
};

/**
 *  RDG:Texture utils class for nDisplay.
 */
class FDisplayClusterShadersRDGTextureUtils
	: public FDisplayClusterShadersTextureUtils
{
public:
	virtual ~FDisplayClusterShadersRDGTextureUtils() = default;	
	FDisplayClusterShadersRDGTextureUtils(FRDGBuilder& InGraphBuilder)
		: FDisplayClusterShadersTextureUtils(), GraphBuilder(InGraphBuilder)
	{ }

public:
	//~Begin IDisplayClusterShadersTextureUtils
	virtual FRDGBuilder& GetOrCreateRDGBuilder() override
	{
		return GraphBuilder;
	}
	//~~End IDisplayClusterShadersTextureUtils

public:
	//~Begin FDisplayClusterShadersTextureUtils
	virtual bool TransitionAndCopyTexture(const FDisplayClusterShadersTextureViewport& Input, const FDisplayClusterShadersTextureViewport& Output, const FDisplayClusterShadersTextureUtilsSettings& Settings) override;
	virtual bool ResampleColorEncodingCopyRect(const FDisplayClusterShadersTextureViewportContext& Input, const FDisplayClusterShadersTextureViewportContext& Output, const FDisplayClusterShadersTextureUtilsSettings& Settings) override;
	virtual bool CloneTextureViewportResourceForRenderPass(FDisplayClusterShadersTextureViewport& OutTextureViewport, const FDisplayClusterShadersTextureViewport& InTextureViewport, const TCHAR* InDebugName) override;
	virtual bool InitializeTextureViewportForRenderPass(FDisplayClusterShadersTextureViewport& InOutTextureViewport, const TCHAR* InDebugName) override;
	virtual bool ShouldUseRDG() const override
	{
		return true;
	}
	//~~End FDisplayClusterShadersTextureUtils


private:
	// Stored RDG api ref
	FRDGBuilder& GraphBuilder;
};