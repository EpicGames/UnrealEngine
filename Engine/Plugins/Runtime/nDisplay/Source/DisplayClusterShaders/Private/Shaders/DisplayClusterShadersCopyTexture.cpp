// Copyright Epic Games, Inc. All Rights Reserved.

#include "Shaders/DisplayClusterShadersCopyTexture.h"
#include "DisplayClusterShadersLog.h"

#include "GlobalShader.h"
#include "ShaderParameters.h"
#include "ShaderParameterUtils.h"
#include "TextureResource.h"

#include "RHIStaticStates.h"

#include "RenderResource.h"
#include "RenderingThread.h"
#include "CommonRenderResources.h"
#include "PixelShaderUtils.h"

#include "ClearQuad.h"

#include "GlobalShader.h"
#include "ShaderParameters.h"
#include "ShaderParameterUtils.h"

#include "ScreenRendering.h"
#include "RenderGraphUtils.h"
#include "RenderGraphResources.h"
#include "PostProcess/DrawRectangle.h"
#include "PostProcess/PostProcessMaterialInputs.h"

#include "SceneView.h"
#include "ScreenPass.h"

#include "Engine/TextureRenderTarget.h"

namespace UE::DisplayClusterShaders::Private
{
	/** Returns the RHI blend state for the requested DC blend mode. */
	FRHIBlendState* GetBlendStateRHI(const EColorWriteMask InColorMask)
	{
		if (InColorMask == EColorWriteMask::CW_ALPHA)
		{
			// Copy alpha channel from source to dest
			return TStaticBlendState <CW_ALPHA, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero>::GetRHI();

		}
		else if (InColorMask == EColorWriteMask::CW_RGB)
		{
			// Copy only RGB channels from source to dest
			return TStaticBlendState <CW_RGB, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero>::GetRHI();
		}

		return TStaticBlendState<>::GetRHI();
	}

	/** Enumerations that cover definitions in the shader. */
	enum class ESourceEncoding : uint32
	{
		Linear  = 0, // #define ESourceEncoding_Linear  0
		Gamma   = 1, // #define ESourceEncoding_Gamma   1
		sRGB    = 2, // #define ESourceEncoding_sRGB    2
		MediaPQ = 3, // #define ESourceEncoding_MediaPQ 3
	};

	enum class EColorPremultiply : uint32
	{
		None          = 0, // #define EColorPremultiply_None         0
		Alpha         = 1, // #define EColorPremultiply_Alpha        1
		InvertedAlpha = 2, // #define EColorPremultiply_InvertedAlpha 2
	};

	/** Enumerations that cover definitions in the shader. */
	enum class EOverrideAlpha : uint32
	{
		None   = 0, // #define EOverrideAlpha_None   0
		Invert = 1, // #define EOverrideAlpha_Invert 1
		One    = 2, // #define EOverrideAlpha_One    2
		Zero   = 3, // #define EOverrideAlpha_Zero   3
	};

	/** Enumerations that cover definitions in the shader. */
	enum class ESideFeather : uint32
	{
		None   = 0,  // #define ESideFeather_None     0
		Linear = 1,  // #define ESideFeather_Linear   1
		Smooth = 2,  // #define ESideFeather_Smooth   2
	};

	namespace FColorEncodingCopyRectPSPermutation
	{
		// Shared permutation for picp warp
		class FPermutationEncodeInput   : SHADER_PERMUTATION_BOOL("ENCODE_INPUT");
		class FPermutationEncodeOutput  : SHADER_PERMUTATION_BOOL("ENCODE_OUTPUT");
		class FPermutationOverrideAlpha : SHADER_PERMUTATION_BOOL("OVERRIDE_ALPHA");
		class FPermutationColorPremultiply : SHADER_PERMUTATION_BOOL("COLOR_PREMULTIPLY");

		using FCommonPSDomain = TShaderPermutationDomain<
			FPermutationEncodeInput,
			FPermutationEncodeOutput,
			FPermutationOverrideAlpha,
			FPermutationColorPremultiply
		>;

		bool ShouldCompileCommonPSPermutation(const FCommonPSDomain& PermutationVector)
		{
			return true;
		}
	};

	/** RDG copy texture parameters. */
	BEGIN_SHADER_PARAMETER_STRUCT(FDisplayClusterCopyTextureParameters, )
		RDG_TEXTURE_ACCESS(Input, ERHIAccess::CopySrc)
		RDG_TEXTURE_ACCESS(Output, ERHIAccess::CopyDest)
	END_SHADER_PARAMETER_STRUCT()

	/** RDG screen pixel shader parameters. */
	BEGIN_SHADER_PARAMETER_STRUCT(FDisplayClusterScreenPixelShaderTextureParameters, )
		RDG_TEXTURE_ACCESS(Input, ERHIAccess::SRVGraphicsPixel)
		RDG_TEXTURE_ACCESS(Output, ERHIAccess::RTV)
	END_SHADER_PARAMETER_STRUCT()

	/**
	* Pixel shaders parameters for 'FColorEncodingCopyRectPS'
	*/
	BEGIN_SHADER_PARAMETER_STRUCT(FColorEncodingCopyRectPSParameters, )
		SHADER_PARAMETER_TEXTURE(Texture2D, InputTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState,  InputTextureSampler)
		SHADER_PARAMETER(FUint32Vector, ColorPremultiply)
		SHADER_PARAMETER(FUint32Vector, Encodings)
		SHADER_PARAMETER(FVector3f, DisplayGamma)
		SHADER_PARAMETER(FVector4f, InnerFeatherMargins)
		SHADER_PARAMETER(FVector4f, OuterFeatherMargins)
	END_SHADER_PARAMETER_STRUCT()

	/**
	 * A pixel shader for TextureShare resource resampling
	 */
	class FColorEncodingCopyRectPS : public FGlobalShader
	{
	public:
		DECLARE_GLOBAL_SHADER(FColorEncodingCopyRectPS);
		SHADER_USE_PARAMETER_STRUCT(FColorEncodingCopyRectPS, FGlobalShader);

		using FPermutationDomain = FColorEncodingCopyRectPSPermutation::FCommonPSDomain;
		using FParameters = FColorEncodingCopyRectPSParameters;

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
		{
			// Only compile for desktop-class "PC" shader platforms (filters out mobile and console).
			if (!IsPCPlatform(Parameters.Platform))
			{
				return false;
			}

			return FColorEncodingCopyRectPSPermutation::ShouldCompileCommonPSPermutation(FPermutationDomain(Parameters.PermutationId));
		}

		/** Initialize shader parameters. */
		static bool InitializeShaderParameters(
			const FDisplayClusterShadersTextureViewportContext& Input,
			const FIntPoint& InputTextureSize,
			const FDisplayClusterShadersTextureViewportContext& Output,
			const FDisplayClusterShadersTextureUtilsSettings& Settings,
			FParameters& OutParameters,
			FColorEncodingCopyRectPSPermutation::FCommonPSDomain& OutPermutationVector)
		{
			OutParameters.InputTextureSampler = (Input.Rect.Size() == Output.Rect.Size())
				? TStaticSamplerState<SF_Point>::GetRHI() // Use the 'Point' sampler if the input and output texture sizes are equal
				: TStaticSamplerState<SF_Bilinear>::GetRHI();

			// Setup color encoding.
			// The default `0` is ESceneColorSourceEncoding::Linear.
			// Also linear encoding is used for HoldoutComposite input
			OutParameters.Encodings = FUint32Vector::ZeroValue;
			OutParameters.DisplayGamma = FVector3f(1.f, 1.f, 1.f);

			// Default values
			OutParameters.InnerFeatherMargins =
				OutParameters.OuterFeatherMargins = FVector4f::Zero();

			const bool UseGammaEncoding = (Settings.ColorMask & EColorWriteMask::CW_RGB)
				&& !Input.ColorEncoding.IsEqualsGammaEncoding(Output.ColorEncoding);

			// Only apply color transform if input and output encodings are not equal and color channels are used.
			if (UseGammaEncoding)
			{
				// Gets source and dest gamma values:
				const float DefaultDisplayGamma = UTextureRenderTarget::GetDefaultDisplayGamma();
				const float SrcGamma  = (Input.ColorEncoding.GammaValue > 0)  ? Input.ColorEncoding.GammaValue : DefaultDisplayGamma;
				const float DestGamma = (Output.ColorEncoding.GammaValue > 0) ? Output.ColorEncoding.GammaValue : DefaultDisplayGamma;

				if (Input.ColorEncoding.Encoding == Output.ColorEncoding.Encoding
					&& Input.ColorEncoding.Encoding == EDisplayClusterColorEncoding::Gamma)
				{
					// Convert Gamma->Gamma in one pow()
					OutParameters.Encodings.X = static_cast<uint32>(ESourceEncoding::Gamma);
					OutParameters.DisplayGamma.X = SrcGamma / DestGamma;
					OutPermutationVector.Set<FColorEncodingCopyRectPSPermutation::FPermutationEncodeInput>(true);
				}
				else // Transformation via linear color space:
				{
					// Convert input encoding to the linear
					switch (Input.ColorEncoding.Encoding)
					{
					case EDisplayClusterColorEncoding::Gamma: // Gamma -> Linear
						OutParameters.Encodings.X = static_cast<uint32>(ESourceEncoding::Gamma);
						OutParameters.DisplayGamma.X = SrcGamma;
						OutPermutationVector.Set<FColorEncodingCopyRectPSPermutation::FPermutationEncodeInput>(true);
						break;

					case EDisplayClusterColorEncoding::sRGB: // sRGB -> Linear
						OutParameters.Encodings.X = static_cast<uint32>(ESourceEncoding::sRGB);
						OutPermutationVector.Set<FColorEncodingCopyRectPSPermutation::FPermutationEncodeInput>(true);
						break;

					case EDisplayClusterColorEncoding::MediaPQ: // MediaPQ -> Linear
						OutParameters.Encodings.X = static_cast<uint32>(ESourceEncoding::MediaPQ);
						OutPermutationVector.Set<FColorEncodingCopyRectPSPermutation::FPermutationEncodeInput>(true);
						break;

					default:
						break;
					}

					// Convert linear to the output encoding
					switch (Output.ColorEncoding.Encoding)
					{
					case EDisplayClusterColorEncoding::Gamma: // Linear -> Gamma
						OutParameters.Encodings.Y = static_cast<uint32>(ESourceEncoding::Gamma);
						OutParameters.DisplayGamma.Y = 1.0f / DestGamma;
						OutPermutationVector.Set<FColorEncodingCopyRectPSPermutation::FPermutationEncodeOutput>(true);
						break;

					case EDisplayClusterColorEncoding::sRGB: // Linear -> sRGB
						OutParameters.Encodings.Y = static_cast<uint32>(ESourceEncoding::sRGB);
						OutPermutationVector.Set<FColorEncodingCopyRectPSPermutation::FPermutationEncodeOutput>(true);
						break;

					case EDisplayClusterColorEncoding::MediaPQ: // Linear -> MediaPQ
						OutParameters.Encodings.Y = static_cast<uint32>(ESourceEncoding::MediaPQ);
						OutPermutationVector.Set<FColorEncodingCopyRectPSPermutation::FPermutationEncodeOutput>(true);
						break;

					default:
						break;
					}
				}
			}

			// Setup Color Premultiply
			OutParameters.ColorPremultiply = FUint32Vector::ZeroValue;

			if (Input.ColorEncoding.Premultiply != Output.ColorEncoding.Premultiply
			|| (UseGammaEncoding && Input.ColorEncoding.Premultiply != EDisplayClusterColorPremultiply::None))
			{
				switch (Input.ColorEncoding.Premultiply)
				{
				case EDisplayClusterColorPremultiply::Premultiply:
					OutParameters.ColorPremultiply.X = static_cast<uint32>(EColorPremultiply::Alpha);
					OutPermutationVector.Set<FColorEncodingCopyRectPSPermutation::FPermutationColorPremultiply>(true);
					break;
				case EDisplayClusterColorPremultiply::InvertPremultiply:
					OutParameters.ColorPremultiply.X = static_cast<uint32>(EColorPremultiply::InvertedAlpha);
					OutPermutationVector.Set<FColorEncodingCopyRectPSPermutation::FPermutationColorPremultiply>(true);
					break;
				default:
					break;
				}
				switch (Output.ColorEncoding.Premultiply)
				{
				case EDisplayClusterColorPremultiply::Premultiply:
					OutParameters.ColorPremultiply.Y = static_cast<uint32>(EColorPremultiply::Alpha);
					OutPermutationVector.Set<FColorEncodingCopyRectPSPermutation::FPermutationColorPremultiply>(true);
					break;
				case EDisplayClusterColorPremultiply::InvertPremultiply:
					OutParameters.ColorPremultiply.Y = static_cast<uint32>(EColorPremultiply::InvertedAlpha);
					OutPermutationVector.Set<FColorEncodingCopyRectPSPermutation::FPermutationColorPremultiply>(true);
					break;
				default:
					break;
				}
			}

			// Override alpha
			switch(Settings.OverrideAlpha)
			{
			case EDisplayClusterShaderTextureUtilsOverrideAlpha::Invert_Alpha:
				OutParameters.Encodings.Z = static_cast<uint32>(EOverrideAlpha::Invert);
				OutPermutationVector.Set<FColorEncodingCopyRectPSPermutation::FPermutationOverrideAlpha>(true);
				break;

			case EDisplayClusterShaderTextureUtilsOverrideAlpha::Set_Alpha_One:
				OutParameters.Encodings.Z = static_cast<uint32>(EOverrideAlpha::One);
				OutPermutationVector.Set<FColorEncodingCopyRectPSPermutation::FPermutationOverrideAlpha>(true);
				break;

			case EDisplayClusterShaderTextureUtilsOverrideAlpha::Set_Alpha_Zero:
				OutParameters.Encodings.Z = static_cast<uint32>(EOverrideAlpha::Zero);
				OutPermutationVector.Set<FColorEncodingCopyRectPSPermutation::FPermutationOverrideAlpha>(true);
				break;
			}

			// Side Feather
			if (Settings.HasAnyFlags(
				  EDisplayClusterShaderTextureUtilsFlags::EnableLinearAlphaFeather
				| EDisplayClusterShaderTextureUtilsFlags::EnableSmoothAlphaFeather))
			{
				// Feathering logic is enabled in the pixel shader under this permutation.
				OutPermutationVector.Set<FColorEncodingCopyRectPSPermutation::FPermutationOverrideAlpha>(true);

				// Only one alpha feathering mode(linear or smooth) should be enabled at a time.
				if (Settings.HasAnyFlags(EDisplayClusterShaderTextureUtilsFlags::EnableLinearAlphaFeather))
				{
					// ColorPremultiply.Z = SideFeather method
					OutParameters.ColorPremultiply.Z = static_cast<uint32>(ESideFeather::Linear);
				}
				else if (Settings.HasAnyFlags(EDisplayClusterShaderTextureUtilsFlags::EnableSmoothAlphaFeather))
				{
					// ColorPremultiply.Z = SideFeather method
					OutParameters.ColorPremultiply.Z = static_cast<uint32>(ESideFeather::Smooth);
				}

				// Feather margins
				//
				//  Outer LT              RT
				//  +---------------------+
				//  |                     |
				//  |  Inner LT      RT   |
				//  |     +----------+    |
				//  |     |          |    |
				//  |     +----------+    |
				//  |     LB         RB   |
				//  +---------------------+
				//  LB                    RB
				//
				// L,R,T,B = Left, Right, Top, Bottom
				//
				// InnerFeatherMargins = <InnerL, 1-InnerR, InnerT, 1-InnerB>  // Fully opaque region
				// OuterFeatherMargins = <OuterL, 1-OuterR, OuterT, 1-OuterB>  // Fully transparent region

				FIntRect InnerRect = Input.Rect;
				Settings.SideMargins.ApplyClampedInsetToRect(InnerRect);

				const FIntRect OuterRect = Input.Rect;

				const float Width  = static_cast<float>(InputTextureSize.X);
				const float Height = static_cast<float>(InputTextureSize.Y);

				// Left
				OutParameters.InnerFeatherMargins.X = float(InnerRect.Min.X) / Width;
				OutParameters.OuterFeatherMargins.X = float(OuterRect.Min.X) / Width;

				// Right
				OutParameters.InnerFeatherMargins.Y = 1.0f - (float(InnerRect.Max.X) / Width);
				OutParameters.OuterFeatherMargins.Y = 1.0f - (float(OuterRect.Max.X) / Width);

				// Top
				OutParameters.InnerFeatherMargins.Z = float(InnerRect.Min.Y) / Height;
				OutParameters.OuterFeatherMargins.Z = float(OuterRect.Min.Y) / Height;

				// Bottom
				OutParameters.InnerFeatherMargins.W = 1.0f - (float(InnerRect.Max.Y) / Height);
				OutParameters.OuterFeatherMargins.W = 1.0f - (float(OuterRect.Max.Y) / Height);
			}

			// Check the permutation vectors. This should prevent a crash when no shader permutation is found.
			if (!FColorEncodingCopyRectPSPermutation::ShouldCompileCommonPSPermutation(OutPermutationVector))
			{
				UE_LOGF(LogDisplayClusterShaders, Warning, "Invalid permutation vector %d for shader `FColorEncodingCopyRectPS`", OutPermutationVector.ToDimensionValueId());

				return false;
			}

			return true;
		}

		/** Render input to output using this shader. */
		static bool RenderPass(
			FRHICommandListImmediate& RHICmdList,
			FRHITexture* SrcTexture,
			FRHITexture* DestTexture,
			const FDisplayClusterShadersTextureViewportContext& InputContext,
			const FDisplayClusterShadersTextureViewportContext& OutputContext,
			const FDisplayClusterShadersTextureUtilsSettings& Settings)
		{
			if (!SrcTexture || !DestTexture)
			{
				return false;
			}

#if UE_BUILD_DEBUG
			// Dest should be RTT (debug purpose)
			check(EnumHasAnyFlags(DestTexture->GetDesc().Flags, ETextureCreateFlags::RenderTargetable));
#endif

			const FIntPoint& SrcTextureSize = SrcTexture->GetDesc().Extent;
			const FIntPoint& DestTextureSize = DestTexture->GetDesc().Extent;

			// Initialize shader parameters and permutation vectors
			FParameters PixelShaderParameters;
			FColorEncodingCopyRectPSPermutation::FCommonPSDomain PermutationVector;
			if (!InitializeShaderParameters(InputContext, SrcTextureSize, OutputContext, Settings, PixelShaderParameters, PermutationVector))
			{
				return false;
			}
			PixelShaderParameters.InputTexture = SrcTexture;

			FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
			TShaderMapRef<FScreenVS> VertexShader(ShaderMap);
			TShaderMapRef<FColorEncodingCopyRectPS> PixelShader(ShaderMap, PermutationVector);
			if (!VertexShader.IsValid() || !PixelShader.IsValid())
			{
				// Always check if shaders are available on the current platform and hardware
				return false;
			}

			const FIntRect& SrcRect = InputContext.Rect;
			const FIntRect& DestRect = OutputContext.Rect;

			FRHIRenderPassInfo RPInfo(DestTexture, ERenderTargetActions::Load_Store);
			RHICmdList.BeginRenderPass(RPInfo, TEXT("nDisplay.Shaders.ColorEncodingCopyRect"));
			
			{
				RHICmdList.SetViewport(0.0f, 0.0f, 0.0f, DestTextureSize.X, DestTextureSize.Y, 1.0f);

				FGraphicsPipelineStateInitializer GraphicsPSOInit;
				RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

				if (Settings.HasAnyFlags(
					EDisplayClusterShaderTextureUtilsFlags::EnableLinearAlphaFeather
				| EDisplayClusterShaderTextureUtilsFlags::EnableSmoothAlphaFeather))
				{
					// Special blending mode for feather
					GraphicsPSOInit.BlendState = TStaticBlendState <CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One>::GetRHI();
				}
				else
				{
					GraphicsPSOInit.BlendState = GetBlendStateRHI(Settings.ColorMask);
				}

				GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
				GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();

				GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
				GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
				GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
				GraphicsPSOInit.PrimitiveType = PT_TriangleList;

				SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

				SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), PixelShaderParameters);

				UE::Renderer::PostProcess::DrawRectangle(
					RHICmdList, VertexShader,
					DestRect.Min.X, DestRect.Min.Y,
					DestRect.Size().X, DestRect.Size().Y,
					SrcRect.Min.X, SrcRect.Min.Y,
					SrcRect.Size().X, SrcRect.Size().Y,
					DestTextureSize, SrcTextureSize
				);
			}

			RHICmdList.EndRenderPass();

			return true;
		}
	};

	IMPLEMENT_GLOBAL_SHADER(FColorEncodingCopyRectPS, "/Plugin/nDisplay/Private/ResourceUtils.usf", "Main", SF_Pixel);

	/** Return CopyTextureInfo structure. */
	static inline FRHICopyTextureInfo GetRHICopyTextureInfo(
		const FDisplayClusterShadersTextureViewport& Input,
		const FDisplayClusterShadersTextureViewport& Output,
		const FDisplayClusterShadersTextureUtilsSettings& InSettings)
	{
		FRHICopyTextureInfo OutCopyInfo;

		OutCopyInfo.SourceSliceIndex = InSettings.SourceSliceIndex;
		OutCopyInfo.DestSliceIndex = InSettings.DestSliceIndex;

		OutCopyInfo.SourcePosition = FIntVector(Input.Rect.Min.X, Input.Rect.Min.Y, 0);
		OutCopyInfo.DestPosition = FIntVector(Output.Rect.Min.X, Output.Rect.Min.Y, 0);

		OutCopyInfo.Size = FIntVector(Output.Rect.Width(), Output.Rect.Height(), 1);

		return OutCopyInfo;
	}
};

bool FDisplayClusterShadersCopyTexture::ColorEncodingCopyRect_RenderThread(
	FRHICommandListImmediate& RHICmdList,
	const FDisplayClusterShadersTextureViewportContext& Input,
	const FDisplayClusterShadersTextureViewportContext& Output,
	const FDisplayClusterShadersTextureUtilsSettings& Settings)
{
	using namespace UE::DisplayClusterShaders::Private;

	RHICmdList.Transition(FRHITransitionInfo(Input.TextureRHI, ERHIAccess::Unknown, ERHIAccess::SRVMask));
	RHICmdList.Transition(FRHITransitionInfo(Output.TextureRHI, ERHIAccess::Unknown, ERHIAccess::RTV));

	bool bResult = FColorEncodingCopyRectPS::RenderPass(RHICmdList, Input.TextureRHI, Output.TextureRHI, Input, Output, Settings);

	RHICmdList.Transition(FRHITransitionInfo(Output.TextureRHI, ERHIAccess::Unknown, ERHIAccess::SRVMask));

	return bResult;
}

/**
* Copy RDG textures via shader.
*/
bool FDisplayClusterShadersCopyTexture::AddPassColorEncodingCopyRect_RenderThread(
	FRDGBuilder& GraphBuilder,
	const FDisplayClusterShadersTextureViewportContext& Input,
	const FDisplayClusterShadersTextureViewportContext& Output,
	const FDisplayClusterShadersTextureUtilsSettings& Settings)
{
	using namespace UE::DisplayClusterShaders::Private;

	if (!Input.TextureRDG || !Output.TextureRDG)
	{
		return false;
	}

	// Initialize render pass parameters.
	FDisplayClusterScreenPixelShaderTextureParameters* Parameters = GraphBuilder.AllocParameters<FDisplayClusterScreenPixelShaderTextureParameters>();
	Parameters->Input  = Input.TextureRDG;
	Parameters->Output = Output.TextureRDG;

	if (Input.bExternalTextureRDG)
	{
		GraphBuilder.SetTextureAccessFinal(Input.TextureRDG, ERHIAccess::SRVGraphics);
	}
	if (Output.bExternalTextureRDG)
	{
		GraphBuilder.SetTextureAccessFinal(Output.TextureRDG, ERHIAccess::RTV);
	}

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("nDisplayShaders.ResampleTexture"),
		Parameters,
		ERDGPassFlags::Raster | ERDGPassFlags::SkipRenderPass | ERDGPassFlags::NeverCull,
		[Parameters, Input, Output, Settings](FRHICommandListImmediate& RHICmdList)
	{
		FColorEncodingCopyRectPS::RenderPass(
			RHICmdList,
			Parameters->Input->GetRHI(),
			Parameters->Output->GetRHI(),
			Input, Output, Settings);
	});

	return true;
}

/**
* Copy RDG textures without shader.
*/
bool FDisplayClusterShadersCopyTexture::AddPassTransitionAndCopyTexture_RenderThread(
	FRDGBuilder& GraphBuilder,
	const FDisplayClusterShadersTextureViewport& Input,
	const FDisplayClusterShadersTextureViewport& Output,
	const FDisplayClusterShadersTextureUtilsSettings& Settings)
{
	using namespace UE::DisplayClusterShaders::Private;

	if (!Input.TextureRDG || !Output.TextureRDG)
	{
		return false;
	}

	if (Input.Rect.Size() != Output.Rect.Size())
	{
		return false;
	}

	// Initialize render pass parameters.
	FDisplayClusterCopyTextureParameters* Parameters = GraphBuilder.AllocParameters<FDisplayClusterCopyTextureParameters>();
	Parameters->Input = Input.TextureRDG;
	Parameters->Output = Output.TextureRDG;

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("nDisplayShaders.CopyTexture"),
		Parameters,
		ERDGPassFlags::Copy | ERDGPassFlags::NeverCull,
		[ Input, Output, Settings](FRHICommandListImmediate& RHICmdList)
		{
			FRHITexture* Src = Input.TextureRDG->GetRHI();
			FRHITexture* Dest = Output.TextureRDG->GetRHI();
			if (Src && Dest)
			{
				TransitionAndCopyTexture(RHICmdList, Src, Dest, GetRHICopyTextureInfo(Input, Output, Settings));
			}
		});

	return true;
}

bool FDisplayClusterShadersCopyTexture::TransitionAndCopyTexture_RenderThread(
	FRHICommandListImmediate& RHICmdList,
	const FDisplayClusterShadersTextureViewport& Input,
	const FDisplayClusterShadersTextureViewport& Output,
	const FDisplayClusterShadersTextureUtilsSettings& InSettings)
{
	using namespace UE::DisplayClusterShaders::Private;

	if (!Input.TextureRHI || !Output.TextureRHI)
	{
		return false;
	}

	if (Input.Rect.Size() != Output.Rect.Size())
	{
		return false;
	}

	TransitionAndCopyTexture(RHICmdList,
		Input.TextureRHI,
		Output.TextureRHI,
		GetRHICopyTextureInfo(Input, Output, InSettings));

	return true;
}
