// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "DisplayClusterShadersTextureUtils.h"

#include "RHI.h"
#include "RHICommandList.h"

class FRDGBuilder;

/** Implementation of texture copying. */
class FDisplayClusterShadersCopyTexture
{
public:
	/** Implements texture copy via pixel shader.
	*
	* @param RHICmdList   - RHI API
	* @param Input        - input texture context
	* @param Output       - output texture context
	* @param InSettings
	*
	* @return - true if success
	*/
	static bool ColorEncodingCopyRect_RenderThread(
		FRHICommandListImmediate& RHICmdList,
		const FDisplayClusterShadersTextureViewportContext& Input,
		const FDisplayClusterShadersTextureViewportContext& Output,
		const FDisplayClusterShadersTextureUtilsSettings& InSettings);

	/** Implements texture copy via pixel shader.
	*
	* @param RHICmdList   - RHI API
	* @param Input        - input texture context
	* @param Output       - output texture context
	* @param InSettings
	*
	* @return - true if success
	*/
	static bool AddPassColorEncodingCopyRect_RenderThread(
		FRDGBuilder& GraphBuilder,
		const FDisplayClusterShadersTextureViewportContext& Input,
		const FDisplayClusterShadersTextureViewportContext& Output,
		const FDisplayClusterShadersTextureUtilsSettings& InSettings);

	/**
	* Call TransitionAndCopyTexture()
	*
	* @param RHICmdList   - RHI API
	* @param Input        - input texture context
	* @param Output       - output texture context
	* @param InSettings
	*
	* @return - true if success
	*/
	static bool TransitionAndCopyTexture_RenderThread(
		FRHICommandListImmediate& RHICmdList,
		const FDisplayClusterShadersTextureViewport& Input,
		const FDisplayClusterShadersTextureViewport& Output,
		const FDisplayClusterShadersTextureUtilsSettings& InSettings);

	/**
	* Call TransitionAndCopyTexture()
	*
	* @param GraphBuilder - RDG builder
	* @param Input        - input texture context
	* @param Output       - output texture context
	* @param InSettings
	*
	* @return - true if success
	*/
	static bool AddPassTransitionAndCopyTexture_RenderThread(
		FRDGBuilder& GraphBuilder,
		const FDisplayClusterShadersTextureViewport& Input,
		const FDisplayClusterShadersTextureViewport& Output,
		const FDisplayClusterShadersTextureUtilsSettings& InSettings);
};
