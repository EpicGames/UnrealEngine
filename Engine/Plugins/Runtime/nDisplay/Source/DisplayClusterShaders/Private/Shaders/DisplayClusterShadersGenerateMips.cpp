// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterShadersGenerateMips.h"

#include "RHIResources.h"
#include "GenerateMips.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"

bool FDisplayClusterShadersGenerateMips::GenerateMips(FRHICommandListImmediate& RHICmdList, FRHITexture* InOutMipsTexture, const FDisplayClusterShaderParameters_GenerateMips& InSettings)
{
	if (InSettings.IsEnabled())
	{
		check(InOutMipsTexture);

		FGenerateMipsParams GenerateMipsParams{ InSettings.MipsSamplerFilter == TF_Nearest ? SF_Point : (InSettings.MipsSamplerFilter == TF_Trilinear ? SF_Trilinear : SF_Bilinear),
			InSettings.MipsAddressU == TA_Wrap ? AM_Wrap : (InSettings.MipsAddressU == TA_Mirror ? AM_Mirror : AM_Clamp),
			InSettings.MipsAddressV == TA_Wrap ? AM_Wrap : (InSettings.MipsAddressV == TA_Mirror ? AM_Mirror : AM_Clamp) };
		
		FRDGBuilder GraphBuilder(RHICmdList);
		TRefCountPtr<IPooledRenderTarget> PoolRenderTarget = CreateRenderTarget(InOutMipsTexture, TEXT("nDisplayViewportMips"));
		FRDGTextureRef MipOutputTexture = GraphBuilder.RegisterExternalTexture(PoolRenderTarget);

		// TODO: get FeatureLevel from an outside source.
		ERHIFeatureLevel::Type FeatureLevel = GMaxRHIFeatureLevel;
		FGenerateMips::Execute(GraphBuilder, FeatureLevel, MipOutputTexture, GenerateMipsParams);

		GraphBuilder.Execute();

		return true;
	}

	return false;
}
