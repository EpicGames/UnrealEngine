// Copyright Epic Games, Inc. All Rights Reserved.

#include "Shaders/DisplayClusterShadersMedia.h"
#include "ShaderParameters/DisplayClusterShaderParameters_Media.h"

#include "PixelFormat.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "RHIResources.h"
#include "RHIStaticStates.h"
#include "SceneView.h"
#include "ScreenPass.h"
#include "TextureResource.h"

#include "DisplayClusterShadersLog.h"


namespace UE::DisplayClusterShaders::Private
{
	static const FString MediaShadersPath = TEXT("/Plugin/nDisplay/Private/MediaShaders.usf");

	///////////////////////////////////////////////////////////////////////////////////
	// Linear-To-PQ
	class FLinearToPQPS : public FGlobalShader
	{
		DECLARE_SHADER_TYPE(FLinearToPQPS, Global);
		SHADER_USE_PARAMETER_STRUCT(FLinearToPQPS, FGlobalShader);

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_TEXTURE(Texture2D, InputTexture)
			SHADER_PARAMETER_SAMPLER(SamplerState, InputSampler)
			RENDER_TARGET_BINDING_SLOTS()
		END_SHADER_PARAMETER_STRUCT()

	public:

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& InParameters)
		{
			// Only compile for desktop-class "PC" shader platforms (filters out mobile and console).
			if (!IsPCPlatform(InParameters.Platform))
			{
				return false;
			}

			return true;
		}

		FParameters* AllocateAndSetParameters(FRDGBuilder& InGraphBuilder, FRDGTextureRef InLinearTexture, FRDGTextureRef InOutputPQTexture)
		{
			FParameters* Parameters = InGraphBuilder.AllocParameters<FParameters>();
			Parameters->InputTexture = InLinearTexture;
			Parameters->InputSampler = TStaticSamplerState<>::GetRHI();
			Parameters->RenderTargets[0] = FRenderTargetBinding{ InOutputPQTexture, ERenderTargetLoadAction::ENoAction };
			return Parameters;
		}
	};

	IMPLEMENT_SHADER_TYPE(, FLinearToPQPS, *MediaShadersPath, TEXT("LinearToPQ_PS"), SF_Pixel);


	///////////////////////////////////////////////////////////////////////////////////
	// PQ-To-Linear
	class FPQToLinearPS : public FGlobalShader
	{
		DECLARE_SHADER_TYPE(FPQToLinearPS, Global);
		SHADER_USE_PARAMETER_STRUCT(FPQToLinearPS, FGlobalShader);

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_TEXTURE(Texture2D, InputTexture)
			SHADER_PARAMETER_SAMPLER(SamplerState, InputSampler)
			RENDER_TARGET_BINDING_SLOTS()
		END_SHADER_PARAMETER_STRUCT()

	public:

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& InParameters)
		{
			// Only compile for desktop-class "PC" shader platforms (filters out mobile and console).
			if (!IsPCPlatform(InParameters.Platform))
			{
				return false;
			}


			return true;
		}

		FParameters* AllocateAndSetParameters(FRDGBuilder& InGraphBuilder, FRDGTextureRef InPQTexture, FRDGTextureRef InOutputLinearTexture)
		{
			FParameters* Parameters = InGraphBuilder.AllocParameters<FParameters>();
			Parameters->InputTexture = InPQTexture;
			Parameters->InputSampler = TStaticSamplerState<>::GetRHI();
			Parameters->RenderTargets[0] = FRenderTargetBinding{ InOutputLinearTexture, ERenderTargetLoadAction::ENoAction };
			return Parameters;
		}
	};

	IMPLEMENT_SHADER_TYPE(, FPQToLinearPS, *MediaShadersPath, TEXT("PQToLinear_PS"), SF_Pixel);


	/** Template function to handle both Linear-PQ and PQ-Linear conversions */
	template<typename TPixelShaderType>
	void AddPQPass(FRDGBuilder& GraphBuilder, const FString& PassName, const FDisplayClusterShaderParameters_MediaPQ& Parameters)
	{
		// Rectangle area to use from source
		const FIntRect ViewRect(Parameters.InputRect);

		//Dummy ViewFamily/ViewInfo created to use built in Draw Screen/Texture Pass
		const FSceneViewFamily ViewFamily(FSceneViewFamily::ConstructionValues(nullptr, nullptr, FEngineShowFlags(ESFIM_Game)).SetTime(FGameTime()));
		FSceneViewInitOptions ViewInitOptions;
		ViewInitOptions.ViewFamily = &ViewFamily;
		ViewInitOptions.SetViewRectangle(ViewRect);
		ViewInitOptions.ViewOrigin = FVector::ZeroVector;
		ViewInitOptions.ViewRotationMatrix = FMatrix::Identity;
		ViewInitOptions.ProjectionMatrix = FMatrix::Identity;
		FSceneView ViewInfo(ViewInitOptions);

		const FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
		const TShaderMapRef<FScreenPassVS> VertexShader(GlobalShaderMap);
		const TShaderMapRef<TPixelShaderType> PixelShader(GlobalShaderMap);
		class TPixelShaderType::FParameters* PixelShaderParameters = PixelShader->AllocateAndSetParameters(GraphBuilder, Parameters.InputTexture, Parameters.OutputTexture);

		AddDrawScreenPass(
			GraphBuilder,
			FRDGEventName(*PassName),
			ViewInfo,
			FScreenPassTextureViewport{ Parameters.OutputTexture, Parameters.OutputRect },
			FScreenPassTextureViewport{ Parameters.InputTexture,  Parameters.InputRect },
			VertexShader,
			PixelShader,
			PixelShaderParameters
		);
	}
}

void FDisplayClusterShadersMedia::AddLinearToPQPass(FRDGBuilder& GraphBuilder, const FDisplayClusterShaderParameters_MediaPQ& Parameters)
{
	using namespace UE::DisplayClusterShaders::Private;

	// Add Linear-To-PQ pass
	AddPQPass<FLinearToPQPS>(GraphBuilder, TEXT("FDCShadersMedia::LinearToPQ"), Parameters);
}

void FDisplayClusterShadersMedia::AddPQToLinearPass(FRDGBuilder& GraphBuilder, const FDisplayClusterShaderParameters_MediaPQ& Parameters)
{
	using namespace UE::DisplayClusterShaders::Private;

	// Add Pq-To-Linear pass
	AddPQPass<FPQToLinearPS>(GraphBuilder, TEXT("FDCShadersMedia::PQToLinear"), Parameters);
}
