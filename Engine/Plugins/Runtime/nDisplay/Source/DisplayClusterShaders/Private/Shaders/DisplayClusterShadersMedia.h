// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FRDGBuilder;
struct FDisplayClusterShaderParameters_MediaPQ;


/**
 * API for media shaders
 */
class FDisplayClusterShadersMedia
{
public:

	/** Adds Linear-To-PQ encoding pass (API wrapper) */
	static void AddLinearToPQPass(FRDGBuilder& GraphBuilder, const FDisplayClusterShaderParameters_MediaPQ& Parameters);

	/** Adds PQ-To-Linear decoding pass (API wrapper) */
	static void AddPQToLinearPass(FRDGBuilder& GraphBuilder, const FDisplayClusterShaderParameters_MediaPQ& Parameters);
};
