// Copyright Epic Games, Inc. All Rights Reserved.

#include "Shaders/DisplayClusterShadersOverlay.h"

#include "PostProcess/DrawRectangle.h"
#include "ShaderParameters/DisplayClusterShaderParameters_Overlay.h"

#include "GlobalShader.h"
#include "PixelFormat.h"
#include "PixelShaderUtils.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "RHIResources.h"
#include "RHIStaticStates.h"
#include "SceneView.h"
#include "ScreenPass.h"
#include "TextureResource.h"

#include "DisplayClusterShadersLog.h"


namespace UE::DisplayClusterShaders::Private
{
	static const FString OverlayShadersPath = TEXT("/Plugin/nDisplay/Private/OverlayShaders.usf");

	///////////////////////////////////////////////////////////////////////////////////
	// DrawOverlay PS
	class FDrawOverlayPS : public FGlobalShader
	{
		DECLARE_SHADER_TYPE(FDrawOverlayPS, Global);
		SHADER_USE_PARAMETER_STRUCT(FDrawOverlayPS, FGlobalShader);

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_TEXTURE(Texture2D, TextureBase)
			SHADER_PARAMETER_RDG_TEXTURE(Texture2D, TextureOverlay)
			SHADER_PARAMETER_SAMPLER(SamplerState, SamplerBase)
			SHADER_PARAMETER_SAMPLER(SamplerState, SamplerOverlay)
			RENDER_TARGET_BINDING_SLOTS()
		END_SHADER_PARAMETER_STRUCT()

	public:

		/** A helper function to initialize shader parameters based on the draw request data */
		FParameters* AllocateAndSetParameters(FRDGBuilder& InGraphBuilder, const FDisplayClusterShaderParameters_Overlay& InParameters)
		{
			FParameters* Parameters = InGraphBuilder.AllocParameters<FParameters>();

			// Input
			Parameters->TextureBase    = InParameters.BaseTexture;
			Parameters->TextureOverlay = InParameters.OverlayTexture;
			Parameters->SamplerBase    = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp>::GetRHI();
			Parameters->SamplerOverlay = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp>::GetRHI();

			// Output
			Parameters->RenderTargets[0] = FRenderTargetBinding{ InParameters.OutputTexture, ERenderTargetLoadAction::ENoAction };

			return Parameters;
		}
	};

	IMPLEMENT_SHADER_TYPE(, FDrawOverlayPS, *OverlayShadersPath, TEXT("DrawOverlay_PS"), SF_Pixel);
}

void FDisplayClusterShadersOverlay::AddOverlayBlendingPass(FRDGBuilder& GraphBuilder, const FDisplayClusterShaderParameters_Overlay& InParameters)
{
	using namespace UE::DisplayClusterShaders::Private;

	// Nothing to do if wrong input
	if (!InParameters.IsValidData())
	{
		return;
	}

	// Initialize shaders
	const FGlobalShaderMap* const GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
	const TShaderMapRef<FScreenPassVS>  VertexShader(GlobalShaderMap);
	const TShaderMapRef<FDrawOverlayPS> PixelShader(GlobalShaderMap);

	// Instantiate PS shader params
	FDrawOverlayPS::FParameters* PassParameters = PixelShader->AllocateAndSetParameters(GraphBuilder, InParameters);

	// Add draw pass
	AddDrawScreenPass(
		GraphBuilder,
		RDG_EVENT_NAME("nDisplay.DrawOverlay_PS"),
		FScreenPassViewInfo(),
		FScreenPassTextureViewport(FIntRect({ 0, 0 }, InParameters.OutputTexture->Desc.Extent)),
		FScreenPassTextureViewport(FIntRect({ 0, 0 }, InParameters.BaseTexture->Desc.Extent)),
		VertexShader,
		PixelShader,
		PassParameters
	);
}
