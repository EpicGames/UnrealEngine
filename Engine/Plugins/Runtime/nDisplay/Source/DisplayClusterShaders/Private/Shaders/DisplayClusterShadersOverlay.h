// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FRDGBuilder;
struct FDisplayClusterShaderParameters_Overlay;


/**
 * API for overlay drawing
 */
class FDisplayClusterShadersOverlay
{
public:

	/** Draws overlay on top of the base texture (refer function arguments) */
	static void AddOverlayBlendingPass(FRDGBuilder& GraphBuilder, const FDisplayClusterShaderParameters_Overlay& Parameters);
};
