// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterShadersPostprocess_Blur.h"

#include "CoreMinimal.h"
#include "RHI.h"
#include "RHIResources.h"

#include "RHIStaticStates.h"
#include "RenderResource.h"
#include "ShaderParameters.h"
#include "Shader.h"
#include "GlobalShader.h"
#include "ShaderParameterUtils.h"

#include "TextureResource.h"


#include "CommonRenderResources.h"
#include "PixelShaderUtils.h"

#include "ShaderParameterUtils.h"

#include "Engine/TextureRenderTarget2D.h"

static TAutoConsoleVariable<int32> CVarEnablePostprocessBlur(
	TEXT("nDisplay.render.postprocess.enable_blur"),
	1,
	TEXT("Enable postprocess blur shaders\n")
	TEXT(" 0: disabled\n")
	TEXT(" 1: enabled\n"),
	ECVF_RenderThreadSafe
);

class FDirectProjectionVS : public FGlobalShader
{
	DECLARE_SHADER_TYPE(FDirectProjectionVS, Global);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		// Only compile for desktop-class "PC" shader platforms (filters out mobile and console).
		if (!IsPCPlatform(Parameters.Platform))
		{
			return false;
		}

		return true;
	}

	/** Default constructor. */
	FDirectProjectionVS() = default;

public:
	/** Initialization constructor. */
	FDirectProjectionVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{
	}
};

class FPicpBlurPostProcessPS : public FGlobalShader
{
public:
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FVector2f, SampleOffset)
		SHADER_PARAMETER(int32, KernelRadius)

		SHADER_PARAMETER_TEXTURE(Texture2D, SrcTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, BilinearClampTextureSampler)
	END_SHADER_PARAMETER_STRUCT()

	FPicpBlurPostProcessPS() = default;
	FPicpBlurPostProcessPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		// Only compile for desktop-class "PC" shader platforms (filters out mobile and console).
		if (!IsPCPlatform(Parameters.Platform))
		{
			return false;
		}

		return true;
	}

	static void ModifyCompilationEnvironment(uint32 ShaderType, const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		switch (static_cast<EDisplayClusterShaderParameters_PostprocessBlur>(ShaderType))
		{
		case EDisplayClusterShaderParameters_PostprocessBlur::Dilate:
			OutEnvironment.SetDefine(TEXT("BLUR_DILATE"), true);
			break;

		case EDisplayClusterShaderParameters_PostprocessBlur::Gaussian:
			OutEnvironment.SetDefine(TEXT("BLUR_GAUSSIAN"), true);
			break;

		default:
			break;
		}
	}
};

template<uint32 ShaderType>
class TPicpBlurPostProcessPS : public FPicpBlurPostProcessPS
{
public:
	DECLARE_SHADER_TYPE(TPicpBlurPostProcessPS, Global);
	SHADER_USE_PARAMETER_STRUCT(TPicpBlurPostProcessPS, FPicpBlurPostProcessPS);

	using FParameters = FPicpBlurPostProcessPS::FParameters;

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FPicpBlurPostProcessPS::ModifyCompilationEnvironment(ShaderType, Parameters, OutEnvironment);
	}
};

typedef TPicpBlurPostProcessPS<(int32)EDisplayClusterShaderParameters_PostprocessBlur::Gaussian>   FPicpBlurPostProcessDefaultPS;
typedef TPicpBlurPostProcessPS<(int32)EDisplayClusterShaderParameters_PostprocessBlur::Dilate>     FPicpBlurPostProcessDilatePS;

#define PostProcessShaderFileName TEXT("/Plugin/nDisplay/Private/PicpPostProcessShaders.usf")

IMPLEMENT_SHADER_TYPE(, FDirectProjectionVS, PostProcessShaderFileName, TEXT("DirectProjectionVS"), SF_Vertex);
IMPLEMENT_SHADER_TYPE(template<>, FPicpBlurPostProcessDefaultPS, PostProcessShaderFileName, TEXT("BlurPostProcessPS"), SF_Pixel);
IMPLEMENT_SHADER_TYPE(template<>, FPicpBlurPostProcessDilatePS,  PostProcessShaderFileName, TEXT("BlurPostProcessPS"), SF_Pixel);

template<uint32 ShaderType>
static void PicpBlurPostProcess_RenderThread(
	FRHICommandListImmediate& RHICmdList,
	FRHITexture* InShaderTexture,
	FRHITexture* OutRenderTargetableTexture,
	const FDisplayClusterShaderParameters_PostprocessBlur& InSettings)
{
	check(IsInRenderingThread());

	const ERHIFeatureLevel::Type RenderFeatureLevel = GMaxRHIFeatureLevel;
	const auto GlobalShaderMap = GetGlobalShaderMap(RenderFeatureLevel);

	const FIntPoint TargetSizeXY = InShaderTexture->GetSizeXY();

	TShaderMapRef<FDirectProjectionVS>                VertexShader(GlobalShaderMap);
	TShaderMapRef<TPicpBlurPostProcessPS<ShaderType>> PixelShader(GlobalShaderMap);
	if (!VertexShader.IsValid() || !PixelShader.IsValid())
	{
		// Always check if shaders are available on the current platform and hardware
		return;
	}

	FPicpBlurPostProcessPS::FParameters PsParameters{};
	PsParameters.SrcTexture = InShaderTexture;
	PsParameters.BilinearClampTextureSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	PsParameters.KernelRadius = InSettings.KernelRadius;

	FRHIRenderPassInfo RPInfo(OutRenderTargetableTexture, ERenderTargetActions::DontLoad_Store);
	RHICmdList.Transition(FRHITransitionInfo(OutRenderTargetableTexture, ERHIAccess::Unknown, ERHIAccess::RTV));

	RHICmdList.BeginRenderPass(RPInfo, TEXT("nDisplay_PicpPostProcessBlur"));
	{
		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
		GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
		GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
		GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
		GraphicsPSOInit.PrimitiveType = PT_TriangleList;
		SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

		// Blur X
		PsParameters.SampleOffset = FVector2f(InSettings.KernelScale / TargetSizeXY.X, 0.0f);
		SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), PsParameters);

		FPixelShaderUtils::DrawFullscreenQuad(RHICmdList, 1);
	
		RHICmdList.EndRenderPass();

		RHICmdList.Transition({
			FRHITransitionInfo(OutRenderTargetableTexture, ERHIAccess::RTV, ERHIAccess::CopySrc),
			FRHITransitionInfo(InShaderTexture, ERHIAccess::Unknown, ERHIAccess::CopyDest),
		});

		RHICmdList.CopyTexture(OutRenderTargetableTexture, InShaderTexture, {});

		RHICmdList.Transition({
			FRHITransitionInfo(OutRenderTargetableTexture, ERHIAccess::CopySrc, ERHIAccess::RTV),
			FRHITransitionInfo(InShaderTexture, ERHIAccess::CopyDest, ERHIAccess::SRVMask),
		});

		RHICmdList.BeginRenderPass(RPInfo, TEXT("nDisplay_PicpPostProcessBlur"));

		// Blur Y
		SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

		PsParameters.SampleOffset = FVector2f(0.0f, InSettings.KernelScale / TargetSizeXY.Y);
		SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), PsParameters);

		FPixelShaderUtils::DrawFullscreenQuad(RHICmdList, 1);
	}
	RHICmdList.EndRenderPass();
	RHICmdList.Transition(FRHITransitionInfo(OutRenderTargetableTexture, ERHIAccess::Unknown, ERHIAccess::SRVMask));
}

DECLARE_GPU_STAT_NAMED(nDisplay_Picp_PostProcess_Compose, TEXT("nDisplay Picp_PostProcess::Compose"));
DECLARE_GPU_STAT_NAMED(nDisplay_Picp_PostProcess_Blur, TEXT("nDisplay Picp_PostProcess::Blur"));

bool FDisplayClusterShadersPostprocess_Blur::RenderPostprocess_Blur(FRHICommandListImmediate& RHICmdList, FRHITexture* InSourceTexture, FRHITexture* InRenderTargetableDestTexture, const FDisplayClusterShaderParameters_PostprocessBlur& InSettings)
{
	check(IsInRenderingThread());

	if (CVarEnablePostprocessBlur.GetValueOnRenderThread() == 0)
	{
		// Blur posprocess disabled
		return false;
	}

	RHI_BREADCRUMB_EVENT_STAT(RHICmdList, nDisplay_Picp_PostProcess_Blur, "nDisplay_Picp_PostProcess_Blur");

	switch (InSettings.Mode)
	{
	case EDisplayClusterShaderParameters_PostprocessBlur::Gaussian:
		PicpBlurPostProcess_RenderThread<(int32)EDisplayClusterShaderParameters_PostprocessBlur::Gaussian>(RHICmdList, InSourceTexture, InRenderTargetableDestTexture, InSettings);
		return true;

	case EDisplayClusterShaderParameters_PostprocessBlur::Dilate:
		PicpBlurPostProcess_RenderThread<(int32)EDisplayClusterShaderParameters_PostprocessBlur::Dilate>(RHICmdList, InSourceTexture, InRenderTargetableDestTexture, InSettings);
		return true;

	default:
		break;
	}

	return false;
}
