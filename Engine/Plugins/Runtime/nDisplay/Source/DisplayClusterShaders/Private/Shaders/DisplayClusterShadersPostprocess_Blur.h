// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RHI.h"
#include "RHICommandList.h"

#include "ShaderParameters/DisplayClusterShaderParameters_PostprocessBlur.h"

class FDisplayClusterShadersPostprocess_Blur
{
public:
	static bool RenderPostprocess_Blur(FRHICommandListImmediate& RHICmdList, FRHITexture* InSourceTexture, FRHITexture* InRenderTargetableDestTexture, const FDisplayClusterShaderParameters_PostprocessBlur& InSettings);
};

