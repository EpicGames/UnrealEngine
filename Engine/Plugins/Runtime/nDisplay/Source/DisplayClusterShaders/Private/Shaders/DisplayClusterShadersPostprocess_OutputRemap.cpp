// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterShadersPostprocess_OutputRemap.h"

#include "CoreMinimal.h"
#include "RHI.h"
#include "RHIResources.h"

#include "CommonRenderResources.h"
#include "PixelShaderUtils.h"

#include "ShaderParameterUtils.h"

#include "HAL/IConsoleManager.h"

#include "RenderResource.h"
#include "ShaderParameters.h"
#include "Shader.h"
#include "GlobalShader.h"
#include "ShaderParameterUtils.h"
#include "ShaderParameterStruct.h"

#include "ClearQuad.h"

#include "Render/Containers/IDisplayClusterRender_MeshComponent.h"
#include "Render/Containers/IDisplayClusterRender_MeshComponentProxy.h"
#include "Render/Containers/DisplayClusterRender_MeshGeometry.h"

#include "IDisplayCluster.h"
#include "Render/IDisplayClusterRenderManager.h"

#define OutputRemapShaderFileName TEXT("/Plugin/nDisplay/Private/OutputRemapShaders.usf")

// Select output remap shader
enum class EVarOutputRemapShaderType : uint8
{
	Default,
	Passthrough,
	Disable,
};

static TAutoConsoleVariable<int32> CVarOutputRemapShaderType(
	TEXT("nDisplay.render.output_remap.shader"),
	(int32)EVarOutputRemapShaderType::Default,
	TEXT("Select shader for output remap:\n")	
	TEXT(" 0: default remap shader\n")
	TEXT(" 1: pass throught shader, test rect mesh\n")
	TEXT(" 2: Disable remap shaders\n")
	,ECVF_RenderThreadSafe
);

BEGIN_SHADER_PARAMETER_STRUCT(FOutputRemapVertexShaderParameters, )
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FOutputRemapPixelShaderParameters, )
	SHADER_PARAMETER_TEXTURE(Texture2D,    PostprocessInput0)
	SHADER_PARAMETER_SAMPLER(SamplerState, PostprocessInput0Sampler)
END_SHADER_PARAMETER_STRUCT()

class FOutputRemapPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FOutputRemapPS);
	SHADER_USE_PARAMETER_STRUCT(FOutputRemapPS, FGlobalShader);

	using FParameters = FOutputRemapPixelShaderParameters;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		// Only compile for desktop-class "PC" shader platforms (filters out mobile and console).
		if (!IsPCPlatform(Parameters.Platform))
		{
			return false;
		}

		return true;
	}
};

IMPLEMENT_SHADER_TYPE(, FOutputRemapPS, OutputRemapShaderFileName, TEXT("OutputRemap_PS"), SF_Pixel);

class FOutputRemapVS : public FGlobalShader
	{
public:
	DECLARE_GLOBAL_SHADER(FOutputRemapVS);
	SHADER_USE_PARAMETER_STRUCT(FOutputRemapVS, FGlobalShader);

	using FParameters = FOutputRemapVertexShaderParameters;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		// Only compile for desktop-class "PC" shader platforms (filters out mobile and console).
		if (!IsPCPlatform(Parameters.Platform))
		{
			return false;
		}

		return true;
	}
};

IMPLEMENT_SHADER_TYPE(, FOutputRemapVS, OutputRemapShaderFileName, TEXT("OutputRemap_VS"), SF_Vertex);

DECLARE_GPU_STAT_NAMED(nDisplay_PostProcess_OutputRemap, TEXT("nDisplay PostProcess::OutputRemap"));

bool FDisplayClusterShadersPostprocess_OutputRemap::RenderPostprocess_OutputRemap(FRHICommandListImmediate& RHICmdList, FRHITexture* InSourceTexture, FRHITexture* InRenderTargetableDestTexture, const IDisplayClusterRender_MeshComponentProxy& InMeshProxy)
{
	check(IsInRenderingThread());

	if (InSourceTexture == nullptr || InRenderTargetableDestTexture == nullptr)
	{
		return false;
	}

	IDisplayClusterRender_MeshComponentProxy const* MeshProxy = &InMeshProxy;

	const EVarOutputRemapShaderType ShaderType = (EVarOutputRemapShaderType)CVarOutputRemapShaderType.GetValueOnAnyThread();
	switch (ShaderType)
	{
		case EVarOutputRemapShaderType::Passthrough:
		{
			// Use simple 1:1 test mesh for shader forwarding
			static TSharedPtr<IDisplayClusterRender_MeshComponent, ESPMode::ThreadSafe> TestMesh_Passthrough = IDisplayCluster::Get().GetRenderMgr()->CreateMeshComponent();

			if (TestMesh_Passthrough.IsValid())
			{
				IDisplayClusterRender_MeshComponentProxy* TestMeshMeshComponentProxy = TestMesh_Passthrough->GetMeshComponentProxy_RenderThread();
				if (TestMeshMeshComponentProxy != nullptr)
				{
					if (!TestMeshMeshComponentProxy->IsEnabled_RenderThread())
					{
						// Initialize once:
						FDisplayClusterRender_MeshGeometry PassthroughMeshGeometry(EDisplayClusterRender_MeshGeometryCreateType::Passthrough);
						TestMesh_Passthrough->AssignMeshGeometry_RenderThread(&PassthroughMeshGeometry, EDisplayClusterRender_MeshComponentProxyDataFunc::OutputRemapScreenSpace);
					}

					MeshProxy = TestMeshMeshComponentProxy;
				}
			}

			break;
		}

		case EVarOutputRemapShaderType::Default:
			break;

		case EVarOutputRemapShaderType::Disable:
		default:
			return false;
	};

	bool bResult = false;

	RHI_BREADCRUMB_EVENT_STAT(RHICmdList, nDisplay_PostProcess_OutputRemap, "nDisplay_PostProcess_OutputRemap");

	FRHIRenderPassInfo RPInfo(InRenderTargetableDestTexture, ERenderTargetActions::Load_Store);
	RHICmdList.Transition(FRHITransitionInfo(InRenderTargetableDestTexture, ERHIAccess::Unknown, ERHIAccess::RTV));

	RHICmdList.BeginRenderPass(RPInfo, TEXT("nDisplay_OutputRemap"));
	{
		const FIntPoint TargetSizeXY = InRenderTargetableDestTexture->GetSizeXY();
		RHICmdList.SetViewport(0, 0, 0.0f, TargetSizeXY.X, TargetSizeXY.Y, 1.0f);

		const ERHIFeatureLevel::Type RenderFeatureLevel = GMaxRHIFeatureLevel;
		const auto GlobalShaderMap = GetGlobalShaderMap(RenderFeatureLevel);

		TShaderMapRef<FOutputRemapVS> VertexShader(GlobalShaderMap);
		TShaderMapRef<FOutputRemapPS> PixelShader(GlobalShaderMap);
		if (!VertexShader.IsValid() || !PixelShader.IsValid())
		{
			// Always check if shaders are available on the current platform and hardware
			return false;
		}

		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		// Set the graphic pipeline state.
		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

		if (MeshProxy->BeginRender_RenderThread(RHICmdList, GraphicsPSOInit))
		{
			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
			GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();

			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();

			GraphicsPSOInit.PrimitiveType = PT_TriangleList;
			GraphicsPSOInit.BlendState = TStaticBlendState <>::GetRHI();
			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

			FOutputRemapPixelShaderParameters PSParameters;
			{
				PSParameters.PostprocessInput0 = InSourceTexture;
				PSParameters.PostprocessInput0Sampler = TStaticSamplerState<SF_Trilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
			}
			SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), PSParameters);

			MeshProxy->FinishRender_RenderThread(RHICmdList);
			bResult = true;
		}
	}
	RHICmdList.EndRenderPass();
	RHICmdList.Transition(FRHITransitionInfo(InRenderTargetableDestTexture, ERHIAccess::Unknown, ERHIAccess::SRVMask));

	return bResult;
}
