// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RHI.h"
#include "RHICommandList.h"

class IDisplayClusterRender_MeshComponentProxy;

class FDisplayClusterShadersPostprocess_OutputRemap
{
public:
	static bool RenderPostprocess_OutputRemap(FRHICommandListImmediate& RHICmdList, FRHITexture* InSourceTexture, FRHITexture* InRenderTargetableDestTexture, const IDisplayClusterRender_MeshComponentProxy& MeshProxy);
};
