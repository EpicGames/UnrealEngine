// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterShadersPreprocess_UVLightCards.h"
#include "Render/Viewport/IDisplayClusterViewportManager.h"

#include "Components/PrimitiveComponent.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "EngineModule.h"
#include "EngineUtils.h"
#include "InstanceCulling/InstanceCullingContext.h"
#include "MeshMaterialShader.h"
#include "MeshPassProcessor.h"
#include "MeshPassProcessor.inl"
#include "PrimitiveSceneInfo.h"
#include "PrimitiveSceneProxy.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphResources.h"
#include "SceneInterface.h"
#include "SceneRendererInterface.h"
#include "SceneTexturesConfig.h"
#include "ScreenPass.h"
#include "ShaderParameters/DisplayClusterShaderParameters_UVLightCards.h"
#include "UnrealClient.h"

class FUVLightCardVS : public FMeshMaterialShader
{
public:
	DECLARE_SHADER_TYPE(FUVLightCardVS, MeshMaterial);

	FUVLightCardVS() { }
	FUVLightCardVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FMeshMaterialShader(Initializer)
	{
		PassUniformBuffer.Bind(Initializer.ParameterMap, FSceneTextureUniformParameters::FTypeInfo::GetStructMetadata()->GetShaderVariableName());
	}

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		// Only compile for desktop-class "PC" shader platforms (filters out mobile and console).
		if (!IsPCPlatform(Parameters.Platform))
		{
			return false;
		}

		return Parameters.VertexFactoryType == FindVertexFactoryType(TEXT("FLocalVertexFactory"));
	}
};

IMPLEMENT_MATERIAL_SHADER_TYPE(, FUVLightCardVS, TEXT("/Plugin/nDisplay/Private/UVLightCardShaders.usf"), TEXT("MainVS"), SF_Vertex);

class FUVLightCardPS : public FMeshMaterialShader
{
public:
	DECLARE_SHADER_TYPE(FUVLightCardPS, MeshMaterial);

	FUVLightCardPS() { }
	FUVLightCardPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FMeshMaterialShader(Initializer)
	{
		PassUniformBuffer.Bind(Initializer.ParameterMap, FSceneTextureUniformParameters::FTypeInfo::GetStructMetadata()->GetShaderVariableName());
	}

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		// Only compile for desktop-class "PC" shader platforms (filters out mobile and console).
		if (!IsPCPlatform(Parameters.Platform))
		{
			return false;
		}

		return Parameters.VertexFactoryType == FindVertexFactoryType(TEXT("FLocalVertexFactory"));
	}
};

IMPLEMENT_MATERIAL_SHADER_TYPE(, FUVLightCardPS, TEXT("/Plugin/nDisplay/Private/UVLightCardShaders.usf"), TEXT("MainPS"), SF_Pixel);

class FUVLightCardPassProcessor : public FMeshPassProcessor
{
	/**
	* Returns the render scene associated with the given view.
	* May return nullptr if the view or its family/scene is invalid.
	*/
	static const FScene* GetRenderSceneFromView(const FSceneView* InView)
	{
		if (InView && InView->Family && InView->Family->Scene)
		{
			return InView->Family->Scene->GetRenderScene();
		}

		return nullptr;
	}

public:
	FUVLightCardPassProcessor(const FSceneView* InView, FMeshPassDrawListContext* InDrawListContext)
		: FMeshPassProcessor(
			EMeshPass::Num, 
			GetRenderSceneFromView(InView),
			InView ? InView->GetFeatureLevel() : GMaxRHIFeatureLevel,
			InView,
			InDrawListContext)
	{ }

	virtual void AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId = -1) override final
	{
		const FMaterialRenderProxy* FallbackMaterialRenderProxy = nullptr;
		const FMaterial& Material = MeshBatch.MaterialRenderProxy->GetMaterialWithFallback(FeatureLevel, FallbackMaterialRenderProxy);
		const FMaterialRenderProxy& MaterialRenderProxy = FallbackMaterialRenderProxy ? *FallbackMaterialRenderProxy : *MeshBatch.MaterialRenderProxy;

		const FVertexFactory* VertexFactory = MeshBatch.VertexFactory;

		TMeshProcessorShaders<FUVLightCardVS, FUVLightCardPS> PassShaders;

		FMaterialShaderTypes ShaderTypes;
		ShaderTypes.AddShaderType<FUVLightCardVS>();
		ShaderTypes.AddShaderType<FUVLightCardPS>();

		FMaterialShaders Shaders;
		if (!Material.TryGetShaders(ShaderTypes, VertexFactory->GetType(), Shaders))
		{
			return;
		}

		if (!Shaders.TryGetVertexShader(PassShaders.VertexShader) || !Shaders.TryGetPixelShader(PassShaders.PixelShader))
		{
			// Always check if shaders are available on the current platform and hardware
			return;
		}

		const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(MeshBatch);
		const ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(Material, OverrideSettings);
		const ERasterizerCullMode MeshCullMode = ComputeMeshCullMode(Material, OverrideSettings);

		FMeshPassProcessorRenderState DrawRenderState;
		{
			DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<false, CF_Always>::GetRHI());

			const EBlendMode BlendMode = Material.GetBlendMode();
			const bool bIsTranslucent = IsTranslucentBlendMode(BlendMode);

			// Substrate materials always use premultiplied alpha for translucent rendering.
			// Apply a custom BlendState so SrcColor is not multiplied by alpha again.
			if (bIsTranslucent && Material.IsSubstrateMaterial())
			{
				// Substrate translucent blending:
				// Output = SrcColor + DstColor * (1 - SrcAlpha)
				DrawRenderState.SetBlendState(
					TStaticBlendState<CW_RGBA,
					BO_Add, BF_One, BF_InverseSourceAlpha,
					BO_Add, BF_One, BF_InverseSourceAlpha>::GetRHI());
			}
			else
			{
				// Standard alpha blending:
				// Output = SrcColor * SrcAlpha + DstColor * (1 - SrcAlpha)
				DrawRenderState.SetBlendState(
					TStaticBlendState<CW_RGBA,
					BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha,
					BO_Add, BF_One, BF_InverseSourceAlpha>::GetRHI());
			}
		}

		FMeshMaterialShaderElementData ShaderElementData;
		ShaderElementData.InitializeMeshMaterialData(ViewIfDynamicMeshCommand, PrimitiveSceneProxy, MeshBatch, StaticMeshId, false);

		FMeshDrawCommandSortKey SortKey = CreateMeshSortKey(MeshBatch, PrimitiveSceneProxy, Material, PassShaders.VertexShader.GetShader(), PassShaders.PixelShader.GetShader());

		BuildMeshDrawCommands(
			MeshBatch,
			BatchElementMask,
			PrimitiveSceneProxy,
			MaterialRenderProxy,
			Material,
			DrawRenderState,
			PassShaders,
			MeshFillMode,
			MeshCullMode,
			SortKey,
			EMeshPassFeatures::Default,
			ShaderElementData);
	}

protected:
	virtual FMeshDrawCommandSortKey CreateMeshSortKey(const FMeshBatch& RESTRICT MeshBatch,
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
	const FMaterial& Material,
	const FMeshMaterialShader* VertexShader,
	const FMeshMaterialShader* PixelShader)
	{
		FMeshDrawCommandSortKey SortKey = FMeshDrawCommandSortKey::Default;
		
		uint16 SortKeyPriority = 0;
		float Distance = 0.0f;

		if (PrimitiveSceneProxy)
		{
			const FPrimitiveSceneInfo* PrimitiveSceneInfo = PrimitiveSceneProxy->GetPrimitiveSceneInfo();
			SortKeyPriority = (uint16)((int32)PrimitiveSceneInfo->Proxy->GetTranslucencySortPriority() - (int32)SHRT_MIN);

			// Use the standard sort by distance method for translucent objects
			const float DistanceOffset = PrimitiveSceneInfo->Proxy->GetTranslucencySortDistanceOffset();
			const FVector BoundsOrigin = PrimitiveSceneProxy->GetBounds().Origin;
			const FVector ViewOrigin = this->ViewIfDynamicMeshCommand->ViewMatrices.GetViewOrigin();
			Distance = (BoundsOrigin - ViewOrigin).Size() + DistanceOffset;
		}

		SortKey.Translucent.MeshIdInPrimitive = MeshBatch.MeshIdInPrimitive;
		SortKey.Translucent.Priority = SortKeyPriority;
		SortKey.Translucent.Distance = (uint32)~BitInvertIfNegativeFloat(*(uint32*)&Distance);

		return SortKey;
	}

private:
	/** Inverts the bits of the floating point number if that number is negative */
	uint32 BitInvertIfNegativeFloat(uint32 FloatBit)
	{
		unsigned Mask = -int32(FloatBit >> 31) | 0x80000000;
		return FloatBit ^ Mask;
	}
};

BEGIN_SHADER_PARAMETER_STRUCT(FUVLightCardPassParameters, )
	SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FInstanceCullingGlobalUniforms, InstanceCulling)
	SHADER_PARAMETER_STRUCT_INCLUDE(FInstanceCullingDrawParams, InstanceCullingDrawParams)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

class FGammaCorrectionPS : public FGlobalShader
{
	DECLARE_SHADER_TYPE(FGammaCorrectionPS, Global);
	SHADER_USE_PARAMETER_STRUCT(FGammaCorrectionPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, InputTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, InputSampler)
		RENDER_TARGET_BINDING_SLOTS()
		END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FGammaCorrectionPS, "/Plugin/nDisplay/Private/UVLightCardGammaCorrection.usf", "GammaCorrectionPS", SF_Pixel);

DECLARE_GPU_STAT_NAMED(nDisplay_UVLightCards_Render, TEXT("nDisplay UVLightCards::Render"));

bool FDisplayClusterShadersPreprocess_UVLightCards::RenderPreprocess_UVLightCards(FRHICommandListImmediate& RHICmdList, FSceneInterface* InScene, FRenderTarget* InRenderTarget, const FDisplayClusterShaderParameters_UVLightCards& InParameters)
{
	check(IsInRenderingThread());

	if (InRenderTarget == nullptr || InParameters.PrimitivesToRender.IsEmpty())
	{
		return false;
	}

	RHI_BREADCRUMB_EVENT_STAT(RHICmdList, nDisplay_UVLightCards_Render, "nDisplay_UVLightCards_Render");

	FRDGBuilder GraphBuilder(RHICmdList);

	FEngineShowFlags EngineShowFlags(ESFIM_Game);
	IDisplayClusterViewportManager::SetupEngineShowFlags(EDisplayClusterViewportCaptureMode::Lightcard, EngineShowFlags);

	FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
		InRenderTarget,
		InScene,
		EngineShowFlags)
		.SetResolveScene(true)
		.SetTime(FGameTime::GetTimeSinceAppStart()));

	FScenePrimitiveRenderingContextScopeHelper ScenePrimitiveRenderingContextScopeHelper(GetRendererModule().BeginScenePrimitiveRendering(GraphBuilder, &ViewFamily));

	FSceneViewInitOptions ViewInitOptions;

	ViewInitOptions.ViewFamily = &ViewFamily;
	ViewInitOptions.ViewLocation = FVector::ZeroVector;
	ViewInitOptions.ViewRotation = FRotator::ZeroRotator;
	ViewInitOptions.ViewOrigin = ViewInitOptions.ViewLocation;

	ViewInitOptions.SetViewRectangle(FIntRect(FIntPoint(0, 0), InRenderTarget->GetSizeXY()));

	ViewInitOptions.ViewRotationMatrix = FInverseRotationMatrix(ViewInitOptions.ViewRotation) * FMatrix(
		FPlane(0, 0, 1, 0),
		FPlane(1, 0, 0, 0),
		FPlane(0, 1, 0, 0),
		FPlane(0, 0, 0, 1));

	const float ZScale = 0.5f / UE_OLD_HALF_WORLD_MAX;
	const float ZOffset = UE_OLD_HALF_WORLD_MAX;
	const float OrthoSize = 0.5f * InParameters.ProjectionPlaneSize;

	ViewInitOptions.ProjectionMatrix = FReversedZOrthoMatrix(
		OrthoSize,
		OrthoSize,
		ZScale,
		ZOffset
	);

	ViewInitOptions.BackgroundColor = FLinearColor::Black;

	GetRendererModule().CreateAndInitSingleView(RHICmdList, &ViewFamily, &ViewInitOptions);
	FSceneView* View = (FSceneView*)ViewFamily.Views[0];

	ViewFamily.SceneCaptureCompositeMode = ESceneCaptureCompositeMode::SCCM_Overwrite;

	// Use the same render settings as for the 3D LC.
	IDisplayClusterViewportManager::SetupSceneView(EDisplayClusterViewportCaptureMode::Lightcard, ViewFamily, *View);

	FRDGTextureRef OutputTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(InRenderTarget->GetRenderTargetTexture(), TEXT("UVLightCardRenderTarget")));
	FRenderTargetBinding OutputRenderTargetBinding(OutputTexture, ERenderTargetLoadAction::EClear);

	// Intermediate render target for the raw color output of the light cards
	FRDGTextureRef ColorTexture = GraphBuilder.CreateTexture(OutputTexture->Desc, TEXT("DisplayClusterUVLightCards.ColorTexture"));
	FRenderTargetBinding ColorRenderTargetBinding(ColorTexture, ERenderTargetLoadAction::EClear);

	// First, render all the UV light cards to a plane which represents the UV plane for the stage
	FUVLightCardPassParameters* PassParameters = GraphBuilder.AllocParameters<FUVLightCardPassParameters>();
	PassParameters->View = View->ViewUniformBuffer;
	PassParameters->Scene = GetSceneUniformBufferRef(GraphBuilder, *View);
	PassParameters->InstanceCulling = FInstanceCullingContext::CreateDummyInstanceCullingUniformBuffer(GraphBuilder);
	PassParameters->RenderTargets[0] = ColorRenderTargetBinding;

	GraphBuilder.AddPass(RDG_EVENT_NAME("DisplayClusterUVLightCards::Render"),
		PassParameters,
		ERDGPassFlags::Raster | ERDGPassFlags::NeverCull,
		[View, &InParameters](FRHICommandList& RHICmdList)
		{
			FIntRect ViewRect = View->UnscaledViewRect;
			RHICmdList.SetViewport(ViewRect.Min.X, ViewRect.Min.Y, 0.0f, ViewRect.Max.X, ViewRect.Max.Y, 1.0f);

			DrawDynamicMeshPass(*View, RHICmdList, [View, &InParameters](FDynamicPassMeshDrawListContext* DynamicMeshPassContext)
			{
				FUVLightCardPassProcessor MeshPassProcessor(View, DynamicMeshPassContext);
				for (FPrimitiveSceneProxy* SceneProxy : InParameters.PrimitivesToRender)
				{
					if (const FMeshBatch* MeshBatch = SceneProxy->GetPrimitiveSceneInfo()->GetMeshBatch(SceneProxy->GetPrimitiveSceneInfo()->StaticMeshes.Num() - 1))
					{
						MeshBatch->MaterialRenderProxy->UpdateUniformExpressionCacheIfNeeded(View->GetFeatureLevel());

						const uint64 BatchElementMask = ~0ull;
						MeshPassProcessor.AddMeshBatch(*MeshBatch, BatchElementMask, SceneProxy);
					}
				}
			});
		});

	// Now gamma correct the UV light card render
	FGammaCorrectionPS::FParameters* GammaCorrectionParameters = GraphBuilder.AllocParameters<FGammaCorrectionPS::FParameters>();
	GammaCorrectionParameters->InputTexture = ColorTexture;
	GammaCorrectionParameters->InputSampler = TStaticSamplerState<>::GetRHI();
	GammaCorrectionParameters->RenderTargets[0] = OutputRenderTargetBinding;

	FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(View->FeatureLevel);
	TShaderMapRef<FScreenPassVS> ScreenPassVS(GlobalShaderMap);
	TShaderMapRef<FGammaCorrectionPS> GammaCorrectionPS(GlobalShaderMap);
	if (!ScreenPassVS.IsValid() || !GammaCorrectionPS.IsValid())
	{
		// Always check if shaders are available on the current platform and hardware
		return false;
	}

	AddDrawScreenPass(
		GraphBuilder,
		RDG_EVENT_NAME("DisplayClusterUVLightCards::GammaCorrection"),
		FScreenPassViewInfo(*View),
		FScreenPassTextureViewport(OutputTexture),
		FScreenPassTextureViewport(ColorTexture),
		ScreenPassVS,
		GammaCorrectionPS,
		GammaCorrectionParameters);

	GraphBuilder.Execute();

	return true;
}
