// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RHICommandList.h"

struct FDisplayClusterShaderParameters_UVLightCards;
class FSceneInterface;
class FRenderTarget;

class FDisplayClusterShadersPreprocess_UVLightCards
{
public:
	static bool RenderPreprocess_UVLightCards(FRHICommandListImmediate& RHICmdList, FSceneInterface* InScene, FRenderTarget* InRenderTarget, const FDisplayClusterShaderParameters_UVLightCards& InParameters);
};