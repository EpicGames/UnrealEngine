// Copyright Epic Games, Inc. All Rights Reserved.

#include "Shaders/DisplayClusterShadersTransformTexture.h"
#include "ShaderParameters/DisplayClusterShaderParameters_TransformTexture.h"

#include "GlobalShader.h"
#include "PixelFormat.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphDefinitions.h"
#include "RenderGraphUtils.h"
#include "RHIDefinitions.h"
#include "ScreenPass.h"
#include "Shader.h"
#include "ShaderParameterStruct.h"


namespace UE::DisplayClusterShaders::Private
{
	/**
	 * Shaders file path
	 */
	static const TCHAR* TransformTextureShadersFilePath = TEXT("/Plugin/nDisplay/Private/TransformTexture.usf");

	/**
	 * TransformTexture pixel shader implementation
	 */
	class FTransformTexturePS : public FGlobalShader
	{
		DECLARE_SHADER_TYPE(FTransformTexturePS, Global);
		SHADER_USE_PARAMETER_STRUCT(FTransformTexturePS, FGlobalShader);

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_TEXTURE(Texture2D, Texture)
			SHADER_PARAMETER_SAMPLER(SamplerState, Sampler)
			SHADER_PARAMETER(FVector4f, TransformMatrix)
			RENDER_TARGET_BINDING_SLOTS()
		END_SHADER_PARAMETER_STRUCT()

	public:

		/** A helper function to initialize shader parameters based on the draw request data */
		FParameters* AllocateAndSetParameters(FRDGBuilder& InGraphBuilder, const FDisplayClusterShaderParameters_TransformTexture& InParameters)
		{
			FParameters* Parameters = InGraphBuilder.AllocParameters<FParameters>();

			// Input
			Parameters->Texture = InParameters.InputTexture;
			Parameters->Sampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp>::GetRHI();
			Parameters->TransformMatrix = FDisplayClusterShadersTransformTexture::GetTransformationMatrix(InParameters);

			// Output
			Parameters->RenderTargets[0] = FRenderTargetBinding{ InParameters.OutputTexture, ERenderTargetLoadAction::ENoAction };

			return Parameters;
		}
	};

	IMPLEMENT_SHADER_TYPE(, FTransformTexturePS, TransformTextureShadersFilePath, TEXT("TransformTexture_PS"), SF_Pixel);
}

void FDisplayClusterShadersTransformTexture::AddTransformTexturePass(FRDGBuilder& GraphBuilder, const FDisplayClusterShaderParameters_TransformTexture& InParameters)
{
	using namespace UE::DisplayClusterShaders::Private;

	// Nothing to do if wrong input
	if (!InParameters.IsValidData())
	{
		return;
	}

	// Nothing to do if no transformation requested
	if (InParameters.TranformationType == FDisplayClusterShaderParameters_TransformTexture::ETranformation::None)
	{
		return;
	}

	// Create output texture if not provided
	if (!InParameters.OutputTexture)
	{
		CreateOutputTexture(GraphBuilder, InParameters);
	}

	// Make sure the output texture has proper size
	{
		const FIntPoint ExpectedSize = FDisplayClusterShadersTransformTexture::GetOutputTextureSize(InParameters);
		if (!InParameters.OutputTexture || (InParameters.OutputTexture->Desc.Extent != ExpectedSize))
		{
			return;
		}
	}

	// Initialize shaders
	const FGlobalShaderMap* const GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
	const TShaderMapRef<FScreenPassVS> VertexShader(GlobalShaderMap);
	const TShaderMapRef<FTransformTexturePS> PixelShader(GlobalShaderMap);

	// Instantiate PS shader params
	FTransformTexturePS::FParameters* const PassParameters = PixelShader->AllocateAndSetParameters(GraphBuilder, InParameters);

	// Choose input region
	const FIntRect InputViewportRect = InParameters.InputRegion.IsSet()
		? InParameters.InputRegion.GetValue() // explicit subrect
		: FIntRect{ FIntPoint::ZeroValue, InParameters.InputTexture->Desc.Extent }; // whole input texture

	// Add draw pass
	AddDrawScreenPass(
		GraphBuilder,
		RDG_EVENT_NAME("nDisplay.TransformTexture_PS"),
		FScreenPassViewInfo(),
		FScreenPassTextureViewport(InParameters.OutputTexture, FIntRect({ 0, 0 }, InParameters.OutputTexture->Desc.Extent)),
		FScreenPassTextureViewport(InParameters.InputTexture,  InputViewportRect),
		VertexShader,
		PixelShader,
		PassParameters
	);
}

void FDisplayClusterShadersTransformTexture::CreateOutputTexture(FRDGBuilder& GraphBuilder, const FDisplayClusterShaderParameters_TransformTexture& Parameters)
{
	// Creation parameters
	const TCHAR* const Name = TEXT("nD.TranslateTexture.Output");
	const FIntPoint Size = GetOutputTextureSize(Parameters);
	const EPixelFormat PixelFormat = Parameters.InputTexture->Desc.Format;
	const ETextureCreateFlags CreateFlags = TexCreate_ShaderResource | TexCreate_RenderTargetable;
	const FClearValueBinding ClearValueBinding = FClearValueBinding::None;

	// Create texture
	const FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(Size, PixelFormat, ClearValueBinding, CreateFlags);
	Parameters.OutputTexture = GraphBuilder.CreateTexture(Desc, Name);
}

FIntPoint FDisplayClusterShadersTransformTexture::GetOutputTextureSize(const FDisplayClusterShaderParameters_TransformTexture& Parameters)
{
	// Is explicit region set?
	const FIntPoint InputSize = Parameters.InputRegion.IsSet()
		? FIntPoint{ Parameters.InputRegion.GetValue().Width(), Parameters.InputRegion.GetValue().Height() }
		: Parameters.InputTexture->Desc.Extent;

	// 90 and 270 rotation swaps the output dimensions
	if (Parameters.TranformationType == FDisplayClusterShaderParameters_TransformTexture::ETranformation::Rotation_90 ||
		Parameters.TranformationType == FDisplayClusterShaderParameters_TransformTexture::ETranformation::Rotation_270)
	{
		return FIntPoint(InputSize.Y, InputSize.X);
	}

	return InputSize;
}

FVector4f FDisplayClusterShadersTransformTexture::GetTransformationMatrix(const FDisplayClusterShaderParameters_TransformTexture& Parameters)
{
	// Here we generate a 4d vector that is treated as a float2x2 matrix in the shader.
	// { v0, v1, v2, v3 } <==> { m00, m01, m10, m11 }
	switch (Parameters.TranformationType)
	{
		case FDisplayClusterShaderParameters_TransformTexture::ETranformation::Rotation_90:
			return FVector4f(0, -1, 1, 0);
		case FDisplayClusterShaderParameters_TransformTexture::ETranformation::Rotation_180:
			return FVector4f(-1, 0, 0, -1);
		case FDisplayClusterShaderParameters_TransformTexture::ETranformation::Rotation_270:
			return FVector4f(0, 1, -1, 0);
		case FDisplayClusterShaderParameters_TransformTexture::ETranformation::Flip_H:
			return FVector4f(-1, 0, 0, 1);
		case FDisplayClusterShaderParameters_TransformTexture::ETranformation::Flip_V:
			return FVector4f(1, 0, 0, -1);

		case FDisplayClusterShaderParameters_TransformTexture::ETranformation::None:
		default:
			return FVector4f(1, 0, 0, 1);
	}
}
