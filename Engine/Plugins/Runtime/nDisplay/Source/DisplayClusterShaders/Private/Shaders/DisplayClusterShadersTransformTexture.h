// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FRDGBuilder;
struct FDisplayClusterShaderParameters_TransformTexture;


/**
 * Encapsulates the texture transformation implementation
 */
class FDisplayClusterShadersTransformTexture
{
public:

	/** Adds a texture transformation pass with the specified parameters */
	static void AddTransformTexturePass(FRDGBuilder& GraphBuilder, const FDisplayClusterShaderParameters_TransformTexture& Parameters);

	/** Creates output texture based on the requested transformation */
	static void CreateOutputTexture(FRDGBuilder& GraphBuilder, const FDisplayClusterShaderParameters_TransformTexture& Parameters);

	/** Finds output texture size according to the input data */
	static FIntPoint GetOutputTextureSize(const FDisplayClusterShaderParameters_TransformTexture& Parameters);

	/** Returns transformation matrix according to the input data */
	static FVector4f GetTransformationMatrix(const FDisplayClusterShaderParameters_TransformTexture& Parameters);
};
