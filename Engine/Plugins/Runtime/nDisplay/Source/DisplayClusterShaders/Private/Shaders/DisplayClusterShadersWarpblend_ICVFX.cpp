// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterShadersWarpblend_ICVFX.h"
#include "DisplayClusterShadersLog.h"

#include "RenderResource.h"
#include "CommonRenderResources.h"
#include "PixelShaderUtils.h"

#include "Shader.h"
#include "GlobalShader.h"

#include "ShaderParameters.h"
#include "ShaderParameterUtils.h"
#include "ShaderParameterStruct.h"
#include "ShaderPermutation.h"

#include "Render/Containers/IDisplayClusterRender_MeshComponentProxy.h"
#include "Render/Containers/IDisplayClusterRender_Texture.h"

#include "ShaderParameters/DisplayClusterShaderParameters_WarpBlend.h"
#include "ShaderParameters/DisplayClusterShaderParameters_ICVFX.h"

#include "IDisplayClusterWarpBlend.h"

#define ICVFX_ShaderFileName "/Plugin/nDisplay/Private/MPCDIOverlayShaders.usf"

enum class EVarIcvfxMPCDIShaderType : uint8
{
	Default = 0,
	DefaultNoBlend,
	Passthrough,
	Disable,
};

static TAutoConsoleVariable<int32> CVarIcvfxMPCDIShaderType(
	TEXT("nDisplay.render.icvfx.shader"),
	(int32)EVarIcvfxMPCDIShaderType::Default,
	TEXT("Select shader for icvfx:\n")
	TEXT(" 0: Warp shader (used by default)\n")
	TEXT(" 1: Warp shader with disabled blend maps\n")
	TEXT(" 2: Passthrough shader\n")
	TEXT(" 4: Disable all warp shaders\n"),
	ECVF_RenderThreadSafe
);

int32 GDisplayClusterShadersICVFXEnableLightCard = 1;
static FAutoConsoleVariableRef CVarDisplayClusterShadersICVFXEnableLightCard(
	TEXT("nDisplay.render.icvfx.LightCard"),
	GDisplayClusterShadersICVFXEnableLightCard,
	TEXT("Enable ICVFX LightCards render (0 - disable).\n"),
	ECVF_RenderThreadSafe
);

int32 GDisplayClusterShadersICVFXEnableUVLightCard = 1;
static FAutoConsoleVariableRef CVarDisplayClusterShadersICVFXEnableUVLightCard(
	TEXT("nDisplay.render.icvfx.UVLightCard"),
	GDisplayClusterShadersICVFXEnableUVLightCard,
	TEXT("Enable ICVFX UVLightCards render (0 - disable).\n"),
	ECVF_RenderThreadSafe
);

int32 GDisplayClusterShadersICVFXOverlapInnerFrustumSoftEdges = 1;
static FAutoConsoleVariableRef CVarDisplayClusterShadersICVFXOverlapInnerFrustumSoftEdges(
	TEXT("nDisplay.render.icvfx.OverlapInnerFrustumSoftEdges"),
	GDisplayClusterShadersICVFXOverlapInnerFrustumSoftEdges,
	TEXT("Enable SoftEdges for Inner Frustum overlap (0 - disable).\n"),
	ECVF_RenderThreadSafe
);

int32 GDisplayClusterShadersICVFXEnableInnerFrustumUnderOverlap = 0;
static FAutoConsoleVariableRef CVarDisplayClusterShadersICVFXEnableInnerFrustumUnderOverlap(
	TEXT("nDisplay.render.icvfx.EnableInnerFrustumUnderOverlap"),
	GDisplayClusterShadersICVFXEnableInnerFrustumUnderOverlap,
	TEXT("Enable Inner Frustum render under overlap area (0 - disable).\n"),
	ECVF_RenderThreadSafe
);

/**
 * ICVFX is rendered in several passes:
 */
enum class EIcvfxPassRenderPass : uint8
{
	None = 0,

	// viewport+overlayUnder+camera1
	Base,

	// Second and next cameras additive passes
	InnerFrustumIterator,

	// LightCard overlay
	LightCardOver,

	// render overlaps for inner cameras
	InnerFrustumChromakeyOverlapIterator,
};

enum class EIcvfxPassRenderState : uint8
{
	None = 0,

	BlendDisabled = 1 << 0,

	LightCardOver  = 1 << 1,
	LightCardUnder = 1 << 2,

	UVLightCardOver   = 1 << 3,
	UVLightCardUnder  = 1 << 4,
	UVLightCardMerged = 1 << 5,

	EnableInnerFrustumChromakeyOverlap = 1 << 6,
};
ENUM_CLASS_FLAGS(EIcvfxPassRenderState);

DECLARE_GPU_STAT_NAMED(nDisplay_IcvfxRenderPass_Base, TEXT("nDisplay Icvfx::RenderPass::Base"));
DECLARE_GPU_STAT_NAMED(nDisplay_IcvfxRenderPass_InnerFrustumIterator, TEXT("nDisplay Icvfx::RenderPass::InnerFrustumIterator"));
DECLARE_GPU_STAT_NAMED(nDisplay_IcvfxRenderPass_LightCardOver, TEXT("nDisplay Icvfx::RenderPass::LightCardOver"));
DECLARE_GPU_STAT_NAMED(nDisplay_IcvfxRenderPass_InnerFrustumChromakeyOverlapIterator, TEXT("nDisplay Icvfx::RenderPass::InnerFrustumChromakeyOverlapIterator"));

enum class EIcvfxShaderType : uint8
{
	Passthrough,  // Viewport frustum (no warpblend, only frustum math)
	WarpAndBlend, // Pure mpcdi warpblend for viewport
	Invalid,
};

/**
* Special enums for the shader parameter 'LightCardMode'
*/
enum class EShaderParameterLightCardMode : uint32
{
	None = 0,
	LIGHTCARD_UNDER = 1 << 0,
	LIGHTCARD_OVER = 1 << 1,
	UVLIGHTCARD_UNDER = 1 << 2,
	UVLIGHTCARD_OVER = 1 << 3,
	UVLIGHTCARD_MERGE = 1 << 4
};
ENUM_CLASS_FLAGS(EShaderParameterLightCardMode);

namespace IcvfxShaderPermutation
{
	// Shared permutation for picp warp
	class FIcvfxShaderViewportInput      : SHADER_PERMUTATION_BOOL("VIEWPORT_INPUT");
	class FIcvfxShaderViewportInputAlpha : SHADER_PERMUTATION_BOOL("VIEWPORT_INPUT_ALPHA");

	class FIcvfxShaderInnerCamera         : SHADER_PERMUTATION_BOOL("INNER_CAMERA");
	class FIcvfxShaderInnerCameraOverlap  : SHADER_PERMUTATION_BOOL("INNER_CAMERA_OVERLAP");

	class FIcvfxShaderChromakey           : SHADER_PERMUTATION_BOOL("CHROMAKEY");
	class FIcvfxShaderChromakeyFrameColor : SHADER_PERMUTATION_BOOL("CHROMAKEYFRAMECOLOR");

	class FIcvfxShaderChromakeyMarker  : SHADER_PERMUTATION_BOOL("CHROMAKEY_MARKER");

	class FIcvfxShaderAlphaMapBlending : SHADER_PERMUTATION_BOOL("ALPHAMAP_BLENDING");
	class FIcvfxShaderBetaMapBlending  : SHADER_PERMUTATION_BOOL("BETAMAP_BLENDING");

	class FIcvfxShaderMeshWarp         : SHADER_PERMUTATION_BOOL("MESH_WARP");

	using FCommonVSDomain = TShaderPermutationDomain<FIcvfxShaderMeshWarp>;

	using FCommonPSDomain = TShaderPermutationDomain<
		FIcvfxShaderViewportInput,
		FIcvfxShaderViewportInputAlpha,

		FIcvfxShaderInnerCamera,
		FIcvfxShaderInnerCameraOverlap,

		FIcvfxShaderChromakey,
		FIcvfxShaderChromakeyFrameColor,
		FIcvfxShaderChromakeyMarker,

		FIcvfxShaderAlphaMapBlending,
		FIcvfxShaderBetaMapBlending,
		FIcvfxShaderMeshWarp
	>;
	
	bool ShouldCompileCommonPSPermutation(const FCommonPSDomain& PermutationVector)
	{
		if (!PermutationVector.Get<FIcvfxShaderMeshWarp>())
		{
			// Chromakey marker require UV_Chromakey (Mesh Warp)
			if (PermutationVector.Get<FIcvfxShaderChromakeyMarker>())
			{
				return false;
			}
		}

		if (PermutationVector.Get<FIcvfxShaderInnerCameraOverlap>())
		{
			// Overlap use chromakeyframecolor as render base
			if (!PermutationVector.Get<FIcvfxShaderChromakeyFrameColor>())
			{
				return false;
			}
		}

		// only one type of chromakey type can by used
		if (PermutationVector.Get<FIcvfxShaderChromakey>() && PermutationVector.Get<FIcvfxShaderChromakeyFrameColor>())
		{
			return false;
		}

		if (!PermutationVector.Get<FIcvfxShaderViewportInput>())
		{
			if (PermutationVector.Get<FIcvfxShaderViewportInputAlpha>())
			{
				return false;
			}
		}

		if (!PermutationVector.Get<FIcvfxShaderInnerCamera>())
		{
			// All innner camera vectors
			if (PermutationVector.Get<FIcvfxShaderInnerCameraOverlap>()
			|| PermutationVector.Get<FIcvfxShaderChromakey>()
			|| PermutationVector.Get<FIcvfxShaderChromakeyFrameColor>()
			|| PermutationVector.Get<FIcvfxShaderChromakeyMarker>())
			{
				return false;
			}
		}

		if (PermutationVector.Get<FIcvfxShaderChromakeyMarker>())
		{
			if (!PermutationVector.Get<FIcvfxShaderChromakey>() && !PermutationVector.Get<FIcvfxShaderChromakeyFrameColor>())
			{
				return false;
			}
		}

		return true;
	}

	bool ShouldCompileCommonVSPermutation(const FCommonVSDomain& PermutationVector)
	{
		return true;
	}
};

BEGIN_SHADER_PARAMETER_STRUCT(FIcvfxVertexShaderParameters, )
	SHADER_PARAMETER(FVector4f, DrawRectanglePosScaleBias)
	SHADER_PARAMETER(FVector4f, DrawRectangleInvTargetSizeAndTextureSize)
	SHADER_PARAMETER(FVector4f, DrawRectangleUVScaleBias)

	SHADER_PARAMETER(FMatrix44f, MeshToStageProjectionMatrix)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FIcvfxPixelShaderParameters, )
	SHADER_PARAMETER_TEXTURE(Texture2D, InputTexture)
	SHADER_PARAMETER_TEXTURE(Texture2D, WarpMapTexture)
	SHADER_PARAMETER_TEXTURE(Texture2D, AlphaMapTexture)
	SHADER_PARAMETER_TEXTURE(Texture2D, BetaMapTexture)
	SHADER_PARAMETER_TEXTURE(Texture2D, InnerCameraTexture)
	SHADER_PARAMETER_TEXTURE(Texture2D, ChromakeyCameraTexture)
	SHADER_PARAMETER_TEXTURE(Texture2D, ChromakeyMarkerTexture)

	SHADER_PARAMETER_TEXTURE(Texture2D, LightCardUnderTexture)
	SHADER_PARAMETER_TEXTURE(Texture2D, LightCardOverTexture)
	SHADER_PARAMETER_TEXTURE(Texture2D, UVLightCardUnderTexture)
	SHADER_PARAMETER_TEXTURE(Texture2D, UVLightCardOverTexture)

	SHADER_PARAMETER_SAMPLER(SamplerState, InputSampler)
	SHADER_PARAMETER_SAMPLER(SamplerState, WarpMapSampler)
	SHADER_PARAMETER_SAMPLER(SamplerState, AlphaMapSampler)
	SHADER_PARAMETER_SAMPLER(SamplerState, BetaMapSampler)
	
	SHADER_PARAMETER_SAMPLER(SamplerState, InnerCameraSampler)
	SHADER_PARAMETER_SAMPLER(SamplerState, ChromakeyCameraSampler)
	SHADER_PARAMETER_SAMPLER(SamplerState, ChromakeyMarkerSampler)

	SHADER_PARAMETER_SAMPLER(SamplerState, LightCardUnderSampler)
	SHADER_PARAMETER_SAMPLER(SamplerState, LightCardOverSampler)
	SHADER_PARAMETER_SAMPLER(SamplerState, UVLightCardUnderSampler)
	SHADER_PARAMETER_SAMPLER(SamplerState, UVLightCardOverSampler)

	SHADER_PARAMETER(FMatrix44f, ViewportTextureProjectionMatrix)
	SHADER_PARAMETER(FMatrix44f, OverlayProjectionMatrix)
	SHADER_PARAMETER(FMatrix44f, InnerCameraProjectionMatrix)

	SHADER_PARAMETER(int32, LightCardMode)
	SHADER_PARAMETER(float, LightCardGamma)

	SHADER_PARAMETER(float, AlphaMapGammaEmbedded)

	SHADER_PARAMETER(int, AlphaMapComponentDepth)
	SHADER_PARAMETER(int, BetaMapComponentDepth)

	SHADER_PARAMETER(FVector4f, InnerCameraSoftEdge)

	SHADER_PARAMETER(FVector4f, InnerCameraBorderColor)
	SHADER_PARAMETER(float, InnerCameraBorderThickness)
	SHADER_PARAMETER(float, InnerCameraFrameAspectRatio)

	SHADER_PARAMETER(FVector4f, ChromakeyColor)
	SHADER_PARAMETER(FVector4f, ChromakeyMarkerColor)

	SHADER_PARAMETER(float, ChromakeyMarkerScale)
	SHADER_PARAMETER(float, ChromakeyMarkerDistance)
	SHADER_PARAMETER(FVector2f, ChromakeyMarkerOffset)

	SHADER_PARAMETER_ARRAY(FMatrix44f, OverlappedInnerCamerasProjectionMatrices, [MAX_INNER_CAMERAS_AMOUNT])
	SHADER_PARAMETER_ARRAY(FVector4f, OverlappedInnerCameraSoftEdges, [MAX_INNER_CAMERAS_AMOUNT])
	SHADER_PARAMETER(int, OverlappedInnerCamerasNum)

END_SHADER_PARAMETER_STRUCT()

class FIcvfxWarpVS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FIcvfxWarpVS);
	SHADER_USE_PARAMETER_STRUCT(FIcvfxWarpVS, FGlobalShader);
		
	using FPermutationDomain = IcvfxShaderPermutation::FCommonVSDomain;
	using FParameters = FIcvfxVertexShaderParameters;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		// Only compile for desktop-class "PC" shader platforms (filters out mobile and console).
		if (!IsPCPlatform(Parameters.Platform))
		{
			return false;
		}

		return IcvfxShaderPermutation::ShouldCompileCommonVSPermutation(FPermutationDomain(Parameters.PermutationId));
	}
};

IMPLEMENT_GLOBAL_SHADER(FIcvfxWarpVS, ICVFX_ShaderFileName, "IcvfxWarpVS", SF_Vertex);

class FIcvfxWarpPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FIcvfxWarpPS);
	SHADER_USE_PARAMETER_STRUCT(FIcvfxWarpPS, FGlobalShader);

	using FPermutationDomain = IcvfxShaderPermutation::FCommonPSDomain;
	using FParameters = FIcvfxPixelShaderParameters;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		// Only compile for desktop-class "PC" shader platforms (filters out mobile and console).
		if (!IsPCPlatform(Parameters.Platform))
		{
			return false;
		}

		return IcvfxShaderPermutation::ShouldCompileCommonPSPermutation(FPermutationDomain(Parameters.PermutationId));
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("MAX_INNER_CAMERAS_AMOUNT"), MAX_INNER_CAMERAS_AMOUNT);
	}
};

IMPLEMENT_GLOBAL_SHADER(FIcvfxWarpPS, ICVFX_ShaderFileName, "IcvfxWarpPS", SF_Pixel);

class FIcvfxPassthroughPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FIcvfxPassthroughPS);
	SHADER_USE_PARAMETER_STRUCT(FIcvfxPassthroughPS, FGlobalShader);
		
	using FParameters = FIcvfxPixelShaderParameters;
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		// Only compile for desktop-class "PC" shader platforms (filters out mobile and console).
		if (!IsPCPlatform(Parameters.Platform))
		{
			return false;
		}

		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("MAX_INNER_CAMERAS_AMOUNT"), MAX_INNER_CAMERAS_AMOUNT);
	}
};

IMPLEMENT_GLOBAL_SHADER(FIcvfxPassthroughPS, ICVFX_ShaderFileName, "Passthrough_PS", SF_Pixel);


struct FIcvfxRenderPassData
{
	IcvfxShaderPermutation::FCommonVSDomain VSPermutationVector;
	FIcvfxVertexShaderParameters            VSParameters;

	IcvfxShaderPermutation::FCommonPSDomain PSPermutationVector;
	FIcvfxPixelShaderParameters             PSParameters;
};

class FIcvfxPassRenderer
{
public:
	FIcvfxPassRenderer(const FDisplayClusterShaderParameters_WarpBlend& InWarpBlendParameters, const FDisplayClusterShaderParameters_ICVFX& InICVFXParameters)
		: WarpBlendParameters(InWarpBlendParameters)
		, ICVFXParameters(InICVFXParameters)
	{
		if (ICVFXParameters.IsLightCardOverUsed() && GDisplayClusterShadersICVFXEnableLightCard)
		{
			EnumAddFlags(RenderState, EIcvfxPassRenderState::LightCardOver);
		}

		if (ICVFXParameters.IsLightCardUnderUsed() && GDisplayClusterShadersICVFXEnableLightCard)
		{
			EnumAddFlags(RenderState, EIcvfxPassRenderState::LightCardUnder);
		}

		if (ICVFXParameters.IsUVLightCardOverUsed() && GDisplayClusterShadersICVFXEnableUVLightCard)
		{
			EnumAddFlags(RenderState, EIcvfxPassRenderState::UVLightCardOver);
		}

		if (ICVFXParameters.IsUVLightCardUnderUsed() && GDisplayClusterShadersICVFXEnableUVLightCard)
		{
			EnumAddFlags(RenderState, EIcvfxPassRenderState::UVLightCardUnder);
		}

		// Merge only when used 2 UVLC textures.
		if (ICVFXParameters.OverrideUVLightCardRenderMode != EDisplayClusterShaderParametersICVFX_LightCardRenderMode::None
			&& EnumHasAllFlags(RenderState, EIcvfxPassRenderState::UVLightCardUnder | EIcvfxPassRenderState::UVLightCardOver))
		{
			EnumAddFlags(RenderState, EIcvfxPassRenderState::UVLightCardMerged);

			// disable overridden use cases
			switch (ICVFXParameters.OverrideUVLightCardRenderMode)
			{
			case EDisplayClusterShaderParametersICVFX_LightCardRenderMode::Under:
				EnumRemoveFlags(RenderState, EIcvfxPassRenderState::UVLightCardOver);
				break;

			case EDisplayClusterShaderParametersICVFX_LightCardRenderMode::Over:
				EnumRemoveFlags(RenderState, EIcvfxPassRenderState::UVLightCardUnder);
				break;

			default:
				break;
			}
		}

		// Camera overlap work only for multi-cameras setup
		if (ICVFXParameters.CameraOverlappingRenderMode != EDisplayClusterShaderParametersICVFX_CameraOverlappingRenderMode::None && ICVFXParameters.IsMultiCamerasUsed())
		{
			EnumAddFlags(RenderState, EIcvfxPassRenderState::EnableInnerFrustumChromakeyOverlap);
		}
	}

private:
	const FDisplayClusterShaderParameters_WarpBlend& WarpBlendParameters;
	const FDisplayClusterShaderParameters_ICVFX& ICVFXParameters;

	EIcvfxShaderType PixelShaderType;

	EIcvfxPassRenderPass  RenderPass = EIcvfxPassRenderPass::None;
	EIcvfxPassRenderState RenderState = EIcvfxPassRenderState::None;
	int32 ActiveCameraIndex = INDEX_NONE;

	FMatrix LocalUVMatrix;

private:

	inline FMatrix GetStereoMatrix() const
	{
		FIntPoint WarpDataSrcSize = WarpBlendParameters.Src.Texture->GetSizeXY();
		FIntPoint WarpDataDstSize = WarpBlendParameters.Dest.Texture->GetSizeXY();

		FMatrix StereoMatrix = FMatrix::Identity;
		StereoMatrix.M[0][0] = float(WarpBlendParameters.Src.Rect.Width()) / float(WarpDataSrcSize.X);
		StereoMatrix.M[1][1] = float(WarpBlendParameters.Src.Rect.Height()) / float(WarpDataSrcSize.Y);
		StereoMatrix.M[3][0] = float(WarpBlendParameters.Src.Rect.Min.X) / float(WarpDataSrcSize.X);
		StereoMatrix.M[3][1] = float(WarpBlendParameters.Src.Rect.Min.Y) / float(WarpDataSrcSize.Y);

		return StereoMatrix;
	}

	inline FIntRect GetViewportRect() const
	{
		const int32 PosX = WarpBlendParameters.Dest.Rect.Min.X;
		const int32 PosY = WarpBlendParameters.Dest.Rect.Min.Y;
		const int32 SizeX = WarpBlendParameters.Dest.Rect.Width();
		const int32 SizeY = WarpBlendParameters.Dest.Rect.Height();

		return FIntRect(FIntPoint(PosX, PosY), FIntPoint(PosX + SizeX, PosY + SizeY));
	}

	inline EIcvfxShaderType GetPixelShaderType()
	{
		if (WarpBlendParameters.WarpInterface.IsValid())
		{
			switch (WarpBlendParameters.WarpInterface->GetWarpProfileType())
			{
			case EDisplayClusterWarpProfileType::warp_2D:
			case EDisplayClusterWarpProfileType::warp_A3D:
				return EIcvfxShaderType::WarpAndBlend;
#if 0
			case EDisplayClusterWarpProfileType::warp_SL:
			case EDisplayClusterWarpProfileType::warp_3D:
#endif
			default:
				return EIcvfxShaderType::Passthrough;
			};
		}
		return EIcvfxShaderType::Invalid;
	}

	inline void EnumAddLightCardModeFlags(FIcvfxRenderPassData& RenderPassData, const EShaderParameterLightCardMode InFlags)
	{
		EShaderParameterLightCardMode NewLightCardMode = (EShaderParameterLightCardMode)RenderPassData.PSParameters.LightCardMode;
		EnumAddFlags(NewLightCardMode, InFlags);
		RenderPassData.PSParameters.LightCardMode = (int32)NewLightCardMode;
	}

	inline bool EnumHasAnyLightCardModeFlags(FIcvfxRenderPassData& RenderPassData, const EShaderParameterLightCardMode InFlags) const
	{
		EShaderParameterLightCardMode LightCardMode = (EShaderParameterLightCardMode)RenderPassData.PSParameters.LightCardMode;
		return EnumHasAnyFlags(LightCardMode, InFlags);
	}

public:

	bool GetViewportParameters(FIcvfxRenderPassData& RenderPassData)
	{
		if (RenderPass == EIcvfxPassRenderPass::Base)
		{
			// Input viewport texture
			// Render only on first pass
			RenderPassData.PSParameters.InputTexture = WarpBlendParameters.Src.Texture;
			RenderPassData.PSParameters.InputSampler = TStaticSamplerState<SF_Trilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

			RenderPassData.PSPermutationVector.Set<IcvfxShaderPermutation::FIcvfxShaderViewportInput>(true);

			if (WarpBlendParameters.bRenderAlphaChannel)
			{
				RenderPassData.PSPermutationVector.Set<IcvfxShaderPermutation::FIcvfxShaderViewportInputAlpha>(true);
			}
		}

		// Vertex shader viewport rect
		FIntPoint WarpDataSrcSize = WarpBlendParameters.Src.Texture->GetSizeXY();
		FIntPoint WarpDataDstSize = WarpBlendParameters.Dest.Texture->GetSizeXY();

		float U = WarpBlendParameters.Src.Rect.Min.X / (float)WarpDataSrcSize.X;
		float V = WarpBlendParameters.Src.Rect.Min.Y / (float)WarpDataSrcSize.Y;
		float USize = WarpBlendParameters.Src.Rect.Width() / (float)WarpDataSrcSize.X;
		float VSize = WarpBlendParameters.Src.Rect.Height() / (float)WarpDataSrcSize.Y;

		RenderPassData.VSParameters.DrawRectanglePosScaleBias = FVector4f(1, 1, 0, 0);
		RenderPassData.VSParameters.DrawRectangleInvTargetSizeAndTextureSize = FVector4f(1, 1, 1, 1);
		RenderPassData.VSParameters.DrawRectangleUVScaleBias = FVector4f(USize, VSize, U, V);

		return true;
	}

	bool GetWarpMapParameters(FIcvfxRenderPassData& RenderPassData)
	{
		RenderPassData.PSParameters.ViewportTextureProjectionMatrix = FMatrix44f(LocalUVMatrix * GetStereoMatrix());

		if (WarpBlendParameters.WarpInterface.IsValid())
		{
			switch (WarpBlendParameters.WarpInterface->GetWarpGeometryType())
			{
				case EDisplayClusterWarpGeometryType::WarpMesh:
				case EDisplayClusterWarpGeometryType::WarpProceduralMesh:
				{
					// Use mesh inseat of warp texture
					RenderPassData.PSPermutationVector.Set<IcvfxShaderPermutation::FIcvfxShaderMeshWarp>(true);
					RenderPassData.VSPermutationVector.Set<IcvfxShaderPermutation::FIcvfxShaderMeshWarp>(true);

					RenderPassData.VSParameters.MeshToStageProjectionMatrix = FMatrix44f(WarpBlendParameters.Context.MeshToStageMatrix);

					break;
				}

				case EDisplayClusterWarpGeometryType::WarpMap:
				{
					FRHITexture* WarpMap = WarpBlendParameters.WarpInterface->GetTexture(EDisplayClusterWarpBlendTextureType::WarpMap);
					if (WarpMap == nullptr)
					{
						return false;
					}
					
					RenderPassData.PSParameters.WarpMapTexture = WarpMap;
					RenderPassData.PSParameters.WarpMapSampler = TStaticSamplerState<SF_Trilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

					break;
				}
			}

			return true;
		}

		return false;
	}

	bool GetWarpBlendParameters(FIcvfxRenderPassData& RenderPassData)
	{
		if (WarpBlendParameters.WarpInterface.IsValid())
		{
			TSharedPtr<IDisplayClusterRender_Texture, ESPMode::ThreadSafe> AlphaMap = WarpBlendParameters.WarpInterface->GetTextureInterface(EDisplayClusterWarpBlendTextureType::AlphaMap);
			if (FRHITexture* AlphaMapTexture = AlphaMap.IsValid() ? AlphaMap->GetRHITexture() : nullptr)
			{
				RenderPassData.PSParameters.AlphaMapTexture = AlphaMapTexture;
				RenderPassData.PSParameters.AlphaMapSampler = TStaticSamplerState<SF_Trilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
				RenderPassData.PSParameters.AlphaMapComponentDepth = AlphaMap->GetComponentDepth();
				RenderPassData.PSParameters.AlphaMapGammaEmbedded = WarpBlendParameters.WarpInterface->GetAlphaMapEmbeddedGamma();

				RenderPassData.PSPermutationVector.Set<IcvfxShaderPermutation::FIcvfxShaderAlphaMapBlending>(true);
			}

			TSharedPtr<IDisplayClusterRender_Texture, ESPMode::ThreadSafe> BetaMap = WarpBlendParameters.WarpInterface->GetTextureInterface(EDisplayClusterWarpBlendTextureType::BetaMap);
			if (FRHITexture* BetaMapTexture = BetaMap.IsValid() ? BetaMap->GetRHITexture() : nullptr)
			{
				RenderPassData.PSParameters.BetaMapTexture = BetaMapTexture;
				RenderPassData.PSParameters.BetaMapSampler = TStaticSamplerState<SF_Trilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
				RenderPassData.PSParameters.BetaMapComponentDepth = BetaMap->GetComponentDepth();

				// Note: The MPCDI 2.0 standard does not define an 'EmbeddedGamma' tag value for the BetaMap tag.
				// However, it does require gamma correction for BetaMap.
				// Therefore, this parameter is also used for the BetaMap texture.
				RenderPassData.PSParameters.AlphaMapGammaEmbedded = WarpBlendParameters.WarpInterface->GetAlphaMapEmbeddedGamma();

				RenderPassData.PSPermutationVector.Set<IcvfxShaderPermutation::FIcvfxShaderBetaMapBlending>(true);
			}

			return true;
		}

		return false;
	}

	void InitializeLightCardParameters(FIcvfxRenderPassData& RenderPassData)
	{
		RenderPassData.PSParameters.LightCardMode = (int32)EShaderParameterLightCardMode::None;

		// Since we do not use a permutation vector for light cards, the pixel shader will always contain code with light card textures.
		// To avoid shader compiler errors, we must set all textures even if they are not used by the logic.

		RenderPassData.PSParameters.LightCardUnderSampler = TStaticSamplerState<SF_Trilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		RenderPassData.PSParameters.LightCardUnderTexture = WarpBlendParameters.Src.Texture;

		RenderPassData.PSParameters.LightCardOverSampler = TStaticSamplerState<SF_Trilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		RenderPassData.PSParameters.LightCardOverTexture = WarpBlendParameters.Src.Texture;

		RenderPassData.PSParameters.UVLightCardUnderSampler = TStaticSamplerState<SF_Trilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		RenderPassData.PSParameters.UVLightCardUnderTexture = WarpBlendParameters.Src.Texture;

		RenderPassData.PSParameters.UVLightCardOverSampler = TStaticSamplerState<SF_Trilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		RenderPassData.PSParameters.UVLightCardOverTexture = WarpBlendParameters.Src.Texture;
	}

	bool GetLightCardParameters(FIcvfxRenderPassData& RenderPassData)
	{
		RenderPassData.PSParameters.OverlayProjectionMatrix = FMatrix44f(LocalUVMatrix);

		switch (RenderPass)
		{
		case EIcvfxPassRenderPass::Base:
			if (EnumHasAnyFlags(RenderState, EIcvfxPassRenderState::LightCardUnder))
			{
				// Rendering in one pass along with the first camera
				EnumAddLightCardModeFlags(RenderPassData, EShaderParameterLightCardMode::LIGHTCARD_UNDER);
			}

			if (EnumHasAnyFlags(RenderState, EIcvfxPassRenderState::LightCardOver) && !ICVFXParameters.IsMultiCamerasUsed())
			{
				// If we have only one camera, the top LC layer is rendered in one pass
				EnumAddLightCardModeFlags(RenderPassData, EShaderParameterLightCardMode::LIGHTCARD_OVER);
			}
		break;

		case EIcvfxPassRenderPass::LightCardOver:
			if (EnumHasAllFlags(RenderState, EIcvfxPassRenderState::LightCardOver))
			{
				EnumAddLightCardModeFlags(RenderPassData, EShaderParameterLightCardMode::LIGHTCARD_OVER);
			}
			break;

		default:
			break;
		}

		if (EnumHasAnyLightCardModeFlags(RenderPassData, EShaderParameterLightCardMode::LIGHTCARD_UNDER | EShaderParameterLightCardMode::LIGHTCARD_OVER))
		{
			RenderPassData.PSParameters.LightCardUnderTexture = ICVFXParameters.LightCardUnder.Texture ? ICVFXParameters.LightCardUnder.Texture : ICVFXParameters.LightCardOver.Texture;
			RenderPassData.PSParameters.LightCardOverTexture  = ICVFXParameters.LightCardOver.Texture  ? ICVFXParameters.LightCardOver.Texture  : ICVFXParameters.LightCardUnder.Texture;

			return true;
		}

		return false;
	}

	bool GetUVLightCardParameters(FIcvfxRenderPassData& RenderPassData)
	{
		// UVLightCard require Mesh warp (UV_Chromakey)
		if (!RenderPassData.VSPermutationVector.Get<IcvfxShaderPermutation::FIcvfxShaderMeshWarp>())
		{
			return false;
		}

		if (EnumHasAnyFlags(RenderState, EIcvfxPassRenderState::UVLightCardMerged))
		{
			EnumAddLightCardModeFlags(RenderPassData, EShaderParameterLightCardMode::UVLIGHTCARD_MERGE);
		}

		switch (RenderPass)
		{
		case EIcvfxPassRenderPass::Base:
			if (EnumHasAnyFlags(RenderState, EIcvfxPassRenderState::UVLightCardUnder))
			{
				// Rendering in one pass along with the first camera
				EnumAddLightCardModeFlags(RenderPassData, EShaderParameterLightCardMode::UVLIGHTCARD_UNDER);
			}

			if (EnumHasAnyFlags(RenderState, EIcvfxPassRenderState::UVLightCardOver) && !ICVFXParameters.IsMultiCamerasUsed())
			{
				// If we have only one camera, the top LC layer is rendered in one pass
				EnumAddLightCardModeFlags(RenderPassData, EShaderParameterLightCardMode::UVLIGHTCARD_OVER);
			}
			break;

		case EIcvfxPassRenderPass::LightCardOver:
			if (EnumHasAllFlags(RenderState, EIcvfxPassRenderState::UVLightCardOver))
			{
				EnumAddLightCardModeFlags(RenderPassData, EShaderParameterLightCardMode::UVLIGHTCARD_OVER);
			}
			break;

		default:
			break;
		}

		if (EnumHasAnyLightCardModeFlags(RenderPassData, EShaderParameterLightCardMode::UVLIGHTCARD_UNDER | EShaderParameterLightCardMode::UVLIGHTCARD_OVER))
		{
			RenderPassData.PSParameters.UVLightCardUnderTexture = ICVFXParameters.UVLightCardUnder.Texture ? ICVFXParameters.UVLightCardUnder.Texture : ICVFXParameters.UVLightCardOver.Texture;
			RenderPassData.PSParameters.UVLightCardOverTexture  = ICVFXParameters.UVLightCardOver.Texture  ? ICVFXParameters.UVLightCardOver.Texture  : ICVFXParameters.UVLightCardUnder.Texture;

			return true;
		}

		return false;
	}

	int32 GetOverlappedInnerCameras(FIcvfxRenderPassData& RenderPassData)
	{
		static const FMatrix Game2Render(
			FPlane(0, 0, 1, 0),
			FPlane(1, 0, 0, 0),
			FPlane(0, 1, 0, 0),
			FPlane(0, 0, 0, 1));

		// The total number of InCameras, which are rendered below.
		int32 OutOverlapCameraNum = 0;

		for (int32 CameraIndex = 0; CameraIndex < ActiveCameraIndex && ICVFXParameters.Cameras.IsValidIndex(CameraIndex) && CameraIndex < MAX_INNER_CAMERAS_AMOUNT; CameraIndex++)
		{
			const FDisplayClusterShaderParameters_ICVFX::FCameraSettings& Camera = ICVFXParameters.Cameras[CameraIndex];
			const FRotationTranslationMatrix CameraTranslationMatrix(Camera.ViewProjection.ViewRotation, Camera.ViewProjection.ViewLocation);

			const FMatrix WorldToCamera = CameraTranslationMatrix.Inverse();
			const FMatrix UVMatrix = WorldToCamera * Game2Render * Camera.ViewProjection.PrjMatrix;
			const FMatrix InnerCameraProjectionMatrix = UVMatrix * WarpBlendParameters.Context.TextureMatrix;

			RenderPassData.PSParameters.OverlappedInnerCamerasProjectionMatrices[CameraIndex] = FMatrix44f(InnerCameraProjectionMatrix);
			RenderPassData.PSParameters.OverlappedInnerCameraSoftEdges[CameraIndex] = GDisplayClusterShadersICVFXOverlapInnerFrustumSoftEdges ? FVector4f(Camera.SoftEdge) : FVector4f(0, 0, 0, 0);

			// Count total num of overlapped cameras
			OutOverlapCameraNum++;
		}

		return OutOverlapCameraNum;
	}
	
	EDisplayClusterShaderParametersICVFX_ChromakeySource GetCameraOverlapParameters(FIcvfxRenderPassData& RenderPassData)
	{
		int32 OverlapCameraNum = 0;
		EDisplayClusterShaderParametersICVFX_ChromakeySource OutChromakeySource = EDisplayClusterShaderParametersICVFX_ChromakeySource::Default;

		if (EnumHasAnyFlags(RenderState, EIcvfxPassRenderState::EnableInnerFrustumChromakeyOverlap))
		{
			switch (RenderPass)
			{
			case EIcvfxPassRenderPass::InnerFrustumIterator:
				// Hide Inner Frustum under overlap
				if (!GDisplayClusterShadersICVFXEnableInnerFrustumUnderOverlap)
				{
					OverlapCameraNum = GetOverlappedInnerCameras(RenderPassData);
				}
				break;

			case EIcvfxPassRenderPass::InnerFrustumChromakeyOverlapIterator:
			{
				// Use camera chromakey color to render overlap
				OutChromakeySource = EDisplayClusterShaderParametersICVFX_ChromakeySource::FrameColor;

				// Switch to special shader
				RenderPassData.PSPermutationVector.Set<IcvfxShaderPermutation::FIcvfxShaderInnerCameraOverlap>(true);

				// Get cameraa under active
				OverlapCameraNum = GetOverlappedInnerCameras(RenderPassData);

				break;
			}

			default:
				break;
			}
		}

		RenderPassData.PSParameters.OverlappedInnerCamerasNum = OverlapCameraNum;

		return OutChromakeySource;
	}

	bool GetCameraParameters(FIcvfxRenderPassData& RenderPassData)
	{
		if (!ICVFXParameters.IsCameraUsed(ActiveCameraIndex))
		{
			return false;
		}

		RenderPassData.PSPermutationVector.Set<IcvfxShaderPermutation::FIcvfxShaderInnerCamera>(true);

		const FDisplayClusterShaderParameters_ICVFX::FCameraSettings& Camera = ICVFXParameters.Cameras[ActiveCameraIndex];

		static const FMatrix Game2Render(
			FPlane(0, 0, 1, 0),
			FPlane(1, 0, 0, 0),
			FPlane(0, 1, 0, 0),
			FPlane(0, 0, 0, 1));

		const FRotationTranslationMatrix CameraTranslationMatrix(Camera.ViewProjection.ViewRotation, Camera.ViewProjection.ViewLocation);

		const FMatrix WorldToCamera = CameraTranslationMatrix.Inverse();
		const FMatrix UVMatrix = WorldToCamera * Game2Render * Camera.ViewProjection.PrjMatrix;
		const FMatrix InnerCameraProjectionMatrix = UVMatrix * WarpBlendParameters.Context.TextureMatrix;

		RenderPassData.PSParameters.InnerCameraTexture = Camera.Resource.Texture;
		RenderPassData.PSParameters.InnerCameraSampler = TStaticSamplerState<SF_Trilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

		RenderPassData.PSParameters.InnerCameraProjectionMatrix = FMatrix44f(InnerCameraProjectionMatrix);

		float InnerCameraFrameAspectRatio = 1.f;
		if (Camera.Resource.Texture)
		{
			// Calc inner camera texture aspect ratio:
			const FIntPoint InnerCameraTextureSize = Camera.Resource.Texture->GetSizeXY();
			InnerCameraFrameAspectRatio = float(InnerCameraTextureSize.X) / InnerCameraTextureSize.Y;
		}

		RenderPassData.PSParameters.InnerCameraSoftEdge = FVector4f(Camera.SoftEdge);

		RenderPassData.PSParameters.InnerCameraBorderColor = Camera.InnerCameraBorderColor;
		RenderPassData.PSParameters.InnerCameraBorderThickness = Camera.InnerCameraBorderThickness;
		RenderPassData.PSParameters.InnerCameraFrameAspectRatio = InnerCameraFrameAspectRatio;

		return true;
	}

	bool GetCameraChromakeyParameters(FIcvfxRenderPassData& RenderPassData, const EDisplayClusterShaderParametersICVFX_ChromakeySource InChromakeySource)
	{
		if (!ICVFXParameters.IsCameraUsed(ActiveCameraIndex))
		{
			return false;
		}
		
		const FDisplayClusterShaderParameters_ICVFX::FCameraSettings& Camera = ICVFXParameters.Cameras[ActiveCameraIndex];

		EDisplayClusterShaderParametersICVFX_ChromakeySource ChromakeySource = Camera.ChromakeySource;

		// Override by overlap
		if (InChromakeySource != EDisplayClusterShaderParametersICVFX_ChromakeySource::Default)
		{
			ChromakeySource = InChromakeySource;
		};


		switch (ChromakeySource)
		{
		case EDisplayClusterShaderParametersICVFX_ChromakeySource::ChromakeyLayers:
		{
			// Protection against invalid parameter combinations
			if (RenderPassData.PSParameters.InnerCameraTexture == nullptr || Camera.Chromakey.Texture == nullptr)
			{
				// There is no InCamera texture and no frame colour, we have nothing to render.
				// Disable this camera, wrong combination
				RenderPassData.PSPermutationVector.Set<IcvfxShaderPermutation::FIcvfxShaderInnerCamera>(false);

				return false;
			}

			RenderPassData.PSParameters.ChromakeyColor = Camera.ChromakeyColor;
			RenderPassData.PSParameters.ChromakeyCameraTexture = Camera.Chromakey.Texture;
			RenderPassData.PSParameters.ChromakeyCameraSampler = TStaticSamplerState<SF_Trilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

			RenderPassData.PSPermutationVector.Set<IcvfxShaderPermutation::FIcvfxShaderChromakey>(true);

			return true;
		}

		case EDisplayClusterShaderParametersICVFX_ChromakeySource::FrameColor:
		{
			if (RenderPass == EIcvfxPassRenderPass::InnerFrustumChromakeyOverlapIterator)
			{
				RenderPassData.PSParameters.ChromakeyColor = Camera.OverlapChromakeyColor;
			}
			else
			{
				RenderPassData.PSParameters.ChromakeyColor = Camera.ChromakeyColor;
			}

			RenderPassData.PSPermutationVector.Set<IcvfxShaderPermutation::FIcvfxShaderChromakeyFrameColor>(true);

			return true;
		}

		case EDisplayClusterShaderParametersICVFX_ChromakeySource::Default:
			// By default chromakey is disabled
		case EDisplayClusterShaderParametersICVFX_ChromakeySource::Disabled:
		default:
			break;
		}

		// Protection against invalid parameter combinations
		if (RenderPassData.PSParameters.InnerCameraTexture == nullptr)
		{
			// Disable this camera, wrong combination
			RenderPassData.PSPermutationVector.Set<IcvfxShaderPermutation::FIcvfxShaderInnerCamera>(false);

			return false;
		}

		return false;
	}

	bool GetCameraChromakeyMarkerParameters(FIcvfxRenderPassData& RenderPassData)
	{
		if (!ICVFXParameters.IsCameraUsed(ActiveCameraIndex))
		{
			return false;
		}

		const FDisplayClusterShaderParameters_ICVFX::FCameraSettings& Camera = ICVFXParameters.Cameras[ActiveCameraIndex];

		if (RenderPass == EIcvfxPassRenderPass::InnerFrustumChromakeyOverlapIterator)
		{
			if (Camera.IsOverlapChromakeyMarkerUsed())
			{
				RenderPassData.PSParameters.ChromakeyMarkerColor = Camera.OverlapChromakeyMarkersColor;

				RenderPassData.PSParameters.ChromakeyMarkerTexture = Camera.OverlapChromakeyMarkerTextureRHI;
				RenderPassData.PSParameters.ChromakeyMarkerSampler = TStaticSamplerState<SF_Trilinear, AM_Wrap, AM_Wrap, AM_Wrap>::GetRHI();

				RenderPassData.PSParameters.ChromakeyMarkerScale = Camera.OverlapChromakeyMarkersScale;
				RenderPassData.PSParameters.ChromakeyMarkerDistance = Camera.OverlapChromakeyMarkersDistance;
				RenderPassData.PSParameters.ChromakeyMarkerOffset = FVector2f(Camera.OverlapChromakeyMarkersOffset);

				RenderPassData.PSPermutationVector.Set<IcvfxShaderPermutation::FIcvfxShaderChromakeyMarker>(true);
				return true;
			}
		}
		else
		{
			if (Camera.IsChromakeyMarkerUsed())
			{
				RenderPassData.PSParameters.ChromakeyMarkerColor = Camera.ChromakeyMarkersColor;

				RenderPassData.PSParameters.ChromakeyMarkerTexture = Camera.ChromakeMarkerTextureRHI;
				RenderPassData.PSParameters.ChromakeyMarkerSampler = TStaticSamplerState<SF_Trilinear, AM_Wrap, AM_Wrap, AM_Wrap>::GetRHI();

				RenderPassData.PSParameters.ChromakeyMarkerScale    = Camera.ChromakeyMarkersScale;
				RenderPassData.PSParameters.ChromakeyMarkerDistance = Camera.ChromakeyMarkersDistance;
				RenderPassData.PSParameters.ChromakeyMarkerOffset   = FVector2f(Camera.ChromakeyMarkersOffset);

				RenderPassData.PSPermutationVector.Set<IcvfxShaderPermutation::FIcvfxShaderChromakeyMarker>(true);
				return true;
			}
		}

		return false;
	}

	void InitRenderPass(FIcvfxRenderPassData& RenderPassData)
	{
		// Forward input viewport
		GetViewportParameters(RenderPassData);

		// Configure mutable pixel shader:
		if (WarpBlendParameters.WarpInterface.IsValid())
		{
			GetWarpMapParameters(RenderPassData);

			if (!EnumHasAnyFlags(RenderState, EIcvfxPassRenderState::BlendDisabled))
			{
				GetWarpBlendParameters(RenderPassData);
			}

			// Configure Lightcards:
			{
				// Both LightCards textures use linear gamma, so they have the same gamma correction at the end.
				{
					// The LightCard texture is in linear color space and should be converted to the target gamma.
					// Math: GammaDst = Pow( LinearSrc, (1/Gamma) ).
					if (ICVFXParameters.LightCardGamma > KINDA_SMALL_NUMBER)
					{
						RenderPassData.PSParameters.LightCardGamma = 1.0f / ICVFXParameters.LightCardGamma;
					}
					else
					{
						RenderPassData.PSParameters.LightCardGamma = 1.0f;
					}
				}

				InitializeLightCardParameters(RenderPassData);
				GetLightCardParameters(RenderPassData);
				GetUVLightCardParameters(RenderPassData);
			}

			if (GetCameraParameters(RenderPassData))
			{
				// Overlap is first priority
				EDisplayClusterShaderParametersICVFX_ChromakeySource ChromakeySourceOverride = GetCameraOverlapParameters(RenderPassData);

				// Chromakey inside cam:
				if (GetCameraChromakeyParameters(RenderPassData, ChromakeySourceOverride))
				{
					//Support chromakey markers
					GetCameraChromakeyMarkerParameters(RenderPassData);
				}
			}
		}
	}

	bool ImplBeginRenderPass(FRHICommandListImmediate& RHICmdList, FGraphicsPipelineStateInitializer& GraphicsPSOInit) const
	{
		if (WarpBlendParameters.WarpInterface.IsValid())
		{
			switch (WarpBlendParameters.WarpInterface->GetWarpGeometryType())
			{
			case EDisplayClusterWarpGeometryType::WarpMap:
				GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
				return true;

			case EDisplayClusterWarpGeometryType::WarpMesh:
			case EDisplayClusterWarpGeometryType::WarpProceduralMesh:
			{
				const IDisplayClusterRender_MeshComponentProxy* WarpMeshProxy = WarpBlendParameters.WarpInterface->GetWarpMeshProxy_RenderThread();
				if (WarpMeshProxy != nullptr)
				{
					return WarpMeshProxy->BeginRender_RenderThread(RHICmdList, GraphicsPSOInit);
				}
				break;
			}
			default:
				break;
			}
		}

		return false;
	}

	bool ImplFinishRenderPass(FRHICommandListImmediate& RHICmdList) const
	{
		if (WarpBlendParameters.WarpInterface.IsValid())
		{
			switch (WarpBlendParameters.WarpInterface->GetWarpGeometryType())
			{
			case EDisplayClusterWarpGeometryType::WarpMap:
				// Render quad
				FPixelShaderUtils::DrawFullscreenQuad(RHICmdList, 1);
				return true;

			case EDisplayClusterWarpGeometryType::WarpMesh:
			case EDisplayClusterWarpGeometryType::WarpProceduralMesh:
			{
				const IDisplayClusterRender_MeshComponentProxy* WarpMeshProxy = WarpBlendParameters.WarpInterface->GetWarpMeshProxy_RenderThread();
				if (WarpMeshProxy != nullptr)
				{
					return WarpMeshProxy->FinishRender_RenderThread(RHICmdList);
				}
				break;
			}
			default:
				break;
			}
		}

		return false;
	}
	
	bool RenderPassthrough(FRHICommandListImmediate& RHICmdList, FIcvfxRenderPassData& RenderPassData)
	{
		if (WarpBlendParameters.WarpInterface.IsValid() == false)
		{
			return false;
		}

		// Get mutable shaders:
		FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
		TShaderMapRef<FIcvfxWarpVS> VertexShader(ShaderMap, RenderPassData.VSPermutationVector);
		TShaderMapRef<FIcvfxPassthroughPS> PixelShader(ShaderMap);
		if (!VertexShader.IsValid() || !PixelShader.IsValid())
		{
			// Always check if shaders are available on the current platform and hardware
			return false;
		}

		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		// Set the graphic pipeline state.
		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

		if (ImplBeginRenderPass(RHICmdList, GraphicsPSOInit))
		{

			// Setup graphics pipeline
			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
			GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();

			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();

			GraphicsPSOInit.PrimitiveType = PT_TriangleList;
			GraphicsPSOInit.BlendState = TStaticBlendState <>::GetRHI();
			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

			// Setup shaders data
			SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), RenderPassData.VSParameters);
			SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), RenderPassData.PSParameters);

			ImplFinishRenderPass(RHICmdList);

			return true;
		}

		return false;
	}

	bool RenderCurentPass(FRHICommandListImmediate& RHICmdList, FIcvfxRenderPassData& RenderPassData)
	{
		if (WarpBlendParameters.WarpInterface.IsValid() == false)
		{
			return false;
		}

		// Check the permutation vectors. This should prevent a crash when no shader permutation is found.
		if (!IcvfxShaderPermutation::ShouldCompileCommonPSPermutation(RenderPassData.PSPermutationVector))
		{
			UE_LOGF(LogDisplayClusterShaders, Warning, "Invalid permutation vector %d for shader FIcvfxWarpPS", RenderPassData.PSPermutationVector.ToDimensionValueId());

			return false;
		}

		if (!IcvfxShaderPermutation::ShouldCompileCommonVSPermutation(RenderPassData.VSPermutationVector))
		{
			UE_LOGF(LogDisplayClusterShaders, Warning, "Invalid permutation vector %d for shader FIcvfxWarpVS", RenderPassData.VSPermutationVector.ToDimensionValueId());

			return false;
		}

		// Get mutable shaders:
		FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
		TShaderMapRef<FIcvfxWarpVS> VertexShader(ShaderMap, RenderPassData.VSPermutationVector);
		TShaderMapRef<FIcvfxWarpPS> PixelShader(ShaderMap, RenderPassData.PSPermutationVector);

		if (!VertexShader.IsValid() || !PixelShader.IsValid())
		{
			UE_LOGF(LogDisplayClusterShaders, Warning, "ICVFX shaders are not initialized properly.");

			return false;
		}

		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		// Set the graphic pipeline state.
		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
		if (ImplBeginRenderPass(RHICmdList, GraphicsPSOInit))
		{
			// Setup graphics pipeline
			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
			GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();

			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();

			GraphicsPSOInit.PrimitiveType = PT_TriangleList;

			switch (RenderPass)
			{
			case EIcvfxPassRenderPass::InnerFrustumIterator:
			case EIcvfxPassRenderPass::InnerFrustumChromakeyOverlapIterator:
				// Multicam rendering
				GraphicsPSOInit.BlendState = TStaticBlendState <CW_RGBA, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One>::GetRHI();
				break;

			case EIcvfxPassRenderPass::LightCardOver:
				// render pass for LC overlays
				GraphicsPSOInit.BlendState = TStaticBlendState <CW_RGBA, BO_Add, BF_One, BF_SourceAlpha, BO_Add, BF_Zero, BF_One>::GetRHI();
				break;

			case EIcvfxPassRenderPass::Base:
			default:
				// First pass always override old viewport image
				GraphicsPSOInit.BlendState = TStaticBlendState <>::GetRHI();
				break;
			}

			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

			// Setup shaders data
			SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), RenderPassData.VSParameters);
			SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), RenderPassData.PSParameters);

			ImplFinishRenderPass(RHICmdList);

			return true;
		}

		return false;
	}

	bool Initialize()
	{
		if (WarpBlendParameters.WarpInterface.IsValid() == false)
		{
			return false;
		}

		// Map ProfileType to shader type to use
		PixelShaderType = GetPixelShaderType();
		if (EIcvfxShaderType::Invalid == PixelShaderType)
		{
			return false;
		}

		if (WarpBlendParameters.WarpInterface->GetTexture(EDisplayClusterWarpBlendTextureType::AlphaMap) == nullptr)
		{
			EnumAddFlags(RenderState, EIcvfxPassRenderState::BlendDisabled);
		};

		const EVarIcvfxMPCDIShaderType ShaderType = (EVarIcvfxMPCDIShaderType)CVarIcvfxMPCDIShaderType.GetValueOnAnyThread();

		switch (ShaderType)
		{
		case EVarIcvfxMPCDIShaderType::DefaultNoBlend:
			EnumAddFlags(RenderState, EIcvfxPassRenderState::BlendDisabled);
			break;

		case EVarIcvfxMPCDIShaderType::Passthrough:
			PixelShaderType = EIcvfxShaderType::Passthrough;
			break;

		case EVarIcvfxMPCDIShaderType::Disable:
			return false;

		case EVarIcvfxMPCDIShaderType::Default:
		default:
			// Use default icvfx render
			break;
		};

		return true;
	}

	inline bool SetRenderPass_LightCardOver()
	{
		if (EnumHasAnyFlags(RenderState, EIcvfxPassRenderState::LightCardOver | EIcvfxPassRenderState::UVLightCardOver))
		{
			// Render LC over cameras
			ActiveCameraIndex = INDEX_NONE;
			RenderPass = EIcvfxPassRenderPass::LightCardOver;
			
			return true;
		}

		return false;
	}

	inline bool SetRenderPass_InnerFrustumChromakeyOverlapIterator()
	{
		// Chromakey overlap pass
		if (EnumHasAnyFlags(RenderState, EIcvfxPassRenderState::EnableInnerFrustumChromakeyOverlap))
		{
			// Render camera overlaps
			ActiveCameraIndex = 1;
			RenderPass = EIcvfxPassRenderPass::InnerFrustumChromakeyOverlapIterator;
			
			return true;
		}

		return false;
	}

	// Render Full Icvfx Warp&Blend
	bool DoRenderPass(FRHICommandListImmediate& RHICmdList)
	{
		switch (RenderPass)
		{
		case EIcvfxPassRenderPass::None:
			RenderPass = EIcvfxPassRenderPass::Base;
			if (ICVFXParameters.IsAnyCameraUsed())
			{
				ActiveCameraIndex = 0;
			}
			break;

		case EIcvfxPassRenderPass::Base:
			if (ICVFXParameters.IsMultiCamerasUsed())
			{
				// multi-camera rendering
				RenderPass = EIcvfxPassRenderPass::InnerFrustumIterator;
				ActiveCameraIndex = 1;
				break;
			}

			// Render in single pass completed
			RenderPass = EIcvfxPassRenderPass::None;
			return false;

		case EIcvfxPassRenderPass::InnerFrustumIterator:
			if (!ICVFXParameters.IsCameraUsed(++ActiveCameraIndex))
			{
				// All cameras have been rendered.
				if (!SetRenderPass_InnerFrustumChromakeyOverlapIterator() && !SetRenderPass_LightCardOver())
				{
					// Render completed
					RenderPass = EIcvfxPassRenderPass::None;
					return false;
				}
			}
			break;

		case EIcvfxPassRenderPass::LightCardOver:
			// Render completed
			RenderPass = EIcvfxPassRenderPass::None;

			return false;

		case EIcvfxPassRenderPass::InnerFrustumChromakeyOverlapIterator:
			if (!ICVFXParameters.IsCameraUsed(++ActiveCameraIndex))
			{
				if (!SetRenderPass_LightCardOver())
				{
					// Render completed
					RenderPass = EIcvfxPassRenderPass::None;
					return false;
				}
			}
			break;

		default:
			return false;
		}

		// Build and draw render-pass:
		FIcvfxRenderPassData RenderPassData;
		InitRenderPass(RenderPassData);

		switch (RenderPass)
		{
		case EIcvfxPassRenderPass::Base:
		{
			RHI_BREADCRUMB_EVENT_STAT(RHICmdList, nDisplay_IcvfxRenderPass_Base, "nDisplay_IcvfxRenderPass_Base");

			RenderCurentPass(RHICmdList, RenderPassData);
		}
		break;

		case EIcvfxPassRenderPass::InnerFrustumIterator:
		{
			RHI_BREADCRUMB_EVENT_STAT(RHICmdList, nDisplay_IcvfxRenderPass_InnerFrustumIterator, "nDisplay_IcvfxRenderPass_InnerFrustumIterator");

			RenderCurentPass(RHICmdList, RenderPassData);
		}
		break;

		case EIcvfxPassRenderPass::InnerFrustumChromakeyOverlapIterator:
		{
			RHI_BREADCRUMB_EVENT_STAT(RHICmdList, nDisplay_IcvfxRenderPass_InnerFrustumChromakeyOverlapIterator, "nDisplay_IcvfxRenderPass_InnerFrustumChromakeyOverlapIterator");

			RenderCurentPass(RHICmdList, RenderPassData);
		}
		break;

		case EIcvfxPassRenderPass::LightCardOver:
		{
			RHI_BREADCRUMB_EVENT_STAT(RHICmdList, nDisplay_IcvfxRenderPass_LightCardOver, "nDisplay_IcvfxRenderPass_LightCardOver");

			RenderCurentPass(RHICmdList, RenderPassData);
		}
		break;

		default:
			RenderCurentPass(RHICmdList, RenderPassData);
			break;
		}

		return true;
	}
	
	bool Render(FRHICommandListImmediate& RHICmdList)
	{
		// Setup viewport before render
		FIntRect MPCDIViewportRect = GetViewportRect();
		RHICmdList.SetViewport(MPCDIViewportRect.Min.X, MPCDIViewportRect.Min.Y, 0.0f, MPCDIViewportRect.Max.X, MPCDIViewportRect.Max.Y, 1.0f);

		// Render
		switch (PixelShaderType)
		{
			case EIcvfxShaderType::Passthrough:
			{
				LocalUVMatrix = FMatrix::Identity;
				FIcvfxRenderPassData RenderPassData;
				InitRenderPass(RenderPassData);
				RenderPassthrough(RHICmdList, RenderPassData);
				break;
			}

			case EIcvfxShaderType::WarpAndBlend:
			{
				LocalUVMatrix = WarpBlendParameters.Context.UVMatrix * WarpBlendParameters.Context.TextureMatrix * WarpBlendParameters.Context.RegionMatrix;

				// Multi-pass ICVFX rendering
				while (DoRenderPass(RHICmdList));

				break;
			}
		}

		return true;
	}
};

DECLARE_GPU_STAT_NAMED(nDisplay_Icvfx_WarpBlend, TEXT("nDisplay Icvfx::Warp&Blend"));

bool FDisplayClusterShadersWarpblend_ICVFX::RenderWarpBlend_ICVFX(FRHICommandListImmediate& RHICmdList, const FDisplayClusterShaderParameters_WarpBlend& InWarpBlendParameters, const FDisplayClusterShaderParameters_ICVFX& InICVFXParameters)
{
	check(IsInRenderingThread());

	FIcvfxPassRenderer IcvfxPassRenderer(InWarpBlendParameters, InICVFXParameters);

	if (!IcvfxPassRenderer.Initialize())
	{
		return false;
	}

	RHI_BREADCRUMB_EVENT_STAT(RHICmdList, nDisplay_Icvfx_WarpBlend, "nDisplay_Icvfx_WarpBlend");

	// Do single warp render pass
	bool bIsRenderSuccess = false;
	FRHIRenderPassInfo RPInfo(InWarpBlendParameters.Dest.Texture, ERenderTargetActions::Load_Store);
	RHICmdList.Transition(FRHITransitionInfo(InWarpBlendParameters.Dest.Texture, ERHIAccess::Unknown, ERHIAccess::RTV));

	RHICmdList.BeginRenderPass(RPInfo, TEXT("nDisplay_IcvfxWarpBlend"));
	{
		bIsRenderSuccess = IcvfxPassRenderer.Render(RHICmdList);
	}
	RHICmdList.EndRenderPass();
	RHICmdList.Transition(FRHITransitionInfo(InWarpBlendParameters.Dest.Texture, ERHIAccess::Unknown, ERHIAccess::SRVMask));

	return bIsRenderSuccess;
};
