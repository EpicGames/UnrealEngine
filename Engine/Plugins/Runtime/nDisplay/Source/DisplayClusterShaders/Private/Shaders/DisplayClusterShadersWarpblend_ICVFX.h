// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RHI.h"
#include "RHICommandList.h"

class FDisplayClusterShaderParameters_ICVFX;
struct FDisplayClusterShaderParameters_WarpBlend;

#define MAX_INNER_CAMERAS_AMOUNT 10

class FDisplayClusterShadersWarpblend_ICVFX
{
public:
	static bool RenderWarpBlend_ICVFX(FRHICommandListImmediate& RHICmdList, const FDisplayClusterShaderParameters_WarpBlend& InWarpBlendParameters, const FDisplayClusterShaderParameters_ICVFX& InICVFXParameters);
};
