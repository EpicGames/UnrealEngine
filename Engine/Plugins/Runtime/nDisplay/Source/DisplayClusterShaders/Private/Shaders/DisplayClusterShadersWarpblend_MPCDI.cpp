// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterShadersWarpblend_MPCDI.h"

#include "RHI.h"
#include "RHIResources.h"
#include "RHIStaticStates.h"

#include "RenderResource.h"
#include "CommonRenderResources.h"
#include "PixelShaderUtils.h"

#include "Shader.h"
#include "GlobalShader.h"

#include "ShaderParameters.h"
#include "ShaderParameterUtils.h"
#include "ShaderPermutation.h"
#include "ShaderParameterStruct.h"

#include "Render/Containers/IDisplayClusterRender_MeshComponentProxy.h"
#include "Render/Containers/IDisplayClusterRender_Texture.h"
#include "IDisplayClusterWarpBlend.h"

#include "ShaderParameters/DisplayClusterShaderParameters_WarpBlend.h"


#define MPCDI_ShaderFileName "/Plugin/nDisplay/Private/MPCDIShaders.usf"

// Select mpcdi stereo mode
enum class EVarMPCDIShaderType : uint8
{
	Default = 0,
	DisableBlend,
	Passthrough,
	Disable,
};

static TAutoConsoleVariable<int32> CVarMPCDIShaderType(
	TEXT("nDisplay.render.mpcdi.shader"),
	(int32)EVarMPCDIShaderType::Default,
	TEXT("Select shader for mpcdi:\n")
	TEXT(" 0: Warp shader (used by default)\n")
	TEXT(" 1: Warp shader with disabled blend maps\n")
	TEXT(" 2: Passthrough shader\n")
	TEXT(" 3: Disable all warp shaders"),
	ECVF_RenderThreadSafe
);

enum class EMpcdiShaderType : uint8
{
	Passthrough,  // Viewport frustum (no warpblend, only frustum math)
	WarpAndBlend, // Pure mpcdi warpblend for viewport
	Invalid,
};

namespace MpcdiShaderPermutation
{
	// Shared permutation for mpcdi warp
	class FMpcdiShaderAlphaMapBlending : SHADER_PERMUTATION_BOOL("ALPHAMAP_BLENDING");
	class FMpcdiShaderBetaMapBlending : SHADER_PERMUTATION_BOOL("BETAMAP_BLENDING");

	class FMpcdiShaderViewportInputAlpha : SHADER_PERMUTATION_BOOL("VIEWPORT_INPUT_ALPHA");

	class FMpcdiShaderMeshWarp : SHADER_PERMUTATION_BOOL("MESH_WARP");

	using FCommonVSDomain = TShaderPermutationDomain<
		FMpcdiShaderMeshWarp
	>;

	using FCommonPSDomain = TShaderPermutationDomain<
		FMpcdiShaderAlphaMapBlending,
		FMpcdiShaderBetaMapBlending,
		FMpcdiShaderViewportInputAlpha,
		FMpcdiShaderMeshWarp
	>;

	bool ShouldCompileCommonPSPermutation(const FGlobalShaderPermutationParameters& Parameters, const FCommonPSDomain& PermutationVector)
	{
		return true;
	}

	bool ShouldCompileCommonVSPermutation(const FGlobalShaderPermutationParameters& Parameters, const FCommonVSDomain& PermutationVector)
	{
		return true;
	}
};

BEGIN_SHADER_PARAMETER_STRUCT(FMpcdiVertexShaderParameters, )
	SHADER_PARAMETER(FVector4f, DrawRectanglePosScaleBias)
	SHADER_PARAMETER(FVector4f, DrawRectangleInvTargetSizeAndTextureSize)
	SHADER_PARAMETER(FVector4f, DrawRectangleUVScaleBias)

	SHADER_PARAMETER(FMatrix44f, MeshToStageProjectionMatrix)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FMpcdiPixelShaderParameters, )
	SHADER_PARAMETER_TEXTURE(Texture2D, InputTexture)
	SHADER_PARAMETER_TEXTURE(Texture2D, WarpMapTexture)
	SHADER_PARAMETER_TEXTURE(Texture2D, AlphaMapTexture)
	SHADER_PARAMETER_TEXTURE(Texture2D, BetaMapTexture)

	SHADER_PARAMETER_SAMPLER(SamplerState, InputSampler)
	SHADER_PARAMETER_SAMPLER(SamplerState, WarpMapSampler)
	SHADER_PARAMETER_SAMPLER(SamplerState, AlphaMapSampler)
	SHADER_PARAMETER_SAMPLER(SamplerState, BetaMapSampler)

	SHADER_PARAMETER(FMatrix44f, ViewportTextureProjectionMatrix)

	SHADER_PARAMETER(float, AlphaMapGammaEmbedded)

	SHADER_PARAMETER(int, AlphaMapComponentDepth)
	SHADER_PARAMETER(int, BetaMapComponentDepth)
END_SHADER_PARAMETER_STRUCT()

class FMpcdiWarpVS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FMpcdiWarpVS);
	SHADER_USE_PARAMETER_STRUCT(FMpcdiWarpVS, FGlobalShader);

	using FPermutationDomain = MpcdiShaderPermutation::FCommonVSDomain;
	using FParameters = FMpcdiVertexShaderParameters;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		// Only compile for desktop-class "PC" shader platforms (filters out mobile and console).
		if (!IsPCPlatform(Parameters.Platform))
		{
			return false;
		}

		return MpcdiShaderPermutation::ShouldCompileCommonVSPermutation(Parameters, FPermutationDomain(Parameters.PermutationId));
	}
};

IMPLEMENT_GLOBAL_SHADER(FMpcdiWarpVS, MPCDI_ShaderFileName, "WarpVS", SF_Vertex);

class FMpcdiWarpPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FMpcdiWarpPS);
	SHADER_USE_PARAMETER_STRUCT(FMpcdiWarpPS, FGlobalShader);

	using FPermutationDomain = MpcdiShaderPermutation::FCommonPSDomain;
	using FParameters = FMpcdiPixelShaderParameters;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		// Only compile for desktop-class "PC" shader platforms (filters out mobile and console).
		if (!IsPCPlatform(Parameters.Platform))
		{
			return false;
		}

		return MpcdiShaderPermutation::ShouldCompileCommonPSPermutation(Parameters, FPermutationDomain(Parameters.PermutationId));
	}
};

IMPLEMENT_GLOBAL_SHADER(FMpcdiWarpPS, MPCDI_ShaderFileName, "WarpPS", SF_Pixel);

class FMpcdiPassthroughPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FMpcdiPassthroughPS);
	SHADER_USE_PARAMETER_STRUCT(FMpcdiPassthroughPS, FGlobalShader);

	using FParameters = FMpcdiPixelShaderParameters;
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		// Only compile for desktop-class "PC" shader platforms (filters out mobile and console).
		if (!IsPCPlatform(Parameters.Platform))
		{
			return false;
		}

		return true;
	}
};

IMPLEMENT_GLOBAL_SHADER(FMpcdiPassthroughPS, MPCDI_ShaderFileName, "Passthrough_PS", SF_Pixel);


struct FMpcdiRenderPassData {
	MpcdiShaderPermutation::FCommonVSDomain VSPermutationVector;
	FMpcdiVertexShaderParameters            VSParameters;

	MpcdiShaderPermutation::FCommonPSDomain PSPermutationVector;
	FMpcdiPixelShaderParameters             PSParameters;
};


class FMpcdiPassRenderer
{
public:
	FMpcdiPassRenderer(const FDisplayClusterShaderParameters_WarpBlend& InWarpBlendParameters)
		: WarpBlendParameters(InWarpBlendParameters)
	{ }

	FMatrix LocalUVMatrix;

private:
	const FDisplayClusterShaderParameters_WarpBlend& WarpBlendParameters;

	EMpcdiShaderType PixelShaderType;

	bool bIsBlendDisabled = false;

private:
	FMatrix GetStereoMatrix() const
	{
		FIntPoint WarpDataSrcSize = WarpBlendParameters.Src.Texture->GetSizeXY();
		FIntPoint WarpDataDstSize = WarpBlendParameters.Dest.Texture->GetSizeXY();

		FMatrix StereoMatrix = FMatrix::Identity;
		StereoMatrix.M[0][0] = float(WarpBlendParameters.Src.Rect.Width()) / float(WarpDataSrcSize.X);
		StereoMatrix.M[1][1] = float(WarpBlendParameters.Src.Rect.Height()) / float(WarpDataSrcSize.Y);
		StereoMatrix.M[3][0] = float(WarpBlendParameters.Src.Rect.Min.X) / float(WarpDataSrcSize.X);
		StereoMatrix.M[3][1] = float(WarpBlendParameters.Src.Rect.Min.Y) / float(WarpDataSrcSize.Y);

		return StereoMatrix;
	}

	FIntRect GetViewportRect() const
	{
		const int32 PosX  = WarpBlendParameters.Dest.Rect.Min.X;
		const int32 PosY  = WarpBlendParameters.Dest.Rect.Min.Y;
		const int32 SizeX = WarpBlendParameters.Dest.Rect.Width();
		const int32 SizeY = WarpBlendParameters.Dest.Rect.Height();

		return FIntRect(FIntPoint(PosX, PosY), FIntPoint(PosX + SizeX, PosY + SizeY));
	}

	EMpcdiShaderType GetPixelShaderType()
	{
		if (WarpBlendParameters.WarpInterface.IsValid())
		{
			switch (WarpBlendParameters.WarpInterface->GetWarpProfileType())
			{
				case EDisplayClusterWarpProfileType::warp_2D:
				case EDisplayClusterWarpProfileType::warp_A3D:
					return EMpcdiShaderType::WarpAndBlend;
#if 0
				case EDisplayClusterWarpProfileType::warp_SL:
				case EDisplayClusterWarpProfileType::warp_3D:
#endif
				default:
					return EMpcdiShaderType::Passthrough;
			};
		}

		return EMpcdiShaderType::Invalid;
	}

public:

	bool GetViewportParameters(FMpcdiRenderPassData& RenderPassData)
	{
		// Input viewport texture
		RenderPassData.PSParameters.InputTexture = WarpBlendParameters.Src.Texture;
		RenderPassData.PSParameters.InputSampler = TStaticSamplerState<SF_Trilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

		// Vertex shader viewport rect
		FIntPoint WarpDataSrcSize = WarpBlendParameters.Src.Texture->GetSizeXY();
		FIntPoint WarpDataDstSize = WarpBlendParameters.Dest.Texture->GetSizeXY();

		float U = WarpBlendParameters.Src.Rect.Min.X / (float)WarpDataSrcSize.X;
		float V = WarpBlendParameters.Src.Rect.Min.Y / (float)WarpDataSrcSize.Y;
		float USize = WarpBlendParameters.Src.Rect.Width() / (float)WarpDataSrcSize.X;
		float VSize = WarpBlendParameters.Src.Rect.Height() / (float)WarpDataSrcSize.Y;

		RenderPassData.VSParameters.DrawRectanglePosScaleBias = FVector4f(1, 1, 0, 0);
		RenderPassData.VSParameters.DrawRectangleInvTargetSizeAndTextureSize = FVector4f(1, 1, 1, 1);
		RenderPassData.VSParameters.DrawRectangleUVScaleBias = FVector4f(USize, VSize, U, V);
		return true;
	}

	bool GetWarpMapParameters(FMpcdiRenderPassData& RenderPassData)
	{
		RenderPassData.PSParameters.ViewportTextureProjectionMatrix = FMatrix44f(LocalUVMatrix * GetStereoMatrix());

		if (WarpBlendParameters.WarpInterface.IsValid())
		{
			if (WarpBlendParameters.bRenderAlphaChannel)
			{
				RenderPassData.PSPermutationVector.Set<MpcdiShaderPermutation::FMpcdiShaderViewportInputAlpha>(true);
			}

			switch (WarpBlendParameters.WarpInterface->GetWarpGeometryType())
			{
				case EDisplayClusterWarpGeometryType::WarpMesh:
				case EDisplayClusterWarpGeometryType::WarpProceduralMesh:
				{
					// Use mesh inseat of warp texture
					RenderPassData.PSPermutationVector.Set<MpcdiShaderPermutation::FMpcdiShaderMeshWarp>(true);
					RenderPassData.VSPermutationVector.Set<MpcdiShaderPermutation::FMpcdiShaderMeshWarp>(true);

					RenderPassData.VSParameters.MeshToStageProjectionMatrix = FMatrix44f(WarpBlendParameters.Context.MeshToStageMatrix);
					break;
				}

				case EDisplayClusterWarpGeometryType::WarpMap:
				{
					FRHITexture* WarpMap = WarpBlendParameters.WarpInterface->GetTexture(EDisplayClusterWarpBlendTextureType::WarpMap);
					if (WarpMap == nullptr)
					{
						return false;
					}

					RenderPassData.PSParameters.WarpMapTexture = WarpMap;
					RenderPassData.PSParameters.WarpMapSampler = TStaticSamplerState<SF_Trilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
					break;
				}
			}

			return true;
		}

		return false;
	}

	bool GetWarpBlendParameters(FMpcdiRenderPassData& RenderPassData)
	{
		if (WarpBlendParameters.WarpInterface.IsValid())
		{
			TSharedPtr<IDisplayClusterRender_Texture, ESPMode::ThreadSafe> AlphaMap = WarpBlendParameters.WarpInterface->GetTextureInterface(EDisplayClusterWarpBlendTextureType::AlphaMap);
			if (FRHITexture* AlphaMapTexture = AlphaMap.IsValid() ? AlphaMap->GetRHITexture() : nullptr)
			{
				RenderPassData.PSParameters.AlphaMapTexture = AlphaMapTexture;
				RenderPassData.PSParameters.AlphaMapSampler = TStaticSamplerState<SF_Trilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
				RenderPassData.PSParameters.AlphaMapComponentDepth = AlphaMap->GetComponentDepth();
				RenderPassData.PSParameters.AlphaMapGammaEmbedded = WarpBlendParameters.WarpInterface->GetAlphaMapEmbeddedGamma();

				RenderPassData.PSPermutationVector.Set<MpcdiShaderPermutation::FMpcdiShaderAlphaMapBlending>(true);
			}

			TSharedPtr<IDisplayClusterRender_Texture, ESPMode::ThreadSafe> BetaMap = WarpBlendParameters.WarpInterface->GetTextureInterface(EDisplayClusterWarpBlendTextureType::BetaMap);
			if (FRHITexture* BetaMapTexture = BetaMap.IsValid() ? BetaMap->GetRHITexture() : nullptr)
			{
				RenderPassData.PSParameters.BetaMapTexture = BetaMapTexture;
				RenderPassData.PSParameters.BetaMapSampler = TStaticSamplerState<SF_Trilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
				RenderPassData.PSParameters.BetaMapComponentDepth = BetaMap->GetComponentDepth();

				// Note: The MPCDI 2.0 standard does not define an 'EmbeddedGamma' tag value for the BetaMap tag.
				// However, it does require gamma correction for BetaMap.
				// Therefore, this parameter is also used for the BetaMap texture.
				RenderPassData.PSParameters.AlphaMapGammaEmbedded = WarpBlendParameters.WarpInterface->GetAlphaMapEmbeddedGamma();

				RenderPassData.PSPermutationVector.Set<MpcdiShaderPermutation::FMpcdiShaderBetaMapBlending>(true);
			}

			return true;
		}

		return false;
	}
	
	void InitRenderPass(FMpcdiRenderPassData& RenderPassData)
	{
		// Forward input viewport
		GetViewportParameters(RenderPassData);

		// Configure mutable pixel shader:
		if (WarpBlendParameters.WarpInterface.IsValid())
		{
			GetWarpMapParameters(RenderPassData);

			if (!bIsBlendDisabled)
			{
				GetWarpBlendParameters(RenderPassData);
			}
		}
	}


	bool ImplBeginRenderPass(FRHICommandListImmediate& RHICmdList, FGraphicsPipelineStateInitializer& GraphicsPSOInit) const
	{
		if (WarpBlendParameters.WarpInterface.IsValid())
		{
			switch (WarpBlendParameters.WarpInterface->GetWarpGeometryType())
			{
			case EDisplayClusterWarpGeometryType::WarpMap:
				GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
				return true;

			case EDisplayClusterWarpGeometryType::WarpMesh:
			case EDisplayClusterWarpGeometryType::WarpProceduralMesh:
			{
				const IDisplayClusterRender_MeshComponentProxy* WarpMeshProxy = WarpBlendParameters.WarpInterface->GetWarpMeshProxy_RenderThread();
				if (WarpMeshProxy != nullptr)
				{
					return WarpMeshProxy->BeginRender_RenderThread(RHICmdList, GraphicsPSOInit);
				}
				break;
			}
			default:
				break;
			}
		}

		return false;
	}

	bool ImplFinishRenderPass(FRHICommandListImmediate& RHICmdList) const
	{
		if (WarpBlendParameters.WarpInterface.IsValid())
		{
			switch (WarpBlendParameters.WarpInterface->GetWarpGeometryType())
			{
			case EDisplayClusterWarpGeometryType::WarpMap:
				// Render quad
				FPixelShaderUtils::DrawFullscreenQuad(RHICmdList, 1);
				return true;

			case EDisplayClusterWarpGeometryType::WarpMesh:
			case EDisplayClusterWarpGeometryType::WarpProceduralMesh:
			{
				const IDisplayClusterRender_MeshComponentProxy* WarpMeshProxy = WarpBlendParameters.WarpInterface->GetWarpMeshProxy_RenderThread();
				if (WarpMeshProxy != nullptr)
				{
					return WarpMeshProxy->FinishRender_RenderThread(RHICmdList);
				}
				break;
			}
			default:
				break;
			}
		}

		return false;
	}

	bool RenderPassthrough(FRHICommandListImmediate& RHICmdList, FMpcdiRenderPassData& RenderPassData)
	{
		if (WarpBlendParameters.WarpInterface.IsValid() == false)
		{
			return false;
		}

		// Get mutable shaders:
		FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
		TShaderMapRef<FMpcdiWarpVS> VertexShader(ShaderMap, RenderPassData.VSPermutationVector);
		TShaderMapRef<FMpcdiPassthroughPS> PixelShader(ShaderMap);
		if (!VertexShader.IsValid() || !PixelShader.IsValid())
		{
			// Always check if shaders are available on the current platform and hardware
			return false;
		}

		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		// Set the graphic pipeline state.
		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

		if (ImplBeginRenderPass(RHICmdList, GraphicsPSOInit))
		{

			// Setup graphics pipeline
			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
			GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();

			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();

			GraphicsPSOInit.PrimitiveType = PT_TriangleList;
			GraphicsPSOInit.BlendState = TStaticBlendState <>::GetRHI();
			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

			// Setup shaders data
			SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), RenderPassData.VSParameters);
			SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), RenderPassData.PSParameters);

			ImplFinishRenderPass(RHICmdList);

			return true;
		}

		return false;
	}

	bool RenderCurentPass(FRHICommandListImmediate& RHICmdList, FMpcdiRenderPassData& RenderPassData)
	{
		if (WarpBlendParameters.WarpInterface.IsValid() == false)
		{
			return false;
		}

		// Get mutable shaders:
		FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
		TShaderMapRef<FMpcdiWarpVS> VertexShader(ShaderMap, RenderPassData.VSPermutationVector);
		TShaderMapRef<FMpcdiWarpPS> PixelShader(ShaderMap, RenderPassData.PSPermutationVector);
		if (!VertexShader.IsValid() || !PixelShader.IsValid())
		{
			// Always check if shaders are available on the current platform and hardware
			return false;
		}

		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		// Set the graphic pipeline state.
		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

		if (ImplBeginRenderPass(RHICmdList, GraphicsPSOInit))
		{

			// Setup graphics pipeline
			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
			GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();

			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();

			GraphicsPSOInit.PrimitiveType = PT_TriangleList;

			// First pass always override old viewport image
			GraphicsPSOInit.BlendState = TStaticBlendState <>::GetRHI();

			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

			// Setup shaders data
			SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), RenderPassData.VSParameters);
			SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), RenderPassData.PSParameters);

			ImplFinishRenderPass(RHICmdList);

			return true;
		}

		return false;
	}

	bool Initialize()
	{
		if (WarpBlendParameters.WarpInterface.IsValid() == false)
		{
			return false;
		}

		// Map ProfileType to shader type to use
		PixelShaderType = GetPixelShaderType();
		if (EMpcdiShaderType::Invalid == PixelShaderType)
		{
			return false;
		}

		bIsBlendDisabled = WarpBlendParameters.WarpInterface->GetTexture(EDisplayClusterWarpBlendTextureType::AlphaMap) == nullptr;

		const EVarMPCDIShaderType ShaderType = (EVarMPCDIShaderType)CVarMPCDIShaderType.GetValueOnAnyThread();

		switch (ShaderType)
		{
		case EVarMPCDIShaderType::DisableBlend:
			bIsBlendDisabled = true;
			break;

		case EVarMPCDIShaderType::Passthrough:
			PixelShaderType = EMpcdiShaderType::Passthrough;
			break;

		case EVarMPCDIShaderType::Disable:
			return false;
		};

		return true;
	}

	bool Render(FRHICommandListImmediate& RHICmdList)
	{
		// Setup viewport before render
		FIntRect MPCDIViewportRect = GetViewportRect();
		RHICmdList.SetViewport(MPCDIViewportRect.Min.X, MPCDIViewportRect.Min.Y, 0.0f, MPCDIViewportRect.Max.X, MPCDIViewportRect.Max.Y, 1.0f);

		// Render
		switch (PixelShaderType)
		{
			case EMpcdiShaderType::Passthrough:
			{
				LocalUVMatrix = FMatrix::Identity;
				FMpcdiRenderPassData RenderPassData;
				InitRenderPass(RenderPassData);
				RenderPassthrough(RHICmdList, RenderPassData);

				break;
			}

			case EMpcdiShaderType::WarpAndBlend:
			{
				LocalUVMatrix = WarpBlendParameters.Context.UVMatrix * WarpBlendParameters.Context.TextureMatrix * WarpBlendParameters.Context.RegionMatrix;
				FMpcdiRenderPassData RenderPassData;
				InitRenderPass(RenderPassData);
				RenderCurentPass(RHICmdList, RenderPassData);

				break;
			}

			default:
				return false;
		}

		return true;
	}
};

DECLARE_GPU_STAT_NAMED(nDisplay_Mpcdi_WarpBlend, TEXT("nDisplay Mpcdi::Warp&Blend"));

bool FDisplayClusterShadersWarpblend_MPCDI::RenderWarpBlend_MPCDI(FRHICommandListImmediate& RHICmdList, const FDisplayClusterShaderParameters_WarpBlend& InWarpBlendParameters)
{
	check(IsInRenderingThread());

	FMpcdiPassRenderer MpcdiPassRenderer(InWarpBlendParameters);

	if (!MpcdiPassRenderer.Initialize())
	{
		return false;
	}

	RHI_BREADCRUMB_EVENT_STAT(RHICmdList, nDisplay_Mpcdi_WarpBlend, "nDisplay_Mpcdi_WarpBlend");

	// Do single-pass warp&blend render
	bool bIsRenderSuccess = false;
	FRHIRenderPassInfo RPInfo(InWarpBlendParameters.Dest.Texture, ERenderTargetActions::Load_Store);
	RHICmdList.Transition(FRHITransitionInfo(InWarpBlendParameters.Dest.Texture, ERHIAccess::Unknown, ERHIAccess::RTV));

	RHICmdList.BeginRenderPass(RPInfo, TEXT("nDisplay_MpcdiWarpBlend"));
	{
		bIsRenderSuccess = MpcdiPassRenderer.Render(RHICmdList);
	}
	RHICmdList.EndRenderPass();
	RHICmdList.Transition(FRHITransitionInfo(InWarpBlendParameters.Dest.Texture, ERHIAccess::Unknown, ERHIAccess::SRVMask));

	return bIsRenderSuccess;
};
