// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RHI.h"
#include "RHICommandList.h"

#include "ShaderParameters/DisplayClusterShaderParameters_WarpBlend.h"

class FDisplayClusterShadersWarpblend_MPCDI
{
public:
	static bool RenderWarpBlend_MPCDI(FRHICommandListImmediate& RHICmdList, const FDisplayClusterShaderParameters_WarpBlend& InWarpBlendParameters);
};
