// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "MPCDIGeometryData.generated.h"

USTRUCT(BlueprintType, Category = "MPCDI")
struct FMPCDIGeometryImportData
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "MPCDI")
	int32 Width = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "MPCDI")
	int32 Height = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "MPCDI")
	TArray<FVector>   Vertices;
};


USTRUCT(BlueprintType, Category = "MPCDI")
struct FMPCDIGeometryExportData
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "MPCDI")
	TArray<FVector>   Vertices;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "MPCDI")
	TArray<FVector>   Normal;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "MPCDI")
	TArray<FVector2D> UV;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "MPCDI")
	TArray<int32>   Triangles;

	void PostAddFace(int32 f0, int32 f1, int32 f2);
};
