// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/DisplayClusterColorEncoding.h"

#include "RHIResources.h"
#include "RenderGraphFwd.h"
#include "ScreenPass.h"

#include "Math/MarginSet.h"

/** Texture utiles flags. */
enum class EDisplayClusterShaderTextureUtilsFlags : uint16
{
	None = 0,

	/**
	* Use only output texture as the source and destination.
	* A temporary texture will be used as the output texture, which is then copied back to the output texture.
	* The input texture is not defined.
	* Custom implementations must perform the copying from the output texture to the input texture themselves.
	*/
	UseOutputTextureAsInput = 1 << 0,

	/**
	* Invert direction: From `Output` to the `Input`
	*/
	InvertDirection = 1 << 1,

	/**
	* Trim the input and output rect sizes to the same value.
	* (The input rect size will be equal to the output rect size.)
	*/
	DisableResize = 1 << 4,

	/**
	* When enabled, bypasses all shader processing and performs a simple texture copy only.
	*/
	DisableResampleShader = 1 << 5,

	/**
	* Dont update resource rects, they are expected to be user defined.
	*/
	DisableUpdateResourcesRectsForResolve = 1 << 6,

	/**
	* Applies a linear alpha fade (feathering) to all edges of the rectangle.
	* Requires specifying the feather (attenuation) width in pixels per side.
	* 
	* NOTE: Only one alpha feathering mode(linear or smooth) should be enabled at a time.
	*       Although these are bit flags, do not combine multiple alpha feather modes.
	* 
	* Note: Enabling this flag disables the use of EColorWriteMask,
	*       so all color channels will be written regardless of mask settings.
	*/
	EnableLinearAlphaFeather = 1 << 8,

	/**
	* Enables smooth (cubic Hermite) alpha feathering (fade-out) on all sides of the rectangle.
	* When set, alpha transitions smoothly from opaque to transparent along each edge using a smoothstep (S-curve) function.
	* Requires additional parameters specifying the feather (attenuation) width in pixels for each side.
	* 
	* NOTE: Only one alpha feathering mode(linear or smooth) should be enabled at a time.
	*       Although these are bit flags, do not combine multiple alpha feather modes.
	* 
	* Note: Enabling this flag disables the use of EColorWriteMask,
	*       so all color channels will be written regardless of mask settings.
	*/
	EnableSmoothAlphaFeather = 1 << 9,
};
ENUM_CLASS_FLAGS(EDisplayClusterShaderTextureUtilsFlags);

/** Alpha channel rendering mode. */
enum class EDisplayClusterShaderTextureUtilsOverrideAlpha : uint8
{
	// Alpha does not change
	None = 0,

	// Invert alpha channel
	Invert_Alpha,

	// set alpha to One(1)
	Set_Alpha_One,

	// Set alpha to Zero(0)
	Set_Alpha_Zero,
};

/*
 * A container with a texture on which a rectangular area is located.
 */
struct FDisplayClusterShadersTextureViewport
{
	FDisplayClusterShadersTextureViewport() = default;

	FDisplayClusterShadersTextureViewport(FRHITexture* InTextureRHI, const TCHAR* InDebugName = nullptr)
		: TextureRHI(InTextureRHI), TextureRDG(nullptr), DebugName(InDebugName) { }

	FDisplayClusterShadersTextureViewport(FRDGTextureRef InTextureRDG)
		: TextureRHI(nullptr), TextureRDG(InTextureRDG) { }

	FDisplayClusterShadersTextureViewport(FRHITexture* InTextureRHI, const FIntRect& InTextureRect, const TCHAR* InDebugName = nullptr)
		: TextureRHI(InTextureRHI), TextureRDG(nullptr), Rect(InTextureRect), DebugName(InDebugName) { }

	FDisplayClusterShadersTextureViewport(FRDGTextureRef InTextureRDG, const FIntRect& InTextureRect)
		: TextureRHI(nullptr), TextureRDG(InTextureRDG), Rect(InTextureRect) { }

	FDisplayClusterShadersTextureViewport(const FScreenPassTexture& InScreenPass)
		: TextureRHI(nullptr), TextureRDG(InScreenPass.Texture), Rect(InScreenPass.ViewRect) { }

	/** Returns true if these parameters are valid. */
	inline bool IsValid() const
	{
		return (TextureRHI || TextureRDG) && !Rect.IsEmpty();
	}

	/** Convert to the ScreenPass texture type. */
	inline FScreenPassTexture ToScreenPassTexture() const
	{
		return FScreenPassTexture(TextureRDG, Rect);
	}

	/** Convert to the ScreenPass viewport type. */
	inline FScreenPassTextureViewport ToScreenPassTextureViewport() const
	{
		return FScreenPassTextureViewport(TextureRDG, Rect);
	}

public:
	/** RHI Texture. If this parameter is nullptr, the TextureRDG parameter is used. */
	FRHITexture* TextureRHI = nullptr;

	/** RDG texture.  */
	FRDGTextureRef TextureRDG = nullptr;

	/** true if RDG texture external. */
	bool bExternalTextureRDG = false;

	/** Texture sub-region */
	FIntRect Rect = { FIntPoint::ZeroValue, FIntPoint::ZeroValue };

	/** Optional debug name for this resource, used to name the external RHI resource for RDG. */
	const TCHAR* DebugName = nullptr;
};

/*
 * A container with textures and their configurations to be used as input and output data.
 */
struct FDisplayClusterShadersTextureParameters
{
	/** Returns true if these parameters are valid. */
	bool IsValid() const
	{
		// All defined textures must be valid.
		for (const TPair<uint32, FDisplayClusterShadersTextureViewport>& TextureIt : TextureViewports)
		{
			if(!TextureIt.Value.IsValid())
			{
				return false;
			}
		}

		// At least one texture must be defined.
		return !TextureViewports.IsEmpty();
	}

public:
	/** Textures of all viewport contexts. The default is context 0. */
	TMap<uint32, FDisplayClusterShadersTextureViewport> TextureViewports;

	/** texture color space. */
	FDisplayClusterColorEncoding ColorEncoding;
};

/*
 * Container with the texture utils settings
 */
struct FDisplayClusterShadersTextureUtilsSettings
{
	FDisplayClusterShadersTextureUtilsSettings() = default;

	FDisplayClusterShadersTextureUtilsSettings(
		const FDisplayClusterShadersTextureUtilsSettings& InSettigns,
		const EDisplayClusterShaderTextureUtilsFlags InFlags,
		const uint32 InDestSliceIndex = 0)
		: FDisplayClusterShadersTextureUtilsSettings(InSettigns)
	{
		Flags = InFlags;
		DestSliceIndex = InDestSliceIndex;
	}

	FDisplayClusterShadersTextureUtilsSettings(const EColorWriteMask InColorMask)
		: ColorMask(InColorMask) { }

	FDisplayClusterShadersTextureUtilsSettings(
		const FDisplayClusterShadersTextureUtilsSettings& InSettigns,
		const EColorWriteMask InColorMask)
		: FDisplayClusterShadersTextureUtilsSettings(InSettigns)
	{
		ColorMask = InColorMask;
	}

	FDisplayClusterShadersTextureUtilsSettings(const EDisplayClusterShaderTextureUtilsOverrideAlpha InOverrideAlpha)
		: OverrideAlpha(InOverrideAlpha) {  }

	FDisplayClusterShadersTextureUtilsSettings(
		const EColorWriteMask InColorMask,
		const EDisplayClusterShaderTextureUtilsFlags InFlags,
		const uint32 InDestSliceIndex = 0)
		: ColorMask(InColorMask), Flags(InFlags), DestSliceIndex(InDestSliceIndex) { }

	FDisplayClusterShadersTextureUtilsSettings(
		const EDisplayClusterShaderTextureUtilsFlags InFlags,
		const FIntMarginSet& InSideMargins)
		: Flags(InFlags), SideMargins(InSideMargins) { }

	FDisplayClusterShadersTextureUtilsSettings(
		const EDisplayClusterShaderTextureUtilsFlags InFlags,
		const uint32 InDestSliceIndex = 0)
		: Flags(InFlags), DestSliceIndex(InDestSliceIndex) {
	}

	inline bool HasAnyFlags(const EDisplayClusterShaderTextureUtilsFlags InFlags) const
	{
		return EnumHasAnyFlags(Flags, InFlags);
	}

	inline void AddFlags(const EDisplayClusterShaderTextureUtilsFlags InFlags)
	{
		return EnumAddFlags(Flags, InFlags);
	}

public:
	/** Color mask*/
	EColorWriteMask ColorMask = EColorWriteMask::CW_RGBA;

	/** Additional flags. */
	EDisplayClusterShaderTextureUtilsFlags Flags = EDisplayClusterShaderTextureUtilsFlags::None;

	/** Override alpha channel. */
	EDisplayClusterShaderTextureUtilsOverrideAlpha OverrideAlpha = EDisplayClusterShaderTextureUtilsOverrideAlpha::None;

	/** Additional parameters for the FRHICopyTextureInfo. */
	uint32 SourceSliceIndex = 0;
	uint32 DestSliceIndex = 0;

	/** Per-side margins in pixels. */
	FIntMarginSet SideMargins;
};

/** This container is used for IDisplayClusterShadersTextureUtils::ForEachContextByPredicate() */
struct FDisplayClusterShadersTextureViewportContext : public FDisplayClusterShadersTextureViewport
{
	FDisplayClusterShadersTextureViewportContext() = default;

	FDisplayClusterShadersTextureViewportContext(
		const FDisplayClusterShadersTextureViewport& InTextureViewport,
		const uint32 InContextNum = 0)
		: FDisplayClusterShadersTextureViewport(InTextureViewport), ContextNum(InContextNum) { }
	
	FDisplayClusterShadersTextureViewportContext(
		const FDisplayClusterShadersTextureViewport& InTextureViewport,
		const FDisplayClusterColorEncoding& InColorEncoding,
		const uint32 InContextNum = 0)
		: FDisplayClusterShadersTextureViewport(InTextureViewport), ContextNum(InContextNum), ColorEncoding(InColorEncoding){ }

	/** Texture context num. */
	uint32 ContextNum = 0;

	/** texture color space. */
	FDisplayClusterColorEncoding ColorEncoding;
};

class IDisplayClusterShadersTextureUtils;
using TFunctionDisplayClusterShaders_TextureContextIterator = TFunction<void(
	const FDisplayClusterShadersTextureViewportContext& Input,
	const FDisplayClusterShadersTextureViewportContext& Output)>;
