// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * Rendering mode for chromakey
 */
enum class EDisplayClusterShaderParametersICVFX_ChromakeySource : uint8
{
	// Dont use chromakey
	Disabled = 0,

	// Render color over camera frame
	FrameColor,

	// Render specified layer from scene
	ChromakeyLayers,

	// Use default value.
	Default,
};

/**
 * The rendering order of the lightcard layer
 */
enum class EDisplayClusterShaderParametersICVFX_LightCardRenderMode : uint8
{
	None = 0,

	// Render incamera frame over lightcard
	Under,

	// Over lightcard over incamera frame
	Over,
};

/**
 * Rendering mode for overlapping areas of camera projections
 */
enum class EDisplayClusterShaderParametersICVFX_CameraOverlappingRenderMode : uint8
{
	// Disabled
	None = 0,

	// The overlapping area is rendered at the very end, on top of all the layers
	FinalPass
};
