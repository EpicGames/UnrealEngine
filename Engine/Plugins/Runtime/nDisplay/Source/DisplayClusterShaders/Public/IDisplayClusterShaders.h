// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#include "RHI.h"
#include "RHICommandList.h"
#include "RHIResources.h"

class IDisplayClusterRender_MeshComponentProxy;
class FDisplayClusterShaderParameters_ICVFX;
class FDisplayClusterShaderParameters_Override;
class IDisplayClusterShadersTextureUtils;

class FSceneInterface;
class FRDGBuilder;
class FRenderTarget;

struct FDisplayClusterShaderParameters_GenerateMips;
struct FDisplayClusterShaderParameters_MediaPQ;
struct FDisplayClusterShaderParameters_Overlay;
struct FDisplayClusterShaderParameters_PostprocessBlur;
struct FDisplayClusterShaderParameters_TransformTexture;
struct FDisplayClusterShaderParameters_UVLightCards;
struct FDisplayClusterShaderParameters_WarpBlend;


class IDisplayClusterShaders : public IModuleInterface
{
public:
	static constexpr const TCHAR* ModuleName = TEXT("DisplayClusterShaders");

public:
	virtual ~IDisplayClusterShaders() = default;

public:
	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IDisplayClusterShaders& Get()
	{
		return FModuleManager::LoadModuleChecked<IDisplayClusterShaders>(IDisplayClusterShaders::ModuleName);
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded(IDisplayClusterShaders::ModuleName);
	}

public:
	/**
	* Render warp&blend
	*
	* @param RHICmdList      - RHI command list
	* @param InWarpBlendParameters - warp blend parameters
	*
	* @return - true if success
	*/
	virtual bool RenderWarpBlend_MPCDI(FRHICommandListImmediate& RHICmdList, const FDisplayClusterShaderParameters_WarpBlend& InWarpBlendParameters) const = 0;

	/**
	* Render ICVFX warp&blend
	*
	* @param RHICmdList            - RHI command list
	* @param InWarpBlendParameters - WarpBlend parameters
	* @param InICVFXParameters     - ICVFX parameters
	*
	* @return - true if success
	*/
	virtual bool RenderWarpBlend_ICVFX(FRHICommandListImmediate& RHICmdList, const FDisplayClusterShaderParameters_WarpBlend& InWarpBlendParameters, const FDisplayClusterShaderParameters_ICVFX& InICVFXParameters) const = 0;

	/**
	* Render UV light cards to texture that can be sampled by viewports to place light cards in UV space
	*
	* @param RHICmdList                    - RHI command list
	* @param InScene                       - The scene the light cards live in
	* @param InRenderTargetableDestTexture - Destination RTT texture
	* @param ProjectionPlaneSize           - The size of the plane the UV light cards are projected to
	*
	* @return - true if success
	*/
	UE_DEPRECATED(5.2, "Use version that takes FDisplayClusterShaderParameters_UVLightCards instead")
	virtual bool RenderPreprocess_UVLightCards(FRHICommandListImmediate& RHICmdList, FSceneInterface* InScene, FRenderTarget* InRenderTarget, float ProjectionPlaneSize, bool bRenderFinalColor) const = 0;

	/**
	* Render UV light cards to texture that can be sampled by viewports to place light cards in UV space
	*
	* @param RHICmdList                    - RHI command list
	* @param InScene                       - The scene the light cards live in
	* @param InRenderTarget                - Destination RTT texture
	* @param InParameters                  - Parameters for the render
	*
	* @return - true if success
	*/
	virtual bool RenderPreprocess_UVLightCards(FRHICommandListImmediate& RHICmdList, FSceneInterface* InScene, FRenderTarget* InRenderTarget, const FDisplayClusterShaderParameters_UVLightCards& InParameters) const = 0;

	/**
	* Render postprocess OutputRemap
	*
	* @param RHICmdList                    - RHI command list
	* @param InSourceTexture -             - Source shader resource texture
	* @param InRenderTargetableDestTexture - Destination RTT texture
	* @param MeshProxy                     - mesh for output remapping
	*
	* @return - true if success
	*/
	virtual bool RenderPostprocess_OutputRemap(FRHICommandListImmediate& RHICmdList, FRHITexture* InSourceTexture, FRHITexture* InRenderTargetableDestTexture, const IDisplayClusterRender_MeshComponentProxy& MeshProxy) const = 0;

	/**
	* Render postprocess Blur
	*
	* @param RHICmdList                    - RHI command list
	* @param InSourceTexture -             - Source shader resource texture
	* @param InRenderTargetableDestTexture - Destination RTT texture
	* @param InSettings                    - Blur pp settings
	*
	* @return - true if success
	*/
	virtual bool RenderPostprocess_Blur(FRHICommandListImmediate& RHICmdList, FRHITexture* InSourceTexture, FRHITexture* InRenderTargetableDestTexture, const FDisplayClusterShaderParameters_PostprocessBlur& InSettings) const = 0;

	/**
	* Generate Mips texture
	*
	* @param RHICmdList       - RHI command list
	* @param InOutMipsTexture - Mips textures with assigned mip0
	* @param InSettings       - mips settings
	*
	* @return - true if success
	*/
	virtual bool GenerateMips(FRHICommandListImmediate& RHICmdList, FRHITexture* InOutMipsTexture, const FDisplayClusterShaderParameters_GenerateMips& InSettings) const = 0;

	/**
	* Adds Linear-To-PQ encoding pass
	*
	* @param GraphBuilder - RDG builder
	* @param Parameters   - Conversion parameters
	*/
	virtual void AddLinearToPQPass(FRDGBuilder& GraphBuilder, const FDisplayClusterShaderParameters_MediaPQ& Parameters) const = 0;

	/**
	* Adds PQ-To-Linear decoding pass (API wrapper)
	*
	* @param GraphBuilder - RDG builder
	* @param Parameters   - Conversion parameters
	*/
	virtual void AddPQToLinearPass(FRDGBuilder& GraphBuilder, const FDisplayClusterShaderParameters_MediaPQ& Parameters) const = 0;

	/**
	* Adds overlay drawing pass
	*
	* @param GraphBuilder - RDG builder
	* @param Parameters   - Render parameters
	*/
	virtual void AddDrawOverlayPass(FRDGBuilder& GraphBuilder, const FDisplayClusterShaderParameters_Overlay& Parameters) const = 0;

	/**
	* Adds a texture transformation pass
	*
	* @param GraphBuilder - RDG builder
	* @param Parameters   - Render parameters
	*/
	virtual void AddTransformTexturePass(FRDGBuilder& GraphBuilder, const FDisplayClusterShaderParameters_TransformTexture& Parameters) const = 0;

	/**
	* Return new instance of the resource utils.
	* 
	* @param RHICmdList   - RHI API
	*/
	virtual TSharedRef<IDisplayClusterShadersTextureUtils> CreateTextureUtils_RenderThread(FRHICommandListImmediate& RHICmdList) const = 0;

	/**
	* Return new instance of the resource utils.
	* @param GraphBuilder - RDG builder
	*/
	virtual TSharedRef<IDisplayClusterShadersTextureUtils> CreateTextureUtils_RenderThread(FRDGBuilder& GraphBuilder) const = 0;
};
