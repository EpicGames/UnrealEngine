// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/DisplayClusterShaderContainers_TextureUtils.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_Enums.h"

#include "RHICommandList.h"
#include "RenderGraphBuilder.h"

class IDisplayClusterViewportProxy;

/**
 * Auxiliary Texture utils class for nDisplay.
 */
class IDisplayClusterShadersTextureUtils
{
public:
	/** Default destructor. */
	virtual ~IDisplayClusterShadersTextureUtils() = default;

	/**
	*  Set the input texture viewport for the specified context.
	*
	* @parama InTextureViewport - texture viewport
	* @param InContextNum       - (opt) context(eye)  number, otherwise all contexts will be collected.
	*
	* @return themself
	*/
	virtual TSharedRef<IDisplayClusterShadersTextureUtils> SetInput(const FDisplayClusterShadersTextureViewport& InTextureViewport, const int32 InContextNum = INDEX_NONE) = 0;

	/**
	*  Set the output texture viewport for the specified context.
	*
	* @parama InTextureViewport - texture viewport
	* @param InContextNum       - (opt) context(eye)  number, otherwise all contexts will be collected.
	*
	* @return themself
	*/
	virtual TSharedRef<IDisplayClusterShadersTextureUtils> SetOutput(const FDisplayClusterShadersTextureViewport& InTextureViewport, const int32 InContextNum = INDEX_NONE) = 0;

	/** Set input texture from a viewport proxy object.
	* 
	* @param InViewportProxy  - viewport proxy object with resources to retrieve
	* @param InResourceType   - Type of the viewport resource
	* @param InContextNum       - (opt) context(eye)  number, otherwise all contexts will be collected.
	*
	* @return themself
	*/
	virtual TSharedRef<IDisplayClusterShadersTextureUtils> SetInput(
		const IDisplayClusterViewportProxy* InViewportProxy,
		const EDisplayClusterViewportResourceType InResourceType,
		const int32 InContextNum = INDEX_NONE) = 0;

	/** Set output texture from a viewport proxy object.
	*
	* @param InViewportProxy  - viewport proxy object with resources to retrieve
	* @param InResourceType   - Type of the viewport resource
	* @param InContextNum       - (opt) context(eye)  number, otherwise all contexts will be collected.
	*
	* @return themself
	*/
	virtual TSharedRef<IDisplayClusterShadersTextureUtils> SetOutput(
		const IDisplayClusterViewportProxy* InViewportProxy,
		const EDisplayClusterViewportResourceType InResourceType,
		const int32 InContextNum = INDEX_NONE) = 0;

	/** Set input color encodig for texture
	*
	* @param InColorEncoding  - color encoding
	*
	* @return themself
	*/
	virtual TSharedRef<IDisplayClusterShadersTextureUtils> SetInputEncoding(const FDisplayClusterColorEncoding& InColorEncoding) = 0;

	/** Set output color encodig for texture
	*
	* @param InColorEncoding  - color encoding
	*
	* @return themself
	*/
	virtual TSharedRef<IDisplayClusterShadersTextureUtils> SetOutputEncoding(const FDisplayClusterColorEncoding& InColorEncoding) = 0;


	/** Return input texture parameters. */
	virtual const FDisplayClusterShadersTextureParameters& GetInputTextureParameters() const = 0;

	/** Return input texture parameters. */
	virtual const FDisplayClusterShadersTextureParameters& GetOutputTextureParameters() const = 0;

	/**
	* Resolve all contexts
	* 
	* @return themself
	*/
	virtual TSharedRef<IDisplayClusterShadersTextureUtils> Resolve() = 0;

	/**
	* Resolve all contexts
	*
	* @param ColorMask - Color Recording Mask. Used to copy only selected channels.
	* @param InSettings - (in) render settings
	*
	* @return themself
	*/
	virtual TSharedRef<IDisplayClusterShadersTextureUtils> Resolve(const FDisplayClusterShadersTextureUtilsSettings& InSettings) = 0;

	/**
	* Resolve one context to the other
	*
	* @param Input  - input texture context
	* @param Output - output texture context
	*
	* @return - themself
	*/
	virtual TSharedRef<IDisplayClusterShadersTextureUtils> ResolveTextureContext(
		const FDisplayClusterShadersTextureViewportContext& Input,
		const FDisplayClusterShadersTextureViewportContext& Output) = 0;

	/**
	* Resolve one context to the other
	*
	* @param InSettings - (in) render settings
	* @param Input      - input texture context
	* @param Output     - output texture context
	*
	* @return - themself
	*/
	virtual TSharedRef<IDisplayClusterShadersTextureUtils> ResolveTextureContext(
		const FDisplayClusterShadersTextureUtilsSettings& InSettings,
		const FDisplayClusterShadersTextureViewportContext& Input,
		const FDisplayClusterShadersTextureViewportContext& Output) = 0;

	/**
	* Custom implementation of an action between input and output contexts.
	* Iterate over all contexts on input and output textures.
	* Note: This function creates unique TextureContexts instances for the iterator arguments.
	*       Multiple calls to ForEachContextByPredicate() will create unique contexts for each use.
	* 
	* @param TextureContextIteratorFunc - predicate function that processes pairs of input and output textures (via resolve or any other way).
	* 
	* @ returns themself
	*/
	virtual TSharedRef<IDisplayClusterShadersTextureUtils> ForEachContextByPredicate(TFunctionDisplayClusterShaders_TextureContextIterator&& TextureContextIteratorFunc) = 0;

	/**
	* Custom implementation of an action between input and output contexts.
	* Iterate over all contexts on input and output textures.
	* Note: This function creates unique TextureContexts instances for the iterator arguments.
	*       Multiple calls to ForEachContextByPredicate() will create unique contexts for each use.
	*
	* @param InSettings - (in) render settings
	* @param TextureContextIteratorFunc - predicate function that processes pairs of input and output textures (via resolve or any other way).
	*
	* @ returns themself
	*/
	virtual TSharedRef<IDisplayClusterShadersTextureUtils> ForEachContextByPredicate(
		const FDisplayClusterShadersTextureUtilsSettings& InSettings,
		TFunctionDisplayClusterShaders_TextureContextIterator&& TextureContextIteratorFunc) = 0;

	/**
	* Get or create RDG builder for this texture utils instance
	* Instances created for RHI after this call will only use RDG.
	*/
	virtual FRDGBuilder& GetOrCreateRDGBuilder() = 0;
};