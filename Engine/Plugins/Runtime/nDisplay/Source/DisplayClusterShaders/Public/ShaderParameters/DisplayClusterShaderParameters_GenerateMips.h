// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "RHI.h"
#include "RHIResources.h"
#include "Engine/Texture.h"

struct FDisplayClusterShaderParameters_GenerateMips
{
public:
	// Allow autogenerate num mips for this target
	bool bAutoGenerateMips = false;

	// Control mips generator settings
	TEnumAsByte<enum TextureFilter>  MipsSamplerFilter = TF_Trilinear;
	TEnumAsByte<enum TextureAddress> MipsAddressU = TA_Clamp;
	TEnumAsByte<enum TextureAddress> MipsAddressV = TA_Clamp;

	// Positive values used as max NumMips value
	int32 MaxNumMipsLimit = -1;

public:
	inline void Reset()
	{
		bAutoGenerateMips = false;
	}

	inline bool IsEnabled() const
	{
		return bAutoGenerateMips && (MaxNumMipsLimit < 0 || MaxNumMipsLimit > 1);
	}

	inline int32 GetRequiredNumMips(const FIntPoint& InDim) const
	{
		int32 CurrentNumMips = 0;

		if (IsEnabled())
		{
			// Make sure we only set a number of mips that actually makes sense, given the sample size
			CurrentNumMips = FGenericPlatformMath::FloorToInt(
				FGenericPlatformMath::Log2(float(FGenericPlatformMath::Min(InDim.X, InDim.Y)))
			);

			const int32 MipsLimit = MaxNumMipsLimit;
			if (MipsLimit >= 0)
			{
				CurrentNumMips = FMath::Min(CurrentNumMips, MipsLimit);
			}
		}

		return CurrentNumMips;
	}
};

