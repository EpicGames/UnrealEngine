// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/DisplayClusterShader_Enums.h"
#include "RHI.h"

/**
 * Camera view projection data
 */
struct FDisplayClusterShaderParametersICVFX_CameraViewProjection
{
	FRotator ViewRotation;
	FVector  ViewLocation;
	FMatrix  PrjMatrix;
};

/**
 * ICVFX rendering uses the resources of other viewports
 * During initialization on a game thread, only the name of the viewport is saved here
 * Later on the rendering thread, the resource reference is initialized
 */
struct FDisplayClusterShaderParametersICVFX_ViewportResource
{
public:
	bool IsValid() const
	{
		return Texture != nullptr && Texture->IsValid();
	}

	bool IsDefined() const
	{
		return ViewportId.IsEmpty() == false;
	}

	void Reset()
	{
		ViewportId.Empty();
		Texture = nullptr;
	}

public:
	// Viewport name (Used to resolve viewport resource to texture ref)
	FString      ViewportId;

	// This ref resolved runtime
	FRHITexture* Texture = nullptr;
};

/**
 * This is where the ICVFX data for the outer viewport is stored.
 */
class FDisplayClusterShaderParameters_ICVFX
{
public:
	FDisplayClusterShaderParameters_ICVFX()
	{ }

public:
	inline bool IsLightCardOverUsed() const
	{
		return LightCardOver.IsValid();
	}

	inline bool IsLightCardUnderUsed() const
	{
		return LightCardUnder.IsValid();
	}

	inline bool IsUVLightCardOverUsed() const
	{
		return UVLightCardOver.IsValid();
	}

	inline bool IsUVLightCardUnderUsed() const
	{
		return UVLightCardUnder.IsValid();
	}

	inline bool IsCameraUsed(int32 CameraIndex) const
	{
		return Cameras.IsValidIndex(CameraIndex) && Cameras[CameraIndex].IsUsed();
	}

	inline bool IsAnyCameraUsed() const
	{
		return !Cameras.IsEmpty();
	}

	inline bool IsMultiCamerasUsed() const
	{
		return Cameras.Num() > 1;
	}

	inline bool IsValid() const
	{
		if (LightCardOver.IsValid() || UVLightCardOver.IsValid() || LightCardUnder.IsValid() || UVLightCardUnder.IsValid())
		{
			return true;
		}

		return IsAnyCameraUsed();
	}

public:
	void Reset()
	{
		Cameras.Empty();

		LightCardUnder.Reset();
		LightCardOver.Reset();
		UVLightCardUnder.Reset();
		UVLightCardOver.Reset();
		EnforcedCameraOrder.Reset();

		OverrideUVLightCardRenderMode = EDisplayClusterShaderParametersICVFX_LightCardRenderMode::None;
		LightCardGamma = 2.2f;
	}

	// Implement copy ref and arrays
	void SetParameters(const FDisplayClusterShaderParameters_ICVFX& InParameters)
	{
		Reset();

		Cameras = InParameters.Cameras;

		CameraOverlappingRenderMode = InParameters.CameraOverlappingRenderMode;

		LightCardUnder = InParameters.LightCardUnder;
		LightCardOver = InParameters.LightCardOver;

		UVLightCardUnder = InParameters.UVLightCardUnder;
		UVLightCardOver = InParameters.UVLightCardOver;
		OverrideUVLightCardRenderMode = InParameters.OverrideUVLightCardRenderMode;

		LightCardGamma = InParameters.LightCardGamma;

		EnforcedCameraOrder = InParameters.EnforcedCameraOrder;
	}

	inline void SortCamerasRenderOrder(const bool bReverseOrder = false)
	{
		// Default sorting: by RenderOrder, then alphabetically by ViewportId
		auto DefaultCameraSort = [](const FCameraSettings& It1, const FCameraSettings& It2)
		{
			if (It1.RenderOrder == It2.RenderOrder)
			{
				return It1.Resource.ViewportId.Compare(It2.Resource.ViewportId, ESearchCase::IgnoreCase) < 0;
			}
			return It1.RenderOrder < It2.RenderOrder;
		};

		// If we have an enforced camera order, apply it first
		if (!EnforcedCameraOrder.IsEmpty())
		{
			TArray<FCameraSettings> EnforcedCameras;
			TArray<FCameraSettings> RemainingCameras;

			// Separate cameras into enforced and remaining groups
			for (const FCameraSettings& CameraIt : Cameras)
			{
				const int32 EnforcedIndex = EnforcedCameraOrder.IndexOfByKey(CameraIt.Resource.ViewportId);
				if (EnforcedIndex != INDEX_NONE)
				{
					EnforcedCameras.Add(CameraIt);
				}
				else
				{
					RemainingCameras.Add(CameraIt);
				}
			}

			// Sort remaining cameras using default logic
			RemainingCameras.Sort(DefaultCameraSort);

			// Sort enforced cameras by their position in the EnforcedCameraOrder array
			// Index 0 in EnforcedCameraOrder composes on top, and conveniently appears at top in the details panel.
			// Note: The final order will be reversed if the bReverseOrder option is used.
			EnforcedCameras.Sort([this](const FCameraSettings& It1, const FCameraSettings& It2)
			{
				const int32 Index1 = EnforcedCameraOrder.IndexOfByKey(It1.Resource.ViewportId);
				const int32 Index2 = EnforcedCameraOrder.IndexOfByKey(It2.Resource.ViewportId);
				return Index1 > Index2; // Reverse: lower index renders later (on top)
			});

			// Combine EnforcedCameras with RemainingCameras. EnforcedCameras fill the top of the stack,
			// while the remaining ones are appended and preserve their original relative order.
			Cameras = RemainingCameras;
			Cameras.Append(EnforcedCameras);
		}
		else
		{
			// No enforced order, use default sorting
			Cameras.Sort(DefaultCameraSort);
		}

		// Apply reverse order as the final step
		if (bReverseOrder)
		{
			Algo::Reverse(Cameras);
		}
	}

public:
	/**
	 * Incamera render settings
	 */
	struct FCameraSettings
	{
		inline bool IsUsed() const
		{
			return (ChromakeySource == EDisplayClusterShaderParametersICVFX_ChromakeySource::FrameColor) || Resource.IsValid();
		}

		inline bool IsChromakeyMarkerUsed() const
		{
			return ChromakeMarkerTextureRHI.IsValid();
		}

		/** Gets whether a valid texture is available to render chromakey markers for any overlapping ICVFX frustums  */
		inline bool IsOverlapChromakeyMarkerUsed() const
		{
			return OverlapChromakeyMarkerTextureRHI.IsValid();
		}

		inline void SetViewProjection(const FDisplayClusterShaderParametersICVFX_CameraViewProjection& InCameraViewProjection, const FTransform& InOrigin2WorldTransform)
		{
			// Transforming the camera view from "world" space to "origin" space.
			// The "origin" is the local space for the warp geometry.
			ViewProjection.ViewRotation = InOrigin2WorldTransform.InverseTransformRotation(InCameraViewProjection.ViewRotation.Quaternion()).Rotator();
			ViewProjection.ViewLocation = InOrigin2WorldTransform.InverseTransformPosition(InCameraViewProjection.ViewLocation);
			ViewProjection.PrjMatrix = InCameraViewProjection.PrjMatrix;
		}

		/**
		 * Iterate over all defined viewport resources with a predicate functor.
		 */
		template <typename Predicate>
		void IterateViewportResourcesByPredicate(Predicate Pred)
		{
			if (Resource.IsDefined())
			{
				::Invoke(Pred, Resource);
			}

			if (Chromakey.IsDefined())
			{
				::Invoke(Pred, Chromakey);
			}
		}

		/**
		 * Copying camera settings from the source camera.
		 * The function is moved here from the DisplayClusterMedia module, because it must also be updated when new settings are added.
		 * 
		 * @param InCameraSettings - source camera settings
		 * @param bIncludeResources - enables resource copying
		 */
		inline void SetCameraSettings(const FCameraSettings& InCameraSettings, const bool bIncludeResources)
		{
			if (bIncludeResources)
			{
				Resource = InCameraSettings.Resource;
				Chromakey = InCameraSettings.Chromakey;
				ChromakeMarkerTextureRHI = InCameraSettings.ChromakeMarkerTextureRHI;
				OverlapChromakeyMarkerTextureRHI = InCameraSettings.OverlapChromakeyMarkerTextureRHI;
			}

			SoftEdge = InCameraSettings.SoftEdge;

			InnerCameraBorderColor      = InCameraSettings.InnerCameraBorderColor;
			InnerCameraBorderThickness  = InCameraSettings.InnerCameraBorderThickness;

			ViewProjection = InCameraSettings.ViewProjection;

			ChromakeySource = InCameraSettings.ChromakeySource;
			ChromakeyColor = InCameraSettings.ChromakeyColor;
			OverlapChromakeyColor = InCameraSettings.OverlapChromakeyColor;

			ChromakeyMarkersColor = InCameraSettings.ChromakeyMarkersColor;
			ChromakeyMarkersScale = InCameraSettings.ChromakeyMarkersScale;
			ChromakeyMarkersDistance = InCameraSettings.ChromakeyMarkersDistance;
			ChromakeyMarkersOffset = InCameraSettings.ChromakeyMarkersOffset;

			OverlapChromakeyMarkersColor = InCameraSettings.OverlapChromakeyMarkersColor;
			OverlapChromakeyMarkersScale = InCameraSettings.OverlapChromakeyMarkersScale;
			OverlapChromakeyMarkersDistance = InCameraSettings.OverlapChromakeyMarkersDistance;
			OverlapChromakeyMarkersOffset = InCameraSettings.OverlapChromakeyMarkersOffset;

			RenderOrder = InCameraSettings.RenderOrder;
		}

	public:
		// resource with the camera image
		FDisplayClusterShaderParametersICVFX_ViewportResource Resource;

		FVector4 SoftEdge;

		FLinearColor InnerCameraBorderColor = FLinearColor::Black;
		float InnerCameraBorderThickness = 0.1f;

		// Camera view projection data
		FDisplayClusterShaderParametersICVFX_CameraViewProjection ViewProjection;

		// Chromakey settings:
		EDisplayClusterShaderParametersICVFX_ChromakeySource  ChromakeySource = EDisplayClusterShaderParametersICVFX_ChromakeySource::Disabled;
		FDisplayClusterShaderParametersICVFX_ViewportResource Chromakey;
		FLinearColor ChromakeyColor = FLinearColor::Black;

		/** The color to use when rendering chromakey for any regions of overlapping ICVFX frustums */
		FLinearColor OverlapChromakeyColor = FLinearColor::Black;

		// Chromakey markers settings:
		FLinearColor ChromakeyMarkersColor = FLinearColor::Black;
		float ChromakeyMarkersScale;
		float ChromakeyMarkersDistance;
		FVector2D ChromakeyMarkersOffset;
		FTextureRHIRef ChromakeMarkerTextureRHI;

		/** The color of the chroamkey markers for any regions of overlapping ICVFX frustums */
		FLinearColor OverlapChromakeyMarkersColor = FLinearColor::Black;

		/** The scale of the chroamkey markers for any regions of overlapping ICVFX frustums */
		float OverlapChromakeyMarkersScale;

		/** The distance between the chroamkey markers for any regions of overlapping ICVFX frustums */
		float OverlapChromakeyMarkersDistance;

		/** The offset of the chroamkey markers for any regions of overlapping ICVFX frustums */
		FVector2D OverlapChromakeyMarkersOffset;

		/** The texture to use to render the chromakey markers for any regions of overlaping ICVFX frustums */
		FTextureRHIRef OverlapChromakeyMarkerTextureRHI;

		/**
		 * Camera render order priority. Lower values render first (appear underneath), higher values render last (appear on top).
		 * When multiple cameras have the same RenderOrder, they are sorted alphabetically by viewport ID.
		 * Can be overridden per-viewport using EnforcedCameraOrder array.
		 */
		int32 RenderOrder = -1;
	};

	/**
	 * Iterate over all defined viewport resources with a predicate functor.
	 */
	template <typename Predicate>
	void IterateViewportResourcesByPredicate(Predicate Pred)
	{
		if (LightCardUnder.IsDefined())
		{
			::Invoke(Pred, LightCardUnder);
		}

		if (LightCardOver.IsDefined())
		{
			::Invoke(Pred, LightCardOver);
		}

		if (UVLightCardUnder.IsDefined())
		{
			::Invoke(Pred, UVLightCardUnder);
		}

		if (UVLightCardOver.IsDefined())
		{
			::Invoke(Pred, UVLightCardOver);
		}

		for (FCameraSettings& CameraIt : Cameras)
		{
			CameraIt.IterateViewportResourcesByPredicate(Pred);
		}
	}

	/**
	 * Find camera settings by viewport name
	 */
	inline FCameraSettings* FindCameraByName(const FString& InViewportId)
	{
		return Cameras.FindByPredicate([InViewportId](const FCameraSettings& CameraIt)
			{
				return CameraIt.Resource.ViewportId == InViewportId;
			});
	}

	// Remove unused cameras from render
	inline bool CleanupCamerasForRender()
	{
		const TArray<FCameraSettings> InCameras = Cameras;
		Cameras.Empty();

		for (const FCameraSettings& CameraIt : InCameras)
		{
			if (CameraIt.IsUsed())
			{
				Cameras.Add(CameraIt);
			}
		}

		return Cameras.Num() == InCameras.Num();
	}

	// All cameras that render on this viewport
	TArray<FCameraSettings> Cameras;

	// Rendering mode for overlapping areas of camera projections
	EDisplayClusterShaderParametersICVFX_CameraOverlappingRenderMode CameraOverlappingRenderMode = EDisplayClusterShaderParametersICVFX_CameraOverlappingRenderMode::None;

	// LightCard resources
	FDisplayClusterShaderParametersICVFX_ViewportResource    LightCardUnder;
	FDisplayClusterShaderParametersICVFX_ViewportResource    LightCardOver;

	// UV LightCard resources
	FDisplayClusterShaderParametersICVFX_ViewportResource    UVLightCardUnder;
	FDisplayClusterShaderParametersICVFX_ViewportResource    UVLightCardOver;

	// Force UVLC to render merged (per-viewport LC)
	EDisplayClusterShaderParametersICVFX_LightCardRenderMode OverrideUVLightCardRenderMode = EDisplayClusterShaderParametersICVFX_LightCardRenderMode::None;

	// The gamma that the light card renders have been encoded with, used to linearize during final composite
	float LightCardGamma = 2.2f;

	// Enforced camera order for custom sorting. Contains the camera viewport ids, where index 0 should be composed on top, index 1 below it, and so on. 
	TArray<FString> EnforcedCameraOrder;
};
