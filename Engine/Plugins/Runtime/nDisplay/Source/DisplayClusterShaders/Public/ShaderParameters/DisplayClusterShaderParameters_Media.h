// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FRDGTexture;


/*
 * A container to pass PQ coding & decoding parameters
 */
struct FDisplayClusterShaderParameters_MediaPQ
{
	/** Source texture for coding/decoding */
	FRDGTexture* InputTexture = nullptr;

	/** Source texture sub-region */
	FIntRect InputRect = { FIntPoint::ZeroValue, FIntPoint::ZeroValue };

	/** Output texture for coding/decoding */
	FRDGTexture* OutputTexture = nullptr;

	/** Output texture sub-region */
	FIntRect OutputRect = { FIntPoint::ZeroValue, FIntPoint::ZeroValue };
};
