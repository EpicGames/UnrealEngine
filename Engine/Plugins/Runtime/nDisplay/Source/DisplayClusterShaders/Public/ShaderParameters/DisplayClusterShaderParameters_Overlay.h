// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FRDGTexture;


/*
 * Holds overlay drawing parameters
 */
struct FDisplayClusterShaderParameters_Overlay
{
	/** Base texture (bottom layer) */
	FRDGTexture* BaseTexture = nullptr;

	/** Overlay texture (top layer) */
	FRDGTexture* OverlayTexture = nullptr;

	/** Output texture (the result of source-over blending) */
	FRDGTexture* OutputTexture = nullptr;

public:

	/** Parameters validation */
	bool IsValidData() const
	{
		return BaseTexture && OverlayTexture && OutputTexture;
	}
};
