// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "RHI.h"
#include "RHIResources.h"

class FDisplayClusterShaderParameters_Override
{
public:
	FDisplayClusterShaderParameters_Override()
	{}

	~FDisplayClusterShaderParameters_Override()
	{
		TextureRHI.SafeRelease();
	}

public:
	inline bool IsEnabled() const
	{
		return TextureRHI.IsValid();
	}

	// Copy with resource ref
	inline void SetParameters(const FDisplayClusterShaderParameters_Override& InParameters)
	{
		TextureRHI.SafeRelease();
		TextureRHI = InParameters.TextureRHI;

		Rect = InParameters.Rect;
	}

	inline void Reset()
	{
		TextureRHI.SafeRelease();
	}

public:
	FTextureRHIRef   TextureRHI;
	FIntRect         Rect;
};

