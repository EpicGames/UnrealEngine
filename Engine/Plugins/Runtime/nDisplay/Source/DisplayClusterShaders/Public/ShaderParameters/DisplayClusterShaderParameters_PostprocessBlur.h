// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "RHI.h"
#include "RHIResources.h"

enum class EDisplayClusterShaderParameters_PostprocessBlur : uint8
{
	// Not use blur postprocess
	None = 0,

	// Blur viewport using Gaussian method
	Gaussian,

	// Blur viewport using Dilate method
	Dilate,
};

struct FDisplayClusterShaderParameters_PostprocessBlur
{
	EDisplayClusterShaderParameters_PostprocessBlur Mode = EDisplayClusterShaderParameters_PostprocessBlur::None;
	int32   KernelRadius = 1;
	float KernelScale = 1;

public:
	inline void Reset()
	{
		Mode = EDisplayClusterShaderParameters_PostprocessBlur::None;
	}

	inline bool IsEnabled() const
	{
		return Mode != EDisplayClusterShaderParameters_PostprocessBlur::None;
	}
};

