// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RenderGraphResources.h"


/**
 * TransformTexture shader parameters
 */
struct FDisplayClusterShaderParameters_TransformTexture
{
public:

	/**
	 * Supported texture transformations
	 */
	enum class ETranformation : uint8
	{
		None, // No transformation
		Rotation_90,  // Rotate  90 deg CW (same as 270 CCW)
		Rotation_180, // Rotate 180 deg CW (same as 180 CCW)
		Rotation_270, // Rotate 270 deg CW (same as  90 CCW)
		Flip_H, // Horizontal mirroring
		Flip_V, // Vertical mirroring
	};

public:

	/** Input texture */
	FRDGTexture* InputTexture = nullptr;

	/** Subregion to extract and transform, or whole texture if not set */
	TOptional<FIntRect> InputRegion;

	/** Transformation type to apply */
	ETranformation TranformationType = ETranformation::None;

	/** Output texture. If not set, a new one will be allocated */
	mutable FRDGTexture* OutputTexture = nullptr;

public:

	/** Parameters validation */
	bool IsValidData() const
	{
		// Validate input texture
		if (!InputTexture)
		{
			return false;
		}

		// Validate input subregion if specified
		if (InputRegion.IsSet())
		{
			const FIntRect& InputRect = InputRegion.GetValue();

			const bool bValidRegion =
				InputRect.Min.X >= 0 &&
				InputRect.Min.Y >= 0 &&
				InputRect.Max.X <= InputTexture->Desc.Extent.X &&
				InputRect.Max.Y <= InputTexture->Desc.Extent.Y &&
				InputRect.Width() > 0 &&
				InputRect.Height() > 0;

			if (!bValidRegion)
			{
				return false;
			}
		}

		return true;
	}
};
