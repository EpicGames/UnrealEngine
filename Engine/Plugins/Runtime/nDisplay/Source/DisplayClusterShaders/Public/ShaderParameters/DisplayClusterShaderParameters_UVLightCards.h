// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FPrimitiveSceneProxy;

/** A data struct that contains parameters for performing the UV light card render pass */
struct FDisplayClusterShaderParameters_UVLightCards
{
	/** A list of primitive scene proxies to render */
	TArray<FPrimitiveSceneProxy*> PrimitivesToRender;

	/** The size of the plane in world units to project the UV space onto */
	float ProjectionPlaneSize = 200.0f;
};
