// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "RHI.h"
#include "RHIResources.h"

#include "Containers/DisplayClusterWarpContext.h"

struct FDisplayClusterShaderParameters_WarpBlend
{
	struct FResourceWithRect
	{
		FRHITexture* Texture;
		FIntRect       Rect;

		void Set(FRHITexture* InTexture, const FIntRect& InRect)
		{
			Texture = InTexture;
			Rect = InRect;
		}
	};

	// In\Out resources for warp
	FResourceWithRect Src;
	FResourceWithRect Dest;

	// Warp interface
	TSharedPtr<class IDisplayClusterWarpBlend, ESPMode::ThreadSafe> WarpInterface;

	// Context data
	FDisplayClusterWarpContext Context;

	// Render alpha channel from input texture to warp output
	bool bRenderAlphaChannel = false;
};
