// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class DisplayClusterStageMonitoring : ModuleRules
{
	public DisplayClusterStageMonitoring(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"DeveloperSettings",
				"DisplayCluster",
				"DisplayClusterConfiguration",
				"Engine",
				"RenderCore",
				"RHI",
				"StageDataCore",
			});

		PrivateIncludePathModuleNames.AddRange([
			"D3D11RHI",
			"D3D12RHI",
		]);

		AddEngineThirdPartyPrivateStaticDependencies(Target, "NVAPI");
	}
}
