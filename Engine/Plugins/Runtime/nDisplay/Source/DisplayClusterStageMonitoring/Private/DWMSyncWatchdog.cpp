// Copyright Epic Games, Inc. All Rights Reserved.

#include "DWMSyncWatchdog.h"

#include "Async/Async.h"
#include "Engine/Engine.h"
#include "Engine/GameViewportClient.h"
#include "IDisplayCluster.h"
#include "IDisplayClusterCallbacks.h"
#include "IStageDataProvider.h"
#include "Logging/LogMacros.h"
#include "Render/IDisplayClusterRenderManager.h"
#include "Render/Presentation/IDisplayClusterPresentation.h"
#include "RendererInterface.h"
#include "UnrealClient.h"


#pragma warning(push)
#pragma warning (disable : 4005) 	// Disable macro redefinition warning for compatibility with Windows SDK 8+
#include "Windows/AllowWindowsPlatformTypes.h"
PRAGMA_DISABLE_NONPORTABLE_INCLUDE_PATH_WARNING
#include "ThirdParty/Windows/DirectX/Include/DXGI.h"
PRAGMA_ENABLE_NONPORTABLE_INCLUDE_PATH_WARNING
#include "Windows/HideWindowsPlatformTypes.h"
#pragma warning(pop)


DEFINE_LOG_CATEGORY_STATIC(LogDisplayClusterStageMonitoringDWM, Log, All)


FDWMSyncWatchdog::FDWMSyncWatchdog()
{
	UE_LOGF(LogDisplayClusterStageMonitoringDWM, Log, "DWM Sync watchdog active");
	IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPresentationPreSynchronization_RHIThread().AddRaw(this, &FDWMSyncWatchdog::OnPresentationPreSynchronization_RHIThread);
	IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPresentationPostSynchronization_RHIThread().AddRaw(this, &FDWMSyncWatchdog::OnPresentationPostSynchronization_RHIThread);
}

FDWMSyncWatchdog::~FDWMSyncWatchdog()
{
	IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPresentationPreSynchronization_RHIThread().RemoveAll(this);
	IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPresentationPostSynchronization_RHIThread().RemoveAll(this);
}

void FDWMSyncWatchdog::OnPresentationPreSynchronization_RHIThread(FRHIViewport* Viewport)
{
	const uint64 PreSyncCycles = FPlatformTime::Cycles64();
	LastPreSyncDeltaCycles = PreSyncCycles - LastPreSyncCycles;
	LastPreSyncCycles = PreSyncCycles;

	//Very verbose debugging to understand frame stats on stage setup
	if (IDXGISwapChain* const SwapChain = static_cast<IDXGISwapChain*>(Viewport->GetNativeSwapChain()))
	{
		DXGI_FRAME_STATISTICS Stats;
		if (SwapChain->GetFrameStatistics(&Stats) == S_OK)
		{
			uint32 ThisPresentId = 0;
			if (SwapChain->GetLastPresentCount(&ThisPresentId) == S_OK)
			{
				UE_LOGF(LogDisplayClusterStageMonitoringDWM, VeryVerbose, "PreSync(%d): PresentCount: %u, LastPresentCount: %u, PresentRefreshCount: %u, SyncRefreshCount: %u"
					, GFrameNumberRenderThread
					, Stats.PresentCount
					, ThisPresentId
					, Stats.PresentRefreshCount
					, Stats.SyncRefreshCount);
			}
		}
	}
}

void FDWMSyncWatchdog::OnPresentationPostSynchronization_RHIThread(FRHIViewport* Viewport)
{
	const uint64 PostSyncCycles = FPlatformTime::Cycles64();

	bool bIsValidFrame = false;

	if (IDXGISwapChain* const SwapChain = static_cast<IDXGISwapChain*>(Viewport->GetNativeSwapChain()))
	{
		DXGI_FRAME_STATISTICS Stats;
		if (SwapChain->GetFrameStatistics(&Stats) == S_OK)
		{
			uint32 LastPresentCount = 0;
			if (SwapChain->GetLastPresentCount(&LastPresentCount) == S_OK)
			{
				bIsValidFrame = true;

				UE_LOGF(LogDisplayClusterStageMonitoringDWM, VeryVerbose, "PostSync(%d): PresentCount: %u, LastPresentCount: %u, PresentRefreshCount: %u, SyncRefreshCount: %u"
					, GFrameNumberRenderThread
					, Stats.PresentCount
					, LastPresentCount
					, Stats.PresentRefreshCount
					, Stats.SyncRefreshCount);

				//We need a valid previous count to validate relative frame counter difference 
				if (PreviousFrameCount.IsSet())
				{
					const uint32 DeltaFrame = Stats.PresentRefreshCount >= PreviousFrameCount.GetValue() ?
						Stats.PresentRefreshCount - PreviousFrameCount.GetValue()
						: MAX_uint32 - PreviousFrameCount.GetValue() + Stats.PresentRefreshCount + 1;

					// In some cases, we have the same vblank count meaning we were called faster than the refresh rate. 
					// Adding logging for this would be useful 
					if (DeltaFrame > 1)
					{
						//VBlanks we missed, let stage monitor know, from the game thread
						const uint32 MissedFrames = DeltaFrame - 1;
						const uint32 PresentCount = Stats.PresentCount;
						const uint32 PresentRefreshCount = Stats.PresentRefreshCount;

						UE_LOGF(LogDisplayClusterStageMonitoringDWM, Warning, "'%d' VBlanks were missed.", MissedFrames);
						AsyncTask(ENamedThreads::GameThread, [MissedFrames, PresentCount, LastPresentCount, PresentRefreshCount]()
						{
							IStageDataProvider::SendMessage<FDWMSyncEvent>(EStageMessageFlags::Reliable, MissedFrames, PresentCount, LastPresentCount, PresentRefreshCount);
						});
					}
				}

				PreviousFrameCount = Stats.PresentRefreshCount;
			}
		}
	}

	if (bIsValidFrame == false)
	{
		PreviousFrameCount.Reset();
	}
}

FString FDWMSyncEvent::ToString() const
{
	return FString::Printf(TEXT("Missed %d sync signals between two custom present."), MissedFrames);
}
