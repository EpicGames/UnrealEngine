// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "StageMessages.h"
#include "RHIFwd.h"
#include "DWMSyncWatchdog.generated.h"


USTRUCT()
struct FDWMSyncEvent : public FStageProviderEventMessage
{
	GENERATED_BODY()

public:
	FDWMSyncEvent() = default;

	FDWMSyncEvent(uint32 InMissedFrames, uint32 InPresentCount, uint32 InLastPresentCount, uint32 InPresentRefreshCount)
		: MissedFrames(InMissedFrames)
		, PresentCount(InPresentCount)
		, LastPresentCount(InLastPresentCount)
		, PresentRefreshCount(InPresentRefreshCount)
	{}

public:
	virtual FString ToString() const override;

public:

	/** Number of sync counts that were missed between presents */
	UPROPERTY(VisibleAnywhere, Category = "NvidiaSync")
	uint32 MissedFrames = 0;

	UPROPERTY(VisibleAnywhere, Category = "NvidiaSync")
	uint32 PresentCount = 0;

	UPROPERTY(VisibleAnywhere, Category = "NvidiaSync")
	uint32 LastPresentCount = 0;

	UPROPERTY(VisibleAnywhere, Category = "NvidiaSync")
	uint32 PresentRefreshCount = 0;
};


/**
 * Verifies the frame counter using DWM stats
 */
class FDWMSyncWatchdog
{
public:
	FDWMSyncWatchdog();
	~FDWMSyncWatchdog();

private:

	void OnCustomPresentCreated();
	void OnPresentationPreSynchronization_RHIThread(FRHIViewport* Viewport);
	void OnPresentationPostSynchronization_RHIThread(FRHIViewport* Viewport);

private:

	/** 
	 * Previous vblank counter queried from frame stats 
	 * We expect this to increment by 1 every frame or else a hitch happened
	 */
	TOptional<uint32> PreviousFrameCount;

	/** Cycle count when last pre synchronization happened */
	uint64 LastPreSyncCycles = 0;

	/** Cycles it took between two pre synchronization i.e. time it took to render and present the frame */
	uint64 LastPreSyncDeltaCycles = 0;
};
