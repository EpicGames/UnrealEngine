// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#include "DisplayClusterConfigurationStrings.h"
#include "DisplayClusterStageMonitoringSettings.h"
#include "IDisplayCluster.h"
#include "IDisplayClusterCallbacks.h"
#include "DWMSyncWatchdog.h"
#include "NvidiaSyncWatchdog.h"
#include "Render/IDisplayClusterRenderManager.h"
#include "Render/Synchronization/IDisplayClusterRenderSyncPolicy.h"


/**
 * Display Cluster stage monitoring module.
 */
class FDisplayClusterStageMonitoringModule : public IModuleInterface
{
public:
	virtual void StartupModule() override
	{
		IDisplayCluster::Get().GetCallbacks().OnDisplayClusterStartSession().AddRaw(this, &FDisplayClusterStageMonitoringModule::OnDisplayClusterStartSession);
	}

	virtual void ShutdownModule() override
	{
		if (IDisplayCluster::IsAvailable())
		{
			IDisplayCluster::Get().GetCallbacks().OnDisplayClusterStartSession().RemoveAll(this);
		}
	}

private:
	void OnDisplayClusterStartSession()
	{
		NvidiaSyncWatchdog.Reset();

		if (IDisplayCluster::IsAvailable())
		{
			static const FName NvidiaPolicy = DisplayClusterConfigurationStrings::config::cluster::render_sync::Nvidia;
			static const FName EthernetPolicy = DisplayClusterConfigurationStrings::config::cluster::render_sync::Ethernet;
			static const FName None = DisplayClusterConfigurationStrings::config::cluster::render_sync::None;
			static const FName NvidiaSwapBarrier = DisplayClusterConfigurationStrings::config::cluster::render_sync::NvidiaSwapBarrier;
			static const FName NvidiaSwapGroup = DisplayClusterConfigurationStrings::config::cluster::render_sync::NvidiaSwapGroup;
			TSharedPtr<IDisplayClusterRenderSyncPolicy> CurrentSyncPolicy = IDisplayCluster::Get().GetRenderMgr()->GetCurrentSynchronizationPolicy();
			if (CurrentSyncPolicy.IsValid())
			{
				const FName CurrentSyncPolicyName = CurrentSyncPolicy->GetName();
				if (CurrentSyncPolicyName == NvidiaPolicy)
				{
					if (GetDefault<UDisplayClusterStageMonitoringSettings>()->ShouldEnableNvidiaWatchdog() && WITH_NVAPI)
					{
						NvidiaSyncWatchdog = MakeUnique<FNvidiaSyncWatchdog>();
					}
				}
				//TODO !!Remove when sync policy tests are done
				//else if (CurrentSyncPolicy->GetName() == EthernetPolicy)
				else if(CurrentSyncPolicyName != None
						&& CurrentSyncPolicyName != NvidiaPolicy
						&& CurrentSyncPolicyName != NvidiaSwapGroup
						&& CurrentSyncPolicyName != NvidiaSwapBarrier)
				{
					if (GetDefault<UDisplayClusterStageMonitoringSettings>()->ShouldEnableDWMWatchdog())
					{
						//We could detect if dwm is enabled and create the dwm watchdog if so instead on relying on sync policy 1
						DWMSyncWatchdog = MakeUnique<FDWMSyncWatchdog>();
					}
				}
			}
		}
	}

private:

	/** If sync policy is nvidia, watchdog used to check for hitches */
	TUniquePtr<FNvidiaSyncWatchdog> NvidiaSyncWatchdog;

	/** If sync policy is ethernet, watchdog used to check for hitches */
	TUniquePtr<FDWMSyncWatchdog> DWMSyncWatchdog;

};

IMPLEMENT_MODULE(FDisplayClusterStageMonitoringModule, DisplayClusterStageMonitoring);

