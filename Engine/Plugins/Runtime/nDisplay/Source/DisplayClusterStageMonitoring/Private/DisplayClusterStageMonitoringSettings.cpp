// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterStageMonitoringSettings.h"

#include "Misc/CommandLine.h"

UDisplayClusterStageMonitoringSettings::UDisplayClusterStageMonitoringSettings()
{
	int32 CommandLineEnableHitchVar = 0;
	if (FParse::Value(FCommandLine::Get(), TEXT("-EnableDisplayClusterNvidiaHitchDetect="), CommandLineEnableHitchVar))
	{
		CommandLineEnableNvidiaHitch = CommandLineEnableHitchVar == 1 ? true : false;
	}

	if (FParse::Value(FCommandLine::Get(), TEXT("-EnableDisplayClusterDWMHitchDetect="), CommandLineEnableHitchVar))
	{
		CommandLineEnableDWMHitch = CommandLineEnableHitchVar == 1 ? true : false;
	}
}

FName UDisplayClusterStageMonitoringSettings::GetCategoryName() const
{
	return TEXT("Plugins");
}

#if WITH_EDITOR
FText UDisplayClusterStageMonitoringSettings::GetSectionText() const
{
	return NSLOCTEXT("nDisplayPlugin", "nDisplayStageMonitoring", "nDisplay Stage Monitoring");
}
#endif

bool UDisplayClusterStageMonitoringSettings::ShouldEnableNvidiaWatchdog() const
{
	if (CommandLineEnableNvidiaHitch.IsSet())
	{
		return CommandLineEnableNvidiaHitch.GetValue();
	}

	return bEnableNvidiaHitchDetection;
}

bool UDisplayClusterStageMonitoringSettings::ShouldEnableDWMWatchdog() const
{
	if (CommandLineEnableDWMHitch.IsSet())
	{
		return CommandLineEnableDWMHitch.GetValue();
	}

	return bEnableDWMHitchDetection;
}

