// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"

#include "DisplayClusterStageMonitoringSettings.generated.h"


/**
 * Settings for the DisplayCluster StageMonitor hooks plugin modules. 
 */
UCLASS(config=Game)
class DISPLAYCLUSTERSTAGEMONITORING_API UDisplayClusterStageMonitoringSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UDisplayClusterStageMonitoringSettings();

	//~ Begin UDevelopperSettings interface
	virtual FName GetCategoryName() const;
#if WITH_EDITOR
	virtual FText GetSectionText() const override;
#endif
	//~ End UDevelopperSettings interface

public:

	/** Whether nvidia hitch detection should be enabled or not */
	bool ShouldEnableNvidiaWatchdog() const;


	/** Whether DWM hitch detection should be enabled or not */
	bool ShouldEnableDWMWatchdog() const;

protected:

	/**
	 * Whether DWM hitch detection is enabled by default
	 * @note It may be overriden via the command line, "-EnableDisplayClusterDWMHitchDetect=1"
	 * @note Only works with sync policy 1
	 */
	UPROPERTY(config, EditAnywhere, Category = "Settings")
	bool bEnableDWMHitchDetection = true;

	/**
	 * Whether Nvidia hitch detection is enabled by default
	 * @note It may be overriden via the command line, "-EnableDisplayClusterNvidiaHitchDetect=1"
	 * @note Only works with sync policy 2
	 */
	UPROPERTY(config, EditAnywhere, Category = "Settings")
	bool bEnableNvidiaHitchDetection = true;

public:

	/** Command line value to override enable settings */
	TOptional<bool> CommandLineEnableNvidiaHitch;

	/** Command line value to override enable settings */
	TOptional<bool> CommandLineEnableDWMHitch;
};