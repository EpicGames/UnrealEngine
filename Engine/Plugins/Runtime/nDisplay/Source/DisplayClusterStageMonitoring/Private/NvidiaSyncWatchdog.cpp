// Copyright Epic Games, Inc. All Rights Reserved.

#include "NvidiaSyncWatchdog.h"

#include "Async/Async.h"
#include "IDisplayCluster.h"
#include "IDisplayClusterCallbacks.h"
#include "IStageDataProvider.h"
#include "Logging/LogMacros.h"
#include "Render/IDisplayClusterRenderManager.h"
#include "Render/Presentation/IDisplayClusterPresentation.h"
#include "RendererInterface.h"

#include "ID3D11DynamicRHI.h"
#include "ID3D12DynamicRHI.h"

#pragma warning(push)
#pragma warning (disable : 4005) 	// Disable macro redefinition warning for compatibility with Windows SDK 8+
#include "Windows/AllowWindowsPlatformTypes.h"
#include "d3d11.h"
#if WITH_NVAPI
#include "nvapi.h"
#endif
#include "Windows/HideWindowsPlatformTypes.h"
#pragma warning(pop)


DEFINE_LOG_CATEGORY_STATIC(LogDisplayClusterStageMonitoring, Log, All)


FNvidiaSyncWatchdog::FNvidiaSyncWatchdog()
{
	IDisplayCluster::Get().GetCallbacks().OnDisplayClusterCustomPresentSet().AddRaw(this, &FNvidiaSyncWatchdog::OnCustomPresentCreated);
}

FNvidiaSyncWatchdog::~FNvidiaSyncWatchdog()
{
	IDisplayCluster::Get().GetCallbacks().OnDisplayClusterCustomPresentSet().RemoveAll(this);

	IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPresentationPreSynchronization_RHIThread().RemoveAll(this);
	IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPresentationPostSynchronization_RHIThread().RemoveAll(this);

	//If we have an async task in flight, wait for it to complete so D3DDevice isn't used anymore
	if (AsyncFrameCountFuture.IsValid())
	{
		AsyncFrameCountFuture.WaitFor(FTimespan::FromMilliseconds(100.0));
	}
}

void FNvidiaSyncWatchdog::OnCustomPresentCreated()
{
	D3DDevice = nullptr;
	switch (RHIGetInterfaceType())
	{
	case ERHIInterfaceType::D3D11: D3DDevice = GetID3D11DynamicRHI()->RHIGetDevice(); break;
	case ERHIInterfaceType::D3D12: D3DDevice = GetID3D12DynamicRHI()->RHIGetDevice_NoMGPU(); break;
	}

	if (D3DDevice)
	{
		UE_LOGF(LogDisplayClusterStageMonitoring, VeryVerbose, "Nvidia Sync watchdog active");
		IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPresentationPreSynchronization_RHIThread().AddRaw(this, &FNvidiaSyncWatchdog::OnPresentationPreSynchronization_RHIThread);
		IDisplayCluster::Get().GetCallbacks().OnDisplayClusterPresentationPostSynchronization_RHIThread().AddRaw(this, &FNvidiaSyncWatchdog::OnPresentationPostSynchronization_RHIThread);
	}
}

void FNvidiaSyncWatchdog::OnPresentationPreSynchronization_RHIThread(FRHIViewport* Viewport)
{
	const uint64 PreSyncCycles = FPlatformTime::Cycles64();
	LastPreSyncDeltaCycles = PreSyncCycles - LastPreSyncCycles;
	LastPreSyncCycles = PreSyncCycles;
}

void FNvidiaSyncWatchdog::OnPresentationPostSynchronization_RHIThread(FRHIViewport* Viewport)
{
#if WITH_NVAPI
	const uint64 PostSyncCycles = FPlatformTime::Cycles64();

	//First time, this won't be valid
	if (AsyncFrameCountFuture.IsValid())
	{
		//Our async task is quite small so it should always have completed in a frame
		//In the case it hasn't, we trigger a new read so we'll miss the result of the previous one
		//We could block here until the future has completed though.
		if (AsyncFrameCountFuture.IsReady())
		{
			const FQueryFrameCounterResult FutureResult = AsyncFrameCountFuture.Get();
			if (FutureResult.QueryResult != NVAPI_OK)
			{
				UE_LOGF(LogDisplayClusterStageMonitoring, Verbose, "Couldn't query frame count from nvapi, error code 0x%x", FutureResult.QueryResult);
				PreviousFrameCount.Reset();
			}
			else
			{
				//We need a valid previous count to validate relative frame counter difference 
				if (PreviousFrameCount.IsSet())
				{
					const uint32 DeltaFrame = FutureResult.FrameCount >= PreviousFrameCount.GetValue() ?
											  FutureResult.FrameCount - PreviousFrameCount.GetValue()
											: MAX_uint32 - PreviousFrameCount.GetValue() + FutureResult.FrameCount + 1;

					if (DeltaFrame != 1)
					{
						//Hitch was detected, let stage monitor know, from the game thread
						const float LastFrameTimeMS = FPlatformTime::ToMilliseconds64(LastPreSyncDeltaCycles);
						const float SynchronizationTimeMS = FPlatformTime::ToMilliseconds64(PostSyncCycles - LastPreSyncCycles);
						const int32 MissedFrames = DeltaFrame - 1;

						UE_LOGF(LogDisplayClusterStageMonitoring, Verbose, "Missed '%d' frames. Last frame took '%0.5f' ms and waited '%0.5f' for the hardware sync", MissedFrames, LastFrameTimeMS, SynchronizationTimeMS);
						AsyncTask(ENamedThreads::GameThread, [MissedFrames, LastFrameTimeMS, SynchronizationTimeMS]()
						{
							IStageDataProvider::SendMessage<FNvidiaSyncEvent>(EStageMessageFlags::Reliable, MissedFrames, LastFrameTimeMS, SynchronizationTimeMS);
						});
					}
				}
			}
		
			PreviousFrameCount = FutureResult.FrameCount;
		}
		else
		{
			UE_LOGF(LogDisplayClusterStageMonitoring, Warning, "Could not completed query of NvAPI frame counter inside a frame");
		}
	}

	//Trigger a short async task to fetch frame number of the 
	const uint64 PreAsyncTaskCycles = FPlatformTime::Cycles64(); //Used for now to get a sense of timing info about the execution of this
	AsyncFrameCountFuture = Async(EAsyncExecution::ThreadPool, [AsyncD3DDevice = D3DDevice, PreAsyncTaskCycles]()
	{
		FQueryFrameCounterResult Result;
		Result.QueryResult = NvAPI_D3D1x_QueryFrameCount(AsyncD3DDevice, reinterpret_cast<NvU32*>(&Result.FrameCount));

		const float KickTaskToCompletionMilliseconds = FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64() - PreAsyncTaskCycles);
		if (Result.QueryResult == NVAPI_OK)
		{
			UE_LOGF(LogDisplayClusterStageMonitoring, VeryVerbose, "New NvAPI FrameCount = '%d'. Took %0.6fms to complete", Result.FrameCount, KickTaskToCompletionMilliseconds);
		}

		return Result;
	});
#endif
}

FString FNvidiaSyncEvent::ToString() const
{
	return FString::Printf(TEXT("Missed %d sync signals between two custom present."), MissedFrames);
}
