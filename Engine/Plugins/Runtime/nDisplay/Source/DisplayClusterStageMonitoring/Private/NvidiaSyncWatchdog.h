// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Async/Future.h"
#include "StageMessages.h"
#include "RHIFwd.h"
#include "NvidiaSyncWatchdog.generated.h"


struct IUnknown;


USTRUCT()
struct FNvidiaSyncEvent : public FStageProviderEventMessage
{
	GENERATED_BODY()

public:
	FNvidiaSyncEvent() = default;

	FNvidiaSyncEvent(int32 InMissedFrames, float InFrameDuration, float InSynchronizationDuration)
		: MissedFrames(InMissedFrames)
		, LastFrameDuration(InFrameDuration)
		, SynchronizationDuration(InSynchronizationDuration)
	{}

public:
	virtual FString ToString() const override;

public:

	/** Number of sync counts that were missed between presents */
	UPROPERTY(VisibleAnywhere, Category = "NvidiaSync")
	int32 MissedFrames = 0;

	UPROPERTY(VisibleAnywhere, Category = "NvidiaSync", meta = (Unit = "ms"))
	float LastFrameDuration = 0.0f;

	UPROPERTY(VisibleAnywhere, Category = "NvidiaSync", meta = (Unit = "ms"))
	float SynchronizationDuration = 0.0f;
};


/**
 * Verifies the frame counter using NvAPI
 */
class FNvidiaSyncWatchdog
{
public:
	FNvidiaSyncWatchdog();
	~FNvidiaSyncWatchdog();

private:

	void OnCustomPresentCreated();
	void OnPresentationPreSynchronization_RHIThread(FRHIViewport* Viewport);
	void OnPresentationPostSynchronization_RHIThread(FRHIViewport* Viewport);

private:

	/** 
	 * Previous frame counter queried from nvapi quadro card 
	 * We expect this to increment by 1 every frame or else a hitch happened
	 */
	TOptional<uint32> PreviousFrameCount;

	/** Current DynamicRHI as D3DDevice */
	IUnknown* D3DDevice = nullptr;

	/** Cycle count when last pre synchronization happened */
	uint64 LastPreSyncCycles = 0;

	/** Cycles it took between two pre synchronization i.e. time it took to render and present the frame */
	uint64 LastPreSyncDeltaCycles = 0;

	struct FQueryFrameCounterResult
	{
		uint32 FrameCount = 0;
		int32 QueryResult = 0;
	};

	TFuture<FQueryFrameCounterResult> AsyncFrameCountFuture;
};
