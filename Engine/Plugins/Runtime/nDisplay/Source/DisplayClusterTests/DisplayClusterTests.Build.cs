// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class DisplayClusterTests : ModuleRules
{
	public DisplayClusterTests(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"DisplayCluster",
				"DisplayClusterConfiguration",
				"DisplayClusterConfigurator",
				"Core",
				"CoreUObject",
				"Engine",
				"OpenColorIO",
				"SubobjectDataInterface",
				"UnrealEd",
				"SlateCore"
			});
		
		OptimizeCode = CodeOptimization.Never;
		PCHUsage = PCHUsageMode.NoPCHs;
	}
}
