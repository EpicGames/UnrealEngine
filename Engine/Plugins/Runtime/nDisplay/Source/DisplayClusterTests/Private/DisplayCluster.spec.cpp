// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetRegistry/AssetRegistryModule.h"
#include "Blueprints/DisplayClusterBlueprint.h"
#include "Blueprints/DisplayClusterBlueprintGeneratedClass.h"
#include "ClusterConfiguration/DisplayClusterConfiguratorClusterUtils.h"
#include "ComponentAssetBroker.h"
#include "Components/DisplayClusterScreenComponent.h"
#include "Components/DisplayClusterXformComponent.h"
#include "DisplayClusterRootActor.h"
#include "DisplayClusterTestUtils.h"
#include "Engine/SCS_Node.h"
#include "Engine/SimpleConstructionScript.h"
#include "Kismet2/ComponentEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Misc/AutomationTest.h"
#include "OpenColorIOConfiguration.h"

#if WITH_DEV_AUTOMATION_TESTS

/**
 * Convenience macro to register a property propagation test. Must be called within an FDisplayClusterSpec member function.
 * 
 * @param HANDLE_OVERRIDE_TYPE			The type to cast the passed value to before trying to set the property handle's value.
 * @param OWNER_NAME					The name of the objects containing the value. "Asset" and "Instanced" will be prepended to find the
 *										variables containing the propagated asset and receiving instance respectively.
 * @param ASSET_VALUE					The value to set on the asset, and which should be propagated to the instance (unless this is an
 *										instance override test, in which case it should not be propagated).
 * @param INSTANCE_VALUE				The value to set on the instance if this is an instance override test. Otherwise, this is
 *										ignored.
 * @param VALUE_FIELD_PATH				A list of field names leading to the property, where the first name is the top-most field starting
 *										from the owner, and subsequent names are nested fields (e.g. a field within a struct).
 * @param ALLOW_INSTANCE_OVERRIDE_TEST	If false, this test will be excluded when doing instance override tests. Use this when the
 *										test isn't meaningful (e.g. overriding and propagation are ignored when the value is the same as
 *										before, so for a bool, there's no way to trigger both in the same test).
 */
#define PROPAGATION_TEST(HANDLE_OVERRIDE_TYPE, OWNER_NAME, ASSET_VALUE, INSTANCE_VALUE, VALUE_FIELD_PATH, ALLOW_INSTANCE_OVERRIDE_TEST) \
	DoPropertyPropagationTest<std::remove_pointer<decltype(Asset ## OWNER_NAME)>::type, decltype(ASSET_VALUE), HANDLE_OVERRIDE_TYPE>( \
		[this]() { return Asset ## OWNER_NAME; }, \
		[this]() { return Instanced ## OWNER_NAME; }, \
		[this]() { return ASSET_VALUE; }, \
		[this]() { return INSTANCE_VALUE; }, \
		#OWNER_NAME, \
		(VALUE_FIELD_PATH), \
		BeforeTestAction, \
		nullptr, \
		bDoInstanceOverrideTest, \
		ALLOW_INSTANCE_OVERRIDE_TEST)

/**
 * Convenience macro to register a simple property propagation test where HANDLE_OVERRIDE_TYPE == VALUE_TYPE.
 */
#define PROPAGATION_TEST_SIMPLE(OWNER_NAME, ASSET_VALUE, INSTANCE_VALUE, VALUE_FIELD_PATH) \
	PROPAGATION_TEST(decltype(ASSET_VALUE), OWNER_NAME, ASSET_VALUE, INSTANCE_VALUE, VALUE_FIELD_PATH, true)

#define PROPAGATION_TEST_SIMPLE_NO_INSTANCE_OVERRIDE(OWNER_NAME, ASSET_VALUE, VALUE_FIELD_PATH) \
	PROPAGATION_TEST(decltype(ASSET_VALUE), OWNER_NAME, ASSET_VALUE, ASSET_VALUE, VALUE_FIELD_PATH, false)

/**
 * Convenience macro to register a simple property propagation test for an enum, where HANDLE_OVERRIDE_TYPE is the
 * underlying type of the enum.
 */
#define PROPAGATION_TEST_ENUM(OWNER_NAME, ASSET_VALUE, INSTANCE_VALUE, VALUE_FIELD_PATH) \
	PROPAGATION_TEST(__underlying_type(decltype(ASSET_VALUE)), OWNER_NAME, ASSET_VALUE, INSTANCE_VALUE, VALUE_FIELD_PATH, true)

#define PROPAGATION_TEST_ENUM_NO_INSTANCE_OVERRIDE(OWNER_NAME, ASSET_VALUE, VALUE_FIELD_PATH) \
	PROPAGATION_TEST(__underlying_type(decltype(ASSET_VALUE)), OWNER_NAME, ASSET_VALUE, ASSET_VALUE, VALUE_FIELD_PATH, false)

/**
 * Convenience macro to register a set of property propagation tests for color grading rendering settings (FDisplayClusterConfigurationViewport_ColorGradingRenderingSettings).
 * @param OWNER_NAME The name of the objects containing the settings. "Asset" and "Instanced" will be prepended to find the variables containing the propagated asset and receiving instance respectively.
 * @param SETTINGS_FIELD The variable name of the settings struct, starting from the the owning asset (e.g. "Settings.ColorGrading" would result in Asset->Settings.ColorGrading).
 * @param VALUE_FIELD_PATH A list of field names leading to the settings property, where the first name is the top-most field starting from the owner, and subsequent names are nested fields (e.g. a field within a struct).
 */
#define PROPAGATION_TEST_COLOR_GRADING_RENDERING_SETTINGS(OWNER_NAME, SETTINGS_FIELD, VALUE_FIELD_PATH) \
	DoColorGradingRenderingSettingsPropagationTests<std::remove_pointer<decltype(Asset ## OWNER_NAME)>::type>( \
		[this]() { return Asset ## OWNER_NAME; }, \
		[this]() { return Instanced ## OWNER_NAME; }, \
		[this](std::remove_pointer<decltype(Asset ## OWNER_NAME)>::type* Owner) { return &Owner->SETTINGS_FIELD; }, \
		#OWNER_NAME, \
		(VALUE_FIELD_PATH), \
		BeforeTestAction, \
		bDoInstanceOverrideTest)

/**
 * Convenience macro to register a set of property propagation tests for OCIO display configurations (FOpenColorIODisplayConfiguration).
 * @param OWNER_NAME The name of the objects containing the settings. "Asset" and "Instanced" will be prepended to find the variables containing the propagated asset and receiving instance respectively.
 * @param VALUE_FIELD_PATH A list of field names leading to the configuration property, where the first name is the top-most field starting from the owner, and subsequent names are nested fields (e.g. a field within a struct).
 */
#define PROPAGATION_TEST_OCIO_CONFIG(OWNER_NAME, VALUE_FIELD_PATH) \
	DoOCIOConfigurationPropagationTests<std::remove_pointer<decltype(Asset ## OWNER_NAME)>::type>( \
		[this]() { return Asset ## OWNER_NAME; }, \
		[this]() { return Instanced ## OWNER_NAME; }, \
		#OWNER_NAME, \
		(VALUE_FIELD_PATH), \
		BeforeTestAction, \
		bDoInstanceOverrideTest)

BEGIN_DEFINE_SPEC(FDisplayClusterSpec, "DisplayCluster", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)

	const FString TestNodeName = "TestNode";
	const FString TestViewportName = "TestViewport";
	const FString TestActorName = "TestActor";
	const FString TestComponentName = "TestComponent";

	UWorld* World;

	UDisplayClusterBlueprint* ClusterAsset;
	UDisplayClusterConfigurationData* AssetClusterConfig;
	UDisplayClusterConfigurationCluster* AssetRootCluster;
	UDisplayClusterConfigurationClusterNode* AssetClusterNode;
	UDisplayClusterConfigurationViewport* AssetViewport;
	UDisplayClusterXformComponent* AssetComponent;

	ADisplayClusterRootActor* ClusterActor;
	UDisplayClusterConfigurationData* InstancedClusterConfig;
	UDisplayClusterConfigurationCluster* InstancedRootCluster;
	UDisplayClusterConfigurationClusterNode* InstancedClusterNode;
	UDisplayClusterConfigurationViewport* InstancedViewport;
	UDisplayClusterXformComponent* InstancedComponent;

	TWeakObjectPtr<UOpenColorIOConfiguration> OCIOConfig;
	TWeakObjectPtr<UOpenColorIOConfiguration> AltOCIOConfig;

	AActor* AltActor;

	/**
	 * Register a test (using It) confirming that a property's value is propagated from the asset to its instance.
	 *
	 * @param OwnerType The type of object that the property exists on.
	 * @param ValueType The type of the property that we want to test.
	 * @param HandleOverrideType The type to cast the passed value to before trying to set the property handle's value.
	 * 
	 * @param AssetOwnerAccessor Function returning a pointer to the asset that owns the property.
	 * @param InstancedOwnerAccessor Function returning a pointer to the instance that owns the property.
	 * @param AssetValueFn Function which returns the value to set on the asset, and which should be propagated to the instance.
	 * @param InstanceValueFn Function which returns the value to set on the instance if this is an instance override test. If not, this is ignored.
	 * @param OwnerName The name to refer to the owner by in the test description.
	 * @param FieldNames A list of field names leading to the property, where the first name is the top-most field starting from the owner, and subsequent names are nested fields (e.g. a field within a struct).
	 * @param BeforeTestAction A function to call after the property is set on the asset, but before the value is checked on the instance.
	 * @param EnableFunc Function to call before setting the property's value (e.g. to enable override for the field).
	 * @param bDoInstanceOverrideTest If false, test that the value is propagated from the asset to the instance. If true, set the instance's value first, then test that the value is *not* propagated from the asset to the instance.
	 * @param bAllowInstanceOverrideTest If false and bDoInstanceOverrideTest is true, don't register a test.
	 */
	template <typename OwnerType, typename ValueType, typename HandleOverrideType = ValueType>
	void DoPropertyPropagationTest(TFunction<OwnerType*()> AssetOwnerAccessor, TFunction<OwnerType*()> InstancedOwnerAccessor, TFunction<ValueType()> AssetValueFn,
		TFunction<ValueType()> InstanceValueFn, FString OwnerName, const TArray<FName>& FieldNames, TFunction<void()> BeforeTestAction, TFunction<void(OwnerType*)> EnableFunc,
		bool bDoInstanceOverrideTest, bool bAllowInstanceOverrideTest);

	/**
	 * Register tests for property propagation of an FDisplayClusterConfigurationViewport_ColorGradingSettings.
	 *
	 * @param OwnerType The type of object that the settings struct exists on.
	 *
	 * @param AssetOwnerAccessor Function returning a pointer to the asset that owns the property.
	 * @param InstancedOwnerAccessor Function returning a pointer to the instance that owns the property.
	 * @param AssetSettingsAccessor Function returning a pointer to the given owner's copy of the settings object.
	 * @param OwnerName The name to refer to the owner by in the test description.
	 * @param FieldNames A list of field names leading to the settings' property, where the first name is the top-most field starting from the owner, and subsequent names are nested fields (e.g. a field within a struct).
	 * @param BeforeTestAction A function to call after the property is set on the asset, but before the value is checked on the instance.
	 * @param bDoInstanceOverrideTest If false, test that the value is propagated from the asset to the instance. If true, set the instance's value first, then test that the value is *not* propagated from the asset to the instance.
	 */
	template <typename OwnerType>
	void DoColorGradingSettingsPropagationTests(TFunction<OwnerType*()> AssetOwnerAccessor, TFunction<OwnerType*()> InstancedOwnerAccessor,
		TFunction<FDisplayClusterConfigurationViewport_ColorGradingSettings*(OwnerType*)> AssetSettingsAccessor,
		FString OwnerName, const TArray<FName>& FieldNames, TFunction<void()> BeforeTestAction, bool bDoInstanceOverrideTest);

	/**
	 * Register tests for property propagation of an FDisplayClusterConfigurationViewport_ColorGradingSettings.
	 *
	 * @param OwnerType The type of object that the settings struct exists on.
	 *
	 * @param AssetOwnerAccessor Function returning a pointer to the asset that owns the property.
	 * @param InstancedOwnerAccessor Function returning a pointer to the instance that owns the property.
	 * @param AssetSettingsAccessor Function returning a pointer to the given owner's copy of the settings object.
	 * @param OwnerName The name to refer to the owner by in the test description.
	 * @param FieldNames A list of field names leading to the settings' property, where the first name is the top-most field starting from the owner, and subsequent names are nested fields (e.g. a field within a struct).
	 * @param BeforeTestAction A function to call after the property is set on the asset, but before the value is checked on the instance.
	 * @param bDoInstanceOverrideTest If false, test that the value is propagated from the asset to the instance. If true, set the instance's value first, then test that the value is *not* propagated from the asset to the instance.
	 */
	template <typename OwnerType>
	void DoColorGradingRenderingSettingsPropagationTests(TFunction<OwnerType*()> AssetOwnerAccessor, TFunction<OwnerType*()> InstancedOwnerAccessor,
		TFunction<FDisplayClusterConfigurationViewport_ColorGradingRenderingSettings*(OwnerType*)> AssetSettingsAccessor,
		FString OwnerName, const TArray<FName>& FieldNames, TFunction<void()> BeforeTestAction, bool bDoInstanceOverrideTest);

	/**
	 * Register tests for property propagation of an FOpenColorIODisplayConfiguration.
	 *
	 * @param OwnerType The type of object that the configuration struct exists on.
	 *
	 * @param AssetOwnerAccessor Function returning a pointer to the asset that owns the property.
	 * @param InstancedOwnerAccessor Function returning a pointer to the instance that owns the property.
	 * @param OwnerName The name to refer to the owner by in the test description.
	 * @param FieldNames A list of field names leading to the configuration's property, where the first name is the top-most field starting from the owner, and subsequent names are nested fields (e.g. a field within a struct).
	 * @param BeforeTestAction A function to call after the property is set on the asset, but before the value is checked on the instance.
	 * @param bDoInstanceOverrideTest If false, test that the value is propagated from the asset to the instance. If true, set the instance's value first, then test that the value is *not* propagated from the asset to the instance.
	 */
	template <typename OwnerType>
	void DoOCIOConfigurationPropagationTests(TFunction<OwnerType*()> AssetOwnerAccessor, TFunction<OwnerType*()> InstancedOwnerAccessor,
		FString OwnerName, const TArray<FName>& FieldNames, TFunction<void()> BeforeTestAction, bool bDoInstanceOverrideTest);

	/**
	 * Register all tests for property or subobject propagation.
	 *
	 * Tests for property and subobject propagation are collected in this function so we can reuse the same test logic with
	 * both Unreal and custom property propagation.
	 *
	 * @param BeforeTestAction An optional action to run after the logic has executed, but before the tests are performed.
	 */
	void RegisterPropagationTests(TFunction<void()> BeforeTestAction = []{});

	/**
	 * Register tests for property propagation (excluding subobject propagation).
	 *
	 * @param bDoInstanceOverrideTest If false, test that the value is propagated from the asset to the instance. If true, set the instance's value first, then test that the value is *not* propagated from the asset to the instance.
	 * @param BeforeTestAction An optional action to run after the logic has executed, but before the tests are performed.
	 */
	void RegisterPropertyPropagationTests(bool bDoInstanceOverrideTest, TFunction<void()> BeforeTestAction = []{});

	/**
	 * Cache the pointers to the asset's first cluster node and viewport.
	 * These pointers may be invalidated when the asset's node/viewport maps change, so this function lets us update
	 * the pointers to the new locations.
	 */
	void CacheAssetClusterNodeAndViewport();

	/**
	 * Cache pointers to the instanced cluster's data.
	 * These pointers may be invalidated under some circumstances such as compiling (and therefore reinstancing),
	 * so this function lets us update the pointers to the new locations.
	 * 
	 * @param bFindActor If true, find the ClusterActor in the world by name. This is useful after compiling in case the actor has been reinstanced.
	 * @param bCacheTestNodeAndViewport If false, this will not attempt to cache pointers to the test node and viewports and they will be set to null. This should be set to false if they haven't been created yet.
	 */
	void CacheInstancedClusterPointers(bool bFindActor = false, bool bCacheTestNodeAndViewport = true);

	/**
	 * Make a change to instanced data so that the custom DisplayCluster property propagation will be used.
	 */
	void TriggerCustomPropagation();

	/**
	 * Generic TestEqual function so we can add extra specializations for types we commonly compare.
	 */
	template <typename T>
	bool TestEqual(const TCHAR* What, T Actual, T Expected);

END_DEFINE_SPEC(FDisplayClusterSpec)

void FDisplayClusterSpec::Define()
{
	BeforeEach([this]()
	{
		ClusterAsset = DisplayClusterTestUtils::CreateDisplayClusterAsset();
		if (ClusterAsset)
		{
			AssetClusterConfig = ClusterAsset->GetOrLoadConfig();
			if (AssetClusterConfig)
			{
				AssetRootCluster = AssetClusterConfig->Cluster;
			}
		}
	});

	Describe("Creation", [this]()
	{
		Describe("From factory", [this]()
		{
			It("should create a valid UDisplayClusterBlueprint", [this]()
			{
				TestNotNull(TEXT("Asset is a valid UDisplayClusterBlueprint"), ClusterAsset);

				if (ClusterAsset)
				{
					const UClass* GeneratedClass = ClusterAsset->GetGeneratedClass();
					TestNotNull(TEXT("Generated class is a subclass of UDisplayClusterBlueprintGeneratedClass"), Cast<UDisplayClusterBlueprintGeneratedClass>(GeneratedClass));
				}
			});

			It("should create one nDisplay screen component", [this]()
			{
				int32 ScreenNodeCount = 0;

				if (ClusterAsset)
				{
					const TArray<USCS_Node*>& SCSNodes = ClusterAsset->SimpleConstructionScript->GetAllNodes();
					for (const USCS_Node* Node : SCSNodes)
					{
						if (Node && Node->ComponentClass == UDisplayClusterScreenComponent::StaticClass())
						{
							++ScreenNodeCount;
						}
					}
				}

				TestEqual(TEXT("Exactly one nDisplay screen component present in construction script"), ScreenNodeCount, 1);
			});

			It("should produce a valid config", [this]()
			{
				TestNotNull(TEXT("Asset returns a config"), AssetClusterConfig);
				TestNotNull(TEXT("Returned config contains a valid Cluster"), AssetRootCluster);
			});
		});
	});
	
	Describe("Property propagation", [this]()
	{
		BeforeEach([this]()
		{
			// Add some data to the root cluster so we can compare properties later
			AssetClusterNode = DisplayClusterTestUtils::AddClusterNodeToCluster(ClusterAsset, AssetRootCluster, TestNodeName);
			AssetViewport = DisplayClusterTestUtils::AddViewportToClusterNode(ClusterAsset, AssetClusterNode, TestViewportName);

			// Create a world and spawn an instance of the cluster actor into it
			World = DisplayClusterTestUtils::CreateWorld();
			TestNotNull(TEXT("Created world"), World);

			FActorSpawnParameters ActorSpawnParams;
			ActorSpawnParams.Name = FName(TestActorName);
			
			ClusterActor = World->SpawnActor<ADisplayClusterRootActor>(ClusterAsset->GetGeneratedClass(), ActorSpawnParams);
			if (TestNotNull(TEXT("Created display cluster actor"), ClusterActor))
			{
				CacheInstancedClusterPointers();
			}

			AltActor = World->SpawnActor<AActor>();
		});
		
		It("should keep the instance and CDO node maps in sync after compile", [this]()
		{
			// Repro for UE-136629:
			// 1. Create new nDisplay BP.
			// 2. Add two cluster nodes in the editor.
			// 3. Delete the first cluster node.
			// 4. Compile.
			// 5. Add a new cluster node. This crashes the editor.
			//
			// The crash happens because the map property keys for the instance don't match the CDO, so propagating
			// a new node to the instance's map results in an invalid access. This test catches 
			
			DisplayClusterTestUtils::AddClusterNodeToCluster(ClusterAsset, AssetRootCluster);
			UE::DisplayClusterConfiguratorClusterUtils::RemoveClusterNodeFromCluster(AssetClusterNode);
			FKismetEditorUtilities::CompileBlueprint(ClusterAsset);

			// We recompiled, so reacquire the instanced pointers
			CacheInstancedClusterPointers(true, false);
			
			DisplayClusterTestUtils::AddClusterNodeToCluster(ClusterAsset, AssetRootCluster);

			TestEqual(TEXT("instance key count (compared to CDO key count)"), InstancedRootCluster->Nodes.Num(), AssetRootCluster->Nodes.Num());
			for (const TPair<FString, TObjectPtr<UDisplayClusterConfigurationClusterNode>>& Pair : AssetRootCluster->Nodes)
			{
				if (TestTrue("Instance has entry corresponding to CDO key", InstancedRootCluster->Nodes.Contains(Pair.Key)))
				{
					TestNotNull("Instance has valid entry corresponding to CDO key", ToRawPtr(InstancedRootCluster->Nodes[Pair.Key]));
				}
			}
		});
		
		Describe("Unreal propagation", [this]
		{
			Describe("Propagation from asset to instance", [this]
			{
				RegisterPropagationTests();
			});
			
			Describe("Instance override prevents propagation from asset", [this]
			{
				RegisterPropertyPropagationTests(true);
			});
		});
		
		Describe("Custom propagation", [this]
		{
			BeforeEach([this]()
			{
				TriggerCustomPropagation();
			});
			
			Describe("Propagation from asset to instance", [this]
			{
				RegisterPropagationTests();
			});
			
			Describe("Instance override prevents propagation from asset", [this]
			{
				RegisterPropertyPropagationTests(true);
			});
		});
		
		Describe("Custom propagation + compilation", [this]
		{
			BeforeEach([this]()
			{
				TriggerCustomPropagation();
			});

			auto BeforeTestAction = [this]
			{
				FKismetEditorUtilities::CompileBlueprint(ClusterAsset);

				CacheInstancedClusterPointers(true);

				TestNotNull("Cluster actor after reinstancing", ClusterActor);
			};
			
			Describe("Propagation from asset to instance", [this, BeforeTestAction]
			{
				RegisterPropagationTests(BeforeTestAction);
			});
			
			Describe("Instance override prevents propagation from asset", [this, BeforeTestAction]
			{
				RegisterPropertyPropagationTests(true, BeforeTestAction);
			});
		});
	});

	AfterEach([this]()
	{
		DisplayClusterTestUtils::CleanUpAssetAndPackage(ClusterAsset);
		ClusterAsset = nullptr;

		if (World)
		{
			DisplayClusterTestUtils::CleanUpWorld(World);
			World = nullptr;
		}
	});
}

void FDisplayClusterSpec::RegisterPropertyPropagationTests(bool bDoInstanceOverrideTest, TFunction<void()> BeforeTestAction)
{
	// RenderFrameSettings
	PROPAGATION_TEST_SIMPLE(ClusterConfig, 7.f, 4.2f,
		TArray({ GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationData, RenderFrameSettings), GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationRenderFrame, ClusterICVFXOuterViewportBufferRatioMult) }));

	// StageSettings
	PROPAGATION_TEST_SIMPLE_NO_INSTANCE_OVERRIDE(ClusterConfig, false,
		TArray({ GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationData, StageSettings), GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_StageSettings, bEnableInnerFrustums) }));
	
	PROPAGATION_TEST_SIMPLE_NO_INSTANCE_OVERRIDE(ClusterConfig, false,
		TArray({
			GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationData, StageSettings),
			GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_StageSettings, ViewportOCIO),
			GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_ViewportOCIO, AllViewportsOCIOConfiguration),
			GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationOCIOConfiguration, bIsEnabled)
			}));

	// StageSettings.HideList
	PROPAGATION_TEST_SIMPLE(ClusterConfig, FName("TestName"), FName("OverrideName"),
		TArray({ GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationData, StageSettings), GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_StageSettings, HideList),
			GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_VisibilityList, ActorLayers), GET_MEMBER_NAME_CHECKED(FActorLayer, Name) }));
	
	PROPAGATION_TEST_SIMPLE(ClusterConfig, static_cast<UObject*>(ClusterActor), static_cast<UObject*>(AltActor),
		TArray({ GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationData, StageSettings), GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_StageSettings, HideList),
			GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_VisibilityList, Actors) }));
	
	// StageSettings.OuterViewportHideList
	PROPAGATION_TEST_SIMPLE(ClusterConfig, FName("TestName"), FName("OverrideName"),
		TArray({ GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationData, StageSettings), GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_StageSettings, OuterViewportHideList),
			GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_VisibilityList, ActorLayers), GET_MEMBER_NAME_CHECKED(FActorLayer, Name) }));
	
	PROPAGATION_TEST_SIMPLE(ClusterConfig, static_cast<UObject*>(ClusterActor), static_cast<UObject*>(AltActor),
		TArray({ GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationData, StageSettings), GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_StageSettings, OuterViewportHideList),
			GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_VisibilityList, Actors) }));
	
	// Cluster node settings
	PROPAGATION_TEST_SIMPLE(ClusterNode, 42, 17,
		TArray({ GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationClusterNode, WindowRect), GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationRectangle, X) }));
	
	// RenderSettings
	PROPAGATION_TEST_SIMPLE(Viewport, 112.f, 87.f,
		TArray({ GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationViewport, RenderSettings), GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_RenderSettings, Overscan),
			GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_Overscan, Right) }));

	// StageSettings.ViewportOCIO.AllViewportsOCIOConfiguration
	Describe("AllViewportsOCIOConfiguration", [this, BeforeTestAction, bDoInstanceOverrideTest]()
	{
		PROPAGATION_TEST_OCIO_CONFIG(ClusterConfig,
			TArray({
				GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationData, StageSettings),
				GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_StageSettings, ViewportOCIO),
				GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_ViewportOCIO, AllViewportsOCIOConfiguration)
				}));
	});

	// StageSettings.ViewportOCIO.PerViewportOCIOProfiles
	Describe("PerViewportOCIOProfiles", [this, BeforeTestAction, bDoInstanceOverrideTest]()
	{
		BeforeEach([this]()
		{
			const bool bSuccessfullyEnabled = DisplayClusterTestUtils::SetBlueprintPropertyValue(AssetClusterConfig, ClusterAsset,
				{
					GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationData, StageSettings),
					GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_StageSettings, ViewportOCIO),
					GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_ViewportOCIO, PerViewportOCIOProfiles),
					GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationOCIOProfile, bIsEnabled)
				},
				true);

			TestTrue("Successfully enabled OCIO profile", bSuccessfullyEnabled);
		});
		
		PROPAGATION_TEST_OCIO_CONFIG(ClusterConfig,
			TArray({
				GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationData, StageSettings),
				GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_StageSettings, ViewportOCIO),
				GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_ViewportOCIO, PerViewportOCIOProfiles)
				}));
		
		PROPAGATION_TEST_SIMPLE(ClusterConfig, FString("TestEntry"), FString("OverrideEntry"),
			TArray({
				GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationData, StageSettings),
				GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_StageSettings, ViewportOCIO),
				GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_ViewportOCIO, PerViewportOCIOProfiles),
				GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationOCIOProfile, ApplyOCIOToObjects)
				}));
	});

	// Light Cards OCIO
	{
		// StageSettings.Lightcard.LightcardOCIO.CustomOCIO.AllViewportsOCIOConfiguration
		Describe("Lightcard.AllViewportsOCIOConfiguration", [this, BeforeTestAction, bDoInstanceOverrideTest]()
			{
				PROPAGATION_TEST_OCIO_CONFIG(ClusterConfig,
					TArray({
						GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationData, StageSettings),
						GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_StageSettings, Lightcard),
						GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_LightcardSettings, LightcardOCIO),
						GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_LightcardOCIO, CustomOCIO),
						GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_LightcardCustomOCIO, AllViewportsOCIOConfiguration)
					}));
			});

		// StageSettings.Lightcard.LightcardOCIO.CustomOCIO.PerViewportOCIOProfiles
		Describe("Lightcard.PerViewportOCIOProfiles", [this, BeforeTestAction, bDoInstanceOverrideTest]()
			{
				BeforeEach([this]()
					{
						const bool bSuccessfullyEnabled = DisplayClusterTestUtils::SetBlueprintPropertyValue(AssetClusterConfig, ClusterAsset,
							{
								GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationData, StageSettings),
								GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_StageSettings, Lightcard),
								GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_LightcardSettings, LightcardOCIO),
								GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_LightcardOCIO, CustomOCIO),
								GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_LightcardCustomOCIO, PerViewportOCIOProfiles),
								GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationOCIOProfile, bIsEnabled)
							},
							true);

						TestTrue("Successfully enabled OCIO profile", bSuccessfullyEnabled);
					});

		PROPAGATION_TEST_OCIO_CONFIG(ClusterConfig,
			TArray({
				GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationData, StageSettings),
				GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_StageSettings, Lightcard),
				GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_LightcardSettings, LightcardOCIO),
				GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_LightcardOCIO, CustomOCIO),
				GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_LightcardCustomOCIO, PerViewportOCIOProfiles)
				}));

		PROPAGATION_TEST_SIMPLE(ClusterConfig, FString("TestEntry"), FString("OverrideEntry"),
			TArray({
				GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationData, StageSettings),
				GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_StageSettings, Lightcard),
				GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_LightcardSettings, LightcardOCIO),
				GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_LightcardOCIO, CustomOCIO),
				GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_LightcardCustomOCIO, PerViewportOCIOProfiles),
				GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationOCIOProfile, ApplyOCIOToObjects)
				}));
			});
	}
	
	// StageSettings.EntireClusterColorGrading
	Describe("EntireClusterColorGrading", [this, BeforeTestAction, bDoInstanceOverrideTest]()
	{
		BeforeEach([this]()
		{
			const bool bSuccessfullyEnabled = DisplayClusterTestUtils::SetBlueprintPropertyValue(AssetClusterConfig, ClusterAsset,
				{ GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationData, StageSettings), GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_StageSettings, EntireClusterColorGrading),
					GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_EntireClusterColorGrading, bEnableEntireClusterColorGrading) },
				true);
			TestTrue("Successfully enabled OCIO profile", bSuccessfullyEnabled);
			TestTrue("Entire cluster color grading enabled on asset", AssetClusterConfig->StageSettings.EntireClusterColorGrading.bEnableEntireClusterColorGrading);
			TestTrue("Entire cluster color grading enabled on instance", InstancedClusterConfig->StageSettings.EntireClusterColorGrading.bEnableEntireClusterColorGrading);
		});

		PROPAGATION_TEST_COLOR_GRADING_RENDERING_SETTINGS(ClusterConfig, StageSettings.EntireClusterColorGrading.ColorGradingSettings,
			TArray({ GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationData, StageSettings), GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_StageSettings, EntireClusterColorGrading),
				GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_EntireClusterColorGrading, ColorGradingSettings) }));
	});
	
	// StageSettings.PerViewportColorGrading
	Describe("PerViewportColorGrading", [this, BeforeTestAction, bDoInstanceOverrideTest]()
	{
		BeforeEach([this]()
		{
			const bool bSuccessfullyEnabled = DisplayClusterTestUtils::SetBlueprintPropertyValue(AssetClusterConfig, ClusterAsset,
				{ GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationData, StageSettings), GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_StageSettings, PerViewportColorGrading),
					GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_PerViewportColorGrading, bIsEnabled) },
				true);
			TestTrue("Successfully enabled OCIO profile", bSuccessfullyEnabled);
			TestTrue("Entire cluster color grading enabled on asset", AssetClusterConfig->StageSettings.PerViewportColorGrading[0].bIsEnabled);
			TestTrue("Entire cluster color grading enabled on instance", InstancedClusterConfig->StageSettings.PerViewportColorGrading[0].bIsEnabled);
		});

		PROPAGATION_TEST_COLOR_GRADING_RENDERING_SETTINGS(ClusterConfig, StageSettings.PerViewportColorGrading[0].ColorGradingSettings,
			TArray({ GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationData, StageSettings), GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_StageSettings, PerViewportColorGrading),
				GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_PerViewportColorGrading, ColorGradingSettings) }));

		PROPAGATION_TEST_SIMPLE_NO_INSTANCE_OVERRIDE(ClusterConfig, false,
			TArray({ GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationData, StageSettings), GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_StageSettings, PerViewportColorGrading),
				GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_PerViewportColorGrading, bIsEntireClusterEnabled) }));

		PROPAGATION_TEST_SIMPLE(ClusterConfig, AssetViewport->GetName(), FString("OverrideName"),
			TArray({ GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationData, StageSettings), GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_StageSettings, PerViewportColorGrading),
				GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_PerViewportColorGrading, ApplyPostProcessToObjects) }));
	});

	// StageSettings.Lightcard
	Describe("Lightcard settings", [this, BeforeTestAction, bDoInstanceOverrideTest]()
	{
		BeforeEach([this]()
		{
			const bool bSuccessfullyEnabled = DisplayClusterTestUtils::SetBlueprintPropertyValue(AssetClusterConfig, ClusterAsset,
				{ GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationData, StageSettings), GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_StageSettings, Lightcard),
					GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_LightcardSettings, bEnable) },
				true);
			TestTrue("Successfully enabled lightcard", bSuccessfullyEnabled);
		});
		
		PROPAGATION_TEST_SIMPLE_NO_INSTANCE_OVERRIDE(ClusterConfig, false,
			TArray({ GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationData, StageSettings), GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_StageSettings, Lightcard),
				GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_LightcardSettings, bEnable) }));
		
		PROPAGATION_TEST_ENUM_NO_INSTANCE_OVERRIDE(ClusterConfig, EDisplayClusterConfigurationICVFX_LightcardRenderMode::Over,
			TArray({ GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationData, StageSettings), GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_StageSettings, Lightcard),
				GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_LightcardSettings, Blendingmode) }));
		
		PROPAGATION_TEST_SIMPLE(ClusterConfig, FName("TestName"), FName("OverrideName"),
			TArray({ GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationData, StageSettings), GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_StageSettings, Lightcard),
				GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_LightcardSettings, ShowOnlyList), GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_VisibilityList, ActorLayers),
				GET_MEMBER_NAME_CHECKED(FActorLayer, Name) }));
		
		PROPAGATION_TEST_SIMPLE(ClusterConfig, static_cast<UObject*>(ClusterActor), static_cast<UObject*>(AltActor),
			TArray({ GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationData, StageSettings), GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_StageSettings, Lightcard),
				GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_LightcardSettings, ShowOnlyList), GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationICVFX_VisibilityList, Actors) }));
	});

	Describe("Component propagation", [this, BeforeTestAction, bDoInstanceOverrideTest]()
	{
		BeforeEach([this]()
		{
			// Add a component to the CDO
			USCS_Node* NewNode = ClusterAsset->SimpleConstructionScript->CreateNode(UDisplayClusterXformComponent::StaticClass(), FName(TestComponentName));
			FComponentAssetBrokerage::AssignAssetToComponent(NewNode->ComponentTemplate, ClusterAsset);

			AssetComponent = CastChecked<UDisplayClusterXformComponent>(NewNode->GetActualComponentTemplate(ClusterAsset->GetGeneratedClass()));
			ClusterAsset->SimpleConstructionScript->GetRootNodes()[0]->AddChildNode(NewNode);
			
			FKismetEditorUtilities::GenerateBlueprintSkeleton(ClusterAsset, true);
			FBlueprintEditorUtils::PostEditChangeBlueprintActors(ClusterAsset);

			InstancedComponent = ClusterActor->GetComponentByName<UDisplayClusterXformComponent>(TestComponentName);
			
			TestNotNull("Component exists on instance", InstancedComponent);
			TestNotNull("Component exists on asset", AssetComponent);
		});
	
		It("should propagate component data", [this, BeforeTestAction, bDoInstanceOverrideTest]()
		{
			const FVector RelativeLocationOverride = FVector(-200.f, 325.f, 610.f);
			if (bDoInstanceOverrideTest)
			{
				InstancedComponent->SetRelativeLocation(RelativeLocationOverride);
			}
			
			// DisplayCluster utils for working with property handles don't play nice outside the context of the cluster config,
			// so instead we set the value manually here and force it to propagate
			const FVector OldRelativeLocation = AssetComponent->GetRelativeLocation();
			const FVector RelativeLocation = FVector(300.f, -150.f, 42.f);
			AssetComponent->SetRelativeLocation(RelativeLocation);

			const FProperty* Property = FindFProperty<FProperty>(UDisplayClusterXformComponent::StaticClass(), USceneComponent::GetRelativeLocationPropertyName());
			TSet<USceneComponent*> UpdatedInstances;
			FComponentEditorUtils::PropagateDefaultValueChange(AssetComponent, Property, OldRelativeLocation, RelativeLocation, UpdatedInstances);

			BeforeTestAction();

			// Get the instanced component again in case BeforeTestAction invalidated the pointer
			InstancedComponent = ClusterActor->GetComponentByName<UDisplayClusterXformComponent>(TestComponentName);

			if (bDoInstanceOverrideTest)
			{
				TestEqual(TEXT("Set asset component's location"), AssetComponent->GetRelativeLocation(), RelativeLocation);
				TestEqual(TEXT("Did not propagate over overridden instance location"), InstancedComponent->GetRelativeLocation(), RelativeLocationOverride);
			}
			else
			{
				TestEqual(TEXT("Propagated asset component's location to instance"), InstancedComponent->GetRelativeLocation(), RelativeLocation);
			}
		});
	});
}

void FDisplayClusterSpec::RegisterPropagationTests(TFunction<void()> BeforeTestAction)
{
	RegisterPropertyPropagationTests(false, BeforeTestAction);
	
	It("should propagate a new cluster node", [this, BeforeTestAction]()
	{
		DisplayClusterTestUtils::AddClusterNodeToCluster(ClusterAsset, AssetRootCluster);

		BeforeTestAction();

		if (TestEqual(TEXT("Both nodes are present in asset's cluster"), AssetRootCluster->Nodes.Num(), 2))
		{
			for (const TPair<FString, TObjectPtr<UDisplayClusterConfigurationClusterNode>>& Pair : AssetRootCluster->Nodes)
			{
				TestNotNull(TEXT("All asset nodes are valid"), ToRawPtr(Pair.Value));
			}
		}
		
		if (TestEqual(TEXT("Both nodes are present in instanced cluster"), InstancedRootCluster->Nodes.Num(), 2))
		{
			for (const TPair<FString, TObjectPtr<UDisplayClusterConfigurationClusterNode>>& Pair : AssetRootCluster->Nodes)
			{
				TestNotNull(TEXT("All instance nodes are valid"), ToRawPtr(Pair.Value));
			}
		}
	});
	
	It("should propagate a new viewport", [this, BeforeTestAction]()
	{
		DisplayClusterTestUtils::AddViewportToClusterNode(ClusterAsset, AssetClusterNode);

		BeforeTestAction();

		if (TestEqual(TEXT("Both viewports are present in asset's node"), AssetClusterNode->Viewports.Num(), 2))
		{
			for (const TPair<FString, TObjectPtr<UDisplayClusterConfigurationViewport>>& Pair : AssetClusterNode->Viewports)
			{
				TestNotNull(TEXT("All asset viewports are valid"), ToRawPtr(Pair.Value));
			}
		}
		
		if (TestEqual(TEXT("Both viewports are present in instanced node"), InstancedClusterNode->Viewports.Num(), 2))
		{
			for (const TPair<FString, TObjectPtr<UDisplayClusterConfigurationViewport>>& Pair : AssetClusterNode->Viewports)
			{
				TestNotNull(TEXT("All instanced viewports are valid"), ToRawPtr(Pair.Value));
			}
		}
	});
}

///////////////////////////////////////
/// Parameterized propagation tests ///
///////////////////////////////////////

template <typename OwnerType, typename ValueType, typename HandleOverrideType>
void FDisplayClusterSpec::DoPropertyPropagationTest(TFunction<OwnerType*()> AssetOwnerAccessor, TFunction<OwnerType*()> InstancedOwnerAccessor, TFunction<ValueType()> AssetValueFn,
	TFunction<ValueType()> InstanceValueFn, FString OwnerName, const TArray<FName>& FieldNames, TFunction<void()> BeforeTestAction, TFunction<void(OwnerType*)> EnableFunc,
	bool bDoInstanceOverrideTest, bool bAllowInstanceOverrideTest)
{
	if (bDoInstanceOverrideTest && !bAllowInstanceOverrideTest)
	{
		return;
	}

	FString CombinedFieldName = FieldNames[0].ToString();
	for (int i = 1; i < FieldNames.Num(); ++i)
	{
		// Use -> instead of . since Unreal uses . as a test category separator, so it'll mess up the display
		CombinedFieldName += "->" + FieldNames[i].ToString();
	}
	
	const FString ItDescription = FString("should propagate " + OwnerName + "'s " + CombinedFieldName + " from asset to instance");
	
	It(*ItDescription, [this, AssetOwnerAccessor, InstancedOwnerAccessor, AssetValueFn, InstanceValueFn, FieldNames, BeforeTestAction, EnableFunc, bDoInstanceOverrideTest]
	{
		OwnerType* AssetOwner = AssetOwnerAccessor();
		OwnerType* InstancedOwner = InstancedOwnerAccessor();

		if (!(TestNotNull(TEXT("Asset owner exists"), AssetOwner) && TestNotNull(TEXT("Instanced owner exists"), InstancedOwner)))
		{
			return;
		}

		if (bDoInstanceOverrideTest)
		{
			if (EnableFunc)
			{
				EnableFunc(InstancedOwner);
			}

			// Confirm that the values aren't already the same since this wouldn't trigger propagation in the first place
			HandleOverrideType PrevInstanceValue;
			DisplayClusterTestUtils::GetBlueprintPropertyValue(InstancedOwner, FieldNames, PrevInstanceValue);
			const ValueType NewInstanceValue = InstanceValueFn();
			if (!TestNotEqual("Test value for instance must not equal default value", static_cast<HandleOverrideType>(NewInstanceValue), PrevInstanceValue))
			{
				return;
			}
			
			// We want to test that the value does *NOT* propagate when the instanced value already has an override, so
			// set the value on the instance first.
			const bool bWasSetOnInstance = DisplayClusterTestUtils::SetBlueprintPropertyValue<HandleOverrideType>(InstancedOwnerAccessor(),
				nullptr, FieldNames, static_cast<HandleOverrideType>(NewInstanceValue));

			if (!TestTrue(TEXT("Successfully set property value on instance"), bWasSetOnInstance))
			{
				return;
			}
		}

		// Enable value on asset if necessary
		if (EnableFunc)
		{
			EnableFunc(AssetOwner);

			// We can't access edit condition properties through a PropertyHandle, so instead, make sure we trigger the
			// code path for changing this as a Blueprint property. This will notify the instance that it should be
			// updated.
			ClusterAsset->MarkPackageDirty();
			FBlueprintEditorUtils::PostEditChangeBlueprintActors(ClusterAsset);
		}

		// Confirm that the values aren't already the same since this wouldn't trigger propagation in the first place
		HandleOverrideType PrevAssetValue;
		DisplayClusterTestUtils::GetBlueprintPropertyValue(AssetOwner, FieldNames, PrevAssetValue);
		const ValueType NewAssetValue = AssetValueFn();
		if (!TestNotEqual("Test value for asset must not equal default value", static_cast<HandleOverrideType>(NewAssetValue), PrevAssetValue))
		{
			return;
		}
	
		// Set the value on the asset
		const bool bWasSetOnAsset = DisplayClusterTestUtils::SetBlueprintPropertyValue<HandleOverrideType>(AssetOwner,
			ClusterAsset, FieldNames, static_cast<HandleOverrideType>(NewAssetValue));
			
		if (TestTrue(TEXT("Successfully set property value on asset"), bWasSetOnAsset))
		{
			BeforeTestAction();

			// Get the instanced owner again in case it was changed by e.g. reinstancing
			InstancedOwner = InstancedOwnerAccessor();
			
			HandleOverrideType AssetValue, InstanceValue;
			TestTrue(TEXT("Succesfully retrieved asset value"), DisplayClusterTestUtils::GetBlueprintPropertyValue(AssetOwner, FieldNames, AssetValue));
			TestTrue(TEXT("Succesfully retrieved instanced value"), DisplayClusterTestUtils::GetBlueprintPropertyValue(InstancedOwner, FieldNames, InstanceValue));

			if (bDoInstanceOverrideTest)
			{
				TestEqual(TEXT("Asset's value matches expected value"), AssetValue, static_cast<HandleOverrideType>(AssetValueFn()));
				TestEqual(TEXT("Instance's value was not changed"), InstanceValue, static_cast<HandleOverrideType>(InstanceValueFn()));
			}
			else
			{
				TestEqual(TEXT("Instance's value matches asset's"), InstanceValue, AssetValue);
			}
		}
	});
}

template <typename OwnerType>
void FDisplayClusterSpec::DoColorGradingSettingsPropagationTests(TFunction<OwnerType*()> AssetOwnerAccessor, TFunction<OwnerType*()> InstancedOwnerAccessor,
	TFunction<FDisplayClusterConfigurationViewport_ColorGradingSettings*(OwnerType*)> AssetSettingsAccessor,
	FString OwnerName, const TArray<FName>& FieldNames, TFunction<void()> BeforeTestAction, bool bDoInstanceOverrideTest)
{
	#define COLOR_GRADING_SETTINGS_VECTOR_TEST_INTERNAL(VECTOR_NAME) \
	{ \
		TArray<FName> SubFieldNames = FieldNames; \
		SubFieldNames.Add(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_ColorGradingSettings, VECTOR_NAME)); \
		\
		DoPropertyPropagationTest<OwnerType, FVector4>( \
			AssetOwnerAccessor, \
			InstancedOwnerAccessor, \
			[]() { return FVector4(0.5f, 0.1f, 0.9f, 0.4f); }, \
			[]() { return FVector4(0.2f, 0.9f, 0.3f, 0.6f); }, \
			OwnerName, \
			SubFieldNames, \
			BeforeTestAction, \
			[AssetSettingsAccessor] (OwnerType* Owner) { AssetSettingsAccessor(Owner)->bOverride_ ## VECTOR_NAME = true; }, \
			bDoInstanceOverrideTest, \
			true); \
	}
	
	const FString GroupName = FString::Printf(TEXT("Color grading settings (%s)"), *FieldNames.Last().ToString()); 
	Describe(*GroupName, [this, FieldNames, BeforeTestAction, AssetOwnerAccessor, InstancedOwnerAccessor, AssetSettingsAccessor, OwnerName, bDoInstanceOverrideTest]
	{
		COLOR_GRADING_SETTINGS_VECTOR_TEST_INTERNAL(Saturation);
		COLOR_GRADING_SETTINGS_VECTOR_TEST_INTERNAL(Contrast);
		COLOR_GRADING_SETTINGS_VECTOR_TEST_INTERNAL(Gamma);
		COLOR_GRADING_SETTINGS_VECTOR_TEST_INTERNAL(Gain);
		COLOR_GRADING_SETTINGS_VECTOR_TEST_INTERNAL(Offset);
	});
}

template <typename OwnerType>
void FDisplayClusterSpec::DoColorGradingRenderingSettingsPropagationTests(TFunction<OwnerType*()> AssetOwnerAccessor, TFunction<OwnerType*()> InstancedOwnerAccessor,
	TFunction<FDisplayClusterConfigurationViewport_ColorGradingRenderingSettings*(OwnerType*)> AssetSettingsAccessor,
	FString OwnerName, const TArray<FName>& FieldNames, TFunction<void()> BeforeTestAction, bool bDoInstanceOverrideTest)
{
	#define COLOR_GRADING_RENDERING_SETTINGS_SETTINGS_TEST_INTERNAL(SETTINGS_NAME) \
	{ \
		TArray<FName> SETTINGS_NAME ## FieldNames = FieldNames; \
		SETTINGS_NAME ## FieldNames.Add(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_ColorGradingRenderingSettings, SETTINGS_NAME)); \
		\
		DoColorGradingSettingsPropagationTests<OwnerType>( \
			AssetOwnerAccessor, \
			InstancedOwnerAccessor, \
			[AssetSettingsAccessor](OwnerType* Owner) { return &AssetSettingsAccessor(Owner)->SETTINGS_NAME; }, \
			OwnerName, \
			SETTINGS_NAME ## FieldNames, \
			BeforeTestAction, \
			bDoInstanceOverrideTest); \
	}

	#define COLOR_GRADING_RENDERING_SETTINGS_TEST_INTERNAL(OWNER_TYPE, ASSET_VALUE, INSTANCE_VALUE, FIELD_NAME) \
	{ \
		TArray<FName> SubFieldNames = FieldNames; \
		SubFieldNames.Add(GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_ColorGradingRenderingSettings, FIELD_NAME)); \
		\
		DoPropertyPropagationTest<OWNER_TYPE, decltype(ASSET_VALUE)>( \
			AssetOwnerAccessor, \
			InstancedOwnerAccessor, \
			[]() { return ASSET_VALUE; }, \
			[]() { return INSTANCE_VALUE; }, \
			OwnerName, \
			SubFieldNames, \
			BeforeTestAction, \
			[AssetSettingsAccessor] (OwnerType* Owner) { AssetSettingsAccessor(Owner)->bOverride_ ## FIELD_NAME = true; }, \
			bDoInstanceOverrideTest, \
			true); \
	}

	#define COLOR_GRADING_RENDERING_SETTINGS_MISC_TEST_INTERNAL(OWNER_TYPE, ASSET_VALUE, INSTANCE_VALUE, FIELD_NAME) \
	{ \
		TArray<FName> SubFieldNames = FieldNames; \
		SubFieldNames.Append({ \
			GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_ColorGradingRenderingSettings, Misc), \
			GET_MEMBER_NAME_CHECKED(FDisplayClusterConfigurationViewport_ColorGradingMiscSettings, FIELD_NAME) \
		}); \
		\
		DoPropertyPropagationTest<OWNER_TYPE, decltype(ASSET_VALUE)>( \
			AssetOwnerAccessor, \
			InstancedOwnerAccessor, \
			[]() { return ASSET_VALUE; }, \
			[]() { return INSTANCE_VALUE; }, \
			OwnerName, \
			SubFieldNames, \
			BeforeTestAction, \
			[AssetSettingsAccessor] (OwnerType* Owner) { AssetSettingsAccessor(Owner)->Misc.bOverride_ ## FIELD_NAME = true; }, \
			bDoInstanceOverrideTest, \
			true); \
	}

	COLOR_GRADING_RENDERING_SETTINGS_SETTINGS_TEST_INTERNAL(Global);
	COLOR_GRADING_RENDERING_SETTINGS_SETTINGS_TEST_INTERNAL(Shadows);
	COLOR_GRADING_RENDERING_SETTINGS_SETTINGS_TEST_INTERNAL(Midtones);
	COLOR_GRADING_RENDERING_SETTINGS_SETTINGS_TEST_INTERNAL(Highlights);

	COLOR_GRADING_RENDERING_SETTINGS_TEST_INTERNAL(UDisplayClusterConfigurationData, 13.7f, 11.5f, AutoExposureBias);
	COLOR_GRADING_RENDERING_SETTINGS_TEST_INTERNAL(UDisplayClusterConfigurationData, 0.78f, 0.6f, ColorCorrectionShadowsMax);
	COLOR_GRADING_RENDERING_SETTINGS_TEST_INTERNAL(UDisplayClusterConfigurationData, -0.34f, -0.2f, ColorCorrectionHighlightsMin);
	COLOR_GRADING_RENDERING_SETTINGS_TEST_INTERNAL(UDisplayClusterConfigurationData, 1.34f, 3.2f, ColorCorrectionHighlightsMax);
	COLOR_GRADING_RENDERING_SETTINGS_MISC_TEST_INTERNAL(UDisplayClusterConfigurationData, 0.22f, 0.42f, BlueCorrection);
	COLOR_GRADING_RENDERING_SETTINGS_MISC_TEST_INTERNAL(UDisplayClusterConfigurationData, 0.67f, 0.45f, ExpandGamut);
	COLOR_GRADING_RENDERING_SETTINGS_MISC_TEST_INTERNAL(UDisplayClusterConfigurationData, FLinearColor(0.6f, 0.2f, 0.9f, 0.3f), FLinearColor(0.2f, 0.5f, 0.1f, 0.8f), SceneColorTint);
}

template <typename OwnerType>
void FDisplayClusterSpec::DoOCIOConfigurationPropagationTests(TFunction<OwnerType*()> AssetOwnerAccessor, TFunction<OwnerType*()> InstancedOwnerAccessor,
	FString OwnerName, const TArray<FName>& FieldNames, TFunction<void()> BeforeTestAction, bool bDoInstanceOverrideTest)
{
	#define OCIO_CONFIG_TEST_INTERNAL(ASSET_VALUE, INSTANCE_VALUE, APPENDED_FIELD_NAMES) \
	{ \
		TArray<FName> SubFieldNames = FieldNames; \
		SubFieldNames.Append(APPENDED_FIELD_NAMES); \
		\
		DoPropertyPropagationTest<OwnerType, decltype(ASSET_VALUE)>( \
			AssetOwnerAccessor, \
			InstancedOwnerAccessor, \
			[this]() { return ASSET_VALUE; }, \
			[this]() { return INSTANCE_VALUE; }, \
			OwnerName, \
			SubFieldNames, \
			BeforeTestAction, \
			nullptr, \
			bDoInstanceOverrideTest, \
			true); \
	}

	BeforeEach([this]()
	{
		OCIOConfig = NewObject<UOpenColorIOConfiguration>(ClusterAsset->GetPackage(), FName("TestOCIOConfig"), RF_Transient | RF_Public);
		OCIOConfig->AddToRoot();
		FAssetRegistryModule::AssetCreated(OCIOConfig.Get());
		
		AltOCIOConfig = NewObject<UOpenColorIOConfiguration>(ClusterAsset->GetPackage(), FName("TestOCIOConfig"), RF_Transient | RF_Public);
		AltOCIOConfig->AddToRoot();
		FAssetRegistryModule::AssetCreated(AltOCIOConfig.Get());
	});
	
	const FString GroupName = FString::Printf(TEXT("Color grading settings (%s)"), *FieldNames.Last().ToString()); 
	Describe(*GroupName, [this, FieldNames, BeforeTestAction, AssetOwnerAccessor, InstancedOwnerAccessor, OwnerName, bDoInstanceOverrideTest]
	{
		OCIO_CONFIG_TEST_INTERNAL(static_cast<UObject*>(OCIOConfig.Get()), static_cast<UObject*>(AltOCIOConfig.Get()),
			TArray({ GET_MEMBER_NAME_CHECKED(FOpenColorIODisplayConfiguration, ColorConfiguration), GET_MEMBER_NAME_CHECKED(FOpenColorIOColorConversionSettings, ConfigurationSource) }));
		
		OCIO_CONFIG_TEST_INTERNAL(FString("TestName"), FString("OverrideName"),
			TArray({ GET_MEMBER_NAME_CHECKED(FOpenColorIODisplayConfiguration, ColorConfiguration), GET_MEMBER_NAME_CHECKED(FOpenColorIOColorConversionSettings, DestinationColorSpace),
				GET_MEMBER_NAME_CHECKED(FOpenColorIOColorSpace, ColorSpaceName) }));
		
		OCIO_CONFIG_TEST_INTERNAL(FString("TestName"), FString("OverrideName"),
			TArray({ GET_MEMBER_NAME_CHECKED(FOpenColorIODisplayConfiguration, ColorConfiguration), GET_MEMBER_NAME_CHECKED(FOpenColorIOColorConversionSettings, SourceColorSpace),
				GET_MEMBER_NAME_CHECKED(FOpenColorIOColorSpace, ColorSpaceName) }));
	});

	AfterEach([this]()
	{
		DisplayClusterTestUtils::CleanUpAsset(OCIOConfig.Get());
		OCIOConfig.Reset();
		
		DisplayClusterTestUtils::CleanUpAsset(AltOCIOConfig.Get());
		AltOCIOConfig.Reset();
	});
}

///////////////////////
// Utility functions //
///////////////////////

void FDisplayClusterSpec::CacheAssetClusterNodeAndViewport()
{
	AssetViewport = nullptr;
	AssetClusterNode = AssetRootCluster->Nodes.FindChecked(TestNodeName);

	if (TestNotNull(TEXT("Got valid cluster node for Asset data"), AssetClusterNode) && AssetClusterNode->Viewports.Num() > 0)
	{
		AssetViewport = AssetClusterNode->Viewports.FindChecked(TestViewportName);
		TestNotNull(TEXT("Got valid viewport for Asset data"), AssetViewport);
	}
}

void FDisplayClusterSpec::CacheInstancedClusterPointers(bool bFindActor, bool bCacheTestNodeAndViewport)
{
	if (bFindActor)
	{
		// Actor may have been reinstanced after compilation, so our pointer would be stale. Find the actor
		// again in the world so we're pointing at the fresh one.
		ClusterActor = nullptr;
		for (TActorIterator<ADisplayClusterRootActor> It(World); It; ++It)
		{
			if (It->GetName() == TestActorName)
			{
				ClusterActor = *It;
				break;
			}
		}
	}

	InstancedClusterConfig = nullptr;
	InstancedRootCluster = nullptr;
	InstancedClusterNode = nullptr;
	InstancedViewport = nullptr;

	if (TestNotNull(TEXT("Got valid cluster actor"), ClusterActor))
	{
		// Get pointers to each of the instanced objects we care about
		InstancedClusterConfig = ClusterActor->GetConfigData();
		if (TestNotNull(TEXT("Got valid config for instanced data"), InstancedClusterConfig))
		{
			InstancedRootCluster = InstancedClusterConfig->Cluster;
				
			if (TestNotNull(TEXT("Got valid root cluster for instanced data"), InstancedRootCluster) && InstancedRootCluster->Nodes.Num() > 0)
			{
				if (bCacheTestNodeAndViewport)
				{
					InstancedClusterNode = InstancedRootCluster->Nodes.FindChecked(TestNodeName);
				
					if (TestNotNull(TEXT("Got valid cluster node for instanced data"), InstancedClusterNode) && InstancedClusterNode->Viewports.Num() > 0)
					{
						InstancedViewport = InstancedClusterNode->Viewports.FindChecked(TestViewportName);

						TestNotNull(TEXT("Got valid viewport for instanced data"), InstancedViewport);
					}
				}
			}
		}
	}
}

void FDisplayClusterSpec::TriggerCustomPropagation()
{
	// Change the buffer ratio on the instance so that it's forced to use custom propagation
	const bool bSuccess = DisplayClusterTestUtils::SetBlueprintPropertyValue(InstancedViewport, nullptr,
		{ GET_MEMBER_NAME_CHECKED(UDisplayClusterConfigurationViewport, RenderSettings), GET_MEMBER_NAME_CHECKED(FDisplayClusterViewport_RenderSettings, BufferRatio) },
		3.f);

	TestTrue(TEXT("Successfully set BufferRatio value"), bSuccess);
}

template <typename T>
bool FDisplayClusterSpec::TestEqual(const TCHAR* What, T Actual, T Expected)
{
	return FAutomationTestBase::TestEqual(What, Actual, Expected);
}

// Specialization for FVector4 since it would normally be implicitly converted to FVector and lose its 4th value
template <>
bool FDisplayClusterSpec::TestEqual(const TCHAR* What, FVector4 Actual, FVector4 Expected)
{
	constexpr float Tolerance = KINDA_SMALL_NUMBER;
	if (!Expected.Equals(Actual, Tolerance))
	{
		AddError(FString::Printf(TEXT("Expected '%s' to be %s, but it was %s within tolerance %f."), What, *Expected.ToString(), *Actual.ToString(), Tolerance), 1);
		return false;
	}
	return true;
}

#endif //WITH_DEV_AUTOMATION_TESTS
