// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterTestUtils.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "Blueprints/DisplayClusterBlueprint.h"
#include "ClusterConfiguration/DisplayClusterConfiguratorClusterUtils.h"
#include "DisplayClusterConfiguratorFactory.h"
#include "DisplayClusterTestsModule.h"
#include "Editor/UnrealEdEngine.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "ISinglePropertyView.h"
#include "ObjectTools.h"
#include "PackageTools.h"
#include "Tests/AutomationEditorCommon.h"
#include "UnrealEdGlobals.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace DisplayClusterTestUtils
{
	UDisplayClusterBlueprint* CreateDisplayClusterAsset()
	{
		UObject* Asset = nullptr;
		UPackage* Package = nullptr;
		
		if (UDisplayClusterConfiguratorFactory* Factory = NewObject<UDisplayClusterConfiguratorFactory>())
		{
			const FString AssetName("NewClusterAsset");
			Package = CreatePackage(*FString::Printf(TEXT("/Temp/%s"), *AssetName));
			if (Package)
			{
				constexpr EObjectFlags ObjectFlags = RF_Transient | RF_Public;
				Asset = Factory->FactoryCreateNew(UDisplayClusterBlueprint::StaticClass(), Package, FName(*AssetName), ObjectFlags, nullptr, GWarn);
				
				if (Asset)
				{
					FAssetRegistryModule::AssetCreated(Asset);
				}
				else
				{
					UE_LOGF(LogDisplayClusterTests, Error, "Failed to create asset");
				}
			}
			else
			{
				UE_LOGF(LogDisplayClusterTests, Error, "Failed to create package");
			}
		}
		else
		{
			UE_LOGF(LogDisplayClusterTests, Error, "Failed to create factory");
		}

		UDisplayClusterBlueprint* DisplayClusterBlueprint = Cast<UDisplayClusterBlueprint>(Asset);
		if (!DisplayClusterBlueprint)
		{
			// Output will be null, so package/asset can't be cleaned up later based on the asset pointer (which is the
			// only thing we return. Clean it up now instead.
			CleanUpPackage(Package);

			if (Asset)
			{
				CleanUpAsset(Asset);
			}
		}

		// Prevent garbage collection of the asset and package
		Asset->AddToRoot();
		Package->AddToRoot();

		return DisplayClusterBlueprint;
	}

	UDisplayClusterConfigurationClusterNode* AddClusterNodeToCluster(UBlueprint* Blueprint, UDisplayClusterConfigurationCluster* RootCluster, FString Name, bool bCallPostEditChange)
	{
		UDisplayClusterConfigurationClusterNode* NodeTemplate = NewObject<UDisplayClusterConfigurationClusterNode>(Blueprint);

		// This must be set to avoid errors about illegal cross-package references
		NodeTemplate->SetFlags(RF_Transactional);

		UDisplayClusterConfigurationClusterNode* NewNode = UE::DisplayClusterConfiguratorClusterUtils::AddClusterNodeToCluster(NodeTemplate, RootCluster, Name);

		// Node template is no longer needed, leave it to be cleaned up
		NodeTemplate->MarkAsGarbage();

		if (bCallPostEditChange)
		{
			// Trigger Blueprint updates as if we were in an editor. This will re-run construction scripts.
			FBlueprintEditorUtils::PostEditChangeBlueprintActors(Blueprint);
		}

		return NewNode;
	}

	UDisplayClusterConfigurationViewport* AddViewportToClusterNode(UBlueprint* Blueprint, UDisplayClusterConfigurationClusterNode* Node, FString Name, bool bCallPostEditChange)
	{
		UDisplayClusterConfigurationViewport* ViewportTemplate = NewObject<UDisplayClusterConfigurationViewport>(Blueprint);

		// This must be set to avoid errors about illegal cross-package references
		ViewportTemplate->SetFlags(RF_Transactional);

		UDisplayClusterConfigurationViewport* NewViewport = UE::DisplayClusterConfiguratorClusterUtils::AddViewportToClusterNode(ViewportTemplate, Node, Name);

		// Node template is no longer needed, leave it to be cleaned up
		ViewportTemplate->MarkAsGarbage();
		
		if (bCallPostEditChange)
		{
			// Trigger Blueprint updates as if we were in an editor. This will re-run construction scripts.
			FBlueprintEditorUtils::PostEditChangeBlueprintActors(Blueprint);
		}

		return NewViewport;
	}

	UWorld* CreateWorld()
	{
		UWorld* World = UWorld::CreateWorld(EWorldType::Game, false, FName(TEXT("DisplayClusterTestWorld")));
		
		UPackage* Package = World->GetPackage();
		Package->SetFlags(RF_Transient | RF_Public);
		Package->AddToRoot();
		
		FWorldContext& WorldContext = GEngine->CreateNewWorldContext(EWorldType::Game);
		WorldContext.SetCurrentWorld(World);

		return World;
	}
	
	void CleanUpPackage(UPackage* Package)
	{
		if (!Package)
		{
			return;
		}

		Package->RemoveFromRoot();
	
		TArray<UPackage*> PackagesToDelete;
		PackagesToDelete.Add(Package);
		// Let the package auto-saver know that it needs to ignore the deleted packages
		GUnrealEd->GetPackageAutoSaver().OnPackagesDeleted(PackagesToDelete);

		Package->SetDirtyFlag(false);

		// Unload the packages and collect garbage.
		UPackageTools::UnloadPackages(PackagesToDelete);
	}

	void CleanUpAsset(UObject* Asset)
	{
		if (!Asset)
		{
			return;
		}

		// Rather than calling ObjectTools::ForceDeleteObjects, we just mark the asset and any of its Blueprint instances
		// as garbage and trigger collection. This runs much faster (important since we need to do it after each test), but
		// assumes that we haven't created any other references to the asset in the process of running the test.
		
		FAutomationEditorCommonUtils::NullReferencesToObject(Asset);
		Asset->RemoveFromRoot();

		UBlueprint* BlueprintObject = Cast<UBlueprint>(Asset);
		if (BlueprintObject && BlueprintObject->GeneratedClass && BlueprintObject->GeneratedClass->GetDefaultObject(false))
		{
			TArray<UObject*> InstancesToDelete;
			BlueprintObject->GeneratedClass->GetDefaultObject(false)->GetArchetypeInstances( InstancesToDelete );
			
			for (TArray<UObject*>::TConstIterator InstanceItr( InstancesToDelete ); InstanceItr; ++InstanceItr)
			{
				UObject* CurrentInstance = *InstanceItr;
				FAutomationEditorCommonUtils::NullReferencesToObject(CurrentInstance);
				CurrentInstance->MarkAsGarbage();
			}
		}
		
		Asset->MarkAsGarbage();
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
	}

	void CleanUpAssetAndPackage(UObject* Asset)
	{
		if (!Asset)
		{
			return;
		}
	
		UPackage* Package = Asset->GetPackage();

		CleanUpAsset(Asset);
		CleanUpPackage(Package);
	}

	void CleanUpWorld(UWorld* World)
	{
		UPackage* Package = World->GetPackage();

		World->RemoveFromRoot();
		GEngine->DestroyWorldContext(World);
		World->DestroyWorld(false);
		World->MarkAsGarbage();
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);

		CleanUpPackage(Package);
	}
	
	bool GetPropertyViewAndHandleFromFieldNames(UObject* Owner, const TArray<FName>& FieldNames, bool bAllowAdd, TSharedPtr<ISinglePropertyView>& PropertyView, TSharedPtr<IPropertyHandle>& PropertyHandle)
	{
		if (FieldNames.Num() == 0)
		{
			return false;
		}
		
		// Get the root property view and handle -- equivalent to UE::DisplayClusterConfiguratorPropertyUtils::GetPropertyView
		const FSinglePropertyParams InitParams;
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyView = PropertyEditorModule.CreateSingleProperty(
			Owner,
			FieldNames[0],
			InitParams);
		
		if (!PropertyView.IsValid())
		{
			return false;
		}

		PropertyHandle = PropertyView->GetPropertyHandle();

		if (!PropertyHandle.IsValid())
		{
			return false;
		}

		// Continue down the chain of child field names
		for (int i = 1; i < FieldNames.Num(); ++i)
		{
			PropertyHandle = PropertyHandle->GetChildHandle(FieldNames[i]);
		
			if (!PropertyHandle.IsValid())
			{
				return false;
			}

			if (const TSharedPtr<IPropertyHandleArray> ArrayHandle = PropertyHandle->AsArray())
			{
				// This property is an array, so get the first element if it exists
				uint32 NumElements;
				if (!ArrayHandle->GetNumElements(NumElements))
				{
					return false;
				}

				if (NumElements == 0)
				{
					if (!bAllowAdd)
					{
						// We aren't allowed to change the array, so we can't access this element
						return false;
					}

					// Automatically add an entry to the array so there's always at least one item to access
					if (ArrayHandle->AddItem() != FPropertyAccess::Success)
					{
						return false;
					}
				}

				// We should now have an item to access
				PropertyHandle = PropertyHandle->GetChildHandle(0);
		
				if (!PropertyHandle.IsValid())
				{
					return false;
				}
			}
		}

		return true;
	}
}

#endif