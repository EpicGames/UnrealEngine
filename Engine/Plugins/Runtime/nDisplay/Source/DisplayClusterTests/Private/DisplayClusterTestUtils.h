// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DisplayClusterConfigurationTypes_Viewport.h"

#if WITH_DEV_AUTOMATION_TESTS

#include "Kismet2/BlueprintEditorUtils.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"

class UDisplayClusterBlueprint;
class UDisplayClusterConfigurationCluster;
class UDisplayClusterConfigurationClusterNode;

/**
 * Utility functions used for display cluster tests.
 **/
namespace DisplayClusterTestUtils
{
	/**
	 * Create a display cluster asset using a new UDisplayClusterConfiguratorFactory.
	 * If the asset is invalid or not a UDisplayClusterBlueprint, the package will be immediately cleaned up.
	 */
	UDisplayClusterBlueprint* CreateDisplayClusterAsset();

	/**
	 * Add a cluster node to a cluster asset. This will automatically create the template node and discard it when
	 * finished.
	 */
	UDisplayClusterConfigurationClusterNode* AddClusterNodeToCluster(UBlueprint* Blueprint, UDisplayClusterConfigurationCluster* RootCluster, FString Name = "", bool bCallPostEditChange = true);

	/**
	 * Add a viewport to a cluster asset. This will automatically create the template viewport and discard it when
	 * finished.
	 */
	UDisplayClusterConfigurationViewport* AddViewportToClusterNode(UBlueprint* Blueprint, UDisplayClusterConfigurationClusterNode* Node, FString Name = "", bool bCallPostEditChange = true);

	/**
	 * Create a world and set it as the current world.
	 */
	UWorld* CreateWorld();
	
	/**
	 * Clean up a package that was created during a test.
	 */
	void CleanUpPackage(UPackage* Package);

	/**
	 * Clean up an asset that was created during a test.
	 */
	void CleanUpAsset(UObject* Asset);

	/**
	 * Clean up an asset that was created during a test as well as its containing package (if the package exists).
	 */
	void CleanUpAssetAndPackage(UObject* Asset);

	/**
	 * Clean up a world that was created during a test.
	 */
	void CleanUpWorld(UWorld* World);

	/**
	 * Get a property view based on a list of nested field names.
	 *
	 * @param Owner The root owner of the field.
	 * @param FieldNames A list of field names, where the first name is the top-most field, and subsequent names are nested fields (e.g. a field within a struct).
	 * @param bAllowAdd If true and one of the fields encountered is a container, an element will be added. If false, this will return false in the same situation.
	 * @param OutPropertyView The property will be stored here.
	 * @param OutPropertyHandle The property handle will be stored here.
	 * 
	 * @return Whether the property view and handle were successfully found. 
	 */
	bool GetPropertyViewAndHandleFromFieldNames(UObject* Owner, const TArray<FName>& FieldNames, bool bAllowAdd, TSharedPtr<ISinglePropertyView>& OutPropertyView, TSharedPtr<IPropertyHandle>& OutPropertyHandle);

	/**
	 * Set the value of a property handle.
	 */
	template <typename T>
	FPropertyAccess::Result SetPropertyHandleValue(TSharedPtr<IPropertyHandle> Handle, const T& Value);

	/**
	 * Get the value of a property handle.
	 */
	template <typename T>
	FPropertyAccess::Result GetPropertyHandleValue(TSharedPtr<IPropertyHandle> Handle, T& OutValue);

	/**
	 * Set a property (or nested property) of an object using a PropertyView and notify its root blueprint of any
	 * changes as if the change happened through the Blueprint editor.
	 *
	 * @param Owner The root owner of the field.
	 * @param Blueprint The Blueprint asset to notify after changing the value (optional).
	 * @param FieldNames A list of field names, where the first name is the top-most field, and subsequent names are nested fields (e.g. a field within a struct).
	 * @param Value The value to set the field to.
	 *
	 * @return Whether the field was successfully found and set.
	 */
	template <typename T>
	bool SetBlueprintPropertyValue(UObject* Owner, UBlueprint* Blueprint, const TArray<FName>& FieldNames, const T& Value);
	
	/**
	 * Get the value of a property (or nested property) of an object using a PropertyView.
	 *
	 * @param Owner The root owner of the field.
	 * @param FieldNames A list of field names, where the first name is the top-most field, and subsequent names are nested fields (e.g. a field within a struct).
	 * @param OutValue The value of the property will be stored here.
	 *
	 * @return Whether the value was successfully retrieved.
	 */
	template <typename T>
	bool GetBlueprintPropertyValue(UObject* Owner, const TArray<FName>& FieldNames, T& OutValue);
}

template <typename T>
FPropertyAccess::Result DisplayClusterTestUtils::SetPropertyHandleValue(TSharedPtr<IPropertyHandle> Handle, const T& Value)
{
	return Handle->SetValue(Value);
}

template <typename T>
FPropertyAccess::Result DisplayClusterTestUtils::GetPropertyHandleValue(TSharedPtr<IPropertyHandle> Handle, T& OutValue)
{
	return Handle->GetValue(OutValue);
}

// Get/set specializations for colors since they have to be set/retrieved as strings rather than directly
template <>
inline FPropertyAccess::Result DisplayClusterTestUtils::SetPropertyHandleValue(TSharedPtr<IPropertyHandle> Handle, const FLinearColor& Value)
{
	return Handle->SetValueFromFormattedString(Value.ToString());
}
	
template <>
inline FPropertyAccess::Result DisplayClusterTestUtils::GetPropertyHandleValue(TSharedPtr<IPropertyHandle> Handle, FLinearColor& Value)
{
	FString StringData;
	const FPropertyAccess::Result Result = Handle->GetValueAsFormattedString(StringData);
	if (Result == FPropertyAccess::Success)
	{
		Value.InitFromString(StringData);
	}
	else
	{
		Value = FLinearColor();
	}

	return Result;
}

template <typename T>
bool DisplayClusterTestUtils::SetBlueprintPropertyValue(UObject* Owner, UBlueprint* Blueprint, const TArray<FName>& FieldNames, const T& OutValue)
{
	TSharedPtr<ISinglePropertyView> PropertyView;
	TSharedPtr<IPropertyHandle> PropertyHandle;
	if (!GetPropertyViewAndHandleFromFieldNames(Owner, FieldNames, true, PropertyView, PropertyHandle))
	{
		return false;
	}

	const FPropertyAccess::Result Result = SetPropertyHandleValue<T>(PropertyHandle, OutValue);
	
	if (Result != FPropertyAccess::Success)
	{
		return false;
	}

	if (Blueprint)
	{
		// Trigger Blueprint updates as if we were in an editor. This will re-run construction scripts
		FBlueprintEditorUtils::PostEditChangeBlueprintActors(Blueprint);
	}

	return true;
}

template <typename T>
bool DisplayClusterTestUtils::GetBlueprintPropertyValue(UObject* Owner, const TArray<FName>& FieldNames, T& OutValue)
{
	TSharedPtr<ISinglePropertyView> PropertyView;
	TSharedPtr<IPropertyHandle> PropertyHandle;
	if (!GetPropertyViewAndHandleFromFieldNames(Owner, FieldNames, false, PropertyView, PropertyHandle))
	{
		return false;
	}

	return GetPropertyHandleValue<T>(PropertyHandle, OutValue) == FPropertyAccess::Success;
}

#endif