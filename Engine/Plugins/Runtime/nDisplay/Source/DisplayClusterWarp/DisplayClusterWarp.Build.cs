// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DisplayClusterWarp : ModuleRules
{
	public DisplayClusterWarp(ReadOnlyTargetRules ROTargetRules) : base(ROTargetRules)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"CinematicCamera",
				"Core",
				"CoreUObject",
				"DisplayCluster",
				"Engine",
				"Projects"
			});

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"DisplayCluster",
				"ProceduralMeshComponent",
				"RenderCore",
				"RHI",
				"Renderer",
				"ScalableMPCDI"
			});

		if (Target.bBuildEditor == true)
		{
			PrivateDependencyModuleNames.Add("UnrealEd");
		}
	}
}
