// Copyright Epic Games, Inc. All Rights Reserved.

#include "Blueprints/DisplayClusterWarpGeometry.h"

//------------------------------------------------------------------------------
// FDisplayClusterWarpGeometryOBJ
//------------------------------------------------------------------------------
void FDisplayClusterWarpGeometryOBJ::PostAddFace(int32 f0, int32 f1, int32 f2)
{
	Triangles.Add(f0);
	Triangles.Add(f1);
	Triangles.Add(f2);

	//Update normal
	const FVector FaceDir1 = Vertices[f1] - Vertices[f0];
	const FVector FaceDir2 = Vertices[f2] - Vertices[f0];

	const FVector FaceNornal = FVector::CrossProduct(FaceDir1, FaceDir2);

	Normal[f0] = FaceNornal;
	Normal[f1] = FaceNornal;
	Normal[f2] = FaceNornal;
}
