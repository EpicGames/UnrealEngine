// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DisplayClusterInFrustumFitCameraComponent.h"
#include "Components/BillboardComponent.h"

#include "CineCameraActor.h"
#include "Camera/CameraComponent.h"
#include "CineCameraComponent.h"

#include "DisplayClusterWarpLog.h"
#include "PDisplayClusterWarpStrings.h"

#include "IDisplayCluster.h"
#include "IDisplayClusterWarpBlend.h"

#include "Render/IDisplayClusterRenderManager.h"

#include "Render/Viewport/IDisplayClusterViewportManager.h"
#include "Render/Viewport/IDisplayClusterViewport.h"
#include "Render/Projection/IDisplayClusterProjectionPolicy.h"
#include "Render/Warp/IDisplayClusterWarpPolicyFactory.h"
#include "Render/Warp/IDisplayClusterWarpPolicy.h"

#include "Materials/Material.h"
#include "UObject/ConstructorHelpers.h"

#include "DisplayClusterRootActor.h"

namespace UE::DisplayClusterWarp::ViewPointComponent
{
	static inline TSharedPtr<IDisplayClusterWarpPolicy, ESPMode::ThreadSafe> ImplCreateWarpPolicy(const FString& InWarpPolicyType, const FString& InWarpPolicyName)
	{
		static IDisplayCluster& DisplayClusterSingleton = IDisplayCluster::Get();
		if (IDisplayClusterRenderManager* const DCRenderManager = DisplayClusterSingleton.GetRenderMgr())
		{
			TSharedPtr<IDisplayClusterWarpPolicyFactory> WarpPolicyFactory = DCRenderManager->GetWarpPolicyFactory(InWarpPolicyType);
			if (WarpPolicyFactory.IsValid())
			{
				return WarpPolicyFactory->Create(InWarpPolicyType, InWarpPolicyName);
			}
		}

		return nullptr;
	}
};

//--------------------------------------------------------------------------------
// UDisplayClusterInFrustumFitCameraComponent
//--------------------------------------------------------------------------------
UDisplayClusterInFrustumFitCameraComponent::UDisplayClusterInFrustumFitCameraComponent(const FObjectInitializer& ObjectInitializer)
	: UDisplayClusterCameraComponent(ObjectInitializer)
{
	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.TickGroup = TG_PostUpdateWork;
	PrimaryComponentTick.bStartWithTickEnabled = true;
	bAutoActivate = true;
}

TObjectPtr<UMaterial> UDisplayClusterInFrustumFitCameraComponent::GetDisplayDeviceMaterial(const EDisplayClusterDisplayDeviceMeshType InMeshType, const EDisplayClusterDisplayDeviceMaterialType InMaterialType) const
{
	// Uses its own material to display additional deformed preview meshes in front of the camera.
	if (IsEnabled() && WarpPolicy.IsValid())
	{
		// Special preview material is used for editable meshes: they should fly in front of the camera and deform according to its frustum.
		if (InMeshType == EDisplayClusterDisplayDeviceMeshType::PreviewEditableMesh)
		{
			static TObjectPtr<UMaterial> InFrustumFitMaterial = LoadObject<UMaterial>(nullptr, UE::DisplayClusterWarpStrings::InFrustumFit::material::Name, nullptr, LOAD_None, nullptr);
			if (InFrustumFitMaterial && InMeshType == EDisplayClusterDisplayDeviceMeshType::PreviewEditableMesh)
			{
				switch (InMaterialType)
				{
				case EDisplayClusterDisplayDeviceMaterialType::PreviewMeshMaterial:
				case EDisplayClusterDisplayDeviceMaterialType::PreviewMeshTechvisMaterial:
					// Note: Add additional techvis material for 'InFrustumFitCamera' if needed.
					return InFrustumFitMaterial.Get();

				default:
					break;
				}
			}
		}
	}

	return nullptr;
}

void UDisplayClusterInFrustumFitCameraComponent::OnUpdateDisplayDeviceMeshAndMaterialInstance(IDisplayClusterViewportPreview& InViewportPreview, const EDisplayClusterDisplayDeviceMeshType InMeshType, const EDisplayClusterDisplayDeviceMaterialType InMaterialType, UMeshComponent* InMeshComponent, UMaterialInstanceDynamic* InMeshMaterialInstance) const
{
	if (IsEnabled() && WarpPolicy.IsValid())
	{
		// The preview material used for editable meshes requires a set of unique parameters that are set from the warp policy.
		if (InMeshComponent && InMeshMaterialInstance && InMeshType == EDisplayClusterDisplayDeviceMeshType::PreviewEditableMesh)
		{
			switch (InMaterialType)
			{
			case EDisplayClusterDisplayDeviceMaterialType::PreviewMeshMaterial:
			case EDisplayClusterDisplayDeviceMaterialType::PreviewMeshTechvisMaterial:
				WarpPolicy->OnUpdateDisplayDeviceMeshAndMaterialInstance(InViewportPreview, InMeshType, InMaterialType, InMeshComponent, InMeshMaterialInstance);
				break;

			default:
				break;
			}
		}
	}
}

const UDisplayClusterInFrustumFitCameraComponent& UDisplayClusterInFrustumFitCameraComponent::GetConfigurationInFrustumFitCameraComponent(IDisplayClusterViewportConfiguration& InViewportConfiguration) const
{
	if (ADisplayClusterRootActor* ConfigurationRootActor = InViewportConfiguration.GetRootActor(EDisplayClusterRootActorType::Configuration))
	{
		if (ConfigurationRootActor != GetOwner())
		{
			if (UDisplayClusterInFrustumFitCameraComponent* ConfigurationCameraComponent = ConfigurationRootActor->GetComponentByName<UDisplayClusterInFrustumFitCameraComponent>(GetName()))
			{
				return *ConfigurationCameraComponent;
			}
		}
	}

	return *this;
}

bool UDisplayClusterInFrustumFitCameraComponent::IsEnabled() const
{
	return bEnableCameraProjection;
}

bool UDisplayClusterInFrustumFitCameraComponent::IsICVFXCameraBeingUsed() const
{
	// When using InFrustum projection, ignore the camera component from the parent class
	if (IsEnabled())
	{
		return false;
	}

	return Super::IsICVFXCameraBeingUsed();
}

bool UDisplayClusterInFrustumFitCameraComponent::IsViewPointOverrideCameraPosition() const
{
	// If InFrustumFit uses an external camera, use it as a ViewPoint.
	if (IsEnabled())
	{
		return false;
	}

	return Super::IsViewPointOverrideCameraPosition();
}


void UDisplayClusterInFrustumFitCameraComponent::GetEyePosition(const IDisplayClusterViewportConfiguration& InViewportConfiguration, FVector& OutViewLocation, FRotator& OutViewRotation)
{
	Super::GetEyePosition(InViewportConfiguration, OutViewLocation, OutViewRotation);

	// The observer's eye is located inside the InFrustumFit component.
	if (IsEnabled())
	{
		OutViewLocation = GetComponentLocation();
		OutViewRotation = GetComponentRotation();
	}
}

void UDisplayClusterInFrustumFitCameraComponent::TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	// Tick warp policy instance
	if (WarpPolicy.IsValid())
	{
		if (ADisplayClusterRootActor* ParentRootActor = Cast<ADisplayClusterRootActor>(GetOwner()))
		{
			if (IDisplayClusterViewportManager* ViewportManager = ParentRootActor->GetViewportManager())
			{
				WarpPolicy->Tick(ViewportManager, DeltaTime);
			}
		}
	}
}

bool UDisplayClusterInFrustumFitCameraComponent::ShouldUseEntireClusterViewports(IDisplayClusterViewportManager* InViewportManager) const
{
	 const UDisplayClusterInFrustumFitCameraComponent& ConfigurationCameraComponent = InViewportManager ? GetConfigurationInFrustumFitCameraComponent(InViewportManager->GetConfiguration()) : *this;

	// Only when this component is enabled should viewports be created for the entire cluster that accesses this component.
	return ConfigurationCameraComponent.IsEnabled();
}

IDisplayClusterWarpPolicy* UDisplayClusterInFrustumFitCameraComponent::GetWarpPolicy(IDisplayClusterViewportManager* InViewportManager)
{
	using namespace UE::DisplayClusterWarp::ViewPointComponent;

	// We can ask for different types of warp policies, depending on the rules of the user settings
	const FString NewWaprPolicyType = UE::DisplayClusterWarpStrings::warp::InFrustumFit;

	// when returns different type, this will recreate warp policy instance
	if (WarpPolicy.IsValid() && WarpPolicy->GetType() != NewWaprPolicyType)
	{
		WarpPolicy.Reset();
	}

	if (!WarpPolicy.IsValid())
	{
		WarpPolicy = ImplCreateWarpPolicy(NewWaprPolicyType, GetName());
	}

	return WarpPolicy.Get();
}

void UDisplayClusterInFrustumFitCameraComponent::OnRegister()
{
	Super::OnRegister();


#if WITH_EDITOR
	if (GIsEditor && !IsRunningCommandlet())
	{
		if (SpriteComponent)
		{
			SpriteComponent->SpriteInfo.Category = TEXT("NDisplayCameraViewOrigin");
			SpriteComponent->SpriteInfo.DisplayName = NSLOCTEXT("DisplayClusterInFrustumFitCameraComponent", "DisplayClusterInFrustumFitCameraComponentSpriteInfo", "nDisplay InFrustumFit View Point");
		}
	}

	RefreshVisualRepresentation();
#endif
}

#if WITH_EDITOR
bool UDisplayClusterInFrustumFitCameraComponent::CanEditChange(const FProperty* InProperty) const
{
	// If other logic prevents editing, we want to respect that
	bool bIsEditable = Super::CanEditChange(InProperty);
	if (bIsEditable && InProperty)
	{
		// InFrustum projection requires external cinecamera
		const FName PropertyName = InProperty->GetFName();
		if (PropertyName == GET_MEMBER_NAME_CHECKED(UDisplayClusterCameraComponent, ICVFXCameraComponentName))
		{
			return !IsEnabled();
		}
	}

	return bIsEditable;
}

void UDisplayClusterInFrustumFitCameraComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = (PropertyChangedEvent.Property != nullptr) ? PropertyChangedEvent.Property->GetFName() : NAME_None;

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UDisplayClusterCameraComponent, TargetCameraType)
	|| PropertyName == GET_MEMBER_NAME_CHECKED(UDisplayClusterInFrustumFitCameraComponent, bEnableCameraProjection))
	{
		if (IsEnabled() && TargetCameraType == EDisplayClusterTargetCameraType::ICVFXCameraComponent)
		{
			// When using projection, the internal target camera type cannot be used.
			// We have to switch to the external target camera type.
			TargetCameraType = EDisplayClusterTargetCameraType::None;
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif
