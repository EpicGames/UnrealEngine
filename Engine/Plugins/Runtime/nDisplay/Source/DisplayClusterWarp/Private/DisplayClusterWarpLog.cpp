// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterWarpLog.h"


DEFINE_LOG_CATEGORY(LogDisplayClusterWarp);
DEFINE_LOG_CATEGORY(LogDisplayClusterWarpBlend);
DEFINE_LOG_CATEGORY(LogDisplayClusterWarpInFrustumFitPolicy);
