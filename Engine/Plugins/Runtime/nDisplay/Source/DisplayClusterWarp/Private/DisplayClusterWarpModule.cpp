// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterWarpModule.h"

#include "DisplayClusterWarpLog.h"
#include "DisplayClusterWarpStrings.h"

#include "IDisplayCluster.h"
#include "Render/IDisplayClusterRenderManager.h"
#include "Policy/InFrustumFit/DisplayClusterWarpInFrustumFitPolicyFactory.h"

//////////////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterWarpModule
//////////////////////////////////////////////////////////////////////////////////////////////
FDisplayClusterWarpModule::FDisplayClusterWarpModule()
{
	TSharedPtr<IDisplayClusterWarpPolicyFactory> Factory;

	// InFrustumFit warp policy
	Factory = MakeShared<FDisplayClusterWarpInFrustumFitPolicyFactory>();
	WarpPolicyFactories.Emplace(UE::DisplayClusterWarpStrings::warp::InFrustumFit, Factory);

	UE_LOGF(LogDisplayClusterWarp, Log, "Warp module has been instantiated");
}

FDisplayClusterWarpModule::~FDisplayClusterWarpModule()
{
	UE_LOGF(LogDisplayClusterWarp, Log, "Warp module has been destroyed");
}

void FDisplayClusterWarpModule::StartupModule()
{
	UE_LOGF(LogDisplayClusterWarp, Log, "Warp module startup");

	IDisplayClusterRenderManager* RenderMgr = IDisplayCluster::Get().GetRenderMgr();
	if (RenderMgr)
	{
		for (auto it = WarpPolicyFactories.CreateIterator(); it; ++it)
		{
			UE_LOGF(LogDisplayClusterWarp, Log, "Registering <%ls> warp policy factory...", *it->Key);

			if (!RenderMgr->RegisterWarpPolicyFactory(it->Key, it->Value))
			{
				UE_LOGF(LogDisplayClusterWarp, Warning, "Couldn't register <%ls> warp policy factory", *it->Key);
			}
		}
	}

	UE_LOGF(LogDisplayClusterWarp, Log, "Warp module has started");
}

void FDisplayClusterWarpModule::ShutdownModule()
{
	UE_LOGF(LogDisplayClusterWarp, Log, "Warp module shutdown");

	IDisplayClusterRenderManager* RenderMgr = IDisplayCluster::Get().GetRenderMgr();
	if (RenderMgr)
	{
		for (auto it = WarpPolicyFactories.CreateConstIterator(); it; ++it)
		{
			UE_LOGF(LogDisplayClusterWarp, Log, "Un-registering <%ls> warp factory...", *it->Key);

			if (!RenderMgr->UnregisterWarpPolicyFactory(it->Key))
			{
				UE_LOGF(LogDisplayClusterWarp, Warning, "An error occurred during un-registering the <%ls> warp factory", *it->Key);
			}
		}
	}

	UE_LOGF(LogDisplayClusterWarp, Log, "Warp module has been shutdown.");
}

IMPLEMENT_MODULE(FDisplayClusterWarpModule, DisplayClusterWarp);
