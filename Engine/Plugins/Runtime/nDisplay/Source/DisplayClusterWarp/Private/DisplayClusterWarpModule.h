// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDisplayClusterWarp.h"
#include "Render/Warp/IDisplayClusterWarpPolicyFactory.h"
#include "WarpBlend/Loader/DisplayClusterWarpBlendLoader_MPCDI.h"
#include "WarpBlend/Loader/DisplayClusterWarpBlendLoader_MeshComponent.h"

/**
 * Implement DisplayClusterWarp module
 */
class FDisplayClusterWarpModule
	: public IDisplayClusterWarp
{
public:
	FDisplayClusterWarpModule();
	virtual ~FDisplayClusterWarpModule();

public:
	//////////////////////////////////////////////////////////////////////////////////////////////
	// IModuleInterface
	//////////////////////////////////////////////////////////////////////////////////////////////
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

public:
	virtual TSharedPtr<IDisplayClusterWarpBlend, ESPMode::ThreadSafe> Create(const FDisplayClusterWarpInitializer_MPCDIFile& InConstructParameters) const override
	{
		return FDisplayClusterWarpBlendLoader_MPCDI::Create(InConstructParameters);
	}

	virtual TSharedPtr<IDisplayClusterWarpBlend, ESPMode::ThreadSafe> Create(const FDisplayClusterWarpInitializer_MPCDIFile_Profile2DScreen& InConstructParameters) const override
	{
		return FDisplayClusterWarpBlendLoader_MPCDI::Create(InConstructParameters);
	}

	virtual TSharedPtr<IDisplayClusterWarpBlend, ESPMode::ThreadSafe> Create(const FDisplayClusterWarpInitializer_PFMFile& InConstructParameters) const override
	{
		return FDisplayClusterWarpBlendLoader_MPCDI::Create(InConstructParameters);
	}

	virtual TSharedPtr<IDisplayClusterWarpBlend, ESPMode::ThreadSafe> Create(const FDisplayClusterWarpInitializer_StaticMesh& InConstructParameters) const override
	{
		return FDisplayClusterWarpBlendLoader_MeshComponent::Create(InConstructParameters);
	}

	virtual TSharedPtr<IDisplayClusterWarpBlend, ESPMode::ThreadSafe> Create(const FDisplayClusterWarpInitializer_ProceduralMesh& InConstructParameters) const override
	{
		return FDisplayClusterWarpBlendLoader_MeshComponent::Create(InConstructParameters);
	}

	virtual bool ReadMPCDFileStructure(const FString& InMPCDIFileName, TMap<FString, TMap<FString, FDisplayClusterWarpMPCDIAttributes>>& OutMPCDIFileStructure) const override
	{
		return FDisplayClusterWarpBlendLoader_MPCDI::ReadMPCDFileStructure(InMPCDIFileName, OutMPCDIFileStructure);
	}

private:
	// Available factories
	TMap<FString, TSharedPtr<IDisplayClusterWarpPolicyFactory>> WarpPolicyFactories;
};
