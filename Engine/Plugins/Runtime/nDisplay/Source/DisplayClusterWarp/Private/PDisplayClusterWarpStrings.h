// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DisplayClusterWarpStrings.h"

namespace UE::DisplayClusterWarpStrings
{
	namespace InFrustumFit
	{
		namespace material
		{
			static constexpr const TCHAR* Name = TEXT("/nDisplay/Materials/Preview/M_InFrustumFitPreview");

			namespace attr
			{
				static constexpr const TCHAR* GlobalScale = TEXT("GlobalScale");
				static constexpr const TCHAR* GlobalForward = TEXT("GlobalForward");
				static constexpr const TCHAR* GlobalRight = TEXT("GlobalRight");
				static constexpr const TCHAR* GlobalUp = TEXT("GlobalUp");
			}
		}
	}
};
