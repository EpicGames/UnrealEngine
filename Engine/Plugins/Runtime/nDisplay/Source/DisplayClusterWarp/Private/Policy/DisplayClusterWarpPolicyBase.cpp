// Copyright Epic Games, Inc. All Rights Reserved.

#include "Policy/DisplayClusterWarpPolicyBase.h"
#include "Render/Viewport/IDisplayClusterViewport.h"
#include "Render/Projection/IDisplayClusterProjectionPolicy.h"

#include "Components/DisplayClusterCameraComponent.h"

#include "Containers/DisplayClusterWarpEye.h"

#include "DisplayClusterWarpLog.h"

//////////////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterWarpPolicyBase
//////////////////////////////////////////////////////////////////////////////////////////////
FDisplayClusterWarpPolicyBase::FDisplayClusterWarpPolicyBase(const FString& InType, const FString& InWarpPolicyName)
	: PolicyInstanceId(FString::Printf(TEXT("WarpPolicy_%s_%s"), *InType, *InWarpPolicyName))
{ }
