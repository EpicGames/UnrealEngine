// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Render/Warp/IDisplayClusterWarpPolicy.h"
#include "Containers/DisplayClusterWarpEye.h"
#include "Templates/SharedPointer.h"

class IDisplayClusterWarpBlend;

/**
 * Base warp policy
 */
class FDisplayClusterWarpPolicyBase
	: public IDisplayClusterWarpPolicy
	, public TSharedFromThis<FDisplayClusterWarpPolicyBase, ESPMode::ThreadSafe>
{
public:
	FDisplayClusterWarpPolicyBase(const FString& InType, const FString& InWarpPolicyName);

	virtual TSharedPtr<IDisplayClusterWarpPolicy, ESPMode::ThreadSafe> ToSharedPtr() override
	{
		return AsShared();
	}

	virtual TSharedPtr<const IDisplayClusterWarpPolicy, ESPMode::ThreadSafe> ToSharedPtr() const override
	{
		return AsShared();
	}

public:
	virtual const FString& GetId() const override
	{
		return PolicyInstanceId;
	}

private:
	// unique name
	const FString PolicyInstanceId;
};
