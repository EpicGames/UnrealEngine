// Copyright Epic Games, Inc. All Rights Reserved.

#include "Policy/InFrustumFit/DisplayClusterWarpInFrustumFitPolicy.h"

#include "DisplayClusterRootActor.h"
#include "Containers/DisplayClusterWarpEye.h"

#include "Render/Viewport/IDisplayClusterViewportPreview.h"
#include "Render/Viewport/IDisplayClusterViewport.h"
#include "Render/Projection/IDisplayClusterProjectionPolicy.h"

#include "IDisplayClusterWarpBlend.h"
#include "PDisplayClusterWarpStrings.h"

#include "Components/DisplayClusterCameraComponent.h"
#include "Components/DisplayClusterInFrustumFitCameraComponent.h"
#include "Components/MeshComponent.h"
#include "Materials/MaterialInstanceDynamic.h"

int32 GDisplayClusterWarpInFrustumFitPolicyDrawFrustum = 0;
static FAutoConsoleVariableRef CVarDisplayClusterWarpInFrustumFitPolicyDrawFrustum(
	TEXT("nDisplay.warp.InFrustumFit.DrawFrustum"),
	GDisplayClusterWarpInFrustumFitPolicyDrawFrustum,
	TEXT("Toggles drawing the stage geometry frustum and bounding box\n"),
	ECVF_Default
);

//-------------------------------------------------------------------
// FDisplayClusterWarpInFrustumFitPolicy
//-------------------------------------------------------------------
FDisplayClusterWarpInFrustumFitPolicy::FDisplayClusterWarpInFrustumFitPolicy(const FString& InWarpPolicyName)
	: FDisplayClusterWarpPolicyBase(GetType(), InWarpPolicyName)
{ }

const FString& FDisplayClusterWarpInFrustumFitPolicy::GetType() const
{
	static const FString Type(UE::DisplayClusterWarpStrings::warp::InFrustumFit);

	return Type;
}

void FDisplayClusterWarpInFrustumFitPolicy::HandleNewFrame(const TArray<TSharedPtr<IDisplayClusterViewport, ESPMode::ThreadSafe>>& InViewports)
{
	// Reset all special data that was used in the prev frame.
	// In the new frame, we have to find a new solution because the viewer's position or geometry may have changed.
	// The number of viewports can also be changed at runtime. This changes the shape of the united geometry.
	{
		// Reset united geometry AABB
		OptUnitedGeometryWorldAABB.Reset();

		// Reset the solution from the previous frame.
		OptUnitedGeometryWarpProjection.Reset();
		OptOverrideWorldViewTarget.Reset();
	}

	if (InViewports.IsEmpty() || !InViewports[0].IsValid())
	{
		return;
	}

	UDisplayClusterInFrustumFitCameraComponent* ConfigurationCameraComponent = Cast<UDisplayClusterInFrustumFitCameraComponent>(InViewports[0]->GetViewPointCameraComponent(EDisplayClusterRootActorType::Configuration));
	UDisplayClusterInFrustumFitCameraComponent* SceneCameraComponent = Cast<UDisplayClusterInFrustumFitCameraComponent>(InViewports[0]->GetViewPointCameraComponent(EDisplayClusterRootActorType::Scene));
	if(!ConfigurationCameraComponent || !SceneCameraComponent)
	{
		return;
	}

	// calculate GroupFrustum for a single context
	const uint32 ContextNum = 0;

	const float WorldToMeters = InViewports[0]->GetConfiguration().GetWorldToMeters();
	const float WorldScale = WorldToMeters / 100.f;

	switch (ConfigurationCameraComponent->CameraViewTarget)
	{
	case EDisplayClusterWarpCameraViewTarget::GeometricCenter:
	{
		// In this case, we need to find a symmetric frustum
		// 
		// The united geometry frustum is built from geometric points projected onto a special plane.
		// This plane is called the 'projection plane' and is created from two quantities: the view direction vector and the eye position.
		// Therefore, when we change the view direction vector, it leads to a change in the "projection plane" and, then, to new frustum values.
		// When we set out to create a symmetrical frustum, we need to solve this math problem.
		// An easy way is to do this in a few iterations and stop when we find a suitable view direction that provides a nearly symmetrical frustum with acceptable precision.

		// Set the center of the AABB of united geometry as the view target:
		{
			// Calculate AABB for group of viewports:
			FDisplayClusterWarpAABB UnitedGeometryWorldAABBox;
			if (!CalcUnitedGeometryWorldAABBox(InViewports, WorldScale, UnitedGeometryWorldAABBox))
			{
				return;
			}
			OptUnitedGeometryWorldAABB = UnitedGeometryWorldAABBox;

			// in the first iteration, use the center of the united AABB geometry as the view target.
			OptOverrideWorldViewTarget = UnitedGeometryWorldAABBox.GetCenter();
		}

		// Iterate frustum to a nearly symmetrical frustum with acceptable precision:
		{
			FSymmetricFrustumData FrustumData(WorldScale, SceneCameraComponent->GetComponentTransform());
			while (true)
			{
				if (CalcUnitedGeometrySymmetricFrustum(InViewports, ContextNum, FrustumData))
				{
					// A symmetric frustum has been found, use it to fit
					OptUnitedGeometryWarpProjection = *FrustumData.NewUnitedSymmetricWarpProjection;
					break;
				}

				// If the maximum iteration number has been reached, or new frustum can't be calculated.
				if (FrustumData.IterationNum == INDEX_NONE
				|| !FrustumData.NewWorldViewTarget.IsSet())
				{	
					// Use the best saved symmetric projection over all iterations.
					if (FrustumData.BestSymmetricWarpProjection.IsSet())
					{
						check(FrustumData.BestWorldViewTarget.IsSet());

						OptUnitedGeometryWarpProjection = FrustumData.BestSymmetricWarpProjection->Value;
						OptOverrideWorldViewTarget = *FrustumData.BestWorldViewTarget;
					}

					break;
				}

				// Set the new location of the view target and perform the next iteration.
				OptOverrideWorldViewTarget = *FrustumData.NewWorldViewTarget;
			}
		}

		break;
	}

	case EDisplayClusterWarpCameraViewTarget::MatchViewOrigin:
	{
		// In this case, the viewing direction is the X-axis of the ViewPoint component.
		// This value is obtained in the FDisplayClusterWarpInFrustumFitPolicy::BeginCalcFrustum() function.
		FDisplayClusterWarpProjection UnitedWarpProjection;
		if (CalcUnitedGeometryFrustum(InViewports, ContextNum, WorldScale, UnitedWarpProjection))
		{
			// If the view target is set to a fixed value instead of being computed by the group AABB, we do not want to alter the view direction,
			// but make the frustum symmetric around that fixed direction. This involves expanding the asymmetric frustum so that its left and right,
			// top and bottom angles are equal

			const double MaxHorizontal = FMath::Max(FMath::Abs(UnitedWarpProjection.Left), FMath::Abs(UnitedWarpProjection.Right));
			const double MaxVertical = FMath::Max(FMath::Abs(UnitedWarpProjection.Top), FMath::Abs(UnitedWarpProjection.Bottom));

			FDisplayClusterWarpProjection UnitedWarpSymmetricProjection = UnitedWarpProjection;

			UnitedWarpSymmetricProjection.Left = -MaxHorizontal;
			UnitedWarpSymmetricProjection.Right = MaxHorizontal;

			UnitedWarpSymmetricProjection.Bottom = -MaxVertical;
			UnitedWarpSymmetricProjection.Top = MaxVertical;

			// use this united  frustum for fit:
			OptUnitedGeometryWarpProjection = UnitedWarpSymmetricProjection;
		}
		break;
	}

	default:
		break;
	}
}

void FDisplayClusterWarpInFrustumFitPolicy::Tick(IDisplayClusterViewportManager* InViewportManager, float DeltaSeconds)
{
#if WITH_EDITOR
	if (GDisplayClusterWarpInFrustumFitPolicyDrawFrustum)
	{
		if (TArray<TSharedPtr<IDisplayClusterViewport, ESPMode::ThreadSafe>> Viewports = InViewportManager->GetEntireClusterViewportsForWarpPolicy(SharedThis(this)); Viewports.Num() > 0)
		{
			// Getting data from the first viewport, since all viewports use the same ViewPoint component
			const TSharedPtr<IDisplayClusterViewport, ESPMode::ThreadSafe>& Viewport = Viewports[0];
			if (UDisplayClusterInFrustumFitCameraComponent* SceneCameraComponent = Cast<UDisplayClusterInFrustumFitCameraComponent>(Viewport->GetViewPointCameraComponent(EDisplayClusterRootActorType::Scene)))
			{
				if (ADisplayClusterRootActor* SceneRootActor = InViewportManager->GetConfiguration().GetRootActor(EDisplayClusterRootActorType::Scene))
				{
					if (SceneCameraComponent)
					{
						DrawDebugGroupFrustum(SceneRootActor, SceneCameraComponent, FColor::Blue);
					}

					DrawDebugGroupBoundingBox(SceneRootActor, FColor::Red);
				}
			}
		}
	}
#endif
}

bool FDisplayClusterWarpInFrustumFitPolicy::HasPreviewEditableMesh(IDisplayClusterViewport* InViewport)
{
	// This warp policy is based on IDisplayClusterWarpBlend only.
	// Process only viewports with a projection policy based on the warpblend interface.
	TSharedPtr<IDisplayClusterWarpBlend, ESPMode::ThreadSafe> WarpBlend;
	if (!InViewport || !InViewport->GetProjectionPolicy().IsValid() || !InViewport->GetProjectionPolicy()->GetWarpBlendInterface(WarpBlend))
	{
		return false;
	}

	// If the preview is not used in this configuration
	if (!InViewport->GetConfiguration().IsPreviewRendering() || InViewport->GetConfiguration().GetRootActor(EDisplayClusterRootActorType::Preview) == nullptr)
	{
		return false;
	}

	// If owner DCRA world is EditorPreview dont show editable mesh (Configurator, ICVFX Panel, etc)
	if (InViewport->GetConfiguration().IsRootActorWorldHasAnyType(EDisplayClusterRootActorType::Preview, EWorldType::EditorPreview))
	{
		return false;
	}

	// The editable mesh is an option for the UDisplayClusterInFrustumFitCameraComponent.
	if (UDisplayClusterInFrustumFitCameraComponent* ConfigurationCameraComponent = Cast<UDisplayClusterInFrustumFitCameraComponent>(InViewport->GetViewPointCameraComponent(EDisplayClusterRootActorType::Configuration)))
	{
		if (ConfigurationCameraComponent->bShowPreviewFrustumFit)
		{
			return true;
		}
	}

	return false;
}

void FDisplayClusterWarpInFrustumFitPolicy::OnUpdateDisplayDeviceMeshAndMaterialInstance(IDisplayClusterViewportPreview& InViewportPreview, const EDisplayClusterDisplayDeviceMeshType InMeshType, const EDisplayClusterDisplayDeviceMaterialType InMaterialType, UMeshComponent* InMeshComponent, UMaterialInstanceDynamic* InMeshMaterialInstance) const
{
	// The preview material used for editable meshes requires a set of unique parameters that are set from the warp policy.
	check(InMeshComponent && InMeshMaterialInstance);

	if (InMeshType != EDisplayClusterDisplayDeviceMeshType::PreviewEditableMesh)
	{
		// Only for editable mesh
		return;
	}

	// Process only viewports with a projection policy based on the warpblend interface.
	TSharedPtr<IDisplayClusterWarpBlend, ESPMode::ThreadSafe> WarpBlend;
	TSharedPtr<IDisplayClusterViewport, ESPMode::ThreadSafe> OwningViewport = InViewportPreview.GetOwningViewport();
	if (!OwningViewport .IsValid()|| !OwningViewport->GetProjectionPolicy().IsValid() || !OwningViewport->GetProjectionPolicy()->GetWarpBlendInterface(WarpBlend))
	{
		return;
	}

	// Not all projection policies support a editable mesh.
	const FDisplayClusterWarpData& WarpData = WarpBlend->GetWarpData(0);
	if (WarpData.bValid && WarpData.bHasWarpPolicyChanges)
	{
		const FTransform CameraTransform(WarpData.WarpProjection.CameraRotation.Quaternion(), WarpData.WarpProjection.CameraLocation);

		const float HScale = (WarpData.WarpProjection.Left - WarpData.WarpProjection.Right) / (WarpData.GeometryWarpProjection.Left - WarpData.GeometryWarpProjection.Right);
		const float VScale = (WarpData.WarpProjection.Top - WarpData.WarpProjection.Bottom) / (WarpData.GeometryWarpProjection.Top - WarpData.GeometryWarpProjection.Bottom);

		checkf(FMath::IsNearlyEqual(HScale, VScale), TEXT("Streching the stage geometry to fit a different aspect ratio is not supported!"));
		const FVector Scale(1, HScale, HScale);

		const FMatrix CameraBasis = FRotationMatrix::Make(WarpData.WarpProjection.CameraRotation).Inverse();

		// Compute the relative transform from the origin to the geometry
		FTransform RelativeTransform = FTransform(WarpData.WarpContext.MeshToStageMatrix * WarpData.Local2World.Inverse());
		RelativeTransform.ScaleTranslation(Scale);

		// Final transform is computed from the relative transform of the geometry to the view point, the frustum fit transform
		// which will scale and position the geometry based on the fitted frustum, and the camera transform
		const FTransform FinalTransform = RelativeTransform * CameraTransform;
		InMeshComponent->SetRelativeTransform(FinalTransform);

		// Since the mesh needs to be skewed to scale appropriately, and since Unreal Engine does not support a skew transform
		// through FTransform, the mesh needs to be skewed through the vertex shader using WorldPositionOffset,
		// so pass in the "global" scale to the preview mesh's material instance
		InMeshMaterialInstance->SetVectorParameterValue(UE::DisplayClusterWarpStrings::InFrustumFit::material::attr::GlobalScale, Scale);
		InMeshMaterialInstance->SetVectorParameterValue(UE::DisplayClusterWarpStrings::InFrustumFit::material::attr::GlobalForward, CameraBasis.GetUnitAxis(EAxis::X));
		InMeshMaterialInstance->SetVectorParameterValue(UE::DisplayClusterWarpStrings::InFrustumFit::material::attr::GlobalRight, CameraBasis.GetUnitAxis(EAxis::Y));
		InMeshMaterialInstance->SetVectorParameterValue(UE::DisplayClusterWarpStrings::InFrustumFit::material::attr::GlobalUp, CameraBasis.GetUnitAxis(EAxis::Z));
	}
}

#if WITH_EDITOR
#include "Components/LineBatchComponent.h"

void FDisplayClusterWarpInFrustumFitPolicy::DrawDebugGroupBoundingBox(ADisplayClusterRootActor* SceneRootActor, const FLinearColor& Color)
{
	// DCRA uses its own LineBatcher
	ULineBatchComponent* LineBatcher = SceneRootActor ? SceneRootActor->GetLineBatchComponent() : nullptr;
	UWorld* World = SceneRootActor ? SceneRootActor->GetWorld() : nullptr;
	if (LineBatcher && World && OptUnitedGeometryWorldAABB.IsSet())
	{
		const float Thickness = 1.f;
		const float PointSize = 5.f;
		const FBox WorldBox = *OptUnitedGeometryWorldAABB;

		LineBatcher->DrawBox(WorldBox.GetCenter(), WorldBox.GetExtent(), Color, 0, SDPG_World, Thickness);
		LineBatcher->DrawPoint(WorldBox.GetCenter(), Color, PointSize, SDPG_World);
	}
}

void FDisplayClusterWarpInFrustumFitPolicy::DrawDebugGroupFrustum(ADisplayClusterRootActor* SceneRootActor, UDisplayClusterInFrustumFitCameraComponent* CameraComponent, const FLinearColor& Color)
{
	if (!OptUnitedGeometryWarpProjection.IsSet())
	{
		// A united frastum is required.
		return;
	}

	// DCRA uses its own LineBatcher
	ULineBatchComponent* LineBatcher = SceneRootActor ? SceneRootActor->GetLineBatchComponent() : nullptr;
	if (LineBatcher && CameraComponent)
	{
		UWorld* World = SceneRootActor->GetWorld();
		IDisplayClusterViewportConfiguration* ViewportConfiguration = SceneRootActor->GetViewportConfiguration();
		if (ViewportConfiguration && World)
		{
			const float Thickness = 1.0f;

			// Get the configuration in use
			const UDisplayClusterInFrustumFitCameraComponent& ConfigurationCameraComponent = CameraComponent->GetConfigurationInFrustumFitCameraComponent(*ViewportConfiguration);

			const float NearPlane = 10;
			const float FarPlane = 1000;

		
			const FVector CameraLoc = CameraComponent->GetComponentLocation();
			FVector ViewDirection;
			{
				if (ConfigurationCameraComponent.CameraViewTarget == EDisplayClusterWarpCameraViewTarget::MatchViewOrigin)
				{
					ViewDirection = CameraComponent->GetComponentRotation().RotateVector(FVector::XAxisVector);
				}
				else if(OptUnitedGeometryWorldAABB.IsSet())
				{
					const FVector ViewTarget = OptOverrideWorldViewTarget.IsSet() ? *OptOverrideWorldViewTarget : OptUnitedGeometryWorldAABB->GetCenter();
					ViewDirection = (ViewTarget - CameraComponent->GetComponentLocation()).GetSafeNormal();
				}
				else
				{
					return;
				}
			}

			FDisplayClusterWarpProjection UnitedGeometryWarpProjection = *OptUnitedGeometryWarpProjection;

			const FRotator ViewRotator = ViewDirection.ToOrientationRotator();
			const FVector FrustumTopLeft     = ViewRotator.RotateVector(FVector(UnitedGeometryWarpProjection.ZNear, UnitedGeometryWarpProjection.Left,  UnitedGeometryWarpProjection.Top)    / UnitedGeometryWarpProjection.ZNear);
			const FVector FrustumTopRight    = ViewRotator.RotateVector(FVector(UnitedGeometryWarpProjection.ZNear, UnitedGeometryWarpProjection.Right, UnitedGeometryWarpProjection.Top)    / UnitedGeometryWarpProjection.ZNear);
			const FVector FrustumBottomLeft  = ViewRotator.RotateVector(FVector(UnitedGeometryWarpProjection.ZNear, UnitedGeometryWarpProjection.Left,  UnitedGeometryWarpProjection.Bottom) / UnitedGeometryWarpProjection.ZNear);
			const FVector FrustumBottomRight = ViewRotator.RotateVector(FVector(UnitedGeometryWarpProjection.ZNear, UnitedGeometryWarpProjection.Right, UnitedGeometryWarpProjection.Bottom) / UnitedGeometryWarpProjection.ZNear);

			const FVector FrustumVertices[8] =
			{
				CameraLoc + FrustumTopLeft * NearPlane,
				CameraLoc + FrustumTopRight * NearPlane,
				CameraLoc + FrustumBottomRight * NearPlane,
				CameraLoc + FrustumBottomLeft * NearPlane,

				CameraLoc + FrustumTopLeft * FarPlane,
				CameraLoc + FrustumTopRight * FarPlane,
				CameraLoc + FrustumBottomRight * FarPlane,
				CameraLoc + FrustumBottomLeft * FarPlane,
			};

			LineBatcher->DrawLine(CameraLoc, CameraLoc + ViewDirection * 50, Color, SDPG_World, Thickness, 0.f);

			// Near plane rectangle
			LineBatcher->DrawLine(FrustumVertices[0], FrustumVertices[1], Color, SDPG_World, Thickness, 0.f);
			LineBatcher->DrawLine(FrustumVertices[1], FrustumVertices[2], Color, SDPG_World, Thickness, 0.f);
			LineBatcher->DrawLine(FrustumVertices[2], FrustumVertices[3], Color, SDPG_World, Thickness, 0.f);
			LineBatcher->DrawLine(FrustumVertices[3], FrustumVertices[0], Color, SDPG_World, Thickness, 0.f);

			// Frustum
			LineBatcher->DrawLine(FrustumVertices[0], FrustumVertices[4], Color, SDPG_World, Thickness, 0.f);
			LineBatcher->DrawLine(FrustumVertices[1], FrustumVertices[5], Color, SDPG_World, Thickness, 0.f);
			LineBatcher->DrawLine(FrustumVertices[2], FrustumVertices[6], Color, SDPG_World, Thickness, 0.f);
			LineBatcher->DrawLine(FrustumVertices[3], FrustumVertices[7], Color, SDPG_World, Thickness, 0.f);

			// Far plane rectangle
			LineBatcher->DrawLine(FrustumVertices[4], FrustumVertices[5], Color, SDPG_World, Thickness, 0.f);
			LineBatcher->DrawLine(FrustumVertices[5], FrustumVertices[6], Color, SDPG_World, Thickness, 0.f);
			LineBatcher->DrawLine(FrustumVertices[6], FrustumVertices[7], Color, SDPG_World, Thickness, 0.f);
			LineBatcher->DrawLine(FrustumVertices[7], FrustumVertices[4], Color, SDPG_World, Thickness, 0.f);
		}
	}
}
#endif
