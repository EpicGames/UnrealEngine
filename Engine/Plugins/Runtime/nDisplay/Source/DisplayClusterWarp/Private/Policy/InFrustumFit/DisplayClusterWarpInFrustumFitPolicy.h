// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Policy/DisplayClusterWarpPolicyBase.h"
#include "Render/Viewport/IDisplayClusterViewportConfiguration.h"
#include "Blueprints/DisplayClusterWarpBlueprint_Enums.h"

#include "Containers/DisplayClusterWarpContext.h"

class ADisplayClusterRootActor;
class UDisplayClusterInFrustumFitCameraComponent;
class FDisplayClusterWarpEye;

/**
 * InFrustumFit warp policy
 */
class FDisplayClusterWarpInFrustumFitPolicy
	: public FDisplayClusterWarpPolicyBase
{
public:
	FDisplayClusterWarpInFrustumFitPolicy(const FString& InWarpPolicyName);

public:
	//~ Begin IDisplayClusterWarpPolicy
	virtual const FString& GetType() const override;
	virtual void HandleNewFrame(const TArray<TSharedPtr<IDisplayClusterViewport, ESPMode::ThreadSafe>>& InViewports) override;
	virtual void Tick(IDisplayClusterViewportManager* InViewportManager, float DeltaSeconds) override;

	virtual void BeginCalcFrustum(IDisplayClusterViewport* InViewport, const uint32 ContextNum) override;
	virtual void EndCalcFrustum(IDisplayClusterViewport* InViewport, const uint32 ContextNum) override;

	virtual bool HasPreviewEditableMesh(IDisplayClusterViewport* InViewport) override;

	virtual void OnUpdateDisplayDeviceMeshAndMaterialInstance(IDisplayClusterViewportPreview& InViewportPreview, const EDisplayClusterDisplayDeviceMeshType InMeshType, const EDisplayClusterDisplayDeviceMaterialType InMaterialType, UMeshComponent* InMeshComponent, UMaterialInstanceDynamic* InMeshMaterialInstance) const override;
	//~~ End IDisplayClusterWarpPolicy

private:
	/** Apply frustum fit to the specified warp projection */
	FDisplayClusterWarpProjection ApplyInFrustumFit(IDisplayClusterViewport* InViewport, const FTransform& World2OriginTransform, const FDisplayClusterWarpProjection& InWarpProjection) const;

	/** Find final projection scale.*/
	FVector2D FindFrustumFit(const EDisplayClusterWarpCameraProjectionMode InProjectionMode, const FVector2D& InCameraFOV, const FVector2D& InGeometryFOV) const;

	/** Return true, if this viewport context can be processed. */
	bool GetWarpBlendAPI(IDisplayClusterViewport* InViewport, const uint32 ContextNum, TSharedPtr<IDisplayClusterWarpBlend, ESPMode::ThreadSafe>& OutWarpBlend, TObjectPtr<UDisplayClusterInFrustumFitCameraComponent>& OutConfigurationCameraComponent) const;

	/** Calculate united AABB for group of viewports: */
	bool CalcUnitedGeometryWorldAABBox(const TArray<TSharedPtr<IDisplayClusterViewport, ESPMode::ThreadSafe>>& InViewports, const float WorldScale, FDisplayClusterWarpAABB& OutUnitedGeometryWorldAABBox) const;

	/** Calculate united geometry frustum for group of viewports
	* 
	* @param InViewports - viewports with warp geometry
	* @param InContextNum - context number (i.e., eye index)
	* @param WorldScale - world scale
	* @param OutUnitedGeometryWarpProjection (out) calculated
	*/
	bool CalcUnitedGeometryFrustum(const TArray<TSharedPtr<IDisplayClusterViewport, ESPMode::ThreadSafe>>& InViewports, const int32 InContextNum, const float WorldScale, FDisplayClusterWarpProjection& OutUnitedGeometryWarpProjection);

private:
	/** Auxiliary container for the CalcUnitedGeometrySymmetricFrustum() function. */
	struct FSymmetricFrustumData
	{
		FSymmetricFrustumData(const float InWorldScale, const FTransform& InCameraComponent2WorldTransform)
			: WorldScale(InWorldScale), CameraComponent2WorldTransform(InCameraComponent2WorldTransform)
		{ }

	public:
		// World scale value
		const float WorldScale;

		// Transform from the FrustumFit component space to the world space.
		const FTransform CameraComponent2WorldTransform;

		// Index of the current iteration. Used to stop the search when the maximum number of iterations is reached.
		// Note:
		//   The united geometry frustum is built from geometric points projected onto a special plane.
		//   This plane is called the 'projection plane' and is created from two quantities: the view direction vector and the eye position.
		//   Therefore, when we change the view direction vector, it leads to a change in the "projection plane" and, then, to new frustum values.
		//   When we set out to create a symmetrical frustum, we need to solve this math problem.
		//   An easy way is to do this in a few iterations and stop when we find a suitable view direction that provides a nearly symmetrical frustum with acceptable precision.
		int32 IterationNum = 0;

		// United symmetric WarpProjection
		TOptional<FDisplayClusterWarpProjection> NewUnitedSymmetricWarpProjection;

		// new location of the viewing target in the world space.
		TOptional<FVector> NewWorldViewTarget;

		// Store the best symmetric projection, which is used when the iterative method has not found a projection within the expected precision.
		TOptional<TPair<double, FDisplayClusterWarpProjection>> BestSymmetricWarpProjection;
		TOptional<FVector> BestWorldViewTarget;
	};

private:
	/** Calculate united symmetric frustum for group of viewports:
	*
	* @param InViewports - viewports with warp geometry
	* @param InContextNum - context number (i.e., eye index)
	* @param InOutSymmetricFrustumData - (in, out) data used by the iteration step logic.
	*/
	bool CalcUnitedGeometrySymmetricFrustum(const TArray<TSharedPtr<IDisplayClusterViewport, ESPMode::ThreadSafe>>& InViewports, const int32 InContextNum, FSymmetricFrustumData& InOutSymmetricFrustumData);

private:
#if WITH_EDITOR
	/** Renders a debug visualization for the group bounding box */
	void DrawDebugGroupBoundingBox(ADisplayClusterRootActor* RootActor, const FLinearColor& Color);

	/** Renders a debug visualization for the group frustum */
	void DrawDebugGroupFrustum(ADisplayClusterRootActor* RootActor, UDisplayClusterInFrustumFitCameraComponent* CameraComponent, const FLinearColor& Color);
#endif

private:
	// Warp projection data
	TOptional<FDisplayClusterWarpProjection> OptUnitedGeometryWarpProjection;

	// United geometry AABB
	// (The world space is used because each viewport can use its own origin.)
	TOptional<FDisplayClusterWarpAABB> OptUnitedGeometryWorldAABB;

	// The view target position (world space).
	// This value is used to obtain the direction of view and to build the "projection plane".
	// (The world space is used because each viewport can use its own origin.)
	TOptional<FVector> OptOverrideWorldViewTarget;
};
