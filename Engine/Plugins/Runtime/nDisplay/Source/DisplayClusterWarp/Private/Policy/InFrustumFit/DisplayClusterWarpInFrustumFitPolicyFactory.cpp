// Copyright Epic Games, Inc. All Rights Reserved.

#include "Policy/InFrustumFit/DisplayClusterWarpInFrustumFitPolicyFactory.h"
#include "Policy/InFrustumFit/DisplayClusterWarpInFrustumFitPolicy.h"

#include "DisplayClusterWarpStrings.h"
#include "DisplayClusterWarpLog.h"

//////////////////////////////////////////////////////////////////////////////////////////////
// FDisplayClusterWarpCameraFitPolicyFactory
//////////////////////////////////////////////////////////////////////////////////////////////
TSharedPtr<IDisplayClusterWarpPolicy, ESPMode::ThreadSafe> FDisplayClusterWarpInFrustumFitPolicyFactory::Create(const FString& InWarpPolicyType, const FString& InWarpPolicyName)
{
	TSharedPtr<IDisplayClusterWarpPolicy, ESPMode::ThreadSafe> Result;

	if (InWarpPolicyType == UE::DisplayClusterWarpStrings::warp::InFrustumFit)
	{
		Result = MakeShared<FDisplayClusterWarpInFrustumFitPolicy, ESPMode::ThreadSafe>(InWarpPolicyName);
	}

	if (Result.IsValid())
	{
		UE_LOGF(LogDisplayClusterWarpInFrustumFitPolicy, Verbose, "Instantiating warp policy <%ls> id='%ls'", *InWarpPolicyType, *InWarpPolicyName);
	}

	return  Result;
}
