// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Render/Warp/IDisplayClusterWarpPolicyFactory.h"

/**
 * Implements warp policy factory for the 'InFrustumFit' policy
 */
class FDisplayClusterWarpInFrustumFitPolicyFactory
	: public IDisplayClusterWarpPolicyFactory
{
public:
	//////////////////////////////////////////////////////////////////////////////////////////////
	// IDisplayClusterWarpPolicyFactory
	//////////////////////////////////////////////////////////////////////////////////////////////
	virtual TSharedPtr<IDisplayClusterWarpPolicy, ESPMode::ThreadSafe> Create(const FString& InWarpPolicyType, const FString& InWarpPolicyName) override;
};
