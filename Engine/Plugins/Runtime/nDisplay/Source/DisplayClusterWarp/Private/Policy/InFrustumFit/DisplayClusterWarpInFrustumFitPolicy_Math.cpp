// Copyright Epic Games, Inc. All Rights Reserved.

#include "Policy/InFrustumFit/DisplayClusterWarpInFrustumFitPolicy.h"

#include "Render/Viewport/IDisplayClusterViewport.h"
#include "Render/Projection/IDisplayClusterProjectionPolicy.h"

#include "IDisplayClusterWarpBlend.h"

#include "Components/DisplayClusterInFrustumFitCameraComponent.h"
#include "Camera/CameraTypes.h"

#include "HAL/IConsoleManager.h"

// Debug: enable camera position fittig
int32 GDisplayClusterWarpInFrustumFitPolicyEnableCameraPositionFit = 1;
static FAutoConsoleVariableRef CVarDisplayClusterWarpInFrustumFitPolicyEnableCameraPositionFit(
	TEXT("nDisplay.warp.InFrustumFit.EnableCameraPositionFit"),
	GDisplayClusterWarpInFrustumFitPolicyEnableCameraPositionFit,
	TEXT("(debug) Enable projection angles fitting (0 - disable)\n"),
	ECVF_Default
);

// Debug: enable projection angle fitting
int32 GDisplayClusterWarpInFrustumFitPolicyEnableProjectionAnglesFit = 1;
static FAutoConsoleVariableRef CVarDisplayClusterWarpInFrustumFitPolicyEnableProjectionAnglesFit(
	TEXT("nDisplay.warp.InFrustumFit.EnableProjectionAnglesFit"),
	GDisplayClusterWarpInFrustumFitPolicyEnableProjectionAnglesFit,
	TEXT("(debug) Enable projection angles fitting (0 - disable)\n"),
	ECVF_Default
);

// Experimental: Enable static view direction for mpcdi 2d
int32 GDisplayClusterWarpInFrustumFitPolicyUseStaticViewDirectionForMPCDIProfile2D = 1;
static FAutoConsoleVariableRef CVarDisplayClusterWarpInFrustumFitPolicyUseStaticViewDirectionForMPCDIProfile2D(
	TEXT("nDisplay.warp.InFrustumFit.UseStaticViewDirectionForMPCDIProfile2D"),
	GDisplayClusterWarpInFrustumFitPolicyUseStaticViewDirectionForMPCDIProfile2D,
	TEXT("Experimental: Enable static view direction for mpcdi 2d (0 - disable)\n"),
	ECVF_Default
);

int32 GDisplayClusterWarpInFrustumFitPolicySymmetricMaxIterations = 10;
static FAutoConsoleVariableRef CVarDisplayClusterWarpInFrustumFitPolicySymmetricMaxIterations(
	TEXT("nDisplay.warp.InFrustumFit.Symmetric.MaxIterations"),
	GDisplayClusterWarpInFrustumFitPolicySymmetricMaxIterations,
	TEXT("Maximum number of iterations to find a symmetric frustum (10 by default)\n"),
	ECVF_Default
);

float GDisplayClusterWarpInFrustumFitPolicySymmetricPrecision = 0.5f;
static FAutoConsoleVariableRef CVarDisplayClusterWarpInFrustumFitPolicySymmetricPrecision(
	TEXT("nDisplay.warp.InFrustumFit.Symmetric.Precision"),
	GDisplayClusterWarpInFrustumFitPolicySymmetricPrecision,
	TEXT("The symmetrical frustum is accepted when this accuracy is achieved (0.5 degree by default) \n"),
	ECVF_Default
);

bool FDisplayClusterWarpInFrustumFitPolicy::GetWarpBlendAPI(IDisplayClusterViewport* InViewport, const uint32 ContextNum, TSharedPtr<IDisplayClusterWarpBlend, ESPMode::ThreadSafe>& OutWarpBlend, TObjectPtr<UDisplayClusterInFrustumFitCameraComponent>& OutConfigurationCameraComponent) const
{
	if (!InViewport || !InViewport->GetProjectionPolicy().IsValid())
	{
		return false;
	}

	// Stereo is currently not supported by InFrustumFit
	if (ContextNum != 0)
	{
		return false;
	}


	if (!InViewport->GetProjectionPolicy()->GetWarpBlendInterface(OutWarpBlend))
	{
		return false;
	}

	OutConfigurationCameraComponent = Cast<UDisplayClusterInFrustumFitCameraComponent>(InViewport->GetViewPointCameraComponent(EDisplayClusterRootActorType::Configuration));
	if (!OutConfigurationCameraComponent)
	{
		return false;
	}

	return true;
}

void FDisplayClusterWarpInFrustumFitPolicy::BeginCalcFrustum(IDisplayClusterViewport* InViewport, const uint32 ContextNum)
{
	TSharedPtr<IDisplayClusterWarpBlend, ESPMode::ThreadSafe> WarpBlend;
	TObjectPtr<UDisplayClusterInFrustumFitCameraComponent> ConfigurationCameraComponent;
	if (!GetWarpBlendAPI(InViewport, ContextNum, WarpBlend, ConfigurationCameraComponent))
	{
		// WarpBlend api and InFrustumFit component are required.
		return;
	}

	FDisplayClusterWarpData& WarpData = WarpBlend->GetWarpData(ContextNum);

	// Override view direction for entire group
	if (WarpData.WarpEye.IsValid())
	{
		// Overrides the view direction vector to build a custom "projection plane".
		if (GDisplayClusterWarpInFrustumFitPolicyUseStaticViewDirectionForMPCDIProfile2D && WarpBlend->GetWarpProfileType() == EDisplayClusterWarpProfileType::warp_2D)
		{
			// [experimental] use static view direction for MPCDI profile 2D
			WarpData.WarpEye->OverrideViewDirection = FVector(1, 0, 0);
		}
		else if (ConfigurationCameraComponent->CameraViewTarget == EDisplayClusterWarpCameraViewTarget::MatchViewOrigin)
		{
			// Use view direction from the ViewPoint component
			// Note: WarpEye already in the Origin space.
			WarpData.WarpEye->OverrideViewDirection = WarpData.WarpEye->ViewPoint.Rotation.RotateVector(FVector::XAxisVector);
		}
		else if (OptOverrideWorldViewTarget.IsSet())
		{
			if (const USceneComponent* const OriginComp = InViewport->GetProjectionPolicy()->GetOriginComponent())
			{
				// WarpBlend math uses the Origin space.
				const FTransform World2OriginTransform = OriginComp->GetComponentTransform();

				// Transform the world space position of the view target to the Origin space.
				const FVector OriginViewTarget = World2OriginTransform.InverseTransformPosition(*OptOverrideWorldViewTarget);
				WarpData.WarpEye->OverrideViewTarget = OriginViewTarget;
			}
		}
		else
		{
			// United frustum can't be builded properly in this case.
			// do nothing and render without customization (as Default ViewPoint)
			return;
		}
	}

	// Todo: This feature now not supported here. need to be fixed.
	WarpData.bEnabledRotateFrustumToFitContextSize = false;
}

void FDisplayClusterWarpInFrustumFitPolicy::EndCalcFrustum(IDisplayClusterViewport* InViewport, const uint32 ContextNum)
{
	if (!OptUnitedGeometryWarpProjection.IsSet())
	{
		// Frustum fit is applied only if OptUnitedGeometryWarpProjection is defined.
		return;
	}

	TSharedPtr<IDisplayClusterWarpBlend, ESPMode::ThreadSafe> WarpBlend;
	TObjectPtr<UDisplayClusterInFrustumFitCameraComponent> ConfigurationCameraComponent;
	if (!GetWarpBlendAPI(InViewport, ContextNum, WarpBlend, ConfigurationCameraComponent))
	{
		// WarpBlend api and InFrustumFit component are required.
		return;
	}

	FDisplayClusterWarpData& WarpData = WarpBlend->GetWarpData(ContextNum);

	// Apply camera frustum fitting:
	FDisplayClusterWarpProjection NewWarpProjection = ApplyInFrustumFit(InViewport, WarpData.WarpEye->World2LocalTransform, WarpData.WarpProjection);
	if (NewWarpProjection.IsValidProjection())
	{
		WarpData.WarpProjection = NewWarpProjection;

		// The warp policy Tick() function uses warp data, and it must be sure that this data is updated in the previous frame.
		//.This value must be set to true from the EndCalcFrustum() warp policy function when changes are made to this structure.
		WarpData.bHasWarpPolicyChanges = true;
	}
}

FDisplayClusterWarpProjection FDisplayClusterWarpInFrustumFitPolicy::ApplyInFrustumFit(IDisplayClusterViewport* InViewport, const FTransform& World2OriginTransform, const FDisplayClusterWarpProjection& InWarpProjection) const
{
	check(OptUnitedGeometryWarpProjection.IsSet());

	UDisplayClusterInFrustumFitCameraComponent* SceneCameraComponent = Cast<UDisplayClusterInFrustumFitCameraComponent>(InViewport->GetViewPointCameraComponent(EDisplayClusterRootActorType::Scene));
	if (!SceneCameraComponent)
	{
		// By default, this structure has invalid values.
		return FDisplayClusterWarpProjection();
	}

	// Get the configuration in use
	const UDisplayClusterInFrustumFitCameraComponent& ConfigurationCameraComponent = SceneCameraComponent->GetConfigurationInFrustumFitCameraComponent(InViewport->GetConfiguration());

	FDisplayClusterWarpProjection OutWarpProjection(InWarpProjection);

	// Get location from scene component
	FMinimalViewInfo CameraViewInfo;
	SceneCameraComponent->GetDesiredView(InViewport->GetConfiguration(), CameraViewInfo);

	// Use camera position to render:
	if (GDisplayClusterWarpInFrustumFitPolicyEnableCameraPositionFit)
	{
		OutWarpProjection.CameraRotation = World2OriginTransform.InverseTransformRotation(CameraViewInfo.Rotation.Quaternion()).Rotator();
		OutWarpProjection.CameraLocation = World2OriginTransform.InverseTransformPosition(CameraViewInfo.Location);
	}

	// Fit the frustum to the rules:

	FDisplayClusterWarpProjection ViewportAngles = InWarpProjection;
	FDisplayClusterWarpProjection UnitedGeometryWarpProjection = *OptUnitedGeometryWarpProjection;

	const FVector2D GeometryFOV(FMath::Abs(
		UnitedGeometryWarpProjection.Right - UnitedGeometryWarpProjection.Left),
		FMath::Abs(UnitedGeometryWarpProjection.Top - UnitedGeometryWarpProjection.Bottom));

	// Convert frustum angles to group FOV space, normalized to 0..1
	const FVector2D ViewportMin((ViewportAngles.Left - UnitedGeometryWarpProjection.Left) / GeometryFOV.X, (ViewportAngles.Bottom - UnitedGeometryWarpProjection.Bottom) / GeometryFOV.Y);
	const FVector2D ViewportMax((ViewportAngles.Right - UnitedGeometryWarpProjection.Left) / GeometryFOV.X, (ViewportAngles.Top - UnitedGeometryWarpProjection.Bottom) / GeometryFOV.Y);

	// And convert back to camera space:
	const float CameraHalfFOVDegrees = CameraViewInfo.FOV * 0.5f;
	const float CameraHalfFOVProjection = UnitedGeometryWarpProjection.ConvertDegreesToProjection(CameraHalfFOVDegrees);
	const FVector2D CameraHalfFOV(CameraHalfFOVProjection, CameraHalfFOVProjection / CameraViewInfo.AspectRatio);
	const FVector2D GeometryHalfFOV = GeometryFOV * 0.5;

	// Receive configuration from InCfgComponent
	const FVector2D FinalHalfFOV = FindFrustumFit(ConfigurationCameraComponent.CameraProjectionMode, CameraHalfFOV, GeometryHalfFOV);

	if (GDisplayClusterWarpInFrustumFitPolicyEnableProjectionAnglesFit)
	{
		// Sample code: convert back to projection angles
		OutWarpProjection.Left   = -FinalHalfFOV.X + ViewportMin.X * FinalHalfFOV.X * 2;
		OutWarpProjection.Right  = -FinalHalfFOV.X + ViewportMax.X * FinalHalfFOV.X * 2;
		OutWarpProjection.Top    = -FinalHalfFOV.Y + ViewportMax.Y * FinalHalfFOV.Y * 2;
		OutWarpProjection.Bottom = -FinalHalfFOV.Y + ViewportMin.Y * FinalHalfFOV.Y * 2;
	}

	return OutWarpProjection;
}

FVector2D FDisplayClusterWarpInFrustumFitPolicy::FindFrustumFit(const EDisplayClusterWarpCameraProjectionMode InProjectionMode, const FVector2D& InCameraFOV, const FVector2D& InGeometryFOV) const
{
	FVector2D OutFOV(InCameraFOV);

	double DestAspectRatio = InGeometryFOV.X / InGeometryFOV.Y;

	switch (InProjectionMode)
	{
	case EDisplayClusterWarpCameraProjectionMode::Fit:
		if (InCameraFOV.Y * DestAspectRatio < InCameraFOV.X)
		{
			OutFOV.Y = InCameraFOV.Y;
			OutFOV.X = OutFOV.Y * DestAspectRatio;
		}
		else
		{

			OutFOV.X = InCameraFOV.X;
			OutFOV.Y = OutFOV.X / DestAspectRatio;
		}
		break;

	case EDisplayClusterWarpCameraProjectionMode::Fill:
		if (InCameraFOV.Y * DestAspectRatio > InCameraFOV.X)
		{
			OutFOV.Y = InCameraFOV.Y;
			OutFOV.X = OutFOV.Y * DestAspectRatio;

			// todo : add check ve 180 degree fov
		}
		else
		{

			OutFOV.X = InCameraFOV.X;
			OutFOV.Y = OutFOV.X / DestAspectRatio;
		}
		break;
	}

	return OutFOV;
}

bool FDisplayClusterWarpInFrustumFitPolicy::CalcUnitedGeometryWorldAABBox(const TArray<TSharedPtr<IDisplayClusterViewport, ESPMode::ThreadSafe>>& InViewports, const float WorldScale, FDisplayClusterWarpAABB& OutUnitedGeometryWorldAABBox) const
{
	// Reset AABB
	OutUnitedGeometryWorldAABBox = FDisplayClusterWarpAABB();

	for (const TSharedPtr<IDisplayClusterViewport, ESPMode::ThreadSafe>& Viewport : InViewports)
	{
		TSharedPtr<IDisplayClusterWarpBlend, ESPMode::ThreadSafe> WarpBlend;
		if (!Viewport.IsValid()
			|| !Viewport->GetProjectionPolicy().IsValid()
			|| !Viewport->GetProjectionPolicy()->GetWarpBlendInterface(WarpBlend)
			|| !WarpBlend->UpdateGeometryContext(WorldScale))
		{
			return false;
		}

		const USceneComponent* const OriginComp = Viewport->GetProjectionPolicy()->GetOriginComponent();
		if (!OriginComp)
		{
			return false;
		}

		// Transform from the Origin component space to the world space.
		const FBox WorldSpaceAABBox = WarpBlend->GetGeometryContext().AABBox.TransformBy(OriginComp->GetComponentTransform());
		OutUnitedGeometryWorldAABBox +=WorldSpaceAABBox;
	}

	return true;
}

bool FDisplayClusterWarpInFrustumFitPolicy::CalcUnitedGeometryFrustum(const TArray<TSharedPtr<IDisplayClusterViewport, ESPMode::ThreadSafe>>& InViewports, const int32 InContextNum, const float WorldScale, FDisplayClusterWarpProjection& OutUnitedGeometryWarpProjection)
{
	// The united geometry frustum is built from geometric points projected onto a special plane.
	// This plane is called the 'projection plane' and is created from two quantities: the view direction vector and the eye position.
	check(!OptUnitedGeometryWarpProjection.IsSet()); // when we calculate the united geometry frustum, we must disable the frustum fit logic.

	OutUnitedGeometryWarpProjection.ResetProjectionAngles();

	bool bResult = true;
	for (const TSharedPtr<IDisplayClusterViewport, ESPMode::ThreadSafe>& Viewport : InViewports)
	{
		TSharedPtr<IDisplayClusterWarpBlend, ESPMode::ThreadSafe> WarpBlend;
		if (!Viewport.IsValid()
			|| !Viewport->GetProjectionPolicy().IsValid()
			|| !Viewport->GetProjectionPolicy()->GetWarpBlendInterface(WarpBlend))
		{
			return false;
		}

		// Note: This code is partially copied from FDisplayClusterProjectionMPCDIPolicy::CalculateView().
		// Override viewpoint - MPCDI always expects the location of the viewpoint component (eye location from the real world)
		FVector ViewOffset = FVector::ZeroVector;
		FVector ViewLocation;
		FRotator ViewRotation;
		if (!Viewport->GetViewPointCameraEye(InContextNum, ViewLocation, ViewRotation, ViewOffset))
		{
			return false;
		}

		const USceneComponent* const OriginComp = Viewport->GetProjectionPolicy()->GetOriginComponent();
		if (!OriginComp)
		{
			return false;
		}

		// Setup eye data:
		TSharedPtr<FDisplayClusterWarpEye, ESPMode::ThreadSafe> WarpEye = MakeShared<FDisplayClusterWarpEye, ESPMode::ThreadSafe>(Viewport, 0);

		WarpEye->World2LocalTransform = (OriginComp ? OriginComp->GetComponentTransform() : FTransform::Identity);

		// Get our base camera location and view offset in local space (MPCDI space)
		WarpEye->ViewPoint.Location = WarpEye->World2LocalTransform.InverseTransformPosition(ViewLocation - ViewOffset);
		WarpEye->ViewPoint.EyeOffset = WarpEye->World2LocalTransform.InverseTransformPosition(ViewLocation) - WarpEye->ViewPoint.Location;
		WarpEye->ViewPoint.Rotation = WarpEye->World2LocalTransform.InverseTransformRotation(ViewRotation.Quaternion()).Rotator();

		WarpEye->WorldScale = WorldScale;
		WarpEye->WarpPolicy = SharedThis(this);

		if (!WarpBlend->CalcFrustumContext(WarpEye))
		{
			bResult = false;
		}

		// Merge two projection
		const FDisplayClusterWarpData& WarpData = WarpBlend->GetWarpData(InContextNum);
		OutUnitedGeometryWarpProjection.ExpandProjectionAngles(WarpData.GeometryWarpProjection);
	}

	return bResult;
}

bool FDisplayClusterWarpInFrustumFitPolicy::CalcUnitedGeometrySymmetricFrustum(const TArray<TSharedPtr<IDisplayClusterViewport, ESPMode::ThreadSafe>>& InViewports, const int32 InContextNum, FSymmetricFrustumData& InOutSymmetricFrustumData)
{
	check(OptOverrideWorldViewTarget.IsSet()); // this function requires this value

	if (++InOutSymmetricFrustumData.IterationNum > GDisplayClusterWarpInFrustumFitPolicySymmetricMaxIterations)
	{
		// we are reaching the iteration limit, so we can use the value from the previous iteration step.
		InOutSymmetricFrustumData.IterationNum = INDEX_NONE;

		return false;
	}

	// Begin new iteration
	InOutSymmetricFrustumData.NewWorldViewTarget.Reset();
	InOutSymmetricFrustumData.NewUnitedSymmetricWarpProjection.Reset();

	// Calculate the united  frustum for the current view target.
	FDisplayClusterWarpProjection UnitedGeometryWarpProjection;

	const double OffsetH = 0.5 * (UnitedGeometryWarpProjection.Left + UnitedGeometryWarpProjection.Right);
	const double OffsetV = 0.5 * (UnitedGeometryWarpProjection.Bottom + UnitedGeometryWarpProjection.Top);

	const double OffsetDegreesH = UnitedGeometryWarpProjection.ConvertProjectionToDegrees(OffsetH);
	const double OffsetDegreesV = UnitedGeometryWarpProjection.ConvertProjectionToDegrees(OffsetV);

	const bool bFrustumValid = CalcUnitedGeometryFrustum(InViewports, InContextNum, InOutSymmetricFrustumData.WorldScale, UnitedGeometryWarpProjection);
	if (bFrustumValid)
	{
		const double MaxOffset = FMath::Max(FMath::Abs(OffsetH), FMath::Abs(OffsetV));

		// Update best value
		if (!InOutSymmetricFrustumData.BestSymmetricWarpProjection.IsSet()
			|| InOutSymmetricFrustumData.BestSymmetricWarpProjection->Key > MaxOffset)
		{
			InOutSymmetricFrustumData.BestSymmetricWarpProjection = { MaxOffset , UnitedGeometryWarpProjection };
			InOutSymmetricFrustumData.BestWorldViewTarget = *OptOverrideWorldViewTarget;
		}

		if (InOutSymmetricFrustumData.BestSymmetricWarpProjection->Key < MaxOffset)
		{
			// @todo Add a new method to get a better result (multiple rays in a sphere, etc.)

			// current value if worst, stop iterator
			InOutSymmetricFrustumData.IterationNum = INDEX_NONE;

			return false;
		}

		// Check the precision.
		if (FMath::Max(FMath::Abs(OffsetDegreesH), FMath::Abs(OffsetDegreesV)) <= FMath::Abs(GDisplayClusterWarpInFrustumFitPolicySymmetricPrecision))
		{
			// A symmetric frustum has been found:
			InOutSymmetricFrustumData.NewUnitedSymmetricWarpProjection = UnitedGeometryWarpProjection;

			return true;
		}
	}
	else
	{
		// Frustum not exist
		if (InOutSymmetricFrustumData.IterationNum > 1)
		{
			// on the first iteration dont exit and try to find something from the base AABB center-point
			return false;
		}
	}

	// Calculate the new location of the view target from the values of the united geometry frustum.
	const FVector ViewTarget    = InOutSymmetricFrustumData.CameraComponent2WorldTransform.InverseTransformPosition(*OptOverrideWorldViewTarget);
	const double ViewTargetSize = ViewTarget.Length();

	// Create a projection space matrix from the view direction vector.
	// This code is similar to the code used inside the WarpBlend api in the FDisplayClusterWarpBlendMath_Frustum::ImplCalcViewProjectionMatrices() function.
	const FVector ViewDirection = ViewTarget.GetSafeNormal();
	const FMatrix Projection2Local = (FMath::Abs(ViewDirection.Z) < (1.f - KINDA_SMALL_NUMBER))
		? FRotationMatrix::MakeFromXZ(ViewDirection, FVector(0.f, 0.f, 1.f))
		: FRotationMatrix::MakeFromXY(ViewDirection, FVector(0.f, 1.f, 0.f));

	// Calc new view direction vector in projection space
	const FRotator CorrectionRotator(OffsetDegreesV, OffsetDegreesH, 0);
	const FVector ProjectionNewViewDirection = CorrectionRotator.RotateVector(FVector::XAxisVector);

	// Transform a new view direction vector from projection space to local space.
	const FVector NewViewDirection
		= Projection2Local.GetUnitAxis(EAxis::X) * ProjectionNewViewDirection.X
		+ Projection2Local.GetUnitAxis(EAxis::Y) * ProjectionNewViewDirection.Y
		+ Projection2Local.GetUnitAxis(EAxis::Z) * ProjectionNewViewDirection.Z;

	// Get the new position of the view target in the local space.
	const FVector NewViewTarget = NewViewDirection * ViewTargetSize;

	// Transform the location of the view target from local space to world space.
	const FVector NewWorldViewTarget = InOutSymmetricFrustumData.CameraComponent2WorldTransform.TransformPosition(NewViewTarget);

	// Use new view target for the next iteration:
	InOutSymmetricFrustumData.NewWorldViewTarget = NewWorldViewTarget;

	// Save last valid projection
	InOutSymmetricFrustumData.NewUnitedSymmetricWarpProjection = UnitedGeometryWarpProjection;

	return false;
}
