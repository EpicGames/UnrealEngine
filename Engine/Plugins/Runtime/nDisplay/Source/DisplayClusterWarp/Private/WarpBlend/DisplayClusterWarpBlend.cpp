// Copyright Epic Games, Inc. All Rights Reserved.

#include "WarpBlend/DisplayClusterWarpBlend.h"
#include "WarpBlend/Math/DisplayClusterWarpBlendMath_Frustum.h"

#include "Render/Containers/IDisplayClusterRender_MeshComponent.h"
#include "Render/Projection/IDisplayClusterProjectionPolicy.h"
#include "Render/Viewport/IDisplayClusterViewport.h"

#include "Blueprints/DisplayClusterWarpGeometry.h"

#include "HAL/IConsoleManager.h"
#include "UObject/UObjectGlobals.h"

#include "ProceduralMeshComponent.h"
#include "Components/StaticMeshComponent.h"
#include "ProceduralMeshComponent.h"

// Setup frustum projection cache
static TAutoConsoleVariable<int32> CVarMPCDIFrustumCacheDepth(
	TEXT("nDisplay.render.mpcdi.cache_depth"),
	0,// Default disabled
	TEXT("Frustum values cache (depth, num).\n")
	TEXT("By default cache is disabled. For better performance (EDisplayClusterWarpBlendFrustumType::FULL) set value to 512).\n")
	TEXT(" 0: Disabled\n")
	TEXT(" N: Cache size, integer\n"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarMPCDIFrustumCachePrecision(
	TEXT("nDisplay.render.mpcdi.cache_precision"),
	0.1f, // 1mm
	TEXT("Frustum cache values comparison precision (float, unit is sm).\n"),
	ECVF_RenderThreadSafe
);

//--------------------------------------------------------------------------------------------
// FDisplayClusterWarpBlend
//--------------------------------------------------------------------------------------------
FDisplayClusterWarpBlend::FDisplayClusterWarpBlend()
{
	WarpData.AddDefaulted(2);
}

FDisplayClusterWarpBlend::~FDisplayClusterWarpBlend()
{
	// Release resources
	GeometryContext.GeometryProxy.ReleaseResources();
}

bool FDisplayClusterWarpBlend::MarkWarpGeometryComponentDirty(const FName& InComponentName)
{
	return GeometryContext.GeometryProxy.MarkWarpFrustumGeometryComponentDirty(InComponentName);
}

bool FDisplayClusterWarpBlend::ShouldSupportICVFX() const
{
	if (GetWarpProfileType() == EDisplayClusterWarpProfileType::warp_A3D)
	{
		return true;
	}

	return false;
}

bool FDisplayClusterWarpBlend::UpdateGeometryContext(const float InWorldScale)
{
	return GeometryContext.UpdateGeometryContext(InWorldScale);
}

const FDisplayClusterWarpGeometryContext& FDisplayClusterWarpBlend::GetGeometryContext() const
{
	return GeometryContext.Context;
}

bool FDisplayClusterWarpBlend::CalcFrustumContext(const TSharedPtr<FDisplayClusterWarpEye, ESPMode::ThreadSafe>& InWarpEye)
{
	check(WarpData.IsValidIndex(InWarpEye->ContextNum));

	// Update current warp data
	FDisplayClusterWarpData& CurrentWarpData = WarpData[InWarpEye->ContextNum];

	CurrentWarpData.WarpEye = InWarpEye;

	CurrentWarpData.bValid = false;
	CurrentWarpData.bHasWarpPolicyChanges = false;

	if(!InWarpEye.IsValid())
	{
		return false;
	}

	FDisplayClusterWarpBlendMath_Frustum Frustum(CurrentWarpData, GeometryContext);

	if (!GeometryContext.UpdateGeometryContext(InWarpEye->WorldScale))
	{
		// wrong geometry
		return false;
	}

	// Override settings from warp policy
	BeginCalcFrustum(InWarpEye);

	if (!Frustum.CalcFrustum())
	{
		return false;
	}

	// Now this data becomes valid.
	CurrentWarpData.bValid = true;

	return true;
}

void FDisplayClusterWarpBlend::BeginCalcFrustum(const TSharedPtr<FDisplayClusterWarpEye, ESPMode::ThreadSafe>& InWarpEye)
{
	check(InWarpEye.IsValid());

	if (InWarpEye->WarpPolicy.IsValid())
	{
		if (IDisplayClusterViewport* Viewport = InWarpEye->GetViewport())
		{
			InWarpEye->WarpPolicy->BeginCalcFrustum(Viewport, InWarpEye->ContextNum);
		}
	}
}

FDisplayClusterWarpData& FDisplayClusterWarpBlend::GetWarpData(const uint32 ContextNum)
{
	check(WarpData.IsValidIndex(ContextNum));

	return WarpData[ContextNum];

}

const FDisplayClusterWarpData& FDisplayClusterWarpBlend::GetWarpData(const uint32 ContextNum) const
{
	check(WarpData.IsValidIndex(ContextNum));

	return WarpData[ContextNum];

}

UMeshComponent* FDisplayClusterWarpBlend::GetOrCreatePreviewMeshComponent(IDisplayClusterViewport* InViewport, bool& bExistingComponent) const
{
	return GetOrCreatePreviewMeshComponentImpl(InViewport, false, bExistingComponent);
}

UMeshComponent* FDisplayClusterWarpBlend::GetOrCreatePreviewEditableMeshComponent(IDisplayClusterViewport* InViewport) const
{
	bool bExistingComponentDummy;
	return GetOrCreatePreviewMeshComponentImpl(InViewport, true, bExistingComponentDummy);
}

UMeshComponent* FDisplayClusterWarpBlend::GetOrCreatePreviewMeshComponentImpl(IDisplayClusterViewport* InViewport, bool bEditableMesh, bool& bExistingComponent) const
{
	const TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe> ProjectionPolicy = InViewport ? InViewport->GetProjectionPolicy() : nullptr;

	switch(GetWarpGeometryType())
	{
	case EDisplayClusterWarpGeometryType::WarpMesh:
	case EDisplayClusterWarpGeometryType::WarpProceduralMesh:
		// use the existing DCRA component
		if (USceneComponent* PreviewMeshComponent = GeometryContext.GeometryProxy.PreviewMeshComponentRef.GetOrFindSceneComponent())
		{
			if (PreviewMeshComponent->IsA<UMeshComponent>())
			{
				UMeshComponent* ExistMeshComponent = static_cast<UMeshComponent*>(PreviewMeshComponent);

				if (bEditableMesh)
				{
					// create a mesh component copy
					bExistingComponent = false;

					// Get editable mesh root
					USceneComponent* SceneOriginComp = ProjectionPolicy->GetPreviewEditableMeshOriginComponent(InViewport);

					const FString CompName = FString::Printf(TEXT("DCWarpBlend_EditableMesh_%s"), *ProjectionPolicy->GetId());
					const EObjectFlags ObjectFlags = EObjectFlags::RF_DuplicateTransient | RF_Transient | RF_TextExportTransient;
					if (UMeshComponent* PreviewMeshComponentCopy = NewObject<UMeshComponent>(SceneOriginComp, ExistMeshComponent->GetClass(), *CompName, ObjectFlags, ExistMeshComponent))
					{
						PreviewMeshComponentCopy->RegisterComponent();
						PreviewMeshComponentCopy->AttachToComponent(SceneOriginComp, FAttachmentTransformRules(EAttachmentRule::KeepRelative, false));
						PreviewMeshComponentCopy->SetHiddenInGame(true);

#if WITH_EDITOR
						PreviewMeshComponentCopy->SetIsVisualizationComponent(true);
#endif

						return PreviewMeshComponentCopy;
					}
				}
				else
				{
					bExistingComponent = true;

					return ExistMeshComponent;
				}
				
			}
		}

		return nullptr;

	case EDisplayClusterWarpGeometryType::WarpMap:
		if (ProjectionPolicy.IsValid())
		{
			// use the existing DCRA component
			if (USceneComponent* PreviewMeshComponent = GeometryContext.GeometryProxy.PreviewMeshComponentRef.GetOrFindSceneComponent())
			{
				if (PreviewMeshComponent->IsA<UMeshComponent>())
				{
					UMeshComponent* ExistMeshComponent = static_cast<UMeshComponent*>(PreviewMeshComponent);

					bExistingComponent = true;
					return ExistMeshComponent;
				}
			}

			// create a new mesh component
			bExistingComponent = false;

			// Downscale preview mesh dimension to max limit
			const uint32 PreviewGeometryDimLimit = 128;

			USceneComponent* PreviewOriginComp = bEditableMesh ? ProjectionPolicy->GetPreviewEditableMeshOriginComponent(InViewport) : ProjectionPolicy->GetPreviewMeshOriginComponent(InViewport);

			// Create new WarpMesh component
			FDisplayClusterWarpGeometryOBJ MeshData;
			if (PreviewOriginComp && ExportWarpMapGeometry(MeshData, PreviewGeometryDimLimit))
			{
				const FString CompName = FString::Printf(TEXT("DCWarpBlend_PFM_%s_mesh"), *ProjectionPolicy->GetId());

				// Creta new object
				UProceduralMeshComponent* MeshComp = NewObject<UProceduralMeshComponent>(PreviewOriginComp, FName(*CompName), EObjectFlags::RF_DuplicateTransient | RF_Transient | RF_TextExportTransient);
				if (MeshComp)
				{
					MeshComp->RegisterComponent();
					MeshComp->AttachToComponent(PreviewOriginComp, FAttachmentTransformRules(EAttachmentRule::KeepRelative, false));
					MeshComp->CreateMeshSection(0, MeshData.Vertices, MeshData.Triangles, MeshData.Normal, MeshData.UV, TArray<FColor>(), TArray<FProcMeshTangent>(), false);
#if WITH_EDITOR
					MeshComp->SetIsVisualizationComponent(true);
#endif

					// Because of "nDisplay.render.show.visualizationcomponents" we need extra flag to exclude this geometry from render
					MeshComp->SetHiddenInGame(true);

					return MeshComp;
				}
			}
		}

	break;

	default:
		break;
	}

	return nullptr;
}

bool FDisplayClusterWarpBlend::ExportWarpMapGeometry(FDisplayClusterWarpGeometryOBJ& OutMeshData, uint32 InMaxDimension) const
{
	return FDisplayClusterWarpBlendExporter_WarpMap::ExportWarpMap(GeometryContext, OutMeshData, InMaxDimension);
}

bool FDisplayClusterWarpBlend::HandleStartScene(IDisplayClusterViewport* InViewport)
{
	return true;
}

void FDisplayClusterWarpBlend::HandleEndScene(IDisplayClusterViewport* InViewport)
{ }
