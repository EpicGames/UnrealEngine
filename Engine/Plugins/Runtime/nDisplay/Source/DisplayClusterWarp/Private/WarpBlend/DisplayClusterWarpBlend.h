// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "IDisplayClusterWarpBlend.h"
#include "WarpBlend/DisplayClusterWarpBlend_GeometryContext.h"
#include "WarpBlend/Exporter/DisplayClusterWarpBlendExporter_WarpMap.h"
#include "Render/Containers/IDisplayClusterRender_Texture.h"
#include "Templates/SharedPointer.h"

/**
 * WarpBlend interface implementation for MPCDI and mesh projection policies
 */
class FDisplayClusterWarpBlend
	: public IDisplayClusterWarpBlend
	, public TSharedFromThis<FDisplayClusterWarpBlend, ESPMode::ThreadSafe>
{
public:
	FDisplayClusterWarpBlend();

	//~ IDisplayClusterWarpBlend
	virtual ~FDisplayClusterWarpBlend() override;

	virtual TSharedPtr<IDisplayClusterWarpBlend, ESPMode::ThreadSafe> ToSharedPtr() override
	{
		return AsShared();
	}

	virtual TSharedPtr<const IDisplayClusterWarpBlend, ESPMode::ThreadSafe> ToSharedPtr() const override
	{
		return AsShared();
	}

	virtual bool HandleStartScene(IDisplayClusterViewport* InViewport) override;
	virtual void HandleEndScene(IDisplayClusterViewport* InViewport) override;

	virtual bool MarkWarpGeometryComponentDirty(const FName& InComponentName) override;

	virtual bool UpdateGeometryContext(const float InWorldScale) override;
	virtual const FDisplayClusterWarpGeometryContext& GetGeometryContext() const override;

	virtual bool CalcFrustumContext(const TSharedPtr<FDisplayClusterWarpEye, ESPMode::ThreadSafe>& InWarpEye) override;

	/** Get texture by type. */
	virtual class FRHITexture* GetTexture(EDisplayClusterWarpBlendTextureType TextureType) const override
	{
		const TSharedPtr<IDisplayClusterRender_Texture, ESPMode::ThreadSafe> pTexture = GeometryContext.GeometryProxy.ImplGetTexture(TextureType);
		return pTexture.IsValid() ? pTexture->GetRHITexture() : nullptr;
	}

	virtual TSharedPtr<IDisplayClusterRender_Texture, ESPMode::ThreadSafe> GetTextureInterface(EDisplayClusterWarpBlendTextureType InTextureType) const override
	{
		return GeometryContext.GeometryProxy.ImplGetTexture(InTextureType);
	}

	virtual float GetAlphaMapEmbeddedGamma() const override
	{ 
		return GeometryContext.GeometryProxy.AlphaMapEmbeddedGamma; 
	}

	virtual const IDisplayClusterRender_MeshComponentProxy* GetWarpMeshProxy_RenderThread() const override
	{
		return GeometryContext.GeometryProxy.GetWarpMeshProxy_RenderThread();
	}

	// Return current warp profile type
	virtual EDisplayClusterWarpProfileType  GetWarpProfileType() const override
	{
		return GeometryContext.GeometryProxy.MPCDIAttributes.ProfileType;
	}

	virtual EDisplayClusterWarpGeometryType GetWarpGeometryType() const override
	{
		return GeometryContext.GeometryProxy.GeometryType;
	}

	virtual EDisplayClusterWarpFrustumGeometryType GetWarpFrustumGeometryType() const override
	{
		return GeometryContext.GeometryProxy.FrustumGeometryType;
	}

	virtual const FDisplayClusterWarpMPCDIAttributes& GetMPCDIAttributes() const override
	{
		return GeometryContext.GeometryProxy.MPCDIAttributes;
	}

	virtual bool ExportWarpMapGeometry(FDisplayClusterWarpGeometryOBJ& OutMeshData, uint32 InMaxDimension = 0) const override final;

	virtual bool ShouldSupportICVFX() const override;

	virtual FDisplayClusterWarpData& GetWarpData(const uint32 ContextNum) override;
	virtual const FDisplayClusterWarpData& GetWarpData(const uint32 ContextNum) const override;

	virtual UMeshComponent* GetOrCreatePreviewMeshComponent(IDisplayClusterViewport* InViewport, bool& bExistingComponent) const override;
	virtual UMeshComponent* GetOrCreatePreviewEditableMeshComponent(IDisplayClusterViewport* InViewport) const override;

	//~!IDisplayClusterWarpBlend

private:
	/**
	* Get or create a mesh component for previewing
	* 
	* @param InViewport - The viewport that do this request 
	* @param bCreateCopy - true to always create a copy of the mesh
	* @param bExistingComponent - (out) true if the mesh component is not a copy
	*/
	UMeshComponent* GetOrCreatePreviewMeshComponentImpl(IDisplayClusterViewport* InViewport, bool bEditableMesh, bool& bExistingComponent) const;

	/** Begin frustum calc. */
	void BeginCalcFrustum(const TSharedPtr<FDisplayClusterWarpEye, ESPMode::ThreadSafe>& InWarpEye);

public:
	FDisplayClusterWarpBlend_GeometryContext GeometryContext;

private:
	TArray<FDisplayClusterWarpData> WarpData;
};
