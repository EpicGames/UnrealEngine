// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterWarpBlend_GeometryContext.h"
#include "WarpBlend/DisplayClusterWarpBlend_GeometryProxy.h"

//------------------------------------------------------------------------------
// FDisplayClusterWarpBlend_GeometryContext
//------------------------------------------------------------------------------
bool FDisplayClusterWarpBlend_GeometryContext::UpdateGeometryContext(const double InWorldScale)
{
	if (!GeometryProxy.UpdateFrustumGeometry())
	{
		return false;
	}

	// Update runtime data:
	Context.GeometryToOrigin = GeometryProxy.GeometryCache.GeometryToOrigin.ToMatrixWithScale();

	Context.SurfaceViewNormal = Context.GeometryToOrigin.TransformVector(GeometryProxy.GeometryCache.SurfaceViewNormal);
	Context.SurfaceViewPlane = Context.GeometryToOrigin.TransformVector(GeometryProxy.GeometryCache.SurfaceViewPlane);

	// Use the built in bounding box transform method, which will correctly transform the box using origin and extent instead
	// of just transforming the min and max properties, which will not correctly transform the box
	FBox TransformedBox = GeometryProxy.GeometryCache.AABBox.TransformBy(Context.GeometryToOrigin);

	Context.AABBox.Max = TransformedBox.Max * InWorldScale;
	Context.AABBox.Min = TransformedBox.Min * InWorldScale;

	return true;
}

FMatrix FDisplayClusterWarpBlend_GeometryContext::GetTextureMatrix() const
{
	switch (GetWarpProfileType())
	{
	case EDisplayClusterWarpProfileType::warp_A3D:
		// Fetching from a 3D source
		return FMatrix(
			FPlane(0.5f, 0, 0, 0),
			FPlane(0, -0.5f, 0, 0),
			FPlane(0, 0, 1, 0),
			FPlane(0.5f, 0.5f, 0, 1));

	case EDisplayClusterWarpProfileType::warp_2D:
		// Fetching from a 2D source
		return FMatrix(
			FPlane(1, 0, 0, 0),
			FPlane(0, -1, 0, 0),
			FPlane(0, 0, 1, 0),
			FPlane(0, 1, 0, 1));

	default:
		break;
	}

	return FMatrix::Identity;
}

FMatrix FDisplayClusterWarpBlend_GeometryContext::GetRegionMatrix() const
{
	if (Context.bUseRegionMatrix)
	{
		const FVector2D& Size = GeometryProxy.MPCDIAttributes.Region.Size;
		const FVector2D& Pos  = GeometryProxy.MPCDIAttributes.Region.Pos;

		// Build Region matrix
		return FMatrix(
			FPlane(Size.X, 0,      0, 0),
			FPlane(0,      Size.Y, 0, 0),
			FPlane(0,      0,      1, 0),
			FPlane(Pos.X,  Pos.Y,  0, 1));
	}

	return FMatrix::Identity;
}
