// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/DisplayClusterWarpEye.h"
#include "WarpBlend/DisplayClusterWarpBlend_GeometryProxy.h"
#include "Containers/DisplayClusterWarpContext.h"

/**
 * Math context for Warp
 */
class FDisplayClusterWarpBlend_GeometryContext
{
public:
	FDisplayClusterWarpBlend_GeometryContext()
	{ }

	~FDisplayClusterWarpBlend_GeometryContext()
	{ }

public:
	/** Update internal geometry context (AABB, matrices, etc)
	*/
	bool UpdateGeometryContext(const double InWorldScale);

	/** Get matrix for WarpMap texture. */
	FMatrix GetTextureMatrix() const;

	/** Obtaining a matrix for the area on the input texture (mpcdi 2d profile). */
	FMatrix GetRegionMatrix() const;

	/** Returns the profile type used. */
	inline EDisplayClusterWarpProfileType  GetWarpProfileType() const
	{
		return GeometryProxy.MPCDIAttributes.ProfileType;
	}

	/** Assignment of the profile type to be used. */
	inline void SetWarpProfileType(const EDisplayClusterWarpProfileType InWarpProfileType)
	{
		GeometryProxy.MPCDIAttributes.ProfileType = InWarpProfileType;
	}

public:
	// Geometry container
	FDisplayClusterWarpBlend_GeometryProxy GeometryProxy;

	// Geometry context
	FDisplayClusterWarpGeometryContext Context;
};
