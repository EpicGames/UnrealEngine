// Copyright Epic Games, Inc. All Rights Reserved.

#include "WarpBlend/DisplayClusterWarpBlend_GeometryProxy.h"

#include "Render/Containers/IDisplayClusterRender_MeshComponent.h"

#include "WarpBlend/Math/DisplayClusterWarpBlendMath_WarpMesh.h"
#include "WarpBlend/Math/DisplayClusterWarpBlendMath_WarpProceduralMesh.h"
#include "WarpBlend/Math/DisplayClusterWarpBlendMath_WarpMap.h"

#include "WarpBlend/Exporter/DisplayClusterWarpBlendExporter_WarpMap.h"

/////////////////////////////////////////////////////////////////////////////////
/// FDisplayClusterWarpBlend_GeometryProxy
/////////////////////////////////////////////////////////////////////////////////
bool FDisplayClusterWarpBlend_GeometryProxy::UpdateFrustumGeometry()
{
	bIsGeometryValid = false;
	switch (FrustumGeometryType)
	{
	case EDisplayClusterWarpFrustumGeometryType::WarpMesh:
		bIsGeometryValid = ImplUpdateFrustumGeometry_WarpMesh();
		break;

	case EDisplayClusterWarpFrustumGeometryType::WarpProceduralMesh:
		bIsGeometryValid = ImplUpdateFrustumGeometry_WarpProceduralMesh();
		break;

	case EDisplayClusterWarpFrustumGeometryType::WarpMap:
		bIsGeometryValid = ImplUpdateFrustumGeometry_WarpMap();
		break;

	case EDisplayClusterWarpFrustumGeometryType::MPCDIAttributes:
		bIsGeometryValid = ImplUpdateFrustumGeometry_MPCDIAttributes();
		break;

	default:
		break;
	}

	if (!bIsGeometryValid)
	{
		// In case of an error, the cached data is invalidated
		bIsGeometryCacheValid = false;
	}
	
	return bIsGeometryValid;
}

const IDisplayClusterRender_MeshComponentProxy* FDisplayClusterWarpBlend_GeometryProxy::GetWarpMeshProxy_RenderThread() const
{
	check(IsInRenderingThread());

	switch (GeometryType)
	{
	case EDisplayClusterWarpGeometryType::WarpMesh:
	case EDisplayClusterWarpGeometryType::WarpProceduralMesh:
		return WarpMeshComponent.IsValid() ? WarpMeshComponent->GetMeshComponentProxy_RenderThread() : nullptr;

	default:
		break;
	}

	return nullptr;
}

bool FDisplayClusterWarpBlend_GeometryProxy::MarkWarpFrustumGeometryComponentDirty(const FName& InComponentName)
{
	switch (FrustumGeometryType)
	{
	case EDisplayClusterWarpFrustumGeometryType::WarpMesh:
	case EDisplayClusterWarpFrustumGeometryType::WarpProceduralMesh:
		if (WarpMeshComponent.IsValid())
		{
			if (InComponentName == NAME_None || WarpMeshComponent->EqualsMeshComponentName(InComponentName))
			{
				WarpMeshComponent->MarkMeshComponentRefGeometryDirty();
				return true;
			}
		}
		break;
	default:
		break;
	}

	return false;
}

bool FDisplayClusterWarpBlend_GeometryProxy::ImplUpdateFrustumGeometry_MPCDIAttributes()
{
	GeometryCache.GeometryToOrigin = FTransform::Identity;

	return ImplUpdateFrustumGeometryCache_MPCDIAttributes();
}

bool FDisplayClusterWarpBlend_GeometryProxy::ImplUpdateFrustumGeometry_WarpMap()
{
	GeometryCache.GeometryToOrigin = FTransform::Identity;

	return ImplUpdateFrustumGeometryCache_WarpMap();
}

bool FDisplayClusterWarpBlend_GeometryProxy::ImplUpdateFrustumGeometry_WarpMesh()
{
	if (!WarpMeshComponent.IsValid())
	{
		return false;
	}

	UStaticMeshComponent* StaticMeshComponent = WarpMeshComponent->GetStaticMeshComponent();
	USceneComponent*      OriginComponent     = WarpMeshComponent->GetOriginComponent();

	const FStaticMeshLODResources* StaticMeshLODResources = (StaticMeshComponent!=nullptr) ? WarpMeshComponent->GetStaticMeshComponentLODResources(StaticMeshComponentLODIndex) : nullptr;
	if (StaticMeshLODResources == nullptr)
	{
		// mesh deleted?
		WarpMeshComponent->ReleaseProxyGeometry();
		bIsMeshComponentLost = true;

		return false;
	};

	// If StaticMesh geometry changed, update mpcdi math and RHI resources
	if (WarpMeshComponent->IsMeshComponentRefGeometryDirty() || bIsMeshComponentLost)
	{
		WarpMeshComponent->AssignStaticMeshComponentRefs(StaticMeshComponent, WarpMeshUVs, OriginComponent, StaticMeshComponentLODIndex);
		bIsMeshComponentLost = false;
	}
	
	// Update caches
	if (!ImplUpdateFrustumGeometryCache_WarpMesh())
	{
		return false;
	}

	if (OriginComponent)
	{
		FMatrix MeshToWorldMatrix = StaticMeshComponent->GetComponentTransform().ToMatrixWithScale();
		FMatrix WorldToOriginMatrix = OriginComponent->GetComponentTransform().ToInverseMatrixWithScale();

		GeometryCache.GeometryToOrigin.SetFromMatrix(MeshToWorldMatrix * WorldToOriginMatrix);
	}
	else
	{
		GeometryCache.GeometryToOrigin = StaticMeshComponent->GetRelativeTransform();
	}

	return true;
}

bool FDisplayClusterWarpBlend_GeometryProxy::ImplUpdateFrustumGeometry_WarpProceduralMesh()
{
	if (!WarpMeshComponent.IsValid())
	{
		return false;
	}

	UProceduralMeshComponent* ProceduralMeshComponent = WarpMeshComponent->GetProceduralMeshComponent();
	USceneComponent*          OriginComponent         = WarpMeshComponent->GetOriginComponent();

	const FProcMeshSection* ProcMeshSection = (ProceduralMeshComponent != nullptr) ? WarpMeshComponent->GetProceduralMeshComponentSection(ProceduralMeshComponentSectionIndex) : nullptr;
	if (ProcMeshSection == nullptr)
	{
		// mesh deleted, lost or section not defined
		WarpMeshComponent->ReleaseProxyGeometry();
		bIsMeshComponentLost = true;
		return false;
	};

	// If ProceduralMesh geometry changed, update mpcdi math and RHI resources
	if (WarpMeshComponent->IsMeshComponentRefGeometryDirty() || bIsMeshComponentLost)
	{
		WarpMeshComponent->AssignProceduralMeshComponentRefs(ProceduralMeshComponent, WarpMeshUVs, OriginComponent, ProceduralMeshComponentSectionIndex);
		bIsMeshComponentLost = false;
	}

	// Update caches
	if (!ImplUpdateFrustumGeometryCache_WarpProceduralMesh())
	{
		return false;
	}

	FMatrix GeometryToOriginMatrix;
	if (OriginComponent)
	{
		FMatrix MeshToWorldMatrix = ProceduralMeshComponent->GetComponentTransform().ToMatrixWithScale();
		FMatrix WorldToOriginMatrix = OriginComponent->GetComponentTransform().ToInverseMatrixWithScale();

		GeometryToOriginMatrix = MeshToWorldMatrix * WorldToOriginMatrix;
	}
	else
	{
		GeometryToOriginMatrix = ProceduralMeshComponent->GetRelativeTransform().ToMatrixWithScale();
	}

	GeometryCache.GeometryToOrigin.SetFromMatrix(GeometryToOriginMatrix);

	return true;
}

bool FDisplayClusterWarpBlend_GeometryProxy::ImplUpdateFrustumGeometryCache_WarpMesh()
{
	if (WarpMeshComponent.IsValid())
	{
		const FStaticMeshLODResources* StaticMeshLODResources = WarpMeshComponent->GetStaticMeshComponentLODResources(StaticMeshComponentLODIndex);
		if (StaticMeshLODResources != nullptr)
		{
			if (bIsGeometryCacheValid)
			{
				return true;
			}

			FDisplayClusterWarpBlendMath_WarpMesh MeshHelper(*StaticMeshLODResources);

			GeometryCache.AABBox = MeshHelper.CalcAABBox();
			MeshHelper.CalcSurfaceVectors(GeometryCache.SurfaceViewNormal, GeometryCache.SurfaceViewPlane);

			bIsGeometryCacheValid = true;

			return true;
		}
	}

	bIsGeometryCacheValid = false;

	return false;
}

bool FDisplayClusterWarpBlend_GeometryProxy::ImplUpdateFrustumGeometryCache_WarpProceduralMesh()
{
	if (WarpMeshComponent.IsValid())
	{
		const FProcMeshSection* ProcMeshSection = WarpMeshComponent->GetProceduralMeshComponentSection(ProceduralMeshComponentSectionIndex);
		if (ProcMeshSection != nullptr)
		{
			if (bIsGeometryCacheValid)
			{
				return true;
			}

			FDisplayClusterWarpBlendMath_WarpProceduralMesh ProceduralMeshHelper(*ProcMeshSection);

			GeometryCache.AABBox = ProceduralMeshHelper.CalcAABBox();
			ProceduralMeshHelper.CalcSurfaceVectors(GeometryCache.SurfaceViewNormal, GeometryCache.SurfaceViewPlane);

			bIsGeometryCacheValid = true;

			return true;
		}
	}

	bIsGeometryCacheValid = false;

	return false;
}

bool FDisplayClusterWarpBlend_GeometryProxy::ImplUpdateFrustumGeometryCache_WarpMap()
{

	if (WarpMapTexture.IsValid() && WarpMapTexture->IsEnabled())
	{
		if (bIsGeometryCacheValid)
		{
			// use cached values
			return true;
		}

		// Update cache
		FDisplayClusterWarpBlendMath_WarpMap DataHelper(WarpMapTexture.ToSharedRef());
		if (DataHelper.CanBeUsed())
		{
			GeometryCache.AABBox = DataHelper.GetAABBox();
			GeometryCache.SurfaceViewNormal = DataHelper.GetSurfaceViewNormal();
			GeometryCache.SurfaceViewPlane = DataHelper.GetSurfaceViewPlane();

			bIsGeometryCacheValid = true;

			return true;
		}
	}

	bIsGeometryCacheValid = false;

	return false;
}

bool FDisplayClusterWarpBlend_GeometryProxy::ImplUpdateFrustumGeometryCache_MPCDIAttributes()
{
	switch (MPCDIAttributes.ProfileType)
	{
	case EDisplayClusterWarpProfileType::warp_2D:
		if (bIsGeometryCacheValid)
		{
			// use cached value
			return true;
		}

		// Update cached value
		GeometryCache.SurfaceViewNormal = FVector(1, 0, 0);
		GeometryCache.SurfaceViewPlane = FVector(1, 0, 0);

		// Calc AABB for 2D profile geometry:
		{
			TArray<FVector> ScreenPoints;
			FDisplayClusterWarpBlendExporter_WarpMap::Get2DProfileGeometry(MPCDIAttributes, ScreenPoints);

			FDisplayClusterWarpAABB WarpAABB;
			WarpAABB.UpdateAABB(ScreenPoints);

			GeometryCache.AABBox = WarpAABB;
		}

		bIsGeometryCacheValid = true;

		return true;

	default:
		break;
	}

	bIsGeometryCacheValid = false;

	return false;
}

bool FDisplayClusterWarpBlend_GeometryProxy::UpdateFrustumGeometryLOD(const FIntPoint& InSizeLOD)
{
	check(InSizeLOD.X > 0 && InSizeLOD.Y > 0);

	GeometryCache.IndexLOD.Empty();

	if (WarpMapTexture.IsValid() && WarpMapTexture->IsEnabled())
	{
		switch (FrustumGeometryType)
		{
		case EDisplayClusterWarpFrustumGeometryType::WarpMap:
		{
			// Generate valid points for texturebox method:
			return FDisplayClusterWarpBlendMath_WarpMap(WarpMapTexture.ToSharedRef()).BuildIndexLOD(InSizeLOD, GeometryCache.IndexLOD);
		}
		default:
			break;
		}
	}

	return false;
}

const FStaticMeshLODResources* FDisplayClusterWarpBlend_GeometryProxy::GetStaticMeshComponentLODResources() const
{
	return WarpMeshComponent.IsValid() ? WarpMeshComponent->GetStaticMeshComponentLODResources(StaticMeshComponentLODIndex) : nullptr;
}

const FProcMeshSection* FDisplayClusterWarpBlend_GeometryProxy::GetProceduralMeshComponentSection() const
{
	return WarpMeshComponent.IsValid() ? WarpMeshComponent->GetProceduralMeshComponentSection(ProceduralMeshComponentSectionIndex) : nullptr;
}

void FDisplayClusterWarpBlend_GeometryProxy::ReleaseResources()
{
	WarpMapTexture.Reset();
	AlphaMapTexture.Reset();
	BetaMapTexture.Reset();
	WarpMeshComponent.Reset();
	PreviewMeshComponentRef.ResetSceneComponent();
}
