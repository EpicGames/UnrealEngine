// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/DisplayClusterWarpContainers.h"
#include "Render/Containers/IDisplayClusterRender_MeshComponent.h"
#include "Render/Containers/IDisplayClusterRender_Texture.h"
#include "Misc/DisplayClusterObjectRef.h"

class IDisplayClusterRender_MeshComponentProxy;
struct FStaticMeshLODResources;
struct FProcMeshSection;

/**
 * Geometry container for warp
 */
class FDisplayClusterWarpBlend_GeometryProxy
{
public:
	FDisplayClusterWarpBlend_GeometryProxy()
	{ }

	~FDisplayClusterWarpBlend_GeometryProxy()
	{
		ReleaseResources();
	}

public:
	/** Get texture resource by type. */
	const TSharedPtr<IDisplayClusterRender_Texture, ESPMode::ThreadSafe> ImplGetTexture(EDisplayClusterWarpBlendTextureType TextureType) const
	{
		switch (TextureType)
		{
		case EDisplayClusterWarpBlendTextureType::WarpMap:
			return WarpMapTexture;

		case EDisplayClusterWarpBlendTextureType::AlphaMap:
			return AlphaMapTexture;

		case EDisplayClusterWarpBlendTextureType::BetaMap:
			return BetaMapTexture;

		default:
			break;
		}
		return nullptr;
	}

	bool MarkWarpFrustumGeometryComponentDirty(const FName& InComponentName);

	bool UpdateFrustumGeometry();
	bool UpdateFrustumGeometryLOD(const FIntPoint& InSizeLOD);

	const IDisplayClusterRender_MeshComponentProxy* GetWarpMeshProxy_RenderThread() const;

	const FStaticMeshLODResources* GetStaticMeshComponentLODResources() const;
	const FProcMeshSection* GetProceduralMeshComponentSection() const;

private:
	bool ImplUpdateFrustumGeometry_WarpMesh();
	bool ImplUpdateFrustumGeometry_WarpProceduralMesh();
	bool ImplUpdateFrustumGeometry_WarpMap();
	bool ImplUpdateFrustumGeometry_MPCDIAttributes();

public:
	bool bIsGeometryCacheValid = false;
	bool bIsGeometryValid = false;

	// Geometry source for render
	EDisplayClusterWarpGeometryType GeometryType = EDisplayClusterWarpGeometryType::Invalid;

	// Geometry source for frustum
	EDisplayClusterWarpFrustumGeometryType FrustumGeometryType = EDisplayClusterWarpFrustumGeometryType::Invalid;
	
	void ReleaseResources();

public:
	// MPCDI attributes
	FDisplayClusterWarpMPCDIAttributes MPCDIAttributes;

	// Render resources:
	TSharedPtr<IDisplayClusterRender_Texture, ESPMode::ThreadSafe> WarpMapTexture;
	TSharedPtr<IDisplayClusterRender_Texture, ESPMode::ThreadSafe> AlphaMapTexture;
	TSharedPtr<IDisplayClusterRender_Texture, ESPMode::ThreadSafe> BetaMapTexture;

	TSharedPtr<IDisplayClusterRender_MeshComponent, ESPMode::ThreadSafe> WarpMeshComponent;
	FDisplayClusterSceneComponentRef PreviewMeshComponentRef;

	FDisplayClusterMeshUVs WarpMeshUVs;

	// for huge warp mesh geometry, change this value, and use LOD geometry in frustum math
	int32 StaticMeshComponentLODIndex = 0;
	int32 ProceduralMeshComponentSectionIndex = 0;

	float AlphaMapEmbeddedGamma = 1.f;

private:
	bool bIsMeshComponentLost = false;

private:
	bool ImplUpdateFrustumGeometryCache_WarpMesh();
	bool ImplUpdateFrustumGeometryCache_WarpProceduralMesh();
	bool ImplUpdateFrustumGeometryCache_WarpMap();
	bool ImplUpdateFrustumGeometryCache_MPCDIAttributes();

	inline EDisplayClusterWarpProfileType  GetWarpProfileType() const
	{
		return MPCDIAttributes.ProfileType;
	}

public:
	// Cached values for geometry (updated only if geometry changed)
	class FDisplayClusterWarpBlend_GeometryCache
	{
	public:
		FTransform GeometryToOrigin;

		FBox       AABBox;
		FVector    SurfaceViewNormal;
		FVector    SurfaceViewPlane;

		TArray<int32> IndexLOD;
	};

	FDisplayClusterWarpBlend_GeometryCache GeometryCache;
};
