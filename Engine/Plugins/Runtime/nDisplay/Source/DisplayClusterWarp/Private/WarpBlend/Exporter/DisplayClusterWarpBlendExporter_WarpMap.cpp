// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterWarpBlendExporter_WarpMap.h"

#include "WarpBlend/DisplayClusterWarpBlend_GeometryContext.h"
#include "Blueprints/DisplayClusterWarpGeometry.h"

namespace UE::DisplayClusterWarp::PFM
{
	// Export geometry from texture (a3 profile)
	static bool ExportProfileA3(const TSharedPtr<IDisplayClusterRender_Texture, ESPMode::ThreadSafe>& InWarpMap, struct FDisplayClusterWarpGeometryOBJ& Dst, uint32 InMaxDimension)
	{
		if (!InWarpMap.IsValid() || !InWarpMap->IsEnabled())
		{
			return false;
		}

		const uint32 Width = InWarpMap->GetWidth();
		const uint32 Height = InWarpMap->GetHeight();

		const FVector4f* WarpData = (FVector4f*)InWarpMap->GetData();

		uint32 DownScaleFactor = 1;

		if (InMaxDimension > 0)
		{
			uint32 MeshDimension = FMath::Max(Width, Height);
			if (MeshDimension > InMaxDimension)
			{
				DownScaleFactor = FMath::RoundToInt(float(MeshDimension) / InMaxDimension);
			}
		}


		TMap<int32, int32> VIndexMap;
		int32 VIndex = 0;

		const uint32 MaxHeight = Height / DownScaleFactor;
		const uint32 MaxWidth = Width / DownScaleFactor;

		{
			//Pts + Normals + UV
			const float ScaleU = 1.0f / float(MaxWidth);
			const float ScaleV = 1.0f / float(MaxHeight);

			for (uint32 j = 0; j < MaxHeight; ++j)
			{
				const uint32 MeshY = (j == (MaxHeight - 1)) ? Height : (j * DownScaleFactor);
				for (uint32 i = 0; i < MaxWidth; ++i)
				{
					const uint32 MeshX = (i == (MaxWidth - 1)) ? Width : (i * DownScaleFactor);

					const int32 SrcIdx = MeshX + (MeshY)*Width;
					const FVector4f& v = WarpData[SrcIdx];
					if (v.W > 0)
					{
						Dst.Vertices.Add(FVector(v.X, v.Y, v.Z));
						VIndexMap.Add(SrcIdx, VIndex++);

						Dst.UV.Add(FVector2D(
							float(i) * ScaleU,
							float(j) * ScaleV
						));

						Dst.Normal.Add(FVector(0, 0, 0)); // Fill on face pass
					}
				}
			}
		}

		{
			//faces
			for (uint32 j = 0; j < MaxHeight - 1; ++j)
			{
				const uint32 MeshY = (j == (MaxHeight - 1)) ? Height : (j * DownScaleFactor);
				const uint32 NextMeshY = ((j + 1) == (MaxHeight - 1)) ? Height : ((j + 1) * DownScaleFactor);

				for (uint32 i = 0; i < MaxWidth - 1; ++i)
				{
					const uint32 MeshX = (i == (MaxWidth - 1)) ? Width : (i * DownScaleFactor);
					const uint32 NextMeshX = ((i + 1) == (MaxWidth - 1)) ? Width : ((i + 1) * DownScaleFactor);

					int32 idx[4];

					idx[0] = (MeshX + MeshY * Width);
					idx[1] = (NextMeshX + MeshY * Width);
					idx[2] = (MeshX + NextMeshY * Width);
					idx[3] = (NextMeshX + NextMeshY * Width);

					for (int32 a = 0; a < 4; a++)
					{
						if (VIndexMap.Contains(idx[a]))
						{
							idx[a] = VIndexMap[idx[a]];
						}
						else
						{
							idx[a] = -1;
						}
					}

					if (idx[0] >= 0 && idx[2] >= 0 && idx[3] >= 0)
					{
						Dst.PostAddFace(idx[0], idx[2], idx[3]);
					}

					if (idx[3] >= 0 && idx[1] >= 0 && idx[0] >= 0)
					{
						Dst.PostAddFace(idx[3], idx[1], idx[0]);
					}
				}
			}
		}

		return true;
	};
};
using namespace UE::DisplayClusterWarp::PFM;

//---------------------------------------------------------------
// FDisplayClusterWarpBlendExporter_WarpMap
//---------------------------------------------------------------
void FDisplayClusterWarpBlendExporter_WarpMap::Get2DProfileGeometry(const FDisplayClusterWarpMPCDIAttributes& InAttributes, TArray<FVector>& OutGeometryPoints, TArray<FVector>* OutNormal, TArray<FVector2D>* OutUV)
{
	FVector ScreenPosition;
	FVector2D ScreenSize;
	if (!InAttributes.CalcProfile2DScreen(ScreenPosition, ScreenSize))
	{
		return;
	}

	// Rectangle stands on the floor, centered at zero
	const float X = ScreenPosition.X;
	const float Y0 = ScreenPosition.Y - (ScreenSize.X * 0.5f);
	const float Y1 = ScreenPosition.Y + (ScreenSize.X * 0.5f);
	const float Z0 = ScreenPosition.Z - (ScreenSize.Y * 0.5f);
	const float Z1 = ScreenPosition.Z + (ScreenSize.Y * 0.5f);

	// Create a vertices
	// Y - right, Z - Up
	OutGeometryPoints.Add(FVector(X, Y0, Z1));
	OutGeometryPoints.Add(FVector(X, Y1, Z1));
	OutGeometryPoints.Add(FVector(X, Y0, Z0));
	OutGeometryPoints.Add(FVector(X, Y1, Z0));

	if (OutNormal)
	{
		OutNormal->Add(FVector(-1, 0, 0)); // Fill on face pass
		OutNormal->Add(FVector(-1, 0, 0)); // Fill on face pass
		OutNormal->Add(FVector(-1, 0, 0)); // Fill on face pass
		OutNormal->Add(FVector(-1, 0, 0)); // Fill on face pass
	}

	if (OutUV)
	{
		OutUV->Add(FVector2D(0, 0));
		OutUV->Add(FVector2D(1, 0));
		OutUV->Add(FVector2D(0, 1));
		OutUV->Add(FVector2D(1, 1));
	}
}

bool FDisplayClusterWarpBlendExporter_WarpMap::ExportWarpMap(const FDisplayClusterWarpBlend_GeometryContext& InContext, struct FDisplayClusterWarpGeometryOBJ& Dst, uint32 InMaxDimension)
{
	switch (InContext.GetWarpProfileType())
	{
	case EDisplayClusterWarpProfileType::warp_2D:
		FDisplayClusterWarpBlendExporter_WarpMap::Get2DProfileGeometry(InContext.GeometryProxy.MPCDIAttributes, Dst.Vertices, &Dst.Normal, &Dst.UV);

		//Create a faces:
		Dst.PostAddFace(0, 2, 3);
		Dst.PostAddFace(3, 1, 0);

		return true;

	case EDisplayClusterWarpProfileType::warp_3D:
		// NOT IMPLEMENTED
		return false;

	case EDisplayClusterWarpProfileType::warp_A3D:
		return InContext.GeometryProxy.WarpMapTexture.IsValid() ? ExportProfileA3(InContext.GeometryProxy.WarpMapTexture, Dst, InMaxDimension) : false;

	case EDisplayClusterWarpProfileType::warp_SL:
		// NOT IMPLEMENTED
		return false;

	default:
		break;
	}

	return false;
}
