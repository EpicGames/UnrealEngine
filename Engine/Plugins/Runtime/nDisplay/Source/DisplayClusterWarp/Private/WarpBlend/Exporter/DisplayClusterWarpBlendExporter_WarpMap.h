// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/** Auxiliary class for exporting WarpMap geometry. */
class FDisplayClusterWarpBlendExporter_WarpMap
{
public:
	/**
	 * Exprot WarpMap texture geometry into the memory structure.
	 * For a 2D profile the geometry does not exist and is created based on the definition of <buffer> and <region>.
	 */
	static bool ExportWarpMap(const class FDisplayClusterWarpBlend_GeometryContext& InContext, struct FDisplayClusterWarpGeometryOBJ& Dst, uint32 InMaxDimension = 0);

	/** [2D] Return fake geometry points for region in mpcdi 2D profile. */
	static void Get2DProfileGeometry(const struct FDisplayClusterWarpMPCDIAttributes& InMPCDIAttributes, TArray<FVector>& OutGeometryPoints, TArray<FVector>* OutNormal = nullptr, TArray<FVector2D>* OutUV = nullptr);
};
