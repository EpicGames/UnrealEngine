// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterWarpBlendLoader_MPCDI.h"
#include "DisplayClusterWarpBlendLoader_MeshComponent.h"

#include "WarpBlend/DisplayClusterWarpBlend.h"
#include "WarpBlend/DisplayClusterWarpBlend_GeometryContext.h"
#include "WarpBlend/DisplayClusterWarpBlend_GeometryProxy.h"

#include "WarpBlend/Loader/DisplayClusterWarpBlendLoader_MPCDIFileLoader.h"
#include "WarpBlend/Loader/DisplayClusterWarpBlendLoader_MPCDIRegionLoader.h"

#include "DisplayClusterWarpLog.h"

TSharedPtr<FDisplayClusterWarpBlend, ESPMode::ThreadSafe> FDisplayClusterWarpBlendLoader_MPCDI::Create(const FDisplayClusterWarpInitializer_MPCDIFile& InConstructParameters)
{
	FDisplayClusterWarpBlendMPCDIRegionLoader MPCDIRegionLoader;
	if(MPCDIRegionLoader.LoadRegionFromMPCDIFile(InConstructParameters.MPCDIFileName, InConstructParameters.BufferId, InConstructParameters.RegionId))
	{
		//ok, Create and initialize warpblend interface
		return MPCDIRegionLoader.CreateWarpBlendInterface();
	}

	UE_LOGF(LogDisplayClusterWarpBlend, Error, "Can't load region '%ls' buffer '%ls' from mpcdi file '%ls'", *InConstructParameters.RegionId, *InConstructParameters.BufferId, *InConstructParameters.MPCDIFileName);

	return nullptr;
}

bool FDisplayClusterWarpBlendLoader_MPCDI::ReadMPCDFileStructure(const FString& InMPCDIFileName, TMap<FString, TMap<FString, FDisplayClusterWarpMPCDIAttributes>>& OutMPCDIFileStructure)
{
	TSharedPtr<FDisplayClusterWarpBlendMPCDIFileLoader, ESPMode::ThreadSafe> FileLoader = FDisplayClusterWarpBlendMPCDIFileLoader::GetOrCreateCachedMPCDILoader(InMPCDIFileName);
	if (FileLoader.IsValid())
	{
		return FileLoader->ReadMPCDFileStructure(OutMPCDIFileStructure);
	}

	UE_LOGF(LogDisplayClusterWarpBlend, Error, "Can't load MPCDI file structure from mpcdi file '%ls'", *InMPCDIFileName);

	return false;
}

TSharedPtr<FDisplayClusterWarpBlend, ESPMode::ThreadSafe> FDisplayClusterWarpBlendLoader_MPCDI::Create(const FDisplayClusterWarpInitializer_PFMFile& InConstructParameters)
{
	FDisplayClusterWarpBlendMPCDIRegionLoader RegionLoader;

	// Filling all internal data manually:
	RegionLoader.AlphaMapGammaEmbedded = InConstructParameters.AlphaMapEmbeddedAlpha;
	RegionLoader.MPCDIAttributes = InConstructParameters.MPCDIAttributes;

	RegionLoader.WarpMap = RegionLoader.CreateTextureFromPFMFile(InConstructParameters.PFMFileName, InConstructParameters.PFMScale, InConstructParameters.bIsUnrealGameSpace);
	if (RegionLoader.WarpMap.IsValid())
	{
		if (InConstructParameters.AlphaMapFileName.IsEmpty() == false)
		{
			RegionLoader.AlphaMap = RegionLoader.CreateTextureFromFile(InConstructParameters.AlphaMapFileName);
		}

		if (InConstructParameters.BetaMapFileName.IsEmpty() == false)
		{
			RegionLoader.BetaMap = RegionLoader.CreateTextureFromFile(InConstructParameters.BetaMapFileName);
		}

		// After all internal data are filled in, the warpblend interface is created and initialized.
		return RegionLoader.CreateWarpBlendInterface();
	}

	UE_LOGF(LogDisplayClusterWarpBlend, Error, "Can't load PFM file '%ls'", *InConstructParameters.PFMFileName);

	return nullptr;
}

TSharedPtr<FDisplayClusterWarpBlend, ESPMode::ThreadSafe> FDisplayClusterWarpBlendLoader_MPCDI::Create(const FDisplayClusterWarpInitializer_MPCDIFile_Profile2DScreen& InConstructParameters)
{
	// Create from file
	const FDisplayClusterWarpInitializer_MPCDIFile& InitializerFromFile = InConstructParameters;
	TSharedPtr<FDisplayClusterWarpBlend, ESPMode::ThreadSafe> WarpBlend = Create(InitializerFromFile);

	if (WarpBlend.IsValid() && WarpBlend->GetWarpProfileType() == EDisplayClusterWarpProfileType::warp_2D)
	{
		// Extra setup for 2D profile:
		FDisplayClusterWarpBlend_GeometryProxy& Proxy = WarpBlend->GeometryContext.GeometryProxy;

		FDisplayClusterMeshUVs MeshUVs;

		Proxy.PreviewMeshComponentRef.SetSceneComponent(InConstructParameters.PreviewMeshComponent);

		Proxy.WarpMeshComponent = FDisplayClusterWarpBlendLoader_MeshComponent::CreateMeshComponent();
		Proxy.WarpMeshComponent->AssignStaticMeshComponentRefs(InConstructParameters.WarpMeshComponent, MeshUVs, InConstructParameters.OriginComponent, 0);

		Proxy.WarpMeshUVs = MeshUVs;

		// Use static mesh geometry for frustum calculations
		Proxy.FrustumGeometryType = EDisplayClusterWarpFrustumGeometryType::WarpMesh;
	}

	return WarpBlend;
}
