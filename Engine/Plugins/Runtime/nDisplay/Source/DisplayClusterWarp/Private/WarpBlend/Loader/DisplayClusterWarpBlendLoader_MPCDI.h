// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/DisplayClusterWarpInitializer.h"
#include "WarpBlend/DisplayClusterWarpBlend.h"

/**
 * An auxiliary class for creating the WarpBlend interface from initializing structures associated with MPCDI.
 */
class FDisplayClusterWarpBlendLoader_MPCDI
{
public:
	/** Create WarpBlend interface from the MPCDI file. */
	static TSharedPtr<FDisplayClusterWarpBlend, ESPMode::ThreadSafe> Create(const FDisplayClusterWarpInitializer_MPCDIFile& InConstructParameters);

	/** Create WarpBlend interface from the PFM file. */
	static TSharedPtr<FDisplayClusterWarpBlend, ESPMode::ThreadSafe> Create(const FDisplayClusterWarpInitializer_PFMFile& InConstructParameters);

	/** Create WarpBlend interface from MPCDI file and DisplayClusterScreen component. */
	static TSharedPtr<FDisplayClusterWarpBlend, ESPMode::ThreadSafe> Create(const FDisplayClusterWarpInitializer_MPCDIFile_Profile2DScreen& InConstructParameters);

	/**
	 * Read MPCDI file structure
	 *
	 * @param InMPCDIFileName        - (in)  filename of the MPCDI file
	 * @param OutMPCDIFileStructure  - (out) readed mpcdi structure in format: TMap<BufferId, TArray<RegionId, MPCDIAttributes>>
	 *
	 * @return true if success
	 */
	static bool ReadMPCDFileStructure(const FString& InMPCDIFileName, TMap<FString, TMap<FString, FDisplayClusterWarpMPCDIAttributes>>& OutMPCDIFileStructure);
};
