// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterWarpBlendLoader_MPCDIFileLoader.h"

THIRD_PARTY_INCLUDES_START
#include "mpcdiAlphaMap.h"
#include "mpcdiBetaMap.h"
#include "mpcdiBuffer.h"
#include "mpcdiDisplay.h"
#include "mpcdiDistortionMap.h"
#include "mpcdiGeometryWarpFile.h"
#include "mpcdiPfmIO.h"
#include "mpcdiProfile.h"
#include "mpcdiPNGReadWrite.h"
#include "mpcdiReader.h"
#include "mpcdiRegion.h"
THIRD_PARTY_INCLUDES_END

#include "DisplayClusterWarpLog.h"

#include "Misc/DisplayClusterDataCache.h"
#include "Misc/DisplayClusterHelpers.h"
#include "Misc/FileHelper.h"


int32 GDisplayClusterRender_MPCDILoaderCacheEnable = 1;
static FAutoConsoleVariableRef CDisplayClusterRender_MPCDILoaderCacheEnable(
	TEXT("nDisplay.cache.MPCDILoader.enable"),
	GDisplayClusterRender_MPCDILoaderCacheEnable,
	TEXT("Enables the use of the MPCDI file loaders cache.\n"),
	ECVF_Default
);

int32 GDisplayClusterRender_MPCDILoaderCacheTimeOutInFrames = 5 * 60 * 60; // Timeout is 5 minutes (for 60 frames per second)
static FAutoConsoleVariableRef CVarDisplayClusterRender_MPCDILoaderCacheTimeOutInFrames(
	TEXT("nDisplay.cache.MPCDILoader.TimeOut"),
	GDisplayClusterRender_MPCDILoaderCacheTimeOutInFrames,
	TEXT("The timeout value in frames  for cached MPCDI file loaders.\n")
	TEXT("-1 - disable timeout.\n"),
	ECVF_Default
);

//--------------------------------------------------------------------------
// FDisplayClusterWarpBlendMPCDIFileLoader
//--------------------------------------------------------------------------
FDisplayClusterWarpBlendMPCDIFileLoader::FDisplayClusterWarpBlendMPCDIFileLoader(const FString& InUniqueName, const FString& InFilePath)
	: UniqueName(InUniqueName)
{
	// Load MPCDI file
	{
		Profile = mpcdi::Profile::CreateProfile();
		if (!Profile)
		{
			return;
		}

		// Read profile data from mpcdi file using mpcdi lib:
		if (mpcdi::Reader* NewReader = mpcdi::Reader::CreateReader())
		{
			UE_LOGF(LogDisplayClusterWarpBlend, Log, "Loading MPCDI file %ls.", *InFilePath);

			std::string version = NewReader->GetSupportedVersions();
			FString Version = version.c_str();
			UE_LOGF(LogDisplayClusterWarpBlend, Verbose, "MPCDI library version: %ls", *Version);

			mpcdi::MPCDI_Error mpcdi_err = NewReader->Read(TCHAR_TO_ANSI(*InFilePath), Profile);
			delete NewReader;

			if (MPCDI_FAILED(mpcdi_err))
			{
				UE_LOGF(LogDisplayClusterWarpBlend, Error, "Error %d reading MPCDI file", int32(mpcdi_err));
				Release();
			}
		}
	}
}

FDisplayClusterWarpBlendMPCDIFileLoader::~FDisplayClusterWarpBlendMPCDIFileLoader()
{
	Release();
}

void FDisplayClusterWarpBlendMPCDIFileLoader::Release()
{
	if (Profile)
	{
		delete Profile;
		Profile = nullptr;
	}
}

bool FDisplayClusterWarpBlendMPCDIFileLoader::FindRegion(const FString& InBufferName, const FString& InRegionId, mpcdi::Buffer*& OutBufferData, mpcdi::Region*& OutRegionData) const
{
	if (IsValid())
	{
		// Find desired region:
		for (mpcdi::Display::BufferIterator itBuffer = Profile->GetDisplay()->GetBufferBegin(); itBuffer != Profile->GetDisplay()->GetBufferEnd(); ++itBuffer)
		{
			mpcdi::Buffer* mpcdiBuffer = itBuffer->second;

			FString BufferId(mpcdiBuffer->GetId().c_str());
			if (InBufferName.Equals(BufferId, ESearchCase::IgnoreCase))
			{
				for (mpcdi::Buffer::RegionIterator it = mpcdiBuffer->GetRegionBegin(); it != mpcdiBuffer->GetRegionEnd(); ++it)
				{
					mpcdi::Region* mpcdiRegion = it->second;
					FString RegionId(mpcdiRegion->GetId().c_str());

					if (InRegionId.Equals(RegionId, ESearchCase::IgnoreCase))
					{
						OutBufferData = mpcdiBuffer;
						OutRegionData = mpcdiRegion;

						return true;
					}
				}
			}
		}
	}

	return false;
}

bool FDisplayClusterWarpBlendMPCDIFileLoader::ReadMPCDFileStructure(TMap<FString, TMap<FString, FDisplayClusterWarpMPCDIAttributes>>& OutMPCDIFileStructure)
{
	if (IsValid())
	{
		// Find desired region:
		for (mpcdi::Display::BufferIterator itBuffer = Profile->GetDisplay()->GetBufferBegin(); itBuffer != Profile->GetDisplay()->GetBufferEnd(); ++itBuffer)
		{
			mpcdi::Buffer* mpcdiBuffer = itBuffer->second;

			FString BufferId(mpcdiBuffer->GetId().c_str());
			OutMPCDIFileStructure.Add(BufferId);
			{
				for (mpcdi::Buffer::RegionIterator it = mpcdiBuffer->GetRegionBegin(); it != mpcdiBuffer->GetRegionEnd(); ++it)
				{
					mpcdi::Region* mpcdiRegion = it->second;
					FString RegionId(mpcdiRegion->GetId().c_str());

					FDisplayClusterWarpMPCDIAttributes MPCDIAttributes;
					LoadMPCDIAttributes(mpcdiBuffer, mpcdiRegion, MPCDIAttributes);

					OutMPCDIFileStructure[BufferId].Emplace(RegionId, MPCDIAttributes);
				}
			}
		}

		return true;
	}

	return false;
}

void FDisplayClusterWarpBlendMPCDIFileLoader::LoadMPCDIAttributes(mpcdi::Buffer* mpcdiBuffer, mpcdi::Region* mpcdiRegion, FDisplayClusterWarpMPCDIAttributes& MPCDIAttributes) const
{
	// Read MPCDI attributes:
	MPCDIAttributes.ProfileType = GetProfileType();

	MPCDIAttributes.Buffer.Resolution.X = mpcdiBuffer->GetXresolution();
	MPCDIAttributes.Buffer.Resolution.Y = mpcdiBuffer->GetYresolution();

	MPCDIAttributes.Region.Resolution.X = mpcdiRegion->GetXresolution();
	MPCDIAttributes.Region.Resolution.Y = mpcdiRegion->GetYresolution();

	MPCDIAttributes.Region.Pos.X = mpcdiRegion->GetX();
	MPCDIAttributes.Region.Pos.Y = mpcdiRegion->GetY();
	MPCDIAttributes.Region.Size.X = mpcdiRegion->GetXsize();
	MPCDIAttributes.Region.Size.Y = mpcdiRegion->GetYsize();

	if (const mpcdi::Frustum* Frustum = mpcdiRegion->GetFrustum())
	{
		EnumAddFlags(MPCDIAttributes.Flags, EDisplayClusterWarpMPCDIAttributesFlags::HasFrustum);

		MPCDIAttributes.Frustum.Rotator.Pitch = Frustum->GetPitch();
		MPCDIAttributes.Frustum.Rotator.Yaw = Frustum->GetYaw();
		MPCDIAttributes.Frustum.Rotator.Roll = Frustum->GetRoll();

		// Frustum angles XYZW = LRTB
		MPCDIAttributes.Frustum.Angles.X = Frustum->GetLeftAngle();
		MPCDIAttributes.Frustum.Angles.Y = Frustum->GetRightAngle();
		MPCDIAttributes.Frustum.Angles.Z = Frustum->GetUpAngle();
		MPCDIAttributes.Frustum.Angles.W = Frustum->GetDownAngle();
	}

	if (mpcdi::CoordinateFrame* CoordinateFrame = mpcdiRegion->GetCoordinateFrame())
	{
		EnumAddFlags(MPCDIAttributes.Flags, EDisplayClusterWarpMPCDIAttributesFlags::HasCoordinateFrame);

		MPCDIAttributes.CoordinateFrame.Pos = FVector(CoordinateFrame->GetPosx(), CoordinateFrame->GetPosy(), CoordinateFrame->GetPosz());
		MPCDIAttributes.CoordinateFrame.Yaw = FVector(CoordinateFrame->GetYawx(), CoordinateFrame->GetYawy(), CoordinateFrame->GetYawz());
		MPCDIAttributes.CoordinateFrame.Pitch = FVector(CoordinateFrame->GetPitchx(), CoordinateFrame->GetPitchy(), CoordinateFrame->GetPitchz());
		MPCDIAttributes.CoordinateFrame.Roll = FVector(CoordinateFrame->GetRollx(), CoordinateFrame->GetRolly(), CoordinateFrame->GetRollz());
	}
}

EDisplayClusterWarpProfileType FDisplayClusterWarpBlendMPCDIFileLoader::GetProfileType() const
{
	if (Profile)
	{
		switch (Profile->GetProfileType())
		{
		case mpcdi::ProfileType2d: return EDisplayClusterWarpProfileType::warp_2D;
		case mpcdi::ProfileType3d: return EDisplayClusterWarpProfileType::warp_3D;
		case mpcdi::ProfileTypea3: return EDisplayClusterWarpProfileType::warp_A3D;
		case mpcdi::ProfileTypesl: return EDisplayClusterWarpProfileType::warp_SL;
		default:
			// Invalid profile type
			break;
		}
	}

	return EDisplayClusterWarpProfileType::Invalid;
}

FString FDisplayClusterWarpBlendMPCDIFileLoader::GetFullPathToFile(const FString& InFilePath)
{
	// Fix relative paths
	FString OutFileName = InFilePath;
	if (FPaths::IsRelative(InFilePath))
	{
		OutFileName = DisplayClusterHelpers::filesystem::GetFullPathForConfigResource(InFilePath);
	}

	if (!FPaths::FileExists(OutFileName))
	{
		UE_LOGF(LogDisplayClusterWarpBlend, Error, "file '%ls' not found", *OutFileName);

		OutFileName.Empty();
	}

	return OutFileName;
}

int32 FDisplayClusterWarpBlendMPCDIFileLoader::GetDataCacheTimeOutInFrames()
{
	return FMath::Max(0, GDisplayClusterRender_MPCDILoaderCacheTimeOutInFrames);
}

bool FDisplayClusterWarpBlendMPCDIFileLoader::IsDataCacheEnabled()
{
	return GDisplayClusterRender_MPCDILoaderCacheEnable != 0;
}

/**
 * The cache for MPCDI file loader objects. (Singleton)
 */
class FDisplayClusterWarpBlendMPCDIFileLoaderCache
	: public TDisplayClusterDataCache<FDisplayClusterWarpBlendMPCDIFileLoader>
{
public:
	static TSharedPtr<FDisplayClusterWarpBlendMPCDIFileLoader, ESPMode::ThreadSafe> GetOrCreateMPCDIFileLoader(const FString& InFilePath)
	{
		static FDisplayClusterWarpBlendMPCDIFileLoaderCache MPCDIFileLoaderCacheSingleton;

		const FString FullPath = FDisplayClusterWarpBlendMPCDIFileLoader::GetFullPathToFile(InFilePath);
		if (FullPath.IsEmpty())
		{
			return nullptr;
		}

		const FString UniqueName = HashString(FullPath);

		TSharedPtr<FDisplayClusterWarpBlendMPCDIFileLoader, ESPMode::ThreadSafe> TextureRef = MPCDIFileLoaderCacheSingleton.Find(UniqueName);
		if (!TextureRef.IsValid())
		{
			TextureRef = MakeShared<FDisplayClusterWarpBlendMPCDIFileLoader, ESPMode::ThreadSafe>(UniqueName, FullPath);
			MPCDIFileLoaderCacheSingleton.Add(TextureRef);
		}

		return TextureRef;
	}
};

TSharedPtr<FDisplayClusterWarpBlendMPCDIFileLoader, ESPMode::ThreadSafe> FDisplayClusterWarpBlendMPCDIFileLoader::GetOrCreateCachedMPCDILoader(const FString& InFilePath)
{
	return FDisplayClusterWarpBlendMPCDIFileLoaderCache::GetOrCreateMPCDIFileLoader(InFilePath);
}
