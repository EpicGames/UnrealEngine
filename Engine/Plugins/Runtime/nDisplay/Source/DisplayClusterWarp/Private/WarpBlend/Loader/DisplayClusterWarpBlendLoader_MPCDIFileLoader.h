// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/DisplayClusterWarpEnums.h"
#include "Containers/DisplayClusterWarpContainers.h"

namespace mpcdi
{
	struct Profile;
	struct Buffer;
	struct Region;
};

/**
 * Loader and container for mpcdi file.
 */
class FDisplayClusterWarpBlendMPCDIFileLoader
{
public:
	FDisplayClusterWarpBlendMPCDIFileLoader(const FString& InUniqueName, const FString& InFilePath);
	~FDisplayClusterWarpBlendMPCDIFileLoader();

public:
	/** Get unique name of this file loader. Generated from file name. */
	const FString& GetName() const
	{
		return UniqueName;
	}
	
	/** Find MPCDI buffer and region by name.
	 * @param InBufferName - buffer name from the xml file inside the mpcdi file
	 * @param InRegionId - region name under the buffer name from the xml file inside the mpcdi file
	 * @param OutBufferData - (out) loaded buffer data
	 * @param OutRegionData - (out) loaded region data
	 * 
	 * @return false if no buffer or region exists
	 */
	bool FindRegion(const FString& InBufferName, const FString& InRegionId, mpcdi::Buffer*& OutBufferData, mpcdi::Region*& OutRegionData) const;

	/**
	 * Read MPCDI file structure
	 *
	 * @param OutMPCDIFileStructure  - (out) readed mpcdi structure in format: TMap<BufferId, TArray<RegionId, MPCDIAttributes>>
	 *
	 * @return true if success
	 */
	bool ReadMPCDFileStructure(TMap<FString, TMap<FString, FDisplayClusterWarpMPCDIAttributes>>& OutMPCDIFileStructure);
	
	/** True if this data valid. */
	bool IsValid() const
	{
		return Profile != nullptr;
	}

	/**
	 * Load attributes
	 */
	void LoadMPCDIAttributes(mpcdi::Buffer* mpcdiBuffer, mpcdi::Region* mpcdiRegion, FDisplayClusterWarpMPCDIAttributes& OutMPCDIAttributes) const;

	/** Get mpcdi profile type. */
	EDisplayClusterWarpProfileType GetProfileType() const;

	/** Convert full file path from relative path to mpcdi files. */
	static FString GetFullPathToFile(const FString& InFilePath);

	/** Get or create an MPCDI loader in the cache. */
	static TSharedPtr<FDisplayClusterWarpBlendMPCDIFileLoader, ESPMode::ThreadSafe> GetOrCreateCachedMPCDILoader(const FString& InFilePath);


	//~ Begin TDisplayClusterDataCache
	/** Return DataCache timeout in frames. */
	static int32 GetDataCacheTimeOutInFrames();

	/** Return true if DataCache is enabled. */
	static bool IsDataCacheEnabled();

	/** Returns the unique name of this MPCDI file resource for DataCache. */
	inline const FString& GetDataCacheName() const
	{
		return UniqueName;
	}

	/** Method for releasing a cached data item, called before its destructor. */
	inline void ReleaseDataCacheItem()
	{ }
	// ~~ End TDisplayClusterDataCache

protected:
	void Release();

private:
	// Unique name generated from the filename.
	const FString UniqueName;

	// loaded mpcdi data.
	mpcdi::Profile* Profile = nullptr;
};
