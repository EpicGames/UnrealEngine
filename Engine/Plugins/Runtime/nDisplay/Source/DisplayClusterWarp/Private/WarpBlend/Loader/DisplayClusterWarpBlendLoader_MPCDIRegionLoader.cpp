// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterWarpBlendLoader_MPCDIRegionLoader.h"

#include "WarpBlend/DisplayClusterWarpBlend.h"
#include "WarpBlend/DisplayClusterWarpBlend_GeometryContext.h"
#include "WarpBlend/DisplayClusterWarpBlend_GeometryProxy.h"

#include "WarpBlend/Loader/DisplayClusterWarpBlendLoader_MPCDIFileLoader.h"
#include "WarpBlend/Loader/DisplayClusterWarpBlendLoader_WarpMap.h"

#include "DisplayClusterWarpLog.h"

#include "IDisplayCluster.h"
#include "Render/IDisplayClusterRenderManager.h"

THIRD_PARTY_INCLUDES_START
#include "mpcdiAlphaMap.h"
#include "mpcdiBetaMap.h"
#include "mpcdiBuffer.h"
#include "mpcdiDisplay.h"
#include "mpcdiDistortionMap.h"
#include "mpcdiGeometryWarpFile.h"
#include "mpcdiPfmIO.h"
#include "mpcdiProfile.h"
#include "mpcdiPNGReadWrite.h"
#include "mpcdiReader.h"
#include "mpcdiRegion.h"
THIRD_PARTY_INCLUDES_END

namespace UE::DisplayClusterWarp::MPCDIRegionLoader
{
	static inline TSharedPtr<IDisplayClusterRender_Texture, ESPMode::ThreadSafe> GetOrCreateCachedTextureImpl(const FString& InUniqueTextureName)
	{
		static IDisplayCluster& DisplayClusterAPI = IDisplayCluster::Get();

		return DisplayClusterAPI.GetRenderMgr()->GetOrCreateCachedTexture(InUniqueTextureName);
	}

	static inline void InitializeTextureFromDataMapImpl(TSharedPtr<IDisplayClusterRender_Texture, ESPMode::ThreadSafe>& InTexture, mpcdi::DataMap* InDataMap)
	{
		check(InDataMap);

		if (InTexture.IsValid() && !InTexture->IsEnabled())
		{
			// new resource, initialize once
			const int32 ComponentDepth = InDataMap->GetComponentDepth();
			const int32 BitDepth = InDataMap->GetBitDepth();
			const uint32_t Width = InDataMap->GetSizeX();
			const uint32_t Height = InDataMap->GetSizeY();
			const void* TextureData = reinterpret_cast<void*>(InDataMap->GetData()->data());
			const bool bHasCPUAccess = false;

			InTexture->CreateTexture(TextureData, ComponentDepth, BitDepth, Width, Height, bHasCPUAccess);
		}
	}
};
using namespace UE::DisplayClusterWarp::MPCDIRegionLoader;

//--------------------------------------------------------------
// FDisplayClusterWarpBlendMPCDIRegionLoader
//--------------------------------------------------------------
TSharedPtr<FDisplayClusterWarpBlend, ESPMode::ThreadSafe> FDisplayClusterWarpBlendMPCDIRegionLoader::CreateWarpBlendInterface()
{
	TSharedPtr<FDisplayClusterWarpBlend, ESPMode::ThreadSafe> WarpBlend = MakeShared<FDisplayClusterWarpBlend, ESPMode::ThreadSafe>();

	WarpBlend->GeometryContext.GeometryProxy.GeometryType = EDisplayClusterWarpGeometryType::WarpMap;
	WarpBlend->GeometryContext.GeometryProxy.MPCDIAttributes = MPCDIAttributes;

	// Frustum geometry source
	switch (MPCDIAttributes.ProfileType)
	{
	case EDisplayClusterWarpProfileType::warp_2D:
	case EDisplayClusterWarpProfileType::warp_3D:
		// Get frustum from attributes
		WarpBlend->GeometryContext.GeometryProxy.FrustumGeometryType = EDisplayClusterWarpFrustumGeometryType::MPCDIAttributes;
		break;

	default:
		WarpBlend->GeometryContext.GeometryProxy.FrustumGeometryType = EDisplayClusterWarpFrustumGeometryType::WarpMap;
		break;
	}

	FDisplayClusterWarpBlend_GeometryProxy& Proxy = WarpBlend->GeometryContext.GeometryProxy;
	Proxy.WarpMapTexture = WarpMap;
	Proxy.AlphaMapEmbeddedGamma = AlphaMapGammaEmbedded;
	Proxy.AlphaMapTexture = AlphaMap;
	Proxy.BetaMapTexture = BetaMap;

	ReleaseResources();

	return WarpBlend;
}

void FDisplayClusterWarpBlendMPCDIRegionLoader::ReleaseResources()
{
	AlphaMap.Reset();
	BetaMap.Reset();
	WarpMap.Reset();
}

TSharedPtr<IDisplayClusterRender_Texture, ESPMode::ThreadSafe> FDisplayClusterWarpBlendMPCDIRegionLoader::CreateTextureFromDataMap(const FString& InUniqueTextureName, mpcdi::DataMap* InDataMap)
{
	check(InDataMap);

	TSharedPtr<IDisplayClusterRender_Texture, ESPMode::ThreadSafe> Texture = GetOrCreateCachedTextureImpl(InUniqueTextureName);
	InitializeTextureFromDataMapImpl(Texture, InDataMap);

	return  Texture.IsValid() && Texture->IsEnabled() ? Texture : nullptr;
}

TSharedPtr<IDisplayClusterRender_Texture, ESPMode::ThreadSafe> FDisplayClusterWarpBlendMPCDIRegionLoader::CreateTextureFromGeometry(const FString& InUniqueTextureName, mpcdi::GeometryWarpFile* InGeometryWarp)
{
	TSharedPtr<IDisplayClusterRender_Texture, ESPMode::ThreadSafe> Texture = GetOrCreateCachedTextureImpl(InUniqueTextureName);
	if (Texture && !Texture->IsEnabled())
	{
		// new resource, initialize once
		FDisplayClusterWarpBlendLoader WarpMapData;
		if (WarpMapData.LoadFromGeometryWarpFile(GetWarpProfileType(), InGeometryWarp))
		{
			Texture->CreateTexture(WarpMapData.GetWarpData(), 4, 32, WarpMapData.GetWidth(), WarpMapData.GetHeight(), true);
		}
	}

	return  Texture.IsValid() && Texture->IsEnabled() ? Texture : nullptr;
}

TSharedPtr<IDisplayClusterRender_Texture, ESPMode::ThreadSafe> FDisplayClusterWarpBlendMPCDIRegionLoader::CreateTextureFromFile(const FString& InFileName)
{
	// new resource, initialize once
	const FString TextureFileFullPath = FDisplayClusterWarpBlendMPCDIFileLoader::GetFullPathToFile(InFileName);
	if (TextureFileFullPath.IsEmpty())
	{
		return nullptr;
	}

	TSharedPtr<IDisplayClusterRender_Texture, ESPMode::ThreadSafe> Texture = GetOrCreateCachedTextureImpl(TextureFileFullPath);
	if (Texture && !Texture->IsEnabled())
	{
		// Currently, only the PNG file format is supported.
		const std::string PNGFileName = TCHAR_TO_ANSI(*TextureFileFullPath);
		mpcdi::DataMap* PngData;
		mpcdi::MPCDI_Error res = mpcdi::PNGReadWrite::Read(PNGFileName, PngData);
		if (mpcdi::MPCDI_SUCCESS == res && PngData)
		{
			InitializeTextureFromDataMapImpl(Texture, PngData);
		}
	}

	return  Texture.IsValid() && Texture->IsEnabled() ? Texture : nullptr;
}

TSharedPtr<IDisplayClusterRender_Texture, ESPMode::ThreadSafe> FDisplayClusterWarpBlendMPCDIRegionLoader::CreateTextureFromPFMFile(const FString& InPFMFile, float PFMScale, bool bIsUnrealGameSpace)
{
	// new resource, initialize once
	const FString FullPath2PFMFile = FDisplayClusterWarpBlendMPCDIFileLoader::GetFullPathToFile(InPFMFile);
	if (FullPath2PFMFile.IsEmpty())
	{
		return nullptr;
	}

	const FString UniqueName = FString::Printf(TEXT("%s=%f"), *FMD5::HashAnsiString(*FullPath2PFMFile.ToLower()), PFMScale);

	TSharedPtr<IDisplayClusterRender_Texture, ESPMode::ThreadSafe> Texture = GetOrCreateCachedTextureImpl(UniqueName);
	if (Texture && !Texture->IsEnabled())
	{
		bool bResult = false;
		const std::string FileName = TCHAR_TO_ANSI(*FullPath2PFMFile);
		mpcdi::PFM* PFMData;
		mpcdi::MPCDI_Error res = mpcdi::PfmIO::Read(FileName, PFMData);
		if (mpcdi::MPCDI_SUCCESS == res && PFMData)
		{
			FDisplayClusterWarpBlendLoader WarpMapData;
			if (WarpMapData.LoadFromPFM(GetWarpProfileType(), PFMData, PFMScale, bIsUnrealGameSpace))
			{
				Texture->CreateTexture(WarpMapData.GetWarpData(), 4, 32, WarpMapData.GetWidth(), WarpMapData.GetHeight(), true);
			}
		}
	}

	return  Texture.IsValid() && Texture->IsEnabled() ? Texture : nullptr;
}

bool FDisplayClusterWarpBlendMPCDIRegionLoader::LoadRegionFromMPCDIFile(const FString& InFilePath, const FString& InBufferName, const FString& InRegionName)
{
	TSharedPtr<FDisplayClusterWarpBlendMPCDIFileLoader, ESPMode::ThreadSafe> FileLoader = FDisplayClusterWarpBlendMPCDIFileLoader::GetOrCreateCachedMPCDILoader(InFilePath);
	if (FileLoader.IsValid())
	{
		mpcdi::Buffer* mpcdiBuffer = nullptr;
		mpcdi::Region* mpcdiRegion = nullptr;
		if (FileLoader->FindRegion(InBufferName, InRegionName, mpcdiBuffer, mpcdiRegion))
		{
			// Load MPCDI attributes
			FileLoader->LoadMPCDIAttributes(mpcdiBuffer, mpcdiRegion, MPCDIAttributes);

			// Load and create region resources:
			const FString UniqueName = FString::Printf(TEXT("%s.%s.%s"), *FileLoader->GetName(), *InBufferName, *InRegionName);

			// ALpha Map
			mpcdi::AlphaMap* AlphaMapSource = mpcdiRegion->GetFileSet()->GetAlphaMap();
			if (AlphaMapSource != nullptr)
			{
				AlphaMap = CreateTextureFromDataMap(FString::Printf(TEXT("%s.AlphaMap"), *UniqueName), AlphaMapSource);
				AlphaMapGammaEmbedded = AlphaMapSource->GetGammaEmbedded();
			}

			// Beta Map
			mpcdi::BetaMap* BetaMapSource = mpcdiRegion->GetFileSet()->GetBetaMap();
			if (BetaMapSource != nullptr)
			{
				BetaMap = CreateTextureFromDataMap(FString::Printf(TEXT("%s.BetaMap"), *UniqueName), BetaMapSource);
			}

			// Warp Map
			if (GetWarpProfileType() != EDisplayClusterWarpProfileType::warp_SL)
			{
				mpcdi::GeometryWarpFile* WarpMapSource = mpcdiRegion->GetFileSet()->GetGeometryWarpFile();
				if (WarpMapSource)
				{
					WarpMap = CreateTextureFromGeometry(FString::Printf(TEXT("%s.WarpMap"), *UniqueName), WarpMapSource);
					if (WarpMap.IsValid())
					{
						return true;
					}
				}
			}

			// Release unused resources
			ReleaseResources();
		}
	}

	return false;
}
