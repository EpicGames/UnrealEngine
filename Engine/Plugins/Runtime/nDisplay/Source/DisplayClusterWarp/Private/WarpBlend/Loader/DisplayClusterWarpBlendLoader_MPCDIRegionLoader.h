// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/DisplayClusterWarpContainers.h"
#include "Render/Containers/IDisplayClusterRender_Texture.h"
#include "WarpBlend/DisplayClusterWarpBlend.h"

namespace mpcdi
{
	struct GeometryWarpFile;
	struct DataMap;
};

/**
 * MPCDI region loader.
 * Creates the IDisplayClusterWarpBlend interface.
 */
class  FDisplayClusterWarpBlendMPCDIRegionLoader
{
public:
	FDisplayClusterWarpBlendMPCDIRegionLoader() = default;
	~FDisplayClusterWarpBlendMPCDIRegionLoader()
	{
		ReleaseResources();
	}

public:
	/** Creates a WarpMap texture from the DataMap. */
	TSharedPtr<IDisplayClusterRender_Texture, ESPMode::ThreadSafe> CreateTextureFromDataMap(const FString& InUniqueTextureName, mpcdi::DataMap* InDataMap);

	/** Creates a WarpMap texture from the GeometryWarp. */
	TSharedPtr<IDisplayClusterRender_Texture, ESPMode::ThreadSafe> CreateTextureFromGeometry(const FString& InUniqueTextureName, mpcdi::GeometryWarpFile* InGeometryWarp);

	/** Creates a WarpMap texture from the PFM file. */
	TSharedPtr<IDisplayClusterRender_Texture, ESPMode::ThreadSafe> CreateTextureFromPFMFile(const FString& InPFMFile, float PFMScale, bool bIsUnrealGameSpace);

	/** Creates a texture from the file.
	 * Supported image formats: PNG.
	 */
	TSharedPtr<IDisplayClusterRender_Texture, ESPMode::ThreadSafe> CreateTextureFromFile(const FString& InFileName);

	/**
	 * Load region data from the MPCDI file into the internal data of this class.
	 */
	bool LoadRegionFromMPCDIFile(const FString& InFilePath, const FString& InBufferName, const FString& InRegionName);

	/**
	 * Creates the WarpBlend interface from the internal data of this class.
	 * This function should be called at the end when all internal data is initialized.
	 */
	TSharedPtr<FDisplayClusterWarpBlend, ESPMode::ThreadSafe> CreateWarpBlendInterface();

private:
	/** Release internal resources. */
	void ReleaseResources();

	inline EDisplayClusterWarpProfileType  GetWarpProfileType() const
	{
		return MPCDIAttributes.ProfileType;
	}

public:
	// mpcdi attributes from xml
	FDisplayClusterWarpMPCDIAttributes MPCDIAttributes;

	// WarpMap texture (required)
	TSharedPtr<IDisplayClusterRender_Texture, ESPMode::ThreadSafe> WarpMap;

	// ALphaMap texture
	TSharedPtr<IDisplayClusterRender_Texture, ESPMode::ThreadSafe> AlphaMap;
	float AlphaMapGammaEmbedded = 1.f;

	// BetaMap texture
	TSharedPtr<IDisplayClusterRender_Texture, ESPMode::ThreadSafe> BetaMap;
};
