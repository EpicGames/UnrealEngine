// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterWarpBlendLoader_MeshComponent.h"

#include "Containers/DisplayClusterWarpInitializer.h"
#include "WarpBlend/DisplayClusterWarpBlend.h"
#include "WarpBlend/DisplayClusterWarpBlend_GeometryContext.h"
#include "WarpBlend/DisplayClusterWarpBlend_GeometryProxy.h"

#include "DisplayClusterWarpLog.h"

#include "Render/IDisplayClusterRenderManager.h"

#include "Misc/DisplayClusterHelpers.h"
#include "Misc/FileHelper.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Paths.h"

#include "Stats/Stats.h"
#include "Engine/Engine.h"

#include "ProceduralMeshComponent.h"

//---------------------------------------------------------------------
// FDisplayClusterWarpBlendLoader_MeshComponent
//---------------------------------------------------------------------
TSharedPtr<IDisplayClusterRender_MeshComponent, ESPMode::ThreadSafe> FDisplayClusterWarpBlendLoader_MeshComponent::CreateMeshComponent()
{
	static IDisplayCluster& DisplayClusterAPI = IDisplayCluster::Get();

	return DisplayClusterAPI.GetRenderMgr()->CreateMeshComponent();
}

TSharedPtr<FDisplayClusterWarpBlend, ESPMode::ThreadSafe> FDisplayClusterWarpBlendLoader_MeshComponent::Create(const FDisplayClusterWarpInitializer_StaticMesh& InConstructParameters)
{
	if (InConstructParameters.WarpMeshComponent != nullptr)
	{
		//ok, Create and initialize warpblend interface:
		TSharedPtr<FDisplayClusterWarpBlend, ESPMode::ThreadSafe> WarpBlend = MakeShared<FDisplayClusterWarpBlend, ESPMode::ThreadSafe>();

		WarpBlend->GeometryContext.SetWarpProfileType(EDisplayClusterWarpProfileType::warp_A3D);

		FDisplayClusterWarpBlend_GeometryProxy& Proxy = WarpBlend->GeometryContext.GeometryProxy;

		FDisplayClusterMeshUVs MeshUVs(InConstructParameters.BaseUVIndex, InConstructParameters.ChromakeyUVIndex);

		Proxy.PreviewMeshComponentRef.SetSceneComponent(InConstructParameters.PreviewMeshComponent);

		Proxy.WarpMeshComponent = CreateMeshComponent();
		Proxy.WarpMeshComponent->AssignStaticMeshComponentRefs(InConstructParameters.WarpMeshComponent, MeshUVs, InConstructParameters.OriginComponent, InConstructParameters.StaticMeshComponentLODIndex);

		Proxy.StaticMeshComponentLODIndex = InConstructParameters.StaticMeshComponentLODIndex;
		Proxy.WarpMeshUVs = MeshUVs;

		Proxy.GeometryType = EDisplayClusterWarpGeometryType::WarpMesh;
		Proxy.FrustumGeometryType = EDisplayClusterWarpFrustumGeometryType::WarpMesh;

		return WarpBlend;
	}

	return nullptr;
}

TSharedPtr<FDisplayClusterWarpBlend, ESPMode::ThreadSafe> FDisplayClusterWarpBlendLoader_MeshComponent::Create(const FDisplayClusterWarpInitializer_ProceduralMesh& InConstructParameters)
{
	if (InConstructParameters.WarpMeshComponent != nullptr)
	{
		//ok, Create and initialize warpblend interface:
		TSharedPtr<FDisplayClusterWarpBlend, ESPMode::ThreadSafe> WarpBlend = MakeShared<FDisplayClusterWarpBlend, ESPMode::ThreadSafe>();

		WarpBlend->GeometryContext.SetWarpProfileType(EDisplayClusterWarpProfileType::warp_A3D);

		FDisplayClusterWarpBlend_GeometryProxy& Proxy = WarpBlend->GeometryContext.GeometryProxy;

		FDisplayClusterMeshUVs MeshUVs(InConstructParameters.BaseUVIndex, InConstructParameters.ChromakeyUVIndex);

		Proxy.PreviewMeshComponentRef.SetSceneComponent(InConstructParameters.PreviewMeshComponent);

		// Assign procedural mesh
		Proxy.WarpMeshComponent = CreateMeshComponent();
		Proxy.WarpMeshComponent->AssignProceduralMeshComponentRefs(InConstructParameters.WarpMeshComponent, MeshUVs, InConstructParameters.OriginComponent, InConstructParameters.ProceduralMeshComponentSectionIndex);

		Proxy.ProceduralMeshComponentSectionIndex = InConstructParameters.ProceduralMeshComponentSectionIndex;
		Proxy.WarpMeshUVs = MeshUVs;

		Proxy.GeometryType = EDisplayClusterWarpGeometryType::WarpProceduralMesh;
		Proxy.FrustumGeometryType = EDisplayClusterWarpFrustumGeometryType::WarpProceduralMesh;

		return WarpBlend;
	}

	return nullptr;
}
