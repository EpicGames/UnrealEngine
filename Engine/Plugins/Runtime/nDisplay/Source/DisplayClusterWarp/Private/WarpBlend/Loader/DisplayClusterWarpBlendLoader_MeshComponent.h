// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/DisplayClusterWarpInitializer.h"

class IDisplayClusterRender_MeshComponent;
class FDisplayClusterWarpBlend;

/**
 * An auxiliary class for creating the WarpBlend interface from initializing structures associated with UE components.
 */
class FDisplayClusterWarpBlendLoader_MeshComponent
{
public:
	/** Create mesh component instance. */
	static TSharedPtr<IDisplayClusterRender_MeshComponent, ESPMode::ThreadSafe> CreateMeshComponent();

	/** Create WarpBlend interface from the StaticMesh component. */
	static TSharedPtr<FDisplayClusterWarpBlend, ESPMode::ThreadSafe> Create(const FDisplayClusterWarpInitializer_StaticMesh& InConstructParameters);

	/** Create WarpBlend interface from the ProceduralMesh component. */
	static TSharedPtr<FDisplayClusterWarpBlend, ESPMode::ThreadSafe> Create(const FDisplayClusterWarpInitializer_ProceduralMesh& InConstructParameters);
};
