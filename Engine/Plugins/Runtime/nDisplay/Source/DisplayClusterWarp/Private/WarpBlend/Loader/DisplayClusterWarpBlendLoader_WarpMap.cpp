// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplayClusterWarpBlendLoader_WarpMap.h"
#include "DisplayClusterWarpLog.h"

THIRD_PARTY_INCLUDES_START
#include "mpcdiAlphaMap.h"
#include "mpcdiBetaMap.h"
#include "mpcdiBuffer.h"
#include "mpcdiDisplay.h"
#include "mpcdiDistortionMap.h"
#include "mpcdiGeometryWarpFile.h"
#include "mpcdiProfile.h"
#include "mpcdiReader.h"
#include "mpcdiRegion.h"
THIRD_PARTY_INCLUDES_END

#include "RHIGlobals.h"

namespace UE::DisplayClusterWarp::PFMHelpers
{
	static constexpr float kEpsilon = 0.00001f;

	static inline float GetUnitToCentimeter(mpcdi::GeometricUnit GeomUnit)
	{
		switch (GeomUnit)
		{
		case mpcdi::GeometricUnitmm:
			return 1.f / 10.f;
		case mpcdi::GeometricUnitcm:
			return 1.f;
		case mpcdi::GeometricUnitdm:
			return 10.f;
		case mpcdi::GeometricUnitm:
			return 100.f;
		case mpcdi::GeometricUnitin:
			return 2.54f;
		case mpcdi::GeometricUnitft:
			return 30.48f;
		case mpcdi::GeometricUnityd:
			return 91.44f;
		case mpcdi::GeometricUnitunknown:
			return 1.f;
		case mpcdi::GeometricUnit2D:
			return 1.f;
		default:
			checkSlow(false);
			return 1.f;
		}
	}

	static inline FMatrix GetMPCDI2UEConventionMatrix(const EDisplayClusterWarpProfileType InProfileType, const float UnitToCentemeter)
	{
		switch (InProfileType)
		{
		case EDisplayClusterWarpProfileType::warp_A3D:
		case EDisplayClusterWarpProfileType::warp_SL:
			// Unreal is in cm, so we need to convert to cm.
			// Convert from MPCDI convention to Unreal convention
			// MPCDI is Right Handed (Y is up, X is left, Z is in the screen)
			// Unreal is Left Handed (Z is up, X in the screen, Y is right)
			return FMatrix(
				FPlane(0.f, UnitToCentemeter, 0.f, 0.f),
				FPlane(0.f, 0.f, UnitToCentemeter, 0.f),
				FPlane(-UnitToCentemeter, 0.f, 0.f, 0.f),
				FPlane(0.f, 0.f, 0.f, 1.f));

		default:
			break;
		};

		return FMatrix::Identity;
	}

	static inline FMatrix GetUE2UEConventionMatrix(const float WorldScale)
	{
		return FMatrix(
			FPlane(WorldScale, 0.f, 0.f, 0.f),
			FPlane(0.f, WorldScale, 0.f, 0.f),
			FPlane(0.f, 0.f, WorldScale, 0.f),
			FPlane(0.f, 0.f, 0.f, 1.f));
	}

	static inline FVector ImportMPCDINode(const EDisplayClusterWarpProfileType InProfileType, const mpcdi::NODE& InNode)
	{
		switch (InProfileType)
		{
		case EDisplayClusterWarpProfileType::warp_A3D:
		case EDisplayClusterWarpProfileType::warp_SL:
			return FVector(InNode.r, InNode.g, InNode.b);

		default:
			break;
		}

		// ignore Z value for 2d profiles
		return FVector(InNode.r, InNode.g, 0);
	}

	// This is just a copy of the old rules:
	//     if ((!(FMath::Abs(t.X) < kEpsilon && FMath::Abs(t.Y) < kEpsilon && FMath::Abs(t.Z) < kEpsilon))
	//      && (!FMath::IsNaN(t.X) && !FMath::IsNaN(t.Y) && !FMath::IsNaN(t.Z)))
	// 
	// Todo: improve it if necessary
	static inline bool IsMPCDINodeValid(const FVector& InNode)
	{
		if (FMath::Abs(InNode.X) < kEpsilon && FMath::Abs(InNode.Y) < kEpsilon && FMath::Abs(InNode.Z) < kEpsilon)
		{
			return false;
		}

		if(FMath::IsNaN(InNode.X) || FMath::IsNaN(InNode.Y) || FMath::IsNaN(InNode.Z))
		{
			return false;
		}

		return true;
	}
};
using namespace UE::DisplayClusterWarp::PFMHelpers;

//----------------------------------------------------------------------
// FDisplayClusterWarpBlendLoader
//----------------------------------------------------------------------
bool FDisplayClusterWarpBlendLoader::LoadFromGeometryWarpFile(const EDisplayClusterWarpProfileType InProfileType, mpcdi::GeometryWarpFile* SourceWarpMap)
{
	check(SourceWarpMap);

	if (!InitializeWarpDataImpl(SourceWarpMap->GetSizeX(), SourceWarpMap->GetSizeY()))
	{
		return false;
	}

	const FMatrix ConventionMatrix = GetMPCDI2UEConventionMatrix(InProfileType, GetUnitToCentimeter(SourceWarpMap->GetGeometricUnit()));
	for (int32 WarpMapY = 0; WarpMapY < Height; ++WarpMapY)
	{
		for (int32 WarpMapX = 0; WarpMapX < Width; ++WarpMapX)
		{
			FVector4f& OutPts = WarpData[WarpMapX + WarpMapY * Width];

			FVector InputPts = ImportMPCDINode(InProfileType, (*SourceWarpMap)(WarpMapX, WarpMapY));
			if (IsMPCDINodeValid(InputPts))
			{
				const FVector ScaledPts = ConventionMatrix.TransformPosition(InputPts);

				OutPts.X = ScaledPts.X;
				OutPts.Y = ScaledPts.Y;
				OutPts.Z = ScaledPts.Z;
				OutPts.W = 1;
			}
			else
			{
				OutPts = FVector4f(0.f, 0.f, 0.f, -1.f);
			}
		}
	}

	// Remove noise from warp mesh (small areas less than 3*3 quads)
	ClearNoise(InProfileType);

	return true;
}

bool FDisplayClusterWarpBlendLoader::LoadFromPoint(const EDisplayClusterWarpProfileType InProfileType, const TArray<FVector>& InPoints, int32 WarpX, int32 WarpY, float WorldScale, bool bIsUnrealGameSpace)
{
	if (!InitializeWarpDataImpl(WarpX, WarpY))
	{
		return false;
	}

	LoadGeometryImpl(InProfileType, InPoints, WorldScale, bIsUnrealGameSpace);

	return true;
}

bool FDisplayClusterWarpBlendLoader::LoadFromPFM(const EDisplayClusterWarpProfileType InProfileType, mpcdi::PFM* SourcePFM, float PFMScale, bool bIsUnrealGameSpace)
{
	if (InProfileType == EDisplayClusterWarpProfileType::Invalid)
	{
		return false;
	}

	check(SourcePFM);
	if (!InitializeWarpDataImpl(SourcePFM->GetSizeX(), SourcePFM->GetSizeY()))
	{
		return false;
	}

	TArray<FVector> WarpMeshPoints;
	WarpMeshPoints.Reserve(Width * Height);

	for (int32 WarpMapY = 0; WarpMapY < Height; ++WarpMapY)
	{
		for (int32 WarpMapX = 0; WarpMapX < Width; ++WarpMapX)
		{
			WarpMeshPoints.Add(ImportMPCDINode(InProfileType, SourcePFM->operator()(WarpMapX, WarpMapY)));
		}
	}

	LoadGeometryImpl(InProfileType, WarpMeshPoints, PFMScale, bIsUnrealGameSpace);

	return true;
}

bool FDisplayClusterWarpBlendLoader::InitializeWarpDataImpl(int32 InWidth, int32 InHeight)
{
	if (FMath::Min(InWidth, InHeight) <= 1 || FMath::Max(InWidth, InHeight) >= GMaxTextureDimensions)
	{
		UE_LOGF(LogDisplayClusterWarpBlend, Error, "Invalid PFM warpmap data size '%d x %d'", InWidth, InHeight);

		return false;
	}

	Width = InWidth;
	Height = InHeight;
	WarpData = new FVector4f[Width * Height];

	return true;
}

FDisplayClusterWarpBlendLoader::~FDisplayClusterWarpBlendLoader()
{
	if (WarpData != nullptr)
	{
		delete WarpData;
		WarpData = nullptr;
	}
}

void FDisplayClusterWarpBlendLoader::LoadGeometryImpl(EDisplayClusterWarpProfileType ProfileType, const TArray<FVector>& InPoints, float WorldScale, bool bIsUnrealGameSpace)
{
	check(InPoints.Num() == (Width * Height));

	FMatrix ConventionMatrix = bIsUnrealGameSpace ? GetUE2UEConventionMatrix(WorldScale) : GetMPCDI2UEConventionMatrix(ProfileType, WorldScale);

	FVector4f* WarpDataPointIt = WarpData;
	for (const FVector& PointIt : InPoints)
	{
		const FVector& InPts = PointIt;

		FVector4f& OutPts = *WarpDataPointIt;
		WarpDataPointIt++;

		if(IsMPCDINodeValid(InPts))
		{
			const FVector ScaledPts = ConventionMatrix.TransformPosition(InPts);

			OutPts.X = ScaledPts.X;
			OutPts.Y = ScaledPts.Y;
			OutPts.Z = ScaledPts.Z;
			OutPts.W = 1;
		}
		else
		{
			OutPts = FVector4f(0.f, 0.f, 0.f, -1.f);
		}
	}
}

bool FDisplayClusterWarpBlendLoader::Is3DPointValid(int32 X, int32 Y) const
{
	if (WarpData && X >= 0 && X < Width && Y >= 0 && Y < Height)
	{
		const FVector4f& Point = WarpData[X + Y * Width];

		return Point.W > 0;
	}

	return false;
}

void FDisplayClusterWarpBlendLoader::ClearNoise(const EDisplayClusterWarpProfileType InProfileType)
{
	switch (InProfileType)
	{
	case EDisplayClusterWarpProfileType::warp_A3D:
	case EDisplayClusterWarpProfileType::warp_SL:
		// only for 3d geometry
		break;

	default:
		return;
	}

	// Todo: Replace these constants with CVAR if necessary
	const FIntPoint SearchXYDepth(3, 3);
	const FIntPoint AllowedXYDepthRules(2, 3);
	const int32 MinGridSize = 10;
	const int32 MaxLoopsNum = 50;

	if (Width > MinGridSize && Height > MinGridSize)
	{
		//Remove noise for large warp mesh
		int32 MaxLoops = MaxLoopsNum;
		while (MaxLoops-- > 0)
		{
			if (!ClearNoiseImpl(SearchXYDepth, AllowedXYDepthRules))
			{
				break;
			}
		}
	}
}

int32 FDisplayClusterWarpBlendLoader::ClearNoiseImpl(const FIntPoint& SearchLen, const FIntPoint& RemoveRule)
{
	const int32 SearchX = SearchLen.X * Width / 100;
	const int32 SearchY = SearchLen.Y * Height / 100;
	const int32 Rule1X = RemoveRule.X * Width / 100;
	const int32 Rule1Y = RemoveRule.Y * Height / 100;

	int32 TotalChangesCount = 0;
	static const int32 DirIndexValue[] = { -1, 1 };

	for (int32 Y = 0; Y < Height; ++Y)
	{
		for (int32 X = 0; X < Width; ++X)
		{
			if (Is3DPointValid(X, Y))
			{
				int32 XLen = 0;
				int32 YLen = 0;

				for (int32 DirIndex = 0; DirIndex < 2; DirIndex++)
				{
					int32 dx = 0;
					int32 dy = 0;

					for (int32 Offset = 1; Offset <= SearchX; Offset++)
					{
						if (Is3DPointValid(X + DirIndexValue[DirIndex] * Offset, Y))
						{
							dx++;
						}
						else
						{
							break;
						}
					}
					for (int32 Offset = 1; Offset <= SearchY; Offset++)
					{
						if (Is3DPointValid(X, Y + DirIndexValue[DirIndex] * Offset))
						{
							dy++;
						}
						else
						{
							break;
						}
					}

					XLen = FMath::Max(XLen, dx);
					YLen = FMath::Max(YLen, dy);
				}

				const bool Test1 = XLen >= Rule1X && YLen >= Rule1Y;
				const bool Test2 = YLen >= Rule1X && XLen >= Rule1Y;

				if (!Test1 && !Test2)
				{
					// Both test failed, remove it
					WarpData[X + Y * Width] = FVector4f(0.f, 0.f, 0.f, -1.f);
					TotalChangesCount++;
				}
			}
		}
	}

	return TotalChangesCount;
}
