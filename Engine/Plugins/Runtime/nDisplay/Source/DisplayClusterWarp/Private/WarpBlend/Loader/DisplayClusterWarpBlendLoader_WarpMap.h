// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/DisplayClusterWarpEnums.h"

namespace mpcdi
{
	struct GeometryWarpFile;
	struct PFM;
};

/**
 * WarpMap data loader.
 */
class FDisplayClusterWarpBlendLoader
{
public:
	FDisplayClusterWarpBlendLoader() = default;
	~FDisplayClusterWarpBlendLoader();

public:
	/** Get the width of the WarpMap. */
	int32 GetWidth() const
	{
		return Width;
	}

	/** Get the height of the WarpMap. */
	int32 GetHeight() const
	{
		return Height;
	}

	/** Get a pointer to WarpMap data. */
	const FVector4f* GetWarpData() const
	{
		return WarpData;
	}

	/** Load WarpMap from GeometryWarpFile structure. */
	bool LoadFromGeometryWarpFile(const EDisplayClusterWarpProfileType InProfileType, mpcdi::GeometryWarpFile* SourceWarpMap);

	/** Load WarpMap from points. */
	bool LoadFromPoint(const EDisplayClusterWarpProfileType InProfileType, const TArray<FVector>& InPoints, int32 WarpX, int32 WarpY, float WorldScale, bool bIsUnrealGameSpace);

	/** Load WarpMap from PFM structure. */
	bool LoadFromPFM(const EDisplayClusterWarpProfileType InProfileType, mpcdi::PFM* SourcePFM, float PFMScale, bool bIsUnrealGameSpace);

protected:
	/** Initializes WarpData and returns true if successful. */
	bool InitializeWarpDataImpl(int32 InWidth, int32 InHeight);

	/** Load data from points with scaling and space conversion. */
	void LoadGeometryImpl(EDisplayClusterWarpProfileType ProfileType, const TArray<FVector>& InPoints, float WorldScale, bool bIsUnrealGameSpace);

	/** Returns true if this point can be used in the warp. */
	bool Is3DPointValid(int32 X, int32 Y) const;

	/** Helper: The calibration data is not perfect and contains noise.
	 * This function deletes disconnected point areas.
	 * Useful for large warp nets.
	 */
	void ClearNoise(const EDisplayClusterWarpProfileType InProfileType);

	/** Implementation of the ClearNoice(). */
	int32 ClearNoiseImpl(const FIntPoint& SearchLen, const FIntPoint& RemoveRule);

protected:
	// Width of the WarpData
	int32 Width = 0;

	// Height of the WarpData
	int32 Height = 0;

	// WarpData
	FVector4f* WarpData = nullptr;
};
