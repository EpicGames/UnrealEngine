// Copyright Epic Games, Inc. All Rights Reserved.

#include "WarpBlend/Math/DisplayClusterWarpBlendMath_Frustum.h"

#include "Render/Viewport/IDisplayClusterViewport.h"
#include "Render/Containers/IDisplayClusterRender_Texture.h"
#include "Misc/DisplayClusterHelpers.h"


#include "StaticMeshResources.h"
#include "ProceduralMeshComponent.h"

// Select mpcdi frustum calc method
int32 GDisplayClusterWarpBlendMPCDIFrustumMethod = (int32)EDisplayClusterWarpBlendFrustumType::LOD;
static FAutoConsoleVariableRef CVarDisplayClusterWarpBlendMPCDIFrustumMethod(
	TEXT("nDisplay.render.mpcdi.Frustum"),
	GDisplayClusterWarpBlendMPCDIFrustumMethod,
	TEXT("Frustum computation method:\n")
	TEXT(" 0: mesh AABB based, lower quality but fast\n")
	TEXT(" 1: mesh vertices based, best quality but slow\n")
	TEXT(" 2: LOD, get A*B distributed points from texture, fast, good quality for flat panels\n"),
	ECVF_Default
);

// Select mpcdi stereo mode
int32 GDisplayClusterWarpBlendMPCDIStereoMode = (int32)EDisplayClusterWarpBlendStereoMode::AsymmetricAABB;
static FAutoConsoleVariableRef CVarDisplayClusterWarpBlendMPCDIStereoMode(
	TEXT("nDisplay.render.mpcdi.StereoMode"),
	GDisplayClusterWarpBlendMPCDIStereoMode,
	TEXT("Stereo mode:\n")
	TEXT(" 0: Asymmetric to AABB center\n")
	TEXT(" 1: Symmetric to AABB center\n"),
	ECVF_Default
);

// Select mpcdi projection mode
int32 GDisplayClusterWarpBlendMPCDIProjectionMode = (int32)EDisplayClusterWarpBlendProjectionType::StaticSurfaceNormal;
static FAutoConsoleVariableRef CVarDisplayClusterWarpBlendMPCDIProjectionMode(
	TEXT("nDisplay.render.mpcdi.Projection"),
	GDisplayClusterWarpBlendMPCDIProjectionMode,
	TEXT("Projection method:\n")
	TEXT(" 0: Static, aligned to average region surface normal\n")
	TEXT(" 1: Static, aligned to average region surface corners plane\n")
	TEXT(" 2: Dynamic, to view target center\n"),
	ECVF_Default
);

// Frustum projection fix (back-side view planes)
int32 GDisplayClusterWarpBlendMPCDIProjectionAuto = 1;
static FAutoConsoleVariableRef CVarDisplayClusterWarpBlendMPCDIProjectionAuto(
	TEXT("nDisplay.render.mpcdi.ProjectionAuto"),
	GDisplayClusterWarpBlendMPCDIProjectionAuto,
	TEXT("Runtime frustum method, fix back-side view projection.\n")
	TEXT(" 0: Disabled\n")
	TEXT(" 1: Enabled (default)\n"),
	ECVF_Default
);

int32 GDisplayClusterWarpBlendRotateFrustumToFitContextSize = 1;
static FAutoConsoleVariableRef CVarDisplayClusterWarpBlendRotateFrustumToFitContextSize(
	TEXT("nDisplay.render.mpcdi.RotateFrustumToFitContextSize"),
	GDisplayClusterWarpBlendRotateFrustumToFitContextSize,
	TEXT("Enables the method that rotates the frustum according to the size of the viewport area. (0 - disable)\n"),
	ECVF_Default
);

float GDisplayClusterFrustumOrientationFitHysteresis = 0.05f;
static FAutoConsoleVariableRef CVarDisplayClusterFrustumOrientationFitHysteresis(
	TEXT("nDisplay.render.mpcdi.FrustumOrientationFitHysteresis"),
	GDisplayClusterFrustumOrientationFitHysteresis,
	TEXT(
		"Hysteresis margin when fitting frustum orientation with render target aspect ratio.\n"
		"Values closer to 0 will allow immediate fit, while values close to 1 will require a \n"
		"higher degree of mismatch before reorienting the frustum for a better fit."
	),
	ECVF_Default
);

//------------------------------------------------------------
// FDisplayClusterWarpBlendMath_Frustum
//------------------------------------------------------------
FDisplayClusterWarpBlendMath_Frustum::FDisplayClusterWarpBlendMath_Frustum(FDisplayClusterWarpData& InWarpData, FDisplayClusterWarpBlend_GeometryContext& InGeometryContext)
	: GeometryContext(InGeometryContext)
	, WarpData(InWarpData)
{
	check(WarpData.WarpEye.IsValid());

	// Configure logic from CVar:
	WarpData.bEnabledRotateFrustumToFitContextSize = GDisplayClusterWarpBlendRotateFrustumToFitContextSize != 0;
	WarpData.bFindBestProjectionType = GDisplayClusterWarpBlendMPCDIProjectionAuto != 0;
	WarpData.FrustumType = (EDisplayClusterWarpBlendFrustumType)(GDisplayClusterWarpBlendMPCDIFrustumMethod);
	WarpData.StereoMode = (EDisplayClusterWarpBlendStereoMode)(GDisplayClusterWarpBlendMPCDIStereoMode);

	SetParameter((EDisplayClusterWarpBlendProjectionType)(GDisplayClusterWarpBlendMPCDIProjectionMode));
}

bool FDisplayClusterWarpBlendMath_Frustum::CalcFrustum()
{
	// Initialize warp projection data
	InitializeWarpProjectionData();

	// Override frustum
	if (ShouldHandleOverrideCalcFrustum())
	{
		if (OverrideCalcFrustum())
		{
			return EndCalcFrustum();
		}

		return false;
	}

	switch (GeometryContext.GetWarpProfileType())
	{
	case EDisplayClusterWarpProfileType::warp_2D:
		return CalcFrustum2D();

	case EDisplayClusterWarpProfileType::warp_3D:
		return CalcFrustum3D();

	case EDisplayClusterWarpProfileType::warp_A3D:
		return CalcFrustumA3D();

	case EDisplayClusterWarpProfileType::warp_SL:
		return CalcFrustumSL();

	default:
		break;
	}

	return false;
}

FMatrix FDisplayClusterWarpBlendMath_Frustum::CalcProjectionMatrix(const FDisplayClusterWarpProjection& InWarpProjection)
{
	check(WarpData.WarpEye.IsValid())

	if (IDisplayClusterViewport* Viewport = WarpData.WarpEye->GetViewport())
	{
		const uint32 ContextNum = WarpData.WarpEye->ContextNum;
		if (Viewport->GetContexts().IsValidIndex(ContextNum))
		{
			// the input data can be in degrees
			const bool bIsAnglesInput = InWarpProjection.DataType == EDisplayClusterWarpAngleUnit::Degrees;

			Viewport->CalculateProjectionMatrix(ContextNum,
				InWarpProjection.Left,  InWarpProjection.Right,
				InWarpProjection.Top,   InWarpProjection.Bottom,
				InWarpProjection.ZNear, InWarpProjection.ZFar, bIsAnglesInput);

			return Viewport->GetContexts()[ContextNum].ProjectionMatrix;
		}
	}

	return FMatrix::Identity;
}

bool FDisplayClusterWarpBlendMath_Frustum::ShouldRotateFrustumToFitContextSize() const
{
	return WarpData.bEnabledRotateFrustumToFitContextSize && WarpData.bRotateFrustumToFitContextSize;
}

bool FDisplayClusterWarpBlendMath_Frustum::EndCalcFrustum()
{
	// Fit frustum to context size
	ImplFitFrustumToContextSize();

	// Update Camera
	WarpData.GeometryWarpProjection.CameraLocation = WarpData.GeometryWarpProjection.EyeLocation;
	WarpData.GeometryWarpProjection.CameraRotation = WarpData.Local2World.Rotator();

	// Use geometry projection for render. This projection can be overriden
	WarpData.WarpProjection = WarpData.GeometryWarpProjection;

	check(WarpData.WarpEye.IsValid());

	if (WarpData.WarpEye->WarpPolicy.IsValid())
	{
		if (IDisplayClusterViewport* Viewport = WarpData.WarpEye->GetViewport())
		{
			WarpData.WarpEye->WarpPolicy->EndCalcFrustum(Viewport, WarpData.WarpEye->ContextNum);
		}
	}

	// After updating the warp data in the warp policy, we build the final warp context in the code below:

	FRotator CameraRotation = WarpData.WarpProjection.CameraRotation;
	FMatrix RegionMatrix = GeometryContext.GetRegionMatrix();

	// Apply frustum rotation to fit context size:
	WarpData.WarpContext.bFrustumRotated = ShouldRotateFrustumToFitContextSize();
	if (WarpData.WarpContext.bFrustumRotated)
	{
		static FQuat PostRotation = FRotator(0, 0, -90).Quaternion();

		// Rotate camera view 90 degree 
		CameraRotation = (CameraRotation.Quaternion() * PostRotation).Rotator();

		// Rotate projection angles
		WarpData.WarpProjection.RotateProjectionAngles90Degree();

		// Rotate input texture
		static FMatrix Rotate90DegreeCCW(
			FPlane(0, 1, 0, 0),
			FPlane(-1, 0, 0, 0),
			FPlane(0, 0, 1, 0),
			FPlane(1, 0, 0, 1));

		RegionMatrix = RegionMatrix * Rotate90DegreeCCW;
	}

	// Compute projection matricies
	WarpData.WarpContext.GeometryProjectionMatrix = CalcProjectionMatrix(WarpData.GeometryWarpProjection);
	WarpData.WarpContext.ProjectionMatrix = CalcProjectionMatrix(WarpData.WarpProjection);

	// Use camera for rendering, but eye for math
	WarpData.WarpContext.Rotation = CameraRotation;
	WarpData.WarpContext.Location = WarpData.WarpProjection.CameraLocation;

	WarpData.WarpContext.UVMatrix = GetUVMatrix(WarpData.WarpContext.GeometryProjectionMatrix);

	WarpData.WarpContext.TextureMatrix = GeometryContext.GetTextureMatrix();
	WarpData.WarpContext.RegionMatrix = RegionMatrix;

	WarpData.WarpContext.MeshToStageMatrix = GeometryContext.Context.GeometryToOrigin;

	WarpData.WarpContext.bIsValid = true;

	return true;
}

void FDisplayClusterWarpBlendMath_Frustum::ImplFitFrustumToContextSize()
{
	check(WarpData.WarpEye.IsValid());

	const uint32 ContextNum = WarpData.WarpEye->ContextNum;
	const IDisplayClusterViewport* Viewport = WarpData.WarpEye->GetViewport();
	if (!Viewport || !Viewport->GetContexts().IsValidIndex(ContextNum))
	{
		return;
	}

	const FIntPoint ContextSize = Viewport->GetContexts()[ContextNum].ContextSize;

	// See if it should roll by 90 degrees to get a better fit with the context size aspect ratio.
	FDisplayClusterWarpProjection CurrentWarpProjection = WarpData.GeometryWarpProjection;
	if (ShouldRotateFrustumToFitContextSize())
	{
		CurrentWarpProjection.RotateProjectionAngles90Degree();
	}

	// Calculate the frustum Horizontal to Vertical aspect ratio.
	const float HFoV = FMath::Abs(CurrentWarpProjection.Right - CurrentWarpProjection.Left);
	const float VFoV = FMath::Abs(CurrentWarpProjection.Top - CurrentWarpProjection.Bottom);

	// Validate the values to protect from division by zero

	const bool bValidValues =
		!FMath::IsNearlyZero(HFoV)
		&& !FMath::IsNearlyZero(VFoV)
		&& (ContextSize.X > 0)
		&& (ContextSize.Y > 0);

	if (!bValidValues)
	{
		return;
	}

	const bool bFrustumIsWide = HFoV > VFoV;
	const bool bContextSizeIsWide = ContextSize.X > ContextSize.Y;

	// If both aspect ratios are of the same type already, then there is nothing else to do.
	if (bContextSizeIsWide == bFrustumIsWide)
	{
		return;
	}

	// If we are here, we must decide if we should toggle the 90 degree roll.
	// The toggle will have hysteresis to avoid jumping back and forth.
	// We will use the frustum aspect ratio to determine this

	// Don't flip the rotation if we're inside hystersis
	if (FMath::IsNearlyEqual(HFoV / VFoV, 1.0, GDisplayClusterFrustumOrientationFitHysteresis))
	{
		return;
	}

	// If we're here, it is because we should toggle the rotation fit.
	WarpData.bRotateFrustumToFitContextSize = !WarpData.bRotateFrustumToFitContextSize;
}

FMatrix FDisplayClusterWarpBlendMath_Frustum::GetUVMatrix(const FMatrix& InProjectionMatrix) const
{
	// These matrices were copied from LocalPlayer.cpp.
	// They change the coordinate system from the Unreal "Game" coordinate system to the Unreal "Render" coordinate system
	static const FMatrix Game2Render(
		FPlane(0, 0, 1, 0),
		FPlane(1, 0, 0, 0),
		FPlane(0, 1, 0, 0),
		FPlane(0, 0, 0, 1));

	switch (GeometryContext.GetWarpProfileType())
	{
	case EDisplayClusterWarpProfileType::warp_A3D:
		return  WarpData.Local2World.Inverse() * Game2Render * InProjectionMatrix;

	default:
		break;
	}

	return FMatrix::Identity;
}

bool FDisplayClusterWarpBlendMath_Frustum::ShouldUseCustomViewDirection() const
{
	return WarpData.WarpEye->OverrideViewTarget.IsSet() || WarpData.WarpEye->OverrideViewDirection.IsSet();
}

FVector FDisplayClusterWarpBlendMath_Frustum::GetViewTarget() const
{
	return WarpData.WarpEye->OverrideViewTarget.IsSet()  ? WarpData.WarpEye->OverrideViewTarget.GetValue() : GeometryContext.Context.AABBox.GetCenter();
}

FVector FDisplayClusterWarpBlendMath_Frustum::GetViewDirection()
{
	check(WarpData.WarpEye.IsValid());

	if (WarpData.WarpEye->OverrideViewDirection.IsSet())
	{
		return WarpData.WarpEye->OverrideViewDirection.GetValue();
	}

	// Get view target location
	const FVector ViewTargetLocation = GetViewTarget();

	// Get view direction vector
	if (WarpData.StereoMode == EDisplayClusterWarpBlendStereoMode::AsymmetricAABB)
	{
		// Create view transform matrix from look direction vector:
		return (ViewTargetLocation - WarpData.WarpEye->ViewPoint.GetEyeLocation()).GetSafeNormal();
	}

	return (ViewTargetLocation - WarpData.WarpEye->ViewPoint.Location).GetSafeNormal();
}

bool FDisplayClusterWarpBlendMath_Frustum::GetProjectionClip(const FVector4& InPoint, const FMatrix& InGeometryToFrustumA3D, FDisplayClusterWarpProjection& InOutWarpProjection)
{
	if (InPoint.W > 0)
	{
		FVector4 ProjectedVertice = InGeometryToFrustumA3D.TransformFVector4(InPoint);

		// Use only points over view plane, ignore backside pts
		if (isnan(ProjectedVertice.X) || isnan(ProjectedVertice.Y) || isnan(ProjectedVertice.Z) || ProjectedVertice.X <= 0 || FMath::IsNearlyZero(ProjectedVertice.X, (FVector4::FReal)1.e-6f))
		{
			// This point out of view plane
			return false;
		}

		const double Scale = InOutWarpProjection.ZNear / ProjectedVertice.X;

		ProjectedVertice.Y *= Scale;
		ProjectedVertice.Z *= Scale;

		InOutWarpProjection.Left = FMath::Min(InOutWarpProjection.Left, ProjectedVertice.Y);
		InOutWarpProjection.Right = FMath::Max(InOutWarpProjection.Right, ProjectedVertice.Y);

		InOutWarpProjection.Top = FMath::Max(InOutWarpProjection.Top, ProjectedVertice.Z);
		InOutWarpProjection.Bottom = FMath::Min(InOutWarpProjection.Bottom, ProjectedVertice.Z);
	}

	return true;
}

bool FDisplayClusterWarpBlendMath_Frustum::ShouldHandleOverrideCalcFrustum() const
{
	check(WarpData.WarpEye.IsValid());

	if (WarpData.WarpEye->WarpPolicy.IsValid())
	{
		if (IDisplayClusterViewport* Viewport = WarpData.WarpEye->GetViewport())
		{
			return WarpData.WarpEye->WarpPolicy->ShouldOverrideCalcFrustum(Viewport);
		}
	}

	return false;
}

bool FDisplayClusterWarpBlendMath_Frustum::OverrideCalcFrustum()
{
	check(WarpData.WarpEye.IsValid());

	if (WarpData.WarpEye->WarpPolicy.IsValid())
	{
		if (IDisplayClusterViewport* Viewport = WarpData.WarpEye->GetViewport())
		{
			return WarpData.WarpEye->WarpPolicy->OverrideCalcFrustum(Viewport, WarpData.WarpEye->ContextNum);
		}
	}

	return false;
}
