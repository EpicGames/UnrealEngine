// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/DisplayClusterWarpContext.h"
#include "Containers/DisplayClusterWarpEye.h"
#include "WarpBlend/DisplayClusterWarpBlend_GeometryContext.h"

class IDisplayClusterViewport;

/**
 * The WarpBlend math implementation.
 */
class FDisplayClusterWarpBlendMath_Frustum
{
public:
	FDisplayClusterWarpBlendMath_Frustum(FDisplayClusterWarpData& InWarpData, FDisplayClusterWarpBlend_GeometryContext& InGeometryContext);

public:
	/** Entry point. */
	bool CalcFrustum();

private:
	/** Math for mpcdi 2D profile. */
	bool CalcFrustum2D();

	/** Math for mpcdi 3D profile. */
	bool CalcFrustum3D();

	/** Math for mpcdi A3D profile. */
	bool CalcFrustumA3D();

	/** Math for mpcdi SL profile. */
	bool CalcFrustumSL();

protected:
	/** Calculate projection matrix from angles*/
	FMatrix CalcProjectionMatrix(const FDisplayClusterWarpProjection& InWarpProjection);

	/** Get warp profile type. */
	inline EDisplayClusterWarpProfileType  GetWarpProfileType() const
	{
		return GeometryContext.GetWarpProfileType();
	}

public:
	inline void SetParameter(EDisplayClusterWarpBlendProjectionType  InParameterValue)
	{ 
		WarpData.ProjectionType = InParameterValue;

		// Protect runtime methods:
		if (WarpData.ProjectionType >= EDisplayClusterWarpBlendProjectionType::RuntimeProjectionModes)
		{
			// Runtime method disabled for console vars
			WarpData.ProjectionType = EDisplayClusterWarpBlendProjectionType::DynamicAABBCenter;
		}
	}

private:
	bool ShouldRotateFrustumToFitContextSize() const;

	inline bool ImplUpdateProjectionType()
	{
		switch (WarpData.ProjectionType)
		{
		// Accept any frustum for aabb center method
		case EDisplayClusterWarpBlendProjectionType::DynamicAABBCenter:
			break;

		case EDisplayClusterWarpBlendProjectionType::StaticSurfaceNormal:
			WarpData.ProjectionType = EDisplayClusterWarpBlendProjectionType::RuntimeStaticSurfaceNormalInverted; // Make mirror for backside eye position
			return true;

		case EDisplayClusterWarpBlendProjectionType::StaticSurfacePlane:
			WarpData.ProjectionType = EDisplayClusterWarpBlendProjectionType::RuntimeStaticSurfacePlaneInverted; // Make mirror for backside eye position
			return true;

		default:
			WarpData.ProjectionType = EDisplayClusterWarpBlendProjectionType::DynamicAABBCenter;
			return true;
		}

		// Accept current projection
		return false;
	}

	FMatrix GetUVMatrix(const FMatrix& InProjectionMatrix) const;

	bool ShouldUseCustomViewDirection() const;

	FVector GetViewTarget() const;

	/** Get view direction. */
	FVector GetViewDirection();

	/** Initialize warp projection data before search. */
	void InitializeWarpProjectionData();

	/** build projection plane matrices. */
	void ImplCalcViewProjectionMatrices();

	// Frustum build logics:
	bool ImplCalcFrustumProjection();
	bool ImplCalcFrustumProjection_AABB(const FMatrix& InGeometryToFrustumMatrix);
	bool ImplCalcFrustumProjection_WarpMap(const FMatrix& InGeometryToFrustumMatrix);
	bool ImplCalcFrustumProjection_WarpMesh(const FMatrix& InGeometryToFrustumMatrix);
	bool ImplCalcFrustumProjection_WarpProceduralMesh(const FMatrix& InGeometryToFrustumMatrix);
	bool ImplCalcFrustumProjection_MPCDIAttributes(const FMatrix& InGeometryToFrustumMatrix);

	bool EndCalcFrustum();

	// WarpBlend

	bool ShouldHandleOverrideCalcFrustum() const;
	bool OverrideCalcFrustum();

	/** 
	 * When necessary, rotates by 90 deg. the frustum to better fit the aspect ratio of the viewport.
	 */
	void ImplFitFrustumToContextSize();

private:
	static bool GetProjectionClip(const FVector4& InPoint, const FMatrix& InGeometryToFrustumA3D, FDisplayClusterWarpProjection& InOutWarpProjection);

private:
	FDisplayClusterWarpBlend_GeometryContext& GeometryContext;

	// context data
	FDisplayClusterWarpData& WarpData;

	// legacy WarpMap, now always use x16 downscale ratio
	const int32 WarpMapLODRatio = 16;
};
