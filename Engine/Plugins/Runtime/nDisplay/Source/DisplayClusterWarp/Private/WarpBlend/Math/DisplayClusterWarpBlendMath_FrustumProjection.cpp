// Copyright Epic Games, Inc. All Rights Reserved.

#include "WarpBlend/Math/DisplayClusterWarpBlendMath_Frustum.h"

#include "Render/Containers/IDisplayClusterRender_Texture.h"
#include "Render/Viewport/IDisplayClusterViewport.h"
#include "Misc/DisplayClusterHelpers.h"

#include "WarpBlend/Exporter/DisplayClusterWarpBlendExporter_WarpMap.h"

#include "StaticMeshResources.h"
#include "ProceduralMeshComponent.h"

//---------------------------------------------------------------------------------------
// FDisplayClusterWarpBlendMath_Frustum
//---------------------------------------------------------------------------------------
bool FDisplayClusterWarpBlendMath_Frustum::ImplCalcFrustumProjection()
{
	// build projection plane
	ImplCalcViewProjectionMatrices();

	// Reset extend of the frustum
	WarpData.GeometryWarpProjection.ResetProjectionAngles();

	const FMatrix GeometryToFrustumMatrix = GeometryContext.Context.GeometryToOrigin * WarpData.Local2World.Inverse();

	switch (GeometryContext.GeometryProxy.FrustumGeometryType)
	{
	case EDisplayClusterWarpFrustumGeometryType::WarpMap:
		return ImplCalcFrustumProjection_WarpMap(GeometryToFrustumMatrix);

	case EDisplayClusterWarpFrustumGeometryType::WarpMesh:
		return ImplCalcFrustumProjection_WarpMesh(GeometryToFrustumMatrix);

	case EDisplayClusterWarpFrustumGeometryType::WarpProceduralMesh:
		return ImplCalcFrustumProjection_WarpProceduralMesh(GeometryToFrustumMatrix);

	case EDisplayClusterWarpFrustumGeometryType::MPCDIAttributes:
		return ImplCalcFrustumProjection_MPCDIAttributes(GeometryToFrustumMatrix);

	default:
		break;
	}

	return false;
}

bool FDisplayClusterWarpBlendMath_Frustum::ImplCalcFrustumProjection_MPCDIAttributes(const FMatrix& InGeometryToFrustumMatrix)
{
	const FDisplayClusterWarpMPCDIAttributes& MPCDIAttributes = GeometryContext.GeometryProxy.MPCDIAttributes;
	switch (GetWarpProfileType())
	{
	case EDisplayClusterWarpProfileType::warp_2D:
	{
		// Create geometry from attributes
		TArray<FVector> ScreenPoints;
		FDisplayClusterWarpBlendExporter_WarpMap::Get2DProfileGeometry(MPCDIAttributes, ScreenPoints);

		bool bOutOfFrustumPoints = false;

		// Search a camera space frustum
		for (const FVector& Pts : ScreenPoints)
		{
			bOutOfFrustumPoints |= !GetProjectionClip(Pts, InGeometryToFrustumMatrix, WarpData.GeometryWarpProjection);
		}

		return bOutOfFrustumPoints == false;
	}

	case EDisplayClusterWarpProfileType::warp_3D:
	{
		// Todo: need to be implemeted. This is an experimental code
		const float ZNear = WarpData.GeometryWarpProjection.ZFar;

		// Frustum angles XYZW = LRTB in degrees
		WarpData.GeometryWarpProjection.Left = (ZNear * FMath::Tan(FMath::DegreesToRadians(MPCDIAttributes.Frustum.Angles.X)));
		WarpData.GeometryWarpProjection.Right = (ZNear * FMath::Tan(FMath::DegreesToRadians(MPCDIAttributes.Frustum.Angles.Y)));
		WarpData.GeometryWarpProjection.Top = (ZNear * FMath::Tan(FMath::DegreesToRadians(MPCDIAttributes.Frustum.Angles.Z)));
		WarpData.GeometryWarpProjection.Bottom = (ZNear * FMath::Tan(FMath::DegreesToRadians(MPCDIAttributes.Frustum.Angles.W)));
	}
	return true;

	default:
		break;
	}

	return false;
}

void FDisplayClusterWarpBlendMath_Frustum::ImplCalcViewProjectionMatrices()
{
	// Customize view direction
	FVector ViewDir = GetViewDirection();

	if (!ShouldUseCustomViewDirection())
	{
		switch (WarpData.ProjectionType)
		{
		case EDisplayClusterWarpBlendProjectionType::StaticSurfacePlane:
			ViewDir = GeometryContext.Context.SurfaceViewPlane;
			break;

		case EDisplayClusterWarpBlendProjectionType::RuntimeStaticSurfacePlaneInverted:
			// Frustum projection fix (back-side view planes)
			ViewDir = -GeometryContext.Context.SurfaceViewPlane;
			break;

		case EDisplayClusterWarpBlendProjectionType::StaticSurfaceNormal:
			// Use fixed surface view normal
			ViewDir = GeometryContext.Context.SurfaceViewNormal;
			break;

		case EDisplayClusterWarpBlendProjectionType::RuntimeStaticSurfaceNormalInverted:
			// Frustum projection fix (back-side view planes)
			ViewDir = -GeometryContext.Context.SurfaceViewNormal;
			break;

		default:
			break;
		}
	}

	FVector const NewX = ViewDir.GetSafeNormal();
	if (FMath::Abs(NewX.Z) < (1.f - KINDA_SMALL_NUMBER))
	{
		WarpData.Local2World = FRotationMatrix::MakeFromXZ(NewX, FVector(0.f, 0.f, 1.f));
	}
	else
	{
		WarpData.Local2World = FRotationMatrix::MakeFromXY(NewX, FVector(0.f, 1.f, 0.f));
	}

	// Finally, set the eye location for the view matrix.
	WarpData.Local2World.SetOrigin(WarpData.GeometryWarpProjection.EyeLocation);
}

void FDisplayClusterWarpBlendMath_Frustum::InitializeWarpProjectionData()
{
	check(WarpData.WarpEye);

	// Reset extend of the frustum
	WarpData.GeometryWarpProjection = FDisplayClusterWarpProjection();

	if (IDisplayClusterViewport* Viewport = WarpData.WarpEye->GetViewport())
	{
		const FVector2D ClippingPLanes = Viewport->GetClippingPlanes();

		WarpData.GeometryWarpProjection.ZNear = ClippingPLanes.X;
		WarpData.GeometryWarpProjection.ZFar  = ClippingPLanes.Y;
	}
	else
	{
		WarpData.GeometryWarpProjection.ZNear = GNearClippingPlane;
		WarpData.GeometryWarpProjection.ZFar = WarpData.GeometryWarpProjection.ZNear;
	}

	WarpData.GeometryWarpProjection.WorldScale = WarpData.WarpEye->WorldScale;

	WarpData.GeometryWarpProjection.EyeLocation = WarpData.WarpEye->ViewPoint.GetEyeLocation();
}

bool FDisplayClusterWarpBlendMath_Frustum::ImplCalcFrustumProjection_AABB(const FMatrix& InGeometryToFrustumMatrix)
{
	bool bOutOfFrustumPoints = false;

	// Search a camera space frustum
	for (int32 AABBPointIndex = 0; AABBPointIndex < 8; AABBPointIndex++)
	{
		bOutOfFrustumPoints |= !GetProjectionClip(GeometryContext.Context.AABBox.GetAABBPts(AABBPointIndex), InGeometryToFrustumMatrix, WarpData.GeometryWarpProjection);
	}

	return bOutOfFrustumPoints == false;
}

bool FDisplayClusterWarpBlendMath_Frustum::ImplCalcFrustumProjection_WarpMap(const FMatrix& InGeometryToFrustumMatrix)
{
	const TSharedPtr<IDisplayClusterRender_Texture, ESPMode::ThreadSafe>& WarpMapTexture = GeometryContext.GeometryProxy.WarpMapTexture;
	if (const FVector4f* SourcePts = (FVector4f*)(WarpMapTexture.IsValid() ? WarpMapTexture->GetData() : nullptr))
	{
		const FIntPoint WarpMapTextureSize(WarpMapTexture->GetWidth(), WarpMapTexture->GetHeight());
		if (WarpMapTextureSize.GetMin() < 2)
		{
			// Minimal texture size should be at least 2x2
			return false;
		}

		bool bAllPointsInFrustum = true;

		const int32 PointsAmount = WarpMapTextureSize.X * WarpMapTextureSize.Y;

		switch (WarpData.FrustumType)
		{
		case EDisplayClusterWarpBlendFrustumType::AABB:
			// WarpMap supports AABB frustum
			return ImplCalcFrustumProjection_AABB(InGeometryToFrustumMatrix);

		case EDisplayClusterWarpBlendFrustumType::FULL:
			// Search a camera space frustum
			for (int32 PointIndex = 0; PointIndex < PointsAmount; ++PointIndex)
			{
				const FVector4 Pts(SourcePts[PointIndex]);
				if (!GetProjectionClip(Pts, InGeometryToFrustumMatrix, WarpData.GeometryWarpProjection))
				{
					bAllPointsInFrustum = false;
				}
			}
			return bAllPointsInFrustum;

		case EDisplayClusterWarpBlendFrustumType::LOD:
		{
			const TArray<int32>& IndexLOD = GeometryContext.GeometryProxy.GeometryCache.IndexLOD;
			if (IndexLOD.Num() == 0)
			{
				check(WarpMapLODRatio > 0);

				GeometryContext.GeometryProxy.UpdateFrustumGeometryLOD(FIntPoint(WarpMapLODRatio));
			}

			// Search a camera space frustum
			for (const int32& PtsIndex : IndexLOD)
			{
				if (PtsIndex >= 0 && PtsIndex < PointsAmount)
				{
					const FVector4 Pts(SourcePts[PtsIndex]);
					if (!GetProjectionClip(Pts, InGeometryToFrustumMatrix, WarpData.GeometryWarpProjection))
					{
						bAllPointsInFrustum = false;
					}
				}
			}
			return bAllPointsInFrustum;
		}

		default:
			break;
		}
	}

	return false;
}

bool FDisplayClusterWarpBlendMath_Frustum::ImplCalcFrustumProjection_WarpMesh(const FMatrix& InGeometryToFrustumMatrix)
{
	if (!GeometryContext.GeometryProxy.WarpMeshComponent.IsValid())
	{
		return false;
	}

	if (WarpData.FrustumType == EDisplayClusterWarpBlendFrustumType::AABB)
	{
		// WarpMesh supports AABB frustum
		return ImplCalcFrustumProjection_AABB(InGeometryToFrustumMatrix);
	}

	bool bAllPointsInFrustum = true;

	const FStaticMeshLODResources* StaticMeshLODResources = GeometryContext.GeometryProxy.GetStaticMeshComponentLODResources();
	if (StaticMeshLODResources == nullptr)
	{
		return false;
	}

	const FPositionVertexBuffer& VertexPosition = StaticMeshLODResources->VertexBuffers.PositionVertexBuffer;
	for (uint32 i = 0; i < VertexPosition.GetNumVertices(); i++)
	{
		if (!GetProjectionClip(FVector4(FVector(VertexPosition.VertexPosition(i)), 1.f), InGeometryToFrustumMatrix, WarpData.GeometryWarpProjection))
		{
			bAllPointsInFrustum = false;
		}
	}

	return bAllPointsInFrustum;
}

bool FDisplayClusterWarpBlendMath_Frustum::ImplCalcFrustumProjection_WarpProceduralMesh(const FMatrix& InGeometryToFrustumMatrix)
{
	if (!GeometryContext.GeometryProxy.WarpMeshComponent.IsValid())
	{
		return false;
	}

	if (WarpData.FrustumType == EDisplayClusterWarpBlendFrustumType::AABB)
	{
		// WarpProceduralMesh supports AABB frustum
		return ImplCalcFrustumProjection_AABB(InGeometryToFrustumMatrix);
	}

	bool bAllPointsInFrustum = true;

	const FProcMeshSection* ProcMeshSection = GeometryContext.GeometryProxy.WarpMeshComponent->GetProceduralMeshComponentSection();
	if (ProcMeshSection == nullptr)
	{
		return false;
	}

	for (const FProcMeshVertex& VertexIt : ProcMeshSection->ProcVertexBuffer)
	{
		if (!GetProjectionClip(FVector4(VertexIt.Position, 1), InGeometryToFrustumMatrix, WarpData.GeometryWarpProjection))
		{
			bAllPointsInFrustum = false;
		}
	}

	return bAllPointsInFrustum;
}
