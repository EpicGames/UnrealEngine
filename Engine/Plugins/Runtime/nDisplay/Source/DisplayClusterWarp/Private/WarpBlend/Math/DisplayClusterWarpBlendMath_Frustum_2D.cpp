// Copyright Epic Games, Inc. All Rights Reserved.

#include "WarpBlend/Math/DisplayClusterWarpBlendMath_Frustum.h"
#include "WarpBlend/Exporter/DisplayClusterWarpBlendExporter_WarpMap.h"
#include "WarpBlend/DisplayClusterWarpBlend.h"

#include "Render/Viewport/IDisplayClusterViewport.h"
#include "Render/Containers/IDisplayClusterRender_Texture.h"
#include "Misc/DisplayClusterHelpers.h"

//------------------------------------------------------------
// FDisplayClusterWarpBlendMath_Frustum
//------------------------------------------------------------
bool FDisplayClusterWarpBlendMath_Frustum::CalcFrustum2D()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(DCWarpBlendMath_Frustum::CalcFrustum2D);

	// Calc projection
	return ImplCalcFrustumProjection() && EndCalcFrustum();
}
