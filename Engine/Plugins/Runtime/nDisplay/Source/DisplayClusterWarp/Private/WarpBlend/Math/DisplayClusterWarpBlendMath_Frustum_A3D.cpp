// Copyright Epic Games, Inc. All Rights Reserved.

#include "WarpBlend/Math/DisplayClusterWarpBlendMath_Frustum.h"

#include "Render/Containers/IDisplayClusterRender_Texture.h"
#include "Render/Viewport/IDisplayClusterViewport.h"
#include "Misc/DisplayClusterHelpers.h"

#include "StaticMeshResources.h"
#include "ProceduralMeshComponent.h"

//---------------------------------------------------------------------------------------
// FDisplayClusterWarpBlendMath_Frustum
//---------------------------------------------------------------------------------------
bool FDisplayClusterWarpBlendMath_Frustum::CalcFrustumA3D()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(DCWarpBlendMath_Frustum::CalcFrustumA3D);

	if (!GeometryContext.GeometryProxy.bIsGeometryValid)
	{
		// Bad source geometry
		return false;
	}

	// When a custom ViewTarget is used, we do not use adaptation algorithms.
	if (ShouldUseCustomViewDirection())
	{
		return ImplCalcFrustumProjection() && EndCalcFrustum();
	}

	if (WarpData.bFindBestProjectionType == false)
	{
		ImplCalcFrustumProjection();
	}
	else
	{
		// Frustum projection with auto-fix (back-side view planes)
		// Projection type changed runtime. Validate frustum points, all must be over view plane:
		EDisplayClusterWarpBlendProjectionType BaseProjectionType = WarpData.ProjectionType;
		EDisplayClusterWarpBlendFrustumType    BaseFrustumType = WarpData.FrustumType;

		// Optimize for PerfectCPU:
		if (WarpData.FrustumType == EDisplayClusterWarpBlendFrustumType::FULL && GeometryContext.GeometryProxy.FrustumGeometryType == EDisplayClusterWarpFrustumGeometryType::WarpMap)
		{
			while (true)
			{
				WarpData.FrustumType = EDisplayClusterWarpBlendFrustumType::LOD;

				// Fast check for bad view
				if (ImplCalcFrustumProjection())
				{
					break;
				}

				if (!ImplUpdateProjectionType())
				{
					break;
				}
			}
		}

		// Restore base parameter
		WarpData.FrustumType = BaseFrustumType;

		// Search valid projection mode:
		if (BaseProjectionType == WarpData.ProjectionType)
		{

			while (true)
			{
				// Full check for bad projection:
				if (ImplCalcFrustumProjection())
				{
					break;
				}

				if (!ImplUpdateProjectionType())
				{
					break;
				}
			}
		}
	}

	return EndCalcFrustum();
}
