// Copyright Epic Games, Inc. All Rights Reserved.

#include "WarpBlend/Math/DisplayClusterWarpBlendMath_Frustum.h"

#include "Render/Containers/IDisplayClusterRender_Texture.h"
#include "Render/Viewport/IDisplayClusterViewport.h"
#include "Misc/DisplayClusterHelpers.h"

//------------------------------------------------------------
// FDisplayClusterWarpBlendMath_Frustum
//------------------------------------------------------------
bool FDisplayClusterWarpBlendMath_Frustum::CalcFrustumSL()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(DCWarpBlendMath_Frustum::CalcFrustumSL);

	// NOT IMPLEMENTED
	// Currently we have implemented only the math of the A3D profile. We have to write a new implementation to support the SL profile.
	// Some of the math from the A3D profile can be reused.

	return false;
}
