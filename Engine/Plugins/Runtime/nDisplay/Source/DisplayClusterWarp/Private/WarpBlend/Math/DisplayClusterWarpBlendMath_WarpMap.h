// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/DisplayClusterWarpAABB.h"
#include "Render/Containers/IDisplayClusterRender_Texture.h"

/**
 * Implement math for WarpMap texture as a source of geometry
 */
class FDisplayClusterWarpBlendMath_WarpMap
{
public:
	FDisplayClusterWarpBlendMath_WarpMap(const TSharedRef<IDisplayClusterRender_Texture, ESPMode::ThreadSafe>& InWarpMapTexture)
		: WarpMapTexture(InWarpMapTexture)
	{ }

	/** Check if WarpMap data can be used. */
	inline bool CanBeUsed() const
	{
		return GetWidth() >= 2
			&& GetHeight() >= 2
			&& GetData();
	}

public:
	/** Build a bbox from valid points. */
	FBox GetAABBox() const
	{
		// The input texture must be valid and have a size of at least 2x2.
		if (!CanBeUsed())
		{
			return FDisplayClusterWarpAABB();
		}

		FDisplayClusterWarpAABB WarpAABB;
		for (int32 Y = 0; Y < GetHeight(); ++Y)
		{
			for (int32 X = 0; X < GetWidth(); ++X)
			{
				WarpAABB.UpdateAABB(GetPoint(X, Y));
			}
		}

		return WarpAABB;
	}

	/** Build an LOD mesh with the size specified in InSizeLOD. */
	bool BuildIndexLOD(const FIntPoint& InSizeLOD, TArray<int32>& OutIndexLOD)
	{
		OutIndexLOD.Reset();

		// The input texture must be valid and have a size of at least 2x2.
		if (!CanBeUsed())
		{
			return false;
		}

		// Let's limit the LOD size to the size of the input texture.
		const int32 DivX = FMath::Clamp(InSizeLOD.X, 2, GetWidth());
		const int32 DivY = FMath::Clamp(InSizeLOD.Y, 2, GetHeight());

		OutIndexLOD.Reserve(DivX * DivY);

		// Generate valid points for texturebox method:
		for (int32 low_y = 0; low_y < DivY; low_y++)
		{
			int32 y = (GetHeight() - 1) * (float(low_y) / (DivY - 1)); //-V609 UE-349513 

			for (int32 low_x = 0; low_x < DivX; low_x++)
			{
				int32 x = (GetWidth() - 1) * (float(low_x) / (DivX - 1)); //-V609 UE-349513 

				if (IsValidPoint(x, y))
				{
					// Just use direct point
					OutIndexLOD.Add(GetPointIndex(x, y));
				}
				else
				{
					// Search for nearset valid point
					if (FindValidPoint(x, y))
					{
						OutIndexLOD.Add(GetSavedPointIndex());
					}
				}
			}
		}

		return true;
	}

private:
	int32 GetSavedPointIndex()
	{
		return GetPointIndex(ValidPointX, ValidPointY);
	}

	FORCEINLINE int32 GetPointIndex(int32 InX, int32 InY) const
	{
		return InX + InY * GetWidth();
	}

	FORCEINLINE const FVector4f& GetPoint(int32 InX, int32 InY) const
	{
		return GetData()[GetPointIndex(InX, InY)];
	}

	FORCEINLINE const FVector4f& GetPoint(int32 PointIndex) const
	{
		return GetData()[PointIndex];
	}

	bool IsValidPoint(int32 InX, int32 InY)
	{
		return GetPoint(InX, InY).W > 0;
	}

	bool FindValidPoint(int32 InX, int32 InY)
	{
		X0 = InX;
		Y0 = InY;

		for (int32 Range = 1; Range < GetWidth(); Range++)
		{
			if (FindValidPointInRange(Range))
			{
				return true;
			}
		}

		return false;
	}

public:
	FVector GetSurfaceViewNormal()
	{
		// The input texture must be valid and have a size of at least 2x2.
		if (!CanBeUsed())
		{
			return FVector::ZeroVector;
		}

		int32 Ncount = 0;
		double Nxyz[3] = { 0,0,0 };

		for (int32 ItY = 0; ItY < (GetHeight() - 2); ++ItY)
		{
			for (int32 ItX = 0; ItX < (GetWidth() - 2); ++ItX)
			{
				const FVector4f& Pts0 = GetPoint(ItX, ItY);
				const FVector4f& Pts1 = GetPoint(ItX + 1, ItY);
				const FVector4f& Pts2 = GetPoint(ItX, ItY + 1);

				if (Pts0.W > 0 && Pts1.W > 0 && Pts2.W > 0)
				{
					const FVector N1 = FVector4(Pts1 - Pts0);
					const FVector N2 = FVector4(Pts2 - Pts0);
					const FVector N = FVector::CrossProduct(N2, N1).GetSafeNormal();

					for (int32 AxisIndex = 0; AxisIndex < 3; AxisIndex++)
					{
						Nxyz[AxisIndex] += N[AxisIndex];
					}

					Ncount++;
				}
			}
		}

		if (!Ncount)
		{
			return FVector::ZeroVector;
		}

		double Scale = double(1) / Ncount;
		for (int32 AxisIndex = 0; AxisIndex < 3; AxisIndex++)
		{
			Nxyz[AxisIndex] *= Scale;
		}

		return FVector(Nxyz[0], Nxyz[1], Nxyz[2]).GetSafeNormal();
	}

	FVector GetSurfaceViewPlane()
	{
		// The input texture must be valid and have a size of at least 2x2.
		if (!CanBeUsed())
		{
			return FVector::ZeroVector;
		}

		const FVector4f& Pts0 = GetValidPoint(0, 0);
		const FVector4f& Pts1 = GetValidPoint(GetWidth() - 1, 0);
		const FVector4f& Pts2 = GetValidPoint(0, GetHeight() - 1);

		const FVector N1 = FVector4(Pts1 - Pts0);
		const FVector N2 = FVector4(Pts2 - Pts0);
		return FVector::CrossProduct(N2, N1).GetSafeNormal();
	}

private:
	const FVector4f& GetValidPoint(int32 InX, int32 InY)
	{
		if (!IsValidPoint(InX, InY))
		{
			if (FindValidPoint(InX, InY))
			{
				return GetPoint(GetSavedPointIndex());
			}
		}

		return GetPoint(InX, InY);
	}

	bool FindValidPointInRange(int32 Range)
	{
		for (int32 RangeIt = -Range; RangeIt <= Range; RangeIt++)
		{
			// Top or bottom rows
			if (IsValid(X0 + RangeIt, Y0 - Range) || IsValid(X0 + RangeIt, Y0 + Range))
			{
				return true;
			}

			// Left or Right columns
			if (IsValid(X0 - Range, Y0 + RangeIt) || IsValid(X0 + Range, Y0 + RangeIt))
			{
				return true;
			}
		}

		return false;
	}

	bool IsValid(int32 newX, int32 newY)
	{
		if (newX < 0 || newY < 0 || newX >= GetWidth() || newY >= GetHeight())
		{
			// Out of texture
			return false;
		}

		if (GetData()[GetPointIndex(newX, newY)].W > 0)
		{
			// Store valid result
			ValidPointX = newX;
			ValidPointY = newY;

			return true;
		}

		return false;
	}

private:

	/** Return WarpMap texture width. */
	inline const int32 GetWidth() const
	{
		return WarpMapTexture->GetWidth();
	}

	/** Return WarpMap texture height. */
	inline const int32 GetHeight() const
	{
		return WarpMapTexture->GetHeight();
	}

	/** Return WarpMap texture points data. */
	inline const FVector4f* GetData() const
	{
		return (FVector4f*)WarpMapTexture->GetData();
	}

private:
	// WarpMap texture reference
	const TSharedRef<IDisplayClusterRender_Texture, ESPMode::ThreadSafe> WarpMapTexture;

	// Internal logic
	int32 ValidPointX = 0;
	int32 ValidPointY = 0;

	int32 X0 = 0;
	int32 Y0 = 0;
};
