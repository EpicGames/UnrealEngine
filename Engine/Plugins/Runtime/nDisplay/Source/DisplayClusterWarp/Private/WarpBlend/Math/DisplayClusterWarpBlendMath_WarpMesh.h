// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "StaticMeshResources.h"
#include "Containers/DisplayClusterWarpAABB.h"

/**
 * Implement math for StaticMesh component as a source of geometry
 */
class FDisplayClusterWarpBlendMath_WarpMesh
{
public:
	FDisplayClusterWarpBlendMath_WarpMesh(const FStaticMeshLODResources& InMeshLODResources)
		: MeshLODResources(InMeshLODResources)
	{ }

public:
	FBox CalcAABBox()
	{
		FDisplayClusterWarpAABB WarpAABB;
		const FPositionVertexBuffer& PositionBuffer = MeshLODResources.VertexBuffers.PositionVertexBuffer;
		for (uint32 VertIdx = 0; VertIdx < PositionBuffer.GetNumVertices(); ++VertIdx)
		{
			WarpAABB.UpdateAABB((FVector)PositionBuffer.VertexPosition(VertIdx));
		}

		return WarpAABB;
	}

	void CalcSurfaceVectors(FVector& OutSurfaceViewNormal, FVector& OutSurfaceViewPlane)
	{
		// Calc static normal and plane
		const int32 IdxNum = MeshLODResources.IndexBuffer.GetNumIndices();
		const int32 TriNum = IdxNum / 3;

		if (TriNum <= 0)
		{
			return;
		}

		FVector SurfaceNormalViewSum = FVector::ZeroVector;
		const FPositionVertexBuffer& PositionBuffer = MeshLODResources.VertexBuffers.PositionVertexBuffer;

		for (int32 TriIdx = 0; TriIdx < TriNum; ++TriIdx)
		{
			const int32 Index0 = MeshLODResources.IndexBuffer.GetIndex(TriIdx * 3 + 0);
			const int32 Index1 = MeshLODResources.IndexBuffer.GetIndex(TriIdx * 3 + 1);
			const int32 Index2 = MeshLODResources.IndexBuffer.GetIndex(TriIdx * 3 + 2);

			const FVector& Pts1 = (FVector)PositionBuffer.VertexPosition(Index0);
			const FVector& Pts0 = (FVector)PositionBuffer.VertexPosition(Index1);
			const FVector& Pts2 = (FVector)PositionBuffer.VertexPosition(Index2);

			const FVector N1 = Pts1 - Pts0;
			const FVector N2 = Pts2 - Pts0;

			const FVector Normal = FVector::CrossProduct(N2, N1);
			const double Magnitude = Normal.Length();

			SurfaceNormalViewSum += (Normal * Magnitude);
		}

		OutSurfaceViewNormal = SurfaceNormalViewSum.GetSafeNormal();

		//@todo: MeshSurfaceViewPlane not implemented, use MeshSurfaceViewNormal
		// this is for the case when normals are set in the side direction to have same sampling quality over the multiple heavily curved surfaces
		OutSurfaceViewPlane = OutSurfaceViewNormal;
	}

private:
	const FStaticMeshLODResources& MeshLODResources;
};
