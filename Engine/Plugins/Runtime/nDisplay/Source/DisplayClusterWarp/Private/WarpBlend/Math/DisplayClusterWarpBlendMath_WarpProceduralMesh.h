// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ProceduralMeshComponent.h"
#include "Containers/DisplayClusterWarpAABB.h"

/**
 * Implement math for ProceduralMesh component as a source of geometry
 */
class FDisplayClusterWarpBlendMath_WarpProceduralMesh
{
public:
	FDisplayClusterWarpBlendMath_WarpProceduralMesh(const FProcMeshSection& InProcMeshSection)
		: ProcMeshSection(InProcMeshSection)
	{ }

public:
	FBox CalcAABBox()
	{
		FDisplayClusterWarpAABB WarpAABB;
		for (const FProcMeshVertex& VertexIt : ProcMeshSection.ProcVertexBuffer)
		{
			WarpAABB.UpdateAABB(VertexIt.Position);
		}

		return WarpAABB;
	}

	void CalcSurfaceVectors(FVector& OutSurfaceViewNormal, FVector& OutSurfaceViewPlane)
	{
		// Calc static normal and plane
		const int32 IdxNum = ProcMeshSection.ProcIndexBuffer.Num();
		const int32 TriNum = IdxNum / 3;

		if (TriNum <= 0)
		{
			return;
		}

		// Calc static normal and plane
		bool bAverageNormalValid = false;
		FVector AverageNormal = FVector::ZeroVector;

		for (int32 TriIdx = 0; TriIdx < TriNum; ++TriIdx)
		{
			const uint32& Index0 = ProcMeshSection.ProcIndexBuffer[TriIdx * 3 + 0];
			const uint32& Index1 = ProcMeshSection.ProcIndexBuffer[TriIdx * 3 + 1];
			const uint32& Index2 = ProcMeshSection.ProcIndexBuffer[TriIdx * 3 + 2];

			const FVector& Pts0 = ProcMeshSection.ProcVertexBuffer[Index0].Position;
			const FVector& Pts1 = ProcMeshSection.ProcVertexBuffer[Index1].Position;
			const FVector& Pts2 = ProcMeshSection.ProcVertexBuffer[Index2].Position;

			const FVector N1 = Pts0 - Pts1;
			const FVector N2 = Pts2 - Pts1;

			const FVector N = FVector::CrossProduct(N2, N1).GetSafeNormal();

			AverageNormal = bAverageNormalValid ? (N + AverageNormal).GetSafeNormal() : N;

			bAverageNormalValid = true;
		}

		OutSurfaceViewNormal = AverageNormal;

		//@todo: MeshSurfaceViewPlane not implemented, use MeshSurfaceViewNormal
		OutSurfaceViewPlane = OutSurfaceViewNormal;
	}

private:
	const FProcMeshSection& ProcMeshSection;
};
