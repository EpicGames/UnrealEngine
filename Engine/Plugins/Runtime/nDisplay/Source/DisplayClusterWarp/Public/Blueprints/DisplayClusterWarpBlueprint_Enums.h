// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"

#include "DisplayClusterWarpBlueprint_Enums.generated.h"

/**
 * Projection mode for the camera that is used as an image source
 * This projection does not directly use slices from the camera image,
 * but calculates the camera sub-frustum used to render the sub-images of camera for a particular viewport.
 * Since the aspect ratio of the AABB geometry in the viewport cannot be equal
 * to the aspect ratio of the camera, we must use several methods to fit the camera to the geometry.
 */
UENUM()
enum class EDisplayClusterWarpCameraProjectionMode : uint8
{
	/** Fit the stage geometry entirely within the camera's frustum */
	Fit UMETA(DisplayName = "Fit"),

	/** Fill the camera's frustum entire with the stage geometry */
	Fill  UMETA(DisplayName = "Fill"),
};

/** A set of modes used to determine the view target of the stage's geometry frustum */
UENUM()
enum class EDisplayClusterWarpCameraViewTarget
{
	/** The camera will point in the direction of the geometric center of the stage's geometry */
	GeometricCenter UMETA(DisplayName = "Geometric Center"),

	/** The camera will point in the same direction as the frustum fit view point. */
	MatchViewOrigin UMETA(DisplayName = "Match View Point")
};
