// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "DisplayClusterWarpGeometry.generated.h"

/**
 * 3D geometry that can be used for warping, in an PFM-like format
 * UE scale used: 1 unit = 1 centimeter
 */
USTRUCT(BlueprintType, Category = "nDisplay|WarpGeometry")
struct FDisplayClusterWarpGeometryPFM
{
	GENERATED_BODY()

public:
	/** Number of vertices horizontally. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "nDisplay|WarpGeometry")
	int32 Width = 0;

	/** Number of vertices vertically. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "nDisplay|WarpGeometry")
	int32 Height = 0;

	/** An array with vertices. The total number of vertices in this array must be equal to Width*Height. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "nDisplay|WarpGeometry")
	TArray<FVector>   Vertices;
};

/**
 * 3D geometry that can be used for warping, in an OBJ-like format
 */
USTRUCT(BlueprintType, Category = "nDisplay|WarpGeometry")
struct FDisplayClusterWarpGeometryOBJ
{
	GENERATED_BODY()

public:
	void PostAddFace(int32 f0, int32 f1, int32 f2);

public:
	/** Vertices of the mesh. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "nDisplay|WarpGeometry")
	TArray<FVector>   Vertices;

	/** Normal of the vertices of the mesh. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "nDisplay|WarpGeometry")
	TArray<FVector>   Normal;

	/** UV of the vertices of the mesh. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "nDisplay|WarpGeometry")
	TArray<FVector2D> UV;

	/** Triangles with mesh vertex indices. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "nDisplay|WarpGeometry")
	TArray<int32>   Triangles;
};
