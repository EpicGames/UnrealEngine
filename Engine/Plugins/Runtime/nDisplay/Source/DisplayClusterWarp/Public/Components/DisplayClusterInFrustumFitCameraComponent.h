// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"

#include "Components/DisplayClusterCameraComponent.h"
#include "Blueprints/DisplayClusterWarpBlueprint_Enums.h"

#include "DisplayClusterInFrustumFitCameraComponent.generated.h"

class ACineCameraActor;
class UCameraComponent;
class IDisplayClusterWarpPolicy;

/**
 * 3D point in space used to project the camera view onto a group of nDisplay viewports.
 * Support projection policies: mpcdi/pfm 2d/a3d, mesh.
 */
UCLASS(ClassGroup = (DisplayCluster), meta = (BlueprintSpawnableComponent, DisplayName = "NDisplay Frustum Fit View Point"))
class DISPLAYCLUSTERWARP_API UDisplayClusterInFrustumFitCameraComponent : public UDisplayClusterCameraComponent
{
	GENERATED_BODY()

public:
	UDisplayClusterInFrustumFitCameraComponent(const FObjectInitializer& ObjectInitializer);

	//~Begin UActorComponent
	virtual void OnRegister() override;
	virtual void TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	//~~~End UActorComponent

	//~Begin UObject
#if WITH_EDITOR
	virtual bool CanEditChange(const FProperty* InProperty) const override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~~~End UObject

	//~Begin UDisplayClusterCameraComponent
	//virtual void GetDesiredView(IDisplayClusterViewportConfiguration& InViewportConfiguration, FMinimalViewInfo& InOutViewInfo, float* OutCustomNearClippingPlane = nullptr) override;
	virtual bool ShouldUseEntireClusterViewports(IDisplayClusterViewportManager* InViewportManager) const override;
	virtual IDisplayClusterWarpPolicy* GetWarpPolicy(IDisplayClusterViewportManager* InViewportManager) override;

	virtual TObjectPtr<UMaterial> GetDisplayDeviceMaterial(const EDisplayClusterDisplayDeviceMeshType InMeshType, const EDisplayClusterDisplayDeviceMaterialType InMaterialType) const override;
	virtual void OnUpdateDisplayDeviceMeshAndMaterialInstance(IDisplayClusterViewportPreview& InViewportPreview, const EDisplayClusterDisplayDeviceMeshType InMeshType, const EDisplayClusterDisplayDeviceMaterialType InMaterialType, UMeshComponent* InMeshComponent, UMaterialInstanceDynamic* InMeshMaterialInstance) const override;

	virtual bool IsViewPointOverrideCameraPosition() const override;
	virtual void GetEyePosition(const IDisplayClusterViewportConfiguration& InViewportConfiguration, FVector& OutViewLocation, FRotator& OutViewRotation) override;

protected:
	virtual bool IsICVFXCameraBeingUsed() const override;
	//~~End UDisplayClusterCameraComponent

public:
	/** Return component that used for configuration. */
	const UDisplayClusterInFrustumFitCameraComponent& GetConfigurationInFrustumFitCameraComponent(IDisplayClusterViewportConfiguration& InViewportConfiguration) const;

private:
	/** true, if camera projection is used. */
	bool IsEnabled() const;

public:
	/** Camera projection mode is used. */
	UPROPERTY(EditAnywhere, Category = "Frustum Fit", meta = (DisplayName = "Enable Frustum Fit"))
	bool bEnableCameraProjection = true;

	/** Enable special rendering mode for all viewports using this viewpoint. */
	UPROPERTY(EditAnywhere, Category = "Frustum Fit", meta = (DisplayName = "Frustum Fit Mode", EditCondition = "bEnableCameraProjection", EditConditionHides))
	EDisplayClusterWarpCameraProjectionMode CameraProjectionMode = EDisplayClusterWarpCameraProjectionMode::Fit;

	/** Indicates which camera facing mode is used when frustum fitting the stage geometry */
	UPROPERTY(EditAnywhere, Category = "Frustum Fit", meta = (DisplayName = "Frustum Fit Target", EditCondition = "bEnableCameraProjection", EditConditionHides))
	EDisplayClusterWarpCameraViewTarget CameraViewTarget = EDisplayClusterWarpCameraViewTarget::GeometricCenter;

	/** Show additional warped preview meshes before the camera. */
	UPROPERTY(EditAnywhere, Category = "Frustum Fit", meta = (DisplayName = "Show Frustum Fit Preview", EditCondition = "bEnableCameraProjection", EditConditionHides))
	bool bShowPreviewFrustumFit = false;

	UE_DEPRECATED(5.5, "Use the camera settings from the UDisplayClusterCameraComponent instead")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use the Camera from the UDisplayClusterCameraComponent instead"))
	TSoftObjectPtr<ACineCameraActor> ExternalCameraActor;

	UE_DEPRECATED(5.5, "Use the bEnablePostProcess from the UDisplayClusterCameraComponent instead")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use the bEnablePostProcess from the UDisplayClusterCameraComponent instead"))
	bool bUseCameraPostprocess = false;

private:
	// a unique type of warp policy for this component
	// this policy class knows the properties of the component and implements the corresponding logic
	TSharedPtr<IDisplayClusterWarpPolicy, ESPMode::ThreadSafe> WarpPolicy;
};
