// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * Implement AABB math
 */
struct FDisplayClusterWarpAABB
	: public FBox
{
	FDisplayClusterWarpAABB()
		: FBox(FVector(FLT_MAX, FLT_MAX, FLT_MAX), FVector(-FLT_MAX, -FLT_MAX, -FLT_MAX))
	{ }

	/** Expand the value in AABB with the new values. */
	template<typename FArg>
	inline void UpdateAABB(const FArg X, const FArg Y, const FArg Z)
	{
		Min.X = FMath::Min(Min.X, X);
		Min.Y = FMath::Min(Min.Y, Y);
		Min.Z = FMath::Min(Min.Z, Z);

		Max.X = FMath::Max(Max.X, X);
		Max.Y = FMath::Max(Max.Y, Y);
		Max.Z = FMath::Max(Max.Z, Z);
	}

	/** Expand value in AABB with a new point. */
	inline void UpdateAABB(const FVector4f& InPts)
	{
		if (InPts.W > 0)
		{
			UpdateAABB(InPts.X, InPts.Y, InPts.Z);
		}
	}
	/** Expand value in AABB with a new point. */

	inline void UpdateAABB(const FVector& InPts)
	{
		UpdateAABB(InPts.X, InPts.Y, InPts.Z);
	}

	/** Expand the value in AABB using the input point list. */
	inline void UpdateAABB(const TArray<FVector>& InPoints)
	{
		for (const FVector& Pts : InPoints)
		{
			UpdateAABB(Pts);
		}
	}

	/** Expand the value in AABB using the input AABB. */
	inline void UpdateAABB(const FDisplayClusterWarpAABB& InAABB)
	{
		operator+=(InAABB);
	}

	/** The AABB defined by 8 points
	 * Get AABB 3d-cube points:
	 */
	inline FVector GetAABBPts(const uint32 InPtsIndex) const
	{
		switch (InPtsIndex)
		{
		case 0:
			return FVector(Max.X, Max.Y, Max.Z);

		case 1:
			return FVector(Max.X, Max.Y, Min.Z);

		case 2:
			return FVector(Min.X, Max.Y, Min.Z);

		case 3:
			return FVector(Min.X, Max.Y, Max.Z);

		case 4:
			return FVector(Max.X, Min.Y, Max.Z);

		case 5:
			return FVector(Max.X, Min.Y, Min.Z);

		case 6:
			return FVector(Min.X, Min.Y, Min.Z);

		case 7:
			return FVector(Min.X, Min.Y, Max.Z);

		default:
			break;
		}

		return GetCenter();
	}
};
