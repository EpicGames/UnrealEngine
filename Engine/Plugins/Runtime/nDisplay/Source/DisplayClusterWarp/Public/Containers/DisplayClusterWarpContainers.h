// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "Containers/DisplayClusterWarpEnums.h"

/**
 * Frustum projection angles.
 */
struct FDisplayClusterWarpProjection
{
	inline void ResetProjectionAngles()
	{
		Left = DBL_MAX;
		Right = -DBL_MAX;
		Top = -DBL_MAX;
		Bottom = DBL_MAX;
	}

	inline bool IsValidProjection() const
	{
		return Left < Right && Bottom < Top;
	}

	inline void ExpandProjectionAngles(const FDisplayClusterWarpProjection& In)
	{
		Left   = FMath::Min(Left,   In.Left);
		Right  = FMath::Max(Right,  In.Right);
		Top    = FMath::Max(Top,    In.Top);
		Bottom = FMath::Min(Bottom, In.Bottom);

		ZNear = In.ZNear;
		ZFar = In.ZFar;
	}

	inline void RotateProjectionAngles90Degree()
	{
		double InLeft   = Left;
		double InRight  = Right;
		double InTop    = Top;
		double InBottom = Bottom;

		Left   = InBottom;
		Right  = InTop;
		Top    = -InLeft;
		Bottom = -InRight;
	}

	inline double ConvertDegreesToProjection(double InAngle) const
	{
		const double ProjectedAngle = ZNear * FMath::Tan(FMath::DegreesToRadians(FMath::Abs(InAngle)));

		return (InAngle < 0) ? -ProjectedAngle : ProjectedAngle;
	}

	inline double ConvertProjectionToDegrees(double InProjectedAngle) const
	{
		const double Angle = FMath::RadiansToDegrees(FMath::Atan(FMath::Abs(InProjectedAngle) / ZNear));

		return (InProjectedAngle < 0) ? -Angle : Angle;
	}

	inline void ConvertProjectionAngles(const EDisplayClusterWarpAngleUnit InUnitType)
	{
		if (InUnitType != DataType)
		{
			if (DataType == EDisplayClusterWarpAngleUnit::Degrees && InUnitType == EDisplayClusterWarpAngleUnit::Default)
			{
				Left   = ConvertDegreesToProjection(Left);
				Right  = ConvertDegreesToProjection(Right);
				Bottom = ConvertDegreesToProjection(Bottom);
				Top    = ConvertDegreesToProjection(Top);

				DataType = InUnitType;

				return;
			}

			if (DataType == EDisplayClusterWarpAngleUnit::Default && InUnitType == EDisplayClusterWarpAngleUnit::Degrees)
			{
				Left   = ConvertProjectionToDegrees(Left);
				Right  = ConvertProjectionToDegrees(Right);
				Bottom = ConvertProjectionToDegrees(Bottom);
				Top    = ConvertProjectionToDegrees(Top);

				DataType = InUnitType;

				return;
			}
		}
	}

	// Unit type for values
	EDisplayClusterWarpAngleUnit DataType = EDisplayClusterWarpAngleUnit::Default;

	// Projection angles
	double Left = DBL_MAX;
	double Right = -DBL_MAX;
	double Top = -DBL_MAX;
	double Bottom = DBL_MAX;

	// Clipping planes
	double ZNear = 1.f;
	double ZFar = 1.f;

	// Scale
	double WorldScale = 1.f;

	// Warp projection ViewPoint
	FVector EyeLocation = FVector::ZeroVector;

	// Camera ViewPoint
	FRotator CameraRotation = FRotator::ZeroRotator;
	FVector  CameraLocation = FVector::ZeroVector;
};

/**
 * WarpBlend ViewPoint data
 */
struct FDisplayClusterWarpViewPoint
{
	inline FVector GetEyeLocation() const
	{
		return Location + EyeOffset;
	}

	inline bool IsEqual(const FDisplayClusterWarpViewPoint& InWarpViewPoint, float Precision) const
	{
		return (GetEyeLocation() - InWarpViewPoint.GetEyeLocation()).Size() < Precision;
	}

public:
	// ViewPoint location
	FVector Location = FVector::ZeroVector;

	// Eye offset
	FVector EyeOffset = FVector::ZeroVector;

	// ViewPoint rotation
	FRotator Rotation = FRotator::ZeroRotator;
};

/**
 * MPCDI attributes
 */
struct FDisplayClusterWarpMPCDIAttributes
{
	// MPCDI profile type
	EDisplayClusterWarpProfileType ProfileType = EDisplayClusterWarpProfileType::Invalid;

	/**
	* Special settings for creating 3D geometry of the mpcdi 2D profile.
	*/
	struct FProfile2DSettings
	{
		// The focal length will be calculated for that FOV and the geometry will be shifted by that value.
		float DesiredFOV = 90.f;

		/** The amount to scale an MPCDI buffer by when converting from pixels to world coordinates. */
		float BufferPixelsToWorldUnits = 0.1f;

	} Profile2DSettings;


	// Special flags
	EDisplayClusterWarpMPCDIAttributesFlags Flags = EDisplayClusterWarpMPCDIAttributesFlags::None;

	// additional mpcdi attributes: <Buffer>
	struct FBuffer
	{
		FBuffer()
			: Resolution(1024, 1024)
		{ }

		FIntPoint Resolution;

	} Buffer;

	// additional mpcdi attributes: <Region>
	struct FRegion
	{
		FRegion()
			: Resolution(1024, 1024)
			, Pos(0.0, 0.0)
			, Size(1.0, 1.0)
		{ }

		FIntPoint Resolution;
		FVector2D Pos;
		FVector2D Size;

	} Region;

	// additional mpcdi attributes <frustum>
	struct FFrustum
	{
		FFrustum()
			: Rotator(FRotator::ZeroRotator)
			, Angles(FVector4::Zero())
		{ }

		// Frustum direction
		FRotator Rotator;

		// Frustum angles XYZW = LRTB
		FVector4 Angles;

	} Frustum;

	// additional mpcdi attributes <coordinateFrame>
	struct FCoordinateFrame
	{
		FCoordinateFrame()
			: Pos(FVector::ZeroVector)
			, Yaw(FVector::ZeroVector)
			, Pitch(FVector::ZeroVector)
			, Roll(FVector::ZeroVector)
		{ }

		FVector Pos;
		FVector Yaw;
		FVector Pitch;
		FVector Roll;

	} CoordinateFrame;

	/** 
	* Calculates the position and screen size for a 2d profile.
	* Returns true if the success.
	*/
	inline bool CalcProfile2DScreen(FVector& OutScreenPosition, FVector2D& OutScreenSize) const
	{
		if (ProfileType != EDisplayClusterWarpProfileType::warp_2D)
		{
			return false;
		}

		// Computes the buffer size in world units from the resolution in pixels.
		const FVector2D BufferSize(Buffer.Resolution.X* Profile2DSettings.BufferPixelsToWorldUnits, Buffer.Resolution.Y* Profile2DSettings.BufferPixelsToWorldUnits);

		OutScreenSize = FVector2D(Region.Size.X * BufferSize.X, Region.Size.Y * BufferSize.Y);

		// Calculates the position and size of the region.
		const FVector2D  RegionPos(Region.Pos.X* BufferSize.X, Region.Pos.Y* BufferSize.Y);

		float FocalLength = 0.f;
		// Moves the buffer position along the X axis to achieve a DesiredFOV.
		if (Profile2DSettings.DesiredFOV > 0.f && Profile2DSettings.DesiredFOV < 180.f)
		{
			// Convert FOV to focal length,
			// 
			// fov = 2 * atan(d/(2*f))
			// where,
			//   d = sensor dimension
			//   f = focal length
			// 
			// f = 0.5 * d * (1/tan(fov/2))
			const float TanHalfFOV = FMath::Tan(FMath::DegreesToRadians(Profile2DSettings.DesiredFOV * 0.5f));
			FocalLength = (BufferSize.X * 0.5f) / TanHalfFOV;
		}

		const FVector2D RegionCenterPos = RegionPos + (OutScreenSize * 0.5) - (BufferSize * 0.5);

		OutScreenPosition = FVector(FocalLength, RegionCenterPos.X, RegionCenterPos.Y);

		return true;
	}
};
