// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/DisplayClusterWarpContainers.h"
#include "Containers/DisplayClusterWarpAABB.h"

class FDisplayClusterWarpEye;

/**
 *  Warp context
 */
struct FDisplayClusterWarpContext
{
	bool bIsValid = false;

	// Frustum has been rotated to fit the size of the context.
	// This value contains the result of the ShouldRotateFrustumToFitContextSize() function.
	bool bFrustumRotated = false;

	// Camera
	FRotator Rotation = FRotator::ZeroRotator;
	FVector  Location = FVector::ZeroVector;

	// Frustum projection matrix
	FMatrix  GeometryProjectionMatrix = FMatrix::Identity;

	// Frustum projection matrix
	FMatrix  ProjectionMatrix = FMatrix::Identity;

	// From the texture's perspective
	FMatrix  UVMatrix = FMatrix::Identity;

	FMatrix  TextureMatrix = FMatrix::Identity;
	FMatrix  RegionMatrix = FMatrix::Identity;

	// From the mesh local space to cave
	FMatrix  MeshToStageMatrix = FMatrix::Identity;

	// Origin viewpoint transform to world space.
	FTransform Origin2WorldTransform = FTransform::Identity;
};

/** Internal warp data for the one context of the viewport. */
struct FDisplayClusterWarpData
{
	// Input data from projection policy
	TSharedPtr<FDisplayClusterWarpEye, ESPMode::ThreadSafe> WarpEye;

	// Warp projection data for geometry
	FDisplayClusterWarpProjection GeometryWarpProjection;

	// Warp projection for render
	FDisplayClusterWarpProjection WarpProjection;

	// Results
	FDisplayClusterWarpContext WarpContext;

	// Projection plane matrices
	FMatrix Local2World = FMatrix::Identity;

	// Current world scale
	double WorldScale = 1;

	// Indicates if the frustum was rotated from base algorithm to better fit context size
	// Used to implement hysteresis and avoid potential jumps back and forth.
	bool bRotateFrustumToFitContextSize = false;

	// Allows to use bRotateFrustumToFitContextSize
	bool bEnabledRotateFrustumToFitContextSize = true;

	// Other warp settings:
	EDisplayClusterWarpBlendFrustumType       FrustumType = EDisplayClusterWarpBlendFrustumType::AABB;
	EDisplayClusterWarpBlendStereoMode         StereoMode = EDisplayClusterWarpBlendStereoMode::AsymmetricAABB;

	// Control projection type
	bool bFindBestProjectionType = false;
	EDisplayClusterWarpBlendProjectionType ProjectionType = EDisplayClusterWarpBlendProjectionType::DynamicAABBCenter;

	// After successful warpdata calculation, this value is set to true.
	bool bValid = false;

	// The warp policy Tick() function uses warp data, and it must be sure that this data is updated in the previous frame.
	// This value must be set to true from the EndCalcFrustum() warp policy function when changes are made to this structure.
	bool bHasWarpPolicyChanges = false;
};

struct FDisplayClusterWarpGeometryContext
{
	FMatrix GeometryToOrigin = FMatrix::Identity;

	// AABB of the warp geometry (in the space of the Origin component).
	FDisplayClusterWarpAABB AABBox;

	// Static surface average normal for this region
	FVector SurfaceViewNormal;

	// Static surface average normal from 4 corner points
	FVector SurfaceViewPlane;

	// Force update caches
	bool bGeometryChanged = false;

	// Allows you to use a matrix of regions (MPCDI 2D profile area defines a sub-rectangle of the input image to be cut and used for warp).
	bool bUseRegionMatrix = false;
};
