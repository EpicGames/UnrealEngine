// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"

/**
 * Angle values data type.
 */
enum class EDisplayClusterWarpAngleUnit : uint8
{
	// in projected space
	Default = 0,

	// In degress 0..360
	Degrees
};

/**
 * MPCDI profile type
 */
enum class EDisplayClusterWarpProfileType : uint8
{
	// 2D mode
	warp_2D = 0,
	// 3D mode (2D + static Frustum)
	warp_3D,
	// Advanced 3D mode
	warp_A3D,
	// Shader lamps
	warp_SL,

	Invalid
};

/**
 * Type of geometry source
 */
enum class EDisplayClusterWarpGeometryType : uint8
{
	// Use texture warpmap as geometry source
	WarpMap = 0,
	
	// Use mesh component as geometry source
	WarpMesh,

	// Use procedural mesh component as geometry source
	WarpProceduralMesh,

	// Undefined source
	Invalid
};

/**
 * Type of frustum geometry source
 */
enum class EDisplayClusterWarpFrustumGeometryType : uint8
{
	// Use mpcdi attributes  to build virtual frustum geometry (profile 2D)
	MPCDIAttributes = 0,

	// Use texture warpmap as frustum geometry source
	WarpMap,

	// Use mesh component as frustum geometry source
	WarpMesh,

	// Use procedural mesh component as frustum geometry source
	WarpProceduralMesh,

	// Undefined source
	Invalid
};

/**
 * The type of data stored in the texture (MPCDI or PFM only)
 */
enum class EDisplayClusterWarpBlendTextureType :uint8
{
	// Geometry data will be stored as a three-component (RGB) PFM file (Portable Float Map Format)
	WarpMap = 0,

	// Blend grayscale texture, color multiplier. Use in basic color correction: color = color * alpha;
	AlphaMap,

	// Bate grayscale texture, for advanced collor correction. Use with AlphaMap: color = (color*alpha*(1 - beta)) + beta;
	BetaMap
};

/**
 * The method used to calc a frustum from geometric points
 */
enum class EDisplayClusterWarpBlendFrustumType : uint8
{
	// Build frustum from 8 points of AABB
	AABB = 0,

	// Build frustum from all geometry points
	FULL,

	// Performance: Build frustum from part of geometry (LOD)
	LOD,
};

/**
 * Type of normal used to make the projection surface
 */
enum class EDisplayClusterWarpBlendProjectionType : uint8
{
	StaticSurfaceNormal = 0,
	StaticSurfacePlane,
	DynamicAABBCenter,

	RuntimeProjectionModes,
	RuntimeStaticSurfaceNormalInverted,
	RuntimeStaticSurfacePlaneInverted,
};

/**
 * The method used to calculate a stereo projection
 */
enum class EDisplayClusterWarpBlendStereoMode : uint8
{
	AsymmetricAABB = 0,
	SymmetricAABB,
};

/**
 * MPCDI attributes flags
 */
enum class EDisplayClusterWarpMPCDIAttributesFlags : uint8
{
	None = 0,

	// Frustum data defined
	HasFrustum = 1 << 0,

	// CoordinateFrame data defined
	HasCoordinateFrame = 1 << 1,
};
ENUM_CLASS_FLAGS(EDisplayClusterWarpMPCDIAttributesFlags);
