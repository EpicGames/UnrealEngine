// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/DisplayClusterWarpContainers.h"
#include "Render/Warp/IDisplayClusterWarpPolicy.h"
#include "Templates/SharedPointer.h"

class IDisplayClusterViewport;

/**
 * WarpBlend eye data
 */
class FDisplayClusterWarpEye
	: public TSharedFromThis<FDisplayClusterWarpEye, ESPMode::ThreadSafe>
{
public:
	FDisplayClusterWarpEye(const TSharedPtr<IDisplayClusterViewport, ESPMode::ThreadSafe>& InViewport, const uint32 InContextNum)
		: ViewportWeakPtr(InViewport)
		, ContextNum(InContextNum)
	{ }

	inline bool IsEqual(const FDisplayClusterWarpEye& InEye, float Precision) const
	{
		if (FMath::IsNearlyEqual(WorldScale, InEye.WorldScale))
		{
			return ViewPoint.IsEqual(InEye.ViewPoint, Precision);
		}

		return false;
	}

	inline IDisplayClusterViewport* GetViewport() const
	{
		return ViewportWeakPtr.IsValid() ? ViewportWeakPtr.Pin().Get() : nullptr;
	}

public:
	const TWeakPtr<IDisplayClusterViewport, ESPMode::ThreadSafe> ViewportWeakPtr;
	const uint32 ContextNum;

	FDisplayClusterWarpViewPoint ViewPoint;

	// Current scene additional settings
	float WorldScale = 1.f;

	// DCRA world2local
	FTransform World2LocalTransform;

	// Warp policy
	TSharedPtr<IDisplayClusterWarpPolicy, ESPMode::ThreadSafe> WarpPolicy;

	// (optional) this a view target point. By default used AABB center.
	TOptional<FVector> OverrideViewTarget;

	// (Optional) is the view direction (when this value is used, it also overrides the OverrideViewTarget parameter).
	TOptional<FVector> OverrideViewDirection;
};
