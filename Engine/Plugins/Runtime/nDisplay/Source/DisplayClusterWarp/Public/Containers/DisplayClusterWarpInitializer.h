// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/DisplayClusterWarpContainers.h"

class USceneComponent;
class UDisplayClusterScreenComponent;
class UStaticMeshComponent;
class UProceduralMeshComponent;

/**
 * Initialize WarpBlend from MPCDI file
 */
struct FDisplayClusterWarpInitializer_MPCDIFile
{
	// The file with MPCDI calibration data (extension of this file '.mpcdi')
	FString MPCDIFileName;

	// Name of buffer (mpcdi internal topology)
	FString BufferId;

	// Name of region (mpcdi internal topology)
	FString RegionId;
};

/**
 * Initialize WarpBlend from MPCDI file and DisplayClusterScreen component
 */
struct FDisplayClusterWarpInitializer_MPCDIFile_Profile2DScreen
	: public FDisplayClusterWarpInitializer_MPCDIFile
{
	// Origin component ptr
	USceneComponent* OriginComponent = nullptr;

	// StaticMesh component ptr (warp geometry data source)
	UStaticMeshComponent* WarpMeshComponent = nullptr;

	// Preview StaticMesh component ptr (warp geometry data source)
	UStaticMeshComponent* PreviewMeshComponent = nullptr;
};

/**
 * Initialize WarpBlend from PFM file
 */
struct FDisplayClusterWarpInitializer_PFMFile
{
	// The file with region calibration data (extension of this file '.pfm')
	FString PFMFileName;

	// Geometry scale of this file to UE space (1 = 1 centemeter)
	float   PFMScale = 1.f;

	// Wrap axis to mpcdi space for unreal 
	bool    bIsUnrealGameSpace = false;

	// AlphaMap file (supported format: PNG)
	FString AlphaMapFileName;

	// Control alpha multiplier for AlphaMap texture
	float   AlphaMapEmbeddedAlpha = 0.f;

	// BetaMap file (supported format: PNG)
	FString BetaMapFileName;

	// Additional attributes from xml inside MPCDI file
	FDisplayClusterWarpMPCDIAttributes MPCDIAttributes;
};

/**
 * Base struct, not used as initialized
 */
struct FDisplayClusterWarpInitializer_BaseMesh
{
	// Origin component ptr
	USceneComponent* OriginComponent = nullptr;

	// Map source mesh geometry UVs to WarpMesh UVs
	int32 BaseUVIndex = INDEX_NONE;
	int32 ChromakeyUVIndex = INDEX_NONE;
};

/**
 * Initialize WarpBlend from static mesh component
 */
struct FDisplayClusterWarpInitializer_StaticMesh
	: public FDisplayClusterWarpInitializer_BaseMesh
{
	// StaticMesh component ptr (warp geometry data source)
	UStaticMeshComponent* WarpMeshComponent = nullptr;

	// Preview StaticMesh component ptr (warp geometry data source)
	UStaticMeshComponent* PreviewMeshComponent = nullptr;

	// (optional) get geometry data from specified LOD index
	int32 StaticMeshComponentLODIndex = 0;
};

/**
 * Initialize WarpBlend from procedural mesh component
 */
struct FDisplayClusterWarpInitializer_ProceduralMesh
	: public FDisplayClusterWarpInitializer_BaseMesh
{
	// ProceduralMesh component ptr (warp geometry data source)
	UProceduralMeshComponent* WarpMeshComponent = nullptr;

	// Preview ProceduralMesh component ptr (warp geometry data source)
	UProceduralMeshComponent* PreviewMeshComponent = nullptr;

	// (optional) get geometry data from specified SectionIndex
	int32 ProceduralMeshComponentSectionIndex = 0;
};
