// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace UE::DisplayClusterWarpStrings
{
	namespace warp
	{
		static constexpr const TCHAR* InFrustumFit = TEXT("InFrustumFit");
	}
};
