// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#include "Containers/DisplayClusterWarpInitializer.h"
#include "IDisplayClusterWarpBlend.h"

class IDisplayClusterWarp : public IModuleInterface
{
public:
	static constexpr const TCHAR* ModuleName = TEXT("DisplayClusterWarp");

public:
	virtual ~IDisplayClusterWarp() = default;

public:
	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IDisplayClusterWarp& Get()
	{
		return FModuleManager::LoadModuleChecked<IDisplayClusterWarp>(IDisplayClusterWarp::ModuleName);
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded(IDisplayClusterWarp::ModuleName);
	}

public:
	/**
	 * Create new WarpBlend interface from MPCDI File
	 */
	virtual TSharedPtr<IDisplayClusterWarpBlend, ESPMode::ThreadSafe> Create(const FDisplayClusterWarpInitializer_MPCDIFile& InConstructParameters) const = 0;

	
	/**
	 * Create WarpBlend interface from MPCDI file and DisplayClusterScreen component
	 */
	virtual TSharedPtr<IDisplayClusterWarpBlend, ESPMode::ThreadSafe> Create(const FDisplayClusterWarpInitializer_MPCDIFile_Profile2DScreen& InConstructParameters) const = 0;

	/**
	 * Create new WarpBlend interface from PFM File
	 */
	virtual TSharedPtr<IDisplayClusterWarpBlend, ESPMode::ThreadSafe> Create(const FDisplayClusterWarpInitializer_PFMFile& InConstructParameters) const = 0;

	/**
	 * Create new WarpBlend interface from static mesh
	 */
	virtual TSharedPtr<IDisplayClusterWarpBlend, ESPMode::ThreadSafe> Create(const FDisplayClusterWarpInitializer_StaticMesh& InConstructParameters) const = 0;

	/**
	 * Create new WarpBlend interface from procedural mesh
	 */
	virtual TSharedPtr<IDisplayClusterWarpBlend, ESPMode::ThreadSafe> Create(const FDisplayClusterWarpInitializer_ProceduralMesh& InConstructParameters) const = 0;

	/**
	 * Read MPCDI file structure
	 * (This function uses MPCDICachedLoader's internal implementation, which greatly reduces the load time of huge MPCDI files.)
	 * 
	 * @param InMPCDIFileName        - (in)  filename of the MPCDI file
	 * @param OutMPCDIFileStructure  - (out) readed mpcdi structure in format: TMap<BufferId, TMap<RegionId, MPCDIAttributes>>
	 * 
	 * @return true if success
	 */
	virtual bool ReadMPCDFileStructure(const FString& InMPCDIFileName, TMap<FString, TMap<FString, FDisplayClusterWarpMPCDIAttributes>>& OutMPCDIFileStructure) const = 0;
};
