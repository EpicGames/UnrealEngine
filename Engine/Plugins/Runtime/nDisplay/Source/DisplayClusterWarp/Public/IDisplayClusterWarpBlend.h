// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "Containers/DisplayClusterWarpContext.h"
#include "Containers/DisplayClusterWarpEye.h"

class UMeshComponent;
class IDisplayClusterViewport;

/**
 * WarpBlend interface for MPCDI and mesh projection policies
 */
class IDisplayClusterWarpBlend
{
public:
	virtual ~IDisplayClusterWarpBlend() = default;

public:
	/** Get TSharedPtr from self. */
	virtual TSharedPtr<IDisplayClusterWarpBlend, ESPMode::ThreadSafe> ToSharedPtr() = 0;
	virtual TSharedPtr<const IDisplayClusterWarpBlend, ESPMode::ThreadSafe> ToSharedPtr() const = 0;

	/**
	* Called each time a new game level starts
	*
	* @param InViewport - a owner viewport
	*/
	virtual bool HandleStartScene(IDisplayClusterViewport* InViewport) = 0;

	/**
	* Called when current level is going to be closed (i.e. before loading a new map)
	*
	* @param InViewport - a owner viewport
	*/
	virtual void HandleEndScene(IDisplayClusterViewport* InViewport) = 0;


	/** Update internal geometry cached data
	 */
	virtual bool UpdateGeometryContext(const float InWorldScale) = 0;

	/** Get geometry context data. */
	virtual const FDisplayClusterWarpGeometryContext& GetGeometryContext() const = 0;

	/**
	* Calculate warp context data for new eye
	*
	* @param InEye - Current eye and scene
	*
	* @return - true if the context calculated successfully
	*/
	virtual bool CalcFrustumContext(const TSharedPtr<FDisplayClusterWarpEye, ESPMode::ThreadSafe>& InWarpEye) = 0;

	/**
	 * Access to resources by type
	 */
	virtual class FRHITexture* GetTexture(EDisplayClusterWarpBlendTextureType InTextureType) const = 0;

	/**
	* Get texture interface by type
	*/
	virtual TSharedPtr<class IDisplayClusterRender_Texture, ESPMode::ThreadSafe> GetTextureInterface(EDisplayClusterWarpBlendTextureType InTextureType) const = 0;

	/**
	 * Return AlphaMap embedded gamma value
	 */
	virtual float GetAlphaMapEmbeddedGamma() const = 0;

	/**
	 * Get mesh proxy
	 */
	virtual const class IDisplayClusterRender_MeshComponentProxy* GetWarpMeshProxy_RenderThread() const = 0;

	/**
	 * MPCDI profile type
	 */
	virtual EDisplayClusterWarpProfileType  GetWarpProfileType() const = 0;

	/**
	 * Get MPCDI attributes (or their default values if they are not defined)
	 */
	virtual const FDisplayClusterWarpMPCDIAttributes& GetMPCDIAttributes() const = 0;

	/**
	 * Warp geometry type
	 */
	virtual EDisplayClusterWarpGeometryType GetWarpGeometryType() const = 0;

	/**
	 * Warp frustum geometry type
	 */
	virtual EDisplayClusterWarpFrustumGeometryType GetWarpFrustumGeometryType() const = 0;

	/**
	 * Export warp geometry to memory
	 */
	virtual bool ExportWarpMapGeometry(struct FDisplayClusterWarpGeometryOBJ& OutMeshData, uint32 InMaxDimension = 0) const = 0;

	/**
	* Mark internal component ref as dirty for geometry update
	*
	* @param InComponentName - (optional) the name of the internal geometry ref component. Empty string for any component name
	*
	* @return - true, if there is a marked.
	*/
	virtual bool MarkWarpGeometryComponentDirty(const FName& InComponentName) = 0;

	/** Return true, if this WarpBlend support ICVFX pipeline. */
	virtual bool ShouldSupportICVFX() const = 0;

	/** Get internal warp context data. */
	virtual FDisplayClusterWarpData& GetWarpData(const uint32 ContextNum) = 0;

	/** Get internal warp context data. */
	virtual const FDisplayClusterWarpData& GetWarpData(const uint32 ContextNum) const = 0;

	/** Get a mesh component with the geometry used for the warp blend.
	* 
	* @param InViewport - The viewport with the projection policy to which the instance belongs
	* @param bExistingComponent - true if the component already exists in the DCRA and does not need to be deleted externally.
	* 
	* @return - ptr to the mesh component with the geometry, or nullptr in case of failure
	*/
	virtual UMeshComponent* GetOrCreatePreviewMeshComponent(IDisplayClusterViewport* InViewport, bool& bExistingComponent) const = 0;

	/**
	* Build preview editable mesh
	* This MeshComponent is a copy of the preview mesh and can be moved freely with the UI visualization.
	*
	* @param InViewport - Projection specific parameters.
	*/
	virtual UMeshComponent* GetOrCreatePreviewEditableMeshComponent(IDisplayClusterViewport* InViewport) const = 0;
};
