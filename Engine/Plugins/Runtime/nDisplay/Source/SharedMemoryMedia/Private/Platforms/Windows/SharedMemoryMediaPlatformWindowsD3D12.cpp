// Copyright Epic Games, Inc. All Rights Reserved.

#include "SharedMemoryMediaPlatformWindowsD3D12.h"

#include "ID3D12DynamicRHI.h"

#include "SharedMemoryMediaModule.h"
#include "Misc/EnumClassFlags.h"

// This should run when the module starts and register the creation function for this rhi platform.
bool FSharedMemoryMediaPlatformWindowsD3D12::bRegistered = FSharedMemoryMediaPlatformFactory::Get()->RegisterPlatformForRhi(
	ERHIInterfaceType::D3D12, &FSharedMemoryMediaPlatformWindowsD3D12::CreateInstance);

namespace UE::FSharedMemoryMediaPlatformWindowsD3D12
{
	DXGI_FORMAT FindSharedResourceDXGIFormat(DXGI_FORMAT InFormat, bool bSRGB)
	{
		if (bSRGB)
		{
			switch (InFormat)
			{
			case DXGI_FORMAT_B8G8R8X8_TYPELESS:    return DXGI_FORMAT_B8G8R8X8_UNORM_SRGB;
			case DXGI_FORMAT_B8G8R8A8_TYPELESS:    return DXGI_FORMAT_B8G8R8A8_UNORM_SRGB;
			case DXGI_FORMAT_R8G8B8A8_TYPELESS:    return DXGI_FORMAT_R8G8B8A8_UNORM_SRGB;
			case DXGI_FORMAT_BC1_TYPELESS:         return DXGI_FORMAT_BC1_UNORM_SRGB;
			case DXGI_FORMAT_BC2_TYPELESS:         return DXGI_FORMAT_BC2_UNORM_SRGB;
			case DXGI_FORMAT_BC3_TYPELESS:         return DXGI_FORMAT_BC3_UNORM_SRGB;
			case DXGI_FORMAT_BC7_TYPELESS:         return DXGI_FORMAT_BC7_UNORM_SRGB;
			};
		}
		else
		{
			switch (InFormat)
			{
			case DXGI_FORMAT_B8G8R8X8_TYPELESS:    return DXGI_FORMAT_B8G8R8X8_UNORM;
			case DXGI_FORMAT_B8G8R8A8_TYPELESS: return DXGI_FORMAT_B8G8R8A8_UNORM;
			case DXGI_FORMAT_R8G8B8A8_TYPELESS: return DXGI_FORMAT_R8G8B8A8_UNORM;
			case DXGI_FORMAT_BC1_TYPELESS:      return DXGI_FORMAT_BC1_UNORM;
			case DXGI_FORMAT_BC2_TYPELESS:      return DXGI_FORMAT_BC2_UNORM;
			case DXGI_FORMAT_BC3_TYPELESS:      return DXGI_FORMAT_BC3_UNORM;
			case DXGI_FORMAT_BC7_TYPELESS:      return DXGI_FORMAT_BC7_UNORM;
			};
		}
		switch (InFormat)
		{
		case DXGI_FORMAT_R32G32B32A32_TYPELESS: return DXGI_FORMAT_R32G32B32A32_UINT;
		case DXGI_FORMAT_R32G32B32_TYPELESS:    return DXGI_FORMAT_R32G32B32_UINT;
		case DXGI_FORMAT_R16G16B16A16_TYPELESS: return DXGI_FORMAT_R16G16B16A16_UNORM;
		case DXGI_FORMAT_R32G32_TYPELESS:       return DXGI_FORMAT_R32G32_UINT;
		case DXGI_FORMAT_R10G10B10A2_TYPELESS:  return DXGI_FORMAT_R10G10B10A2_UNORM;
		case DXGI_FORMAT_R16G16_TYPELESS:       return DXGI_FORMAT_R16G16_UNORM;
		case DXGI_FORMAT_R8G8_TYPELESS:         return DXGI_FORMAT_R8G8_UNORM;
		case DXGI_FORMAT_R8_TYPELESS:           return DXGI_FORMAT_R8_UNORM;

		case DXGI_FORMAT_BC4_TYPELESS:         return DXGI_FORMAT_BC4_UNORM;
		case DXGI_FORMAT_BC5_TYPELESS:         return DXGI_FORMAT_BC5_UNORM;



		case DXGI_FORMAT_R24G8_TYPELESS: return DXGI_FORMAT_R24_UNORM_X8_TYPELESS;
		case DXGI_FORMAT_R32_TYPELESS: return DXGI_FORMAT_R32_FLOAT;
		case DXGI_FORMAT_R16_TYPELESS: return DXGI_FORMAT_R16_UNORM;
			// Changing Depth Buffers to 32 bit on Dingo as D24S8 is actually implemented as a 32 bit buffer in the hardware
		case DXGI_FORMAT_R32G8X24_TYPELESS: return DXGI_FORMAT_R32_FLOAT_X8X24_TYPELESS;
		}
		return InFormat;
	}
}

TSharedPtr<FSharedMemoryMediaPlatform, ESPMode::ThreadSafe> FSharedMemoryMediaPlatformWindowsD3D12::CreateInstance()
{
	return MakeShared<FSharedMemoryMediaPlatformWindowsD3D12, ESPMode::ThreadSafe>();
}

FSharedMemoryMediaPlatformWindowsD3D12::FSharedMemoryMediaPlatformWindowsD3D12()
{
	for (int32 Idx = 0; Idx < UE::SharedMemoryMedia::SenderNumBuffers; ++Idx)
	{
		SharedHandle[Idx] = INVALID_HANDLE_VALUE;
		CommittedResource[Idx] = nullptr;
	}
}


FSharedMemoryMediaPlatformWindowsD3D12::~FSharedMemoryMediaPlatformWindowsD3D12()
{
	// Close shared handles
	for (int32 BufferIdx = 0; BufferIdx < UE::SharedMemoryMedia::SenderNumBuffers; ++BufferIdx)
	{
		ReleaseSharedTexture(BufferIdx);
	}
}

FTextureRHIRef FSharedMemoryMediaPlatformWindowsD3D12::CreateSharedTexture(EPixelFormat Format, bool bSrgb, int32 Width, int32 Height, const FGuid& Guid, uint32 BufferIdx, bool bCrossGpu)
{
	using namespace UE::FSharedMemoryMediaPlatformWindowsD3D12;

	check(BufferIdx < UE::SharedMemoryMedia::SenderNumBuffers);
	check(SharedHandle[BufferIdx] == INVALID_HANDLE_VALUE);
	check(CommittedResource[BufferIdx] == nullptr);

	ID3D12Device* D3D12Device = GetID3D12DynamicRHI()->RHIGetDevice_NoMGPU();
	check(D3D12Device);

	const DXGI_FORMAT DxgiFormat = FindSharedResourceDXGIFormat(DXGI_FORMAT(GPixelFormats[Format].PlatformFormat), bSrgb);

	D3D12_RESOURCE_FLAGS Flags = D3D12_RESOURCE_FLAG_ALLOW_SIMULTANEOUS_ACCESS;

	if (bCrossGpu)
	{
		Flags |= D3D12_RESOURCE_FLAG_ALLOW_CROSS_ADAPTER;
	}

	const D3D12_TEXTURE_LAYOUT Layout = bCrossGpu ? D3D12_TEXTURE_LAYOUT_ROW_MAJOR : D3D12_TEXTURE_LAYOUT_UNKNOWN;

	D3D12_RESOURCE_DESC SharedCrossGpuTextureDesc = CD3DX12_RESOURCE_DESC::Tex2D(
		DxgiFormat,
		Width,
		Height,
		1, // arraySize
		1, // mipLevels
		1, // sampleCount
		0, // sampleQuality
		Flags, // flags
		Layout // layout
	);

	CD3DX12_HEAP_PROPERTIES HeapProperties(D3D12_HEAP_TYPE_DEFAULT);
	D3D12_HEAP_FLAGS HeapFlags = D3D12_HEAP_FLAG_SHARED;

	if (bCrossGpu)
	{
		HeapFlags |= D3D12_HEAP_FLAG_SHARED_CROSS_ADAPTER;
	}

	HRESULT HResult = D3D12Device->CreateCommittedResource(
		&HeapProperties,
		HeapFlags,
		&SharedCrossGpuTextureDesc,
		D3D12_RESOURCE_STATE_COPY_DEST,
		nullptr, // pOptimizedClearValue
		IID_PPV_ARGS(&CommittedResource[BufferIdx])
	);

	if (FAILED(HResult))
	{
		UE_LOGF(LogSharedMemoryMedia, Error, "D3D12Device->CreateCommittedResource failed when creating a cross GPU texture:\n"
			"0x%lX - %ls. Texture was %dx%d, EPixelFormat %d and DXGI type %d. Use -d3ddebug for more information. ",
			HResult, *GetD3D12ComErrorDescription(HResult), Width, Height, Format, DxgiFormat);

		return nullptr;
	}

	check(HResult == S_OK);
	check(CommittedResource[BufferIdx]);

	const FString SharedGpuTextureName = FString::Printf(TEXT("Global\\%s"),
		*Guid.ToString(EGuidFormats::DigitsWithHyphensInBraces));

	HResult = D3D12Device->CreateSharedHandle(
		CommittedResource[BufferIdx],         // pObject
		nullptr,                              // pAttributes
		GENERIC_ALL,                          // Access
		*SharedGpuTextureName,                // Name
		&SharedHandle[BufferIdx]              // pHandle
	);

	check(HResult == S_OK);
	check(SharedHandle[BufferIdx] != INVALID_HANDLE_VALUE);

	return GetID3D12DynamicRHI()->RHICreateTexture2DFromResource(
		Format,
		TexCreate_DisableSRVCreation,
		FClearValueBinding::None,
		CommittedResource[BufferIdx]
	);
}

void FSharedMemoryMediaPlatformWindowsD3D12::ReleaseSharedTexture(uint32 BufferIdx)
{
	check(BufferIdx < UE::SharedMemoryMedia::SenderNumBuffers);
	if (SharedHandle[BufferIdx] != INVALID_HANDLE_VALUE)
	{
		::CloseHandle(SharedHandle[BufferIdx]);
		SharedHandle[BufferIdx] = INVALID_HANDLE_VALUE;
	}

	// We need to manually release committed resources because FD3D12Resource::~FD3D12Resource()
	// that is created via RHICreateTexture2DFromResource will not release it unless it is a bBackBuffer.
	if (CommittedResource[BufferIdx])
	{
		CommittedResource[BufferIdx]->Release();
		CommittedResource[BufferIdx] = nullptr;
	}
}

FTextureRHIRef FSharedMemoryMediaPlatformWindowsD3D12::OpenSharedTextureByGuid(const FGuid& Guid, FSharedMemoryMediaTextureDescription& OutTextureDescription)
{
	using namespace UE::FSharedMemoryMediaPlatformWindowsD3D12;

	ID3D12Device* D3D12Device = GetID3D12DynamicRHI()->RHIGetDevice_NoMGPU();
	check(D3D12Device);

	const FString SharedGpuTextureName = FString::Printf(TEXT("Global\\%s"), *Guid.ToString(EGuidFormats::DigitsWithHyphensInBraces));

	ID3D12Resource* SharedCrossGpuTexture = nullptr;

	HANDLE NamedSharedGpuTextureHandle = INVALID_HANDLE_VALUE;
	HRESULT HResult = D3D12Device->OpenSharedHandleByName(*SharedGpuTextureName, GENERIC_ALL, &NamedSharedGpuTextureHandle);

	if (FAILED(HResult))
	{
		UE_LOGF(LogSharedMemoryMedia, Error, "D3D12Device->OpenSharedHandleByName(%ls) failed: 0x%lX - %ls",
			*SharedGpuTextureName, HResult, *GetD3D12ComErrorDescription(HResult));

		return nullptr;
	}

	HResult = D3D12Device->OpenSharedHandle(NamedSharedGpuTextureHandle, IID_PPV_ARGS(&SharedCrossGpuTexture));

	UE_CLOGF(FAILED(HResult), LogSharedMemoryMedia, Error, "D3D12Device->OpenSharedHandle(0x%p) failed: 0x%lX - %ls",
		NamedSharedGpuTextureHandle, HResult, *GetD3D12ComErrorDescription(HResult));

	::CloseHandle(NamedSharedGpuTextureHandle);

	if (FAILED(HResult))
	{
		return nullptr;
	}

	NamedSharedGpuTextureHandle = INVALID_HANDLE_VALUE;

	check(SharedCrossGpuTexture);

	// Query texture dimensions and use them to describe downstream resources
	{
		D3D12_RESOURCE_DESC SharedGpuTextureDesc = SharedCrossGpuTexture->GetDesc();

		OutTextureDescription.Height = SharedGpuTextureDesc.Height;
		OutTextureDescription.Width = SharedGpuTextureDesc.Width;
		OutTextureDescription.Format = EPixelFormat::PF_Unknown;
		OutTextureDescription.bSrgb = false;

		// Find EPixelFormat from platform format
		for (int32 FormatIdx = 0; FormatIdx < PF_MAX; FormatIdx++)
		{
			constexpr bool SrgbOptions[2] = { false, true };

			for (const bool bSrgb : SrgbOptions)
			{
				const DXGI_FORMAT ResultingPlatformFormat = FindSharedResourceDXGIFormat(DXGI_FORMAT(GPixelFormats[FormatIdx].PlatformFormat), bSrgb);

				if (ResultingPlatformFormat == SharedGpuTextureDesc.Format)
				{
					OutTextureDescription.Format = EPixelFormat(FormatIdx);
					OutTextureDescription.BytesPerPixel = GPixelFormats[FormatIdx].BlockBytes;
					OutTextureDescription.Stride = OutTextureDescription.Width * OutTextureDescription.BytesPerPixel;
					OutTextureDescription.bSrgb = bSrgb;
					break;
				}
			}

			if (OutTextureDescription.Format != EPixelFormat::PF_Unknown)
			{
				break;
			}
		}

		if (OutTextureDescription.Format == EPixelFormat::PF_Unknown)
		{
			UE_LOGF(LogSharedMemoryMedia, Error, "Could not find a known pixel format for SharedCrossGpuTexture DXGI_FORMAT %d.",
				SharedGpuTextureDesc.Format);

			SharedCrossGpuTexture->Release();
			SharedCrossGpuTexture = nullptr;

			return nullptr;
		}
	}

	ETextureCreateFlags Flags = TexCreate_DisableSRVCreation;
	
	if (OutTextureDescription.bSrgb)
	{
		Flags |= TexCreate_SRGB;
	}

	return GetID3D12DynamicRHI()->RHICreateTexture2DFromResource(
		OutTextureDescription.Format,
		Flags,
		FClearValueBinding::None,
		SharedCrossGpuTexture
	);
}


const FString FSharedMemoryMediaPlatformWindowsD3D12::GetD3D12ComErrorDescription(HRESULT Hresult)
{
	constexpr uint32 BufSize = 1024;
	WIDECHAR Buffer[BufSize];

	if (::FormatMessageW(FORMAT_MESSAGE_FROM_SYSTEM,
		nullptr,
		Hresult,
		MAKELANGID(LANG_NEUTRAL, SUBLANG_NEUTRAL),
		Buffer,
		sizeof(Buffer) / sizeof(*Buffer),
		nullptr))
	{
		return Buffer;
	}
	else
	{
		return FString::Printf(TEXT("[Could not find a d3d12 error description for HRESULT %d]"), Hresult);
	}
}
