// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHI.h"

#include "Windows/AllowWindowsPlatformTypes.h"
#include "d3d12.h"
#include "Windows/HideWindowsPlatformTypes.h"

#include "SharedMemoryMediaPlatform.h"
#include "SharedMemoryMediaCapture.h"

/** 
 * D3D12 RHI specific implementation of FSharedMemoryMediaPlatform. 
 */
class FSharedMemoryMediaPlatformWindowsD3D12 : public FSharedMemoryMediaPlatform
{
private:

	/** Pointer to the D3D12 shared memory heap. */
	ID3D12Resource* CommittedResource[UE::SharedMemoryMedia::SenderNumBuffers]{ 0 };

	/** Handles to the shared handles of the cross gpu resources */
	HANDLE SharedHandle[UE::SharedMemoryMedia::SenderNumBuffers]{ 0 };

	/** Dummy bool to get the registration call made */
	static bool bRegistered;

	/** Factory function to create this rhi platform instance. It is registed with and called by FSharedMemoryMediaPlatform */
	static TSharedPtr<FSharedMemoryMediaPlatform> CreateInstance();

	/** Convenience function to stringify an HRESULT */
	static const FString GetD3D12ComErrorDescription(HRESULT Hresult);

public:

	//~ Begin FSharedMemoryMediaPlatform interface
	virtual FTextureRHIRef CreateSharedTexture(EPixelFormat Format, bool bSrgb, int32 Width, int32 Height, const FGuid& Guid, uint32 BufferIdx, bool bCrossGpu) override;
	virtual void ReleaseSharedTexture(uint32 BufferIdx) override;
	virtual FTextureRHIRef OpenSharedTextureByGuid(const FGuid& Guid, FSharedMemoryMediaTextureDescription& OutTextureDescription) override;
	//~ End FSharedMemoryMediaPlatform interface

	FSharedMemoryMediaPlatformWindowsD3D12();
	virtual ~FSharedMemoryMediaPlatformWindowsD3D12();
};
