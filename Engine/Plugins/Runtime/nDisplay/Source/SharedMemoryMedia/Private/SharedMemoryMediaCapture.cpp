// Copyright Epic Games, Inc. All Rights Reserved.

#include "SharedMemoryMediaCapture.h"
#include "SharedMemoryMediaOutput.h"

#include "GenericPlatform/GenericPlatformMemory.h"
#include "Internationalization/TextLocalizationResource.h"
#include "MediaShaders.h"
#include "RenderGraphUtils.h"
#include "SceneView.h"
#include "ScreenPass.h"

#include "SharedMemoryMediaModule.h"
#include "SharedMemoryMediaPlatform.h"
#include "SharedMemoryMediaTypes.h"



DECLARE_GPU_STAT(SharedMemory_Capture);

BEGIN_SHADER_PARAMETER_STRUCT(FCopyToSharedGpuTexturePass, )
	RDG_TEXTURE_ACCESS(SrcTexture, ERHIAccess::CopySrc)
	RDG_TEXTURE_ACCESS(DstTexture, ERHIAccess::CopyDest)
END_SHADER_PARAMETER_STRUCT()


namespace UE::SMMCapture
{
	/**
	 * Checks if a given Rect is completely contained within a size of a texture.
	 *
	 * @param Size The size of the texture
	 * @param Rect The rectangle to check
	 * @return True iff the rect is completely contained within the bounds of the texture size.
	 */
	static bool IsRectContainedInTextureSize(const FIntRect& Rect, const FIntVector& Size)
	{
		return Rect.Min.X >= 0         && Rect.Min.Y >= 0          // top-left should be 0+
			&& Rect.Max.X <= Size.X    && Rect.Max.Y <= Size.Y     // bottom-right should be within the size bounds
			&& Rect.Min.X < Rect.Max.X && Rect.Min.Y < Rect.Max.Y; // Min should be less than Max.
	}
}


bool USharedMemoryMediaCapture::InitializeCapture()
{
	// Validate Media Output type

	USharedMemoryMediaOutput* SharedMemoryMediaOutput = Cast<USharedMemoryMediaOutput>(MediaOutput);

	if (!SharedMemoryMediaOutput)
	{
		UE_LOGF(LogSharedMemoryMedia, Error, "Invalid MediaOutput for '%ls', cannot InitializeCapture", *GetName());
		return false;
	}

	// Get an RHI type specific implementation.
	if (!PlatformData.IsValid())
	{
		const ERHIInterfaceType RhiInterfaceType = GDynamicRHI->GetInterfaceType();

		PlatformData = FSharedMemoryMediaPlatformFactory::Get()->CreateInstanceForRhi(RhiInterfaceType);

		if (!PlatformData.IsValid())
		{
			UE_LOGF(LogSharedMemoryMedia, Error, "Unfortunately, SharedMemoryMedia doesn't support the current RHI type '%ls'",
				*FSharedMemoryMediaPlatformFactory::GetRhiTypeString(RhiInterfaceType));

			return false;
		}
	}

	const SIZE_T SharedMemorySize = sizeof(FSharedMemoryMediaFrameMetadata);

	for (int32 BufferIdx = 0; BufferIdx < NUMBUFFERS; ++BufferIdx)
	{
		// Generate the shared memory Guid from the MediaOutput user set unique name.
		const FGuid Guid = UE::SharedMemoryMedia::GenerateSharedMemoryGuid(SharedMemoryMediaOutput->UniqueName, BufferIdx);

		const FString SharedMemoryRegionName = Guid.ToString(EGuidFormats::DigitsWithHyphensInBraces);

		// Open existing shared memory region, in case it exists:

		const uint32 AccessMode = FPlatformMemory::ESharedMemoryAccess::Read | FPlatformMemory::ESharedMemoryAccess::Write;

#if !NO_LOGGING
		// Disable LogHAL warnings caused by failing to open the shared memory.
		const ELogVerbosity::Type LogHALVerbosity = LogHAL.GetVerbosity();
		LogHAL.SetVerbosity(ELogVerbosity::Error);
#endif // !NO_LOGGING

		FPlatformMemory::FSharedMemoryRegion* SharedMemoryRegion = FPlatformMemory::MapNamedSharedMemoryRegion(
			*SharedMemoryRegionName, false /* bCreate */, AccessMode, SharedMemorySize
		);

#if !NO_LOGGING
		// Restore the verbosity level of LogHAL to the previous value
		LogHAL.SetVerbosity(LogHALVerbosity);
#endif // !NO_LOGGING

		// If it doesn't exist, then we allocate and zero-initialize it.
		if (!SharedMemoryRegion)
		{
			// Create
			SharedMemoryRegion = FPlatformMemory::MapNamedSharedMemoryRegion(
				*SharedMemoryRegionName,
				true /* bCreate */,
				AccessMode,
				SharedMemorySize
			);

			// Zero
			if (SharedMemoryRegion)
			{
				check(SharedMemoryRegion->GetAddress());
				FMemory::Memzero(SharedMemoryRegion->GetAddress(), SharedMemoryRegion->GetSize());

				// Except some special data
				FSharedMemoryMediaFrameMetadata* Data = static_cast<FSharedMemoryMediaFrameMetadata*>(SharedMemoryRegion->GetAddress());
				Data->Initialize();

				UE_LOGF(LogSharedMemoryMedia, Verbose, "Created SharedMemoryRegion[%d] = %ls for UniqueName '%ls'", 
					BufferIdx, *SharedMemoryRegionName, *SharedMemoryMediaOutput->UniqueName
				);
			}
		}

		// Verify that the shared memory creation succeeded
		if (!SharedMemoryRegion || !SharedMemoryRegion->GetAddress())
		{
			SetState(EMediaCaptureState::Error);
			return false;
		}

		SharedMemory[BufferIdx] = SharedMemoryRegion;
	}

	for (uint32 BufferIdx = 0; BufferIdx < NUMBUFFERS; BufferIdx++)
	{
		// Initialize fences
		if (!TextureReadyFences[BufferIdx])
		{
			TextureReadyFences[BufferIdx] = RHICreateGPUFence(*FString::Printf(TEXT("SharedMemoryMediaOutputFence_%d"), BufferIdx));
		}
	}			

	SetState(EMediaCaptureState::Capturing);

	return true;
}

void USharedMemoryMediaCapture::StopCaptureImpl(bool bAllowPendingFrameToBeProcess)
{
	// Note: This gets called by StopCapture which already changed the state to Stopped and called FlushRenderingCommands.

	using namespace UE::SharedMemoryMedia;

	// Since FlushRenderingCommands was already called by StopCapture, we can safely release all the resources

	// Wait for any pending tasks to finish, which could be trying to use the resources as well.
	while (RunningTasksCount > 0)
	{
		FPlatformProcess::SleepNoStats(SpinWaitTimeSeconds);
	}

	check(RunningTasksCount == 0);

	for (int32 BufferIdx = 0; BufferIdx < NUMBUFFERS; ++BufferIdx)
	{
		check(!bTextureReadyFenceBusy[BufferIdx]);
		TextureReadyFences[BufferIdx].SafeRelease();

		if (SharedMemory[BufferIdx])
		{
			// Let the receivers know that we're closed
			{
				FSharedMemoryMediaFrameMetadata* SharedMemoryData = static_cast<FSharedMemoryMediaFrameMetadata*>(SharedMemory[BufferIdx]->GetAddress());

				if (SharedMemoryData)
				{
					SharedMemoryData->Sender.Magic = 0;
					SharedMemoryData->Sender.TextureGuid = ZeroGuid;
				}
			}

			FPlatformMemory::UnmapNamedSharedMemoryRegion(SharedMemory[BufferIdx]);
			SharedMemory[BufferIdx] = nullptr;
		}

		SharedCrossGpuTextures[BufferIdx].SafeRelease(); // This should release the platform specific resources

		if (PlatformData.IsValid())
		{
			PlatformData->ReleaseSharedTexture(BufferIdx);
		}

		SharedCrossGpuTextureGuids[BufferIdx] = FGuid();
	}

	// Free platform specific resources
	PlatformData.Reset();
}

bool USharedMemoryMediaCapture::ShouldCaptureRHIResource() const
{
	return true;
}

FIntPoint USharedMemoryMediaCapture::GetCustomOutputSize(const FIntPoint& InSize) const
{
	// We pass back the desired size
	return InSize;
}

EMediaCaptureResourceType USharedMemoryMediaCapture::GetCustomOutputResourceType() const
{
	return EMediaCaptureResourceType::Texture;
}

void USharedMemoryMediaCapture::OnCustomCapture_RenderingThread(
	FRDGBuilder& GraphBuilder, 
	const FCaptureBaseData& InBaseData, 
	TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, 
	FRDGTextureRef InSourceTexture, 
	FRDGTextureRef OutputTexture, 
	const FRHICopyTextureInfo& CopyInfo, 
	FVector2D CropU, 
	FVector2D CropV)
{
	RDG_EVENT_SCOPE_STAT(GraphBuilder, SharedMemory_Capture, "SharedMemory_Capture");

	TRACE_CPUPROFILER_EVENT_SCOPE(USharedMemoryMediaCapture::OnCustomCapture_RenderingThread);

	// We'll be needing the output options.
	const USharedMemoryMediaOutput* SharedMemoryMediaOutput = Cast<USharedMemoryMediaOutput>(MediaOutput);
	check(SharedMemoryMediaOutput);

	// Initialize shared gpu textures if needed.

	check(PlatformData.IsValid());

	for (uint32 Idx = 0; Idx < NUMBUFFERS; Idx++)
	{
		if (!SharedCrossGpuTextures[Idx].IsValid())
		{
			const FGuid Guid = FGuid::NewGuid();

			// Shared texture size follows output texture which has already gone through MediaCapture's resizing logic.
			const FIntVector SharedSize = OutputTexture->Desc.GetSize();
						
			SharedCrossGpuTextures[Idx] = PlatformData->CreateSharedTexture(
				InSourceTexture->Desc.Format,
				EnumHasAnyFlags(InSourceTexture->Desc.Flags, TexCreate_SRGB),
				SharedSize.X,
				SharedSize.Y,
				Guid, 
				Idx,
				SharedMemoryMediaOutput->bCrossGpu
			);

			if (!SharedCrossGpuTextures[Idx].IsValid())
			{
				UE_LOGF(LogSharedMemoryMedia, Error, "Unable to create cross GPU texture of the requested type for Unique Name '%ls'", 
					*SharedMemoryMediaOutput->UniqueName
				);

				SetState(EMediaCaptureState::Error);
				return;
			}

			SharedCrossGpuTextureGuids[Idx] = Guid;

			UE_LOGF(LogSharedMemoryMedia, Verbose, "Created SharedGpuTextureGuid[%d] = %ls for UniqueName '%ls'", 
				Idx, *SharedCrossGpuTextureGuids[Idx].ToString(), *SharedMemoryMediaOutput->UniqueName
			);
		}

		// Verify that the shared memory media size is still the same as the OutputTexture

		const FIntVector SharedSize = SharedCrossGpuTextures[Idx]->GetDesc().GetSize();
		const FIntVector OutputSize = OutputTexture->Desc.GetSize();

		if (SharedSize != OutputSize)
		{
			UE_LOGF(LogSharedMemoryMedia, Warning,
				"Cross GPU texture is out of date for Unique Name '%ls'. It's size was (%d,%d) but OutputTexture was (%d,%d)",
				*SharedMemoryMediaOutput->UniqueName, SharedSize.X, SharedSize.Y, OutputSize.X, OutputSize.Y
			);

			SetState(EMediaCaptureState::Error);
			return;
		}
	}

	// Sanity check the copy regions are contained inside the input and output texture sizes.
	{
		const FIntRect Rect = CopyInfo.GetSourceRect();
		const FIntVector Size = InSourceTexture->Desc.GetSize();

		if (!UE::SMMCapture::IsRectContainedInTextureSize(Rect, Size))
		{
			UE_LOGF(LogSharedMemoryMedia, Error, 
				"Invalid source CopyInfo passed SharedMemoryMedia with Unique Name '%ls'. (%d,%d,%d,%d) not contained in (%d,%d)",
				*SharedMemoryMediaOutput->UniqueName, Rect.Min.X, Rect.Min.Y, Rect.Max.X, Rect.Max.Y, Size.X, Size.Y
			);

			SetState(EMediaCaptureState::Error);
			return;
		}
	}
	{
		const FIntRect Rect = CopyInfo.GetDestRect();
		const FIntVector Size = OutputTexture->Desc.GetSize();

		if (!UE::SMMCapture::IsRectContainedInTextureSize(Rect, Size))
		{
			UE_LOGF(LogSharedMemoryMedia, Error, 
				"Invalid destination CopyInfo passed SharedMemoryMedia with Unique Name '%ls'. [(%d,%d),(%d,%d)] not contained in (%d,%d)",
				*SharedMemoryMediaOutput->UniqueName, Rect.Min.X, Rect.Min.Y, Rect.Max.X, Rect.Max.Y, Size.X, Size.Y
			);

			SetState(EMediaCaptureState::Error);
			return;
		}
	}

	FRDGTextureRef SourceTexture = InSourceTexture;
	FRHICopyTextureInfo CurrentCopyInfo = CopyInfo; // This may be modified if there are intermediate steps (i.e. invert alpha pass)

	// When enabled, add pass to invert alpha

	if (SharedMemoryMediaOutput->bInvertAlpha)
	{
		ETextureCreateFlags InvertedAlphaTextureFlags = ETextureCreateFlags::ResolveTargetable;

		if (EnumHasAnyFlags(SourceTexture->Desc.Flags, TexCreate_SRGB))
		{
			InvertedAlphaTextureFlags |= TexCreate_SRGB;
		}

		const FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(
			FIntPoint(
				OutputTexture->Desc.GetSize().X,
				OutputTexture->Desc.GetSize().Y
			),
			SourceTexture->Desc.Format,
			FClearValueBinding::Black,
			InvertedAlphaTextureFlags
		);

		FRDGTextureRef InvertedAlphaTexture = GraphBuilder.CreateTexture(Desc, TEXT("SharedMemoryMediaInvertedAlphaTexture"));
		check(InvertedAlphaTexture);

		AddInvertAlphaConversionPass(GraphBuilder, SourceTexture, InvertedAlphaTexture, CurrentCopyInfo);

		SourceTexture = InvertedAlphaTexture;

		// Adjust the copy info's Source Position since SourceTexture is now InvertedAlphaTexture which is like OutputTexture.
		CurrentCopyInfo.SourcePosition = CurrentCopyInfo.DestPosition;
	}

	// Add the copy texture pass
	AddCopyToSharedGpuTexturePass(GraphBuilder, SourceTexture, GFrameCounterRenderThread % NUMBUFFERS, CurrentCopyInfo);
}

void USharedMemoryMediaCapture::AddInvertAlphaConversionPass(FRDGBuilder& GraphBuilder, const FRDGTextureRef& SourceTexture, FRDGTextureRef DestTexture, const FRHICopyTextureInfo& CopyInfo)
{
	FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
	TShaderMapRef<FScreenPassVS> VertexShader(GlobalShaderMap);

	// Configure source/output viewport to get the right UV scaling from source texture to output texture
	FScreenPassTextureViewport InputViewport(SourceTexture, CopyInfo.GetSourceRect());
	FScreenPassTextureViewport OutputViewport(DestTexture, CopyInfo.GetDestRect());

	// In cases where texture is converted from a format that doesn't have A channel, we want to force set it to 1.
	EMediaCaptureConversionOperation MediaConversionOperation = EMediaCaptureConversionOperation::INVERT_ALPHA;
	FModifyAlphaSwizzleRgbaPS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FModifyAlphaSwizzleRgbaPS::FConversionOp>(static_cast<int32>(MediaConversionOperation));

	TShaderMapRef<FModifyAlphaSwizzleRgbaPS> PixelShader(GlobalShaderMap, PermutationVector);
	FModifyAlphaSwizzleRgbaPS::FParameters* Parameters = PixelShader->AllocateAndSetParameters(GraphBuilder, SourceTexture, DestTexture);

	// Dummy SceneView created to use built in Draw Screen/Texture Pass

	FSceneViewFamily ViewFamily(FSceneViewFamily::ConstructionValues(nullptr, nullptr, FEngineShowFlags(ESFIM_Game))
		.SetTime(FGameTime())
	);

	FSceneViewInitOptions ViewInitOptions;

	ViewInitOptions.ViewFamily = &ViewFamily;
	ViewInitOptions.SetViewRectangle(CopyInfo.GetSourceRect());
	ViewInitOptions.ViewOrigin = FVector::ZeroVector;
	ViewInitOptions.ViewRotationMatrix = FMatrix::Identity;
	ViewInitOptions.ProjectionMatrix = FMatrix::Identity;

	FSceneView View = FSceneView(ViewInitOptions);

	AddDrawScreenPass(
		GraphBuilder, 
		RDG_EVENT_NAME("SharedMemoryMediaOutputInvertAlpha"), 
		View, 
		OutputViewport, 
		InputViewport, 
		VertexShader, 
		PixelShader, 
		Parameters
	);
}


void USharedMemoryMediaCapture::AddCopyToSharedGpuTexturePass(FRDGBuilder& GraphBuilder, FRDGTextureRef InSourceTexture, uint32 SharedTextureIdx, const FRHICopyTextureInfo& CopyInfo)
{
	using namespace UE::SharedMemoryMedia;

	FCopyToSharedGpuTexturePass* PassParameters = GraphBuilder.AllocParameters<FCopyToSharedGpuTexturePass>();

	PassParameters->SrcTexture = InSourceTexture;

	PassParameters->DstTexture = GraphBuilder.RegisterExternalTexture(
		CreateRenderTarget(SharedCrossGpuTextures[SharedTextureIdx], *FString::Printf(TEXT("SharedCrossGpuTextures_%d"), SharedTextureIdx)));

	// We increment RunningTasksCount here because the RDG build will run on an RHI related thread,
	// so for the purposes of waiting for resources to be freed, we must include the AddPass lambda execution,
	// which contains the async task that will decrement RunningTasksCount.
	RunningTasksCount++;

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("Copy%sToSharedGpuTexture", InSourceTexture->Name),
		PassParameters,
		ERDGPassFlags::Copy,
		[InSourceTexture, SharedTextureIdx, CopyInfo, this](FRHICommandList& RHICmdList)
		{
			// bTextureReadyFenceBusy will also signal that the resource is safe to reuse
			if (bTextureReadyFenceBusy[SharedTextureIdx])
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(SharedMemMediaOutputFenceBusy);

				UE_LOGF(LogSharedMemoryMedia, Verbose, "bTextureReadyFenceBusy[%d] for frame %lld was busy, so we wait",
					SharedTextureIdx, GFrameCounterRenderThread
				);

				while (bTextureReadyFenceBusy[SharedTextureIdx])
				{
					FPlatformProcess::SleepNoStats(SpinWaitTimeSeconds);
				}
			}

			// This flag will be cleared by the async task when the receiver is done with the shared cross gpu texture
			bTextureReadyFenceBusy[SharedTextureIdx] = true;

			// Do the copy
			RHICmdList.CopyTexture(InSourceTexture->GetRHI(), SharedCrossGpuTextures[SharedTextureIdx], CopyInfo);

			// Write GPU fence
			RHICmdList.WriteGPUFence(TextureReadyFences[SharedTextureIdx]);

			// Spawn a thread that via shared ram will notify receiver that data is ready
			// It will also verify that the data has been consumed (with a timeout).
			UE::Tasks::Launch(UE_SOURCE_LOCATION, [FrameNumber = GFrameCounterRenderThread, SharedTextureIdx, this]()
			{
				// Decrement RunningTasksCount when the task exits
				ON_SCOPE_EXIT
				{
					RunningTasksCount--;
				};

				const FString CopyThreadName = FString::Printf(TEXT("SharedMemMediaOutputGpuTextureInTransitForFrame_%lld"), FrameNumber % 100);
				TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*CopyThreadName);

				// Wait for fence that indicates that the gpu texture has the data
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(WaitForGpuTextureReadyFence);

					while (TextureReadyFences[SharedTextureIdx] && !TextureReadyFences[SharedTextureIdx]->Poll())
					{
						FPlatformProcess::SleepNoStats(0);
					}
				}

				// Update shared memory metadata to indicate to the receiver that there is new data

				FSharedMemoryMediaFrameMetadata* SharedMemoryData = static_cast<FSharedMemoryMediaFrameMetadata*>(SharedMemory[SharedTextureIdx]->GetAddress());
				{
					FSharedMemoryMediaFrameMetadata::FSender SenderMetadata;
					SenderMetadata.FrameNumber = FrameNumber;
					SenderMetadata.TextureGuid = SharedCrossGpuTextureGuids[SharedTextureIdx];

					// We only send the sender structure
					FMemory::Memcpy(&SharedMemoryData->Sender, &SenderMetadata, sizeof(FSharedMemoryMediaFrameMetadata::FSender));
				}

				// Wait for FrameNumber ack. Since we may have more than one receiver, we must wait until 
				// all active receivers have acked a frame number equal or greater than FrameNumber.
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(WaitForGpuTextureAck)

					const double StartTimeSeconds = FPlatformTime::Seconds();
					constexpr double TimeoutSeconds = 0.5;

					while (!SharedMemoryData->AllReceiversAckedFrameNumber(FrameNumber))
					{
						FPlatformProcess::SleepNoStats(SpinWaitTimeSeconds);

						if ((FPlatformTime::Seconds() - StartTimeSeconds) > TimeoutSeconds)
						{
							UE_LOGF(LogSharedMemoryMedia, Warning, 
								"FSharedMemoryMediaCapture timed out waiting for its receiver to ack frame %lld", FrameNumber
							);

							// We break this while loop because we will not be waiting any longer, 
							// even if receivers haven't acked the frame.
							// This includes any newly connected receivers that may have joined during this time.
							break;
						}
					}
				}

				// Decrement the keep alive. The receiver must keep resetting it to avoid letting it expire.
				SharedMemoryData->DecrementKeepAlives();

				// Clear fence and flag that we're ready for a new frame

				TextureReadyFences[SharedTextureIdx]->Clear();
				bTextureReadyFenceBusy[SharedTextureIdx] = false;
			});
		});
}
