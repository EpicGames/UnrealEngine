// Copyright Epic Games, Inc. All Rights Reserved.

#include "SharedMemoryMediaModule.h"

#include "IMediaModule.h"
#include "Modules/ModuleManager.h"
#include "SharedMemoryMediaPlayerFactory.h"

DEFINE_LOG_CATEGORY(LogSharedMemoryMedia);

/**
 * Implements the SharedMemoryMedia module.
 */
class FSharedMemoryMediaModule : public IModuleInterface
{

public:

	/** */
	void StartupModule() override
	{
		// Create Player Factories
		PlayerFactories.Add(MakeUnique<FSharedMemoryMediaPlayerFactory>());

		// register share memory player factory
		if (IMediaModule* MediaModule = FModuleManager::LoadModulePtr<IMediaModule>("Media"))
		{
			for (TUniquePtr<IMediaPlayerFactory>& Factory : PlayerFactories)
			{
				MediaModule->RegisterPlayerFactory(*Factory);
			}
		}
	}

	/** */
	void ShutdownModule()
	{
		UE_LOGF(LogSharedMemoryMedia, Log, "Shutting down module SharedMemoryMedia'...");

		// unregister share memory player factory
		if (IMediaModule* MediaModule = FModuleManager::LoadModulePtr<IMediaModule>("Media"))
		{
			for (TUniquePtr<IMediaPlayerFactory>& Factory : PlayerFactories)
			{
				MediaModule->UnregisterPlayerFactory(*Factory);
			}
		}
	}

private:

	/** Media player factories */
	TArray<TUniquePtr<IMediaPlayerFactory>> PlayerFactories;

};


IMPLEMENT_MODULE(FSharedMemoryMediaModule, SharedMemoryMedia);
