// Copyright Epic Games, Inc. All Rights Reserved.

#include "SharedMemoryMediaOutput.h"

#include "MediaOutput.h"
#include "Engine/RendererSettings.h"
#include "SharedMemoryMediaCapture.h"
#include "SharedMemoryMediaTypes.h"

bool USharedMemoryMediaOutput::Validate(FString& OutFailureReason) const
{
	if (!Super::Validate(OutFailureReason))
	{
		return false;
	}
	
	return true;
}

FIntPoint USharedMemoryMediaOutput::GetRequestedSize() const
{
	return RequestCaptureSourceSize;
}

EPixelFormat USharedMemoryMediaOutput::GetRequestedPixelFormat() const
{
	// We are copying to a shared cross gpu texture allocated outside the MF, so this PF needs to be valid but is not relevant.
	// @todo is there a way to not have the MF render target allocated since we're not using it ?

	static const auto CVarDefaultBackBufferPixelFormat = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.DefaultBackBufferPixelFormat"));
	return EDefaultBackBufferPixelFormat::Convert2PixelFormat(EDefaultBackBufferPixelFormat::FromInt(CVarDefaultBackBufferPixelFormat->GetValueOnGameThread()));
}

EMediaCaptureConversionOperation USharedMemoryMediaOutput::GetConversionOperation(EMediaCaptureSourceType InSourceType) const
{
	return EMediaCaptureConversionOperation::CUSTOM;
}

UMediaCapture* USharedMemoryMediaOutput::CreateMediaCaptureImpl()
{
	UMediaCapture* Result = NewObject<USharedMemoryMediaCapture>();
	if (Result)
	{
		Result->SetMediaOutput(this);
	}
	return Result;
}
