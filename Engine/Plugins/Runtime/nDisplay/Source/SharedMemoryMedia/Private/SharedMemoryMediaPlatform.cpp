// Copyright Epic Games, Inc. All Rights Reserved.

#include "SharedMemoryMediaPlatform.h"

FString FSharedMemoryMediaPlatformFactory::GetRhiTypeString(const ERHIInterfaceType RhiType)
{
	static TMap<ERHIInterfaceType, FString> Rhis;

	if (!Rhis.Num())
	{
		Rhis.Emplace(ERHIInterfaceType::Hidden, TEXT("Hidden"));
		Rhis.Emplace(ERHIInterfaceType::Null, TEXT("Null"));
		Rhis.Emplace(ERHIInterfaceType::D3D11, TEXT("D3D11"));
		Rhis.Emplace(ERHIInterfaceType::D3D12, TEXT("D3D12"));
		Rhis.Emplace(ERHIInterfaceType::Vulkan, TEXT("Vulkan"));
		Rhis.Emplace(ERHIInterfaceType::Metal, TEXT("Metal"));
		Rhis.Emplace(ERHIInterfaceType::Agx, TEXT("Agx"));
		Rhis.Emplace(ERHIInterfaceType::OpenGL, TEXT("OpenGL"));
	};

	const FString* RhiString = Rhis.Find(RhiType);

	return RhiString ? *RhiString : TEXT("Unknown");
}

bool FSharedMemoryMediaPlatformFactory::RegisterPlatformForRhi(ERHIInterfaceType RhiType, CreateSharedMemoryMediaPlatform PlatformCreator)
{
	PlatformCreators.Add(RhiType, PlatformCreator);
	return true;
}

TSharedPtr<FSharedMemoryMediaPlatform, ESPMode::ThreadSafe> FSharedMemoryMediaPlatformFactory::CreateInstanceForRhi(ERHIInterfaceType RhiType)
{
	if (const CreateSharedMemoryMediaPlatform* Creator = PlatformCreators.Find(RhiType))
	{
		return (*Creator)();
	}

	return nullptr;
}

FSharedMemoryMediaPlatformFactory* FSharedMemoryMediaPlatformFactory::Get()
{
	static FSharedMemoryMediaPlatformFactory Instance;
	return &Instance;
}


