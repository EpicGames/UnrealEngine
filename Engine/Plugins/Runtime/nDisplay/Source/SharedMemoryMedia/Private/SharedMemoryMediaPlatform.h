// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHI.h"

#include "SharedMemoryMediaTypes.h"


/** This is the interface to the platform specific functions needed by shared memory media. 
 *  It is mostly about the platform specific implementation of cross gpu textures.
 */
class FSharedMemoryMediaPlatform
{

public:

	/** Creates a cross gpu texture */
	virtual FTextureRHIRef CreateSharedTexture(EPixelFormat Format, bool bSrgb, int32 Width, int32 Height, const FGuid& Guid, uint32 BufferIdx, bool bCrossGpu)
	{
		return nullptr;
	}

	/** Opens a cross gpu texture specified by a Guid */
	virtual FTextureRHIRef OpenSharedTextureByGuid(const FGuid& Guid, FSharedMemoryMediaTextureDescription& OutTextureDescription)
	{
		return nullptr;
	}

	/** Release any platform specific resources related to the indexed texture */
	virtual void ReleaseSharedTexture(uint32 BufferIdx) {};
};

/** Factory of registered FSharedMemoryMediaPlatform RHI implementations. Use RegisterPlatformForRhi to register them. */
class FSharedMemoryMediaPlatformFactory
{
public:

	/** This is the type definition of the rhi platform factory function that each rhi platfom implementation can register with.  */
	typedef TSharedPtr<FSharedMemoryMediaPlatform>(*CreateSharedMemoryMediaPlatform)();

public:

	/** Rhi platform implementations call this function to register their factory creation function */
	bool RegisterPlatformForRhi(ERHIInterfaceType RhiType, CreateSharedMemoryMediaPlatform PlatformCreator);

	/** This factory function will create an instance of the rhi platform specific implementation, if it has been registered. */
	TSharedPtr<FSharedMemoryMediaPlatform, ESPMode::ThreadSafe> CreateInstanceForRhi(ERHIInterfaceType RhiType);

	/** Gets the singleton instance of this factory */
	static FSharedMemoryMediaPlatformFactory* Get();

	/** Helper function to get a stringified ERHIInterfaceType */
	static FString GetRhiTypeString(const ERHIInterfaceType RhiType);

private:

	/** TMap of the collection of registered rhi platform factory functions */
	TMap<ERHIInterfaceType, CreateSharedMemoryMediaPlatform> PlatformCreators;
};
