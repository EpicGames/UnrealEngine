// Copyright Epic Games, Inc. All Rights Reserved.

#include "SharedMemoryMediaPlayer.h"

#include "Engine/Texture2D.h"
#include "IMediaEventSink.h"
#include "IMediaTextureSampleConverter.h"
#include "Misc/Timespan.h"
#include "Tasks/Task.h"

#include "RHIUtilities.h"
#include "TextureResource.h"

#include "SharedMemoryMediaOutput.h"
#include "SharedMemoryMediaPlatform.h"
#include "SharedMemoryMediaSourceOptions.h"
#include "SharedMemoryMediaSample.h"
#include "SharedMemoryMediaSamples.h"
#include "SharedMemoryMediaTextureSampleConverter.h"
#include "SharedMemoryMediaTypes.h"


#define LOCTEXT_NAMESPACE "FSharedMemoryMediaPlayer"

DECLARE_GPU_STAT_NAMED(SharedMemoryMedia_CopyToSampleCommon, TEXT("SharedMemoryMedia_CopyToSampleCommon"));
DECLARE_GPU_STAT_NAMED(SharedMemoryMedia_UnlockTexture, TEXT("SharedMemoryMedia_UnlockTexture"));
DECLARE_GPU_STAT_NAMED(SharedMemoryMedia_LockTexture, TEXT("SharedMemoryMedia_LockTexture"));
DECLARE_GPU_STAT_NAMED(SharedMemoryMedia_WaitForPixels, TEXT("SharedMemoryMedia_WaitForPixels"));


using namespace UE::SharedMemoryMedia;

int32 GDisplayClusterSharedMemoryLatency = -1;
static FAutoConsoleVariableRef CVarDisplayClusterSharedMemoryLatency(
	TEXT("DC.SharedMemoryMedia.Latency"),
	GDisplayClusterSharedMemoryLatency,
	TEXT("Forces frame latency on all shared memory media players. Valid values are -1 (don't force), 0 and 1."),
	ECVF_RenderThreadSafe
);



FSharedMemoryMediaPlayer::FSharedMemoryMediaPlayer()
	: Super()
{
	Samples = new FSharedMemoryMediaSamples();

	// Invalidate the receiver indices, indicating that we need to find a spot in the shared metadata
	// to interact with the sender.
	for (int32 Idx = 0; Idx < NUMSHAREDMEM; ++Idx)
	{
		ReceiverIndex[Idx] = INDEX_NONE;
	}
}

FSharedMemoryMediaPlayer::~FSharedMemoryMediaPlayer()
{
	Close();
	delete Samples;
}


bool FSharedMemoryMediaPlayer::Open(const FString& Url, const IMediaOptions* Options)
{
	// Get an RHI type specific implementation.
	if (!PlatformData.IsValid())
	{
		const ERHIInterfaceType RhiInterfaceType = GDynamicRHI->GetInterfaceType();

		PlatformData = FSharedMemoryMediaPlatformFactory::Get()->CreateInstanceForRhi(RhiInterfaceType);

		if (!PlatformData.IsValid())
		{
			UE_LOGF(LogSharedMemoryMedia, Error, "Unfortunately, SharedMemoryMedia doesn't support the current RHI type '%ls'",
				*FSharedMemoryMediaPlatformFactory::GetRhiTypeString(RhiInterfaceType));
			Close();
			return true;
		}
	}

	// Cache the Url for later query responses
	OpenUrl = Url;

	// Grab unique name
	{
		const FString EmptyUniqueName;
		UniqueName = Options->GetMediaOption(SharedMemoryMediaOption::UniqueName, EmptyUniqueName);

		if (!UniqueName.Len())
		{
			UE_LOGF(LogSharedMemoryMedia, Error, "SharedMemoryMediaSource must have a UniqueName that is not empty, and should match the MediaOutput's UniqueName");
			Close();
			return true;
		}
	}

	// Grab zero latency option
	{
		bZeroLatency = Options->GetMediaOption(SharedMemoryMediaOption::ZeroLatency, true);
	}

	// Grab Mode of operation
	{
		Mode = ESharedMemoryMediaSourceMode(Options->GetMediaOption(SharedMemoryMediaOption::Mode, int64(ESharedMemoryMediaSourceMode::Framelocked)));
	}

	// Initialize gpu fences
	for (int32 BufferIdx = 0; BufferIdx < NUMSHAREDMEM; BufferIdx++)
	{
		if (!FrameAckFences[BufferIdx])
		{
			FrameAckFences[BufferIdx] = RHICreateGPUFence(TEXT("SharedMemoryMediaFrameAckFence"));
			bFrameAckFenceBusy[BufferIdx] = false;
		}
	}

	// Set a playing state.
	PlayerState = EMediaState::Playing;

	return true;
}

bool FSharedMemoryMediaPlayer::Open(const TSharedRef<FArchive, ESPMode::ThreadSafe>& Archive, const FString& OriginalUrl, const IMediaOptions* Options)
{
	return false;
}

void FSharedMemoryMediaPlayer::Close()
{
	OpenUrl.Empty();
	Samples->FlushSamples();

	// We flush rendering commands so that we can safely release the resources
	FlushRenderingCommands();

	// Wait for any pending tasks to finish, which could be trying to use the resources as well.
	while (RunningTasksCount > 0)
	{
		FPlatformProcess::SleepNoStats(SpinWaitTimeSeconds);
	}

	// Release resources related to the indexed textures
	for (int32 BufferIdx = 0; BufferIdx < NUMSHAREDMEM; ++BufferIdx)
	{
		// Free cross gpu texture
		SharedCrossGpuTextures[BufferIdx].SafeRelease();

		// Free shared memory
		if (SharedMemory[BufferIdx])
		{
			// Let the sender know that we're not monitoring this and that it should not wait for acks

			FSharedMemoryMediaFrameMetadata* Data = static_cast<FSharedMemoryMediaFrameMetadata*>(SharedMemory[BufferIdx]->GetAddress());

			if (Data)
			{
				Data->DisconnectReceiverFromSlot(ReceiverId, ReceiverIndex[BufferIdx]); // Internally checks for INDEX_NONE
			}

			// Close the shared memory
			FPlatformMemory::UnmapNamedSharedMemoryRegion(SharedMemory[BufferIdx]);
			SharedMemory[BufferIdx] = nullptr;
		}

		// Unconditionally invalidate the receiver indices
		ReceiverIndex[BufferIdx] = INDEX_NONE;

		// Free gpu fences
		{
			// Fences should not be busy since we flush rendering commands.
			check(!bFrameAckFenceBusy[BufferIdx]);

			FrameAckFences[BufferIdx].SafeRelease();
		}
	}

	// Free sample common texture
	SampleCommonTexture = nullptr;

	// Free platform specific resources
	PlatformData.Reset();

	// Reset our state variables
	ModeState.Reset();

	// Update our player state
	PlayerState = EMediaState::Closed;
}

FGuid FSharedMemoryMediaPlayer::GetPlayerPluginGUID() const
{
	return UE::SharedMemoryMedia::PlayerGuid;
}

FString FSharedMemoryMediaPlayer::GetStats() const
{
	return FString();
}

FTimespan FSharedMemoryMediaPlayer::GetTime() const
{
	// We fabricate a time based on the frame number, to control what sample is used for which frame.
	return FTimespan(GFrameCounter);
}

void FSharedMemoryMediaPlayer::TickFetch(FTimespan DeltaTime, FTimespan Timecode)
{
	Super::TickFetch(DeltaTime, Timecode);

	// Don't populate anything if we're not playing
	if (GetState() != EMediaState::Playing)
	{
		return;
	}

	// Make sure we have platform data
	if (!PlatformData.IsValid())
	{
		return;
	}

	// Make sure the sender's shared memory metadata is available. If not, don't bother creating a media sample.
	//
	for (int32 MemIdx = 0; MemIdx < NUMSHAREDMEM; ++MemIdx)
	{
		if (!SharedMemory[MemIdx])
		{
			const FGuid SharedMemoryGuid = UE::SharedMemoryMedia::GenerateSharedMemoryGuid(UniqueName, MemIdx);
			const FString SharedMemoryRegionName = SharedMemoryGuid.ToString(EGuidFormats::DigitsWithHyphensInBraces);

			const SIZE_T SharedMemorySize = sizeof(FSharedMemoryMediaFrameMetadata);

			// Open existing shared memory region, in case it exists:

			const uint32 AccessMode = FPlatformMemory::ESharedMemoryAccess::Read | FPlatformMemory::ESharedMemoryAccess::Write;

			SharedMemory[MemIdx] = FPlatformMemory::MapNamedSharedMemoryRegion(
				*SharedMemoryRegionName, false /* bCreate */, AccessMode, SharedMemorySize
			);

			if (SharedMemory[MemIdx])
			{
				UE_LOGF(LogSharedMemoryMedia, Verbose, "Opened SharedMemory named '%ls'", *SharedMemoryRegionName);

				FSharedMemoryMediaFrameMetadata* Data = static_cast<FSharedMemoryMediaFrameMetadata*>(SharedMemory[MemIdx]->GetAddress());
				check(Data);
			}
			else
			{
				UE_LOGF(LogSharedMemoryMedia, Warning, "Could not open SharedMemory named '%ls'", *SharedMemoryRegionName);
			}
		}

		// No point in continuing if there is no shared memory region available
		if (!SharedMemory[MemIdx])
		{
			return;
		}
	}

	// Since we can't send a sample without a valid texture, we are going to wait until we get a valid shared gpu texture
	{
		const uint32 ExpectedFrameNumber = InputTextureFrameNumberForFrameNumber(GFrameCounter);
		const int32 MemIdx = ExpectedFrameNumber % NUMSHAREDMEM;

		FSharedMemoryMediaFrameMetadata* SharedMemoryData = static_cast<FSharedMemoryMediaFrameMetadata*>(SharedMemory[MemIdx]->GetAddress());

		check(SharedMemoryData);

		if (SharedMemoryData->Sender.Magic != FSharedMemoryMediaFrameMetadata::MAGIC)
		{
			return;
		}

		if (SharedMemoryData->Sender.TextureGuid == UE::SharedMemoryMedia::ZeroGuid)
		{
			return;
		}
	}

	// Update shared textures
	//
	for (uint32 MemIdx = 0; MemIdx < NUMSHAREDMEM; MemIdx++)
	{
		// If our sender is not there, keep idling

		check(SharedMemory[MemIdx]);
		FSharedMemoryMediaFrameMetadata* SharedMemoryMetadataPtr = static_cast<FSharedMemoryMediaFrameMetadata*>(SharedMemory[MemIdx]->GetAddress());
		check(SharedMemoryMetadataPtr);

		const FSharedMemoryMediaFrameMetadata::FSender SenderData = SharedMemoryMetadataPtr->Sender;

		if (SenderData.Magic != FSharedMemoryMediaFrameMetadata::MAGIC)
		{
			continue;
		}

		// See if we already have the gpu texture
		if (SharedCrossGpuTextures[MemIdx].IsValid())
		{
			// We have a texture, now see if it needs to be updated based on the Guid

			if (SenderData.TextureGuid == SharedCrossGpuTextureDescriptions[MemIdx].Guid)
			{
				// Same guid as before, so we continue normally
				continue;
			}

			// The texture guid has changed, so we need to update it if there is a valid one.

			if (SenderData.TextureGuid == UE::SharedMemoryMedia::ZeroGuid)
			{
				// Guid isn't valid anymore, so we keep idling, we expect that there will be a valid one soon.
				continue;
			}

			// We flush rendering commands so that we can safely update
			FlushRenderingCommands();

			// Release our current gpu texture
			SharedCrossGpuTextures[MemIdx].SafeRelease();

			// Release our sample common texture as well, since its description will likely not match the cross gpu texture anymore
			SampleCommonTexture = nullptr;

			// No need for the sender to wait for us while we figure out the new textures.
			SharedMemoryMetadataPtr->DisconnectReceiverFromSlot(ReceiverId, ReceiverIndex[MemIdx]); // Already checks for INDEX_NONE
			ReceiverIndex[MemIdx] = INDEX_NONE;

			// We now exit this block, which will treat the invalid gpu texture case.
		}

		// Reset the cross gpu texture description
		SharedCrossGpuTextureDescriptions[MemIdx] = FSharedMemoryMediaTextureDescription();

		// Check if the shared memory has the cross gpu texture data that we need

		const FGuid& SharedGpuTextureGuid = SenderData.TextureGuid;

		// Guid of zero is not valid
		if (SharedGpuTextureGuid == UE::SharedMemoryMedia::ZeroGuid)
		{
			UE_LOGF(LogSharedMemoryMedia, Warning, "Invalid zero guid");
			continue;
		}

		// Now that we have the shared texture guid from the sender, use it to open the associated texture

		SharedCrossGpuTextures[MemIdx] = PlatformData->OpenSharedTextureByGuid(SharedGpuTextureGuid, SharedCrossGpuTextureDescriptions[MemIdx]);

		if (!SharedCrossGpuTextures[MemIdx].IsValid())
		{
			continue;
		}

		// Since the texture is valid, set the Guid in its description.
		SharedCrossGpuTextureDescriptions[MemIdx].Guid = SenderData.TextureGuid;
	}

	// Don't create samples if we don't have shared gpu memory handles yet
	for (uint32 MemIdx = 0; MemIdx < NUMSHAREDMEM; MemIdx++)
	{
		if (!SharedCrossGpuTextures[MemIdx].IsValid())
		{
			return;
		}
	}

	// The descriptions of all buffers should match
	for (uint32 MemIdx = 1; MemIdx < NUMSHAREDMEM; MemIdx++)
	{
		if (!SharedCrossGpuTextureDescriptions[MemIdx - 1].IsEquivalentTo(SharedCrossGpuTextureDescriptions[MemIdx]))
		{
			return;
		}
	}

	// Initialize sample texture if needed
	if (!SampleCommonTexture.IsValid())
	{
		SampleCommonTexture.Reset(UTexture2D::CreateTransient(
			SharedCrossGpuTextureDescriptions[0].Width,
			SharedCrossGpuTextureDescriptions[0].Height,
			SharedCrossGpuTextureDescriptions[0].Format,
			MakeUniqueObjectName(nullptr, UTexture2D::StaticClass(), FName(*FString::Printf(TEXT("SampleCommonTexture_%s"), *UniqueName)))
		));

		SampleCommonTexture->SRGB = SharedCrossGpuTextureDescriptions[0].bSrgb;

		SampleCommonTexture->UpdateResource();
	}

	check(SampleCommonTexture.IsValid() && SampleCommonTexture->GetResource());

	// Don't create samples if we don't have the sample common resource yet
	if (!SampleCommonTexture->GetResource()->GetTextureRHI()) // Could be empty the first time
	{
		return;
	}

	// Make sure that we have a receiver slot, and keep it alive.
	for (int32 MemIdx = 0; MemIdx < NUMSHAREDMEM; ++MemIdx)
	{
		FSharedMemoryMediaFrameMetadata* SharedMemoryMetadataPtr = static_cast<FSharedMemoryMediaFrameMetadata*>(SharedMemory[MemIdx]->GetAddress());

		// Make sure our current index is valid
		if (!SharedMemoryMetadataPtr->IsReceiverConnectedToSlot(ReceiverId, ReceiverIndex[MemIdx]))
		{
			ReceiverIndex[MemIdx] = INDEX_NONE;
		}

		// Find a slot if we don't have one
		
		if (ReceiverIndex[MemIdx] == INDEX_NONE)
		{
			ReceiverIndex[MemIdx] = SharedMemoryMetadataPtr->ConnectToSlot(ReceiverId);

			// If we don't have a valid slot at this point, we must return and try again later.
			if (ReceiverIndex[MemIdx] == INDEX_NONE)
			{
				return;
			}
		}

		// Let sender know that we're monitoring this shared memory.
		// We expect the sender to not wait for acks if nobody is monitoring it.
		if (!SharedMemoryMetadataPtr->KeepAlive(ReceiverIndex[MemIdx]))
		{
			// If keep alive failed, it means that we're not connected anymore.
			ReceiverIndex[MemIdx] = INDEX_NONE;
			return;
		}
	}

	// Create a new media sample and populate it with any needed data
	{
		Samples->CurrentSample = MakeShared<FSharedMemoryMediaSample>();
		Samples->CurrentSample->Player = AsShared();
		Samples->CurrentSample->Texture = SampleCommonTexture->GetResource()->GetTextureRHI();
		Samples->CurrentSample->Dim = FIntPoint(SharedCrossGpuTextureDescriptions[0].Width, SharedCrossGpuTextureDescriptions[0].Height);
		Samples->CurrentSample->Stride = SharedCrossGpuTextureDescriptions[0].Stride;
		Samples->CurrentSample->bSrgb = SharedCrossGpuTextureDescriptions[0].bSrgb;
		
		// Try to preserve the common input formats when specifying EMediaTextureSampleFormat

		switch (SharedCrossGpuTextureDescriptions[0].Format)
		{
		case PF_FloatRGB   : Samples->CurrentSample->Format = EMediaTextureSampleFormat::FloatRGB;    break;
		case PF_FloatRGBA  : Samples->CurrentSample->Format = EMediaTextureSampleFormat::FloatRGBA;   break;
		case PF_B8G8R8A8   : Samples->CurrentSample->Format = EMediaTextureSampleFormat::CharBGRA;    break;
		case PF_A2B10G10R10: Samples->CurrentSample->Format = EMediaTextureSampleFormat::CharBGR10A2; break;
		default            : Samples->CurrentSample->Format = EMediaTextureSampleFormat::CharBGR10A2; break;
		}

		// Ticks are "frames" in our player. W/o constraining the sample timing, it would be possible for the
		// render thread to purge the sample of the next frame, in particular when the game thread stalls.
		Samples->CurrentSample->Time = FTimespan(GFrameCounter);
	}
}

void FSharedMemoryMediaPlayer::TickInput(FTimespan DeltaTime, FTimespan Timecode)
{
	Super::TickInput(DeltaTime, Timecode);

	// @todo update our state
}

bool FSharedMemoryMediaPlayer::GetPlayerFeatureFlag(EFeatureFlag Flag) const
{
	if (Super::GetPlayerFeatureFlag(Flag))
	{
		return true;
	}

	return false;
}

IMediaSamples& FSharedMemoryMediaPlayer::GetSamples()
{
	return *Samples;
}

IMediaCache& FSharedMemoryMediaPlayer::GetCache()
{
	return *this;
}

IMediaControls& FSharedMemoryMediaPlayer::GetControls() 
{
	return *this;
}

FString FSharedMemoryMediaPlayer::GetInfo() const 
{
	return FString();
}

IMediaTracks& FSharedMemoryMediaPlayer::GetTracks() 
{
	return *this;
}

FString FSharedMemoryMediaPlayer::GetUrl() const 
{
	return OpenUrl;
}

IMediaView& FSharedMemoryMediaPlayer::GetView()
{
	return *this;
}


bool FSharedMemoryMediaPlayer::DetermineNextSourceFrameFramelockMode(uint64 FrameNumber, uint32& OutExpectedFrameNumber, uint32& OutSharedMemoryIdx)
{
	// When framelocked, the next source frame is determined by the current local frame number.
	OutExpectedFrameNumber = InputTextureFrameNumberForFrameNumber(FrameNumber);
	OutSharedMemoryIdx = OutExpectedFrameNumber % NUMSHAREDMEM;

	return true;
}

bool FSharedMemoryMediaPlayer::DetermineNextSourceFrameGenlockMode(uint64 FrameNumber, uint32& OutExpectedFrameNumber, uint32& OutSharedMemoryIdx)
{
	// Look for the closest thing to the next consecutive frame.

	FModeState::FGenlockModeState& State = ModeState.Genlock;

	FSharedMemoryMediaFrameMetadata* SharedMemoryData[NUMSHAREDMEM] = { 0 };
	FSharedMemoryMediaFrameMetadata::FSender SenderMetadata[NUMSHAREDMEM];

	TArray<uint32, TInlineAllocator<NUMSHAREDMEM>> ValidFrameNumbers;

	for (int32 MemIdx = 0; MemIdx < NUMSHAREDMEM; ++MemIdx)
	{
		SharedMemoryData[MemIdx] = static_cast<FSharedMemoryMediaFrameMetadata*>(SharedMemory[MemIdx]->GetAddress());
		check(SharedMemoryData[MemIdx]);

		FMemory::Memcpy(&SenderMetadata[MemIdx], SharedMemoryData[MemIdx], sizeof(FSharedMemoryMediaFrameMetadata::FSender));

		// Data needs to be valid
		if (SenderMetadata[MemIdx].Magic != FSharedMemoryMediaFrameMetadata::MAGIC)
		{
			continue;
		}

		ValidFrameNumbers.Add(SenderMetadata[MemIdx].FrameNumber);
	}

	// If none were valid, return. We will not wait any time for them to arrive at this point.
	if (!ValidFrameNumbers.Num())
	{
		return false;
	}

	uint32 MinDistanceFromLastPicked = INT_MAX;

	for (const uint32& ValidFrameNumber : ValidFrameNumbers)
	{
		const uint32 DistanceFromPicked = 
			State.LastSourceFrameNumberPicked > ValidFrameNumber ? 
			State.LastSourceFrameNumberPicked - ValidFrameNumber : ValidFrameNumber - State.LastSourceFrameNumberPicked;

		MinDistanceFromLastPicked = FMath::Min(DistanceFromPicked, MinDistanceFromLastPicked);
	}

	// If our frame number seems far from what is in the buffers, reset our state
	if (MinDistanceFromLastPicked > 2)
	{
		UE_LOGF(LogSharedMemoryMedia, Warning, "SharedMemoryMedia: In genlock mode, the sender's frame count seems to have reset to earlier than %u",
			State.LastSourceFrameNumberPicked);

		State.Reset();

		ValidFrameNumbers.Sort();
		State.LastSourceFrameNumberPicked = ValidFrameNumbers.Last() - 1;
	}

	OutExpectedFrameNumber = ++State.LastSourceFrameNumberPicked;
	OutSharedMemoryIdx = OutExpectedFrameNumber % NUMSHAREDMEM;

	return true;
}

bool FSharedMemoryMediaPlayer::DetermineNextSourceFrameFreerunMode(uint64 FrameNumber, uint32& OutExpectedFrameNumber, uint32& OutSharedMemoryIdx)
{
	// Use the latest frame, ack the others (if they haven't already been acked).

	FModeState::FFreerunModeState& State = ModeState.Freerun;

	// See what is on all the reception buffers

	FSharedMemoryMediaFrameMetadata* SharedMemoryData[NUMSHAREDMEM];
	FSharedMemoryMediaFrameMetadata::FSender SenderMetadata[NUMSHAREDMEM];

	int32 HighestFrameNumberIdx = -1;
	uint32 HighestFrameNumber = 0;

	int32 LowestFrameNumberIdx = -1;
	uint32 LowestFrameNumber = ~0;

	for (int32 MemIdx = 0; MemIdx < NUMSHAREDMEM; ++MemIdx)
	{
		SharedMemoryData[MemIdx] = static_cast<FSharedMemoryMediaFrameMetadata*>(SharedMemory[MemIdx]->GetAddress());
		check(SharedMemoryData[MemIdx]);

		FMemory::Memcpy(&SenderMetadata[MemIdx], SharedMemoryData[MemIdx], sizeof(FSharedMemoryMediaFrameMetadata::FSender));

		// Data needs to be valid
		if (SenderMetadata[MemIdx].Magic != FSharedMemoryMediaFrameMetadata::MAGIC)
		{
			continue;
		}

		if (HighestFrameNumber < SenderMetadata[MemIdx].FrameNumber)
		{
			HighestFrameNumber = SenderMetadata[MemIdx].FrameNumber;
			HighestFrameNumberIdx = MemIdx;
		}

		if (LowestFrameNumber > SenderMetadata[MemIdx].FrameNumber)
		{
			LowestFrameNumber = SenderMetadata[MemIdx].FrameNumber;
			LowestFrameNumberIdx = MemIdx;
		}
	}

	// If none were valid, return. We will not wait any time for them to arrive at this point.
	if (HighestFrameNumberIdx < 0)
	{
		return false;
	}

	check(LowestFrameNumberIdx >= 0);

	// No point in rendering the same frame
	if (State.LastSourceFrameNumberPicked == HighestFrameNumber)
	{
		return false;
	}
	else if (State.LastSourceFrameNumberPicked > HighestFrameNumber)
	{
		// We have picked in the past a frame number higher than the highest in the buffers.
		// This is unexpected, something may have happened to our source.
		UE_LOGF(LogSharedMemoryMedia, Warning, "SharedMemoryMedia: In freerun mode, the sender's frame count seems to have reset from %u to %u.",
			State.LastSourceFrameNumberPicked, HighestFrameNumber);

		OutExpectedFrameNumber = LowestFrameNumber;
		OutSharedMemoryIdx = LowestFrameNumberIdx;

		State.Reset();
		State.LastSourceFrameNumberConsideredAtIdx[OutSharedMemoryIdx] = OutExpectedFrameNumber;
		State.LastSourceFrameNumberPicked = OutExpectedFrameNumber;

		return true;
	}

	check(State.LastSourceFrameNumberPicked < HighestFrameNumber);

	// ack the ones we're skipping in favor of the latest one
	for (int32 MemIdx = 0; MemIdx < NUMSHAREDMEM; ++MemIdx)
	{
		const uint32 LastSourceFrameNumberConsideredAtThisIdx = State.LastSourceFrameNumberConsideredAtIdx[MemIdx];

		State.LastSourceFrameNumberConsideredAtIdx[MemIdx] = SenderMetadata[MemIdx].FrameNumber;

		// Don't ack the latest one just yet (we'll ack it once we're done with it)
		if (HighestFrameNumberIdx == MemIdx)
		{
			continue;
		}

		// Ack the skipped frames (that haven't been considered and therefore acked before)
		if (LastSourceFrameNumberConsideredAtThisIdx < SenderMetadata[MemIdx].FrameNumber)
		{
			check(ReceiverIndex[MemIdx] != INDEX_NONE); // We don't expect this function to be called if we didn't have a valid slot.

			SharedMemoryData[MemIdx]->Receivers[ReceiverIndex[MemIdx]].FrameNumberAcked = SenderMetadata[MemIdx].FrameNumber;
		}
	}

	OutExpectedFrameNumber = HighestFrameNumber;
	OutSharedMemoryIdx = HighestFrameNumberIdx;

	State.LastSourceFrameNumberPicked = OutExpectedFrameNumber;

	return true;

}

bool FSharedMemoryMediaPlayer::DetermineNextSourceFrame(uint64 FrameNumber, uint32& OutExpectedFrameNumber, uint32& OutSharedMemoryIdx)
{
	switch (Mode)
	{
	case ESharedMemoryMediaSourceMode::Framelocked:
		return DetermineNextSourceFrameFramelockMode(FrameNumber, OutExpectedFrameNumber, OutSharedMemoryIdx);

	case ESharedMemoryMediaSourceMode::Freerun:
		return DetermineNextSourceFrameFreerunMode(FrameNumber, OutExpectedFrameNumber, OutSharedMemoryIdx);

	case ESharedMemoryMediaSourceMode::Genlocked:
		return DetermineNextSourceFrameGenlockMode(FrameNumber, OutExpectedFrameNumber, OutSharedMemoryIdx);

	default:
		unimplemented();
		break;
	}

	return false;
}

void FSharedMemoryMediaPlayer::JustInTimeSampleRender(FRHICommandListImmediate& RHICmdList)
{
	// We only allow this function to run once per frame.
	if (LastFrameNumberThatUpdatedJustInTime == GFrameCounterRenderThread)
	{
		UE_LOGF(LogSharedMemoryMedia, Verbose, 
			"FSharedMemoryMediaPlayer '%ls' JustInTimeSampleRender called more than once in GFrameCounterRenderThread %llu",
			*UniqueName, GFrameCounterRenderThread
		);

		return;
	}

	LastFrameNumberThatUpdatedJustInTime = GFrameCounterRenderThread;

	UE_LOGF(LogSharedMemoryMedia, VeryVerbose, "FSharedMemoryMediaPlayer '%ls' JustInTimeSampleRender called for GFrameCounterRenderThread %llu", 
		*UniqueName, GFrameCounterRenderThread
	);

	for (int32 MemIdx = 0; MemIdx < NUMSHAREDMEM; ++MemIdx)
	{
		// No point in continuing if there is no shared memory region available
		if (!SharedMemory[MemIdx])
		{
			return;
		}
	}

	uint32 ExpectedFrameNumber;
	uint32 SharedMemoryIdx;

	if (!DetermineNextSourceFrame(GFrameCounterRenderThread, ExpectedFrameNumber, SharedMemoryIdx))
	{
		return;
	}

	check(SharedMemoryIdx < NUMSHAREDMEM);

	FSharedMemoryMediaFrameMetadata* SharedMemoryData = static_cast<FSharedMemoryMediaFrameMetadata*>(SharedMemory[SharedMemoryIdx]->GetAddress());
	check(SharedMemoryData);

	// Enqueue a lambda that will wait for the cross gpu texture for this frame have the data populated.
	// It will determine this by polling the sender's shared memory metadata.
	// To avoid any hangs, it will give up after some time.
	{
		RHI_BREADCRUMB_EVENT_STAT(RHICmdList, SharedMemoryMedia_WaitForPixels, "SharedMemoryMedia_WaitForPixels");

		// Since we are going to enqueue a lambda that can potentially sleep in the RHI thread if the pixels haven't arrived,
		// we dispatch the existing commands (including the draw event start timing in the SCOPED_DRAW_EVENT above) before any potential sleep.
		RHICmdList.ImmediateFlush(EImmediateFlushType::DispatchToRHIThread);

		RHICmdList.EnqueueLambda(
			[
				FrameNumber = GFrameCounterRenderThread, 
				SharedMemoryIdx, 
				ExpectedFrameNumber, 
				SharedMemoryData,
				this
			](FRHICommandListImmediate& RHICmdList)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(SharedMemMediaWaitForSharedGpuTexture);

				alignas(128) FSharedMemoryMediaFrameMetadata::FSender SenderMetadata;

				const double StartTimeSeconds = FPlatformTime::Seconds();
				constexpr double TimeoutSeconds = 0.5;

				while (true)
				{
					FMemory::Memcpy(&SenderMetadata, SharedMemoryData, sizeof(FSharedMemoryMediaFrameMetadata::FSender));

					if (SenderMetadata.Magic == FSharedMemoryMediaFrameMetadata::MAGIC)
					{
						if (SenderMetadata.FrameNumber >= ExpectedFrameNumber)
						{
							if (SenderMetadata.FrameNumber > ExpectedFrameNumber)
							{
								UE_LOGF(LogSharedMemoryMedia, Warning, "Using too recent frame. Expected %u and used %u", 
									ExpectedFrameNumber, SenderMetadata.FrameNumber);
							}

							break;
						}
					}

					FPlatformProcess::SleepNoStats(SpinWaitTimeSeconds);

					if ((FPlatformTime::Seconds() - StartTimeSeconds) > TimeoutSeconds)
					{
						UE_LOGF(LogSharedMemoryMedia, Warning, "FSharedMemoryMediaPlayer timed out waiting for ExpectedFrameNumber %u for frame %llu, only saw up to frame %u",
							ExpectedFrameNumber, FrameNumber, SenderMetadata.FrameNumber);

						// Note: It would be desirable to stop the copy texture from happening, but at this point it has already been enqueued.

						break;
					}
				}
			}
		);
	}

	// Copy to sample common texture

	check(SampleCommonTexture.IsValid() && SampleCommonTexture->GetResource());

	// But first wait for the fence to be cleared, in case it hasn't (which is not usual).
	if (bFrameAckFenceBusy[SharedMemoryIdx])
	{
		UE_LOGF(LogSharedMemoryMedia, Warning, "bFrameAckFenceBusy[%d] was busy for ExpectedFrameNumber %d",
			SharedMemoryIdx, ExpectedFrameNumber);

		TRACE_CPUPROFILER_EVENT_SCOPE(SharedMemMediaWaitForFrameAckFenceBusyToClear);

		while (bFrameAckFenceBusy[SharedMemoryIdx]) // The flag is guaranteed to eventually clear by the async task
		{
			FPlatformProcess::SleepNoStats(SpinWaitTimeSeconds);
		}
	}

	// Enqueue the copy from cross gpu texture to sample common texture
	if (SharedCrossGpuTextures[SharedMemoryIdx].IsValid())
	{
		FRHICopyTextureInfo CopyInfo;

		RHI_BREADCRUMB_EVENT_STAT(RHICmdList, SharedMemoryMedia_CopyToSampleCommon, "SharedMemoryMedia_CopyToSampleCommon");

		TransitionAndCopyTexture(RHICmdList, SharedCrossGpuTextures[SharedMemoryIdx], SampleCommonTexture->GetResource()->GetTextureRHI(), CopyInfo);
	}
	else
	{
		UE_LOGF(LogSharedMemoryMedia, Error, "SharedCrossGpuTextures[%d] was unexpectedly invalid", SharedMemoryIdx);
	}

	// Write gpu fence to indicate we're done with the shared cross gpu texture
	{
		// This is our atomic flag that avoids this thread from re-using the fence before it is cleared.
		// It will be cleared by the async task.
		bFrameAckFenceBusy[SharedMemoryIdx] = true;

		RHICmdList.WriteGPUFence(FrameAckFences[SharedMemoryIdx]);
	}

	// spawn async task that will wait on FrameAckFence, ack the frame to the Media Capture, and flag that it is clear for re-use

	RunningTasksCount++;

	UE::Tasks::Launch(UE_SOURCE_LOCATION, 
		[
			SharedMemoryData, 
			ExpectedFrameNumber, 
			SharedMemoryIdx, 
			ReceiverIdx = ReceiverIndex[SharedMemoryIdx], 
			this
	    ]()
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(SharedMemoryMediaFrameAckTask);

			// Decrement RunningTasksCount when the task exits
			ON_SCOPE_EXIT
			{
				RunningTasksCount--;
			};

			do
			{
				const bool bFenceReached = !FrameAckFences[SharedMemoryIdx] || FrameAckFences[SharedMemoryIdx]->Poll();

				if (bFenceReached)
				{
					break;
				}

				FPlatformProcess::SleepNoStats(SpinWaitTimeSeconds);

			} while (true);

			// We clear the fence and signal that it can be re-used.
			FrameAckFences[SharedMemoryIdx]->Clear();
			bFrameAckFenceBusy[SharedMemoryIdx] = false;

			// Ack the frame to the Media Capture, indicating that the cross gpu texture can be re-used.
			SharedMemoryData->Receivers[ReceiverIdx].FrameNumberAcked = ExpectedFrameNumber;
		}
	);
}

uint32 FSharedMemoryMediaPlayer::InputTextureFrameNumberForFrameNumber(uint32 FrameNumber) const
{
	int32 Latency = bZeroLatency ? 0 : 1;

	if (GDisplayClusterSharedMemoryLatency == 0)
	{
		Latency = 0;
	}
	else if (GDisplayClusterSharedMemoryLatency == 1)
	{
		Latency = 1;
	}

	return FrameNumber - Latency;
}


#undef LOCTEXT_NAMESPACE
