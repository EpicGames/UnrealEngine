// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GenericPlatform/GenericPlatformMemory.h"
#include "MediaIOCorePlayerBase.h"
#include "MediaTexture.h"
#include "SharedMemoryMediaCapture.h"
#include "SharedMemoryMediaSource.h"
#include "SharedMemoryMediaTypes.h"
#include "UObject/StrongObjectPtr.h"


class UTexture2D;
class FSharedMemoryMediaPlatform;

/**
 * Implements a media player using shared cross gpu textures. See USharedMemoryMediaCapture docstring for complementary information.
 * It will open shared system memory based on its UniqueName setting, which much match that of the corresponding Media Output.
 * This data will inform the Guid of the shared cross gpu textures and other metadata, and it will use this to open them
 * and to detect when the textures are still valid.
 * 
 * It will look at the frame numbers in the metadata when deciding if the data that it is looking for is ready.
 * 
 * It informs the sender that it is actively reading the data using a keep alive shift register that it must continually refresh.
 * Also, it acknowledges that it is done with a given gpu texture by setting the frame number in said shared system memory.
 * 
 * Even if the sender closes the textures, the open handles in the player will keep them valid and avoid undue crashes.
 */
class FSharedMemoryMediaPlayer 
	: public IMediaPlayer
	, protected IMediaCache
	, protected IMediaControls
	, protected IMediaTracks
	, protected IMediaView
	, public TSharedFromThis<FSharedMemoryMediaPlayer>
{
	using Super = IMediaPlayer;

	// @todo remove these friends and create needed getters/setters
	friend class FSharedMemoryMediaSample;
	friend class FSharedMemoryMediaTextureSampleConverter;

public:

	FSharedMemoryMediaPlayer();

	virtual ~FSharedMemoryMediaPlayer();

public:

	//~ Begin IMediaPlayer interface
	virtual void Close() override;
	
	virtual FGuid GetPlayerPluginGUID() const override;

	virtual bool Open(const FString& Url, const IMediaOptions* Options) override;
	virtual bool Open(const TSharedRef<FArchive, ESPMode::ThreadSafe>& Archive, const FString& OriginalUrl, const IMediaOptions* Options) override;

	virtual void TickFetch(FTimespan DeltaTime, FTimespan Timecode) override;
	virtual void TickInput(FTimespan DeltaTime, FTimespan Timecode) override;
	virtual bool GetPlayerFeatureFlag(EFeatureFlag Flag) const;

	virtual FString GetStats() const override;

	virtual IMediaSamples& GetSamples() override;

	virtual IMediaCache& GetCache() override;
	virtual IMediaControls& GetControls() override;
	virtual FString GetInfo() const override;
	virtual IMediaTracks& GetTracks() override;
	virtual FString GetUrl() const override;
	virtual IMediaView& GetView() override;

	//~ End IMediaPlayer interface

public:
	//~ IMediaCache interface

	virtual bool QueryCacheState(EMediaCacheState State, TRangeSet<FTimespan>& OutTimeRanges) const override 
	{ 
		return true; 
	}

	virtual int32 GetSampleCount(EMediaCacheState State) const override 
	{
		return 1;
	}

protected:

	//~ IMediaControls interface

	virtual bool CanControl(EMediaControl Control) const override
	{
		return false;
	}

	virtual FTimespan GetDuration() const override
	{
		return FTimespan::MaxValue();
	}

	virtual float GetRate() const override
	{
		return 1.0;
	}

	virtual EMediaState GetState() const override
	{
		return PlayerState;
	}

	virtual EMediaStatus GetStatus() const override
	{
		return EMediaStatus::None;
	}

	virtual TRangeSet<float> GetSupportedRates(EMediaRateThinning Thinning) const override
	{
		TRangeSet<float> Result;
		Result.Add(TRange<float>(1.0f));
		return Result;
	}

	virtual FTimespan GetTime() const override;

	virtual bool IsLooping() const override
	{
		return false;
	}

	virtual bool Seek(const FTimespan& Time) override
	{
		return false;
	}

	virtual bool SetLooping(bool Looping) override
	{
		return false;
	}

	virtual bool SetRate(float Rate) override
	{
		return FMath::IsNearlyEqual(Rate, 1.0f);
	}

protected:

	//~ IMediaTracks interface

	virtual bool GetAudioTrackFormat(int32 TrackIndex, int32 FormatIndex, FMediaAudioTrackFormat& OutFormat) const override
	{
		return false;
	}

	virtual int32 GetNumTracks(EMediaTrackType TrackType) const override
	{
		return 0;
	}

	virtual int32 GetNumTrackFormats(EMediaTrackType TrackType, int32 TrackIndex) const override
	{
		return 0;
	}

	virtual int32 GetSelectedTrack(EMediaTrackType TrackType) const override
	{
		return INDEX_NONE;
	}

	virtual FText GetTrackDisplayName(EMediaTrackType TrackType, int32 TrackIndex) const override
	{
		return FText::GetEmpty();
	}

	virtual int32 GetTrackFormat(EMediaTrackType TrackType, int32 TrackIndex) const override
	{
		return INDEX_NONE;
	}

	virtual FString GetTrackLanguage(EMediaTrackType TrackType, int32 TrackIndex) const override
	{
		return FString();
	}

	virtual FString GetTrackName(EMediaTrackType TrackType, int32 TrackIndex) const override
	{
		return FString();
	}

	virtual bool GetVideoTrackFormat(int32 TrackIndex, int32 FormatIndex, FMediaVideoTrackFormat& OutFormat) const override
	{
		return false;
	}

	virtual bool SelectTrack(EMediaTrackType TrackType, int32 TrackIndex) override
	{
		return false;
	}

	virtual bool SetTrackFormat(EMediaTrackType TrackType, int32 TrackIndex, int32 FormatIndex) override
	{
		return false;
	}
	
protected:

	/** Enqueues a last minute wait and copy of the expected texture data for the renderer to use */
	virtual void JustInTimeSampleRender(FRHICommandListImmediate& RHICmdList);

	/** Offsets the frame number by the specified latency. Used to associate resources with frame numbers being evaluated */
	uint32 InputTextureFrameNumberForFrameNumber(uint32 FrameNumber) const;

	/** 
	 * Determine that the next source frame number and buffer index is. This depends on the player settings.
	 * 
	 * @param OutExpectedFrameNumber Will contain the expected next source frame number
	 * @param OutSharedMemoryIdx Will contain the memory idx where we should be looking for the frame
	 * 
	 * @return true only if the output values are valid.
	 */
	bool DetermineNextSourceFrame(uint64 FrameNumber, uint32& OutExpectedFrameNumber, uint32& OutSharedMemoryIdx);

	/** DetermineNextSourceFrame implementration for Framelock Mode */
	bool DetermineNextSourceFrameFramelockMode(uint64 FrameNumber, uint32& OutExpectedFrameNumber, uint32& OutSharedMemoryIdx);

	/** DetermineNextSourceFrame implementration for Genlock Mode */
	bool DetermineNextSourceFrameGenlockMode(uint64 FrameNumber, uint32& OutExpectedFrameNumber, uint32& OutSharedMemoryIdx);

	/** DetermineNextSourceFrame implementration for Freerun Mode */
	bool DetermineNextSourceFrameFreerunMode(uint64 FrameNumber, uint32& OutExpectedFrameNumber, uint32& OutSharedMemoryIdx);


protected:

	/** Number of resources used for pipelining the data flow */
	static constexpr int32 NUMSHAREDMEM = UE::SharedMemoryMedia::SenderNumBuffers;

	/** Platform specific data or resources */
	TSharedPtr<FSharedMemoryMediaPlatform, ESPMode::ThreadSafe> PlatformData;

	/** Set of cross Gpu textures that the Media Capture allocates and the player opens */
	FTextureRHIRef SharedCrossGpuTextures[NUMSHAREDMEM];

	/** Receiver Index used to know what part of the shared metadata corresponds to this receiver */
	int32 ReceiverIndex[NUMSHAREDMEM] = { 0 };

	/** Unique number that identifies this receiver from any other */
	FGuid ReceiverId = FGuid::NewGuid();

	/** Shared memory allocated by the Media Capture for IPC purposes with the Media Player */
	FPlatformMemory::FSharedMemoryRegion* SharedMemory[NUMSHAREDMEM] = { 0 };

	/** Indicates that we're done reading the SharedCrossGpuTexture and we can ack the frame to the Media Capture for re-use */
	FGPUFenceRHIRef FrameAckFences[NUMSHAREDMEM];

	/** Needs to be cleared before the corresponding FrameAckFence is reused. */
	std::atomic<bool> bFrameAckFenceBusy[NUMSHAREDMEM] = { 0 };

	/** For all media samples to re-use */
	TStrongObjectPtr<UTexture2D> SampleCommonTexture; // @todo we sure we don't need 2 to avoid tearing

	/** Used to ensure that JustInTimeSampleRender is only run once per frame */
	uint32 LastFrameNumberThatUpdatedJustInTime = 0;

	/** The Media Samples used by this player */
	class FSharedMemoryMediaSamples* Samples = nullptr;

	/** Cached Url currently playing. Used for queries from the outside. */
	FString OpenUrl;

	/** Description of SharedCrossGpuTextures */
	FSharedMemoryMediaTextureDescription SharedCrossGpuTextureDescriptions[NUMSHAREDMEM];

	/** Unique Name that must match the corresponding MediaOutput setting. It is used to find allocated shared memory by name */
	FString UniqueName;

	/** Reception mode. See ESharedMemoryMediaSourceMode for more details. */
	ESharedMemoryMediaSourceMode Mode = ESharedMemoryMediaSourceMode::Framelocked;

	/** Zero latency option to wait for the cross gpu texture rendered on the same frame. May adversely affect fps. Only applicable when bUseFrameNumbers is true */
	bool bZeroLatency = true;

	/** Counter of running tasks used to detect when to release resources */
	std::atomic<int32> RunningTasksCount{ 0 };

	/** State of each Mode */
	struct FModeState
	{
		/** Freerun mode state */
		struct FFreerunModeState
		{
			/** Keep track of the last source frame that was picked. Used to avoid picking the same frame and detect unexpected source frame changes. */
			uint32 LastSourceFrameNumberPicked = 0;

			/** Keep track of the last frame considered at each buffer index. Used to manage acks. */
			uint32 LastSourceFrameNumberConsideredAtIdx[NUMSHAREDMEM]{ 0 };

			/** Reset the state, as if the stream were started again. */
			void Reset()
			{
				LastSourceFrameNumberPicked = 0;

				FMemory::Memset(LastSourceFrameNumberConsideredAtIdx, 0, sizeof(LastSourceFrameNumberConsideredAtIdx));
			}
		} Freerun;

		/** Genlock mode state */
		struct FGenlockModeState
		{
			/** Keep track of last picked frame. Used to aim for consecutiveness. */
			uint32 LastSourceFrameNumberPicked = 0;

			/** Reset the state, as if the stream were started again. */
			void Reset()
			{
				LastSourceFrameNumberPicked = 0;
			}
		} Genlock;

		/** Resets the state for all modes */
		void Reset()
		{
			Freerun.Reset();
			Genlock.Reset();
		}

	} ModeState;

	/** Current state of the player */
	EMediaState PlayerState = EMediaState::Stopped;

};
