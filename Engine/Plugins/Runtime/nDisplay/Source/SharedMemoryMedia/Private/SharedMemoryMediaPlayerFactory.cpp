// Copyright Epic Games, Inc. All Rights Reserved.

#include "SharedMemoryMediaPlayerFactory.h"

#include "SharedMemoryMediaPlayer.h"


#define LOCTEXT_NAMESPACE "SharedMemoryMediaFactory"


FSharedMemoryMediaPlayerFactory::FSharedMemoryMediaPlayerFactory()
{
	// supported platforms
	SupportedPlatforms.Add(TEXT("Windows"));

	// supported schemes
	SupportedUriSchemes.Add(TEXT("dcsm"));
}

bool FSharedMemoryMediaPlayerFactory::CanPlayUrl(const FString& Url, const IMediaOptions* Options, TArray<FText>* OutWarnings, TArray<FText>* OutErrors) const
{
	return GetPlayabilityConfidenceScore(Url, Options, OutWarnings, OutErrors) > 0 ? true : false;
}

int32 FSharedMemoryMediaPlayerFactory::GetPlayabilityConfidenceScore(const FString& Url, const IMediaOptions* Options, TArray<FText>* OutWarnings, TArray<FText>* OutErrors) const
{
	FString Scheme;
	FString Location;

	// extract scheme
	if (!Url.Split(TEXT("://"), &Scheme, &Location, ESearchCase::CaseSensitive))
	{
		if (OutErrors != nullptr)
		{
			OutErrors->Add(LOCTEXT("NoSchemeFound", "No URI scheme found"));
		}

		return 0;
	}

	// see if the scheme is supported
	if (!SupportedUriSchemes.Contains(Scheme))
	{
		if (OutErrors != nullptr)
		{
			OutErrors->Add(FText::Format(LOCTEXT("SchemeNotSupported", "The URI scheme '{0}' is not supported"), FText::FromString(Scheme)));
		}

		return 0;
	}

	return 100;
}

TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> FSharedMemoryMediaPlayerFactory::CreatePlayer(IMediaEventSink& EventSink)
{
	return MakeShared<FSharedMemoryMediaPlayer>();
}

FText FSharedMemoryMediaPlayerFactory::GetDisplayName() const
{
	return LOCTEXT("MediaPlayerDisplayName", "Shared Memory Device Interface");
}

FName FSharedMemoryMediaPlayerFactory::GetPlayerName() const
{
	static FName PlayerName(TEXT("SharedMemoryMedia"));
	return PlayerName;
}

FGuid FSharedMemoryMediaPlayerFactory::GetPlayerPluginGUID() const
{
	return UE::SharedMemoryMedia::PlayerGuid;
}

const TArray<FString>& FSharedMemoryMediaPlayerFactory::GetSupportedPlatforms() const
{
	return SupportedPlatforms;
}

bool FSharedMemoryMediaPlayerFactory::SupportsFeature(EMediaFeature Feature) const
{
	return Feature == EMediaFeature::VideoSamples;
}

#undef LOCTEXT_NAMESPACE
