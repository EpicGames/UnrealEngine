// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "IMediaPlayerFactory.h"


/**
 *  Factory for the shared memory media player.
 */
class FSharedMemoryMediaPlayerFactory : public IMediaPlayerFactory
{

private:
	/** List of platforms that the media player support. */
	TArray<FString> SupportedPlatforms;

	/** List of supported URI schemes. */
	TArray<FString> SupportedUriSchemes;

public:

	FSharedMemoryMediaPlayerFactory();

	//~ Begin IMediaPlayerFactory interface

	virtual bool CanPlayUrl(const FString& Url, const IMediaOptions* Options, TArray<FText>* OutWarnings, TArray<FText>* OutErrors) const override;
	virtual int32 GetPlayabilityConfidenceScore(const FString& Url, const IMediaOptions* Options, TArray<FText>* OutWarnings, TArray<FText>* OutErrors) const override;
	virtual TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CreatePlayer(IMediaEventSink& EventSink) override;
	virtual FText GetDisplayName() const override;
	virtual FName GetPlayerName() const override;
	virtual FGuid GetPlayerPluginGUID() const override;
	virtual const TArray<FString>& GetSupportedPlatforms() const override;
	virtual bool SupportsFeature(EMediaFeature Feature) const override;

	//~ End IMediaPlayerFactory interface

};

