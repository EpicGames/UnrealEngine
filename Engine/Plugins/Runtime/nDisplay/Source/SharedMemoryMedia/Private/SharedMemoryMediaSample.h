// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "IMediaTextureSample.h"
#include "SharedMemoryMediaModule.h"
#include "SharedMemoryMediaTextureSampleConverter.h"

class FSharedMemoryMediaPlayer;

/** This is the mostly empty sample that the player generates each frame. 
 *  Its job is mainly to provide the sample converter that will call back the player
 *  when rendering the sample, and fill in the data just in time.
 */
class FSharedMemoryMediaSample : public IMediaTextureSample
{

public:

	/** The player that created this sample */
	TWeakPtr<FSharedMemoryMediaPlayer, ESPMode::ThreadSafe> Player;

	/** Custom converter that will be the one checking back with the player for just in time sample render purposes */
	TSharedPtr<FSharedMemoryMediaTextureSampleConverter, ESPMode::ThreadSafe> Converter;

	/** The texture that will get its pixels just in time when playing back this sample */
	FTextureRHIRef Texture;

	/** Player time for this sample to be evaluated at */
	FTimespan Time;

	/** Texture stride */
	uint32 Stride = 0;

	/** Texture dimensions */
	FIntPoint Dim = FIntPoint::ZeroValue;

	/** Whether the sample is sRGB encoded or not */
	bool bSrgb = false;

	/** Sample format */
	EMediaTextureSampleFormat Format = EMediaTextureSampleFormat::CharBGR10A2;

private:

	/** Used in case the Texture is not valid, so we can provide pixels to the Media Framework */
	TArray<uint8> BackupData;

public:

	//~ Begin IMediaTextureSample interface

	const void* GetBuffer() override
	{
		// This will get called if there wasn't a gpu texture available. 
		// If we return null, FMediaTextureResource::ConvertSample will crash, so we log the error but return something valid.

		UE_LOGF(LogSharedMemoryMedia, Warning, 
			"FSharedMemoryMediaSample::GetBuffer was called, so it must not have had a valid texture, which is unexpected");

		const int32 NumBytes = GetStride() * GetDim().Y;

		if (BackupData.Num() < NumBytes)
		{
			BackupData.AddDefaulted(NumBytes - BackupData.Num());
		}

		return BackupData.GetData();
	}

	FIntPoint GetDim() const override
	{
		return Dim;
	}

	FTimespan GetDuration() const override
	{
		// Ticks are "frames" in our player.
		return FTimespan(1);
	}

	EMediaTextureSampleFormat GetFormat() const override
	{
		return Format;
	}

	FIntPoint GetOutputDim() const override
	{
		return GetDim();
	}

	uint32 GetStride() const override
	{
		return Stride;
	}

	FMediaTimeStamp GetTime() const override
	{
		return FMediaTimeStamp(Time);
	}

	bool IsCacheable() const override
	{
		return false;
	}

	bool IsOutputSrgb() const override
	{
		return bSrgb;
	}

	virtual bool ShouldApplyColorConversion() const override
	{
		return false;
	}

public:

#if WITH_ENGINE

	virtual IMediaTextureSampleConverter* GetMediaTextureSampleConverter() override
	{
		if (!Converter.IsValid())
		{
			Converter = MakeShared<FSharedMemoryMediaTextureSampleConverter>();
		}

		check(Converter.IsValid());

		Converter->Player = Player.Pin().Get();

		return Converter.Get();
	}

#endif // WITH_ENGINE

	virtual FRHITexture* GetTexture() const override
	{
		return Texture;
	}

	//~ End IMediaTextureSample interface
};
