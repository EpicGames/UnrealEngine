// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IMediaSamples.h"
#include "SharedMemoryMediaSample.h"

/** This is the mostly empty media samples that only carries a single sample per frame,
 *  that the player provides.
 */
class FSharedMemoryMediaSamples : public IMediaSamples
{
public:

	FSharedMemoryMediaSamples()
	{

	}

	FSharedMemoryMediaSamples(const FSharedMemoryMediaSamples&) = delete;
	FSharedMemoryMediaSamples& operator=(const FSharedMemoryMediaSamples&) = delete;

public:

	//~ Begin IMediaSamples interface

	virtual bool FetchVideo(TRange<FTimespan> TimeRange, TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>& OutSample) override
	{
		if (!CurrentSample.IsValid())
		{
			return false;
		}

		OutSample = CurrentSample;

		CurrentSample.Reset();

		return true;
	}

	virtual bool PeekVideoSampleTime(FMediaTimeStamp& TimeStamp) override
	{
		return false;
	}

	virtual void FlushSamples() override
	{
		CurrentSample.Reset();
	}

	//~ End IMediaSamples interface

public:

	TSharedPtr<FSharedMemoryMediaSample, ESPMode::ThreadSafe> CurrentSample;
};

