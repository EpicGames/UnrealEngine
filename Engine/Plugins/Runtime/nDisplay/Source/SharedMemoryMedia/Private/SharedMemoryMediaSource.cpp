// Copyright Epic Games, Inc. All Rights Reserved.

#include "SharedMemoryMediaSource.h"

#include "MediaIOCorePlayerBase.h"
#include "SharedMemoryMediaSourceOptions.h"


bool USharedMemoryMediaSource::GetMediaOption(const FName& Key, bool DefaultValue) const
{
	if (Key == SharedMemoryMediaOption::ZeroLatency)
	{
		return bZeroLatency;
	}

	return Super::GetMediaOption(Key, DefaultValue);
}

int64 USharedMemoryMediaSource::GetMediaOption(const FName& Key, int64 DefaultValue) const
{
	if (Key == SharedMemoryMediaOption::Mode)
	{
		return int64(Mode);
	}

	return Super::GetMediaOption(Key, DefaultValue);
}

FString USharedMemoryMediaSource::GetMediaOption(const FName& Key, const FString& DefaultValue) const
{
	if (Key == SharedMemoryMediaOption::UniqueName)
	{
		return UniqueName;
	}

	return Super::GetMediaOption(Key, DefaultValue);
}

bool USharedMemoryMediaSource::HasMediaOption(const FName& Key) const
{
	if (Key == SharedMemoryMediaOption::UniqueName)
	{
		return true;
	}

	if (Key == SharedMemoryMediaOption::ZeroLatency)
	{
		return true;
	}

	if (Key == SharedMemoryMediaOption::Mode)
	{
		return true;
	}

	return Super::HasMediaOption(Key);
}

FString USharedMemoryMediaSource::GetUrl() const
{
	return FString::Printf(TEXT("dcsm://%s"), *UniqueName);
}

bool USharedMemoryMediaSource::Validate() const
{
	return true;
}
