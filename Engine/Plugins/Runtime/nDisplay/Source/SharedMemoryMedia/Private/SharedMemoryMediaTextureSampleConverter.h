// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMediaTextureSampleConverter.h"
#include "SharedMemoryMediaPlayer.h"

class FSharedMemoryMediaPlayer;

class FSharedMemoryMediaTextureSampleConverter : public IMediaTextureSampleConverter
{
public:

	/** The player that this converter will check back with for just in time sample render purposes */
	FSharedMemoryMediaPlayer* Player = nullptr;

public:

	//~ Begin IMediaTextureSampleConverter interface
	virtual uint32 GetConverterInfoFlags() const override
	{
		return ConverterInfoFlags_PreprocessOnly;
	}

	virtual bool Convert(FRHICommandListImmediate& RHICmdList, FTextureRHIRef& InDstTexture, const FConversionHints& Hints) override
	{
		if (!Player)
		{
			return false;
		}

		Player->JustInTimeSampleRender(RHICmdList);

		return true;
	}
	//~ End IMediaTextureSampleConverter interface
};
