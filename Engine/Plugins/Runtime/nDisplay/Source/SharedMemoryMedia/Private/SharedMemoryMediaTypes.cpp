// Copyright Epic Games, Inc. All Rights Reserved.

#include "SharedMemoryMediaTypes.h"


void FSharedMemoryMediaFrameMetadata::Initialize()
{
	for (int32 RxIdx = 0; RxIdx < MaxNumReceivers; ++RxIdx)
	{
		Receivers[RxIdx].Disconnect();
	}
}


bool FSharedMemoryMediaFrameMetadata::IsSlotIndexValid(const int32 SlotIndex) const
{
	if (SlotIndex == INDEX_NONE)
	{
		return false;
	}

	check(SlotIndex >= 0 && SlotIndex < MaxNumReceivers);

	return true;
}


bool FSharedMemoryMediaFrameMetadata::IsReceiverConnectedToSlot(const FGuid& Id, const int32 SlotIndex) const
{
	if (!IsSlotIndexValid(SlotIndex))
	{
		return false;
	}

	check(SlotIndex >= 0 && SlotIndex < MaxNumReceivers);

	// KeepAliveCounter must not be zero and the id should be the owner of the slot
	return (Receivers[SlotIndex].KeepAliveCounter && Receivers[SlotIndex].Id == Id);
}


int32 FSharedMemoryMediaFrameMetadata::ConnectToSlot(const FGuid& Id)
{
	for (uint32 RxIdx = 0; RxIdx < MaxNumReceivers; ++RxIdx)
	{
		uint8 ExpectedValue = 0;

		const bool bReservedSlot = Receivers[RxIdx].KeepAliveCounter.compare_exchange_weak(ExpectedValue, KeepAliveCounterResetValue);

		if (bReservedSlot)
		{
			Receivers[RxIdx].Id = Id;
			return RxIdx;
		}
	}

	return INDEX_NONE;
}


void FSharedMemoryMediaFrameMetadata::DisconnectReceiverFromSlot(const FGuid& Id, const int32 SlotIndex)
{
	// Check if it is already disconnected
	if (!IsReceiverConnectedToSlot(Id, SlotIndex))
	{
		return;
	}

	Receivers[SlotIndex].Disconnect();
}

void FSharedMemoryMediaFrameMetadata::FReceiver::Disconnect()
{
	// Zero out the keep alive to effectively disconnect.
	// 
	// Note that there is a potential race condition, where after the connection verification
	// above the KeepAlive got timed out and a new receiver with a different id connected,
	// causing its premature disconnection.
	KeepAliveCounter = 0;

	// This avoids newly connected receivers from immediately stalling the sender.
	FrameNumberAcked = ~0;
}

bool FSharedMemoryMediaFrameMetadata::KeepAlive(const int32 SlotIndex)
{
	if (!IsSlotIndexValid(SlotIndex))
	{
		return false;
	}

	// Only set the keepalive if it is not already zero. Because if it is already zero,
	// it means that we got disconnected / timed out and need to find a slot again.

	uint8 CurrentValue = Receivers[SlotIndex].KeepAliveCounter.load();

	while (CurrentValue != 0) // loop until CurrentValue is zero or the exchange succeeds
	{
		// try to exchange KeepAliveCounter with KeepAliveCounterResetValue
		if (Receivers[SlotIndex].KeepAliveCounter.compare_exchange_weak(CurrentValue, KeepAliveCounterResetValue))
		{
			// exchange succeeded
			return true;
		}

		// Exchange failed, CurrentValue was updated with the actual value of KeepAliveCounter
	}

	// CurrentValue was zero, so we can't keep it alive anymore (it's dead).
	return false;
}


bool FSharedMemoryMediaFrameMetadata::AllReceiversAckedFrameNumber(const uint32 FrameNumber) const
{
	for (int32 RxIdx = 0; RxIdx < MaxNumReceivers; ++RxIdx)
	{
		if (Receivers[RxIdx].KeepAliveCounter                     // Indicates an active receiver
			&& (Receivers[RxIdx].FrameNumberAcked < FrameNumber)) // Indicates that the receiver hasn't released this frame number
		{
			return false;
		}
	}

	return true;
}


void FSharedMemoryMediaFrameMetadata::DecrementKeepAlives()
{
	// Decrement the keep alive. The receiver must keep resetting it to avoid letting it expire.
	for (int32 RxIdx = 0; RxIdx < MaxNumReceivers; ++RxIdx)
	{
		// We want to decrement the KeepAliveCounter, but making sure not to decrement it below zero, which would wrap around.
		// So we first read its value, and if it is not zero then we try to decrement it. However if the value has changed when we try to 
		// atomically write the decremented value, then we update the current value and try again. This is what the while loop does, and 
		// how the compare_exchange_weak function behaves:
		//
		// compare_exchange_weak atomically compares the value of *this with that of expected. If those are bitwise-equal, replaces the former
		// with desired (performs read-modify-write operation). Otherwise, loads the actual value stored in *this into expected (performs load operation).

		uint8 CurrentValue = Receivers[RxIdx].KeepAliveCounter.load();

		while (
			CurrentValue // Current value must not be zero
			&& !Receivers[RxIdx].KeepAliveCounter.compare_exchange_weak(CurrentValue, CurrentValue - 1) // Replace current value with its minus 1 value.
			)
		{
		}
	}
}
