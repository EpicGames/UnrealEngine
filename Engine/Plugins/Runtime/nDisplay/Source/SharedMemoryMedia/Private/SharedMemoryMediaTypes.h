// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Internationalization/TextKey.h"
#include "PixelFormat.h"

namespace UE::SharedMemoryMedia
{
	// When the implementation requires spin waiting on a fence or polling shared system memory,
	// it will sleep by the amount below in a loop until the condition is met or a given timout occurs.
	constexpr float SpinWaitTimeSeconds = 50 * 1e-6f;

	// Guid that identifies the SharedMemoryMediaPlayer.
	static const FGuid PlayerGuid = FGuid(0xAF8C5107, 0x13CF992C, 0x9FD4EBFE, 0x2E3E049F);

	/**
	 * Used to generate a Guid based on the UniqueName that sender and receiver uses to open the same
	 * shared system memory used for IPC. There will be one shard memory region per buffer index.
	 * There are SenderNumBuffers buffer indices.
	 * A given frame number uses a buffer index defined by:
	 *   BufferIdx = FrameNumber % SenderNumBuffers.
	 */
	static FGuid GenerateSharedMemoryGuid(FString UniqueName, uint32 BufferIdx)
	{
		// Avoid the hard to spot user error of differing spaces in strings.
		UniqueName.RemoveSpacesInline();

		const uint32 UniqueNameHash = TextKeyUtil::HashString(UniqueName);
		return FGuid(0xDA90FCAF, 0x7034A2AA, UniqueNameHash, 0xEB12036B + BufferIdx);
	};

	// Convenience ZeroGuid value.
	static const FGuid ZeroGuid = FGuid(0, 0, 0, 0);
}

struct FSharedMemoryMediaTextureDescription
{
	uint32 Width = 0;
	uint32 Height = 0;
	uint32 BytesPerPixel = 0;
	uint32 Stride = 0;
	EPixelFormat Format = EPixelFormat::PF_Unknown;
	FGuid Guid = UE::SharedMemoryMedia::ZeroGuid;
	bool bSrgb = false;

	bool IsEquivalentTo(const FSharedMemoryMediaTextureDescription& Other) const
	{
		return Width == Other.Width
			&& Height == Other.Height
			&& BytesPerPixel == Other.BytesPerPixel
			&& Stride == Other.Stride
			&& Format == Other.Format
			&& bSrgb == Other.bSrgb
			;
	}
};

/**
 * This structure represents the IPC between sender and receiver of the shared memory media.
 */
struct FSharedMemoryMediaFrameMetadata
{
	// SMMM - Shared Memory Media Metadata
	static constexpr uint32 MAGIC = 0x534D4D4D; 

	// Value that receivers should use when resetting the KeepAliveCounters. Corresponds to number of sender frames.
	static constexpr uint8 KeepAliveCounterResetValue = 12; 

	// Maximum number of concurrent receivers of the shared texture
	static constexpr uint32 MaxNumReceivers = 4;

	// Data that the sender wants to communicate
	struct FSender
	{
		// Used to determine that the rest of the metadata is valid.
		uint32 Magic = MAGIC;

		// Each shared cross gpu texture is assigned a Guid. The player can use this to locate and open them.
		FGuid TextureGuid;

		// This is the sender's frame number of the pixel data in the texture.
		uint32 FrameNumber = 0;
	} Sender;

	// Data that the receiver wants to communicate (although KeepAliveShiftRegister is shifted by the Sender)
	struct FReceiver
	{
		// Frame number that the receiver is done reading. This signals the sender that it can re-use the texture.
		std::atomic<uint32> FrameNumberAcked = ~0u;

		/** 
		 * The receiver sets this register. This signals the sender that it should wait for the related frame number ack before re-using the texture.
		 * The sender will decrement this register each frame, as a way to time out the session in case the receiver disappears without clearing it.
		 */
		std::atomic<uint8> KeepAliveCounter = 0;

		/** Receiver id, used to detect collisions where two receivers think that they own the same slot */
		FGuid Id;

		// Disconnect from this slot. Meant to be called by the receiver.
		void Disconnect();
	};

	// Array of possible receivers
	FReceiver Receivers[MaxNumReceivers] = {};

public:

	// Initialize critical values
	void Initialize();

	// Returns true is the SlotId is valid
	bool IsSlotIndexValid(const int32 SlotIndex) const;

	// Checks if the receiver of the given Id owns the given slot
	bool IsReceiverConnectedToSlot(const FGuid& Id, const int32 SlotIndex) const;

	// Finds a slot for the give receiver. Returns the slot index, or INDEX_NONE if it could not find one.
	int32 ConnectToSlot(const FGuid& Id);

	// Give up control of the slot
	void DisconnectReceiverFromSlot(const FGuid& Id, const int32 SlotIndex);

	// Keeps the connection to the slot alive. True if succeeded.
	bool KeepAlive(const int32 SlotIndex);

	// Returns true only if all active receivers have acked the given frame number
	bool AllReceiversAckedFrameNumber(const uint32 FrameNumber) const;

	// Decrements all keepalives, which is intended to auto-disconnect stale receivers. 
	// The receiver must keep resetting its keepalive to avoid letting it expire.
	void DecrementKeepAlives();
};


