// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MediaCapture.h"

#include "GenericPlatform/GenericPlatformMemory.h"
#include "SharedMemoryMediaOutput.h"

#include "SharedMemoryMediaCapture.generated.h"

class FSharedMemoryMediaPlatform;

namespace UE::SharedMemoryMedia
{
	/**
	 * This defines the number of textures used for communication. Having more that one allows for overlapping
	 * sends and minimize waits on frame acks for texture re-use. Use 3 for best performance, or 2 for
	 * a potential compromise with resource usage.
	 */
	constexpr int32 SenderNumBuffers = 3;
}

/**
 * Output Media for SharedMemory.
 * 
 * The pixels are captured into shared cross gpu textures, that a player can read.
 * Inter-process communication happens via shared system memory. The metadata exchanged is documented 
 * in the FSharedMemoryMediaFrameMetadata structure. The shared memory can be located via a UniqueName
 * that must match in the Media Output and corresponding Media Source settings.
 * 
 * It is mostly intended for use in nDisplay render nodes, which are designed to be frame-locked.
 */
UCLASS(BlueprintType)
class SHAREDMEMORYMEDIA_API USharedMemoryMediaCapture : public UMediaCapture
{
	GENERATED_BODY()

	//~ Begin UMediaCapture interface

protected:

	virtual bool ShouldCaptureRHIResource() const override;
	virtual bool InitializeCapture() override;
	virtual void StopCaptureImpl(bool bAllowPendingFrameToBeProcess) override;
	virtual bool SupportsAnyThreadCapture() const override
	{
		return true;
	}

	/** For custom conversion, methods that need to be overridden */
	virtual FIntPoint GetCustomOutputSize(const FIntPoint& InSize) const override;
	virtual EMediaCaptureResourceType GetCustomOutputResourceType() const override;

	virtual void OnCustomCapture_RenderingThread(
		FRDGBuilder& GraphBuilder, 
		const FCaptureBaseData& InBaseData, 
		TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, 
		FRDGTextureRef InSourceTexture, 
		FRDGTextureRef OutputTexture, 
		const FRHICopyTextureInfo& CopyInfo, 
		FVector2D CropU, 
		FVector2D CropV) override;

	//~ End UMediaCapture interface

	UE_DEPRECATED(5.5, "Use the version of AddCopyToSharedGpuTexturePass with FRHICopyTextureInfo.")
	void AddCopyToSharedGpuTexturePass(FRDGBuilder& GraphBuilder, FRDGTextureRef InSourceTexture, uint32 SharedTextureIdx)
	{
		AddCopyToSharedGpuTexturePass(GraphBuilder, InSourceTexture, SharedTextureIdx, FRHICopyTextureInfo());
	}

	UE_DEPRECATED(5.5, "Use the version of AddInvertAlphaConversionPass with FRHICopyTextureInfo.")
	void AddInvertAlphaConversionPass(FRDGBuilder& GraphBuilder, const FRDGTextureRef& SourceTexture, FRDGTextureRef DestTexture)
	{
		AddInvertAlphaConversionPass(GraphBuilder, SourceTexture, DestTexture, FRHICopyTextureInfo());
	}

	/** Adds to the graph builder the copy from the input texture to the shared gpu texture that corresponds to the current frame number */
	void AddCopyToSharedGpuTexturePass(FRDGBuilder& GraphBuilder, FRDGTextureRef InSourceTexture, uint32 SharedTextureIdx, const FRHICopyTextureInfo& CopyInfo);

	/** Adds a pass to invert the alpha of the texture */
	void AddInvertAlphaConversionPass(FRDGBuilder& GraphBuilder, const FRDGTextureRef& SourceTexture, FRDGTextureRef DestTexture, const FRHICopyTextureInfo& CopyInfo);

private:

	/** Number of shared buffers */
	static constexpr int32 NUMBUFFERS = UE::SharedMemoryMedia::SenderNumBuffers;

	/** Shared RAM used for IPC between media capture and player */
	FPlatformMemory::FSharedMemoryRegion* SharedMemory[NUMBUFFERS] = { 0 };

	/** Shared Cross Gpu Textures. The texture exchange happens using these. */
	FTextureRHIRef SharedCrossGpuTextures[NUMBUFFERS];

	/** Fence for an async task to know when the cross gpu texture has the data to be shared */
	FGPUFenceRHIRef TextureReadyFences[NUMBUFFERS];
	
	/** Flags that the associated TextureReadyFence can be re-used. Otherwise the player may still be reading it. */
	std::atomic<bool> bTextureReadyFenceBusy[NUMBUFFERS] = { 0 };

	/** Platform specific data or resources */
	TSharedPtr<FSharedMemoryMediaPlatform, ESPMode::ThreadSafe> PlatformData;

	/** Guids associated with the cross gpu textures in use. These are communicated to the player for it to open them by name. */
	FGuid SharedCrossGpuTextureGuids[NUMBUFFERS];

	/** Counter of running tasks used to detect when to release resources */
	std::atomic<int32> RunningTasksCount{ 0 };
};
