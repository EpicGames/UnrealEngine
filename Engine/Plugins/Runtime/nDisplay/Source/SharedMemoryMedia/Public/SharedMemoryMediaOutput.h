// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MediaOutput.h"

#include "MediaIOCoreDefinitions.h"
#include "UObject/ObjectMacros.h"

#include "SharedMemoryMediaOutput.generated.h"


/**
 * Output information for a SharedMemory media capture.
 */
UCLASS(BlueprintType)
class SHAREDMEMORYMEDIA_API USharedMemoryMediaOutput : public UMediaOutput
{
	GENERATED_BODY()

public:

	/** Shared memory will be opened by using this name. Should be unique per media output. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Output")
	FString UniqueName = TEXT("UniqueName");

	/** If checked, the alpha channel of the texture will be inverted */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category = "Output")
	bool bInvertAlpha = true;

	/** If checked, the texture will be shared across different GPUs. Disable if not needed for faster performance */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category = "Output")
	bool bCrossGpu = true;

public:
	
	//~ Begin UMediaOutput interface
	virtual bool Validate(FString& FailureReason) const override;
	virtual FIntPoint GetRequestedSize() const override;
	virtual EPixelFormat GetRequestedPixelFormat() const override;
	virtual EMediaCaptureConversionOperation GetConversionOperation(EMediaCaptureSourceType InSourceType) const override;

protected:
	virtual UMediaCapture* CreateMediaCaptureImpl() override;
	//~ End UMediaOutput interface

};
