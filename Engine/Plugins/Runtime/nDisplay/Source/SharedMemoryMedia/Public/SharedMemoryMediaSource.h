// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MediaSource.h"

#include "SharedMemoryMediaSource.generated.h"

/** 
 * Mode of operation when receiving data.
 * Framelocked - Matches source and local frame numbers. Always use this mode in nDisplay.
 * Genlocked - It doesn't match frame numbers, but it also doesn't skip frames, so will hold back the sender if it is faster than the receiver.
 * Freerun - It always grabs the latest frame. It may skip frames if they arrive too fast.
 */
UENUM(BlueprintType)
enum class ESharedMemoryMediaSourceMode : uint8
{
	Framelocked = 0, // Matches source and local frame numbers. Always use this mode in nDisplay.
	Genlocked,       // It doesn't match frame numbers, but it also doesn't skip frames, so will hold back the sender if it is faster than the receiver.
	Freerun,         // It always grabs the latest frame. It may skip frames if they arrive too fast.
};

/**
 * Media source for SharedMemory streams.
 */
UCLASS(BlueprintType, hideCategories=(Platforms,Object))
class SHAREDMEMORYMEDIA_API USharedMemoryMediaSource : public UMediaSource
{
	GENERATED_BODY()

public:

	/** Shared memory will be found by using this name. Should match the media output setting. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Input")
	FString UniqueName = TEXT("UniqueName");

	/**
	 * Mode of operation when receiving data.
	 * Framelocked - Matches source and local frame numbers. Always use this mode in nDisplay.
	 * Genlocked - It doesn't match frame numbers, but it also doesn't skip frames, so will hold back the sender if it is faster than the receiver.
	 * Freerun - It always grabs the latest frame. It may skip frames if they arrive too fast.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Input")
	ESharedMemoryMediaSourceMode Mode;

	/** Zero latency option to wait for the cross gpu texture rendered on the same frame. May adversely affect fps. Only applicable in Framelocked mode. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Input")
	bool bZeroLatency = true;

public:

	//~ Begin UMediaSource interface
	virtual FString GetUrl() const override;
	virtual bool Validate() const override;
	//~ End UMediaSource interface

	//~ Begin IMediaOptions interface
	virtual bool GetMediaOption(const FName& Key, bool DefaultValue) const override;
	virtual int64 GetMediaOption(const FName& Key, int64 DefaultValue) const override;
	virtual FString GetMediaOption(const FName& Key, const FString& DefaultValue) const override;
	virtual bool HasMediaOption(const FName& Key) const override;
	//~ End IMediaOptions interface
};
