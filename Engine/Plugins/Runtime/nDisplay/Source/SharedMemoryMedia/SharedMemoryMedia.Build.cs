// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class SharedMemoryMedia : ModuleRules
	{
		public SharedMemoryMedia(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"MediaIOCore",
				});

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Engine",
					"Media",
					"MediaAssets",
					"RenderCore",
					"Renderer",
					"RHI",
				});

			if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
			{
				AddEngineThirdPartyPrivateStaticDependencies(Target, "DX12");
				PrivateDependencyModuleNames.Add("D3D12RHI");
			}

		}
	}
}
