// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AssetDefinition/AssetDefinition_MediaSource.h"
#include "SharedMemoryMediaSource.h"
#include "AssetDefinition_SharedMemoryMediaSource.generated.h"

UCLASS()
class UAssetDefinition_SharedMemoryMediaSource : public UAssetDefinition_MediaSource
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_SharedMemoryMediaSource", "Shared Memory Media Source"); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return USharedMemoryMediaSource::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Media, NSLOCTEXT("AssetDefinition", "AssetDefinition_SharedMemoryMediaSourceSubMenu", "Media Sources + Outputs"), ECategoryMenuType::Section) };
		return Categories;
	}
	// UAssetDefinition End
};
