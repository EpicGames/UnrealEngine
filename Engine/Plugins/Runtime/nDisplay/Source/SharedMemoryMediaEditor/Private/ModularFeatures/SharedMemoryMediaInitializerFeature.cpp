// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModularFeatures/SharedMemoryMediaInitializerFeature.h"

#include "SharedMemoryMediaOutput.h"
#include "SharedMemoryMediaSource.h"


bool FSharedMemoryMediaInitializerFeature::IsMediaObjectSupported(const UObject* MediaObject)
{
	if (MediaObject)
	{
		return MediaObject->IsA<USharedMemoryMediaSource>() || MediaObject->IsA<USharedMemoryMediaOutput>();
	}

	return false;
}

bool FSharedMemoryMediaInitializerFeature::AreMediaObjectsCompatible(const UObject* MediaSource, const UObject* MediaOutput)
{
	if (MediaSource && MediaOutput)
	{
		return MediaSource->IsA<USharedMemoryMediaSource>() && MediaOutput->IsA<USharedMemoryMediaOutput>();
	}

	return false;
}

bool FSharedMemoryMediaInitializerFeature::GetSupportedMediaPropagationTypes(const UObject* MediaSource, const UObject* MediaOutput, EMediaStreamPropagationType& OutPropagationTypes)
{
	if (!IsMediaObjectSupported(MediaSource) ||
		!IsMediaObjectSupported(MediaOutput) ||
		!AreMediaObjectsCompatible(MediaSource, MediaOutput))
	{
		return false;
	}

	OutPropagationTypes =
		EMediaStreamPropagationType::LocalUnicast |
		EMediaStreamPropagationType::LocalMulticast;

	return true;
}

static constexpr const TCHAR* GetMediaPrefix(const FMediaObjectOwnerInfo::EMediaObjectOwnerType OwnerType)
{
	switch (OwnerType)
	{
	case FMediaObjectOwnerInfo::EMediaObjectOwnerType::ICVFXCamera:
		return TEXT("icam");

	case FMediaObjectOwnerInfo::EMediaObjectOwnerType::Viewport:
		return TEXT("vp");

	case FMediaObjectOwnerInfo::EMediaObjectOwnerType::Backbuffer:
		return TEXT("node");

	default:
		return TEXT("unknown");
	}
}

void FSharedMemoryMediaInitializerFeature::InitializeMediaObjectForTile(UObject* MediaObject, const FMediaObjectOwnerInfo& OnwerInfo, const FIntPoint& TilePos)
{
	const FString UniqueName = FString::Printf(TEXT("%s@%s_tile_%d:%d"), GetMediaPrefix(OnwerInfo.OwnerType), *OnwerInfo.OwnerName, TilePos.X, TilePos.Y);

	if (USharedMemoryMediaSource* SMMediaSource = Cast<USharedMemoryMediaSource>(MediaObject))
	{
		SMMediaSource->UniqueName   = UniqueName;
		SMMediaSource->bZeroLatency = true;
		SMMediaSource->Mode         = ESharedMemoryMediaSourceMode::Framelocked;
	}
	else if (USharedMemoryMediaOutput* SMMediaOutput = Cast<USharedMemoryMediaOutput>(MediaObject))
	{
		SMMediaOutput->UniqueName   = UniqueName;
		SMMediaOutput->bInvertAlpha = true;
		SMMediaOutput->bCrossGpu    = true;
		SMMediaOutput->NumberOfTextureBuffers = 4;
	}
}

void FSharedMemoryMediaInitializerFeature::InitializeMediaObjectForFullFrame(UObject* MediaObject, const FMediaObjectOwnerInfo& OnwerInfo)
{
	const FString UniqueName = FString::Printf(TEXT("%s@%s"), GetMediaPrefix(OnwerInfo.OwnerType), *OnwerInfo.OwnerName);

	if (USharedMemoryMediaSource* SMMediaSource = Cast<USharedMemoryMediaSource>(MediaObject))
	{
		SMMediaSource->UniqueName   = UniqueName;
		SMMediaSource->bZeroLatency = true;
		SMMediaSource->Mode         = ESharedMemoryMediaSourceMode::Framelocked;
	}
	else if (USharedMemoryMediaOutput* SMMediaOutput = Cast<USharedMemoryMediaOutput>(MediaObject))
	{
		SMMediaOutput->UniqueName   = UniqueName;
		SMMediaOutput->bInvertAlpha = true;
		SMMediaOutput->bCrossGpu    = true;
		SMMediaOutput->NumberOfTextureBuffers = 4;
	}
}
