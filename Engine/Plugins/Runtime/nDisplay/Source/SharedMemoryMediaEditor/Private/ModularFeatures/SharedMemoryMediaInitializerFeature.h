// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDisplayClusterModularFeatureMediaInitializer.h"


/**
 * ShareMemory media source/output initializer for nDisplay
 */
class FSharedMemoryMediaInitializerFeature
	: public IDisplayClusterModularFeatureMediaInitializer
{
public:

	//~ Begin IDisplayClusterModularFeatureMediaInitializer
	virtual bool IsMediaObjectSupported(const UObject* MediaObject) override;
	virtual bool AreMediaObjectsCompatible(const UObject* MediaSource, const UObject* MediaOutput) override;
	virtual bool GetSupportedMediaPropagationTypes(const UObject* MediaSource, const UObject* MediaOutput, EMediaStreamPropagationType& OutPropagationTypes) override;
	virtual void InitializeMediaObjectForTile(UObject* MediaObject, const FMediaObjectOwnerInfo& OnwerInfo, const FIntPoint& TilePos) override;
	virtual void InitializeMediaObjectForFullFrame(UObject* MediaObject, const FMediaObjectOwnerInfo& OnwerInfo) override;
	//~ End IDisplayClusterModularFeatureMediaInitializer
};
