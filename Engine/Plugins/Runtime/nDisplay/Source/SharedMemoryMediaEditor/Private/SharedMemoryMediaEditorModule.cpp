// Copyright Epic Games, Inc. All Rights Reserved.

#include "SharedMemoryMediaEditorModule.h"

#include "Features/IModularFeatures.h"
#include "Modules/ModuleManager.h"

#include "ModularFeatures/SharedMemoryMediaInitializerFeature.h"


void FSharedMemoryMediaEditorModule::StartupModule()
{
	RegisterModularFeatures();
}

void FSharedMemoryMediaEditorModule::ShutdownModule()
{
	UnregisterModularFeatures();
}

void FSharedMemoryMediaEditorModule::RegisterModularFeatures()
{
	// Instantiate modular features
	MediaInitializer = MakeUnique<FSharedMemoryMediaInitializerFeature>();

	// Register modular features
	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	ModularFeatures.RegisterModularFeature(FSharedMemoryMediaInitializerFeature::ModularFeatureName, MediaInitializer.Get());
}

void FSharedMemoryMediaEditorModule::UnregisterModularFeatures()
{
	// Unregister modular features
	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	ModularFeatures.UnregisterModularFeature(FSharedMemoryMediaInitializerFeature::ModularFeatureName, MediaInitializer.Get());
}


IMPLEMENT_MODULE(FSharedMemoryMediaEditorModule, SharedMemoryMediaEditor)
