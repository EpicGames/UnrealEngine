// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Templates/UniquePtr.h"

class FSharedMemoryMediaInitializerFeature;


class FSharedMemoryMediaEditorModule
	: public IModuleInterface
{
public:
	
	//~ Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface

private:

	/** Registers modular features for external modules */
	void RegisterModularFeatures();

	/** Unregisters modular features */
	void UnregisterModularFeatures();

private:

	/** MediaInitializer modular feature instance */
	TUniquePtr<FSharedMemoryMediaInitializerFeature> MediaInitializer;
};
