// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class SharedMemoryMediaEditor : ModuleRules
	{
		public SharedMemoryMediaEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"AssetDefinition",
					"Core",
					"CoreUObject",
					"DisplayClusterModularFeaturesEditor",
					"Engine",
					"MediaPlayerEditor",
					"SharedMemoryMedia",
					"UnrealEd",
				});
		}
	}
}
