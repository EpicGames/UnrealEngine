{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "nDisplay",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"Description": "Support for synchronized clustered rendering using multiple PCs in mono or stereo",
	"Category": "Misc",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"Installed": false,
		"SupportedTargetPlatforms": [
			"Win64",
			"Linux"
		],
	"Modules": [
		{
			"Name": "DisplayCluster",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Linux"
			]
		},
		{
			"Name": "DisplayClusterFillDerivedDataCache",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Linux"
			]
		},
		{
			"Name": "DisplayClusterEditor",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Linux"
			]
		},
		{
			"Name": "DisplayClusterProjection",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Linux"
			]
		},
		{
			"Name": "DisplayClusterWarp",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Linux"
			]
		},
		{
			"Name": "DisplayClusterShaders",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit",
			"PlatformAllowList": [
				"Win64",
				"Linux"
			]
		},
		{
			"Name": "DisplayClusterMessageInterception",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64"
			]
		},
		{
			"Name": "DisplayClusterConfiguration",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit",
			"PlatformAllowList": [
				"Win64",
				"Linux"
			]
		},
		{
			"Name": "DisplayClusterOperator",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Linux"
			]
		},
		{
			"Name": "DisplayClusterConfigurator",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Linux"
			]
		},
		{
			"Name": "DisplayClusterLightCardEditor",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Linux"
			]
		},
		{
			"Name": "DisplayClusterLightCardEditorShaders",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit",
			"PlatformAllowList": [
				"Win64",
				"Linux"
			]
		},
		{
			"Name": "DisplayClusterScenePreview",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Linux"
			]
		},
		{
			"Name": "DisplayClusterColorGrading",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Linux"
			]
		},
		{
			"Name": "DisplayClusterDetails",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Linux"
			]
		},
		{
			"Name": "DisplayClusterStageMonitoring",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64"
			]
		},
		{
			"Name": "DisplayClusterRemoteControlInterceptor",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Linux"
			]
		},
		{
			"Name": "DisplayClusterMedia",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Linux"
			]
		},
		{
			"Name": "DisplayClusterMediaEditor",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Linux"
			]
		},
		{
			"Name": "DisplayClusterMonitor",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Linux"
			]
		},
		{
			"Name": "DisplayClusterMonitorEditor",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Linux"
			]
		},
		{
			"Name": "DisplayClusterMultiUser",
			"Type": "UncookedOnly",
			"LoadingPhase": "PostEngineInit",
			"PlatformAllowList": [
				"Win64"
			]
		},
		{
			"Name": "DisplayClusterTests",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Linux"
			]
		},
		{
			"Name": "DisplayClusterMoviePipeline",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Linux"
			]
		},
		{
			"Name": "DisplayClusterMoviePipelineEditor",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Linux"
			]
		},
		{
			"Name": "SharedMemoryMedia",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Linux"
			]
		},
		{
			"Name": "SharedMemoryMediaEditor",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Linux"
			]
		},
		{
			"Name": "DisplayClusterReplication",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Linux"
			]
		}
	],
	"Plugins": [
		{
			"Name": "ColorCorrectRegions",
			"Enabled": true
		},
		{
			"Name": "ColorGrading",
			"Enabled": true
		},
		{
			"Name": "ConcertSyncClient",
			"Enabled": true
		},
		{
			"Name": "EngineAssetDefinitions",
			"Enabled": true
		},
		{
			"Name": "EnhancedInput",
			"Enabled": true
		},
		{
			"Name": "MediaIOFramework",
			"Enabled": true
		},
		{
			"Name": "NDIMedia",
			"Enabled": true
		},
		{
			"Name": "nDisplayModularFeatures",
			"Enabled": true
		},
		{
			"Name": "OpenCV",
			"Enabled": true
		},
		{
			"Name": "ProceduralMeshComponent",
			"Enabled": true
		},
		{
			"Name": "RemoteControlInterception",
			"Enabled": true
		},
		{
			"Name": "MediaPlayerEditor",
			"Enabled": true
		},
		{
			"Name": "MovieRenderPipeline",
			"Enabled": true
		},
		{
			"Name": "OnlineSubsystem",
			"Enabled": true
		},
		{
			"Name": "OnlineSubsystemUtils",
			"Enabled": true
		},
		{
			"Name": "CompositeCore",
			"Enabled": true
		}
	]
}
